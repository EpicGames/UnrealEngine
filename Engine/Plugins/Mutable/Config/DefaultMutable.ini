[CoreRedirects]

+StructRedirects=(OldName="/Script/CustomizableObject.ParameterUIData",NewName="/Script/CustomizableObject.MutableParameterData")
+StructRedirects=(OldName="/Script/CustomizableObject.CompilationOptions",NewName="/Script/CustomizableObject.CompilationOptions_DEPRECATED")
+StructRedirects=(OldName="/Script/CustomizableObject.FCustomizableObjectAssetParameterValue",NewName="/Script/CustomizableObject.FCustomizableObjectTextureParameterValue")

+PropertyRedirects=(OldName="GroupProjectorParameterImage.OptionImage", NewName="OptionTexture")
+PropertyRedirects=(OldName="CustomizableObjectNodeGroupProjectorParameter.OptionImages", NewName="OptionTextures")
+PropertyRedirects=(OldName="CustomizableObjectNodeGroupProjectorParameter.OptionImagesDataTable", NewName="OptionTexturesDataTable")

; Node renames
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshClipDeform", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierClipDeform")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshClipWithMesh", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierClipWithMesh")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshClipMorph", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierClipMorph")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeExtendMaterial", NewName="/Script/CustomizableObjectEditor.CONodeModifierExtendSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeEditMaterial", NewName="/Script/CustomizableObjectEditor.CONodeModifierEditSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMorphMaterial", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierMorphMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeRemoveMesh", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierRemoveMesh")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeRemoveMeshBlocks", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierRemoveMeshBlocks")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeEditMaterialBase", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierEditMeshSectionBase")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeEditLayoutBlocks", NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierEditLayoutBlocks")

+FunctionRedirects=(OldName="/Script/CustomizableObject.CustomizableObject.GetIntParameterOptionUIMetadata",NewName="/Script/CustomizableObject.CustomizableObject.GetEnumParameterValueUIMetadata")
+FunctionRedirects=(OldName="/Script/CustomizableObject.CustomizableObject.GetIntParameterGroupType",NewName="/Script/CustomizableObject.CustomizableObject.GetEnumParameterGroupType")
+FunctionRedirects=(OldName="/Script/CustomizableObject.CustomizableObject.GetIntParameterOptionDataTable",NewName="/Script/CustomizableObject.CustomizableObject.GetEnumParameterValueDataTable")
+FunctionRedirects=(OldName="/Script/CustomizableObject.CustomizableObject.GetIntParameterDefaultValue",NewName="/Script/CustomizableObject.CustomizableObject.GetEnumParameterDefaultValue")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeComponentPassthroughMesh.Mesh",NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeComponentPassthroughMesh.SkeletalMesh")
+PropertyRedirects=(OldName="/Script/CustomizableObject.CustomizableObject.bEnableAnimBpPhysicsAssetsManipualtion",NewName="/Script/CustomizableObject.CustomizableObject.bEnableAnimBpPhysicsAssetsManipulation")
+PropertyRedirects=(OldName="/Script/CustomizableObject.CompileParams.bGatherReferneces",NewName="/Script/CustomizableObject.CompileParams.bGatherReferences")
+PropertyRedirects=(OldName="/Script/CustomizableObject.CompileParams.bSkipIfOutOfDate",NewName="/Script/CustomizableObject.CompileParams.bSkipIfNotOutOfDate")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeComponentPassthroughMesh.MaterialSlotOverridePins",NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeComponentPassthroughMesh.MaterialSlotPins")

; Pin Data renames
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMaterialBreakPinData",NewName="/Script/CustomizableObjectEditor.CONodeMaterialBreakParameterPinData")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMaterialModifyPinData",NewName="/Script/CustomizableObjectEditor.CONodeMaterialModifyParameterPinData")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterial",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialPinDataParameter",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionPinDataParameter")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialBase",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionBase")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialSwitch",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionSwitch")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierEditMeshSection",NewName="/Script/CustomizableObjectEditor.CONodeModifierEditSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierExtendMeshSection",NewName="/Script/CustomizableObjectEditor.CONodeModifierExtendSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierWithMaterial",NewName="/Script/CustomizableObjectEditor.CONodeModifierSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeModifierEditSkeletalMeshSection",NewName="/Script/CustomizableObjectEditor.CONodeModifierEditSkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeCopyMaterial",NewName="/Script/CustomizableObjectEditor.CONodeCopySkeletalMeshSection")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialVariation",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionVariation")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialPinDataImage",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionPinDataImage")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialPinDataVector",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionPinDataVector")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMaterialPinDataScalar",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshSectionPinDataScalar")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeComponentPassthroughMeshMaterialOverridePinData",NewName="/Script/CustomizableObjectEditor.CONodeComponentSkeletalMeshMaterialPinData")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeComponentPassthroughMeshMaterialPinData",NewName="/Script/CustomizableObjectEditor.CONodeComponentSkeletalMeshMaterialPinData")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeSwitchBase",NewName="/Script/CustomizableObjectEditor.CONodeSwitch")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeComponentMeshAddTo",NewName="/Script/CustomizableObjectEditor.CONodeModifierMutableSkeletalMeshMerge")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodePassThroughTexture",NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeTexture")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeModifierMutableSkeletalMeshMerge.ParentComponentName",NewName="/Script/CustomizableObjectEditor.CONodeModifierMutableSkeletalMeshMerge.ParentSkeletalMeshName")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshBreak",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshObjectBreak")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshBreakPinData",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshObjectBreakPinData")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshMake",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshObjectMake")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMutableSkeletalMeshMake",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshMake_V2")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMutableSkeletalMeshMerge",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshMerge")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMutableSkeletalMeshModify",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshModify")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMutableSkeletalMeshMorph",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshMorph")
+ClassRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeMutableSkeletalMeshTransformWithBone",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshTransformWithBone")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshReshape.bReshapeSkeleton",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshReshape.bReshapePose")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshReshape.bReshapeSkeleton",NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshReshape.bReshapePose")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshReshape.bReshapePhysicsVolumes",NewName="/Script/CustomizableObjectEditor.CONodeSkeletalMeshReshape.bReshapePhysics")
+PropertyRedirects=(OldName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshReshape.bReshapePhysicsVolumes",NewName="/Script/CustomizableObjectEditor.CustomizableObjectNodeMeshReshape.bReshapePhysics")

; Settings for the Mutable plugin
[Features]

; If CustomizableObjectNumBoneInfluences is set to N, CustomizableObjects will be allowed to use a max of N bone influences 
; per vertex. The only valid values are 4, 8, 12, Four, Eight, Twelve. 
; If r.GPUSkin.UnlimitedBoneInfluencesThreshold is set to a lower number, then only that number of influences
; will always be used and allocated thus saving memory per vertex when the real number is higher for a vertex.
; If CustomizableObjectNumBoneInfluences is not defined, only the default 4 will be used. This is used at object compile time.
;CustomizableObjectNumBoneInfluences=Eight

; For backwards compatibility bExtraBoneInfluencesEnabled can be used instead, but will be overridden by 
; CustomizableObjectNumBoneInfluences if both are present. If enabled CustomizableObjects will be allowed to 
; use 8 bone influences per vertex instead of the default 4. This is used at object compile time.
bExtraBoneInfluencesEnabled=False

[EditorDefaults]
ClipMorphMaterialName=/Engine/EditorMaterials/LevelGridMaterial.LevelGridMaterial
ClipMeshMaterialName=/Mutable/Materials/ClipMeshMaterial.ClipMeshMaterial
DynamicallyGenerated_DGGUI_Path=/Game/UI/DynamicallyGeneratedGUI_DGGUI/DynamicallyGeneratedGUI_DGGUI.DynamicallyGeneratedGUI_DGGUI_C