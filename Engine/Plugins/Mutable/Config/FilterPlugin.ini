[FilterPlugin]
; This section lists additional files which will be packaged along with your plugin. Paths should be listed relative to the root plugin directory, and
; may include "...", "*", and "?" wildcards to match directories, files, and individual characters respectively.
;
; Examples:
;    /README.txt
;    /Extras/...
;    /Binaries/ThirdParty/*.dll
