{
	"FileVersion": 1,

	"FriendlyName": "Mutable",
	"Version": 1,
	"VersionName": "1.8.0",
	"CreatedBy": "Epic Games, Inc.",
	"Description": "Mutable adds the tools and runtime to create customizable objects for your games.",
	"Category": "CustomizableObjects",
	"Modules": [
		{
			"Name": "MutableRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "CustomizableObject",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MutableTools",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MutableValidation",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "CustomizableObjectEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "SkeletalMerging",
			"Enabled": true
		},
		{
			"Name": "DataValidation",
			"Enabled": true
		}
	],

	"CanContainContent": true,
	"IsExperimentalVersion": false,
	"IsBetaVersion": true,
	"EnabledByDefault": false
}
