// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CustomizableObject : ModuleRules
{
	public CustomizableObject(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		ShortName = "MuCO";

		DefaultBuildSettings = BuildSettingsVersion.Latest;

		//bUseUnity = false;

		PrivateDependencyModuleNames.AddRange([
			"InputCore",
            "SlateCore",
            "CoreUObject",
            "RenderCore",
            "RHI",
            "AppFramework",
            "Projects",
            "ApplicationCore",
			"ClothingSystemRuntimeInterface",
			//"ClothingSystemEditor",
			"UMG",
			"ImageCore",
			"TraceLog",
			"MeshDescription"
		]);

        PublicDependencyModuleNames.AddRange([
	        "Slate",
			"Core",
            "Engine",
			"SkeletalMerging",
			"ClothingSystemRuntimeCommon",
            "GameplayTags",
			"MutableRuntime",
			"AnimGraphRuntime"
        ]);

		if (TargetRules.bBuildEditor == true)
        {
            PublicDependencyModuleNames.Add("UnrealEd");	// @todo api: Only public because of WITH_EDITOR
            PublicDependencyModuleNames.Add("DerivedDataCache");
            PublicDependencyModuleNames.Add("EditorStyle");
            PublicDependencyModuleNames.Add("MessageLog");

            PrivateDependencyModuleNames.Add("MutableTools");		// COI Baking muSystem settings override
        }
    }
}
