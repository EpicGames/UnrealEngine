// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObjectSystem.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR
#include "DerivedDataCachePolicy.h"
#include "DerivedDataCacheKey.h"
#endif

/** Index of the maximum optimization level when compiling CustomizableObjects */
#define UE_MUTABLE_MAX_OPTIMIZATION			2

class UCustomizableObject;


struct FCompilationOptions
{
	/** Enum to know what texture compression should be used. This compression is used only in manual compiles in editor.
	 *  When packaging, ECustomizableObjectTextureCompression::HighQuality is always used. */
	ECustomizableObjectTextureCompression TextureCompression = ECustomizableObjectTextureCompression::Fast;

	// From 0 to UE_MUTABLE_MAX_OPTIMIZATION
	int32 OptimizationLevel = UE_MUTABLE_MAX_OPTIMIZATION;

	/** High (inclusive) limit of the size in bytes of a data block to be included into the compiled object directly instead of stored in a streamable file.
	 *
	 * This option does not modify the DDC Key.
	 */
	uint64 EmbeddedDataBytesLimit = 1024;

	/** Number of minimum mipmaps that we want to always be available in disk regardless of NumHighResImageLODs. */
	int32 MinDiskMips = 7;

	/** Number of image mipmaps that will be flagged as high-res data (possibly to store separately).
	* This is only used if the total mips in the source image is above the MinDiskMips.
	*/
	int32 NumHighResImageMips = 2;

	// Did we have the extra bones enabled when we compiled?
	ECustomizableObjectNumBoneInfluences CustomizableObjectNumBoneInfluences = ECustomizableObjectNumBoneInfluences::Four;

	// Compiling for cook
	bool bIsCooking = false;

	// This can be set for additional settings
	const ITargetPlatform* TargetPlatform = nullptr;

	// Used to enable the use of real time morph targets.
	bool bRealTimeMorphTargetsEnabled = false;

	// Used to enable 16 bit bone weights
	bool b16BitBoneWeightsEnabled = false;

	// Used to enable skin weight profiles.
	bool bSkinWeightProfilesEnabled = false;

	// Used to enable physics asset merge.
	bool bPhysicsAssetMergeEnabled = false;

	// Determines if the max grid size is determined solely by the Skeletal Mesh noded or if the Mesh Section and Material Modify nodes can alter it.
	bool bUseLegacyLayouts = false;
	
	// Used to enable AnimBp override physics mainipualtion.  
	bool bAnimBpPhysicsManipulationEnabled = false;
	
	/** Force a very big number on the mips to skip during compilation. Useful to debug special cooks of the data. */
	bool bForceLargeLODBias = false;
	int32 DebugBias = 0;

	/** If true, gather all game asset references and save them in the Customizable Object. 
	 *
	 * This option does not modify the DDC Key. 
	 */
	bool bGatherReferences = false;

	/** Whether or not the compiler should query a request to load the compiled data from the DDC.
	 *
	 * This option does not modify the DDC Key. 
	 */
	bool bQueryCompiledDatafromDDC = false;

	/** Whether or not the compiler should store the compiled data to the DDC.
	 *
	 * This option does not modify the DDC Key. 
	 */
	bool bStoreCompiledDataInDDC = false;
	
	/** Stores the only option of an Int Param that should be compiled. */
	TMap<FString, FString> ParamNamesToSelectedOptions;
};


enum class ECompilationStatePrivate : uint8
{
	None,
	InProgress,
	Completed
};


enum class ECompilationResultPrivate : uint8
{
	Unknown, // Not compiled yet (compilation may be in progress).
	Success, // No errors or warnings.
	Errors, // At least have one error. Can have warnings.
	Warnings, // Only warnings.
};

