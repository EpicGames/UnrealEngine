// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API CUSTOMIZABLEOBJECT_API

struct FGuid;

// Custom serialization version for assets/classes in the CustomizableObject and CustomizableObjectEditor modules
struct FCustomizableObjectCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		FixBlueprintPinsUseRealNumbers,

		NodeExposePinOnNameChangedDelegateAdded,

		GroupProjectorPinTypeAdded,

		AutomaticNodeMaterial,
		
		AutomaticNodeMaterialPerformance,

		LayoutClassAdded,

		AutomaticNodeMaterialPerformanceBug,

		PinsNamesImageToTexture,

		BugPinsSwitch,

		PostLoadToCustomVersion, // Wrapped old backwards compatible code that was located at PostLoads inside this custom version.

		AutomaticNodeMaterialUXImprovements,

		ExtendMaterialOnlyMutableModeParameters,

		ExtendMaterialOnlyMutableModeParametersBug,
		
		ExtendMaterialRemoveImages,
		
		EditMaterialOnlyMutableModeParameters, // Force refresh to avoid having Images which are not in Mutable mode.

		GroupProjectorIntToScalarIndex,

		FixBlueprintPinsUseRealNumbersAgain,

		NodeMaterialAddTablePin,

		MorphMaterialAddFactorPin,

		NodeSkeletalMeshCorruptedPinRef,

		CustomizableObjectInstanceDescriptor,
		
		DescriptorBuildParameterDecorations,

		DescriptorMultilayerProjectors,

		DeformSkeletonOptionsAdded,

		GroupProjectorImagePin,

		GroupProjectorImagePinRemoved,

		StateTextureCompressionStrategyEnum,

		AddedTableNodesTextureMode,
		
		ConvertAnimationSlotToFName,

		RemovedParameterDecorations,

		AutomaticNodeSkeletalMesh,

		AutomaticNodeSkeletalMeshPinDataOuter,

		AutomaticNodeSkeletalMeshPinDataUProperty,

		EditMaterialMaskPinDesync, // MTBL-979
		
		IgnoreDisabledSections,

		SkeletalMeshNodeDefaultPinWithoutPinData,
		
		AddedTableMaterialSwitch,

		FixPinsNamesImageToTexture2,

		MeshReshapeVertexColorUsageSelection,

		DeduplicateNodeVariant,

		TextureCompressionEnum,

		NodeTextureParameterDefaultToReferenceValue,

		NodeVariationSerializationIssue, // The parent class and child classes had a member with the same name. Unreal was serialized the member from the child class but deserialized it to the parent class.

		RegenerateNodeObjectsIds,

		NodeMaterialPinDataImageDetails,

		CustomizableObjectStateHasSeparateNeverStreamFlag,

		CustomizableObjectNodeHasSeparateNeverStreamFlag,

		AddedColumnIdDataToTableNodePins,

		AddedAnyTextureTypeToPassThroughTextures,

		ProjectorNodesDefaultValueFix,

		StateUIMetadata,

		AddedRenameOptionToParameterNodes,

		CompilationOptions,

		NodeTablePinViewer,

		NewComponentOptions,

		NodeMaterialTypedImagePins,

		FixedMultilayerMaterialIds,

		UseUVRects, // Instead of pointing at layout blocks in the edited parent, define custom absolute UV layout rects.

		AddModifierPin, // Move the modifiers output pin from Material to Modifier

		CorrectlySerializeTableToParamNames,

		MovedCompatibilityFromPostBackwardsCompatibleFixup,

		FixModifierPin, 

		ConvertEditAndExtendToModifiers,

		NodeComponentMesh, // Moved LOD pins from NodeObject to NodeComponentMesh.

		MergeNodeComponents,
		
		UnifyRequiredTags,

		MaterialPinsRename, // Moved from "Material" to "Mesh Section"

		MoveLayoutToNodeSkeletalMesh, // Move the layouts from the NodeLayout to the NodeSkeletalMesh

		RemoveNodeLayout,

		ComponentsArray,

		FixMaterialPinsRename,

		ModifierClipWithMeshCleanup,

		RealTimeMorphTargetOverrideDataStructureRework,
		
		FixAutomaticBlocksStrategyLegacyNodes,

		SnapToBoneComponentIndexToName,

		SetDisplayNamePropertyAsPinNameOfTableNodes,

		FixedNoPinsInRerouteNodes,

		TableNoneOptionsMovedToUnrealCode,

		UpdatedNodesPinName,

		UpdatedNodesPinName2,

		UpdatedNodesPinName3,

		TextureVariationsToVariations,

		MovedLODSettingsToMeshComponentNode,

		AddedTableNodeCompilationFilters,

		ChangedComponentsInheritance,

		ChangedSwitchNodesInputPinsFriendlyNames,

		TableNodeCompilationFilterArray,

		EnableMutableMacros,
			
		EnableMutableMacrosNewVersion,

		BaseObjectNodeNamePin,

		FixMaterialNodeMaterialPinIncorrectLocalization,

		FixStaticMeshNodeLODMaterialDataInvalidation,

		FixStaticMeshNodeLODMaterialDataInvalidationBackOut,

		FixStaticMeshNodeLODMaterialDataInvalidation2,
		
		SetDisplayNamePropertyAsPinNameOfTableNodes2,			// Added in order to apply the same fix for cases where the version was skipped

		NodeTextureParameterPassthroughPin,

		PinSkeletalMesh,

		NodeStaticStringRemoveHiddenInputPin,
		
		PassthroughParameter,
		
		MeshSectionNodeInputPinsAsReferences,
		
		WrongVariationNodeInputOrderAfterSerializationFix,
		
		PassthroughSkeletalMeshComponentOverrideSubtypeFix,
		
		PassthroughSkeletalMeshComponentComponentMaterialOverride,
		
		StoreMaterialPinReferenceFromConstantMaterialNode,
		
		StoreOverlayMaterialPinAsReference,

		RemovedOverlayMaterialUpropertyFromSkeletalMeshComponentNode,
		
		NodeModifierTransformInMeshStorePinsAsReferences,
		
		RemovedTypedSwitches,
		
		SkeletalMeshMakeNodeDeprecations,				// SKM Component has gotten some responsibilities stripped from it and moved to the new Skeletal Mesh Make node

		TextureFromColorPinCaching,
		
		TextureLayerNodePinCaching,
		
		TextureLayerNodeExplicitLayerTypePins,
		
		CopySKMComponentNameToMakeSKMNode,

		NodeComponentMeshAddToNumLODs,
		
		SkeletalMeshParameterMutableSkeletalMeshPin,

		SkeletalMeshParameterMutableSkeletalMeshPinBackout,
		
		NodeComponentMeshAddToNumLODsBackout,
		
		NodeComponentMeshAddToNumLODsRestore,
		
		SkeletalMeshParameterMutableSkeletalMeshPinRestore,

		ComponentAddSkeletalMeshChangedInputsOrder,		// Make the Component Name pin be above the LOD pins of the Component Mesh Add To node.
		
		ComponentAddNodeModifierOutputPin,				// Changed the output type of the Component Mesh Add To node from Component to Modifier.
		
		MorphStackCategoryName,
		
		MergerTextureNodes,

		AddMissingOverlayPin,
		
		FixOverrideAndOverlayMaterialPinNames,					// Those being different for Overlay/Override was confusing the pin remapper.
		
		MergedSkeletalMeshComponentNodes,

		SkeletalMeshBreakMaterialPinType,

		MorphReshapeMigratedToReshapeNode,
		
		MorphStackDefinition_RemovedMorphAutoPopulate,

		UseLegacyLayouts,
		
		ModifierMorphMeshSection_CacheFactorPin,
		
		MorphStackDefinition_RemoveOrphanMeshPin,

		MeshReshapeBonesToDeformAsFName,				// Mesh Reshape node BonesToDeform changed from TArray<FMeshReshapeBoneReference> to TArray<FName>.

		ModifierSkeletalMeshMergeUseSkeletalMeshPin,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

	private:
		FCustomizableObjectCustomVersion() {}
};

#undef UE_API
