// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CustomizableObjectIdentifier.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

USTRUCT(BlueprintType)
struct FCustomizableObjectIdPair
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category = CustomizableObject, BlueprintReadOnly, EditDefaultsOnly)
	FString CustomizableObjectGroupName;

	UPROPERTY(Category = CustomizableObject, BlueprintReadOnly, EditDefaultsOnly)
	FString CustomizableObjectName;

	FCustomizableObjectIdPair() { };
	UE_API FCustomizableObjectIdPair(FString ObjectGroupName, FString ObjectName);

	bool operator ==(const FCustomizableObjectIdPair& Other) const
	{
		return CustomizableObjectGroupName == Other.CustomizableObjectGroupName && CustomizableObjectName == Other.CustomizableObjectName;
	}

	friend FArchive& operator <<(FArchive& Ar, FCustomizableObjectIdPair& IdPair)
	{
		Ar << IdPair.CustomizableObjectGroupName;
		Ar << IdPair.CustomizableObjectName;

		return Ar;
	}
};

#undef UE_API
