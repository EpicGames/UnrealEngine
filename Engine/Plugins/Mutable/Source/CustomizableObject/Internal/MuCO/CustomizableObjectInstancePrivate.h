// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Materials/MaterialInterface.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuR/Instance.h"
#include "MuR/MutableEditorLogger.h"
#include "GameplayTagContainer.h"
#include "MuCO/DescriptorHash.h"
#include "UObject/Package.h"
#include "Tasks/Task.h"

#include "CustomizableObjectInstancePrivate.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

namespace UE::Mutable::Private 
{
	class FPhysicsBody;
	class FMesh;
}

struct FMutableModelImageProperties;
struct FMutableRefSkeletalMeshData;
struct FMutableImageCacheKey;
struct FStreamableManager;
struct FStreamableHandle;
class UPhysicsAsset;
class USkeleton;
class UCustomizableObjectExtension;
class UModelResources;


// Log texts
extern const FString MULTILAYER_PROJECTOR_PARAMETERS_INVALID;

	
// FParameters encoding
CUSTOMIZABLEOBJECT_API extern const FString NUM_LAYERS_PARAMETER_POSTFIX;
CUSTOMIZABLEOBJECT_API extern const FString OPACITY_PARAMETER_POSTFIX;
CUSTOMIZABLEOBJECT_API extern const FString IMAGE_PARAMETER_POSTFIX;
CUSTOMIZABLEOBJECT_API extern const FString POSE_PARAMETER_POSTFIX;


/** \param OnlyLOD: If not 0, extract and convert only one single LOD from the source image.
  * \param ExtractChannel: If different than -1, extract a single-channel image with the specified source channel data. */
CUSTOMIZABLEOBJECT_API void ConvertImage(UTexture2D* Texture, UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage, const FMutableModelImageProperties& Props, int32 OnlyLOD = -1, int32 ExtractChannel = -1);


/** CustomizableObject Instance flags for internal use  */
enum ECOInstanceFlags
{
	ECONone							= 0,  // Should not use the name None here.. it collides with other enum in global namespace

	// Update process
	ReplacePhysicsAssets			= 1 << 1,	// Merge active PhysicsAssets and replace the base physics asset

	HasPendingHighPriorityUpdate	= 1 << 2,
	HasPendingLowPriorityUpdate		= 1 << 3,
};

ENUM_CLASS_FLAGS(ECOInstanceFlags);


USTRUCT()
struct FCustomizableInstanceComponentData
{
	GENERATED_USTRUCT_BODY();
	
	FName ComponentName;
	
	UE::Mutable::Private::FComponentId ComponentId = INDEX_NONE;

#if WITH_EDITORONLY_DATA
	// Just used for mutable.EnableMutableAnimInfoDebugging command
	TArray<FString> MeshPartPaths;
#endif
	
	UE::Mutable::Private::FSkeletalMeshId LastSkeletalMeshId;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInterface> OverlayMaterial;
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInterface>> OverlayMaterials;
};


USTRUCT()
struct FAnimInstanceOverridePhysicsAsset
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	int32 PropertyIndex = 0;

	UPROPERTY(Transient)
	TObjectPtr<UPhysicsAsset> PhysicsAsset;
};


USTRUCT()
struct FAnimBpGeneratedPhysicsAssets
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TArray<FAnimInstanceOverridePhysicsAsset> AnimInstancePropertyIndexAndPhysicsAssets;
};


struct FMaterialReuseCacheKey
{
	TStrongObjectPtr<UMaterialInterface> MaterialTemplate;
	UE::Mutable::Private::FOperation::ADDRESS Root;

	bool operator==(const FMaterialReuseCacheKey&) const = default;
};


uint32 GetTypeHash(const FMaterialReuseCacheKey& Key);


struct FTextureReuseCacheKey
{
	UE::Mutable::Private::FOperation::ADDRESS Root;

	bool operator==(const FTextureReuseCacheKey&) const = default;
};


uint32 GetTypeHash(const FTextureReuseCacheKey& Key);


#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FObjectInstanceTransactedDelegate, const FTransactionObjectEvent&);
#endif //WITH_EDITOR


/** Indicates the status of the generated Skeletal Mesh. */
enum class ESkeletalMeshStatus : uint8
{
	NotGenerated, // Set only when loading the Instance for the first time or after compiling. Any generation, successful or not, can not end up in this state.
	Success, // Generated successfully.
	Error // Not generated. Set only after a failed update.
};


UCLASS(MinimalAPI)
class UCustomizableInstancePrivate : public UObject
{
public:
	GENERATED_BODY()
	
	/** Generated Skeletal Meshes. */
	UPROPERTY(Transient, VisibleAnywhere, Category = NoCategory)
	TMap<FName, TObjectPtr<USkeletalMesh>> SkeletalMeshes; 	// TODO GMT Should be used for display only

	/** Generated Materials. */
	UPROPERTY(Transient, VisibleAnywhere, Category = NoCategory)
	TArray<TObjectPtr<UMaterialInterface>> Materials; // TODO GMT Should be used for display only

	/** Generated Textures. */
	UPROPERTY(Transient, VisibleAnywhere, Category = NoCategory)
	TArray<TObjectPtr<UTexture>> Textures; // TODO GMT Should be used for display only

	/** Generated Extension Data. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<const UObject>> ExtensionData;
	
	/** If check, Mutable will keep the ownership of the generated resources. This means that Mutable may decide to modify or reuse them for performance considerations. */
	UPROPERTY(EditAnywhere, Category = UpdateOptions)
	bool bKeepOwnershipOfGeneratedResources = true;
	
	// Indices of the parameters that are relevant for the given parameter values.
	// This only gets updated if parameter decorations are generated.
	TArray<uint32> RelevantParameters;

	// Only used in LiveUpdateMode to reuse core instances between updates and their temp data to speed up updates, but spend way more memory
	TSharedPtr<UE::Mutable::Private::FLiveInstance> LiveInstance;

#if WITH_EDITOR
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	UE_API void BindObjectDelegates(UCustomizableObject* CurrentCustomizableObject, UCustomizableObject* NewCustomizableObject);

	UE_API void OnPostCompile();
	UE_API void OnObjectStatusChanged(FCustomizableObjectStatus::EState Previous, FCustomizableObjectStatus::EState Next);
#endif
	
	ECOInstanceFlags GetCOInstanceFlags() const { return InstanceFlagsPrivate; }
	void SetCOInstanceFlags(ECOInstanceFlags FlagsToSet) { InstanceFlagsPrivate = (ECOInstanceFlags)(InstanceFlagsPrivate | FlagsToSet); }
	void ClearCOInstanceFlags(ECOInstanceFlags FlagsToClear) { InstanceFlagsPrivate = (ECOInstanceFlags)(InstanceFlagsPrivate & ~FlagsToClear); }
	bool HasCOInstanceFlags(ECOInstanceFlags FlagsToCheck) const { return (InstanceFlagsPrivate & FlagsToCheck) != 0; }

	UE_API void BuildResources(const TSharedRef<FUpdateContextPrivate>& Context, UCustomizableObjectInstance* Public);

	// Returns true if success (?)
	UE_API bool UpdateSkeletalMesh_PostBeginUpdate0(UCustomizableObjectInstance* Instance, const TSharedRef<FUpdateContextPrivate>& Context);
	
	// The following method is basically copied from PostEditChangeProperty and/or SkeletalMesh.cpp to be able to replicate PostEditChangeProperty without the editor
	UE_API void PostEditChangePropertyWithoutEditor();
	
	/** Only use as an emergency fallback! Set the reference SkeletalMesh unconditionally to all actors using this instance. It will hitch. */
	UE_API void ForceSetReferenceSkeletalMesh() const;

	UE_API const TArray<FAnimInstanceOverridePhysicsAsset>* GetGeneratedPhysicsAssetsForAnimInstance(TSubclassOf<UAnimInstance> AnimInstance) const;

#if WITH_EDITORONLY_DATA
	UE_API void RegenerateImportedModels(const TSharedRef<FUpdateContextPrivate>& OperationData);
#endif
	
	UE_API FCustomizableInstanceComponentData* GetComponentData(const FName& ComponentName);
	UE_API FCustomizableInstanceComponentData& GetComponentData(UE::Mutable::Private::FComponentId ComponentId);
	
	UE_API void InvalidateGeneratedData();
	
	UE_API int32 GetState() const;

	UE_API void SetState(int32 InState);

	UE_API FCustomizableObjectInstanceDescriptor& GetDescriptor() const;

	UE_API void CopyParametersFromInstance(UCustomizableObjectInstance* Instance);
	
	/** Finds in IntParameters a parameter with name ParamName, returns the index if found, -1 otherwise. */
	UE_API int32 FindIntParameterNameIndex(const FString& ParamName) const;

	/** Finds in FloatParameters a parameter with name ParamName, returns the index if found, -1 otherwise. */
	UE_API int32 FindFloatParameterNameIndex(const FString& ParamName) const;

	/** Finds in BoolParameters a parameter with name ParamName, returns the index if found, -1 otherwise. */
	UE_API int32 FindBoolParameterNameIndex(const FString& ParamName) const;

	/** Finds in VectorParameters a parameter with name ParamName, returns the index if found, -1 otherwise. */
	UE_API int32 FindVectorParameterNameIndex(const FString& ParamName) const;

	/** Finds in ProjectorParameters a parameter with name ParamName, returns the index if found, -1 otherwise. */
	UE_API int32 FindProjectorParameterNameIndex(const FString& ParamName) const;
	
	UE_API bool IsParameterRelevant(uint32 ParameterIndex) const;

	UE_API UCustomizableObjectInstance* GetPublic() const;

	void RegisterCustomizableObjectInstanceUsage(UCustomizableObjectInstanceUsage& Obj);
	void UnregisterCustomizableObjectInstanceUsage(UCustomizableObjectInstanceUsage& Obj);
	
	UPROPERTY(Transient)
	TArray<FCustomizableInstanceComponentData> ComponentsData;

	UPROPERTY(Transient, Category = Animation, editfixedsize, VisibleAnywhere)
	TMap<TSubclassOf<UAnimInstance>, FAnimBpGeneratedPhysicsAssets> AnimBpPhysicsAssets;

private:
	ECOInstanceFlags InstanceFlagsPrivate = ECOInstanceFlags::ECONone;

public:
	/** Copy of the descriptor of the latest successful update. */
	UPROPERTY(Transient)
	FCustomizableObjectInstanceDescriptor CommittedDescriptor;

	/** Hash of the descriptor copy of the latest successful update. */
	FDescriptorHash CommittedDescriptorHash;
	
	/** Status of the generated Skeletal Mesh. Not to be confused with the Update Result. */
	ESkeletalMeshStatus SkeletalMeshStatus = ESkeletalMeshStatus::NotGenerated;

	bool bShowOnlyRuntimeParameters = true;
	bool bShowOnlyRelevantParameters = true;
	bool bShowUISections = false;
	bool bShowUIThumbnails = false;

#if WITH_EDITORONLY_DATA
	/** Preview Instance Properties search box filter. Saved here to avoid losing the text during UI refreshes. */
	FText ParametersSearchFilter;
#endif

#if WITH_EDITOR
	/** Delegate called when the Instance has been transacted */
	FObjectInstanceTransactedDelegate OnInstanceTransactedDelegate;
#endif // WITH_EDITOR
	
	TMap<FMaterialReuseCacheKey, TStrongObjectPtr<UMaterialInterface>> LastMaterialsReuseCache;
	TMap<FMaterialReuseCacheKey, TStrongObjectPtr<UMaterialInterface>> CurrentMaterialsReuseCache;

	TMap<FTextureReuseCacheKey, TStrongObjectPtr<UTexture2D>> LastTextureReuseCache;
	TMap<FTextureReuseCacheKey, TStrongObjectPtr<UTexture2D>> CurrentTextureReuseCache;

	/** Pointer to a instance of class that logs update messages to an editor widget. */
	TSharedPtr<IMutableEditorLogger> UpdateLogger;

	TArray<UCustomizableObjectInstanceUsage*> InstanceUsages;
};


UE_API UPhysicsAsset* GetOrBuildMainPhysicsAsset(const TSharedRef<FUpdateContextPrivate>& Context, TObjectPtr<UPhysicsAsset> TamplateAsset, const UE::Mutable::Private::FPhysicsBody* PhysicsBody, bool bDisableCollisionBetweenAssets, int32 ComponentIndex);

UE_API USkeleton* MergeSkeletons(const TArray<TObjectPtr<USkeleton>>& SkeletonsToMerge, UCustomizableObject& CustomizableObject, bool& bOutCreatedNewSkeleton);

UE_API void InitSkeletalMeshData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, const FMutableRefSkeletalMeshData& RefSkeletalMeshData, const UCustomizableObject& CustomizableObject, const TSharedPtr<FInstanceUpdateData::FComponent>& Component);

UE_API void BuildOrCopyClothingData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, const UModelResources& CustomizableObjectInstance, int32 InstanceComponentIndex);

UE_API bool BuildSkeletonData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh& SkeletalMesh, int32 ComponentIndex);

UE_API void BuildMeshSockets(USkeletalMesh* SkeletalMesh, const UModelResources& ModelResources, const FMutableRefSkeletalMeshData& RefSkeletalMeshData, UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& MutableMesh);

UE_API bool BuildOrCopyRenderData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, UCustomizableObjectInstance* CustomizableObjectInstance, int32 ComponentIndex);

UE_API void BuildOrCopyElementData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, int32 ComponentIndex);

UE_API void BuildOrCopyMorphTargetsData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, int32 ComponentIndex);

UE_API UE::Tasks::FTask LoadAdditionalAssetsAndData(const TSharedRef<FUpdateContextPrivate>& Context);

UE_API void AdditionalAssetsAsyncLoaded(TSharedRef<FUpdateContextPrivate> Context, UE::Tasks::FTaskEvent Event);


#undef UE_API
