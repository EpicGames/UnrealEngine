// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectInstanceUsagePrivate.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UCustomizableObjectInstanceUsage;
class USkeletalMesh;

UCLASS(MinimalAPI)
class UCustomizableObjectInstanceUsagePrivate : public UObject
{
	GENERATED_BODY()

public:
	// Own interface
	
	/** Common end point of all updates. Even those which failed. */
	UE_API void Callbacks();

	UE_API USkeletalMesh* GetSkeletalMesh() const;

	UE_API USkeletalMesh* GetAttachedSkeletalMesh() const;
	
	UE_API UCustomizableObjectInstanceUsage* GetPublic();

	UE_API const UCustomizableObjectInstanceUsage* GetPublic() const;

	/** Used to replace the Skeletal Mesh of the parent component by the Reference Skeletal Mesh. */ 
	bool bPendingSetReferenceSkeletalMesh = true;

	bool bIsNetModeDedicatedServer = false;
};

#undef UE_API
