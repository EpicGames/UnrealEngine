// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectSystemPrivate.h"
#include "MuCO/FMutableTaskGraph.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/TextureMipDataProviderFactory.h"
#include "MuR/Image.h"
#include "Streaming/TextureMipDataProvider.h"

#include "MuR/System.h"

#include "CustomizableObjectMipDataProvider.generated.h"

enum EPixelFormat : uint8;
namespace UE::Mutable::Private { class FParameters; }
namespace UE::Mutable::Private { class System; }

class FThreadSafeCounter;
class UCustomizableObjectInstance;
class UObject;
class UTexture;
class FMutableStreamRequest;
struct FModelStreamableBulkData;

/** This struct stores the data relevant for the construction of a specific texture. 
* This includes all the data required to rebuild the image (or any of its mips).
*/
class FMutableUpdateImageContext
{
public:

	FMutableUpdateImageContext(TSharedRef<UE::Mutable::Private::FLiveInstance> InLiveInstance)
		: LiveInstance(InLiveInstance)
	{
		
	}
	
	// Benchmarking Utility data (it may not always be present)
	FString CapturedDescriptor;
	bool bLevelBegunPlay = false;
	
	FString CustomizableObjectPathName;
	FString InstancePathName;

	FName CustomizableObjectName;

	TSharedRef<UE::Mutable::Private::FLiveInstance> LiveInstance;
	
	TSharedPtr<UE::Mutable::Private::FSystem> System;

	TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData;
	
};


struct FMutableMipUpdateLevel
{
	FMutableMipUpdateLevel(int32 InMipLevel, void* InDest, int32 InSizeX, int32 InSizeY, int32 InDataSize, EPixelFormat InFormat) :
		Dest(InDest), MipLevel(InMipLevel), SizeX(InSizeX), SizeY(InSizeY), DataSize(InDataSize), Format(InFormat) {}

	void* Dest; // Only access from the FMutableTextureMipDataProvider, owned by the FTextureMipInfoArray so don't delete
	int32 MipLevel;
	int32 SizeX;
	int32 SizeY;
	int32 DataSize;
	EPixelFormat Format;
};

namespace UE::Mutable::Private::MemoryCounters
{
	struct FPrefetchMemoryCounter
	{
		static std::atomic<SSIZE_T>& Get()
		{
			static std::atomic<SSIZE_T> Counter{0};
			return Counter;
		}
	};
}

/** Runtime data used during a mutable image mipmap update */
struct FMutableImageOperationData
{
	/** This option comes from the operation request. It is used to reduce the number of mipmaps that mutable must generate for images.  */
	int32 MipsToSkip = 0;
	FMutableImageReference RequestedImage;

	TSharedPtr<FMutableUpdateImageContext> UpdateContext;

	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> Result;

	TArray<FMutableMipUpdateLevel> Levels;

	uint32 MutableTaskId = FMutableTaskGraph::INVALID_ID;

	using FPrefetchArray = TArray<uint8, UE::Mutable::Private::FDefaultMemoryTrackingAllocator<UE::Mutable::Private::MemoryCounters::FPrefetchMemoryCounter>>;
	FPrefetchArray AllocatedMemory;

	// Used to sync with the FMutableTextureMipDataProvider and FRenderAssetUpdate::Tick
	FThreadSafeCounter* Counter;
	FTextureUpdateSyncOptions::FCallback RescheduleCallback;

	bool bIsCancelled = false;

	/** Access to the Counter must be protected with this because it may be accessed from another thread to null it. */
	FCriticalSection CounterTaskLock;

	// Image Update Memory stats
	int64 ImageUpdateStartBytes = 0;
};


class FMutableTextureMipDataProvider : public FTextureMipDataProvider
{
public:
	FMutableTextureMipDataProvider(const UTexture* Texture, UCustomizableObjectInstance* InCustomizableObjectInstance, const FMutableImageReference& InImageRef);

	virtual void Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) override;
	virtual int32 GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions) override;
	virtual bool PollMips(const FTextureUpdateSyncOptions& SyncOptions) override;
	virtual void CleanUp(const FTextureUpdateSyncOptions& SyncOptions) override;
	virtual void Cancel(const FTextureUpdateSyncOptions& SyncOptions) override;
	virtual ETickThread GetCancelThread() const override;
	void AbortPollMips() override;

private:
	void CancelAsyncTasks();
	void PrintWarningAndAdvanceToCleanup();

public:
	// Todo: Simplify by replacing the reference to the Instance with some static parametrization or hash with enough info to reconstruct the texture
	UCustomizableObjectInstance* CustomizableObjectInstance = nullptr;

	FMutableImageReference ImageRef;
	TSharedPtr<FMutableUpdateImageContext> UpdateContext;

	bool bRequestAborted = false;

	TSharedPtr<FMutableImageOperationData> OperationData;

	TUniquePtr<FMutableStreamRequest> PrefetchRequest;
};


UCLASS(MinimalAPI, hidecategories=Object)
class UMutableTextureMipDataProviderFactory : public UTextureMipDataProviderFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual FTextureMipDataProvider* AllocateMipDataProvider(UTexture* Asset) override
	{
		check(ImageRef.ImageID);
		FMutableTextureMipDataProvider* Result = new FMutableTextureMipDataProvider(Asset, CustomizableObjectInstance, ImageRef);
		Result->UpdateContext = UpdateContext;		
		return Result;
	}

	virtual bool WillProvideMipDataWithoutDisk() const override
	{ 
		return true;
	}

	virtual bool ShouldAllowPlatformTiling(const UTexture* Owner) const override;

	// Todo: Simplify by replacing the reference to the Instance with some static parametrization or hash with enough info to reconstruct the texture
	UPROPERTY(Transient)
	TObjectPtr<UCustomizableObjectInstance> CustomizableObjectInstance = nullptr;

	FMutableImageReference ImageRef;
	TSharedPtr<FMutableUpdateImageContext> UpdateContext;

};
