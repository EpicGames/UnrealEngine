// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/StateMachine.h"
#include "MuCO/CustomizableObjectUIData.h"
#include "MuCO/CustomizableObjectIdentifier.h"
#include "MuR/ResourceID.h"
#include "Serialization/BulkData.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

#if WITH_EDITOR
#include "Misc/Guid.h"
#include "Engine/DataTable.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataCachePolicy.h"
#endif

#include "CustomizableObjectPrivate.generated.h"

class UCustomizableObjectSkeletalMesh;

namespace UE::Mutable::Private 
{ 
	class FModel;
}

#if WITH_EDITOR
namespace UE::Mutable::Private
{
	struct FClassifyNode;
}

namespace UE::DerivedData
{
	struct FValueId;
	struct FCacheKey;
	enum class ECachePolicy : uint32;
}
#endif

class USkeletalMesh;
class USkeleton;
class UPhysicsAsset;
class UMaterialInterface;
class UTexture;
class UAnimInstance;
class UAssetUserData;
class USkeletalMeshLODSettings;
class UModelResources;
class ITargetPlatform;
struct FModelStreamableBulkData;
struct FObjectAndNameAsStringProxyArchive;
struct FCustomizableObjectInstanceDescriptor;


FGuid CUSTOMIZABLEOBJECT_API GenerateIdentifier(const UCustomizableObject& CustomizableObject);

FString CUSTOMIZABLEOBJECT_API GetModelResourcesNameForPlatform(const UCustomizableObject& CustomizableObject, const ITargetPlatform& Platform);

#if WITH_EDITOR

FGuid CUSTOMIZABLEOBJECT_API GenerateDataDistributionIdentifier(const UCustomizableObject& CustomizableObject);

#endif

// Warning! MutableCompiledDataHeader must be the first data serialized in a stream
struct MutableCompiledDataStreamHeader
{
	int32 InternalVersion=0;
	FGuid VersionId;

	MutableCompiledDataStreamHeader() { }
	MutableCompiledDataStreamHeader(int32 InInternalVersion, FGuid InVersionId) : InternalVersion(InInternalVersion), VersionId(InVersionId) { }

	friend FArchive& operator<<(FArchive& Ar, MutableCompiledDataStreamHeader& Header)
	{
		Ar << Header.InternalVersion;
		Ar << Header.VersionId;

		return Ar;
	}
};


USTRUCT()
struct FMutableRemappedBone
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FName Name;
	
	UPROPERTY()
	uint32 Hash = 0;
	
	bool operator==(const FName& InName)
	{
		return Name == InName;
	}
};


USTRUCT()
struct FMutableModelParameterValue
{
	GENERATED_USTRUCT_BODY()

	FMutableModelParameterValue() = default;

	UPROPERTY()
	FString Name;

	UPROPERTY()
	int Value = 0;
};


USTRUCT()
struct FMutableModelParameterProperties
{
	GENERATED_USTRUCT_BODY()

	FMutableModelParameterProperties() = default;
	FString Name;

	UPROPERTY()
	EMutableParameterType Type = EMutableParameterType::None;

	UPROPERTY()
	TArray<FMutableModelParameterValue> PossibleValues;
};


class FSkeletalMeshCache
{
public:
	TStrongObjectPtr<UCustomizableObjectSkeletalMesh> Get(const UE::Mutable::Private::FSkeletalMeshId& Key);

	void Add(const UE::Mutable::Private::FSkeletalMeshId& Key, UCustomizableObjectSkeletalMesh* Value);

	void Clear();

private:
	TMap<UE::Mutable::Private::FSkeletalMeshId, TWeakObjectPtr<UCustomizableObjectSkeletalMesh>> GeneratedSkeletalMeshes;

	
	UE::FMutex Lock;
};



class FTextureCache
{
public:
	struct FId
	{
		UE::Mutable::Private::FImageId Resource;

		int32 SkippedMips = 0;

		bool bIsBake = false;
		
		bool operator==(const FId&) const = default;

		friend uint32 GetTypeHash(const FId& Key);
	};
	
	TStrongObjectPtr<UTexture2D> Get(const FId& Key);

	void Add(const FId& Key, UTexture2D* Value);

	void Remove(const UTexture2D& Value);

	void Clear();

private:
	TMap<FId, TWeakObjectPtr<UTexture2D>> GeneratedTextures;
	
	UE::FMutex Lock;
};


class FSkeletonCache
{
public:
	USkeleton* Get(const TArray<FName>& Key);

	void Add(const TArray<FName>& Key, USkeleton* Value);

	void Clear();

private:
	TMap<TArray<FName>, TWeakObjectPtr<USkeleton>> MergedSkeletons;
};


struct FCustomizableObjectStatusTypes
{
	enum class EState : uint8
	{
		Loading = 0, // Waiting for PostLoad and Asset Registry to finish.
		ModelLoaded, // Model loaded correctly.
		NoModel, // No model (due to no model not found and automatic compilations disabled).
		// Compiling, // Compiling the CO. Equivalent to UCustomizableObject::IsLocked = true.

		Count,
	};
	
	static constexpr EState StartState = EState::NoModel;

	static constexpr bool ValidTransitions[3][3] =
	{
		// TO
		// Loading, ModelLoaded, NoModel // FROM
		{false,   true,        true},  // Loading
		{false,   true,        true},  // ModelLoaded
		{true,    true,        true},  // NoModel
	};
};

using FCustomizableObjectStatus = FStateMachine<FCustomizableObjectStatusTypes>;


USTRUCT()
struct FMutableModelImageProperties
{
	GENERATED_USTRUCT_BODY()

	FMutableModelImageProperties()
		: Filter(TF_Default)
		, SRGB(0)
		, FlipGreenChannel(0)
		, IsPassThrough(0)
		, LODBias(0)
		, MipGenSettings(TextureMipGenSettings::TMGS_FromTextureGroup)
		, LODGroup(TEXTUREGROUP_World)
		, AddressX(TA_Clamp)
		, AddressY(TA_Clamp)
	{}

	FMutableModelImageProperties(const FString& InTextureParameterName, TextureFilter InFilter, uint32 InSRGB,
		uint32 InFlipGreenChannel, uint32 bInIsPassThrough, int32 InLODBias, TEnumAsByte<TextureMipGenSettings> InMipGenSettings, 
		TEnumAsByte<enum TextureGroup> InLODGroup, TEnumAsByte<enum TextureAddress> InAddressX, TEnumAsByte<enum TextureAddress> InAddressY)
		: TextureParameterName(InTextureParameterName)
		, Filter(InFilter)
		, SRGB(InSRGB)
		, FlipGreenChannel(InFlipGreenChannel)
		, IsPassThrough(bInIsPassThrough)
		, LODBias(InLODBias)
		, MipGenSettings(InMipGenSettings)
		, LODGroup(InLODGroup)
		, AddressX(InAddressX)
		, AddressY(InAddressY)
	{}

	// Name in the material.
	UPROPERTY()
	FString TextureParameterName;

	UPROPERTY()
	TEnumAsByte<enum TextureFilter> Filter;

	UPROPERTY()
	uint32 SRGB : 1;

	UPROPERTY()
	uint32 FlipGreenChannel : 1;

	UPROPERTY()
	uint32 IsPassThrough : 1;

	UPROPERTY()
	int32 LODBias;

	UPROPERTY()
	TEnumAsByte<TextureMipGenSettings> MipGenSettings;

	UPROPERTY()
	TEnumAsByte<enum TextureGroup> LODGroup;

	UPROPERTY()
	TEnumAsByte<enum TextureAddress> AddressX;

	UPROPERTY()
	TEnumAsByte<enum TextureAddress> AddressY;

	CUSTOMIZABLEOBJECT_API bool operator!=(const FMutableModelImageProperties& rhs) const;
};


USTRUCT()
struct FMutableRefSocket
{
	GENERATED_BODY()

	UPROPERTY()
	FName SocketName;
	UPROPERTY()
	FName BoneName;

	UPROPERTY()
	FVector RelativeLocation = FVector::ZeroVector;
	UPROPERTY()
	FRotator RelativeRotation = FRotator::ZeroRotator;
	UPROPERTY()
	FVector RelativeScale = FVector::ZeroVector;

	UPROPERTY()
	bool bForceAlwaysAnimated = false;

	CUSTOMIZABLEOBJECT_API bool operator ==(const FMutableRefSocket& Other) const;

#if WITH_EDITORONLY_DATA
	CUSTOMIZABLEOBJECT_API friend FArchive& operator<<(FArchive& Ar, FMutableRefSocket& Data);
#endif
};


USTRUCT()
struct FMutableRefSkeletalMeshSettings
{
	GENERATED_BODY()

	UPROPERTY()
	bool bEnablePerPolyCollision = false;

	UPROPERTY()
	float DefaultUVChannelDensity = 0.f;

#if WITH_EDITORONLY_DATA
	friend FArchive& operator<<(FArchive& Ar, FMutableRefSkeletalMeshSettings& Data);
#endif
};


USTRUCT()
struct FMutableRefSkeletalMeshData
{
	GENERATED_BODY()

	// Reference Skeletal Mesh
	UPROPERTY()
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	// Path to load the ReferenceSkeletalMesh
	UPROPERTY()
	TSoftObjectPtr<USkeletalMesh> SoftSkeletalMesh;

	// Optional USkeletalMeshLODSettings
	UPROPERTY()
	TObjectPtr<USkeletalMeshLODSettings> SkeletalMeshLODSettings;
	
	// Sockets
	UPROPERTY()
	TArray<FMutableRefSocket> Sockets;

	// Bounding Box
	UPROPERTY()
	FBoxSphereBounds Bounds = FBoxSphereBounds(ForceInitToZero);

	// Settings
	UPROPERTY()
	FMutableRefSkeletalMeshSettings Settings;

	// Skeleton
	UPROPERTY()
	TObjectPtr<USkeleton> Skeleton;

	// PhysicsAsset
	UPROPERTY()
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	// Post Processing AnimBP
	UPROPERTY()
	TSoftClassPtr<UAnimInstance> PostProcessAnimInst;

	// Shadow PhysicsAsset
	UPROPERTY()
	TObjectPtr<UPhysicsAsset> ShadowPhysicsAsset;

#if WITH_EDITORONLY_DATA
	friend FArchive& operator<<(FArchive& Ar, FMutableRefSkeletalMeshData& Data);
#endif
};


USTRUCT()
struct FAnimBpOverridePhysicsAssetsInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftClassPtr<UAnimInstance> AnimInstanceClass;

	UPROPERTY()
	int32 PropertyIndex = -1;

	CUSTOMIZABLEOBJECT_API bool operator==(const FAnimBpOverridePhysicsAssetsInfo& Rhs) const;

#if WITH_EDITORONLY_DATA
	friend FArchive& operator<<(FArchive& Ar, FAnimBpOverridePhysicsAssetsInfo& Info);
#endif
};


// TODO: Optimize this struct
USTRUCT()
struct FMutableStreamableBlock
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	uint32 FileId = 0;

	/** Used to store properties of the data, necessary for its recovery. For instance if it is high-res. */
	UPROPERTY()
	uint16 Flags = 0;
	
	uint16 IsPrefetched = 0;

	UPROPERTY()
	uint64 Offset = 0;

	friend FArchive& operator<<(FArchive& Ar, FMutableStreamableBlock& Data)
	{
		Ar << Data.FileId;
		Ar << Data.Flags;
		Ar << Data.Offset;
		return Ar;
	}
};
static_assert(sizeof(FMutableStreamableBlock) == 8 * 2);


struct FMutableParameterIndex
{

	FMutableParameterIndex(int32 InIndex, int32 InTypedIndex)
	{
		Index = InIndex;
		TypedIndex = InTypedIndex;
	}

	int32 Index = INDEX_NONE;
	int32 TypedIndex = INDEX_NONE;
};


USTRUCT()
struct FIntegerParameterOptionKey
{
	GENERATED_BODY()

	UPROPERTY()
	FString ParameterName;

	UPROPERTY()
	FString ParameterOption;

#if WITH_EDITOR
	friend CUSTOMIZABLEOBJECT_API uint32 GetTypeHash(const FIntegerParameterOptionKey& Key);
	bool operator==(const FIntegerParameterOptionKey& Other) const = default;
#endif
};


USTRUCT()
struct FIntegerParameterOptionDataTable
{
	GENERATED_BODY()

	UPROPERTY()
	TSet<TSoftObjectPtr<UDataTable>> DataTables;
};


USTRUCT()
struct FIntegerParameterUIData
{
	GENERATED_BODY()

	FIntegerParameterUIData() = default;
	
	CUSTOMIZABLEOBJECT_API FIntegerParameterUIData(const FMutableParamUIMetadata& InParamUIMetadata);
	
	UPROPERTY()
	FMutableParamUIMetadata ParamUIMetadata;

	friend FArchive& operator<<(FArchive& Ar, FIntegerParameterUIData& Struct);
};


USTRUCT()
struct FMutableParameterData
{
	GENERATED_BODY()	

	FMutableParameterData() = default;
	
	CUSTOMIZABLEOBJECT_API FMutableParameterData(const FMutableParamUIMetadata& InParamUIMetadata, EMutableParameterType InType);

	UPROPERTY()
	FMutableParamUIMetadata ParamUIMetadata;

	/** Parameter type */
	UPROPERTY()
	EMutableParameterType Type = EMutableParameterType::None;

	/** In the case of an integer parameter, store here all options */
	UPROPERTY()
	TMap<FString, FIntegerParameterUIData> ArrayIntegerParameterOption;

	/** How are the different options selected (one, one or none, etc...) */
	UPROPERTY()
	ECustomizableObjectGroupType IntegerParameterGroupType = ECustomizableObjectGroupType::COGT_ONE_OR_NONE;
	
	friend FArchive& operator<<(FArchive& Ar, FMutableParameterData& Struct);
};


USTRUCT()
struct FMutableStateData
{
	GENERATED_BODY()

	UPROPERTY()
	FMutableStateUIMetadata StateUIMetadata;

	/** In this mode instances and their temp data will be reused between updates. It will be much faster but spend as much as ten times the memory.
	 * Useful for customization lockers with few characters that are going to have their parameters changed many times, not for in-game */
	UPROPERTY()
	bool bLiveUpdateMode = false;

	/** If this is enabled, Mesh streaming won't be used for this state, and all LODs will be generated when an instance is first updated. */
	UPROPERTY()
	bool bDisableMeshStreaming = false;

	/** If this is enabled, texture streaming won't be used for this state, and full images will be generated when an instance is first updated. */
	UPROPERTY()
	bool bDisableTextureStreaming = false;
	
	UPROPERTY()
	TMap<FString, FString> ForcedParameterValues;

	friend FArchive& operator<<(FArchive& Ar, FMutableStateData& Struct);
};


/** This is encoded in exact bits so if extended, review its uses everywhere. */
enum class EMutableFileFlags : uint8
{
	None	= 0,
	HighRes = 1 << 0,
	DoesExist = 1 << 1,
	DoesExistCached = 1 << 2
};


namespace UE::Mutable::Private
{
	enum class EStreamableDataType : uint32 // uint32 for padding and DDC purposes
	{
		None = 0,
		Model,
		
		DataTypeCount
	};

#if WITH_EDITOR
	struct FBlock
	{
		/** Used on some data types as the index to the block stored in the CustomizableObject */
		uint32 Id;

		/** Used on some data types to group blocks. */
		uint32 SourceId;

		/** Size of the data block. */
		uint32 Size;

		uint32 Padding = 0;

		/** Offset in the full source streamed data file that is created when compiling. */
		uint64 Offset;

		friend FArchive& operator<<(FArchive& Ar, FBlock& Data)
		{
			Ar << Data.Id;
			Ar << Data.SourceId;
			Ar << Data.Size;
			Ar << Data.Offset;
			return Ar;
		};
	};
	//template<> struct TCanBulkSerialize<FBlock> { enum { Value = true }; };

	struct FFile
	{
		EStreamableDataType  DataType = EStreamableDataType::None;

		/** Rom ResourceType. */
		uint16 ResourceType = 0;

		/** Common flags of the data stored in this file. See EMutableFileFlags.*/
		uint16 Flags = 0;

		/** Id generated from a hash of the file content + offset to avoid collisions. */
		uint32 Id = 0;

		/** File size */
		uint64 Size = 0;

		/** List of blocks that are contained in the file, in order. */
		TArray<FBlock> Blocks;

		/** Get the total size of blocks in this file. */
		CUSTOMIZABLEOBJECT_API uint64 GetSize() const;

		/** Copy the requested block to the requested buffer and return its size. */
		CUSTOMIZABLEOBJECT_API void GetFileData(struct FMutableCachedPlatformData*, TArray64<uint8>& DataDestination, bool bDropData);

		friend FArchive& operator<<(FArchive& Ar, FFile& Data)
		{
			Ar << Data.DataType;
			Ar << Data.ResourceType;
			Ar << Data.Flags;
			Ar << Data.Id;
			Ar << Data.Blocks;
			return Ar;
		};
	};

	struct FFileCategoryID
	{
		FFileCategoryID(EStreamableDataType DataType, uint16 ResourceType, uint16 Flags);

		FFileCategoryID() = default;

		// EDataType
		EStreamableDataType DataType = EStreamableDataType::None;

		/** Rom ResourceType. */
		uint16 ResourceType = 0;

		/** Rom flags  */
		uint16 Flags = 0;

		friend uint32 GetTypeHash(const FFileCategoryID& Key);
		bool operator==(const FFileCategoryID& Other) const = default;
		friend FArchive& operator<<(FArchive& Ar, FFileCategoryID& Data)
		{
			Ar << Data.DataType;
			Ar << Data.ResourceType;
			Ar << Data.Flags;
			return Ar;
		};
	};


	struct FFileCategory
	{
		FFileCategoryID Id;

		// Accumulated size of resources from this category
		uint64 DataSize = 0;

		// Categories within a bucket with a limited number of files will use sequential ID starting at FirstFile
		// and up to FirstFile + NumFiles.
		uint32 FirstFile = 0;
		uint32 NumFiles = 0;
	};


	struct FFileCategoryOverride
	{
		FFileCategoryID Id;
		uint32 NumFiles = 0;
	};


	/** Group bulk data by categories. */
	struct FFileBucket
	{
		// Resources belonging to these categories will be added to the bucket.
		TArray<FFileCategory> Categories;

		// Accumulated size of the resources of all categories within this bucket
		uint64 DataSize = 0;
	};

	struct FModelStreamableData
	{
		void Get(uint32 Key, TArrayView64<uint8> Destination, bool bDropData)
		{
			TArray64<uint8>* Buffer = Data.Find(Key);
			check(Buffer);
			check(Destination.Num() == Buffer->Num());
			FMemory::Memcpy(Destination.GetData(), Buffer->GetData(), Buffer->Num());

			if (bDropData)
			{
				Buffer->Empty();
			}
		}

		void Set(uint32 Key, const uint8* Source, int64 Size)
		{
			check(Source);
			check(Size);
			TArray64<uint8>& Buffer = Data.Add(Key);
			check(Buffer.Num() == 0);
			Buffer.SetNumUninitialized(Size);
			FMemory::Memcpy(Buffer.GetData(), Source, Size);
		}

		// Temp, to be replaced with disk storage
		TMap<uint32, TArray64<uint8> > Data;
	};


	struct FMutableCachedPlatformData
	{
		/** UE::Mutable::Private::Model */
		TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model;
		
		/** UModelResources */
		TStrongObjectPtr<UModelResources> ModelResources;

		/** Streamable resources info such as files and offsets. */
		TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData;

		/** Struct containing map of RomId to RomBytes. */
		FModelStreamableData ModelStreamableData;

		/** List of files to serialize. Each file has a list of binary blocks to be serialized. */
		TArray<FFile> BulkDataFiles;
	};


	/** Generate the list of BulkData files with a restriction to the number of files to generate per bucket.
	 *  Resources will be split into two buckets for non-optional and optional BulkData.
	 */
	void CUSTOMIZABLEOBJECT_API GenerateBulkDataFilesListWithFileLimit(
		FGuid ObjectId,
		TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model,
		FModelStreamableBulkData& ModelStreamableBulkData,
		uint32 NumFilesPerBucket,
		bool bAllowFileCategoryOverride,
		bool bAllowSplit,
		const ITargetPlatform& TargetPlatform,
		TArray<FFile>& OutBulkDataFiles);

	/** Generate the list of BulkData files with a soft restriction to the size of the files.
	 */
	void CUSTOMIZABLEOBJECT_API GenerateBulkDataFilesListWithSizeLimit(
		TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model,
		FModelStreamableBulkData& ModelStreamableBulkData,
		const ITargetPlatform* TargetPlatform,
		uint64 TargetBulkDataFileBytes,
		TArray<FFile>& OutBulkDataFiles);

	/** Compute the number of files and sizes the BulkData will be split into and update
	 * the streamable's FileIds and Offsets.
	 */
	void GenerateBulkDataFilesList(
		TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model,
		FModelStreamableBulkData& StreamableBulkData,
		bool bUseRomTypeAndFlagsToFilter,
		TFunctionRef<void(const FFileCategoryID&, const FClassifyNode&, TArray<FFile>&)> CreateFileList,
		TArray<FFile>& OutBulkDataFiles);

	void CUSTOMIZABLEOBJECT_API SerializeBulkDataFiles(
		FMutableCachedPlatformData& CachedPlatformData,
		TArray<FFile>& BulkDataFiles,
		TFunctionRef<void(FFile&, TArray64<uint8>&, uint32 FileIndex)> WriteFile,
		bool bDropData);

	UE::DerivedData::FValueId CUSTOMIZABLEOBJECT_API GetDerivedDataModelId();
	UE::DerivedData::FValueId CUSTOMIZABLEOBJECT_API GetDerivedDataModelResourcesId();
	UE::DerivedData::FValueId CUSTOMIZABLEOBJECT_API GetDerivedDataModelStreamableBulkDataId();
	UE::DerivedData::FValueId CUSTOMIZABLEOBJECT_API GetDerivedDataBulkDataFilesId();
#endif
}

struct FModelStreamableBulkData
{
	/** Map of Hash to Streaming blocks, used to stream a block of data representing a resource from the BulkData */
	TMap<uint32, FMutableStreamableBlock> ModelStreamables;

	TArray<EMutableFileFlags> BulkDataFlags;
	TArray<FByteBulkData> StreamableBulkData;

#if WITH_EDITORONLY_DATA
	// Used to know if roms and other resources must be streamed from the DDC.
	bool bIsStoredInDDC = false;
	UE::DerivedData::FCacheKey DDCKey = UE::DerivedData::FCacheKey::Empty;
	UE::DerivedData::ECachePolicy DDCDefaultPolicy = UE::DerivedData::ECachePolicy::Default;
	TArray<FIoHash> DDCValues;
#endif

	// File path to stream resources from when not using FByteBulkData or DDC.
	FString FullFilePath;

	CUSTOMIZABLEOBJECT_API void Serialize(FArchive& Ar, UObject* Owner, bool bCooked);

#if WITH_EDITORONLY_DATA
	friend FArchive& operator<<(FArchive& Ar, FModelStreamableBulkData& Data)
	{
		Ar << Data.ModelStreamables;
		Ar << Data.DDCValues;
		// Don't serialize FByteBulkData manually, the data will be skipped.
		
		Ar << Data.FullFilePath;

		return Ar;
	}
#endif
};

/** Interface class to allow custom serialization of FModelStreamableBulkData and its FBulkData. */
UCLASS()
class UModelStreamableData : public UObject
{
	GENERATED_BODY()

	UModelStreamableData();

public:
	virtual void Serialize(FArchive& Ar) override;

	virtual void PostLoad() override;

	TSharedPtr<FModelStreamableBulkData> StreamingData;
};

USTRUCT()
struct FMutableParamNameSet
{
	GENERATED_BODY()

	TSet<FString> ParamNames;
};


/** Class containing all UE resources derived from a CO compilation. These resources will be embedded in the CO at cook time but not in the editor.
  * Editor compilations will serialize this class to disk using the Serialize methods. Ensure new fields are serialized, too.
  * Variables and settings that should not change until the CO is re-compiled should be stored here. */
UCLASS(MinimalAPI)
class UModelResources : public UObject
{
	GENERATED_BODY()

public:
	/** Key is FComponentId.
	 * All the SkeletalMeshes generated for this CustomizableObject instances will use the Reference Skeletal Mesh
	 * properties for everything that Mutable doesn't create or modify. This struct stores the information used from
	 * the Reference Skeletal Meshes to avoid having them loaded at all times. This includes data like LOD distances,
	 * LOD render data settings, Mesh sockets, Bounding volumes, etc. */
	UPROPERTY()
	TArray<FMutableRefSkeletalMeshData> ReferenceSkeletalMeshesData;	
	
	UPROPERTY()
	TMap<uint32, TSoftObjectPtr<UObject>> PassthroughObjects;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<uint32> DuplicateObjects;
	
	/** Runtime referenced textures used by the UE::Mutable::Private::FModel. */
	UPROPERTY()
	TArray<TSoftObjectPtr<UTexture2D>> RuntimeReferencedTextures;
#endif
	
	/** UAnimBlueprint assets gathered from the SkeletalMesh, to be used in mesh generation in-game */
	UPROPERTY()
	TArray<TSoftClassPtr<UAnimInstance>> AnimBPs;

	/** */
	UPROPERTY()
	TArray<FAnimBpOverridePhysicsAssetsInfo> AnimBpOverridePhysiscAssetsInfo;

	UPROPERTY()
	TArray<FMutableModelImageProperties> ImageProperties;
	
	/** Parameter UI metadata information for all the dependencies of this Customizable Object. */
	UPROPERTY()
	TMap<FString, FMutableParameterData> ParameterUIDataMap;

	/** State UI metadata information for all the dependencies of this Customizable Object */
	UPROPERTY()
	TMap<FString, FMutableStateData> StateUIDataMap;

#if WITH_EDITORONLY_DATA
	/** DataTable used by an int parameter and its value. */
	UPROPERTY()
	TMap<FIntegerParameterOptionKey, FIntegerParameterOptionDataTable> IntParameterOptionDataTable;
#endif
	
	/** Currently not used, this option should be selectable from editor maybe as a compilation flag */
	UPROPERTY()
	bool bAllowClothingPhysicsEditsPropagation = true;

#if WITH_EDITORONLY_DATA

	// Stores what param names use a certain table as a table can be used from multiple table nodes, useful for partial compilations to restrict params
	UPROPERTY()
	TMap<FString, FMutableParamNameSet> TableToParamNames;

	/** Map to identify what CustomizableObject owns a parameter. Used to display a tooltip when hovering a parameter
	 * in the Prev. instance panel */
	UPROPERTY()
	TMap<FString, FString> CustomizableObjectPathMap;

	UPROPERTY()
	TMap<FString, FCustomizableObjectIdPair> GroupNodeMap;

	/** If the object is compiled with maximum optimizations. */
	UPROPERTY()
	bool bIsCompiledWithOptimization = true;

	/** This is a non-user-controlled flag to disable streaming (set at object compilation time, depending on optimization). */
	UPROPERTY()
	bool bIsTextureStreamingDisabled = false;

	/** List of external packages that if changed, a compilation is required.
	  * Key is the package name. Value is the the UPackage::Guid, which is regenerated each time the packages is saved.
	  *
	  * Updated each time the CO is compiled and saved in the Derived Data. */
	UPROPERTY()
	TMap<FName, FGuid> ParticipatingObjects;
#endif
	
	// Key is UE::Mutable::Private::OP::ADDRESS. UPROPERTY does not support typedefs. 
	UPROPERTY()
	TMap<uint32, FInstancedStruct> ExternalOperations;
	
	// Value is the compiled version of the operation.
	UPROPERTY()
	TMap<TObjectPtr<const UScriptStruct>, uint32> ExternalOperationVersions;
	
	/** Key is FComponentId. Name of all possible components. */
	UPROPERTY()
	TArray<FName> ComponentNamesPerObjectComponent;

	UPROPERTY()
	TMap<FString, TObjectPtr<UTexture>> TextureParameterDefaultValues;

	UPROPERTY()
	TMap<FString, TObjectPtr<USkeletalMesh>> SkeletalMeshParameterDefaultValues;

	UPROPERTY()
	TMap<FString, TObjectPtr<UMaterialInterface>> MaterialParameterDefaultValues;

	UPROPERTY()
	TMap<FString, FInstancedStruct> ExternalTypeParameterDefaultValues;
	
	UPROPERTY()
	TMap<FString, TObjectPtr<const UScriptStruct>> ExternalTypeParameterTypes;
	
	UPROPERTY()
	FString ReleaseVersion;

	UPROPERTY()
	int32 CodeVersion = 0;

#if WITH_EDITORONLY_DATA
	/** Value of the variable TextureCompression in the last compilation of this CO.
		* this is needed since we can compile a CO through blueprints with a different 
		* compilation setting than the stored in the COE.*/
	UPROPERTY()
	bool bCompiledWithHDTextureCompression = false;
#endif
};


USTRUCT()
struct FMutableMeshComponentData
{
	GENERATED_USTRUCT_BODY()

	/** Name to identify this component. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FName Name;

	/** All the SkeletalMeshes generated for this CustomizableObject instances will use the Reference Skeletal Mesh
	* properties for everything that Mutable doesn't create or modify. This includes data like LOD distances, Physics
	* properties, Bounding Volumes, Skeleton, etc.
	*
	* While a CustomizableObject instance is being created for the first time, and in some situation with lots of
	* objects this may require some seconds, the Reference Skeletal Mesh is used for the actor. This works as a better
	* solution than the alternative of not showing anything, although this can be disabled with the function
	* "SetReplaceDiscardedWithReferenceMeshEnabled" (See the c++ section). */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TObjectPtr<USkeletalMesh> ReferenceSkeletalMesh;
};


#if WITH_EDITORONLY_DATA 
// This is a manual version number for the binary blobs in this asset.
// Increasing it invalidates all the previously compiled models.
UENUM()
enum class ECustomizableObjectVersions : int32
{
	FirstEnumeratedVersion = 450,

	DeterminisiticMeshVertexIds,

	NumRuntimeReferencedTextures,
	
	DeterminisiticLayoutBlockIds,

	BackoutDeterminisiticLayoutBlockIds,

	FixWrappingProjectorLayoutBlockId,

	MeshReferenceSupport,

	ImproveMemoryUsageForStreamableBlocks,

	FixClipMeshWithMeshCrash,

	SkeletalMeshLODSettingsSupport,

	RemoveCustomCurve,

	AddEditorGamePlayTags,

	AddedParameterThumbnailsToEditor,

	ComponentsLODsRedesign,

	ComponentsLODsRedesign2,

	LayoutToPOD,

	AddedRomFlags,

	LayoutNodeCleanup,

	AddSurfaceAndMeshMetadata,

	TablesPropertyNameBug,

	DataTablesParamTrackingForCompileOnlySelected,

	CompilationOptimizationsMeshFormat,

	ModelStreamableBulkData,

	LayoutBlocksAsInt32,
	
	IntParameterOptionDataTable,

	RemoveLODCountLimit,

	IntParameterOptionDataTablePartialBackout,

	IntParameterOptionDataTablePartialRestore,

	CorrectlySerializeTableToParamNames,
	
	AddMaterialSlotNameIndexToSurfaceMetadata,

	NodeComponentMesh,
	
	MoveEditNodesToModifiers,

	DerivedDataCache,

	ComponentsArray,

	FixComponentNames,

	AddedFaceCullStrategyToSomeOperations,

	DDCParticipatingObjects,

	GroupRomsBySource,
	
	RemovedGroupRomsBySource,

	ReGroupRomsBySource,

	UIMetadataGameplayTags,

	TransformInMeshModifier,
	
	SurfaceMetadataSlotNameIndexToName,

	BulkDataFilesNumFilesLimit,

	RemoveModifiersHack,

	SurfaceMetadataSerialized,

	FixesForMeshSectionMultipleOutputs,

	ImageParametersInServerBuilds,

	RemovedUnnecessarySerializationVersioning,

	AddTextureCompressionSettingCompilationInfo,

	RestructureConstantImageData,

	RestructureConstantMeshData,

	RestructureRomData,

	RestructureRomDataRemovingRomHash,

	ModifiedRomCompiledDataSerialization,

	ModelResourcesExtensionData,

	LODsPerComponent,

	LODsPerComponentTypeMismatch,

	ImageHiResLODsUseLODGroupInfo,

	MovedTableRowNoneGenerationToUnreal,

	RemoveObsoletMeshInterpolateAndGeometryOp,

	RemoveObsoleteDataTypesFromEnum,

	ConvertModelResourcesToUObject,

	RemoveObsoletImageGradientOp,

	MeshReferencesExtendedForCompilation,

	RemoveObsoleteBoolOps,

	AddOverlayMaterials,

	PrefetchHighQualityMipMaps,

	AddedMeshParameterOp,

	AddedMeshParameterOpForDDCPollution,

	ExtendedMeshParameterArgumentsWithLODAndSection,
	
	AddAssetUserDataEditor,
	
	MeshDataRomSplit,

	MeshDataRomSplitBackout,

	MovedLODSettingsToMeshComponentNode,
	
	AddedMeshPrepareLayoutOp,

	AddedMeshIDToMeshParamOp,

	ClothMorphMeshMetadata,

	AddedMeshIDToMeshParamOpBackout,

	MeshDataRomSplitSerializationFix,

	ReaddAddedMeshIDToMeshParamOp,

	AddConnectedChildObjectComponentsToPrepass,

	FixMeshReusalDueToLayouts,

	IncorrectBonePoseMerging,

	CoreParameterUObjects,

	MoveMeshMetadataToOperation,
	
	MovePhysicsBodiesToRoms,

	SwitchOpCompactRepresentation,

	AddedBlockMasksToRuntimeLayouts,

	StoreMipTailInASingleRom,

	FixDDCCrashesAndLoadTimes,

	CompactBindingDataStructure,

	MorphMeshParameters,

	MaterialParameter,

	AddLinearTosRGB,

	AddMaterialArgs,

	CacheReferenceTextureSize,

	MaterialBreak,

	AddNodeMeshTransformWithBoneHierarchy,

	MaterialBreakBackOutFix,

	MaterialBreakCheckFix,

	LazyAddressOutsideFImage,

	EnableMeshLODStreaming,

	DeterministicSocketsIds,

	CompilationEarlyOutOnRepeatedParameters,

	FixSkinWeightProfilesConversion,

	MaterialIDMovedToArgs,
	
	FixMissingLODSettings,

	FixSocketPriority,

	FixDefaultBehaviourAndsWithoutOperants,

	BumpVersionDDCCrash,

	ImageParameterConvert,

	InstancedStructType,

	SkeletalMeshParameter,

	RelevantParameters,

	DeduplicateFMaterials,

	AddBonePosePriority,

	RemovedMaterialLayerEncoding,
	
	AddedSkeletalMeshBreakNode,

	ReplaceSurfaceIdWithSharedSurfaceId,

	Deduplicate_MI_CONSTANT,
	
	AddOverrideMaterials,

	TexutreParametersFormatFromReferenceTexture,

	AddMaterialModify,

	AddImageMultiCompose,
	
	DeduplicateFMaterialsSharingPassthroughObject,

	RemovedReuseTextures,
	
	ComponentId,

	PassthroughTexturesForBreakNodes,
	
	VersionBumpBackout,
	
	ComponentId2,

	BackoutGenerateBulkFileWithDataLimit,

	ClothCore,

	ConvertBoneIdsToBoneNames,
	
	AddLayerIndexToMatParamTextures,
	
	ClothCoreSerializationIssue,
	
	ClothCoreSerializationIssue2,

	MovePhysicsAssetCore,

	ChangedLayerIndexDataType,

	MutableSkeletonToPassthroughObject,

	UseCompressedMorphsForAllMorphData,

	TestMoveSocketsToCore,

	RemoveMorphTarget,
	
	AssetUserDataPassthroughObject2,

	NewProgramAddressFormat,
	
	IS_SWITCH,
	
	SkeletalMeshConditionalSwitch,

	GenerateSkeletalMesh,

	SourceImageNoMipmapsSettingPreserve,
	
	MaterialSlotNameToCore,

	GenerateMutableSourceSkeletalMesh,

	AddSkeletalMeshMorphAndSkeletalMaterialModifyOps,
	
	FixSkeletalMeshParameterNotWorking,

	AddSkeletalMeshMergeV1,
	
	SK_CLIPMESHWITHMESH,

	SKO_CONVERT,
	
	SKO_CONVERTBackout,

	SKO_CONVERTRestore,
	
	SK_TRANSFORM,

	SK_TRANSFORMWITHBONE,
	
	SurfaceModifiers,

	MultipleMeshesPerLOD,

	SK_CONVERT,
	
	MI_SKELETALMESHOBJECT_BREAK,

	ReshapeMorphInvertedSelection,
	
	RemovedMorphReshapeOp,

	RemoveSurfaceMetadata,

	SK_RESHAPE,
	
	SkinWeightProfilesCore,
	
	MaterialModifyIssues,

	FixMinQualityLODLevels,

	AddedClothAssetLODIndexMetadata,

	AddedMacroContextToSKMNodeKey,

	RemoveMeshFormatInProjectors,

	AddedSKMMakeNodesInReshapeOps,

	IM_CONFORM,

	// -----<new versions can be added above this line>--------
	LastCustomizableObjectVersion
};
#endif


UCLASS(MinimalAPI)
class UCustomizableObjectPrivate : public UObject
{
	GENERATED_BODY()

	TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> MutableModel;

	/** Stores streamable data info to be used by MutableModel In-Game. Cooked resources. */
	UPROPERTY()
	TObjectPtr<UModelStreamableData> ModelStreamableData;

	/** Stores resources to be used by MutableModel In-Game. Cooked resources. */
	UPROPERTY()
	TObjectPtr<UModelResources> ModelResources;

#if WITH_EDITORONLY_DATA
	/** 
	 * Stores resources to be used by MutableModel in the Editor. Editor resources.
	 * Editor-Only to avoid packaging assets referenced by editor compilations. 
	 */
	UPROPERTY(Transient)
	TObjectPtr<UModelResources> ModelResourcesEditor;

	/**
	 * Stores streamable data info to be used by MutableModel in the Editor. Editor streaming.
	 */
	TSharedPtr<FModelStreamableBulkData> ModelStreamableDataEditor;
#endif

public:
	// UObject interface
	virtual void PostLoad() override;

	/** Must be called after unlocking the CustomizableObject. */
	CUSTOMIZABLEOBJECT_API void SetModel(const TSharedPtr<UE::Mutable::Private::FModel>& Model, const FGuid Identifier, bool bIsCooking);
	CUSTOMIZABLEOBJECT_API const TSharedPtr<UE::Mutable::Private::FModel>& GetModel();
	CUSTOMIZABLEOBJECT_API const TSharedPtr<const UE::Mutable::Private::FModel> GetModel() const;

	CUSTOMIZABLEOBJECT_API UModelResources* GetModelResources();
	CUSTOMIZABLEOBJECT_API const UModelResources* GetModelResources() const;
	CUSTOMIZABLEOBJECT_API UModelResources& GetModelResourcesChecked();
	CUSTOMIZABLEOBJECT_API const UModelResources& GetModelResourcesChecked() const;


#if WITH_EDITORONLY_DATA
	CUSTOMIZABLEOBJECT_API UModelResources* GetModelResources(bool bIsCooking);
	CUSTOMIZABLEOBJECT_API const UModelResources* GetModelResources(bool bIsCooking) const;

	CUSTOMIZABLEOBJECT_API void SetModelResources(UModelResources* InModelResources, bool bIsCooking);
	CUSTOMIZABLEOBJECT_API void SetModelStreamableBulkData(const TSharedPtr<FModelStreamableBulkData>& StreamableData, bool bIsCooking);
#endif
	
	CUSTOMIZABLEOBJECT_API TSharedPtr<FModelStreamableBulkData> GetModelStreamableBulkData(bool bIsCooking = false) const;

	// See UCustomizableObjectSystem::LockObject()
	CUSTOMIZABLEOBJECT_API bool IsLocked() const;

	/** Modify the provided mutable parameters so that the forced values for the given customizable object state are applied. */
	CUSTOMIZABLEOBJECT_API void ApplyStateForcedValuesToParameters(FCustomizableObjectInstanceDescriptor& Descriptor);

	CUSTOMIZABLEOBJECT_API int32 FindParameter(const FString& Name) const;
	CUSTOMIZABLEOBJECT_API int32 FindParameterTyped(const FString& Name, EMutableParameterType Type) const;

	CUSTOMIZABLEOBJECT_API EMutableParameterType GetParameterType(int32 ParamIndex) const;

	CUSTOMIZABLEOBJECT_API int32 FindIntParameterValue(int32 ParamIndex, const FString& Value) const;

	CUSTOMIZABLEOBJECT_API FString GetStateName(int32 StateIndex) const;

#if WITH_EDITORONLY_DATA
	CUSTOMIZABLEOBJECT_API void PostCompile();
#endif

	CUSTOMIZABLEOBJECT_API UCustomizableObject* GetPublic() const;

#if WITH_EDITOR

	/** Compose file name. */
	CUSTOMIZABLEOBJECT_API FString GetCompiledDataFileName(const ITargetPlatform* InTargetPlatform = nullptr, bool bIsDiskStreamer = false) const;

	/** DDC helpers. BuildDerivedDataKey is expensive, try to cache it as much as possible. */
	CUSTOMIZABLEOBJECT_API TArray<uint8> BuildDerivedDataKey(FCompilationOptions Options);
	CUSTOMIZABLEOBJECT_API UE::DerivedData::FCacheKey GetDerivedDataCacheKeyForOptions(FCompilationOptions InOptions);

	/** Log data for debugging purposes */
	CUSTOMIZABLEOBJECT_API void LogMemory();
#endif
	
	/** Rebuild ParameterProperties from the current compiled model. */
	CUSTOMIZABLEOBJECT_API void UpdateParameterPropertiesFromModel(const TSharedPtr<UE::Mutable::Private::FModel>& Model);

	CUSTOMIZABLEOBJECT_API void AddUncompiledCOWarning(const FString& AdditionalLoggingInfo);

#if WITH_EDITOR
	// Create new GUID for this CO
	CUSTOMIZABLEOBJECT_API void UpdateVersionId();
	
	CUSTOMIZABLEOBJECT_API FGuid GetVersionId() const;

	CUSTOMIZABLEOBJECT_API void SaveEmbeddedData(FArchive& Ar);

	// Regenerate the DataDistributionId. Used to keep the same cooked data distribution between builds.
	CUSTOMIZABLEOBJECT_API void UpdateDataDistributionId();

	/** Generic Load methods to read compiled data */
	CUSTOMIZABLEOBJECT_API bool LoadModelResources(FArchive& Ar, const ITargetPlatform* InTargetPlatform, bool bSkipEditorOnlyData = false);
	CUSTOMIZABLEOBJECT_API void LoadModelStreamableBulk(FArchive& Ar, bool bIsCooking);
	CUSTOMIZABLEOBJECT_API void LoadModel(FArchive& Ar, bool bIsCooking);

	/** Load compiled data for the running platform from disk, this is used to load Editor Compilations. */
	CUSTOMIZABLEOBJECT_API void LoadCompiledDataFromDisk();
	
	/** Loads data previously compiled in BeginCacheForCookedPlatformData onto the UProperties in *this,
	  * in preparation for saving the cooked package for *this or for a CustomizableObjectInstance using *this.
      * Returns whether the data was successfully loaded. */
	CUSTOMIZABLEOBJECT_API bool TryLoadCompiledCookDataForPlatform(const ITargetPlatform* TargetPlatform);
#endif

	// Data that may be stored in the asset itself, only in packaged builds.
	CUSTOMIZABLEOBJECT_API void LoadEmbeddedData(FArchive& Ar);
	
	/** Compute bIsChildObject if currently possible to do so. Return whether it was computed. */
	CUSTOMIZABLEOBJECT_API bool TryUpdateIsChildObject();

	CUSTOMIZABLEOBJECT_API void SetIsChildObject(bool bIsChildObject);
	
#if WITH_EDITOR
	/** See ICustomizableObjectEditorModule::IsCompilationOutOfDate. */
	CUSTOMIZABLEOBJECT_API bool IsCompilationOutOfDate(bool bSkipIndirectReferences, TArray<FName>& OutOfDatePackages, TArray<FName>& AddedPackages, TArray<FName>& RemovedPackages, bool& bReleaseVersionDiff) const;
#endif

	CUSTOMIZABLEOBJECT_API TArray<FString>& GetCustomizableObjectClassTags();
	
	CUSTOMIZABLEOBJECT_API TArray<FString>& GetPopulationClassTags();

    CUSTOMIZABLEOBJECT_API TMap<FString, FParameterTags>& GetCustomizableObjectParametersTags();

#if WITH_EDITORONLY_DATA
	CUSTOMIZABLEOBJECT_API TObjectPtr<UEdGraph>& GetSource() const;
#endif

	CUSTOMIZABLEOBJECT_API void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion);

	CUSTOMIZABLEOBJECT_API FName GetComponentName(UE::Mutable::Private::FComponentId ComponentId) const;

#if WITH_EDITORONLY_DATA
	CUSTOMIZABLEOBJECT_API EMutableCompileMeshType GetMeshCompileType() const;
	
	CUSTOMIZABLEOBJECT_API const TArray<TSoftObjectPtr<UCustomizableObject>>& GetWorkingSet() const;
	
	CUSTOMIZABLEOBJECT_API bool IsTableMaterialsParentCheckDisabled() const;

	CUSTOMIZABLEOBJECT_API bool IsRealTimeMorphTargetsEnabled() const;
	
	CUSTOMIZABLEOBJECT_API bool Is16BitBoneWeightsEnabled() const;
	
	CUSTOMIZABLEOBJECT_API bool IsAltSkinWeightProfilesEnabled() const;
	
	CUSTOMIZABLEOBJECT_API bool IsPhysicsAssetMergeEnabled() const;
	
	CUSTOMIZABLEOBJECT_API bool ShouldUseLegacyLayouts() const;
	
	CUSTOMIZABLEOBJECT_API bool IsEnabledAnimBpPhysicsAssetsManipulation() const;
#endif
	
	CUSTOMIZABLEOBJECT_API const FString& GetIntParameterAvailableOption(int32 ParamIndex, int32 K) const;

	CUSTOMIZABLEOBJECT_API int32 GetEnumParameterNumValues(int32 ParamIndex) const;

	CUSTOMIZABLEOBJECT_API FString FindIntParameterValueName(int32 ParamIndex, int32 ParamValue) const;

	CUSTOMIZABLEOBJECT_API int32 FindState(const FString& Name) const;

	CUSTOMIZABLEOBJECT_API int32 GetStateParameterIndex(int32 StateIndex, int32 ParameterIndex) const;

	CUSTOMIZABLEOBJECT_API bool IsParameterMultidimensional(const int32& InParamIndex) const;

	/** Cache of generated Skeletal Meshes. Only takes into account geometry. Materials may be different. Passthrough Skeletal Meshes are not contemplated. */
	FSkeletalMeshCache SkeletalMeshCache;

	/** Cache of generated Textures. Passthrough Textures are not contemplated. */
	FTextureCache TextureCache;
	
	/** Cache of merged Skeletons */
	FSkeletonCache SkeletonCache;
	
	TSharedRef<UE::Mutable::Private::FImageIdRegistry> ImageIdRegistry = MakeShared<UE::Mutable::Private::FImageIdRegistry>();

	TSharedRef<UE::Mutable::Private::FMaterialIdRegistry> MaterialIdRegistry = MakeShared<UE::Mutable::Private::FMaterialIdRegistry>();

	TSharedRef<UE::Mutable::Private::FSkeletalMeshIdRegistry> SkeletalMeshIdRegistry = MakeShared<UE::Mutable::Private::FSkeletalMeshIdRegistry>();

	TSharedPtr<UE::Mutable::Private::FModel> SerializedModel;
	
	// See UCustomizableObjectSystem::LockObject. Must only be modified from the game thread
	bool bLocked = false;

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	TArray<FMutableMeshComponentData> MutableMeshComponents_DEPRECATED;

	/** Unique Identifier - Deterministic. Used to locate Model and Streamable data on disk. Should not be modified. */
	FGuid Identifier;
	
	ECompilationResultPrivate CompilationResult = ECompilationResultPrivate::Unknown;
	
	FPostCompileDelegate PostCompileDelegate;

	/** Map of PlatformName to CachedPlatformData. Only valid while cooking. */
	TMap<FString, UE::Mutable::Private::FMutableCachedPlatformData> CachedPlatformsData;

#endif

	FCustomizableObjectStatus Status;

	// This is information about the parameters in the model that is generated at model compile time.
	UPROPERTY(Transient)
	TArray<FMutableModelParameterProperties> ParameterProperties;

	/** Reference to all UObject used in game. Only updated during the compilation if the user explicitly wants to save all references. */
	UPROPERTY()
	TObjectPtr<UModelResources> ReferencedObjects;
	
	// Map of name to index of ParameterProperties.
	// use this to lookup fast by Name
	TMap<FString, FMutableParameterIndex> ParameterPropertiesLookupTable;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	ECustomizableObjectTextureCompression TextureCompression = ECustomizableObjectTextureCompression::Fast;

	/** From 0 to UE_MUTABLE_MAX_OPTIMIZATION */
	UPROPERTY()
	int32 OptimizationLevel = UE_MUTABLE_MAX_OPTIMIZATION;

	
	/** High (inclusive) limit of the size in bytes of a data block to be included into the compiled object directly instead of stored in a streamable file. */
	UPROPERTY()
	uint64 EmbeddedDataBytesLimit = 1024;

	UPROPERTY()
	bool bDisableTableMissingDataWarning = false;

	/** Used to keep the same cooked data distribution between builds. 
	  * This Id is part of the DDC key of the cooked data distribution. */
	UPROPERTY()
	FGuid CookedDataDistributionId;
#endif

	/** Editor Log list to print all the messages related to this CO. */
	TSharedPtr<class IMutableEditorLogger> CompilationLogger;
	
	static constexpr int32 DerivedDataVersion = 0x2850e5a1;
};

#if WITH_EDITOR

/** Returns the DDC ValueId of the file owning a streamable resource.
	* @ param StreamableDataType - UE level data type
	* @ param ResourceType - UE::Mutable::Private::EDataType
	*/
CUSTOMIZABLEOBJECT_API UE::DerivedData::FValueId GetDerivedDataValueIdForResource(UE::Mutable::Private::EStreamableDataType StreamableDataType, uint32 FileId, uint16 ResourceType, uint16 Flags);

// Compose folder name where the data is stored
CUSTOMIZABLEOBJECT_API FString GetCompiledDataFolderPath();
CUSTOMIZABLEOBJECT_API FString GetDataTypeExtension(UE::Mutable::Private::EStreamableDataType DataType);

CUSTOMIZABLEOBJECT_API uint32 GetECustomizableObjectVersionEnumHash();

CUSTOMIZABLEOBJECT_API TObjectPtr<UModelResources> LoadModelResources_Internal(FArchive& Ar, const UCustomizableObject* Outer, const ITargetPlatform* InTargetPlatform, bool bSkipEditorOnlyData = false);
CUSTOMIZABLEOBJECT_API const TSharedPtr<FModelStreamableBulkData> LoadModelStreamableBulk_Internal(FArchive& Ar);
CUSTOMIZABLEOBJECT_API const TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> LoadModel_Internal(FArchive& Ar);

#endif
