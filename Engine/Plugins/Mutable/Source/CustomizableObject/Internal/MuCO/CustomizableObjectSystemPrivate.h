// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LogBenchmarkUtil.h"
#include "MuCO/DescriptorHash.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "Containers/Ticker.h"
#include "Containers/Deque.h"

#include "MuCO/CustomizableObjectInstanceDescriptor.h"
#include "MuR/Mesh.h"
#include "MuR/Parameters.h"
#include "MuR/System.h"
#include "MuR/Skeleton.h"
#include "MuR/Image.h"
#include "MuR/Material.h"
#include "MuR/ResourceID.h"
#include "Framework/Notifications/NotificationManager.h"
#include "MuCO/FMutableTaskGraph.h"
#include "ContentStreaming.h"
#include "Animation/MorphTarget.h"
#include "MuCO/MutableStreamableManager.h"
#include "UObject/StrongObjectPtr.h"
#include "Tasks/Task.h"

#include "CustomizableObjectSystemPrivate.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class AActor;
class FMutableUpdateImageContext;
class FUnrealMutableResourceProvider;
class FMutableStreamRequest;
class UEditorImageProvider;
class UCustomizableObjectSystem;
struct FParameters;
namespace LowLevelTasks { enum class ETaskPriority : int8; }
struct FTexturePlatformData;

// Split StreamedBulkData into chunks smaller than MUTABLE_STREAMED_DATA_MAXCHUNKSIZE
#define MUTABLE_STREAMED_DATA_MAXCHUNKSIZE		(512 * 1024 * 1024)


extern TAutoConsoleVariable<bool> CVarMutableHighPriorityLoading;


/** Key to identify an image inside a generated mutable runtime instance. */
struct FMutableImageReference
{
	UE::Mutable::Private::FImageId ImageID;

	uint8 BaseMip = 0;

	TArray<int32> ConstantImagesNeededToGenerate;
};

/** 
 * Reconstruct the final Morph Targets using the mesh compressed morphs.
 * @param Mesh Mesh containing the compressed data.
 * @param OutMorphTargets Resulting Morph Targets LOD models filtered by usage.
 * @param UsageFilter Usage filter applied to the OutMorphTargets.
 */
void ReconstructMorphTargetsFromMeshCompressedData(const UE::Mutable::Private::FMesh& Mesh, TArray<FMorphTargetLODModel>& OutMorphTargets, UE::Mutable::Private::EMorphUsageFlags UsageFilter);

#if !WITH_EDITOR
void ReconstructMorphTargetsFromMeshCompressedData(const UE::Mutable::Private::FMesh& Mesh, TArray<FMorphTargetCompressedLODModel>& OutMorphTargets, UE::Mutable::Private::EMorphUsageFlags UsageFilter);
#endif
/** End a Customizable Object Instance Update. All code paths of an update have to end here. */
void FinishUpdateGlobal(const TSharedRef<FUpdateContextPrivate>& Context);


/** Final data per component. */
struct FSkeletalMeshMorphTargets
{
	/** Name of the Morph Target. */
	TArray<FName> RealTimeMorphTargetNames;

	/** First index is the Morph Target (in sync with RealTimeMorphTargetNames). Second index is the LOD. */
	TArray<TArray<FMorphTargetLODModel>> RealTimeMorphsLODData;
};


// Mutable data generated during the update steps.
// We keep it from begin to end update, and it is used in several steps.
struct FInstanceUpdateData
{
	struct FImage
	{
		FName ParameterName; // TODO MATERIAL_PIN This does not belong to the Image but the Material
		UE::Mutable::Private::FImageId ImageID;
		
		// LOD of the ImageId.
		int32 BaseMip;
		
		uint16 FullImageSizeX;
		uint16 FullImageSizeY;
		
		UE::Mutable::Private::EImageFormat Format = UE::Mutable::Private::EImageFormat::None;

		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> Image;

		TArray<int32> ConstantImagesNeededToGenerate;

		int32 ImagePropertiesIndex = INDEX_NONE; // TODO MATERIAL_PIN This does not belong to the Image but the Material

		bool bIsPassThrough = false;
		bool bIsNonProgressive = false;

		int32 MaterialLayer = INDEX_NONE; // TODO MATERIAL_PIN This does not belong to the Image but the Material
	};
	
	struct FMaterial
	{
		/** Range in the Images array */
		uint16 FirstImage = 0;
		uint16 ImageCount = 0;
				
		UE::Mutable::Private::FMaterialId MaterialId;
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial> Material;
	};
	
	struct FSurface
	{
		/** Id of the surface in the mutable core instance. */
		uint32 SurfaceId = 0;
		
		FName MaterialSlotName;
		
		uint32 MaterialIndex = INDEX_NONE;
	};

	struct FLOD
	{		
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh;

		/** Range in the Surfaces array */
		uint16 FirstSurface = 0;
		uint16 SurfaceCount = 0;

		/** Range in the external Bones array */
		uint32 FirstActiveBone = 0;
		uint32 ActiveBoneCount = 0;

		/** Range in the external Bones array */
		uint32 FirstBoneMap = 0;
		uint32 BoneMapCount = 0;
	};

	struct FOverrideMaterial
	{
		FName SlotName;
		
		uint32 MaterialIndex = INDEX_NONE;
	};
	
	struct FOverlayMaterial
	{
		FName SlotName;
		
		uint32 MaterialIndex = INDEX_NONE;
	};

	struct FBone
	{
		FName BoneName;
		FMatrix44f MatrixWithScale;

		bool operator==(const FName& OtherBoneName) const { return BoneName == OtherBoneName; };
	};

	struct FComponent
	{
		UE::Mutable::Private::FComponentId Id = INDEX_NONE;
		
		/** Range in the LODs array */
		uint16 FirstLOD = 0;
		uint16 LODCount = 0;

		uint16 FirstOverrideMaterial = 0;
		uint16 OverrideMaterialCount = 0;
		
		uint16 FirstOverlayMaterial = 0;
		uint16 OverlayMaterialCount = 0;

		uint32 OverlayMaterialIndex = INDEX_NONE;
		
		UE::Mutable::Private::FSkeletalMeshId SkeletalMeshId;
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeletalMesh> SkeletalMesh;

		TArray<FBone> BonePose;
	};

	TArray<TSharedRef<FComponent>> Components;
	TArray<TSharedRef<FLOD>> LODs;
	TArray<FOverrideMaterial> OverrideMaterials;
	TArray<FOverlayMaterial> OverlayMaterials;
	TArray<TSharedRef<FMaterial>> Materials;
	TArray<TSharedRef<FSurface>> Surfaces;
	TArray<TSharedRef<FImage>> Images;

	TArray<FName> ActiveBones;
	TArray<FName> BoneMaps;
	

	/** Key is the Component Name. Value is the final Morph Target data to copy into the Skeletal Mesh. */
	TMap<UE::Mutable::Private::FComponentId, FSkeletalMeshMorphTargets> RealTimeMorphTargets;

	/** */
	void Clear()
	{
		LODs.Empty();
		Components.Empty();
		Surfaces.Empty();
		OverrideMaterials.Empty();
		OverlayMaterials.Empty();
		Materials.Empty();
		Images.Empty();
		RealTimeMorphTargets.Empty();
	}
};


/** Update Context.
 *
 * Alive from the start to the end of the update (both API and LOD update). */
class FUpdateContextPrivate
{
public:
	void Init(bool bUseCommitedDescriptor);
	
	const FCustomizableObjectInstanceDescriptor& GetCapturedDescriptor() const;

	const FDescriptorHash& GetCapturedDescriptorHash() const;

	const FCustomizableObjectInstanceDescriptor&& MoveCommittedDescriptor();
	
	UE::Mutable::Private::FImageOperator::FImagePixelFormatFunc PixelFormatOverride;

	FInstanceUpdateData InstanceUpdateData;

private:
	/** Descriptor which the update will be performed on. */
	FCustomizableObjectInstanceDescriptor CapturedDescriptor;

	/** Hash of the descriptor. */
	FDescriptorHash CapturedDescriptorHash;

public:
	TArray<uint32> RelevantParameters;
	
	/** Hard references to objects. Avoids GC to collect them. */
	TArray<TStrongObjectPtr<const UObject>> Objects;

	// Used during an update to prevent the pass-through textures loaded by LoadAdditionalAssetsAsync() from being unloaded by GC
	// between AdditionalAssetsAsyncLoaded() and their setting into the generated materials in BuildResources()
	TArray<TStrongObjectPtr<const UTexture>> LoadedPassThroughTexturesPendingSetMaterial;
	

	TArray<UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FExtensionData>> ExtensionData;
	
	/** Used to know if the updated instances' meshes are different from the previous ones. 
	  * The index of the array is the instance component's index.
	  * @return true if the mesh is new or new to this instance (e.g. mesh cached by another instance). */
	TArray<bool> MeshChangedPerInstanceComponent;

	/** List of component names. Key is the FComponentId. */
	TArray<FName> ComponentNames;

	/** Materials already build in this update. */
	TMap<UE::Mutable::Private::FMaterialId, TStrongObjectPtr<UMaterialInterface>> MaterialUpdateCache;

	/** Textures already build in this update. */
	TMap<FTextureCache::FId, TStrongObjectPtr<UTexture>> TextureUpdateCache;
	
	/** Calculated at runtime. */
	TMap<FName, uint8> FirstLODAvailable;
	
	/** Calculated at runtime. */
	TMap<FName, uint8> NumLODsAvailable;

	/** Calculated at runtime. */
	TMap<FName, uint8> FirstResidentLOD;

	TMap<UE::Mutable::Private::FImageId, TUniquePtr<FTexturePlatformData>> ImageToPlatformDataMap;
	
	/** If a InstanceUsage is in this array it means that its AttachParent has been modified (USkeletalMesh changed, UMaterial changed...). */
	TArray<UCustomizableObjectInstanceUsage*> AttachedParentUpdated;

	FInstanceUpdateDelegate UpdateCallback;
	FInstanceUpdateNativeDelegate UpdateNativeCallback;

	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FInstance> MutableInstance;

	/** Instance parameters at the time of the operation request. */
	TSharedPtr<UE::Mutable::Private::FParameters> Parameters; 
	TSharedPtr<UE::Mutable::Private::FSystem> MutableSystem;
	TSharedPtr<UE::Mutable::Private::FModel> Model;
	
	TSharedPtr<UE::Mutable::Private::FLiveInstance> LiveInstance;

	TSharedPtr<FUnrealMutableResourceProvider> ExternalResourceProvider;

	TSharedPtr<FMutableUpdateImageContext> UpdateImageContext;
	
	/** Manage loading Passthrough Objects discovered on the StartUpdate. */
	TSharedPtr<UE::Mutable::Private::FPassthroughObjectLoader> PassthroughObjectLoader;
	
	/** Weak reference to the instance we are operating on.
	 *It is weak because we don't want to lock it in case it becomes irrelevant in the game while operations are pending and it needs to be destroyed. */
	TWeakObjectPtr<UCustomizableObjectInstance> Instance;

	/** Customizable Object we are operating on. It can be destroyed between Game Thread tasks. */
	TWeakObjectPtr<UCustomizableObject> Object;

	// Update stats
	double StartQueueTime = 0.0;
	double QueueTime = 0.0;
	
	double StartUpdateTime = 0.0;
	double UpdateTime = 0.0;

	double TaskGetMeshesTimeStart = 0.0;
	double TaskGetMeshesTime = 0.0;
	double TaskGetImagesTimeStart = 0.0;
	double TaskGetImagesTime = 0.0;
	double TaskConvertResourcesTime = 0.0;
	double TaskCallbacksTime = 0.0;

	// Update Memory stats
	int64 UpdateStartBytes = 0;

	int64 UpdateEndPeakBytes = 0;
	
	int64 UpdateEndRealPeakBytes = 0;
	
	/** This option comes from the operation request. It is used to reduce the number of mipmaps that mutable must generate for images.  */
	int32 MipsToSkip = 0;
	
	EUpdateResult UpdateResult = EUpdateResult::Success;
	
	/** This option comes from the operation request */
	uint8 bNeverStreamMips : 1 = false;
	
	/** When this option is enabled it will reuse the Mutable core instance and its temp data between updates.  */
	uint8 bLiveUpdateMode : 1 = false;

	uint8 bUseResueTextureCache : 1 = false;
	
	uint8 bUseSkeletalMeshCache : 1 = false;
	
	uint8 bUseCaches : 1 = true;
	
	/** Whether the mesh to generate should support Mesh LOD streaming or not. */
	uint8 bStreamMeshLODs : 1 = false;

	/** true if the Update has blocked Low Priority Tasks from launching. */
	uint8 bLowPriorityTasksBlocked : 1 = false;

	/** Add to the high priority queue or not */
	uint8 bIsHighPriority : 1 = false;
	
	/** Do not optimize the update. */
	uint8 bForce : 1= false;

	/** Update due to a bake. */
	uint8 bBake : 1 = false;
	
	uint8 bIsUpdateAborted : 1 = false;
		
	uint8 bKeepOwnershipOfGeneratedResources : 1 = false;
	
	uint8 UpdateStarted : 1 = false;

	uint8 bLevelBegunPlay : 1 = false;

	/** true if the update has been optimized (skips all Tasks and calls FinishUpdateGlobal directly on the Enqueue). */
	uint8 bOptimizedUpdate : 1 = false;
	
	/** Determines if the Cachelayer1 (locked and unlocked data) should be cleared or not during the execution of the BeginUpdate_MutableThread. 
	 * It is only useful during the initial generation as we do not need to consider clearing the layer one during LOD or MIP generation
	 */
	bool bClearCacheLayer1 = false;
};


typedef TSharedPtr<FUpdateContextPrivate> FMutableInstanceUpdate;

#if WITH_EDITORONLY_DATA

UENUM()
enum class ECustomizableObjectDDCPolicy : uint8
{
	None = 0,
	Local,
	Default
};


// Struct used to keep a copy of the EditorSettings needed to compile Customizable Objects.
struct FEditorCompileSettings
{
	// General case
	bool bIsMutableEnabled = true;

	// Auto Compile 
	bool bEnableAutomaticCompilation = true;
	bool bCompileObjectsSynchronously = true;

	// DDC settings
	ECustomizableObjectDDCPolicy EditorDerivedDataCachePolicy = ECustomizableObjectDDCPolicy::Default;
	ECustomizableObjectDDCPolicy CookDerivedDataCachePolicy = ECustomizableObjectDDCPolicy::Default;
};

#endif

/** Private part, hidden from outside the plugin.
 *
 * UE STREAMING:
 * 
 * [- NumLODsAvailable -----------------------------] = 8 (State and platform dependent)
 * [- Stripped --[- Packaged -----------------------]
 * 0      1      2      3      4      5      6      7    
 * |------|------|------|------|------|------|------|
 *               [- Streaming --------[- Residents -]
 *               ^                    ^
 *               |                    |
 *               FirstLODAvailable    FirstResidentLOD
 * 
 * [- NumLODsToStream ----------------] = 5 (Compiled constant, ModelResources)
 *
 * NumLODsToStream = 0
 *
 *
 * DEFINITIONS:
 * 
 * FirstLODAvailable =	    First available lod per platform.																Skeletal Mesh Component.
 * FirstResidentLOD =       First LOD generated with geometry.																Skeletal Mesh Component.
 */
UCLASS()
class UCustomizableObjectSystemPrivate : public UObject, public IStreamingManager
{
	GENERATED_BODY()
	
public:
	// Singleton for the unreal mutable system.
	static UCustomizableObjectSystem* SSystem;

	// Pointer to the lower level mutable system that actually does the work.
	TSharedPtr<UE::Mutable::Private::FSystem> MutableSystem;

	/** Store the last streaming memory size in bytes, to change it when it is safe. */
	uint64 LastWorkingMemoryBytes = 0;

	// This object is responsible for streaming data to the MutableSystem.
	TSharedPtr<class FUnrealMutableModelBulkReader> Streamer;

	TDeque<FMutableInstanceUpdate> HighPriorityInstanceUpdates;
	TDeque<FMutableInstanceUpdate> LowPriorityInstanceUpdates;
	int32 PendingInstanceUpdates = 0;

	static int32 EnableMutableProgressiveMipStreaming;
	static int32 EnableMutableLiveUpdate;
	static int32 EnableMutableAnimInfoDebugging;
	static int32 MutableMeshesLODBias;
	static int32 MaxTextureSizeToGenerate;
	static bool bGenerateInstancesWithinRange;
	
	// IStreamingManager interface
	virtual void UpdateResourceStreaming(float DeltaTime, bool bProcessEverything = false) override;
	virtual int32 BlockTillAllRequestsFinished(float TimeLimit = 0.0f, bool bLogResults = false) override;
	virtual void CancelForcedResources() override {}
	virtual void AddLevel(ULevel* Level) override {}
	virtual void RemoveLevel(ULevel* Level) override {}
	virtual void NotifyLevelChange() override {}
	virtual void SetDisregardWorldResourcesForFrames(int32 NumFrames) override {}
	virtual void NotifyLevelOffset(ULevel* Level, const FVector& Offset) override {}

	void UpdateTick(bool bBlocking);

	UE_API void MainTick(bool bBlocking);
	
	UE_API int32 GetRemainingWork() const;
	
	UCustomizableObjectSystem* GetPublic() const;

	CUSTOMIZABLEOBJECT_API void EnqueueUpdateSkeletalMesh(const TSharedRef<FUpdateContextPrivate>& Context);
	
	TSharedPtr<FUpdateContextPrivate> CurrentMutableOperation = nullptr;

	// Handle to the registered TickDelegate.
	FTSTicker::FDelegateHandle TickWarningsDelegateHandle;

	/** Change the current status of Mutable. Enabling/Disabling core features.	
	 * Disabling Mutable will turn off compilation, generation, and streaming and will remove the system ticker. */
	static void OnMutableEnabledChanged(IConsoleVariable* CVar = nullptr);

	/** Update the last set amount of internal memory Mutable can use to build objects. */
	void UpdateMemoryLimit();
	
	bool TryStartUpdate(const TSharedRef<FUpdateContextPrivate>& Context);
	bool TryStartAutomaticUpdateForInstance(UCustomizableObjectInstance& Instance);

	/** Start the actual work of Update Skeletal Mesh process (Update Skeletal Mesh without the queue). */
	void StartUpdate(const TSharedRef<FUpdateContextPrivate>& Context);

	/** See UCustomizableObjectInstance::IsUpdating. */
	bool IsUpdating(const UCustomizableObjectInstance& Instance) const;

	/** Mutable TaskGraph system (Mutable Thread). */
	FMutableTaskGraph MutableTaskGraph;

	/** Last Mutable task from the previous update. The next update can not start until this task has been completed. */
	UE::Tasks::FTask LastUpdateMutableTask = UE::Tasks::MakeCompletedTask<void>();
	
	FLogBenchmarkUtil LogBenchmarkUtil;
	
	bool bAutoCompileCommandletEnabled = false;

	// mutable.GenerateInstancesWithinRange = true
	void UpdateViewLocations();
	bool IsDiscardedByDistance(const UCustomizableObjectInstance& Instance) const;
	void DiscardInstance(UCustomizableObjectInstance& Instance);

	// The list of actors that define a view radius (GenerationRange) around them.
	TSet<TWeakObjectPtr<const AActor>> ViewCenters;
	TArray<FVector> ViewLocations;

	// Instance generation range when mutable.GenerateInstancesWithinRange is true
	float GenerationRangeSquare = 2000.f * 2000.f;
	
	TSharedRef<FMutableStreamableManager> StreamableManager = MakeShared<FMutableStreamableManager>();

#if WITH_EDITOR
	// Copy of the Mutable Editor Settings tied to CO compilation. They are updated whenever changed
	FEditorCompileSettings EditorSettings;
#endif

#if WITH_EDITORONLY_DATA
	// Array to keep track of cached objects
	TArray<FGuid> UncompiledCustomizableObjectIds;

	/** Weak pointer to the Uncompiled Customizable Objects notification */
	TWeakPtr<SNotificationItem> UncompiledCustomizableObjectsNotificationPtr;
#endif

	/** Time when the "Started Update Skeletal Mesh Async" log will be unmuted (in seconds). */
	float LogStartedUpdateUnmute = 0.0;

	/** Time of the last "Started Update Skeletal Mesh Async" log (in seconds). */
	float LogStartedUpdateLast = 0;

public:

	static const TArray<UCustomizableObjectInstance*>& GetCustomizableObjectInstances()
	{
		return CustomizableObjectInstances;
	}
	static void RegisterCustomizableObjectInstance(UCustomizableObjectInstance& Obj);
	static void UnregisterCustomizableObjectInstance(UCustomizableObjectInstance& Obj);


	static void RegisterInstanceToGeneratedList(UCustomizableObjectInstance& Obj);
	static void UnregisterInstancefromGeneratedList(UCustomizableObjectInstance& Obj);
	static void RegisterInstanceToAutomaticUpdateList(UCustomizableObjectInstance& Obj);
	static void UnregisterInstanceFromAutomaticUpdateList(UCustomizableObjectInstance& Obj);

	void TickDiscardInstances();
	void TickAutomaticUpdateInstances();

	static void RegisterInstanceToPendinSetReferenceSkeletalMeshList(UCustomizableObjectInstance& Obj);
	void TickPendingSetReferenceSkeletalMesh();


	/**
	 * Get to know if the settings used by the mutable syustem are optimzed for benchmarking operations or not
	 * @return true if using benchmarking optimized settings, false otherwise.
	 */
	static bool IsUsingBenchmarkingSettings();

	/**
	 * Enable or disable the usage of benchmarking optimized settings
	 * @param bUseBenchmarkingOptimizedSettings True to enable the usage of benchmarking settings, false to disable it.
	 */
	CUSTOMIZABLEOBJECT_API static void SetUsageOfBenchmarkingSettings(bool bUseBenchmarkingOptimizedSettings);

	void LogUpdateMessage(const FString& Message, ELogVerbosity::Type Verbosity, UCustomizableObjectInstance* Instance, bool bClearMessageList = false);

private:
	/** Flag that controls some of the settings used for the generation of instances. */
	inline static bool bUseBenchmarkingSettings = false;

	// Added to in PostInitProperties, removed from in BeginDestroy. No CDOs
	static TArray<UCustomizableObjectInstance*> CustomizableObjectInstances;
	static TArray<UCustomizableObjectInstance*> GeneratedInstances;
	static TArray<UCustomizableObjectInstance*> AutomaticUpdateInstances;

	static TArray<UCustomizableObjectInstance*> PendingSetReferenceSkeletalMesh;

	int32 DiscardIndex = 0;
	int32 AutomaticUpdateIndex = 0;
};


/** Set OnlyLOD to -1 to generate all mips */
CUSTOMIZABLEOBJECT_API TUniquePtr<FTexturePlatformData> MutableCreateImagePlatformData(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage, int32 OnlyLOD, uint16 FullSizeX, uint16 FullSizeY);


/** Return true if Streaming is enabled for the given Object. */
bool IsStreamingEnabled(const UCustomizableObject& Object, int32 State);


template<typename Type>
Type* ToObject(const FString& Parameter)
{
	return Cast<Type>(FSoftObjectPath::ConstructFromStringPath(Parameter).TryLoad());
}
template<typename Type>
Type* ToObject(const FName& Parameter)
{
	return ToObject<Type>(Parameter.ToString());
}


#undef UE_API
