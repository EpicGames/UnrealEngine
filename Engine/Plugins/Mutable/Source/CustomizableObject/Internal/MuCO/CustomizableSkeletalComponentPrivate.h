// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableSkeletalComponentPrivate.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UCustomizableObjectInstanceUsage;

UCLASS(MinimalAPI)
class UCustomizableSkeletalComponentPrivate : public UObject
{
	GENERATED_BODY()

public:
	UE_API UCustomizableSkeletalComponentPrivate();
	
	UPROPERTY(Instanced)
	TObjectPtr<UCustomizableObjectInstanceUsage> InstanceUsage;
};

#undef UE_API
