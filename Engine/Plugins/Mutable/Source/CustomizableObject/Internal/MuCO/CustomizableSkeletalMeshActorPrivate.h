// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "UObject/ObjectPtr.h"
#include "Containers/Array.h"

#include "CustomizableSkeletalMeshActorPrivate.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UCustomizableObjectInstance;
class UCustomizableSkeletalComponent;
class ACustomizableSkeletalMeshActor;


UCLASS(MinimalAPI)
class UCustomizableSkeletalMeshActorPrivate : public UObject
{
	GENERATED_BODY()

public:
	UE_API ACustomizableSkeletalMeshActor* GetPublic();

	UE_API void Init(UCustomizableObjectInstance* Instance);
	
	UE_API TArray<TObjectPtr<UCustomizableSkeletalComponent>>& GetComponents();
};

#undef UE_API
