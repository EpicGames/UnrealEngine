// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObjectInstanceDescriptor.h"

#define UE_API CUSTOMIZABLEOBJECT_API


/** Hash of the Descriptor.
* Can change and is not backwards compatible. Do not serialize. */
class FDescriptorHash
{
public:
	FDescriptorHash() = default;

	UE_API explicit FDescriptorHash(const FCustomizableObjectInstanceDescriptor& Descriptor);
	
	/** Return true if this Hashes are the same. */
	UE_API bool operator==(const FDescriptorHash& Other) const = default;

	UE_API FString ToString() const;

private:
	uint32 Hash = 0;

};

#undef UE_API
