// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/System.h" // For the MUTABLE_USE_NEW_TASKGRAPH define

#include "Tasks/Task.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Queue.h"


extern TAutoConsoleVariable<bool> CVarEnableParallelUpdates;

/** Mutable Tasks System.
 *
 * Allows launching tasks in different threads:
 * - Mutable Thread
 * - Game Thread
 * - Any Thread
 *
 * Concurrency between tasks in the Mutable Thread is forbidden by chaining all tasks through their prerequisites.
 *
 *
 * LOW PRIORITY MUTABLE TASKS:
 *
 * - Only one Low Priority task can be launched at the same time.
 *   This is because once a Task Graph task is launched, it can not be canceled.
 *   To allow canceling them, the system holds them until it can ensure that its execution will be imminent (no other tasks running).
 *
 * - A Low Priority task will not be launched if one of the follow conditions is true (in order):
 *     1. There is a task Low Priority task running.
 *     2. Flag bLaunchMutableTaskLowPriority is false.
 *     3. There is a Normal Priority task running (unless time limit).
 */
class FMutableTaskGraph
{
	struct FMutableThreadLowPriorityTask
	{
		uint32 Id;
		FString DebugName;
		TFunction<void()> Body;
		double CreationTime = FPlatformTime::Seconds();
	};

	struct FGameThreadTask
	{
		FString DebugName;
		TUniqueFunction<void(float)> Body;
		TArray<UE::Tasks::FTask> Prerequisites;
		
		/** Check if all the dependencies of this task have been completed. */
		bool AreDependenciesComplete() const;
	};
	
public:
	~FMutableTaskGraph();

	void AddGameThreadTask(const TCHAR* DebugName, TUniqueFunction<void(float)>&& TaskBody, const TArray<UE::Tasks::FTask>& Prerequisites = {});
	
	/** Create and launch a task on the Mutable Thread with Normal priority. */
	UE::Tasks::FTask AddMutableThreadTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& Prerequisites = {});

	UE::Tasks::FTask AddGenerationTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& Prerequisites = {});

	UE::Tasks::FTask AddImageStreamingTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& Prerequisites = {});

	UE::Tasks::FTask AddMeshStreamingTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& Prerequisites = {});
	
	/** Create and launch a task on the Mutable Thread with Low priority. */
	uint32 AddMutableThreadTaskLowPriority(const TCHAR* DebugName, TFunction<void()>&& TaskBody);

	/** Cancel, if not already launched, a Mutable Thread with Low priority. 
	  * Return true if the task has been canceled before launching it.
	  * Return false if not found or running. */
	bool CancelMutableThreadTaskLowPriority(uint32 Id);
	
	/** Wait for all Mutable Thread tasks. */
	void WaitForMutableTasks();

	/** Wait for the launched low-priority task if it matches the TaskID. */
	void WaitForLaunchedLowPriorityTask(uint32 TaskID);

	/** Allow or disallow launching Mutable Tasks with Low priority.
	 @param bFromMutableTask true if called from a Mutable Task. */
	void AllowLaunchingMutableTaskLowPriority(bool bAllow, bool bFromMutableTask);

	/** Returns the number of remaining tasks. */
	void Tick();
	
	int32 GetRemainingWork() const;

	/** A Mutable Task Low Priority will only be launched if:
	 * - No other low priority task is running.
	 * - Is allowed to launch low priority tasks.
	 *
	 * @param bFromMutableTask true if called from a Mutable Task. */
	void TryLaunchMutableTaskLowPriority(bool bFromMutableTask);

	void TryLaunchGameThreadTask();
	
	/** Return true if the task is completed (or is no longer valid). */
	bool IsTaskCompleted(const UE::Tasks::FTask& Task) const;

	mutable FCriticalSection MutableTaskLock;

	/** Allow or disallow launching low priority tasks. */
	bool bAllowLaunchMutableTaskLowPriority = true;

	/** Queue of low priority tasks. FIFO. */
	TArray<FMutableThreadLowPriorityTask> QueueMutableTasksLowPriority;
	
	/** Queue of Game Thread tasks that need to be executed. */
	TQueue<FGameThreadTask> GameThreadTasks;

public:
	static constexpr uint32 INVALID_ID = 0;

private:
	/** Incremental task ID generator. */
	uint32 TaskIdGenerator = INVALID_ID;

	/** The ID of the Last Mutable Task Low Priority launched to the TaskGraph system. */
	uint32 LastMutableTaskLowPriorityID = INVALID_ID;

	/** Last Mutable Task Low Priority launched to the TaskGraph system. */
	UE::Tasks::FTask LastMutableTaskLowPriority;

	/** Last Mutable Task launched to the TaskGraph system. Low and normal priority. */
	UE::Tasks::FTask LastMutableTask;
	
	mutable FCriticalSection CriticalSectionLastGenerationTask;
	mutable FCriticalSection CriticalSectionLastImageStreaming;
	mutable FCriticalSection CriticalSectionLastMeshStreamingTask;
	
	UE::Tasks::FTask LastGenerationTask;
	UE::Tasks::FTask LastImageStreamingTask;
	UE::Tasks::FTask LastMeshStreamingTask;
};

