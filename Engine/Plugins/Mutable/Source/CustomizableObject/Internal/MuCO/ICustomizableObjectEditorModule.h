// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#include "Templates/SharedPointer.h"
#include "MuCO/CustomizableObject.h"

class FCustomizableObjectEditorLogger;
class ICustomizableObjectEditor;
class ICustomizableObjectInstanceEditor;
class IToolkitHost;
class UCustomizableObject;
class UCustomizableObjectPrivate;
class UCustomizableObjectInstance;
class USkeletalMesh;
class FExtensibilityManager;
class FBakeOperationCompletedDelegate;
class UEdGraph;
struct FBakingConfiguration;
struct FCompilationRequest;
struct FCompilationOptions;

extern const FName CustomizableObjectEditorAppIdentifier;
extern const FName CustomizableObjectInstanceEditorAppIdentifier;
extern const FName CustomizableObjectPopulationEditorAppIdentifier;
extern const FName CustomizableObjectPopulationClassEditorAppIdentifier;
extern const FName CustomizableObjectDebuggerAppIdentifier;
extern const FName CustomizableObjectMacroLibraryEditorAppIdentifier;


#define MODULE_NAME_COE "CustomizableObjectEditor"


/**
 * Customizable object editor module interface
 */
class ICustomizableObjectEditorModule : public IModuleInterface
{
public:
	static ICustomizableObjectEditorModule* Get()
	{
#if WITH_EDITOR
		return FModuleManager::LoadModulePtr<ICustomizableObjectEditorModule>(MODULE_NAME_COE);
#else
		return nullptr; // IsRunningGame
#endif
	}
	
	static ICustomizableObjectEditorModule& GetChecked()
	{
#if !WITH_EDITOR
		check(false); // This module is editor-only. DO NOT try to access it during gameplay
#endif
		return FModuleManager::LoadModuleChecked<ICustomizableObjectEditorModule>(MODULE_NAME_COE);
	}

	virtual TSharedPtr<FExtensibilityManager> GetCustomizableObjectEditorToolBarExtensibilityManager() { return nullptr; }
	virtual TSharedPtr<FExtensibilityManager> GetCustomizableObjectEditorMenuExtensibilityManager() { return nullptr; }
	
	/** Returns the module logger. */
	virtual FCustomizableObjectEditorLogger& GetLogger() = 0;
	
	/** Return if the CO is not compiled or the ParticipatingObjects system has detected a change (participating objects dirty or re-saved since last compilation).
	  * @param Object object to check.
	  * @param bSkipIndirectReferences if true, do not check for added/removed indirect references.
	  * @param OutOfDatePackages list of out of date packages.
	  * @param AddedPackages list of added packages since the last compilation.
	  * @param RemovedPackages list of removed packages since the last compilation.
	  * @param bReleaseVersionDiff true if the Release Version has changed since the last compilation.
	  * @return true if the compilation is out of date. */
	virtual bool IsCompilationOutOfDate(const UCustomizableObject& Object, bool bSkipIndirectReferences, TArray<FName>& OutOfDatePackages, TArray<FName>& AddedPackages, TArray<FName>& RemovedPackages, bool& bReleaseVersionDiff) const = 0;
	
	/** Fixup operations executed once the upstream COs (parent COs) have been loaded.
	 * @param CustomizableObjectCustomVersion Version index used to determine what change in the data should be performed.
	 */
	virtual void OnUpstreamCOsLoadedFixup(UCustomizableObject* Object, int32 CustomizableObjectCustomVersion) const = 0;
	
	using IsCompilationOutOfDateCallback = TFunction<void(bool bOutOfDate, bool bVersionDiff, const TArray<FName>& OutOfDatePackages, const TArray<FName>& AddedPackages, const TArray<FName>& RemovedPackages)>;

	/** Async version. See IsCompilationOutOfDate.
	 * @param MaxTime. Max time it can spend on each Game Thread tick. Use MAX_flt for sync call. */
	virtual void IsCompilationOutOfDate(const UCustomizableObject& Object, bool bSkipIndirectReferences, float MaxTime, const IsCompilationOutOfDateCallback& Callback) const = 0;
	
	/** See GraphTraversal::IsRootObject(...) */
	virtual bool IsRootObject(const UCustomizableObject& Object) const = 0;

	/** Get the current Release Version for the given Object. 
	  * @return Current version as string. */
	virtual FString GetCurrentReleaseVersionForObject(const UCustomizableObject& Object) const = 0;

	/** See GraphTraversal::GetRootObject(...) */
	virtual UCustomizableObject* GetRootObject(UCustomizableObject* ChildObject) const = 0;

	/** See GraphTraversal::GetRootObject(...) */
	virtual const UCustomizableObject* GetRootObject(const UCustomizableObject* ChildObject) const = 0;

	virtual void GetAllObjectsInGraph(UCustomizableObject*, TSet<UCustomizableObject*>& OutRelated ) const = 0;
	
	/**
	 * Execute this method in order to bake the provided instance. It will schedule a special type of instance update before proceeding with the bake itself.
	 * @param InTargetInstance The instance we want to bake
	 * @param InBakingConfig Structure containing the configuration to be used for the baking
	 */
	virtual void BakeCustomizableObjectInstance(UCustomizableObjectInstance* InTargetInstance, const FBakingConfiguration& InBakingConfig) = 0;

	/** Evict scenes from the UCustomizableObjectSkeletalMeshThumbnailRenderer cache used for InSkeletalMesh */
	virtual void EvictSkeletalMeshThumbnailScene(USkeletalMesh* InSkeletalMesh) const = 0;

	/** Compile the given Customizable Object.
	 * If calling inside the Customizable Object Editor module, consider using ICustomizableObjectEditorModulePrivate::EnqueueCompileRequest. */
	virtual void CompileCustomizableObject(UCustomizableObject& Object, const FCompileParams* Params, bool bSilent, bool bForce) = 0;

	virtual void Tick(bool bBlocking) = 0;
	
	virtual int32 GetRemainingWork() const = 0;

	/** Force finish current compile request and cancels all pending requests */
	virtual void CancelCompileRequests() = 0;

	/** Return the number of pending compilation requests. Ongoing requests included. */
	virtual int32 GetNumCompileRequests() = 0;

	virtual USkeletalMesh* GetReferenceSkeletalMesh(const UCustomizableObject& Object, const FName& Component) const = 0;
	
	/** Perform a fast compilation pass to get all participating objects. */
	virtual TMap<FName, FGuid> GetParticipatingObjects(const UCustomizableObject* Object, const FCompilationOptions* Options = nullptr) const = 0;

	virtual void BackwardsCompatibleFixup(UEdGraph& Graph, int32 CustomizableObjectCustomVersion) = 0;
	
	virtual void PostBackwardsCompatibleFixup(UEdGraph& Graph) = 0;

	virtual bool IsCompiling(const UCustomizableObject& Object) const = 0;

	virtual void BeginCacheForCookedPlatformData(UCustomizableObject& Object, const ITargetPlatform* TargetPlatform) = 0;
	
	virtual bool IsCachedCookedPlatformDataLoaded(UCustomizableObject& Object, const ITargetPlatform* TargetPlatform) = 0;
	
	/** Add a CO to the pending - load list.COs need to wait until all related objects are fully loaded before being able
	 * to do things like check-if-up-to-date or compile. */
	virtual void AddPendingLoad(UCustomizableObject* Object) = 0;

	virtual void RegisterExternalOperation(const UScriptStruct* Operation) = 0;

	virtual void UnregisterExternalOperation(const UScriptStruct* Operation) = 0;
};
 
