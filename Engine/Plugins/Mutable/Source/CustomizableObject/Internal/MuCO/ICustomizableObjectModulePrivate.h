// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/ICustomizableObjectModule.h"

#include "Modules/ModuleManager.h"

namespace UE::Mutable
{
	struct FExternalOperation;
}


class ICustomizableObjectModulePrivate : public ICustomizableObjectModule
{
public:
	static ICustomizableObjectModulePrivate& Get()
	{
		return FModuleManager::LoadModuleChecked<ICustomizableObjectModulePrivate>("CustomizableObject");
	}
	
	virtual TSet<TStrongObjectPtr<const UScriptStruct>> GetRegisteredExternalOperations() const = 0;
};

