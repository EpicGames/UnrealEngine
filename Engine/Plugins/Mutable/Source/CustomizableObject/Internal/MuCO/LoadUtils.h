// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ICookInfo.h"
#include "UObject/SoftObjectPtr.h"

class UClass;
class UObject;
struct FAssetData;
struct FSoftObjectPtr;


namespace UE::Mutable::Private
{
	CUSTOMIZABLEOBJECT_API UObject* LoadObject(const FAssetData& DataAsset);


	CUSTOMIZABLEOBJECT_API UObject* LoadObject(const FSoftObjectPath& Path);
	
	
	CUSTOMIZABLEOBJECT_API UObject* LoadObject(const FSoftObjectPtr& SoftObject);

	
	template <typename T>
	T* LoadObject(UObject* Outer, const TCHAR* Name, const TCHAR* Filename = nullptr, uint32 LoadFlags = LOAD_None, UPackageMap* Sandbox = nullptr, const FLinkerInstancingContext* InstancingContext = nullptr)
	{
#if WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif
		return ::LoadObject<T>(Outer, Name, Filename, LoadFlags, Sandbox, InstancingContext);
	}
	

	template<typename T>
	T* LoadObject(const TSoftObjectPtr<T>& SoftObject)
	{
#if WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif
	
		return SoftObject.LoadSynchronous();
	}


	template<typename T>
	UClass* LoadClass(const FSoftClassPath& Path)
	{
#if WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif
		
		return Path.TryLoadClass<T>();
	}

	
	template<typename T>
	UClass* LoadClass(const TSoftClassPtr<T>& SoftClass)
	{
#if WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif

		return SoftClass.LoadSynchronous();
	}	
}

