// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"

class UCustomizableObjectInstance;
class UTexture2D;
struct FCustomizableObjectInstanceDescriptor;


/** Stat which automatically gets updated to Insights when modified. */
#define DECLARE_BENCHMARK_STAT(Name, Type) \
	struct F##Name \
	{ \
	private: \
		Type Value = 0; \
		\
		mutable FCriticalSection Lock##Name; \
		\
	public: \
		Type GetValue() const \
		{ \
			FScopeLock ScopedLock(&Lock##Name); \
			return Value; \
		} \
		\
		F##Name& operator+=(const Type& Rhs) \
		{ \
			FScopeLock ScopedLock(&Lock##Name); \
			Value += Rhs; \
			return *this; \
		} \
		\
		F##Name& operator=(const Type& Rhs) \
		{ \
			FScopeLock ScopedLock(&Lock##Name); \
			Value = Rhs; \
		\
		SET_DWORD_STAT(STAT_Mutable##Name, Value); \
		\
		return *this; \
		} \
	}; \
	F##Name Name;

#define DECLARE_BENCHMARK_INSIGHTS(Name, Description) \
	DECLARE_DWORD_ACCUMULATOR_STAT(Description, STAT_Mutable##Name, STATGROUP_Mutable);


class FUpdateContextPrivate;
// Mutable stats
DECLARE_STATS_GROUP(TEXT("Mutable"), STATGROUP_Mutable, STATCAT_Advanced);

DECLARE_BENCHMARK_INSIGHTS(NumAllocatedSkeletalMeshes, TEXT("Num Allocated Mutable Skeletal Meshes"));
DECLARE_BENCHMARK_INSIGHTS(NumAllocatedTextures, TEXT("Num Allocated Mutable Textures"));
DECLARE_BENCHMARK_INSIGHTS(TextureGPUSize, TEXT("Size of the Mutable Texture mips that are resident on the GPU"));
DECLARE_BENCHMARK_INSIGHTS(NumInstances, TEXT("Num Instances"));
DECLARE_BENCHMARK_INSIGHTS(NumInstancesLOD0, TEXT("Num Instances at LOD 0"));
DECLARE_BENCHMARK_INSIGHTS(NumInstancesLOD1, TEXT("Num Instances at LOD 1"));
DECLARE_BENCHMARK_INSIGHTS(NumInstancesLOD2, TEXT("Num Instances at LOD 2 or more"));
DECLARE_BENCHMARK_INSIGHTS(NumPendingInstanceUpdates, TEXT("Num Pending Instance Updates"));
DECLARE_BENCHMARK_INSIGHTS(NumBuiltInstances, TEXT("Num Built Instances"))
DECLARE_BENCHMARK_INSIGHTS(InstanceBuildTimeAvrg, TEXT("Avrg Instance Build Time"));


extern TAutoConsoleVariable<bool> CVarEnableBenchmark;

/** Object representing the update data of an instance. In practical terms represents one row of the CSV we generate */
struct FInstanceUpdateStats
{
	FString CustomizableObjectPathName = "";
	FString CustomizableObjectInstancePathName = "";
	FString UpdateType = "";
	FString Descriptor = "";			// Simplified for compatibility (maybe in the future we keep the entire descriptor)

	EUpdateResult UpdateResult = EUpdateResult::Error;

	bool bLevelBegunPlay = false;

	uint32 TriangleCount = 0;
	
	double QueueTime = 0.0;
	double UpdateTime = 0.0;
	double TaskGetMeshesTime = 0.0;

	double TaskGetImagesTime = 0.0;
	double TaskConvertResourcesTime = 0.0;
	double TaskCallbacksTime = 0.0;
	
	double UpdatePeakMemory = 0.0;
	double UpdateRealPeakMemory = 0.0;

	double TaskUpdateImageTime = 0.0;
	double TaskUpdateImagePeakMemory = 0.0;
	double TaskUpdateImageRealPeakMemory = 0.0;
};


struct FStatistic
{
	double Min = +TNumericLimits<double>::Max();
	double Max = -TNumericLimits<double>::Max();
	double LowerBound = 0.0f;
	double UpperBound = 0.0f;
};


struct FInstanceUpdateStatistics
{
	FStatistic TriangleCount;
	FStatistic QueueTime;
	FStatistic UpdateTime;
	FStatistic TaskGetMeshTime;
	FStatistic TaskGetImagesTime;
	FStatistic TaskConvertResourcesTime;
	FStatistic TaskCallbacksTime;
	FStatistic UpdatePeakMemory;
	FStatistic UpdateRealPeakMemory;
	FStatistic TaskUpdateImageTime;
	FStatistic TaskUpdateImagePeakMemory;
	FStatistic TaskUpdateImageRealPeakMemory;
};


// Delegate signature
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMeshUpdateReportedSignature, const TSharedRef<FUpdateContextPrivate> /* Context */, FInstanceUpdateStats)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnImageUpdateReportedSignature, FInstanceUpdateStats)


/** Benchmarking system. Gathers stats and send it to Insights an Benchmarking Files. */
class FLogBenchmarkUtil
{
public:
	~FLogBenchmarkUtil();
	
	/**
	 * Enables or disables the benchmarking system from code. Usefull for enabling the benchmarking without having to mess
	 * with CVarEnableBenchmark
	 * @param bIsEnabled True to enable, false to disable and let CVarEnableBenchmark decide .
	 */
	CUSTOMIZABLEOBJECT_API static void SetBenchmarkReportingStateOverride(bool bIsEnabled);
	
	/**
	 * Get to know if the benchmarking is active or not
	 * @return True if the CVarEnableBenchmark CVar or the override are set to true, false otherwise
	 */
	CUSTOMIZABLEOBJECT_API static bool IsBenchmarkingReportingEnabled();
	
	/** Get stats. */
	void GetInstancesStats(int32& OutNumInstances, int32& OutNumBuiltInstances, int32& OutNumAllocatedSkeletalMeshes) const;

	/** Add Mutable created Texture to track. */
	void AddTexture(UTexture2D& Texture);

	/** Update stats which can only be updated on the tick. */
	void UpdateStats();

	/** Gathers update stats when it has finished. */
	void FinishUpdateMesh(const TSharedRef<FUpdateContextPrivate>& Context);

	void FinishUpdateImage(const FString& CustomizableObjectPathName, const FString& InstancePathName, const FString& InstanceDescriptor, const bool bDidLevelBeginPlay, double TaskUpdateImageTime, const int64 TaskUpdateImageMemoryPeak, const int64 TaskUpdateImageRealMemoryPeak);
	
	DECLARE_BENCHMARK_STAT(NumAllocatedTextures, uint32);
	DECLARE_BENCHMARK_STAT(TextureGPUSize, uint64);

	DECLARE_BENCHMARK_STAT(NumAllocatedSkeletalMeshes, int32);

	DECLARE_BENCHMARK_STAT(NumInstances, int32);

	DECLARE_BENCHMARK_STAT(NumPendingInstanceUpdates, int32)
	DECLARE_BENCHMARK_STAT(NumBuiltInstances, int32)
	DECLARE_BENCHMARK_STAT(InstanceBuildTimeAvrg, double);

	TArray<TWeakObjectPtr<UTexture2D>> TextureTrackerArray;

private:
	
	inline static bool bIsEnabledOverride = false;
	
	double TotalUpdateTime = 0;
	uint32 NumUpdates = 0;

	TSharedPtr<FArchive> Archive;

public:

	/** Delegate invoked each time a new mesh update is reported by this utility */
	FOnMeshUpdateReportedSignature OnMeshUpdateReported;

	/** Delegate invoked each time a new mip update is reported by this utility */
	FOnImageUpdateReportedSignature OnImageUpdateReported;
};
