// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"
#include "MuR/Parameters.h"

namespace ProjectorUtils
{
	/** Get with the equivalent type of CO projector provided a mutable internal projector type.
	 * @param ProjectorType The mutable internal projector type to get the unreal's equivalent projector type.
	 * @return The equivalent projector type to the one provided to be used with the CO and COI systems.
	 */
	CUSTOMIZABLEOBJECT_API
	ECustomizableObjectProjectorType GetEquivalentProjectorType (UE::Mutable::Private::EProjectorType ProjectorType);

	/** Get the equivalent mutable projector type to the CO mutable projector type.
	 * @param ProjectorType The mutable CO projector type to get the mutable's internal equivalent projector type.
	 * @return the projector type provided but as a mutable core projector type.
	 */
	CUSTOMIZABLEOBJECT_API
	UE::Mutable::Private::EProjectorType GetEquivalentProjectorType (ECustomizableObjectProjectorType ProjectorType);
}

