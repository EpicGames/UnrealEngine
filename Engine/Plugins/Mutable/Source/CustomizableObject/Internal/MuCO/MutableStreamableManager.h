// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/StreamableManager.h"

#include "Templates/SharedPointer.h"
#include "UObject/ICookInfo.h"

struct FStreamableHandle;


/** Wrapper to the FStreamableManager that allows to kill in-flight async loads and force them sync. */
class FMutableStreamableManager : public TSharedFromThis<FMutableStreamableManager>
{
public:
	template<typename PathContainerType = TArray<FSoftObjectPath>, typename FuncType = FStreamableDelegate>
		TSharedPtr<FStreamableHandle> RequestAsyncLoad(
			PathContainerType&& TargetsToStream,
			FuncType&& DelegateToCall = FStreamableDelegate(),
			TAsyncLoadPriority Priority = FStreamableManager::DefaultAsyncLoadPriority,
			bool bManageActiveHandle = false,
			bool bStartStalled = false,
			FString&& DebugName = FString(TEXT("MutableRequestAsyncLoad")),
			UE::FSourceLocation&& Location = UE::FSourceLocation::Current())
	{
		check(IsInGameThread());
		
		TSharedRef<FMutableStreamableHandle> MutableStreamableHandle = MakeShared<FMutableStreamableHandle>();

		if constexpr (std::is_constructible_v<FStreamableDelegate, FuncType>)
		{
			MutableStreamableHandle->Delegate = DelegateToCall;
		}
		else
		{
			MutableStreamableHandle->Delegate = FStreamableDelegate::CreateLambda(DelegateToCall);
		}

		FStreamableDelegate Delegate = FStreamableDelegate::CreateLambda([MutableStreamableHandle, WeakThis = SharedThis(this)->AsWeak()]
		{
			check(IsInGameThread());
			
			MutableStreamableHandle->Delegate.ExecuteIfBound();
			MutableStreamableHandle->bCompleted = true;

			if (TSharedPtr<FMutableStreamableManager> This = WeakThis.Pin())
			{
				This->StreamableHandles.RemoveSingle(MutableStreamableHandle);
			}
		});
		
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		TSharedPtr<FStreamableHandle> Handle = StreamableManager.RequestAsyncLoad(TargetsToStream, Delegate, Priority, bManageActiveHandle, bStartStalled, MoveTemp(DebugName), MoveTemp(Location));
		if (Handle)
		{
			MutableStreamableHandle->Handle = Handle;
			StreamableHandles.Emplace(MutableStreamableHandle);
		}
		
		return MutableStreamableHandle->Handle;
	}
	
	template<typename PathContainerType = TArray<FSoftObjectPath>>
	TSharedPtr<FStreamableHandle> RequestSyncLoad(
		PathContainerType&& TargetsToStream,
		bool bManageActiveHandle = false,
		FString&& DebugName = FString(TEXT("MutableRequestAsyncLoad")),
		UE::FSourceLocation&& Location = UE::FSourceLocation::Current())
	{
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
		return StreamableManager.RequestSyncLoad(TargetsToStream, bManageActiveHandle, MoveTemp(DebugName), MoveTemp(Location));
	}

	CUSTOMIZABLEOBJECT_API void Tick(bool bBlocking);
	
	int32 GetRemainingWork() const;
	
private:
	struct FMutableStreamableHandle
	{
		/** Always valid. */
		TSharedPtr<FStreamableHandle> Handle;

		FStreamableDelegate Delegate;

		/** Load completed and delegate called.
		 * Used since FStreamableHandle does not have away of knowing when the delegate has been called. */
		bool bCompleted = false;
	};
	
	FStreamableManager StreamableManager;

	TArray<TSharedRef<FMutableStreamableHandle>> StreamableHandles;
};

