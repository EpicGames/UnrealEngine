// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Delegates/Delegate.h" 


/** Simple state machine with transitions check.
 *
 * Type T must implement the following:
 * - EState: Enum of valid states.
 * - StartState: Variable defining the initial state.
 * - ValidTransitions: 2D bool array of valid transitions. */
template<typename T>
class FStateMachine
{
public:
	using EState = typename T::EState;

	static_assert(static_cast<int32>(EState::Count) * static_cast<int32>(EState::Count) == sizeof(T::ValidTransitions), "Size of EState and ValidTransitions do not match.");
	
	void NextState(EState State)
	{
		check(T::ValidTransitions[static_cast<int32>(CurrentState)][static_cast<int32>(State)])

		EState PreviousState = CurrentState; 
		CurrentState = State;
		OnStateChangedDelegate.Broadcast(PreviousState, CurrentState);
	}

	EState Get() const
	{
		return CurrentState;
	}

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStateChangedDelegate, EState, EState)

	FOnStateChangedDelegate& GetOnStateChangedDelegate()
	{
		return OnStateChangedDelegate;
	}
	
private:
	FOnStateChangedDelegate OnStateChangedDelegate;

	EState CurrentState = T::StartState;
};

