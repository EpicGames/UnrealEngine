// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//! Order of the unreal vertex buffers when in mutable data
#define MUTABLE_VERTEXBUFFER_POSITION	0
#define MUTABLE_VERTEXBUFFER_TANGENT	1
#define MUTABLE_VERTEXBUFFER_TEXCOORDS	2

#include "MuR/Ptr.h"
#include "MuR/ManagedPointer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Tasks/Task.h"


class UPhysicsAsset;
struct FClothBufferIndexMapping;
struct FMeshToMeshVertData;
struct FCustomizableObjectMeshToMeshVertData;
struct FMorphTargetMeshData;

namespace UE::Mutable::Private
{
	class FPhysicsBody;
	class FMesh;
	class FSkeletalMesh;
	class FMeshBufferSet;
	class FPhysicsBody;
}
struct FReferenceSkeleton;
struct FSkelMeshRenderSection;
class UModelResources;
class FSkeletalMeshLODRenderData;
class USkeletalMesh;
class USkeleton;
class USkeletalBodySetup;
class FMorphTargetVertexInfoBuffers;

namespace UnrealConversionUtils
{
	struct FSectionClothData
	{
		int32 SectionIndex;
		int32 LODIndex;
		int32 BaseVertex;
		TArrayView<const uint16> SectionIndex16View;
		TArrayView<const uint32> SectionIndex32View;
		TArrayView<const int32> ClothingDataIndicesView;
		TArrayView<const FCustomizableObjectMeshToMeshVertData> ClothingDataView;
		TArray<FMeshToMeshVertData> MappingData;
	};
	
	/*
	 * Shared methods between mutable instance update and viewport mesh generation
	 * These are the result of stripping up parts of the reference code to be able to share it on the two
	 * current pipelines (instance update and USkeletal mesh generation for a mesh viewport)
	 */
	
	/**
	 * Prepares the render sections found on the InSkeletalMesh and sets them up accordingly what the InMutableMesh requires
	 * @param LODResource - LODRenderData whose sections are ought to be updated
	 * @param InMutableMesh - Mutable mesh to be used as reference for the section data update on the Skeletal Mesh
	 * @param InBoneMap - Bones to be set as part of the sections.
	 * @param InFirstBoneMapIndex - Index to the first BoneMap bone that belongs to this LODResource.
	 * @param SectionMetadata - Section metadata for each surface in InMutableMesh. 
	 * @return True if the operation was succesfull, false otherwise.
	 */
	CUSTOMIZABLEOBJECT_API bool SetupRenderSections(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		const FReferenceSkeleton& InRefSkeleton,
		const TArray<FName>& InBoneMap,
		const int32 InFirstBoneMapIndex);


	/** Initializes the LODResource's VertexBuffers with dummy data to prepare it for streaming.
	 * @param LODResource - LODRenderData to update
	 * @param InMutableMesh - Mutable mesh to be used as reference for the section data update on the Skeletal Mesh
	 * @param bAllowCPUAccess - Keeps this LODs data on the CPU so it can be used for things such as sampling in FX.
	 */
	CUSTOMIZABLEOBJECT_API void InitVertexBuffersWithDummyData(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		const bool bAllowCPUAccess);

	/** Performs a copy of the data found on the vertex buffers on the mutable mesh to the buffers of the skeletal mesh
	 * @param LODResource - LODRenderData to update
	 * @param InMutableMesh - Mutable mesh to be used as reference for the vertex data update on the Skeletal Mesh
	 * @param bAllowCPUAccess - Keeps this LODs data on the CPU so it can be used for things such as sampling in FX.
	 */
	CUSTOMIZABLEOBJECT_API void CopyMutableVertexBuffers(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		const bool bAllowCPUAccess);

	
	/**
	 * Initializes the LODResource's IndexBuffers with dummy data to prepare it for streaming.
	 * @param LODResource - LODRenderData to be updated.
	 * @param InMutableMesh - The mutable mesh whose index buffers you want to work with
	 */
	CUSTOMIZABLEOBJECT_API void InitIndexBuffersWithDummyData(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh);

	/**
	 *Performs a copy of the data found on the index buffers on the mutable mesh to the buffers of the skeletal mesh
	 * @param LODResource - LODRenderData to be updated.
	 * @param InMutableMesh - The mutable mesh whose index buffers you want to work with
	 * @param bOutMarkRenderStateDirty
	 * @param RenderToMutableSectionIndexMap - Map form RenderSection Index to MutableSection Index. In case the optional is not set, the identity is used.
	 * @return True if the operation could be performed successfully, false if not.
	 */
	CUSTOMIZABLEOBJECT_API bool CopyMutableIndexBuffers(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		bool& bOutMarkRenderStateDirty,
		TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap = {});
	

	/**
	 *Performs a copy of the data found on the index buffers on the mutable mesh to the buffers of the skeletal mesh
	 * @param LODResource - LODRenderData to be updated.
	 * @param Owner - Outer mesh of the LODResource. 
	 * @param LODIndex - LOD Index of the LODResource.
	 * @param InMutableMesh - Mutable mesh to be used to initialize the SkinWeightProfilesBuffers.
	 */
	CUSTOMIZABLEOBJECT_API void CopyMutableSkinWeightProfilesBuffers(
		FSkeletalMeshLODRenderData& LODResource,
		USkeletalMesh& Owner,
		int32 LODIndex,
		const UE::Mutable::Private::FMesh* InMutableMesh
		);


	/**
	 *Performs a copy of the render data of a specific Skeletal Mesh LOD to another Skeletal Mesh
	 * @param LODResource - LODRenderData to copy to.
	 * @param SourceLODResource - LODRenderData to copy from.
	 * @param Owner - Outer mesh of the LODResource and SourceLODResource. 
	 * @param LODIndex - LOD Index to copy to.
	 * @param bAllowCPUAccess - Keeps this LODs data on the CPU so it can be used for things such as sampling in FX.
	 */
	CUSTOMIZABLEOBJECT_API void CopySkeletalMeshLODRenderData(
		FSkeletalMeshLODRenderData& LODResource,
		FSkeletalMeshLODRenderData& SourceLODResource,
		USkeletalMesh& Owner,
		int32 LODIndex,
		const bool bAllowCPUAccess
	);

	/**
	 * Update SkeletalMeshLODRenderData buffers size.
	 * @param LODResource - LODRenderData to be updated.
	 */
	CUSTOMIZABLEOBJECT_API void UpdateSkeletalMeshLODRenderDataBuffersSize(
		FSkeletalMeshLODRenderData& LODResource
	);

	void MorphTargetVertexInfoBuffers(
			FSkeletalMeshLODRenderData& LODResource, 
			const USkeletalMesh& Owner, 
			const UE::Mutable::Private::FMesh& MutableMesh, 
			int32 LODIndex, 
			bool bGenerateCPUMorphTargets,
			TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap = {});


	CUSTOMIZABLEOBJECT_API void SetMorphData(
			TNotNull<UE::Mutable::Private::FMesh*> Result, 
			TNotNull<const USkeletalMesh*> SkeletalMesh,
			int32 LODIndex, int32 SectionIndex, 
			uint32 SourceSectionBaseVertexIndex, uint32 SourceSectionNumVertices, uint32 SourceMeshNumVertices, 
			FMorphTargetVertexInfoBuffers* OptionalMorphTargetVertexBuffer);

	CUSTOMIZABLEOBJECT_API UE::Tasks::FTask ConvertSkeletalMeshFromRuntimeData(USkeletalMesh* SkeletalMesh, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 ConversionFlags, UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> Result);

	/**
	 * Converts a LOD range of a USkeletalMesh to a FSkeletalMesh.
	 * @param SkeletalMesh - USkeletalMesh to convert.
	 * @param LODBegin - Begin of the LOD range to load.
	 * @param LODEnd - End of the LOD range to load.
	 * @param GeometryLODBegin - Begin of the LOD range for which the geometry will be loaded.
	 * @param GeometryLODEnd - End of the LOD rnage for which the geometry will be loaded.
	 * @param ConversionFlags - Flags used in the conversion to discard or modify some of the data. 
	 * @param Result - Result FSkeletalMesh.
	 * @return The task which upon completion Result will contain the conversion result.
	 */
	CUSTOMIZABLEOBJECT_API UE::Tasks::FTask ConvertSkeletalMeshFromRuntimeData(USkeletalMesh* SkeletalMesh, int32 LODBegin, int32 LODEnd, int32 GeometryLODBegin, int32 GeometryLODEnd, uint8 ConversionFlags, UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeletalMesh> Result);

	TSharedPtr<UE::Mutable::Private::FPhysicsBody> MakePhysicsBodyFromAsset(UPhysicsAsset* Asset, const TArray<uint8>& BodySetupRelevancyMap);
	
	
	void ClothVertexBuffers(
			FSkeletalMeshLODRenderData& LODResource, 
			const UE::Mutable::Private::FMesh& MutableMesh,
			TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap = {});

	CUSTOMIZABLEOBJECT_API void PropagateBoneUsageFlagsThroughMeshPose(UE::Mutable::Private::FMesh& Mesh);

	CUSTOMIZABLEOBJECT_API UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FPhysicsBody> CreatePhysicsBodyForMesh(
		const UPhysicsAsset& PhysicsAsset,
		const UE::Mutable::Private::FMesh& Mesh);
}
