// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/System.h"
#include "Tasks/Task.h"

class UModelResources;
class UTexture;
class USkeletalMesh;

/** Implementation of a mutable core provider for image parameters that are application-specific. */
class FUnrealMutableResourceProvider : public UE::Mutable::Private::FExternalResourceProvider
{
public:

	// UE::Mutable::Private::ExternalResourceProvider interface
	// Thread: worker
	CUSTOMIZABLEOBJECT_API virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetImageAsync(UTexture* Texture, uint8 MipmapsToSkip, bool bLoadMipTail, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage>)>& ResultCallback) override;
	CUSTOMIZABLEOBJECT_API virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetReferencedImageAsync( int32 Id, uint8 MipmapsToSkip, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage>)>& ResultCallback) override;
	CUSTOMIZABLEOBJECT_API virtual UE::Mutable::Private::FExtendedImageDesc GetImageDesc(UTexture* Texture) override;
	CUSTOMIZABLEOBJECT_API virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetMeshAsync(
			USkeletalMesh* SkeletalMesh, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 ConversionFlags, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh>)>& ResultCallback) override;
	
	CUSTOMIZABLEOBJECT_API virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetSkeletalMeshAsync(
			USkeletalMesh* SkeletalMesh, int32 LODBegin, int32 LODEnd, int32 GeometryLODBegin, int32 GeometryLODEnd, uint8 ConversionFlags, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeletalMesh>)>& ResultCallback) override;

#if WITH_EDITOR
	void CacheRuntimeReferencedImages(const TArray<TSoftObjectPtr<UTexture2D>>& RuntimeReferencedTextures);
#endif
	
private:
	static inline const UE::Mutable::Private::FImageDesc DUMMY_IMAGE_DESC = 
			UE::Mutable::Private::FImageDesc {UE::Mutable::Private::FImageSize(32, 32), UE::Mutable::Private::EImageFormat::RGBA_UByte, UE::Mutable::Private::EImageFormat::RGBA_UByte, 1};

	/** This will be called if an image Id has been requested by Mutable core but it has not been provided by any provider. */
	static UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> CreateDummy();
	static UE::Mutable::Private::FExtendedImageDesc CreateDummyDesc();

#if WITH_EDITOR
	TArray<TStrongObjectPtr<UTexture>> Images;
#endif

#if WITH_EDITOR
	FCriticalSection RuntimeReferencedLock;
#endif
};
