// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObjectPrivate.h"
#include "MuR/Serialisation.h"

#if WITH_EDITOR
#include "DerivedDataRequestOwner.h"
#endif

class FArchive;
class IAsyncReadFileHandle;
class IAsyncReadRequest;
class IBulkDataIORequest;
class UCustomizableObject;
namespace UE::Mutable::Private {
	class FModel;
	enum class EDataType : uint8;
}
struct FMutableStreamableBlock;

class FUnrealMutableInputStream : public UE::Mutable::Private::FInputStream
{
public:

	FUnrealMutableInputStream(FArchive& ar);

	// UE::Mutable::Private::FInputStream interface
	void Read(void* pData, uint64 size) override;

private:
	FArchive& m_ar;
};

#if WITH_EDITOR
struct FDDCReadRequest
{
	FDDCReadRequest();

	UE::DerivedData::FRequestOwner RequestOwner;
	bool bSuccess = false;
};
#endif

// Implementation of a mutable streamer using bulk storage.
class FUnrealMutableModelBulkReader : public UE::Mutable::Private::FModelReader
{
public:
	// 
	CUSTOMIZABLEOBJECT_API ~FUnrealMutableModelBulkReader();

	// Own interface

	/** Make sure that the provided object can stream data. */
	CUSTOMIZABLEOBJECT_API bool PrepareStreamingForObject(UCustomizableObject* Object);

#if WITH_EDITOR
	/** Cancel any further streaming operations for the given object. This is necessary if the object compiled data is
	 * going to be modified. This can only happen in the editor, when recompiling.
	 * Any additional streaming requests for this object will fail.
	 */
	CUSTOMIZABLEOBJECT_API void CancelStreamingForObject(const UCustomizableObject* CustomizableObject);

	/** Checks if there are any streaming operations for the parameter object.
	* @return true if there are streaming operations in flight
	*/
	CUSTOMIZABLEOBJECT_API bool AreTherePendingStreamingOperationsForObject(const UCustomizableObject* CustomizableObject) const;
#endif

	/** Release all the pending resources. This disables treamings for all objects. */
	CUSTOMIZABLEOBJECT_API void EndStreaming();

	// UE::Mutable::Private::ModelReader interface
	CUSTOMIZABLEOBJECT_API bool DoesBlockExist(const UE::Mutable::Private::FModel*, uint32 BlockKey) override;
	CUSTOMIZABLEOBJECT_API FOperationID BeginReadBlock(const UE::Mutable::Private::FModel*, uint32 BlockKey, void* pBuffer, uint64 size, UE::Mutable::Private::EDataType ResourceType, TFunction<void(bool bSuccess)>* CompletionCallback) override;
	CUSTOMIZABLEOBJECT_API bool IsReadCompleted(FOperationID) override;
	CUSTOMIZABLEOBJECT_API bool EndRead(FOperationID) override;

private:
	struct FReadRequest
	{
		TSharedPtr<IBulkDataIORequest> BulkReadRequest;
		TSharedPtr<IAsyncReadFileHandle> FileHandle;
		TSharedPtr<IAsyncReadRequest> FileReadRequest;
		TSharedPtr<FAsyncFileCallBack> FileCallback;

		uint32 FileId = 0;

#if WITH_EDITORONLY_DATA
		TSharedPtr<FDDCReadRequest> DDCReadRequest;
#endif
	};
	
	/** Streaming data for one object. */
	struct FObjectData
	{
		TWeakPtr<const UE::Mutable::Private::FModel> Model;

		FString BulkFilePrefix;
		TMap<FOperationID, FReadRequest> CurrentReadRequests;

		TWeakPtr<FModelStreamableBulkData> ModelStreamableBulkData;
	};

	TArray<FObjectData> Objects;
	
	mutable FCriticalSection CriticalSection;

	/** This is used to generate unique ids for read requests. */
	FOperationID LastOperationID = 0;
};


#if WITH_EDITOR

class UnrealMutableOutputStream : public UE::Mutable::Private::FOutputStream
{
public:

	UnrealMutableOutputStream(FArchive& ar);

	// UE::Mutable::Private::FOutputStream interface
	void Write(const void* pData, uint64 size) override;

private:
	FArchive& m_ar;
};


// Implementation of a mutable streamer using bulk storage.
class FUnrealMutableModelBulkWriterEditor : public UE::Mutable::Private::FModelWriter
{
public:
	// 
	CUSTOMIZABLEOBJECT_API FUnrealMutableModelBulkWriterEditor(FArchive* InMainDataArchive = nullptr, FArchive* InStreamedDataArchive = nullptr);

	// UE::Mutable::Private::FModelWriter interface
	CUSTOMIZABLEOBJECT_API void OpenWriteFile(uint32 BlockKey, bool bIsStreamable) override;
	CUSTOMIZABLEOBJECT_API void Write(const void* pBuffer, uint64 size) override;
	CUSTOMIZABLEOBJECT_API void CloseWriteFile() override;

protected:

	// Non-owned pointer to an archive where we'll store the main model data (non-streamable)
	FArchive* MainDataArchive = nullptr;

	// Non-owned pointer to an archive where we'll store the resouces (streamable)
	FArchive* StreamedDataArchive = nullptr;

	FArchive* CurrentWriteFile = nullptr;

};


// Implementation of a mutable streamer using bulk storage.
class FUnrealMutableModelBulkWriterCook : public UE::Mutable::Private::FModelWriter
{
public:
	// 
	CUSTOMIZABLEOBJECT_API FUnrealMutableModelBulkWriterCook(FArchive* InMainDataArchive = nullptr, UE::Mutable::Private::FModelStreamableData* InStreamedData = nullptr);

	// UE::Mutable::Private::FModelWriter interface
	CUSTOMIZABLEOBJECT_API void OpenWriteFile(uint32 BlockKey, bool bIsStreamable) override;
	CUSTOMIZABLEOBJECT_API void Write(const void* pBuffer, uint64 size) override;
	CUSTOMIZABLEOBJECT_API void CloseWriteFile() override;

protected:

	// Non-owned pointer to an archive where we'll store the main model data (non-streamable)
	FArchive* MainDataArchive = nullptr;

	// Non-owned pointer to an archive where we'll store the resouces (streamable)
	UE::Mutable::Private::FModelStreamableData* StreamedData = nullptr;

	uint32 CurrentKey = 0;
	bool CurrentIsStreamable = false;
};


#endif
