// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Runtime/Launch/Resources/Version.h"

/** Helpers to ease portability across unreal engine versions */

#if ENGINE_MAJOR_VERSION==5 && ENGINE_MINOR_VERSION<3

#define ObjectPtrWrap(X)	X
#define ObjectPtrDecay(X)	X

#endif


#if ENGINE_MAJOR_VERSION==5 && ENGINE_MINOR_VERSION>=1

#include "Engine/SkinnedAssetCommon.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "Rendering/SkeletalMeshRenderData.h"

#define UE_MUTABLE_GET_CLASS_PATHNAME(X)		X->GetClassPathName()
#define UE_MUTABLE_GET_CLASSPATHS(X)			X.ClassPaths
#define UE_MUTABLE_OBJECTPATH(X)				FSoftObjectPath(X)
#define UE_MUTABLE_TOPLEVELASSETPATH(X,Y)		FTopLevelAssetPath(X, Y)
#define UE_MUTABLE_ASSETCLASS(X)				X.AssetClassPath
#define UE_MUTABLE_GETOBJECTPATH(X)				X.GetObjectPathString()

#define UE_MUTABLE_SETSKINNEDASSET(X,Y)			X->SetSkinnedAsset(Y)
#define UE_MUTABLE_GETSKELETALMESHASSET(X)		X->GetSkeletalMeshAsset()

#elif ENGINE_MAJOR_VERSION==5 && ENGINE_MINOR_VERSION==0

#define USkinnedAsset USkeletalMesh

#define UE_MUTABLE_GET_CLASS_PATHNAME(X)		X->GetFName()
#define UE_MUTABLE_GET_CLASSPATHS(X)			X.ClassNames
#define UE_MUTABLE_OBJECTPATH(X)				*X
#define UE_MUTABLE_TOPLEVELASSETPATH(X,Y)		(Y)
#define UE_MUTABLE_ASSETCLASS(X)				X.AssetClass
#define UE_MUTABLE_GETOBJECTPATH(X)				X.ObjectPath.ToString()

#define UE_MUTABLE_SETSKINNEDASSET(X,Y)			X->SetSkeletalMesh(Y)
#define UE_MUTABLE_GETSKELETALMESHASSET(X)		X->SkeletalMesh

#endif


