// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "HAL/Platform.h"
#include "Engine/Texture.h"
#include "MuR/ComponentId.h"
#include "MuR/ManagedPointer.h"

#define UE_API CUSTOMIZABLEOBJECT_API

// Flags that can influence the mesh conversion
enum class EMutableMeshConversionFlags : uint8
{
	// 
	None = 0,
	// Ignore the skeleton and skinning
	IgnoreSkinning = 1 << 0,

	// Ignore Physics assets
	IgnorePhysics = 1 << 1,

	// Ignore Morphs
	IgnoreMorphs = 1 << 2,

	// Ignore Texture coordinates
	IgnoreTexCoords = 1 << 3,

	// Ignore AssetUserData
	IgnoreAUD = 1 << 4,
	
	// Add morphs as realtime morphs.
	AddMorphsAsRealTime =  1 << 5,
};


#if WITH_EDITOR

// Forward declarations
class UTexture2D;
class UAnimInstance;
class USkeletalMesh;
struct FMutableCompilationContext;

namespace UE::Mutable::Private
{
	class FImage;
	class FMesh;
}

struct FMutableSourceTextureData
{
	FMutableSourceTextureData() = default;
	
	UE_API FMutableSourceTextureData(const UTexture2D& Texture);
	
	UE_API FTextureSource& GetSource();
	
	UE_API bool GetFlipGreenChannel() const;
	
	UE_API bool HasAlphaChannel() const;

	UE_API bool GetCompressionForceAlpha() const;

	UE_API bool IsNormalComposite() const;

private:
	FTextureSource Source;
	bool bFlipGreenChannel = false;
	bool bHasAlphaChannel = false;
	bool bCompressionForceAlpha = false;
	bool bIsNormalComposite = false;
};


ENUM_CLASS_FLAGS(EMutableMeshConversionFlags)


struct FMutableSourceMeshData
{
	/** Assets involved in the conversion. */
	TSoftObjectPtr<UStreamableRenderAsset> Mesh;
	TSoftClassPtr<UAnimInstance> AnimInstance;
	TSoftObjectPtr<UStreamableRenderAsset> TableReferenceSkeletalMesh;

	UE::Mutable::Private::FComponentId ComponentId = INDEX_NONE;
	
	/** Selection of the mesh section*/
	int8 BaseLODIndex = INDEX_NONE;
	int8 BaseSectionIndex = INDEX_NONE;
	int8 LODOffset = INDEX_NONE;

	/** Required mesh properties. */
	EMutableMeshConversionFlags Flags = EMutableMeshConversionFlags::None;

	/** Required realtime mesh morphs. */
	bool bUseAllRealTimeMorphs = false;
	TArray<FString> UsedRealTimeMorphTargetNames;

	/** */
	bool bOnlyConnectedLOD = false;

	/** Context for log messages. */
	const UObject* MessageContext = nullptr;

	FString OptionalMessageInfo;
	
	FName AnimBPSlotName;

	FGameplayTagContainer GameplayTags;
	
	bool operator==(const FMutableSourceMeshData&) const = default;
};


inline uint32 GetTypeHash(const FMutableSourceMeshData& Key)
{
	uint32 GuidHash = GetTypeHash(Key.Mesh);
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.BaseLODIndex));
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.BaseSectionIndex));
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.LODOffset));
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.bOnlyConnectedLOD));
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.ComponentId));
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.Flags));
	GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.AnimBPSlotName));
	//GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.GameplayTags)); // Hash func not implemented, GetGameplayTagArray order is not deterministic.
	return GuidHash;
}


enum class EUnrealToMutableConversionError 
{
    Success,
    UnsupportedFormat,
    CompositeImageDimensionMismatch,
    CompositeUnsupportedFormat,
    Unknown
};

UE_API EUnrealToMutableConversionError ConvertTextureUnrealSourceToMutable(UE::Mutable::Private::FImage* OutResult, FMutableSourceTextureData&, uint8 MipmapsToSkip, bool bLoadMipTail);

/** Mesh conversion context. */
class FMeshConversionContext
{
public:
	FMeshConversionContext() = default;

public:

	// Source Data
	FMutableSourceMeshData Source;
	FString MorphName;

	TSharedPtr<FMutableCompilationContext> CompilationContext;

	// Keep strong references to meshes and other UObjects involved in the conversion
	TArray<TStrongObjectPtr<UObject>> ReferencedObjects;

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> Result;
};

#endif

#undef UE_API
