// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObject.h"

#include "EdGraph/EdGraph.h"
#include "Engine/Engine.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimInstance.h"
#include "Animation/Skeleton.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Input/Reply.h"
#include "Interfaces/ITargetPlatform.h"
#include "Materials/MaterialInterface.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/CustomizableObjectUIData.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCO/MutableProjectorTypeUtils.h"
#include "MuCO/UnrealMutableModelDiskStreamer.h"
#include "MuR/Model.h"
#include "MuR/Operations.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "Misc/MessageDialog.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Engine/TextureLODSettings.h"

#include "DerivedDataCache.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataRequestOwner.h"
#include "MuCO/LoadUtils.h"
#endif

#include "ExternalOperationProvider.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/CustomizableObjectSkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObject)

#define LOCTEXT_NAMESPACE "CustomizableObject"

DEFINE_LOG_CATEGORY(LogMutable);

#if WITH_EDITOR

TAutoConsoleVariable<bool> CVarMutableAsyncCook(
	TEXT("Mutable.CookAsync"),
	true,
	TEXT("True - Customizable Objects will be compiled asynchronously during cook.\n")
	TEXT("False - Sync compilation.\n"));

#endif

static const FString s_EmptyString;

//-------------------------------------------------------------------------------------------------

UCustomizableObject::UCustomizableObject()
	: UObject()
{
	Private = CreateDefaultSubobject<UCustomizableObjectPrivate>(FName("Private"));

#if WITH_EDITORONLY_DATA
	const FString CVarName = TEXT("r.SkeletalMesh.MinLodQualityLevel");
	const FString ScalabilitySectionName = TEXT("ViewDistanceQuality");
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LODSettings.MinQualityLevelLOD.SetQualityLevelCVarForCooking(*CVarName, *ScalabilitySectionName);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}


#if WITH_EDITOR
bool UCustomizableObject::IsEditorOnly() const
{
	return bIsChildObject;
}


void UCustomizableObjectPrivate::UpdateVersionId()
{
	GetPublic()->VersionId = FGuid::NewGuid();
}


FGuid UCustomizableObjectPrivate::GetVersionId() const
{
	return GetPublic()->VersionId;
}


void UCustomizableObject::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	int32 bIsRoot = 0;

	if (const ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		bIsRoot = Module->IsRootObject(*this) ? 1 : 0;
	}
	
	Context.AddTag(FAssetRegistryTag("IsRoot", FString::FromInt(bIsRoot), FAssetRegistryTag::TT_Numerical));
	Super::GetAssetRegistryTags(Context);
}


void UCustomizableObject::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Update the derived child object flag
	if (GetPrivate()->TryUpdateIsChildObject())
	{
		if (bIsChildObject)
		{
			GetPackage()->SetPackageFlags(PKG_EditorOnly);
		}
		else
		{
			GetPackage()->ClearPackageFlags(PKG_EditorOnly);
		}
	}

	if (ObjectSaveContext.IsCooking() && !bIsChildObject)
	{
		const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();

		// Load cached data before saving
		if (GetPrivate()->TryLoadCompiledCookDataForPlatform(TargetPlatform))
		{
			// Change current platform ModelResources' outer from TransientPkg to CO before save, and
			// give it the deterministic name so the cooked export name is stable across runs/platforms.
			// PostSaveRoot will set the outer to TransientPkg again. This is done to avoid serializing multiple ModelResources when cooking more than one platform at once.
			TObjectPtr<UModelResources> ModelResources = GetPrivate()->GetModelResources(true);

			const FString ModelResourcesName = GetModelResourcesNameForPlatform(*this, *TargetPlatform);

			// A stale object may already occupy the deterministic name under this CO (e.g. a previous
			// save whose object hasn't been GC'd yet). Move it aside so the Rename below doesn't
			// force-destroy a still-referenced object.
			if (UObject* Existing = StaticFindObjectFast(UModelResources::StaticClass(), this, FName(*ModelResourcesName)))
			{
				if (Existing != ModelResources)
				{
					Existing->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
				}
			}

			ModelResources->Rename(*ModelResourcesName, this, REN_DontCreateRedirectors);

			UE::Mutable::Private::FMutableCachedPlatformData& CachedPlatformData = *GetPrivate()->CachedPlatformsData.Find(TargetPlatform->PlatformName());
			TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData = GetPrivate()->GetModelStreamableBulkData(true);

			const int32 NumBulkDataFiles = CachedPlatformData.BulkDataFiles.Num();

			ModelStreamableBulkData->StreamableBulkData.SetNum(NumBulkDataFiles);

			const auto WriteBulkData = [ModelStreamableBulkData](UE::Mutable::Private::FFile& File, TArray64<uint8>& FileBulkData, uint32 FileIndex)
				{
					MUTABLE_CPUPROFILER_SCOPE(WriteBulkData);

					FByteBulkData& ByteBulkData = ModelStreamableBulkData->StreamableBulkData[FileIndex];

					// BulkData file to store the file to. CookedIndex 0 is used as a default for backwards compatibility, +1 to skip it.
					ByteBulkData.SetCookedIndex(FBulkDataCookedIndex((File.Id % MAX_uint8) + 1));

					ByteBulkData.Lock(LOCK_READ_WRITE);
					uint8* Ptr = (uint8*)ByteBulkData.Realloc(FileBulkData.Num());
					FMemory::Memcpy(Ptr, FileBulkData.GetData(), FileBulkData.Num());
					ByteBulkData.Unlock();

					uint32 BulkDataFlags = BULKDATA_PayloadInSeparateFile | BULKDATA_Force_NOT_InlinePayload;
					if (File.Flags == uint16(EMutableFileFlags::HighRes))
					{
						BulkDataFlags |= BULKDATA_OptionalPayload;
					}
					ByteBulkData.SetBulkDataFlags(BulkDataFlags);
				};

			bool bDropData = true;
			UE::Mutable::Private::SerializeBulkDataFiles(CachedPlatformData, CachedPlatformData.BulkDataFiles, WriteBulkData, bDropData);
		}
		else
		{
			UE_LOGF(LogMutable, Warning, "Cook: Customizable Object [%ls] is missing [%ls] platform data.", *GetName(),
				*ObjectSaveContext.GetTargetPlatform()->PlatformName());
			
			// Clear model resources
			GetPrivate()->SetModelResources(nullptr, true /* bIsCooking */);
			GetPrivate()->GetModelStreamableBulkData(true).Reset();
			GetPrivate()->SetModel(nullptr, FGuid(), true);
		}
	}
}


void UCustomizableObject::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);

	if (ObjectSaveContext.IsCooking() && !bIsChildObject)
	{
		const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();

		const UE::Mutable::Private::FMutableCachedPlatformData* PlatformData = GetPrivate()->CachedPlatformsData.Find(TargetPlatform->PlatformName());
		if (PlatformData && PlatformData->ModelResources)
		{
			// Set the outer to TransientPkg again. This is done to avoid serializing multiple ModelResources when cooking more than one platform at once.
			PlatformData->ModelResources->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
		}
	}
}


bool UCustomizableObjectPrivate::TryUpdateIsChildObject()
{
	if (const ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		GetPublic()->bIsChildObject = !Module->IsRootObject(*GetPublic());
		return true;
	}
	else
	{
		return false;
	}
}


bool UCustomizableObject::IsChildObject() const
{
	return bIsChildObject;
}


void UCustomizableObjectPrivate::SetIsChildObject(bool bIsChildObject)
{
	GetPublic()->bIsChildObject = bIsChildObject;
}


bool UCustomizableObjectPrivate::TryLoadCompiledCookDataForPlatform(const ITargetPlatform* TargetPlatform)
{
	const UE::Mutable::Private::FMutableCachedPlatformData* PlatformData = CachedPlatformsData.Find(TargetPlatform->PlatformName());
	if (!PlatformData)
	{
		return false;
	}

	SetModelResources(PlatformData->ModelResources.Get(), true);
	SetModelStreamableBulkData(PlatformData->ModelStreamableBulkData, true);
	SetModel(PlatformData->Model, GenerateIdentifier(*GetPublic()), true);

	return GetModel() != nullptr;
}

#endif // End WITH_EDITOR


bool bShowOldCustomizableObjectWarning = true;


void UCustomizableObject::PostLoad()
{
	Super::PostLoad();

	const int32 CustomVersion = GetLinkerCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (CustomVersion < FCustomizableObjectCustomVersion::UseUVRects) // A bit older than 5.5
	{
		if (bShowOldCustomizableObjectWarning)
		{
			bShowOldCustomizableObjectWarning = false;
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("OldCustomizableObject", "Old Customizable Objects found. Please resave them to avoid future incompatibilities. See output log for more information."));
		}

		UE_LOGF(LogMutable, Warning, "Unsupported old Customizable Object. Please resave it: %ls", *GetPackage()->GetPathName());
	}
	
#if WITH_EDITOR

	if (CustomVersion < FCustomizableObjectCustomVersion::UseLegacyLayouts)
	{
		bUseLegacyLayouts = true;
	}

	if (Source)
	{
		Source->ConditionalPostLoad();
	}
	
	for (int32 Version = CustomVersion + 1; Version <= FCustomizableObjectCustomVersion::LatestVersion; ++Version)
	{
		GetPrivate()->BackwardsCompatibleFixup(Version);
		
		if (Source)
		{
			if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
			{
				// Execute backwards compatible code for all nodes. It requires all nodes to be loaded.
			
				Module->BackwardsCompatibleFixup(*Source, Version);
			}
		}
	}

	if (Source)
	{
		if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
		{
			Module->PostBackwardsCompatibleFixup(*Source);
		}
	}
	
	// Register to dirty delegate so we update derived data version ID each time that the package is marked as dirty.
	if (UPackage* Package = GetOutermost())
	{
		Package->PackageMarkedDirtyEvent.AddWeakLambda(this, [this](UPackage* Pkg, bool bWasDirty)
			{
				if (GetPackage() == Pkg)
				{
					GetPrivate()->UpdateVersionId();
				}
			});
	}
	
	GetPrivate()->Status.NextState(FCustomizableObjectStatusTypes::EState::Loading);

	if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		Module->AddPendingLoad(this);
	}
#endif
}


void UCustomizableObjectPrivate::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
#if WITH_EDITOR
	if (GetPublic()->ReferenceSkeletalMesh_DEPRECATED)
	{
		GetPublic()->ReferenceSkeletalMeshes_DEPRECATED.Add(GetPublic()->ReferenceSkeletalMesh_DEPRECATED);
		GetPublic()->ReferenceSkeletalMesh_DEPRECATED = nullptr;
	}

#if WITH_EDITORONLY_DATA
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::CompilationOptions)
	{
		OptimizationLevel = GetPublic()->CompileOptions_DEPRECATED.OptimizationLevel;
		TextureCompression = GetPublic()->CompileOptions_DEPRECATED.TextureCompression;
		EmbeddedDataBytesLimit = GetPublic()->CompileOptions_DEPRECATED.EmbeddedDataBytesLimit;
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NewComponentOptions)
	{
		if (MutableMeshComponents_DEPRECATED.IsEmpty())
		{
			for (int32 SkeletalMeshIndex = 0; SkeletalMeshIndex < GetPublic()->ReferenceSkeletalMeshes_DEPRECATED.Num(); ++SkeletalMeshIndex)
			{
				FMutableMeshComponentData NewComponent;
				NewComponent.Name = FName(FString::FromInt(SkeletalMeshIndex));
				NewComponent.ReferenceSkeletalMesh = GetPublic()->ReferenceSkeletalMeshes_DEPRECATED[SkeletalMeshIndex];

				MutableMeshComponents_DEPRECATED.Add(NewComponent);
			}

			GetPublic()->ReferenceSkeletalMeshes_DEPRECATED.Empty();
		}
	}
#endif
#endif
}

bool UCustomizableObjectPrivate::IsLocked() const
{
	return bLocked;
}


void UCustomizableObject::Serialize(FArchive& Ar_Asset)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObject::Serialize)
	
	Super::Serialize(Ar_Asset);

	Ar_Asset.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);
	
#if WITH_EDITOR
	if (Ar_Asset.IsCooking())
	{
		if (Ar_Asset.IsSaving())
		{
			UE_LOGF(LogMutable, Verbose, "Serializing cooked data for Customizable Object [%ls].", *GetName());
			GetPrivate()->SaveEmbeddedData(Ar_Asset);
		}
	}
	else
	{
		// Can't remove this or saved customizable objects will fail to load
		int64 InternalVersion = 0;
		Ar_Asset << InternalVersion;
	}
#else
	if (Ar_Asset.IsLoading())
	{
		GetPrivate()->LoadEmbeddedData(Ar_Asset);
	}
#endif
}


#if WITH_EDITOR
void UCustomizableObject::PostRename(UObject * OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	if (Source)
	{
		Source->PostRename(OldOuter, OldName);
	}
}


void UCustomizableObject::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	ICustomizableObjectEditorModule& Module = ICustomizableObjectEditorModule::GetChecked();
	Module.BeginCacheForCookedPlatformData(*this, TargetPlatform);
}


bool UCustomizableObject::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) 
{
	ICustomizableObjectEditorModule& Module = ICustomizableObjectEditorModule::GetChecked();
	return Module.IsCachedCookedPlatformDataLoaded(*this, TargetPlatform);
}


FGuid GenerateIdentifier(const UCustomizableObject& CustomizableObject)
{
	// Generate the Identifier using the path and name of the asset
	uint32 FullPathHash = GetTypeHash(CustomizableObject.GetFullName());
	uint32 OutermostHash = GetTypeHash(GetNameSafe(CustomizableObject.GetOutermost()));
	uint32 OuterHash = GetTypeHash(CustomizableObject.GetName());
	return FGuid(0, FullPathHash, OutermostHash, OuterHash);
}


FString GetModelResourcesNameForPlatform(const UCustomizableObject& CustomizableObject, const ITargetPlatform& InTargetPlatform)
{
	return GenerateIdentifier(CustomizableObject).ToString() + InTargetPlatform.PlatformName();
}

#if WITH_EDITOR
FGuid GenerateDataDistributionIdentifier(const UCustomizableObject& CustomizableObject)
{
	if (CustomizableObject.GetPrivate()->CookedDataDistributionId.IsValid())
	{
		return CustomizableObject.GetPrivate()->CookedDataDistributionId;
	}

	return GenerateIdentifier(CustomizableObject);
}
#endif

bool UCustomizableObjectPrivate::LoadModelResources(FArchive& MemoryReader, const ITargetPlatform* InTargetPlatform, bool bIsCooking)
{
	TObjectPtr<UModelResources> LocalModelResources = LoadModelResources_Internal(MemoryReader, GetPublic(), InTargetPlatform, bIsCooking);
	SetModelResources(LocalModelResources, bIsCooking);

	return LocalModelResources != nullptr;
}


void UCustomizableObjectPrivate::LoadModelStreamableBulk(FArchive& MemoryReader, bool bIsCooking)
{
	SetModelStreamableBulkData(LoadModelStreamableBulk_Internal(MemoryReader), bIsCooking);
}


void UCustomizableObjectPrivate::LoadModel(FArchive& MemoryReader, bool bIsCooking)
{
	SetModel(LoadModel_Internal(MemoryReader), GenerateIdentifier(*GetPublic()), bIsCooking);
}


TObjectPtr<UModelResources> LoadModelResources_Internal(FArchive& MemoryReader, const UCustomizableObject* Outer, const ITargetPlatform* InTargetPlatform, bool bIsCooking)
{
	// Make sure mutable has been initialised.
	UCustomizableObjectSystem::GetInstance();

	TObjectPtr<UModelResources> LocalModelResources = NewObject<UModelResources>((UObject*)GetTransientPackage(), NAME_None, RF_Public);

	FObjectAndNameAsStringProxyArchive ObjectReader(MemoryReader, true);
	LocalModelResources->Serialize(ObjectReader);

	const bool bLoadedSuccessfully = LocalModelResources->CodeVersion == GetECustomizableObjectVersionEnumHash();
	return bLoadedSuccessfully ? LocalModelResources : nullptr;
}


const TSharedPtr<FModelStreamableBulkData> LoadModelStreamableBulk_Internal(FArchive& MemoryReader)
{
	TSharedPtr<FModelStreamableBulkData> LocalModelStreamablesPtr = MakeShared<FModelStreamableBulkData>();
	FModelStreamableBulkData& LocalModelStreamables = *LocalModelStreamablesPtr.Get();
	MemoryReader << LocalModelStreamables;

	return LocalModelStreamablesPtr;
}


const TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> LoadModel_Internal(FArchive& MemoryReader)
{
	FUnrealMutableInputStream Stream(MemoryReader);
	UE::Mutable::Private::FInputArchive Arch(&Stream);
	return UE::Mutable::Private::FModel::StaticUnserialise(Arch);
}


void UCustomizableObjectPrivate::LoadCompiledDataFromDisk()
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObjectPrivate::LoadCompiledDataFromDisk)
	
	// Skip data loading from disk
	if (IsRunningCookCommandlet())
	{
		Status.NextState(FCustomizableObjectStatusTypes::EState::NoModel);
		return;
	}
	
	ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
	const ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
	check(RunningPlatform);

	IFileManager& FileManager = IFileManager::Get();
	TArray<TUniquePtr<IFileHandle>> FileHandles;

	// Compose Folder Name
	const FString FullFileName = GetCompiledDataFileName(RunningPlatform);

	FGuid CompiledVersionId;

	bool bHasCompiledData = true;
	for (int32 DataType = 0; DataType < static_cast<int32>(UE::Mutable::Private::EStreamableDataType::DataTypeCount); ++DataType)
	{
		const FString FilePath = FullFileName + GetDataTypeExtension(static_cast<UE::Mutable::Private::EStreamableDataType>(DataType));
		if (IFileHandle* FileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenRead(*FilePath))
		{
			FileHandles.Emplace(FileHandle);
			
			TArray<uint8> HeaderBytes;

			constexpr int32 HeaderSize = sizeof(MutableCompiledDataStreamHeader);
			HeaderBytes.SetNum(HeaderSize);
			FileHandle->Read(HeaderBytes.GetData(), HeaderSize);
			
			FMemoryReader AuxMemoryReader(HeaderBytes);
			MutableCompiledDataStreamHeader DataTypeHeader;
			AuxMemoryReader << DataTypeHeader;

			if(DataTypeHeader.InternalVersion != GetECustomizableObjectVersionEnumHash()
				|| (CompiledVersionId.IsValid() && CompiledVersionId != DataTypeHeader.VersionId))
			{
				bHasCompiledData = false;
				break;
			}

			CompiledVersionId = DataTypeHeader.VersionId;
		}
		else
		{
			bHasCompiledData = false;
			break;
		}
	}

	if (bHasCompiledData)
	{
		TArray64<uint8> CompiledDataBytes;

		const int64 CompiledDataSize = FileHandles[0]->Size() - FileHandles[0]->Tell();
		CompiledDataBytes.SetNumUninitialized(CompiledDataSize);
		FileHandles[0]->Read(CompiledDataBytes.GetData(), CompiledDataSize);

		FMemoryReaderView MemoryReader(CompiledDataBytes);

		if (LoadModelResources(MemoryReader, RunningPlatform))
		{
			TArray<FName> OutOfDatePackages;
			TArray<FName> AddedPackages;
			TArray<FName> RemovedPackages;
			bool bReleaseVersion;
			const bool bOutOfDate = IsCompilationOutOfDate(false, OutOfDatePackages, AddedPackages, RemovedPackages, bReleaseVersion);
			if (!bOutOfDate)
			{
				LoadModelStreamableBulk(MemoryReader, /* bIsCooking */false);
				LoadModel(MemoryReader, false);
			}
			else
			{
				if (OutOfDatePackages.Num())
				{
					UE_LOGF(LogMutable, Display, "Invalidating compiled data due to changes in %ls.", *OutOfDatePackages[0].ToString());
				}

				PrintParticipatingPackagesDiff(OutOfDatePackages, AddedPackages, RemovedPackages, bReleaseVersion);
			}
		}
	}

	if (!GetModel()) // Failed to load the model
	{
		Status.NextState(FCustomizableObjectStatusTypes::EState::NoModel);
	}
}


bool UCustomizableObject::ConditionalAutoCompile()
{
	check(IsInGameThread());

	// Don't compile objects being compiled
	if (GetPrivate()->IsLocked())
	{
		return false;
	}

	// Don't compile compiled objects
	if (IsCompiled())
	{
		return true;
	}

	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
	if (!System || !System->IsValidLowLevel() || System->HasAnyFlags(RF_BeginDestroyed))
	{
		return false;
	}

	// Don't re-compile objects if they failed to compile. 
	if (GetPrivate()->CompilationResult == ECompilationResultPrivate::Errors)
	{
		return false;
	}

	// By default, don't compile in a commandlet.
	// Notice that the cook is also a commandlet. Do not add a warning/error, otherwise we could end up invalidating the cook for no reason.
	if (IsRunningCookCommandlet() || (IsRunningCommandlet() && !System->IsAutoCompileCommandletEnabled()))
	{
		return false;
	}

	// Don't compile if Mutable or AutoCompile is disabled.
	if (!System->IsActive() || !System->IsAutoCompileEnabled())
	{
		return false;
	}

	if (ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
	{
		if (System->IsAutoCompilationSync() && GetPrivate()->Status.Get() == FCustomizableObjectStatusTypes::EState::Loading)
		{
			return false;
		}
		else
		{
			EditorModule->CompileCustomizableObject(*this, nullptr, true, false);
		}
	}

	return IsCompiled();
}


uint32 GetECustomizableObjectVersionEnumHash()
{
	static const uint32 VersionsHash = []() -> uint32
		{
			UEnum* Enum = StaticEnum<ECustomizableObjectVersions>();
			check(Enum);

			const int32 StartIndex = Enum->GetIndexByValue(static_cast<int64>(ECustomizableObjectVersions::FirstEnumeratedVersion));
			const int32 EndIndex = Enum->GetIndexByValue(static_cast<int64>(ECustomizableObjectVersions::LastCustomizableObjectVersion));
			check(StartIndex < EndIndex);

			uint32 CombinedHash = 0;
			for (int32 Index = StartIndex; Index <= EndIndex; Index++)
			{
				const FString VersionString = Enum->GetNameStringByIndex(Index);
				CombinedHash = HashCombine(GetTypeHash(VersionString), CombinedHash);
			}

			return CombinedHash;
		}();

	return VersionsHash;
}


FString GetCompiledDataFolderPath()
{	
	return FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() + TEXT("MutableStreamedDataEditor/"));
}


FString UCustomizableObjectPrivate::GetCompiledDataFileName(const ITargetPlatform* InTargetPlatform, bool bIsDiskStreamer) const
{
	const FString FilePath = GetCompiledDataFolderPath();
	const FString PlatformName = InTargetPlatform ? InTargetPlatform->PlatformName() : FPlatformProperties::PlatformName();
	const FString FileIdentifier = bIsDiskStreamer ? Identifier.ToString() : GenerateIdentifier(*GetPublic()).ToString();

	return FilePath + PlatformName + FileIdentifier;
}


FString GetDataTypeExtension(UE::Mutable::Private::EStreamableDataType DataType)
{
	switch (DataType)
	{
	case UE::Mutable::Private::EStreamableDataType::None:	return TEXT(".mut");
	case UE::Mutable::Private::EStreamableDataType::Model: return TEXT("_M.mut");
	default:
	{
		unimplemented();
		return s_EmptyString;
	}
	}
}


FString UCustomizableObject::GetDesc()
{
	int32 States = GetStateCount();
	int32 Params = GetParameterCount();
	return FString::Printf(TEXT("%d States, %d Parameters"), States, Params);
}


void UCustomizableObjectPrivate::SaveEmbeddedData(FArchive& Ar)
{
	UE_LOGF(LogMutable, Verbose, "Saving embedded data for Customizable Object [%ls] now at position %d.", *GetName(), int(Ar.Tell()));

	TSharedPtr<UE::Mutable::Private::FModel> Model = GetModel();

	int64 InternalVersion = Model ? (int64)GetECustomizableObjectVersionEnumHash() : INDEX_NONE;
	Ar << InternalVersion;

	if (Model)
	{	
		if (Ar.IsCooking())
		{
			Model->GetProgram().RomsCompileData.Empty();
		}
		UnrealMutableOutputStream Stream(Ar);
		UE::Mutable::Private::FOutputArchive Arch(&Stream);
		UE::Mutable::Private::FModel::Serialise(Model.Get(), Arch);

		UE_LOGF(LogMutable, Verbose, "Saved embedded data for Customizable Object [%ls] now at position %d.", *GetName(), int(Ar.Tell()));
	}
}

void UCustomizableObjectPrivate::UpdateDataDistributionId()
{
	CookedDataDistributionId = FGuid::NewGuid();
}

#endif // End WITH_EDITOR 

void UCustomizableObjectPrivate::LoadEmbeddedData(FArchive& Ar)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObject::LoadEmbeddedData)

	int64 InternalVersion;
	Ar << InternalVersion;

	// If this fails, something went wrong with the packaging: we have data that belongs
	// to a different version than the code.
	if (ensure(InternalVersion != INDEX_NONE))
	{		
		// Load model
		FUnrealMutableInputStream Stream(Ar);
		UE::Mutable::Private::FInputArchive Arch(&Stream);
		SerializedModel = UE::Mutable::Private::FModel::StaticUnserialise(Arch);
	}
}


const UCustomizableObjectPrivate* UCustomizableObject::GetPrivate() const
{
	check(Private);
	return Private;
}


UCustomizableObjectPrivate* UCustomizableObject::GetPrivate()
{
	check(Private);
	return Private;
}


bool UCustomizableObject::IsCompiled() const
{
#if WITH_EDITOR
	const bool bIsCompiled = GetPrivate()->GetModel() != nullptr && GetPrivate()->GetModel()->IsValid();
#else
	const bool bIsCompiled = GetPrivate()->GetModel() != nullptr;
#endif

	return bIsCompiled;
}


bool UCustomizableObject::IsLoading() const
{
	return GetPrivate()->Status.Get() == FCustomizableObjectStatusTypes::EState::Loading;
}


void UCustomizableObjectPrivate::AddUncompiledCOWarning(const FString& AdditionalLoggingInfo)
{
	// Send a warning (on-screen notification, log error, and in-editor notification)
	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
	if (!System || !System->IsValidLowLevel() || System->HasAnyFlags(RF_BeginDestroyed))
	{
		return;
	}

	System->AddUncompiledCOWarning(*GetPublic(), &AdditionalLoggingInfo);
}


USkeletalMesh* UCustomizableObject::GetComponentMeshReferenceSkeletalMesh(const FName& ComponentName) const
{
	// Try to get the data from the CO graph.
#if WITH_EDITORONLY_DATA
	if (!IsCompiled() || GetPrivate()->IsLocked())
	{
		if (const ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
		{
			return Module->GetReferenceSkeletalMesh(*this, ComponentName);
		}
	}
#endif

	if (const UModelResources* ModelResources = Private->GetModelResources())
	{
		// TODO Due to the adding support of multiple components with the same name this is no longer correct. A single name may point to multiple FComponentId.
		UE::Mutable::Private::FComponentId ComponentId = ModelResources->ComponentNamesPerObjectComponent.IndexOfByKey(ComponentName);
		if (ModelResources->ReferenceSkeletalMeshesData.IsValidIndex(ComponentId))
		{
			// Can be nullptr if RefSkeletalMeshes are not loaded yet or if the component has not reference skeletal mesh.
			return ModelResources->ReferenceSkeletalMeshesData[ComponentId].SkeletalMesh;
		}
	}
	
	return nullptr;
}


USkeletalMesh* UCustomizableObject::GetSkeletalMeshComponentReferenceSkeletalMesh(const FName& ComponentName) const
{
	return GetComponentMeshReferenceSkeletalMesh(ComponentName);
}


int32 UCustomizableObject::GetStateCount() const
{
	int32 Result = 0;

	if (Private->GetModel())
	{
		Result = Private->GetModel()->GetStateCount();
	}

	return Result;
}


FString UCustomizableObject::GetStateName(int32 StateIndex) const
{
	return GetPrivate()->GetStateName(StateIndex);
}


FString UCustomizableObjectPrivate::GetStateName(int32 StateIndex) const
{
	FString Result;

	if (GetModel())
	{
		Result = GetModel()->GetStateName(StateIndex);
	}

	return Result;
}


int32 UCustomizableObject::GetStateParameterCount(const FString& StateName) const
{
	int32 StateIndex = GetPrivate()->FindState(StateName);

	int32 Result = 0;

	if (Private->GetModel())
	{
		Result = Private->GetModel()->GetStateParameterCount(StateIndex);
	}

	return Result;
}


FString UCustomizableObject::GetStateParameterName(const FString& StateName, int32 ParameterIndex) const
{
	int32 StateIndex = GetPrivate()->FindState(StateName);

	return GetParameterName(GetPrivate()->GetStateParameterIndex(StateIndex, ParameterIndex));
}


#if WITH_EDITORONLY_DATA
void UCustomizableObjectPrivate::PostCompile()
{
	ImageIdRegistry = MakeShared<UE::Mutable::Private::FImageIdRegistry>();
	
	SkeletalMeshCache.Clear();
	SkeletonCache.Clear();
	TextureCache.Clear();

	for (TObjectIterator<UCustomizableObjectInstance> It; It; ++It)
	{
		if (It->GetCustomizableObject() == this->GetPublic())
		{
			// This cannot be bound to the PostCompileDelegate below because the CO Editor binds to it too and the order of broadcast is indeterminate.
			// The Instance's OnPostCompile() must happen before all the other bindings.
			It->GetPrivate()->OnPostCompile();
		}
	}

	PostCompileDelegate.Broadcast();
}
#endif


UCustomizableObject* UCustomizableObjectPrivate::GetPublic() const
{
	UCustomizableObject* Public = StaticCast<UCustomizableObject*>(GetOuter());
	check(Public);

	return Public;
}

#if WITH_EDITORONLY_DATA
FPostCompileDelegate& UCustomizableObject::GetPostCompileDelegate()
{
	return GetPrivate()->PostCompileDelegate;
}
#endif


UCustomizableObjectInstance* UCustomizableObject::CreateInstance()
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObject::CreateInstance)

	UCustomizableObjectInstance* PreviewInstance = NewObject<UCustomizableObjectInstance>(GetTransientPackage(), NAME_None, RF_Transient);
	PreviewInstance->SetObject(this);
	PreviewInstance->GetPrivate()->bShowOnlyRuntimeParameters = false;

	UE_LOGF(LogMutable, Verbose, "Created Customizable Object Instance.");

	return PreviewInstance;
}


int32 UCustomizableObject::GetComponentCount() const
{
	if (const UModelResources* ModelResources = GetPrivate()->GetModelResources())
	{
		return ModelResources->ComponentNamesPerObjectComponent.Num();
	}

	return 0;
}


FName UCustomizableObjectPrivate::GetComponentName(UE::Mutable::Private::FComponentId ComponentId) const
{
	if (const UModelResources* LocalModelResources = GetModelResources())
	{
		const TArray<FName>& ComponentNames = LocalModelResources->ComponentNamesPerObjectComponent;
		if (ComponentNames.IsValidIndex(ComponentId))
		{
			return ComponentNames[ComponentId];
		}
	}

	return NAME_None;
}


#if WITH_EDITORONLY_DATA
EMutableCompileMeshType UCustomizableObjectPrivate::GetMeshCompileType() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->MeshCompileType;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


const TArray<TSoftObjectPtr<UCustomizableObject>>& UCustomizableObjectPrivate::GetWorkingSet() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->WorkingSet;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool UCustomizableObjectPrivate::IsTableMaterialsParentCheckDisabled() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->bDisableTableMaterialsParentCheck;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool UCustomizableObjectPrivate::IsRealTimeMorphTargetsEnabled() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->bEnableRealTimeMorphTargets;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool UCustomizableObjectPrivate::Is16BitBoneWeightsEnabled() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->bEnable16BitBoneWeights;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool UCustomizableObjectPrivate::IsAltSkinWeightProfilesEnabled() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->bEnableAltSkinWeightProfiles;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool UCustomizableObjectPrivate::IsPhysicsAssetMergeEnabled() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->bEnablePhysicsAssetMerge;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


bool UCustomizableObjectPrivate::ShouldUseLegacyLayouts() const
{
	return GetPublic()->bUseLegacyLayouts;
}


bool UCustomizableObjectPrivate::IsEnabledAnimBpPhysicsAssetsManipulation() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO Remove >5.6
	return GetPublic()->bEnableAnimBpPhysicsAssetsManipulation;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif


const FString& UCustomizableObjectPrivate::GetIntParameterAvailableOption(int32 ParamIndex, int32 K) const
{
	if (ParamIndex >= 0 && ParamIndex < ParameterProperties.Num())
	{
		if (K >= 0 && K < GetEnumParameterNumValues(ParamIndex))
		{
			return ParameterProperties[ParamIndex].PossibleValues[K].Name;
		}
		else
		{
			UE_LOGF(LogMutable, Warning, "Index [%d] out of IntParameterNumOptions bounds at GetIntParameterAvailableOption at CO %ls.", K, *GetName());
		}
	}
	else
	{
		UE_LOGF(LogMutable, Warning, "Index [%d] out of ParameterProperties bounds at GetIntParameterAvailableOption at CO %ls.", ParamIndex, *GetName());
	}

	return s_EmptyString;
}


int32 UCustomizableObjectPrivate::GetEnumParameterNumValues(int32 ParamIndex) const
{
	if (ParamIndex >= 0 && ParamIndex < ParameterProperties.Num())
	{
		return ParameterProperties[ParamIndex].PossibleValues.Num();
	}
	else
	{
		UE_LOGF(LogMutable, Warning, "Index [%d] out of ParameterProperties bounds at GetIntParameterNumOptions at CO %ls.", ParamIndex, *GetName());
	}

	return 0;
}


FString UCustomizableObjectPrivate::FindIntParameterValueName(int32 ParamIndex, int32 ParamValue) const
{
	if (ParamIndex >= 0 && ParamIndex < ParameterProperties.Num())
	{
		const TArray<FMutableModelParameterValue> & PossibleValues = ParameterProperties[ParamIndex].PossibleValues;

		const int32 MinValueIndex = !PossibleValues.IsEmpty() ? PossibleValues[0].Value : 0;
		ParamValue = ParamValue - MinValueIndex;

		if (PossibleValues.IsValidIndex(ParamValue))
		{
			return PossibleValues[ParamValue].Name;
		}
	}
	else
	{
		UE_LOGF(LogMutable, Warning, "Index [%d] out of ParameterProperties bounds at FindIntParameterValueName at CO %ls.", ParamIndex, *GetName());
	}

	return FString();
}


int32 UCustomizableObjectPrivate::FindState(const FString& Name) const
{
	int32 Result = -1;
	if (GetModel())
	{
		Result = GetModel()->FindState(Name);
	}

	return Result;
}


int32 UCustomizableObjectPrivate::GetStateParameterIndex(int32 StateIndex, int32 ParameterIndex) const
{
	int32 Result = 0;

	if (GetModel())
	{
		Result = GetModel()->GetStateParameterIndex(StateIndex, ParameterIndex);
	}

	return Result;
}


FName UCustomizableObject::GetComponentName(int32 ComponentIndex) const
{
	return GetPrivate()->GetComponentName(ComponentIndex);
}


int32 UCustomizableObject::GetParameterCount() const
{
	return GetPrivate()->ParameterProperties.Num();
}


EMutableParameterType UCustomizableObjectPrivate::GetParameterType(int32 ParamIndex) const
{
	if (ParamIndex >= 0 && ParamIndex < ParameterProperties.Num())
	{
		return ParameterProperties[ParamIndex].Type;
	}
	else
	{
		UE_LOGF(LogMutable, Error, "Index [%d] out of ParameterProperties bounds at GetParameterType.", ParamIndex);
	}

	return EMutableParameterType::None;
}


EMutableParameterType UCustomizableObject::GetParameterTypeByName(const FString& Name) const
{
	const int32 Index = GetPrivate()->FindParameter(Name); 
	if (GetPrivate()->ParameterProperties.IsValidIndex(Index))
	{
		return GetPrivate()->ParameterProperties[Index].Type;
	}

	UE_LOGF(LogMutable, Warning, "Name '%ls' does not exist in ParameterProperties lookup table at GetParameterTypeByName at CO %ls.", *Name, *GetName());

	for (int32 ParamIndex = 0; ParamIndex < GetPrivate()->ParameterProperties.Num(); ++ParamIndex)
	{
		if (GetPrivate()->ParameterProperties[ParamIndex].Name == Name)
		{
			return GetPrivate()->ParameterProperties[ParamIndex].Type;
		}
	}

	UE_LOGF(LogMutable, Warning, "Name '%ls' does not exist in ParameterProperties at GetParameterTypeByName at CO %ls.", *Name, *GetName());

	return EMutableParameterType::None;
}


const FString& UCustomizableObject::GetParameterName(int32 ParamIndex) const
{
	if (ParamIndex >= 0 && ParamIndex < GetPrivate()->ParameterProperties.Num())
	{
		return GetPrivate()->ParameterProperties[ParamIndex].Name;
	}
	else
	{
		UE_LOGF(LogMutable, Warning, "Index [%d] out of ParameterProperties bounds at GetParameterName at CO %ls.", ParamIndex, *GetName());
	}

	return s_EmptyString;
}


void UCustomizableObjectPrivate::UpdateParameterPropertiesFromModel(const TSharedPtr<UE::Mutable::Private::FModel>& Model)
{
	if (Model)
	{
		TSharedPtr<UE::Mutable::Private::FParameters> MutableParameters = UE::Mutable::Private::FModel::NewParameters(Model);
		const int32 NumParameters = MutableParameters->GetCount();

		TArray<int32> TypedParametersCount;
		TypedParametersCount.SetNum(static_cast<int32>(UE::Mutable::Private::EParameterType::Count));
		
		ParameterProperties.Reset(NumParameters);
		ParameterPropertiesLookupTable.Empty(NumParameters);
		for (int32 Index = 0; Index < NumParameters; ++Index)
		{
			FMutableModelParameterProperties Data;

			Data.Name = MutableParameters->GetName(Index);
			Data.Type = EMutableParameterType::None;

			UE::Mutable::Private::EParameterType ParameterType = MutableParameters->GetType(Index);
			switch (ParameterType)
			{
			case UE::Mutable::Private::EParameterType::Bool:
			{
				Data.Type = EMutableParameterType::Bool;
				break;
			}

			case UE::Mutable::Private::EParameterType::Int:
			{
				Data.Type = EMutableParameterType::Int;

				const int32 ValueCount = MutableParameters->GetIntPossibleValueCount(Index);
				Data.PossibleValues.Reserve(ValueCount);
				for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
				{
					FMutableModelParameterValue& ValueData = Data.PossibleValues.AddDefaulted_GetRef();
					ValueData.Name = MutableParameters->GetIntPossibleValueName(Index, ValueIndex);
					ValueData.Value = MutableParameters->GetIntPossibleValue(Index, ValueIndex);
				}
				break;
			}

			case UE::Mutable::Private::EParameterType::Float:
			{
				Data.Type = EMutableParameterType::Float;
				break;
			}

			case UE::Mutable::Private::EParameterType::Color:
			{
				Data.Type = EMutableParameterType::Color;
				break;
			}

			case UE::Mutable::Private::EParameterType::Projector:
			{
				Data.Type = EMutableParameterType::Projector;
				break;
			}

			case UE::Mutable::Private::EParameterType::Matrix:
			{
				Data.Type = EMutableParameterType::Transform;
				break;
			}
				
			case UE::Mutable::Private::EParameterType::Texture:
			{
				Data.Type = EMutableParameterType::Texture;
				break;
			}

			case UE::Mutable::Private::EParameterType::SkeletalMesh:
			{
				Data.Type = EMutableParameterType::SkeletalMesh;
				break;
			}
			
			case UE::Mutable::Private::EParameterType::Material:
			{
				Data.Type = EMutableParameterType::Material;
				break;
			}

			case UE::Mutable::Private::EParameterType::InstancedStruct:
			{
				Data.Type = EMutableParameterType::InstancedStruct;
				break;
			}

			default:
				// Unhandled type?
				check(false);
				break;
			}

			ParameterProperties.Add(Data);
			ParameterPropertiesLookupTable.Add(Data.Name, FMutableParameterIndex(Index, TypedParametersCount[static_cast<int32>(ParameterType)]++));
		}
	}
	else
	{
		ParameterProperties.Empty();
		ParameterPropertiesLookupTable.Empty();
	}
}


int32 UCustomizableObject::GetEnumParameterNumValues(const FString& ParamName) const
{
	const int32 ParamIndex = GetPrivate()->FindParameter(ParamName);
	if (ParamIndex != INDEX_NONE)
	{
		return GetPrivate()->GetEnumParameterNumValues(ParamIndex);
	}
	else
	{
		return 0;
	}
}


const FString& UCustomizableObject::GetEnumParameterValue(const FString& ParamName, int32 ValueIndex) const
{
	const int32 ParamIndex = GetPrivate()->FindParameter(ParamName);
	if (ParamIndex != INDEX_NONE)
	{
		return GetPrivate()->GetIntParameterAvailableOption(ParamIndex, ValueIndex);
	}
	else
	{
		return s_EmptyString;
	}
}


bool UCustomizableObject::ContainsEnumParameterValue(const FString& ParameterName, const FString Value) const
{
	const int32 ParamIndex = GetPrivate()->FindParameter(ParameterName);
	if (ParamIndex == INDEX_NONE)
	{
		return false;
	}

	return GetPrivate()->FindIntParameterValue(ParamIndex, Value) != INDEX_NONE;
}


bool UCustomizableObject::ContainsParameter(const FString& ParameterName) const
{
	return GetPrivate()->FindParameter(ParameterName) != INDEX_NONE;
}


int32 UCustomizableObjectPrivate::FindParameter(const FString& Name) const
{
	if (const FMutableParameterIndex* Found = ParameterPropertiesLookupTable.Find(Name))
	{
		return Found->Index;
	}

	return INDEX_NONE;
}


int32 UCustomizableObjectPrivate::FindParameterTyped(const FString& Name, EMutableParameterType Type) const
{
	if (const FMutableParameterIndex* Found = ParameterPropertiesLookupTable.Find(Name))
	{
		if (ParameterProperties[Found->Index].Type == Type)
		{
			return Found->TypedIndex;
		}
	}

	return INDEX_NONE;
}


int32 UCustomizableObjectPrivate::FindIntParameterValue(int32 ParamIndex, const FString& Value) const
{
	int32 MinValueIndex = INDEX_NONE;
	
	if (ParamIndex >= 0 && ParamIndex < ParameterProperties.Num())
	{
		const TArray<FMutableModelParameterValue>& PossibleValues = ParameterProperties[ParamIndex].PossibleValues;
		if (PossibleValues.Num())
		{
			MinValueIndex = PossibleValues[0].Value;

			for (int32 OrderValue = 0; OrderValue < PossibleValues.Num(); ++OrderValue)
			{
				const FString& Name = PossibleValues[OrderValue].Name;

				if (Name == Value)
				{
					int32 CorrectedValue = OrderValue + MinValueIndex;
					check(PossibleValues[OrderValue].Value == CorrectedValue);
					return CorrectedValue;
				}
			}
		}
	}
	
	return MinValueIndex;
}


FMutableParamUIMetadata UCustomizableObject::GetParameterUIMetadata(const FString& ParamName) const
{
	if (const UModelResources* ModelResources = Private->GetModelResources())
	{
		const FMutableParameterData* ParameterData = ModelResources->ParameterUIDataMap.Find(ParamName);
		return ParameterData ? ParameterData->ParamUIMetadata : FMutableParamUIMetadata();
	}

	return FMutableParamUIMetadata();
}


FMutableParamUIMetadata UCustomizableObject::GetEnumParameterValueUIMetadata(const FString& ParamName, const FString& OptionName) const
{
	const UModelResources* ModelResources = Private->GetModelResources();
	if (!ModelResources)
	{
		return FMutableParamUIMetadata();
	}

	const int32 ParameterIndex = GetPrivate()->FindParameter(ParamName);
	if (ParameterIndex == INDEX_NONE)
	{
		return FMutableParamUIMetadata();
	}

	if (const FMutableParameterData* ParameterData = ModelResources->ParameterUIDataMap.Find(ParamName))
	{
		if (const FIntegerParameterUIData* IntegerParameterUIData = ParameterData->ArrayIntegerParameterOption.Find(OptionName))
		{
			return IntegerParameterUIData->ParamUIMetadata;
		}
	}

	return FMutableParamUIMetadata();
}


ECustomizableObjectGroupType UCustomizableObject::GetEnumParameterGroupType(const FString& ParamName) const
{
	const UModelResources* ModelResources = Private->GetModelResources();
	if (!ModelResources)
	{
		return  ECustomizableObjectGroupType::COGT_TOGGLE;
	}

	const int32 ParameterIndex = GetPrivate()->FindParameter(ParamName);
	if (ParameterIndex == INDEX_NONE)
	{
		return ECustomizableObjectGroupType::COGT_TOGGLE;
	}

	if (const FMutableParameterData* ParameterData = ModelResources->ParameterUIDataMap.Find(ParamName))
	{
		return ParameterData->IntegerParameterGroupType;
	}

	return ECustomizableObjectGroupType::COGT_TOGGLE;
}


FMutableStateUIMetadata UCustomizableObject::GetStateUIMetadata(const FString& StateName) const
{
	if (const UModelResources* ModelResources = Private->GetModelResources())
	{
		if (const FMutableStateData* StateData = ModelResources->StateUIDataMap.Find(StateName))
		{
			return StateData->StateUIMetadata;
		}
	}

	return FMutableStateUIMetadata();
}


#if WITH_EDITOR

uint32 GetTypeHash(const FIntegerParameterOptionKey& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterOption));
	return Hash;
}

TArray<TSoftObjectPtr<UDataTable>> UCustomizableObject::GetEnumParameterValueDataTable(const FString& ParamName, const FString& ValueName)
{
	if(const UModelResources* LocalModelResources = GetPrivate()->GetModelResources())
	{
		if (const FIntegerParameterOptionDataTable* Result = LocalModelResources->IntParameterOptionDataTable.Find({ ParamName, ValueName }))
		{
			return Result->DataTables.Array();
		}
	}
	
	return {};
}
#endif


float UCustomizableObject::GetFloatParameterDefaultValue(const FString& InParameterName) const
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(InParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent float parameter [%ls] in the CustomizableObject [%ls].", *InParameterName, *GetName());
		return FCustomizableObjectFloatParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetPrivate()->GetModel();
	if (!Model)
	{
		checkNoEntry();
		return FCustomizableObjectFloatParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	return Model->GetFloatDefaultValue(ParameterIndex);
}


int32 UCustomizableObject::GetEnumParameterDefaultValue(const FString& InParameterName) const
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(InParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent integer parameter [%ls] in the CustomizableObject [%ls].", *InParameterName, *GetName());
		return FCustomizableObjectIntParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetPrivate()->GetModel();
	if (!Model)
	{
		checkNoEntry();
		return FCustomizableObjectIntParameterValue::DEFAULT_PARAMETER_VALUE;
	}
	
	return Model->GetIntDefaultValue(ParameterIndex);
}


bool UCustomizableObject::GetBoolParameterDefaultValue(const FString& InParameterName) const
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(InParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent boolean parameter [%ls] in the CustomizableObject [%ls].", *InParameterName, *GetName());
		return FCustomizableObjectBoolParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetPrivate()->GetModel();
	if (!Model)
	{
		checkNoEntry();
		return FCustomizableObjectBoolParameterValue::DEFAULT_PARAMETER_VALUE;
	}
	
	return Model->GetBoolDefaultValue(ParameterIndex);
}


FLinearColor UCustomizableObject::GetColorParameterDefaultValue(const FString& InParameterName) const
{
	return GetVectorParameterDefaultValue(InParameterName);
}


FVector4f UCustomizableObject::GetVectorParameterDefaultValue(const FString& ParameterName) const
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(ParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent color parameter [%ls] in the CustomizableObject [%ls].", *ParameterName, *GetName());
		return FCustomizableObjectVectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetPrivate()->GetModel();
	if (!Model)
	{
		checkNoEntry();
		return FCustomizableObjectVectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}
	
	FVector4f Value;
	Model->GetColorDefaultValue(ParameterIndex, Value);

	return Value;
}


FTransform UCustomizableObject::GetTransformParameterDefaultValue(const FString& InParameterName) const
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(InParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent color parameter [%ls] in the CustomizableObject [%ls].", *InParameterName, *GetName());
		return FCustomizableObjectTransformParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetPrivate()->GetModel();
	if (!Model)
	{
		checkNoEntry();
		return FCustomizableObjectTransformParameterValue::DEFAULT_PARAMETER_VALUE;
	}
	
	const FMatrix44f Matrix = Model->GetMatrixDefaultValue(ParameterIndex);

	return FTransform(FMatrix(Matrix));
}


FCustomizableObjectProjector UCustomizableObject::GetProjectorParameterDefaultValue(const FString& InParameterName) const 
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(InParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent projector [%ls] in the CustomizableObject [%ls].", *InParameterName, *GetName());
		return FCustomizableObjectProjectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetPrivate()->GetModel();
	if (!Model)
	{
		checkNoEntry();
		return FCustomizableObjectProjectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}

	FCustomizableObjectProjector Value;
	UE::Mutable::Private::EProjectorType Type;
	Model->GetProjectorDefaultValue(ParameterIndex, &Type, &Value.Position, &Value.Direction, &Value.Up, &Value.Scale, &Value.Angle);
	Value.ProjectionType = ProjectorUtils::GetEquivalentProjectorType(Type);
	
	return Value;
}


UTexture* UCustomizableObject::GetTextureParameterDefaultValue(const FString& ParameterName) const
{
	const TObjectPtr<UTexture>* Result = GetPrivate()->GetModelResources()->TextureParameterDefaultValues.Find(ParameterName);
	return Result ? *Result : nullptr;
}
 

UMaterialInterface* UCustomizableObject::GetMaterialParameterDefaultValue(const FString& ParameterName) const
{
	const TObjectPtr<UMaterialInterface>* Result = GetPrivate()->GetModelResources()->MaterialParameterDefaultValues.Find(ParameterName);
	return Result ? *Result : nullptr;
}


USkeletalMesh* UCustomizableObject::GetSkeletalMeshParameterDefaultValue(const FString& ParameterName) const
{
	const TObjectPtr<USkeletalMesh>* Result = GetPrivate()->GetModelResources()->SkeletalMeshParameterDefaultValues.Find(ParameterName);
	return Result ? *Result : nullptr;
}


FInstancedStruct UCustomizableObject::GetExternalTypeParameterDefaultValue(const FString& ParameterName) const
{
	const UModelResources& ModelResources = GetPrivate()->GetModelResourcesChecked();
	
	if (const FInstancedStruct* Result = ModelResources.ExternalTypeParameterDefaultValues.Find(ParameterName))
	{
		return *Result;
	}

	if (const TObjectPtr<const UScriptStruct>* Result = ModelResources.ExternalTypeParameterTypes.Find(ParameterName))
	{
		FInstancedStruct InstancedStruct;
		InstancedStruct.InitializeAs(Result->Get());
		
		return InstancedStruct;
	}
	
	return FCustomizableObjectExternalTypeParameterValue::DEFAULT_PARAMETER_VALUE;
}


bool UCustomizableObject::IsParameterMultidimensional(const FString& InParameterName) const
{
	const int32 ParameterIndex = GetPrivate()->FindParameter(InParameterName);
	if (ParameterIndex == INDEX_NONE)
	{
		UE_LOGF(LogMutable, Error, "Tried to access the default value of the nonexistent parameter [%ls] in the CustomizableObject [%ls].", *InParameterName, *GetName());
		return false;
	}

	return GetPrivate()->IsParameterMultidimensional(ParameterIndex);
}


void UCustomizableObject::Compile(const FCompileParams& Params)
{
	if (ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
	{
		EditorModule->CompileCustomizableObject(*this, &Params, false, true);
	}
	else
	{
		FCompileCallbackParams CallbackParams;
		CallbackParams.bRequestFailed = true;
		CallbackParams.bCompiled = IsCompiled();

		Params.Callback.ExecuteIfBound(CallbackParams);
		Params.CallbackNative.ExecuteIfBound(CallbackParams);
	}
}


bool UCustomizableObjectPrivate::IsParameterMultidimensional(const int32& InParamIndex) const
{
	check(InParamIndex != INDEX_NONE);
	
	if (const TSharedPtr<const UE::Mutable::Private::FModel>& Model = GetModel())
	{
		return Model->IsParameterMultidimensional(InParamIndex);
	}

	return false;
}


void UCustomizableObjectPrivate::ApplyStateForcedValuesToParameters(FCustomizableObjectInstanceDescriptor& Descriptor)
{
	const UModelResources* LocalModelResources = GetModelResources();
	if (!LocalModelResources)
	{
		return;
	}

	const FString& StateName = Descriptor.GetCurrentState();
	const FMutableStateData* StateData = LocalModelResources->StateUIDataMap.Find(StateName);
	if (!StateData)
	{
		return;
	}

	for (FCustomizableObjectIntParameterValue& IntParameter : Descriptor.IntParameters)
	{
		if (const FString* Result = StateData->ForcedParameterValues.Find(IntParameter.ParameterName))
		{
			IntParameter.ParameterValueName = *Result;

			for (FString& Range : IntParameter.ParameterRangeValueNames)
			{
				Range = *Result;
			}
		}
	}

	for (FCustomizableObjectBoolParameterValue& BoolParameter : Descriptor.BoolParameters)
	{
		if (const FString* Result = StateData->ForcedParameterValues.Find(BoolParameter.ParameterName))
		{
			BoolParameter.ParameterValue = Result->ToBool();
		}
	}
}


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------

TStrongObjectPtr<UCustomizableObjectSkeletalMesh> FSkeletalMeshCache::Get(const UE::Mutable::Private::FSkeletalMeshId& Key)
{
	UE::TScopeLock<UE::FMutex> LockGuard(Lock);

	const TWeakObjectPtr<UCustomizableObjectSkeletalMesh>* Result = GeneratedSkeletalMeshes.Find(Key);
	if (!Result)
	{
		return {};
	}
	
	bool bWasPinSuccessful = false;
	return Result->TryPin(bWasPinSuccessful);
}


void FSkeletalMeshCache::Add(const UE::Mutable::Private::FSkeletalMeshId& Key, UCustomizableObjectSkeletalMesh* Value)
{
	if (!Value)
	{
		return;
	}
	
	UE::TScopeLock<UE::FMutex> LockGuard(Lock);

	GeneratedSkeletalMeshes.Add(Key, TWeakObjectPtr(Value));
}


void FSkeletalMeshCache::Clear()
{
	UE::TScopeLock<UE::FMutex> LockGuard(Lock);

	GeneratedSkeletalMeshes.Empty();
}


USkeleton* FSkeletonCache::Get(const TArray<FName>& Key)
{
	const TWeakObjectPtr<USkeleton>* Result = MergedSkeletons.Find(Key);
	return Result ? Result->Get() : nullptr;
}


void FSkeletonCache::Add(const TArray<FName>& Key, USkeleton* Value)
{
	if (!Value)
	{
		return;
	}

	MergedSkeletons.Add(Key, Value);

	// Remove invalid Skeletons from the cache.
	for (auto SkeletonIterator = MergedSkeletons.CreateIterator(); SkeletonIterator; ++SkeletonIterator)
	{
		if (SkeletonIterator.Value().IsStale())
		{
			SkeletonIterator.RemoveCurrent();
		}
	}
}


void FSkeletonCache::Clear()
{
	MergedSkeletons.Empty();
}


TStrongObjectPtr<UTexture2D> FTextureCache::Get(const FId& Key)
{
	UE::TScopeLock<UE::FMutex> LockGuard(Lock);

	const TWeakObjectPtr<UTexture2D>* FoundTexture = GeneratedTextures.Find(Key);
	if (!FoundTexture)
	{
		return TStrongObjectPtr<UTexture2D>();
	}
	
	bool bWasPinSuccessful = false;
	return FoundTexture->TryPin(bWasPinSuccessful);
}


void FTextureCache::Add(const FId& Key, UTexture2D* Value)
{
	if (!Value)
	{
		return;
	}

	{
		UE::TScopeLock<UE::FMutex> LockGuard(Lock);
		
		GeneratedTextures.Add(Key, Value);

		// Remove invalid SkeletalMeshes from the cache.
		for (auto SkeletonIterator = GeneratedTextures.CreateIterator(); SkeletonIterator; ++SkeletonIterator)
		{
			if (SkeletonIterator.Value().IsStale())
			{
				SkeletonIterator.RemoveCurrent();
			}
		}
	}
}


void FTextureCache::Remove(const UTexture2D& Value)
{
	UE::TScopeLock<UE::FMutex> LockGuard(Lock);
	
	for (auto It = GeneratedTextures.CreateIterator(); It; ++It)
	{
		if (It->Value.Get() == &Value)
		{
			It.RemoveCurrent();
			break;
		}
	}
}


void FTextureCache::Clear()
{
	UE::TScopeLock<UE::FMutex> LockGuard(Lock);
	
	GeneratedTextures.Empty();
}


FArchive& operator<<(FArchive& Ar, FIntegerParameterUIData& Struct)
{
	Ar << Struct.ParamUIMetadata;
	
	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableParameterData& Struct)
{
	Ar << Struct.ParamUIMetadata;
	Ar << Struct.Type;
	Ar << Struct.ArrayIntegerParameterOption;
	Ar << Struct.IntegerParameterGroupType;
	
	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableStateData& Struct)
{
	Ar << Struct.StateUIMetadata;
	Ar << Struct.bLiveUpdateMode;
	Ar << Struct.bDisableMeshStreaming;
	Ar << Struct.bDisableTextureStreaming;
	Ar << Struct.ForcedParameterValues;

	return Ar;
}


void FModelStreamableBulkData::Serialize(FArchive& Ar, UObject* Owner, bool bCooked)
{
	MUTABLE_CPUPROFILER_SCOPE(FModelStreamableBulkData::Serialize);

	Ar << ModelStreamables;

	if (bCooked)
	{
		int32 NumBulkDatas = StreamableBulkData.Num();
		Ar << NumBulkDatas;

		StreamableBulkData.SetNum(NumBulkDatas);

		for (FByteBulkData& BulkData : StreamableBulkData)
		{
			BulkData.Serialize(Ar, Owner);
		}
	}
}


UModelStreamableData::UModelStreamableData()
{
	StreamingData = MakeShared<FModelStreamableBulkData>();
}


void UModelStreamableData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (bCooked && !IsTemplate() && !Ar.IsCountingMemory())
	{
		check(StreamingData);
		StreamingData->Serialize(Ar, GetOutermostObject(), bCooked);
	}
}

void UModelStreamableData::PostLoad()
{
	Super::PostLoad();

	if (StreamingData)
	{
		const FString OutermostName = GetOutermost()->GetName();
		FString PackageFilename = FPackageName::LongPackageNameToFilename(OutermostName);
		FPaths::MakeStandardFilename(PackageFilename);
		StreamingData->FullFilePath = PackageFilename;
	}
}


void UCustomizableObjectPrivate::PostLoad()
{
	Super::PostLoad();

	if (SerializedModel)
	{
		SetModel(SerializedModel, FGuid(), false);
	}
}


void UCustomizableObjectPrivate::SetModel(const TSharedPtr<UE::Mutable::Private::FModel>& Model, const FGuid Id, bool bIsCooking)
{
	if (MutableModel == Model
#if WITH_EDITOR
		&& Identifier == Id
#endif
		)
	{
		return;
	}
	
#if WITH_EDITOR
	if (MutableModel)
	{
		MutableModel->Invalidate();
	}

	Identifier = Id;
#endif
	
	MutableModel = Model;

	if (Model)
	{
		UModelResources* LocalModelResources = nullptr;
#if WITH_EDITORONLY_DATA
		LocalModelResources = GetModelResources(bIsCooking);
#else 
		LocalModelResources = GetModelResources();
#endif
		check(LocalModelResources);
		
		UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram();
		Program.ExternalOperationProvider = MakeShared<UE::Mutable::Private::FExternalOperationProvider>(*LocalModelResources);

		for (UE::Mutable::Private::FParameterDesc& Parameter : Program.Parameters)
		{
			switch (Parameter.Type)
			{
			case UE::Mutable::Private::EParameterType::Texture:
				Parameter.DefaultValue.Set<UE::Mutable::Private::FParamTextureType>(TStrongObjectPtr<UTexture>(LocalModelResources->TextureParameterDefaultValues[Parameter.Name]));
				break;
				
			case UE::Mutable::Private::EParameterType::SkeletalMesh:
				Parameter.DefaultValue.Set<UE::Mutable::Private::FParamSkeletalMeshType>(TStrongObjectPtr<USkeletalMesh>(LocalModelResources->SkeletalMeshParameterDefaultValues[Parameter.Name]));
				break;
				
			case UE::Mutable::Private::EParameterType::Material:
				Parameter.DefaultValue.Set<UE::Mutable::Private::FParamMaterialType>(TStrongObjectPtr<UMaterialInterface>(LocalModelResources->MaterialParameterDefaultValues[Parameter.Name]));
				break;
				
			case UE::Mutable::Private::EParameterType::InstancedStruct:
				Parameter.DefaultValue.Set<UE::Mutable::Private::FParamInstancedStructType>(UE::Mutable::Private::MakeManaged<FInstancedStruct>(LocalModelResources->ExternalTypeParameterDefaultValues[Parameter.Name]));
				break;
				
			case UE::Mutable::Private::EParameterType::Matrix:
			case UE::Mutable::Private::EParameterType::String:
			case UE::Mutable::Private::EParameterType::None:
			case UE::Mutable::Private::EParameterType::Bool:
			case UE::Mutable::Private::EParameterType::Int:
			case UE::Mutable::Private::EParameterType::Float:
			case UE::Mutable::Private::EParameterType::Color:
			case UE::Mutable::Private::EParameterType::Projector:
				break;
			default:
				unimplemented();
			}
		}
	}
	
	// Create parameter properties
	UpdateParameterPropertiesFromModel(Model);

	using EState = FCustomizableObjectStatus::EState;
	Status.NextState(Model ? EState::ModelLoaded : EState::NoModel);
}


const TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe>& UCustomizableObjectPrivate::GetModel()
{
	return MutableModel;
}


const TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> UCustomizableObjectPrivate::GetModel() const
{
	return MutableModel;
}

#if WITH_EDITOR
void UCustomizableObjectPrivate::SetModelStreamableBulkData(const TSharedPtr<FModelStreamableBulkData>& StreamableData, bool bIsCooking)
{
	if (bIsCooking)
	{
		if (!ModelStreamableData)
		{
			ModelStreamableData = NewObject<UModelStreamableData>(GetOuter());
		}

		ModelStreamableData->StreamingData = StreamableData;
	}
	else
	{
		ModelStreamableDataEditor = StreamableData;
	}
}
#endif

TSharedPtr<FModelStreamableBulkData> UCustomizableObjectPrivate::GetModelStreamableBulkData(bool bIsCooking) const
{
#if WITH_EDITOR
	if (bIsCooking)
	{
		return ModelStreamableData ? ModelStreamableData->StreamingData : nullptr;
	}

	return ModelStreamableDataEditor;
#else
	return ModelStreamableData ? ModelStreamableData->StreamingData : nullptr;
#endif
}


UModelResources* UCustomizableObjectPrivate::GetModelResources()
{
	const UCustomizableObjectPrivate* ConstThis = this;
	return const_cast<UModelResources*>(ConstThis->GetModelResources());
}


const UModelResources* UCustomizableObjectPrivate::GetModelResources() const
{
#if WITH_EDITORONLY_DATA
	return ModelResourcesEditor;
#else
	return ModelResources;
#endif
}


UModelResources& UCustomizableObjectPrivate::GetModelResourcesChecked()
{
	const UCustomizableObjectPrivate* ConstThis = this;
	return *const_cast<UModelResources*>(&ConstThis->GetModelResourcesChecked());
}


const UModelResources& UCustomizableObjectPrivate::GetModelResourcesChecked() const
{
	const UModelResources* LocalModelResources = GetModelResources();
	check(LocalModelResources);

	return *LocalModelResources;
}


#if WITH_EDITORONLY_DATA
UModelResources* UCustomizableObjectPrivate::GetModelResources(bool bIsCooking)
{
	const UCustomizableObjectPrivate* ConstThis = this;
	return const_cast<UModelResources*>(ConstThis->GetModelResources(bIsCooking));
}


const UModelResources* UCustomizableObjectPrivate::GetModelResources(bool bIsCooking) const
{
	return bIsCooking ? ModelResources : ModelResourcesEditor;
}


void UCustomizableObjectPrivate::SetModelResources(UModelResources* InModelResources, bool bIsCooking)
{
	if (bIsCooking)
	{
		ModelResources = InModelResources;
	}
	else
	{
		ModelResourcesEditor = InModelResources;
	}
}
#endif


#if WITH_EDITOR
bool UCustomizableObjectPrivate::IsCompilationOutOfDate(bool bSkipIndirectReferences, TArray<FName>& OutOfDatePackages, TArray<FName>& AddedPackages, TArray<FName>& RemovedPackages, bool& bReleaseVersionDiff) const
{
	if (const ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		return Module->IsCompilationOutOfDate(*GetPublic(), bSkipIndirectReferences, OutOfDatePackages, AddedPackages, RemovedPackages, bReleaseVersionDiff);
	}

	return false;		
}
#endif


TArray<FString>& UCustomizableObjectPrivate::GetCustomizableObjectClassTags()
{
	return GetPublic()->CustomizableObjectClassTags;
}


TArray<FString>& UCustomizableObjectPrivate::GetPopulationClassTags()
{
	return GetPublic()->PopulationClassTags;
}


TMap<FString, FParameterTags>& UCustomizableObjectPrivate::GetCustomizableObjectParametersTags()
{
	return GetPublic()->CustomizableObjectParametersTags;
}


#if WITH_EDITORONLY_DATA
TObjectPtr<UEdGraph>& UCustomizableObjectPrivate::GetSource() const
{
	return GetPublic()->Source;
}
#endif

#if WITH_EDITOR
namespace UE::Mutable::Private
{
	uint64 FFile::GetSize() const
	{
		uint64 FileSize = 0;
		for (const FBlock& Block : Blocks)
		{
			FileSize += Block.Size;
		}
		return FileSize;
	}


	void FFile::GetFileData(FMutableCachedPlatformData* PlatformData, TArray64<uint8>& DestData, bool bDropData)
	{
		check(PlatformData);

		const uint64 DestSize = DestData.Num();
		const int32 NumBlocks = Blocks.Num();

		const uint8* SourceData = nullptr;
		if (DataType == EStreamableDataType::Model)
		{
			for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
			{
				const FBlock& Block = Blocks[BlockIndex];
				check(Block.Offset + Block.Size <= DestSize);
				PlatformData->ModelStreamableData.Get(Block.Id, TArrayView64<uint8>(DestData.GetData() + Block.Offset, Block.Size), bDropData);
			}
			return;
		}
		else
		{
			checkf(false, TEXT("Unknown file DataType found."));
		}
	}

	FFileCategoryID::FFileCategoryID(EStreamableDataType InDataType, uint16 InResourceType, uint16 InFlags)
	{
		DataType = InDataType;
		ResourceType = InResourceType;
		Flags = InFlags;
	}

	uint32 GetTypeHash(const FFileCategoryID& Key)
	{
		uint32 Hash = (uint32)Key.DataType;
		Hash = HashCombine(Hash, Key.ResourceType);
		Hash = HashCombine(Hash, Key.Flags);
		return Hash;
	}


	TPair<FFileBucket&, FFileCategory&> FindOrAddCategory(TArray<FFileBucket>& Buckets, FFileBucket& DefaultBucket, const FFileCategoryID CategoryID)
	{
		// Find the category
		for (FFileBucket& Bucket : Buckets)
		{
			for (FFileCategory& Category : Bucket.Categories)
			{
				if (Category.Id == CategoryID)
				{
					return TPair<FFileBucket&, FFileCategory&>(Bucket, Category);
				}
			}
		}

		// Category not found, add to default bucket
		FFileCategory& Category = DefaultBucket.Categories.AddDefaulted_GetRef();
		Category.Id = CategoryID;
		return TPair<FFileBucket&, FFileCategory&>(DefaultBucket, Category);
	}


	struct FClassifyNode
	{
		TArray<FBlock> Blocks;
	};


	void AddNode(TMap<FFileCategoryID, FClassifyNode>& Nodes, int32 Slack, const FFileCategoryID& CategoryID, const FBlock& Block)
	{
		FClassifyNode& Root = Nodes.FindOrAdd(CategoryID);
		if (Root.Blocks.IsEmpty())
		{
			Root.Blocks.Reserve(Slack);
		}

		Root.Blocks.Add(Block);
	}
	
	void GenerateBulkDataFilesListWithFileLimit(
		FGuid ObjectId,
		TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model,
		FModelStreamableBulkData& ModelStreamableBulkData,
		uint32 NumFilesPerBucket,
		bool bAllowFileCategoryOverride,
		bool bAllowSplit,
		const ITargetPlatform& TargetPlatform,
		TArray<FFile>& OutBulkDataFiles)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateBulkDataFilesListWithFileLimit);

		if (!Model)
		{
			return;
		}

		/* Overview.
		 *	1. Add categories to the different buckets and accumulate the size of its resources 
		 *	   to know the total size of each category and the size of the buckets.
		 *	2. Use the accumulated sizes to distribute the NumFilesPerBucket between the bucket's categories.
		 *	3. Generate the list of BulkData files based on the number of files per category.
		 */

		// Two buckets. One for non-optional data and one for optional data.
		TArray<FFileBucket> FileBuckets;

		// DefaultBucket is for non-optional BulkData
		FFileBucket& DefaultBucket = FileBuckets.AddDefaulted_GetRef();
		FFileBucket& OptionalBucket = FileBuckets.AddDefaulted_GetRef();

		// Model Roms. Iterate all Model roms to distribute them in categories.
		{
			// Add meshes and low-res textures to the Default bucket 
			DefaultBucket.Categories.Add({ FFileCategoryID(EStreamableDataType::Model, uint16(UE::Mutable::Private::EDataType::Mesh), 0), 0, 0, 0 });
			DefaultBucket.Categories.Add({ FFileCategoryID(EStreamableDataType::Model, uint16(UE::Mutable::Private::EDataType::Image), 0), 0, 0, 0 });

			// Add High-res textures to the Optional bucket
			OptionalBucket.Categories.Add({ FFileCategoryID(EStreamableDataType::Model, uint16(UE::Mutable::Private::EDataType::Image), uint16(EMutableFileFlags::HighRes)), 0, 0, 0});

			const int32 NumRoms = Model->GetProgram().GetRomCount();
			for (int32 RomIndex = 0; RomIndex < NumRoms; ++RomIndex)
			{
				const uint32 BlockSize = Model->GetProgram().GetRomSize(RomIndex);
				const uint16 BlockResourceType = Model->GetProgram().GetRomDataType(RomIndex) == ERomDataType::Mesh ? uint16(UE::Mutable::Private::EDataType::Mesh) : uint16(UE::Mutable::Private::EDataType::Image);
				const EMutableFileFlags BlockFlags = Model->GetProgram().IsRomHighRes(RomIndex) ? EMutableFileFlags::HighRes : EMutableFileFlags::None;

				FFileCategoryID CategoryID = { EStreamableDataType::Model, BlockResourceType, uint16(BlockFlags) };
				TPair<FFileBucket&, FFileCategory&> It = FindOrAddCategory(FileBuckets, DefaultBucket, CategoryID); // Add block to an existing or new category
				It.Key.DataSize += BlockSize;
				It.Value.DataSize += BlockSize;
			}
		}

		
		// Limited number of files in each bucket. Find the ideal file distribution between categories based on the accumulated size of their resources.
		TArray<FFileCategory> Categories;
		TArray<FFileCategoryOverride> FileCategoryOverrides;

		using namespace UE::DerivedData;
		FCacheKey DDCKey;

		if (bAllowFileCategoryOverride)
		{
			// Generate DDC Key;
			TArray<uint8> Bytes;
			FMemoryWriter Ar(Bytes, /*bIsPersistent=*/ true);

			Ar << ObjectId;

			int32 CategoryOverrideDerivedDataVersion = 0x3c507a2f;
			Ar << CategoryOverrideDerivedDataVersion;

			FString PlatformName = TargetPlatform.PlatformName();
			Ar << PlatformName;

			DDCKey.Bucket = FCacheBucket(TEXT("CustomizableObject"));
			DDCKey.Hash = FIoHashBuilder::HashBuffer(MakeMemoryView(Bytes));


			// Request FileCategory Overrides for TargetPlatform
			FCacheGetValueRequest DDCRequest;
			DDCRequest.Key = DDCKey;

			FRequestOwner RequestOwner(UE::DerivedData::EPriority::Blocking);
			GetCache().GetValue({ DDCRequest }, RequestOwner, [&FileCategoryOverrides](FCacheGetValueResponse&& Response) mutable
				{
					if (Response.Status == EStatus::Ok)
					{
						const FCompressedBuffer& CompressedBuffer = Response.Value.GetData();
						if (ensure(!CompressedBuffer.IsNull()))
						{
							check(Response.Value.GetRawSize() % sizeof(FFileCategoryOverride) == 0);
							
							const int32 NumCategories = Response.Value.GetRawSize() / sizeof(FFileCategoryOverride);
							FileCategoryOverrides.SetNum(NumCategories);
							
							bool bSuccess = CompressedBuffer.TryDecompressTo(MakeMemoryView(FileCategoryOverrides));
						}
					}
				});
			RequestOwner.Wait();
		}


		for (FFileBucket& Bucket : FileBuckets)
		{
			uint32 NumFiles = 0;

			bool bCanOverride = bAllowFileCategoryOverride;

			for (FFileCategory& Category : Bucket.Categories)
			{
				if (Category.DataSize > 0)
				{
					double DataDistribution = (double)Category.DataSize / Bucket.DataSize;
					Category.NumFiles = FMath::Max(DataDistribution * NumFilesPerBucket, 1);  // At least one file if size > 0
					Category.FirstFile = NumFiles;

					NumFiles += Category.NumFiles;
				}

				if (bAllowFileCategoryOverride)
				{
					FFileCategoryOverride* CategoryOverride = FileCategoryOverrides.FindByPredicate(
						[Category](const FFileCategoryOverride& C) { return C.Id == Category.Id; });

					// Can't override if there's no override data or if the override doesn't have the minimum required number of files.
					const bool bIsValidOverride = CategoryOverride && (Category.DataSize == 0 || CategoryOverride->NumFiles > 0);
					bCanOverride = bCanOverride && bIsValidOverride;

					if (!CategoryOverride)
					{
						FileCategoryOverrides.Add(FFileCategoryOverride({ .Id = Category.Id, .NumFiles = Category.NumFiles }));
					}
				}
			}

			if (bCanOverride)
			{
				for (FFileCategory& Category : Bucket.Categories)
				{
					FFileCategoryOverride* CategoryOverride = FileCategoryOverrides.FindByPredicate(
						[Category](const FFileCategoryOverride& C) { return C.Id == Category.Id; });
					if (ensure(CategoryOverride))
					{
						UE_LOGF(LogMutable, Display, "Id [%ls] - NumFiles [%d] -> [%d].", *ObjectId.ToString(), Category.NumFiles, CategoryOverride->NumFiles);
						Category.NumFiles = CategoryOverride->NumFiles;
					}
				}
			}

			Categories.Append(Bucket.Categories);
		}

		if (bAllowFileCategoryOverride && !FileCategoryOverrides.IsEmpty())
		{
			for (auto& CategoryOverride : FileCategoryOverrides)
			{
				UE_LOGF(LogMutable, Display, "Id [%ls] - Storing Override NumFiles [%d].", *ObjectId.ToString(), CategoryOverride.NumFiles);
			}

			// Upload FileCategory Overrides to DDC.
			FCachePutValueRequest DDCRequest;
			DDCRequest.Key = DDCKey;
			DDCRequest.Value = FValue::Compress(FSharedBuffer::MakeView(FileCategoryOverrides.GetData(), FileCategoryOverrides.Num() * sizeof(FFileCategoryOverride)));

			FRequestOwner RequestOwner(UE::DerivedData::EPriority::Blocking);
			RequestOwner.KeepAlive();

			GetCache().PutValue({ DDCRequest }, RequestOwner);
		}
		
		// Function to create the list of bulk data files. Blocks will be grouped by source Id.
		const auto CreateFileList = [Categories, bAllowSplit](const FFileCategoryID& CategoryID, const FClassifyNode& Node, TArray<FFile>& OutBulkDataFiles)
			{
				const FFileCategory* Category = Categories.FindByPredicate(
					[CategoryID](const FFileCategory& C) { return C.Id == CategoryID; });
				check(Category);

				const int32 FirstFile = OutBulkDataFiles.Num();
				int32 NumBulkDataFiles = OutBulkDataFiles.Num();
				OutBulkDataFiles.Reserve(NumBulkDataFiles + Category->NumFiles);

				// FileID (File Index) to BulkData file index.
				TArray<int64> BulkDataFileIndex;
				BulkDataFileIndex.Init(INDEX_NONE, Category->NumFiles);

				uint64 MaxSize = 0;

				const int32 NumBlocks = Node.Blocks.Num();
				for (const FBlock& Block : Node.Blocks)
				{
					// Use the module of the source id to determine the file id (FileIndex)
					const uint32 FileID = Block.SourceId % Category->NumFiles;
					int64& FileIndex = BulkDataFileIndex[FileID];

					// Add new file
					if (FileIndex == INDEX_NONE)
					{
						FFile& NewFile = OutBulkDataFiles.AddDefaulted_GetRef();
						NewFile.DataType = CategoryID.DataType;
						NewFile.ResourceType = CategoryID.ResourceType;
						NewFile.Flags = CategoryID.Flags;
						NewFile.Id = FileID + Category->FirstFile;

						FileIndex = NumBulkDataFiles;
						++NumBulkDataFiles;
					}

					// Add block to the file 
					OutBulkDataFiles[FileIndex].Blocks.Add(Block);
					OutBulkDataFiles[FileIndex].Size += Block.Size;

					MaxSize = FMath::Max(OutBulkDataFiles[FileIndex].Size, MaxSize);
				}

				if (bAllowSplit)
				{
					// Split big files
					constexpr uint64 MinSplitSize = 10 * 1024 * 1024;
					const int32 PartitionsPerFile = FMath::Min(FMath::DivideAndRoundDown(Category->NumFiles, (uint32)(NumBulkDataFiles - FirstFile)), 3u);

					if (MaxSize > MinSplitSize && PartitionsPerFile > 1)
					{
						for (int32 FileIndex = FirstFile; FileIndex < NumBulkDataFiles; ++FileIndex)
						{
							FFile& BaseFile = OutBulkDataFiles[FileIndex];
							if (BaseFile.Size < MinSplitSize)
							{
								continue;
							}

							const uint64 MaxPartitionSize = FMath::DivideAndRoundUp(BaseFile.Size, (uint64)PartitionsPerFile);
							BaseFile.Size = 0;

							TArray<FBlock> BaseBlocks;
							Exchange(BaseBlocks, BaseFile.Blocks);

							TArray<int32> NewFileIndices;
							NewFileIndices.Add(FileIndex);
							for (int32 Index = 1; Index < PartitionsPerFile; ++Index)
							{
								NewFileIndices.Add(OutBulkDataFiles.Num());
								FFile NewFile = BaseFile;
								OutBulkDataFiles.Add(MoveTemp(NewFile));
							}

							int32 NewFileIndex = 0;
							for (const FBlock& BaseBlock : BaseBlocks)
							{
								FFile& NewFile = OutBulkDataFiles[NewFileIndices[NewFileIndex]];
								NewFile.Blocks.Add(BaseBlock);
								NewFile.Size += BaseBlock.Size;

								if (NewFile.Size > MaxPartitionSize)
								{
									NewFileIndex++;
								}
							}
						}
					}
				}
			};

		// Generate the list of BulkData files.
		GenerateBulkDataFilesList(Model, ModelStreamableBulkData, true /* bUseRomTypeAndFlagsToFilter */, CreateFileList, OutBulkDataFiles);
	}
	

	void GenerateBulkDataFilesListWithSizeLimit(
		TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model,
		FModelStreamableBulkData& ModelStreamableBulkData,
		const ITargetPlatform* TargetPlatform,
		uint64 ,
		TArray<FFile>& OutBulkDataFiles)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateBulkDataFilesListWithSizeLimit);

		if (!Model)
		{
			return;
		}


		// Unlimited number of files, limited file size. Add blocks to the file if the size limit won't be surpassed. Add at least one block to each file. 
		const auto CreateFileList = [](const FFileCategoryID& CategoryID, const FClassifyNode& Node, TArray<FFile>& OutBulkDataFiles)
			{
				// Temp: Group by order in the array
				for (int32 BlockIndex = 0; BlockIndex < Node.Blocks.Num(); )
				{
					FFile File;
					File.DataType = CategoryID.DataType;
					File.ResourceType = CategoryID.ResourceType;
					File.Flags = CategoryID.Flags;

					uint64 FileSize = 0;
					uint32 FileId = uint32(CategoryID.DataType);

					while (BlockIndex < Node.Blocks.Num())
					{
						const FBlock& CurrentBlock = Node.Blocks[BlockIndex];

						// Block added to file. Set offset and increase file size.
						FileSize += CurrentBlock.Size;

						// Generate cumulative id for this file
						FileId = HashCombine(FileId, CurrentBlock.Id);

						// Add the block to the current file
						File.Blocks.Add(CurrentBlock);

						// Next block
						++BlockIndex;
					}

					const int32 NumFiles = OutBulkDataFiles.Num();

					// Ensure the FileId is unique
					bool bUnique = false;
					while (!bUnique)
					{
						bUnique = true;
						for (int32 PreviousFileIndex = 0; PreviousFileIndex < NumFiles; ++PreviousFileIndex)
						{
							if (OutBulkDataFiles[PreviousFileIndex].Id == FileId)
							{
								bUnique = false;
								++FileId;
								break;
							}
						}
					}

					// Set it to the editor-only file descriptor
					File.Id = FileId;

					OutBulkDataFiles.Add(MoveTemp(File));
				}
			};

		// TODO: Temp. Remove after unifying generated output files code between editor an package. UE-222777
		const bool bUseRomTypeAndFlagsToFilter = TargetPlatform->RequiresCookedData();

		GenerateBulkDataFilesList(Model, ModelStreamableBulkData, bUseRomTypeAndFlagsToFilter, CreateFileList, OutBulkDataFiles);
	}


	void GenerateBulkDataFilesList(
		TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model,
		FModelStreamableBulkData& ModelStreamableBulkData,
		bool bUseRomTypeAndFlagsToFilter,
		TFunctionRef<void(const FFileCategoryID&, const FClassifyNode&, TArray<FFile>&)> CreateFileList,
		TArray<FFile>& OutBulkDataFiles)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateBulkDataFilesList);

		OutBulkDataFiles.Empty();

		if (!Model)
		{
			return;
		}

		// Root nodes by flags.
		const uint32 NumRoms = Model->GetProgram().GetRomCount();
		TMap<FFileCategoryID, FClassifyNode> RootNode;

		// Create blocks data.
		{
			for (uint32 RomIndex = 0; RomIndex < NumRoms; ++RomIndex)
			{
				uint32 SourceBlockId = Model->GetProgram().GetRomSourceId(RomIndex);
				const uint32 BlockSize = Model->GetProgram().GetRomSize(RomIndex);
				
				uint16 BlockResourceType = 0;
				EMutableFileFlags BlockFlags = EMutableFileFlags::None;
				// TODO: Temp. Remove after unifying generated output files code between editor an package. UE-222777 
				if (bUseRomTypeAndFlagsToFilter)
				{
					BlockResourceType = Model->GetProgram().GetRomDataType(RomIndex) == ERomDataType::Mesh ? uint16(UE::Mutable::Private::EDataType::Mesh) : uint16(UE::Mutable::Private::EDataType::Image);
					BlockFlags = Model->GetProgram().IsRomHighRes(RomIndex) ? EMutableFileFlags::HighRes : EMutableFileFlags::None;
				}

				FFileCategoryID CurrentCategory = { EStreamableDataType::Model, BlockResourceType, uint16(BlockFlags) };
				FBlock CurrentBlock = 
				{ 
					.Id = RomIndex, 
					.SourceId = SourceBlockId, 
					.Size = BlockSize, 
					.Offset = 0 
				};

				AddNode(RootNode, NumRoms, CurrentCategory, CurrentBlock);
			}
		}

		// Create Files list
		for (TPair<FFileCategoryID, FClassifyNode>& Node : RootNode)
		{
			CreateFileList(Node.Key, Node.Value, OutBulkDataFiles);
		}

		// Update streamable blocks data
		const int32 NumBulkDataFiles = OutBulkDataFiles.Num();
		for (int32 FileIndex = 0; FileIndex < NumBulkDataFiles; ++FileIndex)
		{
			FFile& File = OutBulkDataFiles[FileIndex];

			uint64 SourceOffset = 0;

			switch (File.DataType)
			{
			case EStreamableDataType::Model:
			{
				for (FBlock& Block : File.Blocks)
				{
					Block.Offset = SourceOffset;
					SourceOffset += Block.Size;

					FMutableStreamableBlock& StreamableBlock = ModelStreamableBulkData.ModelStreamables[Block.Id];
					StreamableBlock.FileId = FileIndex;
					StreamableBlock.Offset = Block.Offset;
				}
				break;
			}
			default:
				UE_LOGF(LogMutable, Error, "Unknown DataType found while fixing streaming block files ids.");
				unimplemented();
				break;
			}
		}
	}

	
	void CUSTOMIZABLEOBJECT_API SerializeBulkDataFiles(
		FMutableCachedPlatformData& CachedPlatformData,
		TArray<FFile>& BulkDataFiles,
		TFunctionRef<void(FFile&, TArray64<uint8>&, uint32)> WriteFile,
		bool bDropData)
	{
		MUTABLE_CPUPROFILER_SCOPE(SerializeBulkDataFiles);

		TArray64<uint8> FileBulkData;

		const uint32 NumBulkDataFiles = BulkDataFiles.Num();
		for (uint32 FileIndex = 0; FileIndex < NumBulkDataFiles; ++FileIndex)
		{
			UE::Mutable::Private::FFile& CurrentFile = BulkDataFiles[FileIndex];

			const int64 FileSize = CurrentFile.GetSize();
			FileBulkData.SetNumUninitialized(FileSize, EAllowShrinking::No);

			// Get the file data in memory
			CurrentFile.GetFileData(&CachedPlatformData, FileBulkData, bDropData);

			WriteFile(CurrentFile, FileBulkData, FileIndex);
		}
	}
	
	UE::DerivedData::FValueId GetDerivedDataModelId()
	{
		UE::DerivedData::FValueId::ByteArray ValueIdBytes = {};
		FMemory::Memset(&ValueIdBytes[0], 1, sizeof(ValueIdBytes));
		return UE::DerivedData::FValueId(ValueIdBytes);
	}

	UE::DerivedData::FValueId GetDerivedDataModelResourcesId()
	{
		UE::DerivedData::FValueId::ByteArray ValueIdBytes = {};		
		FMemory::Memset(&ValueIdBytes[0], 2, sizeof(ValueIdBytes));
		return UE::DerivedData::FValueId(ValueIdBytes);
	}

	UE::DerivedData::FValueId GetDerivedDataModelStreamableBulkDataId()
	{
		UE::DerivedData::FValueId::ByteArray ValueIdBytes = {};
		FMemory::Memset(&ValueIdBytes[0], 3, sizeof(ValueIdBytes));
		return UE::DerivedData::FValueId(ValueIdBytes);
	}

	UE::DerivedData::FValueId GetDerivedDataBulkDataFilesId()
	{
		UE::DerivedData::FValueId::ByteArray ValueIdBytes = {};
		FMemory::Memset(&ValueIdBytes[0], 4, sizeof(ValueIdBytes));
		return UE::DerivedData::FValueId(ValueIdBytes);
	}
}



void SerializeCompilationOptionsForDDC(FArchive& Ar, FCompilationOptions& Options)
{
	FString PlatformName = Options.TargetPlatform ? Options.TargetPlatform->PlatformName() : FString();
	Ar << PlatformName;
	Ar << Options.TextureCompression;
	Ar << Options.OptimizationLevel;
	Ar << Options.CustomizableObjectNumBoneInfluences;
	Ar << Options.bRealTimeMorphTargetsEnabled;
	Ar << Options.b16BitBoneWeightsEnabled;
	Ar << Options.bSkinWeightProfilesEnabled;
	Ar << Options.bPhysicsAssetMergeEnabled;
	Ar << Options.bAnimBpPhysicsManipulationEnabled;
	Ar << Options.ParamNamesToSelectedOptions;
	Ar << Options.bUseLegacyLayouts;
}


void SerializeTextureGroupSettingsForDDC(FArchive& Ar, const ITargetPlatform& Platform)
{
	const UTextureLODSettings& LODSettings = Platform.GetTextureLODSettings();
	for (FTextureLODGroup LODGroup : LODSettings.TextureLODGroups)
	{
		Ar << LODGroup.Filter;
		Ar << LODGroup.Group;
		Ar << LODGroup.LODBias;
		Ar << LODGroup.LODBias_Smaller;
		Ar << LODGroup.LODBias_Smallest;
		Ar << LODGroup.LossyCompressionAmount;
		Ar << LODGroup.MaxAniso;
		Ar << LODGroup.MaxLODMipCount;
		Ar << LODGroup.MaxLODSize;
		Ar << LODGroup.MaxLODSize_Smaller;
		Ar << LODGroup.MaxLODSize_Smallest;
		Ar << LODGroup.MinLODSize;

		FString MipFilter = LODGroup.MipFilter.ToString().ToLower();
		Ar << MipFilter;
		Ar << LODGroup.MipGenSettings;
		Ar << LODGroup.MipLoadOptions;
		Ar << LODGroup.NumStreamedMips;
		Ar << LODGroup.OptionalLODBias;
		Ar << LODGroup.OptionalMaxLODSize;
	}
}


TArray<uint8> UCustomizableObjectPrivate::BuildDerivedDataKey(FCompilationOptions Options)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObjectPrivate::BuildDerivedDataKey)

	check(IsInGameThread());
	
	UCustomizableObject& CustomizableObject = *GetPublic();

	TArray<uint8> Bytes;
	FMemoryWriter Ar(Bytes, /*bIsPersistent=*/ true);
	
	{
		uint32 Version = DerivedDataVersion;
		Ar << Version;
	}
	
	{
		uint32 VersionsHash = GetECustomizableObjectVersionEnumHash();
		Ar << VersionsHash;
	}

	// Custom Version
	{
		int32 CustomVersion = GetLinkerCustomVersion(FCustomizableObjectCustomVersion::GUID);
		Ar << CustomVersion;
	}
	

	// Customizable Object Ids
	{
		FGuid Id = GenerateIdentifier(CustomizableObject);
		Ar << Id;
	}

	{
		FGuid Version = CustomizableObject.VersionId;
		Ar << Version;
	}

	// Compile Options
	{
		SerializeCompilationOptionsForDDC(Ar, Options);
	}

	// Texture Settings
	{
		SerializeTextureGroupSettingsForDDC(Ar, *Options.TargetPlatform);
	}

	// Release Version
	if (const ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		FString Version = Module->GetCurrentReleaseVersionForObject(CustomizableObject);
		Ar << Version;
	}

	// Participating objects hash
	if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		TArray<TTuple<FName, FGuid>> ParticipatingObjects = Module->GetParticipatingObjects(GetPublic(), &Options).Array();
		ParticipatingObjects.Sort([](const TTuple<FName, FGuid>& A, const TTuple<FName, FGuid>& B)
		{
			return A.Get<0>().LexicalLess(B.Get<0>()) && A.Get<1>() < B.Get<1>();
		});
		
		for (const TTuple<FName, FGuid>& Tuple : ParticipatingObjects)
		{
			FString Key = Tuple.Get<0>().ToString();
			Key.ToLowerInline();
			Ar << Key;

			FGuid Id = Tuple.Get<1>();
			Ar << Id;
		}
	}

	// TODO List of plugins and their custom versions

	return Bytes;
}


UE::DerivedData::FCacheKey UCustomizableObjectPrivate::GetDerivedDataCacheKeyForOptions(FCompilationOptions InOptions)
{
	using namespace UE::DerivedData;

	TArray<uint8> DerivedDataKey = BuildDerivedDataKey(InOptions);

	FCacheKey CacheKey;
	CacheKey.Bucket = FCacheBucket(TEXT("CustomizableObject"));
	CacheKey.Hash = FIoHashBuilder::HashBuffer(MakeMemoryView(DerivedDataKey));
	return CacheKey;
}


UE::DerivedData::FValueId GetDerivedDataValueIdForResource(UE::Mutable::Private::EStreamableDataType StreamableDataType, uint32 FileId, uint16 ResourceType, uint16 Flags)
{
	UE::DerivedData::FValueId::ByteArray ValueIdBytes = {};
	
	static_assert(sizeof(UE::DerivedData::FValueId::ByteArray) >= sizeof(UE::Mutable::Private::EStreamableDataType) + sizeof(uint32) + sizeof(uint16) + sizeof(uint16));

	int8 ValueIdOffset = 0;
	FMemory::Memcpy(&ValueIdBytes[ValueIdOffset], &StreamableDataType, sizeof(StreamableDataType));
	ValueIdOffset += sizeof(StreamableDataType);
	FMemory::Memcpy(&ValueIdBytes[ValueIdOffset], &FileId, sizeof(FileId));
	ValueIdOffset += sizeof(FileId);
	uint16 DataType = (uint16)ResourceType;
	FMemory::Memcpy(&ValueIdBytes[ValueIdOffset], &DataType, sizeof(DataType));
	ValueIdOffset += sizeof(DataType);
	FMemory::Memcpy(&ValueIdBytes[ValueIdOffset], &Flags, sizeof(Flags));
	UE::DerivedData::FValueId ResourceId = UE::DerivedData::FValueId(ValueIdBytes);
	return ResourceId;
}


class FMutableMemoryCounterArchive final : public FArchive
{
public:
	FMutableMemoryCounterArchive()
	{
		SetIsSaving(true);
		SetIsPersistent(true);
		ArIsCountingMemory = true;
	}

	virtual void Serialize(void*, int64 Length) override { Size += Length; }
	virtual int64 TotalSize() override { return Size; }

private:
	int64 Size = 0;
};


void UCustomizableObjectPrivate::LogMemory()
{
	// Log in-memory data for the compiled CustomizableObject
	UE_LOGF(LogMutable, Log, "CustomizableObject [%ls] memory report:", *GetPublic()->GetName());
	const UE::Mutable::Private::FModel* MuModel = GetModel().Get();
	if (MuModel)
	{
		const UE::Mutable::Private::FProgram& Program = MuModel->GetProgram();
		int32 ByteCode = Program.ByteCode.GetAllocatedSize();
		int32 Roms = Program.Roms.GetAllocatedSize();
		int32 ConstantImages = Program.ConstantImages.GetAllocatedSize();
		int32 ConstantImageLODIndices = Program.ConstantImageLODIndices.GetAllocatedSize();
		int32 ConstantImageLODPermanent = Program.ConstantImageLODsPermanent.GetAllocatedSize();

		int32 PermanentImages = 0;
		for (const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage>& Entry : Program.ConstantImageLODsPermanent)
		{
			if (Entry)
			{
				PermanentImages += Entry->GetDataSize();
			}
		}
		int32 ImageTotal = ConstantImages + ConstantImageLODIndices + ConstantImageLODPermanent + PermanentImages;

		int32 ConstantMeshes = Program.ConstantMeshes.GetAllocatedSize();
		int32 ConstantMeshContentIndices = Program.ConstantMeshContentIndices.GetAllocatedSize();
		int32 ConstantMeshesPermanent = Program.ConstantMeshesPermanent.GetAllocatedSize();

		int32 PermanentMeshes = 0;
		for (const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& Entry : Program.ConstantMeshesPermanent)
		{
			if (Entry)
			{
				PermanentMeshes += Entry->GetDataSize();
			}
		}
		int32 MeshesTotal = ConstantMeshes + ConstantMeshContentIndices + ConstantMeshesPermanent + PermanentMeshes;

		int32 StringsTotal = Program.ConstantStrings.GetAllocatedSize();
		for (const FString& Entry : Program.ConstantStrings)
		{
			StringsTotal += Entry.GetAllocatedSize();
		}

		int32 ConstantNames = Program.ConstantNames.GetAllocatedSize();
		int32 ConstantUInt32Lists = Program.ConstantUInt32Lists.GetAllocatedSize();
		int32 ConstantInt32Lists = Program.ConstantInt32Lists.GetAllocatedSize();
		int32 ConstantUInt64Lists = Program.ConstantUInt64Lists.GetAllocatedSize();
		int32 ConstantFloatLists = Program.ConstantFloatLists.GetAllocatedSize();
		int32 ConstantBoolLists = Program.ConstantBoolLists.GetAllocatedSize();

		int32 SkeletonsTotal = Program.ConstantSkeletons.GetAllocatedSize();
		for (const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton>& Entry : Program.ConstantSkeletons)
		{
			SkeletonsTotal += sizeof(UE::Mutable::Private::FSkeleton) + Entry->BoneNames.GetAllocatedSize() + Entry->BoneIds.GetAllocatedSize() + Entry->BoneParents.GetAllocatedSize();
		}

		int32 SocketsTotal = Program.ConstantSockets.GetAllocatedSize();

		int32 PhysicsTotal = Program.ConstantPhysicsBodies.GetAllocatedSize();
		for (const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FPhysicsBody>& Entry : Program.ConstantPhysicsBodies)
		{
			PhysicsTotal += sizeof(UE::Mutable::Private::FPhysicsBody) + Entry->BodiesBoneNames.GetAllocatedSize() + Entry->Bodies.GetAllocatedSize();
			for (const UE::Mutable::Private::FPhysicsBodyAggregate& Body : Entry->Bodies)
			{
				PhysicsTotal += Body.Spheres.GetAllocatedSize() + Body.Boxes.GetAllocatedSize() + Body.Convex.GetAllocatedSize() + Body.Sphyls.GetAllocatedSize() + Body.TaperedCapsules.GetAllocatedSize();
			}
		}

		int32 ParametersTotal = Program.Parameters.GetAllocatedSize();
		for (const UE::Mutable::Private::FParameterDesc& Entry : Program.Parameters)
		{
			ParametersTotal += Entry.Ranges.GetAllocatedSize() + Entry.PossibleValues.GetAllocatedSize();
			for (const UE::Mutable::Private::FParameterDesc::FIntValueDesc& ValueDesc : Entry.PossibleValues)
			{
				ParametersTotal += ValueDesc.Name.GetAllocatedSize();
			}
		}

		int32 ModelTotal = ByteCode + Roms 
			+ ImageTotal + MeshesTotal 
			+ StringsTotal 
			+ ConstantNames
			+ ConstantUInt32Lists
			+ ConstantInt32Lists
			+ ConstantUInt64Lists
			+ ConstantFloatLists
			+ ConstantBoolLists
			+ SkeletonsTotal + PhysicsTotal
			+ SocketsTotal
			+ ParametersTotal;


		int32 StreamableData = 0;
		if (ModelStreamableData && ModelStreamableData->StreamingData)
		{
			const FModelStreamableBulkData& Bulk = *ModelStreamableData->StreamingData;
			StreamableData += Bulk.ModelStreamables.GetAllocatedSize();
			StreamableData += Bulk.StreamableBulkData.GetAllocatedSize();
		}

		int32 ModeleResData = 0;
		{
			ModeleResData += sizeof(UModelResources);
			
			FMutableMemoryCounterArchive Arch;
			if (ModelResources)
			{
				Arch << ModelResources->ReferenceSkeletalMeshesData;
				Arch << ModelResources->PassthroughObjects;
				Arch << ModelResources->AnimBPs;
				Arch << ModelResources->AnimBpOverridePhysiscAssetsInfo;
				Arch << ModelResources->ParameterUIDataMap;
				Arch << ModelResources->StateUIDataMap;
				Arch << ModelResources->ComponentNamesPerObjectComponent;
			}
			ModeleResData += Arch.TotalSize();
		}

		UE_LOGF(LogMutable, Log, "Total                : %8d", ModelTotal+StreamableData+ModeleResData);
		UE_LOGF(LogMutable, Log, "  Streamable         : %8d", StreamableData);
		UE_LOGF(LogMutable, Log, "  ModelResources     : %8d", ModeleResData);
		UE_LOGF(LogMutable, Log, "  Model              : %8d", ModelTotal);
		UE_LOGF(LogMutable, Log, "    ByteCode         : %8d", ByteCode);
		UE_LOGF(LogMutable, Log, "    Roms             : %8d", Roms);
		UE_LOGF(LogMutable, Log, "    Strings          : %8d", StringsTotal);
		UE_LOGF(LogMutable, Log, "    Names			   : %8d", ConstantNames);
		UE_LOGF(LogMutable, Log, "    UInt32		   : %8d", ConstantUInt32Lists);
		UE_LOGF(LogMutable, Log, "    Int32			   : %8d", ConstantInt32Lists);
		UE_LOGF(LogMutable, Log, "    UInt64		   : %8d", ConstantUInt64Lists);
		UE_LOGF(LogMutable, Log, "    Float			   : %8d", ConstantFloatLists);
		UE_LOGF(LogMutable, Log, "    Bool			   : %8d", ConstantBoolLists);
		UE_LOGF(LogMutable, Log, "    Images           : %8d", ImageTotal);
		UE_LOGF(LogMutable, Log, "      Buffer         : %8d", ConstantImages);
		UE_LOGF(LogMutable, Log, "      LODIndices     : %8d", ConstantImageLODIndices);
		UE_LOGF(LogMutable, Log, "      PermanentBuf   : %8d", ConstantImageLODPermanent);
		UE_LOGF(LogMutable, Log, "      Permanent      : %8d", PermanentImages);
		UE_LOGF(LogMutable, Log, "    Meshes           : %8d", MeshesTotal);
		UE_LOGF(LogMutable, Log, "      Buffer         : %8d", ConstantMeshes);
		UE_LOGF(LogMutable, Log, "      ContentIndices : %8d", ConstantMeshContentIndices);
		UE_LOGF(LogMutable, Log, "      PermanentBuf   : %8d", ConstantMeshesPermanent);
		UE_LOGF(LogMutable, Log, "      Permanent      : %8d", PermanentMeshes);
		UE_LOGF(LogMutable, Log, "    Skeletons        : %8d", SkeletonsTotal);
		UE_LOGF(LogMutable, Log, "    Sockets	       : %8d", SocketsTotal);
		UE_LOGF(LogMutable, Log, "    PhysicBodies     : %8d", PhysicsTotal);
		UE_LOGF(LogMutable, Log, "    Parameters       : %8d", ParametersTotal);
	}

}
#endif // WITH_EDITOR


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
bool FAnimBpOverridePhysicsAssetsInfo::operator==(const FAnimBpOverridePhysicsAssetsInfo& Rhs) const
{
	return AnimInstanceClass == Rhs.AnimInstanceClass &&
		PropertyIndex == Rhs.PropertyIndex;
}


bool FMutableModelImageProperties::operator!=(const FMutableModelImageProperties& Other) const
{
	return
		TextureParameterName != Other.TextureParameterName ||
		Filter != Other.Filter ||
		SRGB != Other.SRGB ||
		FlipGreenChannel != Other.FlipGreenChannel ||
		IsPassThrough != Other.IsPassThrough ||
		LODBias != Other.LODBias ||
		MipGenSettings != Other.MipGenSettings ||
		LODGroup != Other.LODGroup ||
		AddressX != Other.AddressX ||
		AddressY != Other.AddressY;
}


bool FMutableRefSocket::operator==(const FMutableRefSocket& Other) const
{
	return SocketName == Other.SocketName &&
		BoneName == Other.BoneName &&
		RelativeLocation == Other.RelativeLocation &&
		RelativeRotation == Other.RelativeRotation &&
		RelativeScale == Other.RelativeScale &&
		bForceAlwaysAnimated == Other.bForceAlwaysAnimated;
}


FIntegerParameterUIData::FIntegerParameterUIData(const FMutableParamUIMetadata& InParamUIMetadata)
{
	ParamUIMetadata = InParamUIMetadata;
}


FMutableParameterData::FMutableParameterData(const FMutableParamUIMetadata& InParamUIMetadata, EMutableParameterType InType)
{
	ParamUIMetadata = InParamUIMetadata;
	Type = InType;
}


uint32 GetTypeHash(const FTextureCache::FId& Key)
{
	return HashCombine(GetTypeHash(Key.Resource), Key.SkippedMips, Key.bIsBake);
}


#if WITH_EDITORONLY_DATA
FArchive& operator<<(FArchive& Ar, FAnimBpOverridePhysicsAssetsInfo& Info)
{
	FString AnimInstanceClassPathString;

	if (Ar.IsLoading())
	{
		Ar << AnimInstanceClassPathString;
		Ar << Info.PropertyIndex;

		Info.AnimInstanceClass = TSoftClassPtr<UAnimInstance>(AnimInstanceClassPathString);
	}

	if (Ar.IsSaving())
	{
		AnimInstanceClassPathString = Info.AnimInstanceClass.ToString();

		Ar << AnimInstanceClassPathString;
		Ar << Info.PropertyIndex;
	}

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableRefSocket& Data)
{
	Ar << Data.SocketName;
	Ar << Data.BoneName;
	Ar << Data.RelativeLocation;
	Ar << Data.RelativeRotation;
	Ar << Data.RelativeScale;
	Ar << Data.bForceAlwaysAnimated;

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableRefSkeletalMeshSettings& Data)
{
	Ar << Data.bEnablePerPolyCollision;
	Ar << Data.DefaultUVChannelDensity;

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableRefSkeletalMeshData& Data)
{
	Ar << Data.Sockets;
	Ar << Data.Bounds;
	Ar << Data.Settings;
	Ar << Data.SkeletalMesh;
	Ar << Data.SkeletalMeshLODSettings;
	Ar << Data.Skeleton;
	Ar << Data.ShadowPhysicsAsset;

	return Ar;
}
#endif 


#undef LOCTEXT_NAMESPACE

