// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectCustomVersion.h"

#include "Serialization/CustomVersion.h"

const FGuid FCustomizableObjectCustomVersion::GUID(0xC1270312, 0xAB730B1E, 0x0B1EC745, 0x71C70F73);

// Register the custom version with core
FCustomVersionRegistration GRegisterCustomizableObjectCustomVersion(FCustomizableObjectCustomVersion::GUID, FCustomizableObjectCustomVersion::LatestVersion, TEXT("CustomizableObjectVer"));
