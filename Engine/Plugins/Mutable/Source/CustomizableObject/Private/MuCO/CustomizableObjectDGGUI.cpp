// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectDGGUI.h"

#include "Kismet/GameplayStatics.h"
#include "Misc/ConfigCacheIni.h"
#include "MuCO/LoadUtils.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectDGGUI)

void UDGGUI::OpenDGGUI(const int32 SlotID, UCustomizableObjectInstanceUsage* SelectedCustomizableObjectInstanceUsage, const UWorld* CurrentWorld, const int32 PlayerIndex)
{
#if !UE_BUILD_SHIPPING
	if (APlayerController* Player = UGameplayStatics::GetPlayerController(CurrentWorld, PlayerIndex))
	{
		static FString DGGUIAssetPath;

		if (DGGUIAssetPath.IsEmpty())
		{
			FConfigFile* PluginConfig = GConfig->FindConfigFileWithBaseName("Mutable");
			if (PluginConfig)
			{
				PluginConfig->GetString(TEXT("EditorDefaults"), TEXT("DynamicallyGenerated_DGGUI_Path"), DGGUIAssetPath);
			}
		}

		FSoftClassPath DGUIPath(DGGUIAssetPath);
		if (UClass* DGUI = UE::Mutable::Private::LoadClass<UDGGUI>(DGUIPath))
		{
			UDGGUI* WDGUI = CreateWidget<UDGGUI>(Player, DGUI);
			if (WDGUI)
			{
				WDGUI->SetCustomizableObjectInstanceUsage(SelectedCustomizableObjectInstanceUsage);
				WDGUI->AddToViewport();
				Player->SetShowMouseCursor(true);
			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "Could not find the DynamicallyGenerated_DGGUI class inside the specified path. Check the DefaultMutable.ini file.");
		}
	}
#endif // !UE_BUILD_SHIPPING
}

bool UDGGUI::CloseExistingDGGUI(const UWorld* CurrentWorld)
{
#if !UE_BUILD_SHIPPING
	bool bClosing = false;
	for (TObjectIterator<UDGGUI> PreviousGUI; PreviousGUI; ++PreviousGUI)
	{
		if (PreviousGUI->IsValidLowLevel())
		{
			if (UCustomizableObjectInstanceUsage* CustomizableObjectInstanceUsage = PreviousGUI->GetCustomizableObjectInstanceUsage())
			{
				PreviousGUI->SetCustomizableObjectInstanceUsage(nullptr);
				bClosing = true;
			}
			PreviousGUI->RemoveFromParent();
		}
	}
	if (bClosing)
	{
		if (APlayerController* Player = UGameplayStatics::GetPlayerController(CurrentWorld, 0))
		{
			Player->SetShowMouseCursor(false);
		}
		return true;
	}
#endif // !UE_BUILD_SHIPPING
	return false;
}
