// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"

#include "CustomizableObjectDGGUI.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UAnimInstance;
class UCustomizableObjectInstanceUsage;
class UObject;
class UWorld;


UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable)
class UDGGUI : public UUserWidget
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintImplementableEvent, Category = "DGGUI")
	UE_API class UCustomizableObjectInstanceUsage* GetCustomizableObjectInstanceUsage();

	UFUNCTION(BlueprintImplementableEvent, Category = "DGGUI")
	UE_API void SetCustomizableObjectInstanceUsage(class UCustomizableObjectInstanceUsage* CustomizableObjectInstanceUsage);

	static UE_API void OpenDGGUI(const int32 SlotID, UCustomizableObjectInstanceUsage* SelectedCustomizableObjectInstanceUsage, const UWorld* CurrentWorld, const int32 PlayerIndex = 0);
	static UE_API bool CloseExistingDGGUI(const UWorld* CurrentWorld);
};

#undef UE_API
