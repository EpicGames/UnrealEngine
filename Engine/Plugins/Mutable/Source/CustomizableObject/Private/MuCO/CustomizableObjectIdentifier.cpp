// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectIdentifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectIdentifier)


FCustomizableObjectIdPair::FCustomizableObjectIdPair(FString ObjectGroupName, FString ObjectName)
	: CustomizableObjectGroupName(ObjectGroupName)
	, CustomizableObjectName(ObjectName)
{
}

