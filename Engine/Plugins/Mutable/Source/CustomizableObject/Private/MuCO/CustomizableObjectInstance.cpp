// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectInstance.h"

#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Animation/Skeleton.h"
#include "BoneControllers/AnimNode_RigidBody.h"
#include "ClothConfig.h"
#include "ClothingAsset.h"
#include "MuCO/CustomizableObjectInstanceUsagePrivate.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkeletalMeshLODSettings.h"
#include "MaterialDomain.h"
#include "MutableStreamRequest.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "Tasks/Task.h"

#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/CustomizableObjectSkeletalMesh.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "MuCO/CustomizableObjectMipDataProvider.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/CustomizableObjectInstanceAssetUserData.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCO/UnrealConversionUtils.h"
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuCO/LogBenchmarkUtil.h"

#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/Texture2DResource.h"
#include "RenderingThread.h"
#include "SkeletalMergingLibrary.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/LoadUtils.h"
#include "Serialization/BulkData.h"

#include "Rendering/SkeletalMeshModel.h"

#include "MuR/MutableEditorLogger.h"
#include "MuR/PassthroughObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectInstance)

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "MessageLogModule.h"
#include "UnrealEdMisc.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Application/ThrottleManager.h"
#endif

namespace
{
#ifndef REQUIRES_SINGLEUSE_FLAG_FOR_RUNTIME_TEXTURES
	#define REQUIRES_SINGLEUSE_FLAG_FOR_RUNTIME_TEXTURES !PLATFORM_DESKTOP
#endif


bool bDisableNotifyComponentsOfTextureUpdates = false;
static FAutoConsoleVariableRef CVarDisableNotifyComponentsOfTextureUpdates(
	TEXT("mutable.DisableNotifyComponentsOfTextureUpdates"),
	bDisableNotifyComponentsOfTextureUpdates,
	TEXT("If set to true, disables Mutable notifying the streaming system that a component has had a change in at least one texture of its components."),
	ECVF_Default);

}

const FString MULTILAYER_PROJECTOR_PARAMETERS_INVALID = TEXT("Invalid Multilayer Projector Parameters.");

const FString NUM_LAYERS_PARAMETER_POSTFIX = FString("_NumLayers");
const FString OPACITY_PARAMETER_POSTFIX = FString("_Opacity");
const FString IMAGE_PARAMETER_POSTFIX = FString("_SelectedImages");
const FString POSE_PARAMETER_POSTFIX = FString("_SelectedPoses");


UTexture2D* CreateTexture(const FName& TextureName)
{
	UTexture2D* NewTexture = NewObject<UTexture2D>(
		GetTransientPackage(),
		TextureName,
		RF_Transient
		);
	UCustomizableObjectSystem::GetInstance()->GetPrivate()->LogBenchmarkUtil.AddTexture(*NewTexture);
	NewTexture->SetPlatformData( nullptr );

	return NewTexture;
}


void UCustomizableInstancePrivate::InvalidateGeneratedData()
{
	SkeletalMeshStatus = ESkeletalMeshStatus::NotGenerated;

#if WITH_EDITOR
	if (const ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
	{
		for (const TPair<FName, TObjectPtr<USkeletalMesh>>& SkeletalMeshEntry : SkeletalMeshes)
		{
			EditorModule->EvictSkeletalMeshThumbnailScene(SkeletalMeshEntry.Value);
		}
	}
#endif

	SkeletalMeshes.Empty();
	Materials.Empty();
	Textures.Empty();
	ExtensionData.Empty();
	
	CommittedDescriptor = {};
	CommittedDescriptorHash = {};

	ComponentsData.Empty();

	if (UCustomizableObjectInstance* Instance = GetPublic())
	{
		UCustomizableObjectSystemPrivate::UnregisterInstancefromGeneratedList(*Instance);
		UCustomizableObjectSystemPrivate::RegisterInstanceToAutomaticUpdateList(*Instance);
	}
}


FCustomizableInstanceComponentData* UCustomizableInstancePrivate::GetComponentData(const FName& ComponentName)
{
	return ComponentsData.FindByPredicate([&](const FCustomizableInstanceComponentData& ComponentData)
	{
		return ComponentData.ComponentName == ComponentName;
	});
}


FCustomizableInstanceComponentData& UCustomizableInstancePrivate::GetComponentData(UE::Mutable::Private::FComponentId ComponentId)
{
	FCustomizableInstanceComponentData* Result = ComponentsData.FindByPredicate([&](const FCustomizableInstanceComponentData& ComponentData)
	{
		return ComponentData.ComponentId == ComponentId;
	});
	
	check(Result);
	return *Result;
}


UCustomizableObjectInstance::UCustomizableObjectInstance()
{
	SetFlags(RF_Transactional);
	PrivateData = CreateDefaultSubobject<UCustomizableInstancePrivate>(TEXT("Private"));
}


void UCustomizableInstancePrivate::CopyParametersFromInstance(UCustomizableObjectInstance* Instance)
{
	if (!Instance)
	{
		return;
	}
	
	FCustomizableObjectInstanceDescriptor& Descriptor = Instance->GetPrivate()->GetDescriptor();
	
	UCustomizableObject* InCustomizableObject = Descriptor.GetCustomizableObject();

#if WITH_EDITOR
	// Bind a lambda to the PostCompileDelegate and unbind from the previous object if any.
	BindObjectDelegates(GetPublic()->GetCustomizableObject(), Descriptor.GetCustomizableObject());
#endif

	GetPublic()->Descriptor = Descriptor;
}


uint32 GetTypeHash(const FMaterialReuseCacheKey& Key)
{
	uint32 Hash = GetTypeHash(Key.MaterialTemplate);

	Hash = HashCombine(Hash, Key.Root);
	
	return Hash;
}

uint32 GetTypeHash(const FTextureReuseCacheKey& Key)
{
	return Key.Root;
}


#if WITH_EDITOR
void UCustomizableInstancePrivate::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// Invalidate all generated data to avoid modifying resources shared between CO instances.
	InvalidateGeneratedData();
}


void UCustomizableInstancePrivate::OnPostCompile()
{
	GetDescriptor().ReloadParameters();
	InvalidateGeneratedData();
}


void UCustomizableInstancePrivate::OnObjectStatusChanged(FCustomizableObjectStatus::EState Previous, FCustomizableObjectStatus::EState Next)
{
	if (Previous != Next && Next == FCustomizableObjectStatus::EState::ModelLoaded)
	{
		OnPostCompile();
	}
}


void UCustomizableInstancePrivate::BindObjectDelegates(UCustomizableObject*  CurrentCustomizableObject, UCustomizableObject* NewCustomizableObject)
{
	if (CurrentCustomizableObject == NewCustomizableObject)
	{
		return;
	}

	// Unbind callback from the previous CO
	if (CurrentCustomizableObject)
	{
		CurrentCustomizableObject->GetPrivate()->Status.GetOnStateChangedDelegate().RemoveAll(this);
	}

	// Bind callback to the new CO
	if (NewCustomizableObject)
	{
		NewCustomizableObject->GetPrivate()->Status.GetOnStateChangedDelegate().AddUObject(this, &UCustomizableInstancePrivate::OnObjectStatusChanged);
	}
}


bool UCustomizableObjectInstance::CanEditChange(const FProperty* InProperty) const
{
	bool bIsMutable = Super::CanEditChange(InProperty);
	if (bIsMutable && InProperty != NULL)
	{
		if (InProperty->GetFName() == TEXT("CustomizationObject"))
		{
			bIsMutable = false;
		}

		if (InProperty->GetFName() == TEXT("ParameterName"))
		{
			bIsMutable = false;
		}
	}

	return bIsMutable;
}

void UCustomizableObjectInstance::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		UpdateSkeletalMeshAsync(true, true);
	}

	GetPrivate()->OnInstanceTransactedDelegate.Broadcast(TransactionEvent);
}

#endif // WITH_EDITOR


bool UCustomizableObjectInstance::IsEditorOnly() const
{
	if (UCustomizableObject* CustomizableObject = GetCustomizableObject())
	{
		return CustomizableObject->IsEditorOnly();
	}
	return false;
}

void UCustomizableObjectInstance::PostInitProperties()
{
	Super::PostInitProperties();
	UCustomizableObjectSystemPrivate::RegisterCustomizableObjectInstance(*this);
}


void UCustomizableObjectInstance::BeginDestroy()
{
	if (PrivateData)
	{
#if WITH_EDITOR
		// Unbind Object delegates
		PrivateData->BindObjectDelegates(GetCustomizableObject(), nullptr);
#endif
	}

	UCustomizableObjectSystemPrivate::UnregisterCustomizableObjectInstance(*this);
	Super::BeginDestroy();
}


void UCustomizableObjectInstance::DestroyLiveUpdateInstance()
{
	GetPrivate()->LiveInstance.Reset();
}


void UCustomizableObjectInstance::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	const int32 CustomizableObjectCustomVersion = GetLinkerCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (CustomizableObjectCustomVersion < FCustomizableObjectCustomVersion::GroupProjectorIntToScalarIndex)
	{
		TArray<int32> IntParametersToMove;

		// Find the num layer parameters that were int enums
		for (int32 i = 0; i < IntParameters_DEPRECATED.Num(); ++i)
		{
			if (IntParameters_DEPRECATED[i].ParameterName.EndsWith(NUM_LAYERS_PARAMETER_POSTFIX, ESearchCase::CaseSensitive))
			{
				FString ParameterNamePrefix, Aux;
				const bool bSplit = IntParameters_DEPRECATED[i].ParameterName.Split(NUM_LAYERS_PARAMETER_POSTFIX, &ParameterNamePrefix, &Aux);
				check(bSplit);

				// Confirm this is actually a multilayer param by finding the corresponding pose param
				for (int32 j = 0; j < IntParameters_DEPRECATED.Num(); ++j)
				{
					if (i != j)
					{
						if (IntParameters_DEPRECATED[j].ParameterName.StartsWith(ParameterNamePrefix, ESearchCase::CaseSensitive) &&
							IntParameters_DEPRECATED[j].ParameterName.EndsWith(POSE_PARAMETER_POSTFIX, ESearchCase::CaseSensitive))
						{
							IntParametersToMove.Add(i);
							break;
						}
					}
				}
			}
		}

		// Convert them to float params
		for (int32 i = 0; i < IntParametersToMove.Num(); ++i)
		{
			FloatParameters_DEPRECATED.AddDefaulted();
			FloatParameters_DEPRECATED.Last().ParameterName = IntParameters_DEPRECATED[IntParametersToMove[i]].ParameterName;
			FloatParameters_DEPRECATED.Last().ParameterValue = FCString::Atoi(*IntParameters_DEPRECATED[IntParametersToMove[i]].ParameterValueName);
			FloatParameters_DEPRECATED.Last().Id = IntParameters_DEPRECATED[IntParametersToMove[i]].Id;
		}

		// Remove them from the int params in reverse order
		for (int32 i = IntParametersToMove.Num() - 1; i >= 0; --i)
		{
			IntParameters_DEPRECATED.RemoveAt(IntParametersToMove[i]);
		}
	}
	
	if (CustomizableObjectCustomVersion < FCustomizableObjectCustomVersion::CustomizableObjectInstanceDescriptor)
	{
		Descriptor.CustomizableObject = CustomizableObject_DEPRECATED;

		Descriptor.BoolParameters = BoolParameters_DEPRECATED;
		Descriptor.IntParameters = IntParameters_DEPRECATED;
		Descriptor.FloatParameters = FloatParameters_DEPRECATED;
		Descriptor.TextureParameters = TextureParameters_DEPRECATED;
		Descriptor.VectorParameters = VectorParameters_DEPRECATED;
		Descriptor.ProjectorParameters = ProjectorParameters_DEPRECATED;
	}
}


void UCustomizableObjectInstance::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PrivateData->BindObjectDelegates(nullptr, GetCustomizableObject());
#endif

	// Skip the cost of ReloadParameters in the cook commandlet; it will be reloaded during PreSave. For cooked runtime
	// and editor UI, reload on load because it will not otherwise reload unless the CustomizableObject recompiles.
	Descriptor.ReloadParameters();
	GetPrivate()->InvalidateGeneratedData();
}


FString UCustomizableObjectInstance::GetDesc()
{
	FString ObjectName = "Missing Object";
	if (UCustomizableObject* CustomizableObject = GetCustomizableObject())
	{
		ObjectName = CustomizableObject->GetName();
	}

	return FString::Printf(TEXT("Instance of [%s]"), *ObjectName);
}


int32 UCustomizableObjectInstance::GetProjectorValueRange(const FString& ParamName) const
{
	return Descriptor.GetProjectorValueRange(ParamName);
}


int32 UCustomizableObjectInstance::GetIntValueRange(const FString& ParamName) const
{
	return Descriptor.GetIntValueRange(ParamName);
}


int32 UCustomizableObjectInstance::GetFloatValueRange(const FString& ParamName) const
{
	return Descriptor.GetFloatValueRange(ParamName);
}


int32 UCustomizableObjectInstance::GetTextureValueRange(const FString& ParamName) const
{
	return Descriptor.GetTextureValueRange(ParamName);
}


void UCustomizableObjectInstance::SetObject(UCustomizableObject* InObject)
{
#if WITH_EDITOR
	// Bind a lambda to the PostCompileDelegate and unbind from the previous object if any.
	PrivateData->BindObjectDelegates(GetCustomizableObject(), InObject);
#endif

	Descriptor.SetCustomizableObject(InObject);
}


UCustomizableObject* UCustomizableObjectInstance::GetCustomizableObject() const
{
	return Descriptor.CustomizableObject;
}


bool UCustomizableObjectInstance::GetBuildParameterRelevancy() const
{
	return Descriptor.GetBuildParameterRelevancy();
}


void UCustomizableObjectInstance::SetBuildParameterRelevancy(bool Value)
{
	Descriptor.SetBuildParameterRelevancy(Value);
}


int32 UCustomizableInstancePrivate::GetState() const
{
	return GetPublic()->Descriptor.GetState();
}


void UCustomizableInstancePrivate::SetState(const int32 InState)
{
	GetPublic()->Descriptor.SetState(InState);
}


FString UCustomizableObjectInstance::GetCurrentState() const
{
	return Descriptor.GetCurrentState();
}


void UCustomizableObjectInstance::SetCurrentState(const FString& StateName)
{
	Descriptor.SetCurrentState(StateName);
}


void UCustomizableInstancePrivate::RegisterCustomizableObjectInstanceUsage(UCustomizableObjectInstanceUsage& Obj)
{
	if (!Obj.IsTemplate() && !InstanceUsages.Contains(&Obj))
	{
		InstanceUsages.Add(&Obj);

		UCustomizableObjectInstance* Instance = GetPublic();
		UCustomizableObjectSystemPrivate::RegisterInstanceToAutomaticUpdateList(*Instance);

		if (!Obj.GetSkipSetReferenceSkeletalMesh())
		{
			UCustomizableObjectSystemPrivate::RegisterInstanceToPendinSetReferenceSkeletalMeshList(*Instance);
		}
	}
}


void UCustomizableInstancePrivate::UnregisterCustomizableObjectInstanceUsage(UCustomizableObjectInstanceUsage& Obj)
{
	if (!Obj.IsTemplate())
	{
		InstanceUsages.Remove(&Obj);

		if (InstanceUsages.IsEmpty())
		{
			UCustomizableObjectSystemPrivate::UnregisterInstanceFromAutomaticUpdateList(*GetPublic());
		}
	}
}


bool UCustomizableInstancePrivate::IsParameterRelevant(uint32 ParameterIndex) const
{
	// This should have been precalculated in the last update if the appropriate flag in the instance was set.
	return RelevantParameters.Contains(ParameterIndex);
}


bool UCustomizableObjectInstance::IsParameterRelevant(const FString& ParamName) const
{
	UCustomizableObject* CustomizableObject = GetCustomizableObject();

	if (!CustomizableObject)
	{
		return false;
	}

	// This should have been precalculated in the last update if the appropriate flag in the instance was set.
	int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(ParamName);
	return GetPrivate()->RelevantParameters.Contains(ParameterIndexInObject);
}


bool UCustomizableObjectInstance::IsParameterDirty(const FString& ParamName, const int32 RangeIndex) const
{
	switch (Descriptor.CustomizableObject->GetParameterTypeByName(ParamName))
	{
	case EMutableParameterType::None:
		return false;

	case EMutableParameterType::Projector:
		{
			const FCustomizableObjectProjectorParameterValue* Result = Descriptor.ProjectorParameters.FindByPredicate([&](const FCustomizableObjectProjectorParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			const FCustomizableObjectProjectorParameterValue* ResultCommited = GetPrivate()->CommittedDescriptor.ProjectorParameters.FindByPredicate([&](const FCustomizableObjectProjectorParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			if (Result && ResultCommited)
			{
				if (RangeIndex == INDEX_NONE)
				{
					return Result->Value == ResultCommited->Value;					
				}
				else
				{
					if (Result->RangeValues.IsValidIndex(RangeIndex) && ResultCommited->RangeValues.IsValidIndex(RangeIndex))
					{
						return Result->RangeValues[RangeIndex] == ResultCommited->RangeValues[RangeIndex];
					}
					else
					{
						return Result->RangeValues.Num() != ResultCommited->RangeValues.Num();
					}
				}
			}
			else
			{
				return Result != ResultCommited;
			}
		}		
	case EMutableParameterType::Texture:
		{
			const FCustomizableObjectTextureParameterValue* Result = Descriptor.TextureParameters.FindByPredicate([&](const FCustomizableObjectTextureParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			const FCustomizableObjectTextureParameterValue* ResultCommited = GetPrivate()->CommittedDescriptor.TextureParameters.FindByPredicate([&](const FCustomizableObjectTextureParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			if (Result && ResultCommited)
			{
				if (RangeIndex == INDEX_NONE)
				{
					return Result->ParameterValue == ResultCommited->ParameterValue;					
				}
				else
				{
					if (Result->ParameterRangeValues.IsValidIndex(RangeIndex) && ResultCommited->ParameterRangeValues.IsValidIndex(RangeIndex))
					{
						return Result->ParameterRangeValues[RangeIndex] == ResultCommited->ParameterRangeValues[RangeIndex];
					}
					else
					{
						return Result->ParameterRangeValues.Num() != ResultCommited->ParameterRangeValues.Num();
					}
				}
			}
			else
			{
				return Result != ResultCommited;
			}
		}

	case EMutableParameterType::Bool:
		{
			const FCustomizableObjectBoolParameterValue* Result = Descriptor.BoolParameters.FindByPredicate([&](const FCustomizableObjectBoolParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			const FCustomizableObjectBoolParameterValue* ResultCommited = GetPrivate()->CommittedDescriptor.BoolParameters.FindByPredicate([&](const FCustomizableObjectBoolParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			if (Result && ResultCommited)
			{
				if (RangeIndex == INDEX_NONE)
				{
					return Result->ParameterValue == ResultCommited->ParameterValue;					
				}
				else
				{
					return false;
				}
			}
			else
			{
				return Result != ResultCommited;
			}
		}
	case EMutableParameterType::Int:
		{
			const FCustomizableObjectIntParameterValue* Result = Descriptor.IntParameters.FindByPredicate([&](const FCustomizableObjectIntParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			const FCustomizableObjectIntParameterValue* ResultCommited = GetPrivate()->CommittedDescriptor.IntParameters.FindByPredicate([&](const FCustomizableObjectIntParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			if (Result && ResultCommited)
			{
				if (RangeIndex == INDEX_NONE)
				{
					return Result->ParameterValueName == ResultCommited->ParameterValueName;					
				}
				else
				{
					if (Result->ParameterRangeValueNames.IsValidIndex(RangeIndex) && ResultCommited->ParameterRangeValueNames.IsValidIndex(RangeIndex))
					{
						return Result->ParameterRangeValueNames[RangeIndex] == ResultCommited->ParameterRangeValueNames[RangeIndex];
					}
					else
					{
						return Result->ParameterRangeValueNames.Num() != ResultCommited->ParameterRangeValueNames.Num();
					}
				}
			}
			else
			{
				return Result != ResultCommited;
			}
		}
		
	case EMutableParameterType::Float:
		{
			const FCustomizableObjectFloatParameterValue* Result = Descriptor.FloatParameters.FindByPredicate([&](const FCustomizableObjectFloatParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			const FCustomizableObjectFloatParameterValue* ResultCommited = GetPrivate()->CommittedDescriptor.FloatParameters.FindByPredicate([&](const FCustomizableObjectFloatParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			if (Result && ResultCommited)
			{
				if (RangeIndex == INDEX_NONE)
				{
					return Result->ParameterValue == ResultCommited->ParameterValue;					
				}
				else
				{
					if (Result->ParameterRangeValues.IsValidIndex(RangeIndex) && ResultCommited->ParameterRangeValues.IsValidIndex(RangeIndex))
					{
						return Result->ParameterRangeValues[RangeIndex] == ResultCommited->ParameterRangeValues[RangeIndex];
					}
					else
					{
						return Result->ParameterRangeValues.Num() != ResultCommited->ParameterRangeValues.Num();
					}
				}
			}
			else
			{
				return Result != ResultCommited;
			}
		}
		
	case EMutableParameterType::Color:
		{
			const FCustomizableObjectVectorParameterValue* Result = Descriptor.VectorParameters.FindByPredicate([&](const FCustomizableObjectVectorParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			const FCustomizableObjectVectorParameterValue* ResultCommited = GetPrivate()->CommittedDescriptor.VectorParameters.FindByPredicate([&](const FCustomizableObjectVectorParameterValue& Value)
				{
					return Value.ParameterName == ParamName;
				});
			
			if (Result && ResultCommited)
			{
				if (RangeIndex == INDEX_NONE)
				{
					return Result->ParameterValue == ResultCommited->ParameterValue;					
				}
				else
				{
					return false;
				}
			}
			else
			{
				return Result != ResultCommited;
			}
		}
		
	default:
		unimplemented();
		return false;
	}
}


void UCustomizableInstancePrivate::PostEditChangePropertyWithoutEditor()
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::PostEditChangePropertyWithoutEditor);

	for (TTuple<FName, TObjectPtr<USkeletalMesh>>& Tuple : SkeletalMeshes)
	{
		USkeletalMesh* SkeletalMesh = Tuple.Get<1>();
		
		if (SkeletalMesh && SkeletalMesh->GetResourceForRendering() && !SkeletalMesh->GetResourceForRendering()->IsInitialized())
		{
			MUTABLE_CPUPROFILER_SCOPE(InitResources);

			// reinitialize resources
			SkeletalMesh->InitResources();
		}
	}
}


void UCustomizableObjectInstance::UpdateSkeletalMeshAsync(bool, bool bForceHighPriority)
{
	UCustomizableObjectSystemPrivate* SystemPrivate = UCustomizableObjectSystem::GetInstance()->GetPrivate();

	const TSharedRef<FUpdateContextPrivate> Context = MakeShared<FUpdateContextPrivate>();
	Context->Instance = this;
	Context->bIsHighPriority = bForceHighPriority;
	
	SystemPrivate->EnqueueUpdateSkeletalMesh(Context);
}


void UCustomizableObjectInstance::UpdateSkeletalMeshAsyncResult(FInstanceUpdateDelegate Callback, bool, bool bForceHighPriority)
{
	UCustomizableObjectSystemPrivate* SystemPrivate = UCustomizableObjectSystem::GetInstance()->GetPrivate();

	const TSharedRef<FUpdateContextPrivate> Context = MakeShared<FUpdateContextPrivate>();
	Context->Instance = this;
	Context->bIsHighPriority = bForceHighPriority;
	Context->UpdateCallback = Callback;
	
	SystemPrivate->EnqueueUpdateSkeletalMesh(Context);
}


void UCustomizableObjectInstance::UpdateSkeletalMeshAsyncResult(FInstanceUpdateNativeDelegate Callback, bool, bool bForceHighPriority)
{
	UCustomizableObjectSystemPrivate* SystemPrivate = UCustomizableObjectSystem::GetInstance()->GetPrivate();

	const TSharedRef<FUpdateContextPrivate> Context = MakeShared<FUpdateContextPrivate>();
	Context->Instance = this;
	Context->bIsHighPriority = bForceHighPriority;
	Context->UpdateNativeCallback = Callback;
	
	SystemPrivate->EnqueueUpdateSkeletalMesh(Context);
}


#if !UE_BUILD_SHIPPING
bool AreSkeletonsCompatible(const TArray<TObjectPtr<USkeleton>>& InSkeletons)
{
	MUTABLE_CPUPROFILER_SCOPE(AreSkeletonsCompatible);

	if (InSkeletons.IsEmpty())
	{
		return true;
	}

	bool bCompatible = true;

	struct FBoneToMergeInfo
	{
		FBoneToMergeInfo(const uint32 InBonePathHash, const uint32 InSkeletonIndex, const uint32 InParentBoneSkeletonIndex) :
		BonePathHash(InBonePathHash), SkeletonIndex(InSkeletonIndex), ParentBoneSkeletonIndex(InParentBoneSkeletonIndex)
		{}

		uint32 BonePathHash = 0;
		uint32 SkeletonIndex = 0;
		uint32 ParentBoneSkeletonIndex = 0;
	};

	// Accumulated hierarchy hash from parent-bone to root bone
	TMap<FName, FBoneToMergeInfo> BoneNamesToBoneInfo;
	BoneNamesToBoneInfo.Reserve(InSkeletons[0] ? InSkeletons[0]->GetReferenceSkeleton().GetNum() : 0);
	
	for (int32 SkeletonIndex = 0; SkeletonIndex < InSkeletons.Num(); ++SkeletonIndex)
	{
		const TObjectPtr<USkeleton> Skeleton = InSkeletons[SkeletonIndex];
		check(Skeleton);

		const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
		const TArray<FMeshBoneInfo>& Bones = ReferenceSkeleton.GetRawRefBoneInfo();

		const int32 NumBones = Bones.Num();
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FMeshBoneInfo& Bone = Bones[BoneIndex];

			// Retrieve parent bone name and respective hash, root-bone is assumed to have a parent hash of 0
			const FName ParentName = Bone.ParentIndex != INDEX_NONE ? Bones[Bone.ParentIndex].Name : NAME_None;
			const uint32 ParentHash = Bone.ParentIndex != INDEX_NONE ? GetTypeHash(ParentName) : 0;

			// Look-up the path-hash from root to the parent bone
			const FBoneToMergeInfo* ParentBoneInfo = BoneNamesToBoneInfo.Find(ParentName);
			const uint32 ParentBonePathHash = ParentBoneInfo ? ParentBoneInfo->BonePathHash : 0;
			const uint32 ParentBoneSkeletonIndex = ParentBoneInfo ? ParentBoneInfo->SkeletonIndex : 0;

			// Append parent hash to path to give full path hash to current bone
			const uint32 BonePathHash = HashCombine(ParentBonePathHash, ParentHash);

			// Check if the bone exists in the hierarchy 
			const FBoneToMergeInfo* ExistingBoneInfo = BoneNamesToBoneInfo.Find(Bone.Name);

			// If the hash differs from the existing one it means skeletons are incompatible
			if (!ExistingBoneInfo)
			{
				// Add path hash to current bone
				BoneNamesToBoneInfo.Add(Bone.Name, FBoneToMergeInfo(BonePathHash, SkeletonIndex, ParentBoneSkeletonIndex));
			}
			else if (ExistingBoneInfo->BonePathHash != BonePathHash)
			{
				if (bCompatible)
				{
					// Print the skeletons to merge
					FString Msg = TEXT("Failed to merge skeletons. Skeletons to merge: ");
					for (int32 AuxSkeletonIndex = 0; AuxSkeletonIndex < InSkeletons.Num(); ++AuxSkeletonIndex)
					{
						if (InSkeletons[AuxSkeletonIndex] != nullptr)
						{
							Msg += FString::Printf(TEXT("\n\t- %s"), *InSkeletons[AuxSkeletonIndex].GetName());
						}
					}

					UE_LOGF(LogMutable, Error, "%ls", *Msg);

#if WITH_EDITOR
					FNotificationInfo Info(FText::FromString(TEXT("Mutable: Failed to merge skeletons. Invalid parent chain detected. Please check the output log for more information.")));
					Info.bFireAndForget = true;
					Info.FadeOutDuration = 1.0f;
					Info.ExpireDuration = 10.0f;
					FSlateNotificationManager::Get().AddNotification(Info);
#endif

					bCompatible = false;
				}
				
				// Print the first non compatible bone in the bone chain, since all child bones will be incompatible too.
				if (ExistingBoneInfo->ParentBoneSkeletonIndex != SkeletonIndex)
				{
					// Different skeletons can't be used if they are incompatible with the reference skeleton.
					UE_LOGF(LogMutable, Error, "[%ls] parent bone is different in skeletons [%ls] and [%ls].",
						*Bone.Name.ToString(),
						*InSkeletons[SkeletonIndex]->GetName(),
						*InSkeletons[ExistingBoneInfo->ParentBoneSkeletonIndex]->GetName());
				}
			}
		}
	}

	return bCompatible;
}
#endif


USkeleton* MergeSkeletons(const TArray<TObjectPtr<USkeleton>>& SkeletonsToMerge, UCustomizableObject& CustomizableObject, bool& bOutCreatedNewSkeleton)
{
	MUTABLE_CPUPROFILER_SCOPE(BuildSkeletonData_MergeSkeletons);
	bOutCreatedNewSkeleton = false;
	
	if (SkeletonsToMerge.Contains(nullptr))
	{
		return nullptr;
	}

	// No need to merge skeletons
	if (SkeletonsToMerge.Num() == 1)
	{
		check(IsValid(SkeletonsToMerge[0]));
		return SkeletonsToMerge[0];
	}

	TArray<FName> SkeletonIds;
	SkeletonIds.Reserve(SkeletonsToMerge.Num());

	for (const TObjectPtr<USkeleton>& Skeleton : SkeletonsToMerge)
	{
		SkeletonIds.Add(Skeleton->GetFName());
	}

	if (USkeleton* Skeleton = CustomizableObject.GetPrivate()->SkeletonCache.Get(SkeletonIds))
	{
		// Merged skeleton found in the cache
		if (IsValid(Skeleton))
		{
			return Skeleton;
		}
	}

#if !UE_BUILD_SHIPPING
	// Test Skeleton compatibility before attempting the merge to avoid a crash.
	if (!AreSkeletonsCompatible(SkeletonsToMerge))
	{
		return nullptr;
	}
#endif

	FSkeletonMergeParams Params;
	Params.SkeletonsToMerge = SkeletonsToMerge;

	USkeleton* FinalSkeleton = USkeletalMergingLibrary::MergeSkeletons(Params);
	if (!FinalSkeleton)
	{
		FString Msg = FString::Printf(TEXT("MergeSkeletons failed for Customizable Object [%s]. Skeletons involved: "), *CustomizableObject.GetName());
		
		const int32 SkeletonCount = Params.SkeletonsToMerge.Num();
		for (int32 SkeletonIndex = 0; SkeletonIndex < SkeletonCount; ++SkeletonIndex)
		{
			Msg += FString::Printf(TEXT(" [%s]"), *Params.SkeletonsToMerge[SkeletonIndex]->GetName());
		}
		
		UE_LOGF(LogMutable, Error, "%ls", *Msg);
	}
	else
	{
#if WITH_EDITOR
		uint32 CombinedSkeletonHash = INDEX_NONE;
#endif
		
		// Make the final skeleton compatible with all the merged skeletons and their compatible skeletons.
		for (USkeleton* Skeleton : Params.SkeletonsToMerge)
		{
			if (Skeleton)
			{
				FinalSkeleton->AddCompatibleSkeleton(Skeleton);

				const TArray<TSoftObjectPtr<USkeleton>>& CompatibleSkeletons = Skeleton->GetCompatibleSkeletons();
				for (const TSoftObjectPtr<USkeleton>& CompatibleSkeleton : CompatibleSkeletons)
				{
					FinalSkeleton->AddCompatibleSkeletonSoft(CompatibleSkeleton);
				}

#if WITH_EDITOR
				const uint32 SkeletonHash = GetTypeHash(Skeleton->GetName());
				CombinedSkeletonHash = HashCombine(CombinedSkeletonHash, SkeletonHash);
#endif
			}
		}

		// Add the hash based on the sources for the merged skeleton to make its name unique
#if WITH_EDITOR
		FinalSkeleton->Rename(*FString::Printf(TEXT("%s_%lu"), *FinalSkeleton->GetName(), CombinedSkeletonHash));
#endif
		
		// Add Skeleton to the cache
		CustomizableObject.GetPrivate()->SkeletonCache.Add(SkeletonIds, FinalSkeleton);

		bOutCreatedNewSkeleton = true;
	}
	
	return FinalSkeleton;
}

namespace
{
	FORCEINLINE TObjectPtr<UPhysicsConstraintTemplate> ClonePhysicsConstraintTemplate(
			const TObjectPtr<UPhysicsConstraintTemplate>& From, 
			const TObjectPtr<UObject>& Outer,
			FName Name = NAME_None)
	{
		// We don't use DuplicateObject here beacuse it is too slow.
		TObjectPtr<UPhysicsConstraintTemplate> Result = NewObject<UPhysicsConstraintTemplate>(Outer, Name);
		
		Result->DefaultInstance = From->DefaultInstance;
		Result->ProfileHandles = From->ProfileHandles;
#if WITH_EDITOR
		Result->SetDefaultProfile(From->DefaultInstance);
#endif

		return Result;
	}

	FKAggregateGeom MakeAggGeomFromMutablePhysics(int32 BodyIndex, const UE::Mutable::Private::FPhysicsBody* MutablePhysicsBody)
	{
		FKAggregateGeom BodyAggGeom;

		auto GetCollisionEnabledFormFlags = [](uint32 Flags) -> ECollisionEnabled::Type
		{
			return ECollisionEnabled::Type(Flags & 0xFF);
		};

		auto GetContributeToMassFromFlags = [](uint32 Flags) -> bool
		{
			return static_cast<bool>((Flags >> 8) & 1);
		};

		const int32 NumSpheres = MutablePhysicsBody->GetSphereCount(BodyIndex);
		TArray<FKSphereElem>& AggSpheres = BodyAggGeom.SphereElems;
		AggSpheres.Empty(NumSpheres);
		for (int32 I = 0; I < NumSpheres; ++I)
		{
			uint32 Flags = MutablePhysicsBody->GetSphereFlags(BodyIndex, I);
			FString Name = MutablePhysicsBody->GetSphereName(BodyIndex, I);

			FVector3f Position;
			float Radius;

			MutablePhysicsBody->GetSphere(BodyIndex, I, Position, Radius);
			FKSphereElem& NewElem = AggSpheres.AddDefaulted_GetRef();
			
			NewElem.Center = FVector(Position);
			NewElem.Radius = Radius;
			NewElem.SetContributeToMass(GetContributeToMassFromFlags(Flags));
			NewElem.SetCollisionEnabled(GetCollisionEnabledFormFlags(Flags));
			NewElem.SetName(FName(*Name));
		}
		
		const int32 NumBoxes = MutablePhysicsBody->GetBoxCount(BodyIndex);
		TArray<FKBoxElem>& AggBoxes = BodyAggGeom.BoxElems;
		AggBoxes.Empty(NumBoxes);
		for (int32 I = 0; I < NumBoxes; ++I)
		{
			uint32 Flags = MutablePhysicsBody->GetBoxFlags(BodyIndex, I);
			FString Name = MutablePhysicsBody->GetBoxName(BodyIndex, I);

			FVector3f Position;
			FQuat4f Orientation
				;
			FVector3f Size;
			MutablePhysicsBody->GetBox(BodyIndex, I, Position, Orientation, Size);

			FKBoxElem& NewElem = AggBoxes.AddDefaulted_GetRef();
			
			NewElem.Center = FVector(Position);
			NewElem.Rotation = FRotator(Orientation.Rotator());
			NewElem.X = Size.X;
			NewElem.Y = Size.Y;
			NewElem.Z = Size.Z;
			NewElem.SetContributeToMass(GetContributeToMassFromFlags(Flags));
			NewElem.SetCollisionEnabled(GetCollisionEnabledFormFlags(Flags));
			NewElem.SetName(FName(*Name));
		}

		//const int32 NumConvexes = MutablePhysicsBody->GetConvexCount( BodyIndex );
		//TArray<FKConvexElem>& AggConvexes = BodyAggGeom.ConvexElems;
		//AggConvexes.Empty();
		//for (int32 I = 0; I < NumConvexes; ++I)
		//{
		//	uint32 Flags = MutablePhysicsBody->GetConvexFlags( BodyIndex, I );
		//	FString Name = MutablePhysicsBody->GetConvexName( BodyIndex, I );

		//	const FVector3f* Vertices;
		//	const int32* Indices;
		//	int32 NumVertices;
		//	int32 NumIndices;
		//	FTransform3f Transform;

		//	MutablePhysicsBody->GetConvex( BodyIndex, I, Vertices, NumVertices, Indices, NumIndices, Transform );
		//	
		//	TArrayView<const FVector3f> VerticesView( Vertices, NumVertices );
		//	TArrayView<const int32> IndicesView( Indices, NumIndices );
		//}

		TArray<FKSphylElem>& AggSphyls = BodyAggGeom.SphylElems;
		const int32 NumSphyls = MutablePhysicsBody->GetSphylCount(BodyIndex);
		AggSphyls.Empty(NumSphyls);

		for (int32 I = 0; I < NumSphyls; ++I)
		{
			uint32 Flags = MutablePhysicsBody->GetSphylFlags(BodyIndex, I);
			FString Name = MutablePhysicsBody->GetSphylName(BodyIndex, I);

			FVector3f Position;
			FQuat4f Orientation;
			float Radius;
			float Length;

			MutablePhysicsBody->GetSphyl(BodyIndex, I, Position, Orientation, Radius, Length);

			FKSphylElem& NewElem = AggSphyls.AddDefaulted_GetRef();
			
			NewElem.Center = FVector(Position);
			NewElem.Rotation = FRotator(Orientation.Rotator());
			NewElem.Radius = Radius;
			NewElem.Length = Length;
			
			NewElem.SetContributeToMass(GetContributeToMassFromFlags(Flags));
			NewElem.SetCollisionEnabled(GetCollisionEnabledFormFlags(Flags));
			NewElem.SetName(FName(*Name));
		}	

		TArray<FKTaperedCapsuleElem>& AggTaperedCapsules = BodyAggGeom.TaperedCapsuleElems;
		const int32 NumTaperedCapsules = MutablePhysicsBody->GetTaperedCapsuleCount(BodyIndex);
		AggTaperedCapsules.Empty(NumTaperedCapsules);

		for (int32 I = 0; I < NumTaperedCapsules; ++I)
		{
			uint32 Flags = MutablePhysicsBody->GetTaperedCapsuleFlags(BodyIndex, I);
			FString Name = MutablePhysicsBody->GetTaperedCapsuleName(BodyIndex, I);

			FVector3f Position;
			FQuat4f Orientation;
			float Radius0;
			float Radius1;
			float Length;

			MutablePhysicsBody->GetTaperedCapsule(BodyIndex, I, Position, Orientation, Radius0, Radius1, Length);

			FKTaperedCapsuleElem& NewElem = AggTaperedCapsules.AddDefaulted_GetRef();
			
			NewElem.Center = FVector(Position);
			NewElem.Rotation = FRotator(Orientation.Rotator());
			NewElem.Radius0 = Radius0;
			NewElem.Radius1 = Radius1;
			NewElem.Length = Length;
			
			NewElem.SetContributeToMass(GetContributeToMassFromFlags(Flags));
			NewElem.SetCollisionEnabled(GetCollisionEnabledFormFlags(Flags));
			NewElem.SetName(FName(*Name));
		}	

		return BodyAggGeom;
	};

	TObjectPtr<UPhysicsAsset> MakePhysicsAssetFromTemplateAndMutableBody(
		const TSharedRef<FUpdateContextPrivate>& Context,
		TObjectPtr<UPhysicsAsset> TemplateAsset,
		const UE::Mutable::Private::FPhysicsBody* MutablePhysics,
		int32 ComponentIndex)
	{
		check(TemplateAsset);
		TObjectPtr<UPhysicsAsset> Result = NewObject<UPhysicsAsset>();

		if (!Result)
		{
			return nullptr;
		}

		Result->SolverSettings = TemplateAsset->SolverSettings;
		Result->SolverType = TemplateAsset->SolverType;

		Result->bNotForDedicatedServer = TemplateAsset->bNotForDedicatedServer;

		const TArray<FName>& BonesInUse = MutablePhysics->BodiesBoneNames;

		const int32 PhysicsAssetBodySetupNum = TemplateAsset->SkeletalBodySetups.Num();
		bool bTemplateBodyNotUsedFound = false;

		TArray<uint8> UsageMap;
		UsageMap.Init(1, PhysicsAssetBodySetupNum);

		for (int32 BodySetupIndex = 0; BodySetupIndex < PhysicsAssetBodySetupNum; ++BodySetupIndex)
		{
			const TObjectPtr<USkeletalBodySetup>& BodySetup = TemplateAsset->SkeletalBodySetups[BodySetupIndex];

			int32 MutableBodyIndex = BonesInUse.Find(BodySetup->BoneName);
			if (MutableBodyIndex == INDEX_NONE)
			{
				bTemplateBodyNotUsedFound = true;
				UsageMap[BodySetupIndex] = 0;
				continue;
			}

			TObjectPtr<USkeletalBodySetup> NewBodySetup = NewObject<USkeletalBodySetup>(Result);
			NewBodySetup->BodySetupGuid = FGuid::NewGuid();

			// Copy Body properties 	
			NewBodySetup->BoneName = BodySetup->BoneName;
			NewBodySetup->PhysicsType = BodySetup->PhysicsType;
			NewBodySetup->bConsiderForBounds = BodySetup->bConsiderForBounds;
			NewBodySetup->bMeshCollideAll = BodySetup->bMeshCollideAll;
			NewBodySetup->bDoubleSidedGeometry = BodySetup->bDoubleSidedGeometry;
			NewBodySetup->bGenerateNonMirroredCollision = BodySetup->bGenerateNonMirroredCollision;
			NewBodySetup->bSharedCookedData = BodySetup->bSharedCookedData;
			NewBodySetup->bGenerateMirroredCollision = BodySetup->bGenerateMirroredCollision;
			NewBodySetup->PhysMaterial = BodySetup->PhysMaterial;
			NewBodySetup->CollisionReponse = BodySetup->CollisionReponse;
			NewBodySetup->CollisionTraceFlag = BodySetup->CollisionTraceFlag;
			NewBodySetup->DefaultInstance = BodySetup->DefaultInstance;
			NewBodySetup->WalkableSlopeOverride = BodySetup->WalkableSlopeOverride;
			NewBodySetup->BuildScale3D = BodySetup->BuildScale3D;
			NewBodySetup->bSkipScaleFromAnimation = BodySetup->bSkipScaleFromAnimation;

			// PhysicalAnimationProfiles can't be added with the current UPhysicsAsset API outside the editor.
			// Don't populate them for now.	
			//NewBodySetup->PhysicalAnimationData = BodySetup->PhysicalAnimationData;

			NewBodySetup->AggGeom = MakeAggGeomFromMutablePhysics(MutableBodyIndex, MutablePhysics);

			Result->SkeletalBodySetups.Add(NewBodySetup);
		}

		if (!bTemplateBodyNotUsedFound)
		{
			Result->CollisionDisableTable = TemplateAsset->CollisionDisableTable;

			const int32 NumConstraints = TemplateAsset->ConstraintSetup.Num();
			Result->ConstraintSetup.SetNum(NumConstraints);	
	
			for (int32 ConstraintIndex = 0; ConstraintIndex < NumConstraints; ++ConstraintIndex)
			{
				const TObjectPtr<UPhysicsConstraintTemplate>& TemplateConstraint = TemplateAsset->ConstraintSetup[ConstraintIndex];

				if (!TemplateConstraint)
				{
					continue;
				}

				Result->ConstraintSetup[ConstraintIndex] = ClonePhysicsConstraintTemplate(TemplateConstraint, Result);
			}
		}
		else
		{
			// Recreate the collision disable entry
			Result->CollisionDisableTable.Reserve(TemplateAsset->CollisionDisableTable.Num());
			for (const TPair<FRigidBodyIndexPair, bool>& CollisionDisableEntry : TemplateAsset->CollisionDisableTable)
			{
				const bool bIndex0Used = UsageMap[CollisionDisableEntry.Key.Indices[0]] > 0;
				const bool bIndex1Used = UsageMap[CollisionDisableEntry.Key.Indices[1]] > 0;

				if (bIndex0Used && bIndex1Used)
				{
					Result->CollisionDisableTable.Add(CollisionDisableEntry);
				}
			}

			// Only add constraints that are part of the bones used for the mutable physics volumes description.
			Result->ConstraintSetup.Reserve(TemplateAsset->ConstraintSetup.Num());
			for (const TObjectPtr<UPhysicsConstraintTemplate>& Constraint : TemplateAsset->ConstraintSetup)
			{
				if (!Constraint)
				{
					continue;
				}

				const FName BoneA = Constraint->DefaultInstance.ConstraintBone1;
				const FName BoneB = Constraint->DefaultInstance.ConstraintBone2;

				if (BonesInUse.Find(BoneA) && BonesInUse.Find(BoneB))
				{
					Result->ConstraintSetup.AddDefaulted_GetRef() = ClonePhysicsConstraintTemplate(Constraint, Result);
				}	
			}
		}

		Result->UpdateBodySetupIndexMap();
		Result->UpdateBoundsBodiesArray();

#if WITH_EDITORONLY_DATA
		Result->ConstraintProfiles = TemplateAsset->ConstraintProfiles;
#endif

		return Result;
	}
}


UPhysicsAsset* GetOrBuildMainPhysicsAsset(
	const TSharedRef<FUpdateContextPrivate>& Context,
	TObjectPtr<UPhysicsAsset> TemplateAsset,
	const UE::Mutable::Private::FPhysicsBody* MutablePhysics,
	bool bDisableCollisionsBetweenDifferentAssets,
	int32 ComponentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(MergePhysicsAssets);

	check(MutablePhysics);

	UCustomizableObjectInstance* Instance = Context->Instance.Get();
	if (!Instance)
	{
		return nullptr;
	}

	UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();
	
	UPhysicsAsset* Result = nullptr;

	const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];
	if (!Component)
	{
		return nullptr;
	}

	const FName ComponentName = Context->ComponentNames[Component->Id];
	
	TArray<TObjectPtr<UPhysicsAsset>> ValidAssets;
		
	for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
	{
		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];
		check(LOD->Mesh);

		for (const UE::Mutable::Private::TPassthroughObjectPtr<UPhysicsAsset>& PassthroughPhysicsAsset : LOD->Mesh->PhysicsAssets)
		{
			if (UPhysicsAsset* PhysicsAsset = PassthroughPhysicsAsset.Get())
			{
				ValidAssets.AddUnique(PhysicsAsset);
			}
		}
	}

	if (!ValidAssets.Num())
	{
		return Result;
	}

	// Just get the referenced asset if no reconstruction or merge is needed.
	if (ValidAssets.Num() == 1 && !MutablePhysics->bBodiesModified)
	{
		return ValidAssets[0];
	}

	TemplateAsset = TemplateAsset ? TemplateAsset : ValidAssets[0];
	check(TemplateAsset);

	Result = NewObject<UPhysicsAsset>();

	if (!Result)
	{
		return nullptr;
	}

	Result->SolverSettings = TemplateAsset->SolverSettings;
	Result->SolverType = TemplateAsset->SolverType;

	Result->bNotForDedicatedServer = TemplateAsset->bNotForDedicatedServer;

	const TArray<FName>& BonesInUse = MutablePhysics->BodiesBoneNames;

	// Each array is a set of elements that can collide  
	TArray<TArray<int32, TInlineAllocator<8>>> CollisionSets;

	// {SetIndex, ElementInSetIndex, BodyIndex}
	using CollisionSetEntryType = TTuple<int32, int32, int32>;	
	// Map from BodyName/BoneName to set and index in set.
	TMap<FName, CollisionSetEntryType> BodySetupSetMap;
	
	// Only for elements that belong to two or more different sets. 
	// Contains in which set the elements belong.
	using MultiSetArrayType = TArray<int32, TInlineAllocator<4>>;
	TMap<int32, MultiSetArrayType> MultiCollisionSets;
	TArray<TArray<int32>> SetsIndexMap;

	CollisionSets.SetNum(ValidAssets.Num());
	SetsIndexMap.SetNum(CollisionSets.Num());

	TMap<FRigidBodyIndexPair, bool> CollisionDisableTable;

	// New body index
	int32 CurrentBodyIndex = 0;
	for (int32 CollisionSetIndex = 0;  CollisionSetIndex < ValidAssets.Num(); ++CollisionSetIndex)
	{
		const int32 PhysicsAssetBodySetupNum = ValidAssets[CollisionSetIndex]->SkeletalBodySetups.Num();
		SetsIndexMap[CollisionSetIndex].Init(-1, PhysicsAssetBodySetupNum);

		for (int32 BodySetupIndex = 0; BodySetupIndex < PhysicsAssetBodySetupNum; ++BodySetupIndex) 
		{
			const TObjectPtr<USkeletalBodySetup>& BodySetup = ValidAssets[CollisionSetIndex]->SkeletalBodySetups[BodySetupIndex];
			
			int32 MutableBodyIndex = BonesInUse.Find(BodySetup->BoneName);
			if (MutableBodyIndex == INDEX_NONE)
			{
				continue;
			}

			CollisionSetEntryType* Found = BodySetupSetMap.Find(BodySetup->BoneName);

			if (!Found)
			{
				TObjectPtr<USkeletalBodySetup> NewBodySetup = NewObject<USkeletalBodySetup>(Result);
				NewBodySetup->BodySetupGuid = FGuid::NewGuid();
			
				// Copy Body properties 	
				NewBodySetup->BoneName = BodySetup->BoneName;
				NewBodySetup->PhysicsType = BodySetup->PhysicsType;
				NewBodySetup->bConsiderForBounds = BodySetup->bConsiderForBounds;
				NewBodySetup->bMeshCollideAll = BodySetup->bMeshCollideAll;
				NewBodySetup->bDoubleSidedGeometry = BodySetup->bDoubleSidedGeometry;
				NewBodySetup->bGenerateNonMirroredCollision = BodySetup->bGenerateNonMirroredCollision;
				NewBodySetup->bSharedCookedData = BodySetup->bSharedCookedData;
				NewBodySetup->bGenerateMirroredCollision = BodySetup->bGenerateMirroredCollision;
				NewBodySetup->PhysMaterial = BodySetup->PhysMaterial;
				NewBodySetup->CollisionReponse = BodySetup->CollisionReponse;
				NewBodySetup->CollisionTraceFlag = BodySetup->CollisionTraceFlag;
				NewBodySetup->DefaultInstance = BodySetup->DefaultInstance;
				NewBodySetup->WalkableSlopeOverride = BodySetup->WalkableSlopeOverride;
				NewBodySetup->BuildScale3D = BodySetup->BuildScale3D;	
				NewBodySetup->bSkipScaleFromAnimation = BodySetup->bSkipScaleFromAnimation;
				
				// PhysicalAnimationProfiles can't be added with the current UPhysicsAsset API outside the editor.
				// Don't populate them for now.	
				//NewBodySetup->PhysicalAnimationData = BodySetup->PhysicalAnimationData;

				NewBodySetup->AggGeom = MakeAggGeomFromMutablePhysics(MutableBodyIndex, MutablePhysics);


				Result->SkeletalBodySetups.Add(NewBodySetup);
				
				int32 IndexInSet = CollisionSets[CollisionSetIndex].Add(CurrentBodyIndex);
				BodySetupSetMap.Add(BodySetup->BoneName, {CollisionSetIndex, IndexInSet, CurrentBodyIndex});
				SetsIndexMap[CollisionSetIndex][IndexInSet] = CurrentBodyIndex;

				++CurrentBodyIndex;
			}
			else
			{
				int32 FoundCollisionSetIndex = Found->Get<0>();
				int32 FoundCollisionSetElemIndex = Found->Get<1>();
				int32 FoundBodyIndex = Found->Get<2>();
				
				// No need to add the body again. Volumes that come form mutable are already merged.
				// here we only need to merge properties.
				// TODO: check if there is other properties worth merging. In case of conflict select the more restrictive one? 
				Result->SkeletalBodySetups[FoundBodyIndex]->bConsiderForBounds |= BodySetup->bConsiderForBounds;

				// Mark as removed so no indices are invalidated.
				CollisionSets[FoundCollisionSetIndex][FoundCollisionSetElemIndex] = INDEX_NONE;
				// Add Elem to the set but mark it as removed so we have an index for remapping.
				int32 IndexInSet = CollisionSets[CollisionSetIndex].Add(INDEX_NONE);	
				SetsIndexMap[CollisionSetIndex][IndexInSet] = FoundBodyIndex;
				
				MultiSetArrayType& Sets = MultiCollisionSets.FindOrAdd(FoundBodyIndex);

				// The first time there is a collision (MultSet is empty), add the colliding element set
				// as well as the current set.
				if (!Sets.Num())
				{
					Sets.Add(FoundCollisionSetIndex);
				}
				
				Sets.Add(CollisionSetIndex);
			}
		}
	
		// Remap collision indices removing invalid ones.
		CollisionDisableTable.Reserve(CollisionDisableTable.Num() + ValidAssets[CollisionSetIndex]->CollisionDisableTable.Num());
		for (const TPair<FRigidBodyIndexPair, bool>& DisabledCollision : ValidAssets[CollisionSetIndex]->CollisionDisableTable)
		{
			int32 MappedIdx0 = SetsIndexMap[CollisionSetIndex][DisabledCollision.Key.Indices[0]];
			int32 MappedIdx1 = SetsIndexMap[CollisionSetIndex][DisabledCollision.Key.Indices[1]];

			// This will generate correct disables for the case when two shapes from different sets
			// are merged to the same setup. Will introduce repeated pairs, but this is not a problem.

			// Currently if two bodies / bones have disabled collision in one of the merged assets, the collision
			// will remain disabled even if other merges allow it.   
			if ( MappedIdx0 != INDEX_NONE && MappedIdx1 != INDEX_NONE )
			{
				CollisionDisableTable.Add({MappedIdx0, MappedIdx1}, DisabledCollision.Value);
			}
		}

		// Only add constraints that are part of the bones used for the mutable physics volumes description.
		Result->ConstraintSetup.Reserve(Result->ConstraintSetup.Num() + ValidAssets[CollisionSetIndex]->ConstraintSetup.Num());
		for (const TObjectPtr<UPhysicsConstraintTemplate>& Constraint : ValidAssets[CollisionSetIndex]->ConstraintSetup)
		{
			if (!Constraint)
			{
				continue;
			}

			FName BoneA = Constraint->DefaultInstance.ConstraintBone1;
			FName BoneB = Constraint->DefaultInstance.ConstraintBone2;

			if (BonesInUse.Find(BoneA) && BonesInUse.Find(BoneB))
			{
				Result->ConstraintSetup.AddDefaulted_GetRef() = ClonePhysicsConstraintTemplate(Constraint, Result); 
			}
		}

#if WITH_EDITORONLY_DATA
		Result->ConstraintProfiles.Append(ValidAssets[CollisionSetIndex]->ConstraintProfiles);
#endif
	}

	if (bDisableCollisionsBetweenDifferentAssets)
	{
		// Compute collision disable table size upperbound to reduce number of allocations.
		int32 CollisionDisableTableSize = 0;
		for (int32 S0 = 1; S0 < CollisionSets.Num(); ++S0)
		{
			for (int32 S1 = 0; S1 < S0; ++S1)
			{	
				CollisionDisableTableSize += CollisionSets[S1].Num() * CollisionSets[S0].Num();
			}
		}

		// We already may have elements in the table, but at the moment of 
		// addition we don't know yet the final number of elements.
		// Now a good number of elements will be added and because we know the final number of elements
		// an upperbound to the number of interactions can be computed and reserved. 
		CollisionDisableTable.Reserve(CollisionDisableTableSize);

		// Generate disable collision entry for every element in Set S0 for every element in Set S1 
		// that are not in multiple sets.
		for (int32 S0 = 1; S0 < CollisionSets.Num(); ++S0)
		{
			for (int32 S1 = 0; S1 < S0; ++S1)
			{	
				for (int32 Set0Elem : CollisionSets[S0])
				{
					// Element present in more than one set, will be treated later.
					if (Set0Elem == INDEX_NONE)
					{
						continue;
					}

					for (int32 Set1Elem : CollisionSets[S1])
					{
						// Element present in more than one set, will be treated later.
						if (Set1Elem == INDEX_NONE)
						{
							continue;
						}
						CollisionDisableTable.Add(FRigidBodyIndexPair{Set0Elem, Set1Elem}, false);
					}
				}
			}
		}

		// Process elements that belong to multiple sets that have been merged to the same element.
		for ( const TPair<int32, MultiSetArrayType>& Sets : MultiCollisionSets )
		{
			for (int32 S = 0; S < CollisionSets.Num(); ++S)
			{
				if (!Sets.Value.Contains(S))
				{	
					for (int32 SetElem : CollisionSets[S])
					{
						if (SetElem != INDEX_NONE)
						{
							CollisionDisableTable.Add(FRigidBodyIndexPair{Sets.Key, SetElem}, false);
						}
					}
				}
			}
		}

		CollisionDisableTable.Shrink();
	}

	Result->CollisionDisableTable = MoveTemp(CollisionDisableTable);
	Result->UpdateBodySetupIndexMap();
	Result->UpdateBoundsBodiesArray();

	return Result;
}


static float MutableMeshesMinUVChannelDensity = 100.f;
FAutoConsoleVariableRef CVarMutableMeshesMinUVChannelDensity(
	TEXT("Mutable.MinUVChannelDensity"),
	MutableMeshesMinUVChannelDensity,
	TEXT("Min UV density to set on generated meshes. This value will influence the requested texture mip to stream in. Higher values will result in higher quality mips being streamed in earlier."));


void SetMeshUVChannelDensity(FMeshUVChannelInfo& UVChannelInfo, float Density = 0.f)
{
	Density = Density > 0.f ? Density : 150.f;
	Density = FMath::Max(MutableMeshesMinUVChannelDensity, Density);

	UVChannelInfo.bInitialized = true;
	UVChannelInfo.bOverrideDensities = false;

	for (int32 i = 0; i < TEXSTREAM_MAX_NUM_UVCHANNELS; ++i)
	{
		UVChannelInfo.LocalUVDensities[i] = Density;
	}
}


bool UCustomizableInstancePrivate::UpdateSkeletalMesh_PostBeginUpdate0(UCustomizableObjectInstance* Instance, const TSharedRef<FUpdateContextPrivate>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::UpdateSkeletalMesh_PostBeginUpdate0)

	UCustomizableObject* CustomizableObject = Context->Object.Get();

	if (!CustomizableObject)
	{
		UE_LOGF(LogMutable, Warning, "Failed to generate SkeletalMesh for CO Instance %ls. It does not have a CO.", *Instance->GetName());
		Context->UpdateResult = EUpdateResult::Error;
		return false;
	}
	
	TMap<FName, TObjectPtr<USkeletalMesh>> OldSkeletalMeshes = SkeletalMeshes;
	const UModelResources& ModelResources = CustomizableObject->GetPrivate()->GetModelResourcesChecked();

	SkeletalMeshes.Reset();

#if WITH_EDITOR
	// TODO: This is a workaround to prevent the SkeletalMesh thumbnail renderer scene cache to consumer vasts amounts of
	// video memory address space when generating a lot of transient SkeletalMesh with different path name.
	// The SkeletalMesh thumbnail scene cache evict policy should be revisited, once this is done, this workaround can 
	// be removed alongside the custom thumbnail render that allows external eviction. 
	ON_SCOPE_EXIT
	{
		if (ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
		{
			TArray<TObjectPtr<USkeletalMesh>, TInlineAllocator<32>> CurrentSkeletalMeshes;
			Algo::Transform(SkeletalMeshes, CurrentSkeletalMeshes, &TPair<FName, TObjectPtr<USkeletalMesh>>::Value);

			for (const TPair<FName, TObjectPtr<USkeletalMesh>>& OldSkeletalMeshEntry : OldSkeletalMeshes)
			{
				const int32 FoundIndex = CurrentSkeletalMeshes.Find(OldSkeletalMeshEntry.Value);

				// Evict thumbnail scenes for not reused SkeletalMeshes.
				if (FoundIndex == INDEX_NONE)
				{
					EditorModule->EvictSkeletalMeshThumbnailScene(OldSkeletalMeshEntry.Value);
				}
			}
		}
	};
#endif

	const int32 NumComponents = Context->InstanceUpdateData.Components.Num();
	
	Context->MeshChangedPerInstanceComponent.Init(false, NumComponents);

	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
	{
		const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];
		
		FCustomizableInstanceComponentData& ComponentData = GetComponentData(Component->Id);
		const FName ComponentName = ComponentData.ComponentName;

		Context->MeshChangedPerInstanceComponent[ComponentIndex] = Component->SkeletalMeshId != ComponentData.LastSkeletalMeshId;
		ComponentData.LastSkeletalMeshId = Component->SkeletalMeshId;

		if (Component->SkeletalMesh->PassthroughObject)
		{
			if (USkeletalMesh* SkeletalMesh = Component->SkeletalMesh->PassthroughObject.Get())
			{
				SkeletalMeshes.Add(ComponentName, SkeletalMesh);
			}
			
			continue;
		}
		
		// If the component doesn't need an update copy the previously generated mesh.
		if (!Context->MeshChangedPerInstanceComponent[ComponentIndex])
		{
			if (TObjectPtr<USkeletalMesh>* Result = OldSkeletalMeshes.Find(ComponentName))
			{
				SkeletalMeshes.Add(ComponentName, *Result);
				continue;
			}
		}

		if (Context->bUseSkeletalMeshCache)
		{
			TStrongObjectPtr<UCustomizableObjectSkeletalMesh> SkeletalMesh = CustomizableObject->GetPrivate()->SkeletalMeshCache.Get(Component->SkeletalMeshId);
			if (SkeletalMesh)
			{
				SkeletalMeshes.Add(ComponentName, SkeletalMesh.Get());
				continue;
			}
		}

		// We need the first valid mesh. get it from the component, considering that some LOSs may have been skipped.
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> ComponentMesh;
		for (int32 LODIndex = 0; LODIndex < Component->LODCount && !ComponentMesh; ++LODIndex)
		{
			ComponentMesh = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex]->Mesh;
		}
		
		if (!ComponentMesh)
		{
			continue;
		}

		if (ComponentMesh->GetSurfaceCount() == 0 && !ComponentMesh->IsReference())
		{
			continue;
		}

		if (!ModelResources.ReferenceSkeletalMeshesData.IsValidIndex(Component->Id))
		{
			Context->UpdateResult = EUpdateResult::Error;
			return false;
		}

		// Create and initialize the SkeletalMesh for this component
		MUTABLE_CPUPROFILER_SCOPE(ConstructMesh);

		USkeletalMesh* SkeletalMesh = nullptr;
		{
			FString SkeletalMeshName;
			
			if (!Context->bBake)
			{
				SkeletalMeshName = *Component->SkeletalMesh->Name.ToString();
			}
#if WITH_EDITOR
			else
			{
				const uint32 SkeletalMeshHash = UE::Mutable::Private::GetTypeHashPersistent(Component->SkeletalMeshId);
				SkeletalMeshName = FString::Printf(TEXT("%s_h%lu"), *Component->SkeletalMesh->Name.ToString(), SkeletalMeshHash);
			}
#endif

			// Make name unique to avoid collisions with other objects.
			const FName UniqueSkeletalMeshName = MakeUniqueObjectName(GetTransientPackage(), USkeletalMesh::StaticClass(), *SkeletalMeshName);
			
			if (!Context->bBake)
			{
				SkeletalMesh = NewObject<UCustomizableObjectSkeletalMesh>(GetTransientPackage(), UniqueSkeletalMeshName, RF_Transient);
			}
#if WITH_EDITOR
			else
			{
				SkeletalMesh = NewObject<USkeletalMesh>(GetTransientPackage(), UniqueSkeletalMeshName, RF_Transient);
			}
#endif
				
			check(SkeletalMesh);
			SkeletalMeshes.Add(ComponentName, SkeletalMesh);
		}
		
		const FMutableRefSkeletalMeshData& RefSkeletalMeshData = ModelResources.ReferenceSkeletalMeshesData[Component->Id];

		// Set up the default information any mesh from this component will have (LODArrayInfos, RenderData, Mesh settings, etc). 
		InitSkeletalMeshData(Context, SkeletalMesh, RefSkeletalMeshData, *CustomizableObject, Component);
		
		// Construct a new skeleton, fix up ActiveBones and Bonemap arrays and recompute the RefInvMatrices
		const bool bBuildSkeletonDataSuccess = BuildSkeletonData(Context, *SkeletalMesh, ComponentIndex);
		if (!bBuildSkeletonDataSuccess)
		{
			Context->UpdateResult = EUpdateResult::Error;
			return false;
		}

		// Build PhysicsAsset merging physics assets coming from SubMeshes of the newly generated Mesh
		if (UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FPhysicsBody> MutablePhysics = ComponentMesh->GetPhysicsBody())
		{
			constexpr bool bDisallowCollisionBetweenAssets = true;
			UPhysicsAsset* PhysicsAssetResult = GetOrBuildMainPhysicsAsset(
				Context, RefSkeletalMeshData.PhysicsAsset, MutablePhysics.Get(), bDisallowCollisionBetweenAssets, ComponentIndex);

			SkeletalMesh->SetPhysicsAsset(PhysicsAssetResult);

#if WITH_EDITORONLY_DATA
			if (PhysicsAssetResult && PhysicsAssetResult->GetPackage() == GetTransientPackage())
			{
				constexpr bool bMarkAsDirty = false;
				PhysicsAssetResult->SetPreviewMesh(SkeletalMesh, bMarkAsDirty);
			}
#endif
		}

		const int32 NumAdditionalPhysicsBodies = ComponentMesh->AdditionalPhysicsBodies.Num();
		check(ComponentMesh->AdditionalPhysicsAssets.Num() == NumAdditionalPhysicsBodies)
		for (int32 I = 0; I < NumAdditionalPhysicsBodies; ++I)
		{
			const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FPhysicsBody>& AdditionalPhysicsBody = ComponentMesh->AdditionalPhysicsBodies[I];
			TObjectPtr<UPhysicsAsset> PhysicsAssetTemplate = ComponentMesh->AdditionalPhysicsAssets[I].Get();
			check(AdditionalPhysicsBody && PhysicsAssetTemplate);

			if (!AdditionalPhysicsBody || !PhysicsAssetTemplate)
			{
				continue;
			}

			if (!AdditionalPhysicsBody->bBodiesModified)
			{
				continue;
			}

			const int32 PhysicsBodyExternalId = ComponentMesh->AdditionalPhysicsBodies[I]->CustomId;
			
			const FAnimBpOverridePhysicsAssetsInfo& Info = ModelResources.AnimBpOverridePhysiscAssetsInfo[PhysicsBodyExternalId];

			// Make sure the AnimInstance class is loaded. It is expected to be already loaded at this point though. 
			UClass* AnimInstanceClassLoaded = UE::Mutable::Private::LoadClass(Info.AnimInstanceClass);
			TSubclassOf<UAnimInstance> AnimInstanceClass = TSubclassOf<UAnimInstance>(AnimInstanceClassLoaded);
			if (!ensureAlways(AnimInstanceClass))
			{
				continue;
			}

			FAnimBpGeneratedPhysicsAssets& PhysicsAssetsUsedByAnimBp = AnimBpPhysicsAssets.FindOrAdd(AnimInstanceClass);

			FAnimInstanceOverridePhysicsAsset& Entry =
				PhysicsAssetsUsedByAnimBp.AnimInstancePropertyIndexAndPhysicsAssets.Emplace_GetRef();

			Entry.PropertyIndex = Info.PropertyIndex;
			Entry.PhysicsAsset = MakePhysicsAssetFromTemplateAndMutableBody(
				Context, PhysicsAssetTemplate, AdditionalPhysicsBody.Get(), ComponentIndex);
		}

		// Add sockets from the SkeletalMesh of reference and from the MutableMesh
		BuildMeshSockets(SkeletalMesh, ModelResources, RefSkeletalMeshData, ComponentMesh);
		
		for (const UCustomizableObjectExtension* Extension : ICustomizableObjectModule::Get().GetRegisteredExtensions())
		{
			Extension->OnSkeletalMeshCreated(ComponentName, SkeletalMesh);
		}
		
		BuildOrCopyElementData(Context, SkeletalMesh, ComponentIndex);
		bool const bCopyRenderDataSuccess = BuildOrCopyRenderData(Context, SkeletalMesh, Instance, ComponentIndex);
		if (!bCopyRenderDataSuccess)
		{
			Context->UpdateResult = EUpdateResult::Error;
			return false;
		}

		BuildOrCopyMorphTargetsData(Context, SkeletalMesh, ComponentIndex);
		BuildOrCopyClothingData(Context, SkeletalMesh, ModelResources, ComponentIndex);

		FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		ensure(RenderData && RenderData->LODRenderData.Num() > 0);
		ensure(SkeletalMesh->GetLODNum() > 0);

		if(RenderData)
		{
			for (FSkeletalMeshLODRenderData& LODResource : RenderData->LODRenderData)
			{
				UnrealConversionUtils::UpdateSkeletalMeshLODRenderDataBuffersSize(LODResource);
			}
		}
		
		if (Context->bUseSkeletalMeshCache)
		{
			CustomizableObject->GetPrivate()->SkeletalMeshCache.Add(Component->SkeletalMeshId, CastChecked<UCustomizableObjectSkeletalMesh>(SkeletalMesh));
		}

		if (UCustomizableObjectSkeletalMesh* StreamableMesh = Cast<UCustomizableObjectSkeletalMesh>(SkeletalMesh))
		{
			StreamableMesh->InitMutableStreamingData(Context, ComponentIndex, Component->FirstLOD, Component->LODCount);
		}
	}

	return true;
}


UCustomizableObjectInstance* UCustomizableObjectInstance::Clone()
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObjectInstance::Clone);

	// Default Outer is the transient package.
	UCustomizableObjectInstance* NewInstance = NewObject<UCustomizableObjectInstance>();
	check(NewInstance->PrivateData);
	NewInstance->CopyParametersFromInstance(this);

	return NewInstance;
}


UCustomizableObjectInstance* UCustomizableObjectInstance::CloneStatic(UObject* Outer)
{
	UCustomizableObjectInstance* NewInstance = NewObject<UCustomizableObjectInstance>(Outer, GetClass());
	NewInstance->CopyParametersFromInstance(this);
	NewInstance->GetPrivate()->bShowOnlyRuntimeParameters = false;

	return NewInstance;
}


void UCustomizableObjectInstance::CopyParametersFromInstance(UCustomizableObjectInstance* Instance)
{
	GetPrivate()->CopyParametersFromInstance(Instance);
}


int32 UCustomizableObjectInstance::AddValueToIntRange(const FString& ParamName)
{
	return Descriptor.AddValueToIntRange(ParamName);
}


int32 UCustomizableObjectInstance::AddValueToFloatRange(const FString& ParamName)
{
	return Descriptor.AddValueToFloatRange(ParamName);
}


int32 UCustomizableObjectInstance::AddValueToProjectorRange(const FString& ParamName)
{
	return Descriptor.AddValueToProjectorRange(ParamName);
}


int32 UCustomizableObjectInstance::RemoveValueFromIntRange(const FString& ParamName, int32 RangeIndex)
{
	return Descriptor.RemoveValueFromIntRange(ParamName, RangeIndex);

}


int32 UCustomizableObjectInstance::RemoveValueFromFloatRange(const FString& ParamName, const int32 RangeIndex)
{
	return Descriptor.RemoveValueFromFloatRange(ParamName, RangeIndex);
}


int32 UCustomizableObjectInstance::RemoveValueFromProjectorRange(const FString& ParamName, const int32 RangeIndex)
{
	return Descriptor.RemoveValueFromProjectorRange(ParamName, RangeIndex);
}


int32 UCustomizableObjectInstance::MultilayerProjectorNumLayers(const FName& ProjectorParamName) const
{
	return Descriptor.NumProjectorLayers(ProjectorParamName);
}


void UCustomizableObjectInstance::MultilayerProjectorCreateLayer(const FName& ProjectorParamName, int32 Index)
{
	Descriptor.CreateLayer(ProjectorParamName, Index);
}


void UCustomizableObjectInstance::MultilayerProjectorRemoveLayerAt(const FName& ProjectorParamName, int32 Index)
{
	Descriptor.RemoveLayerAt(ProjectorParamName, Index);
}


FMultilayerProjectorLayer UCustomizableObjectInstance::MultilayerProjectorGetLayer(const FName& ProjectorParamName, int32 Index) const
{
	return Descriptor.GetLayer(ProjectorParamName, Index);
}


void UCustomizableObjectInstance::MultilayerProjectorUpdateLayer(const FName& ProjectorParamName, int32 Index, const FMultilayerProjectorLayer& Layer)
{
	Descriptor.UpdateLayer(ProjectorParamName, Index, Layer);
}


void UCustomizableObjectInstance::SaveDescriptor(FArchive &Ar, bool bUseCompactDescriptor)
{
	Descriptor.SaveDescriptor(Ar, bUseCompactDescriptor);
}


void UCustomizableObjectInstance::LoadDescriptor(FArchive &Ar)
{
	Descriptor.LoadDescriptor(Ar);
}


const FString& UCustomizableObjectInstance::GetIntParameterSelectedOption(const FString& ParamName, const int32 RangeIndex) const
{
	return GetEnumParameterSelectedOption(ParamName, RangeIndex);
}


const FString& UCustomizableObjectInstance::GetEnumParameterSelectedOption(const FString& ParamName, int32 RangeIndex) const
{
	return Descriptor.GetIntParameterSelectedOption(ParamName, RangeIndex);
}


void UCustomizableObjectInstance::SetIntParameterSelectedOption(const FString& ParamName, const FString& SelectedOptionName, const int32 RangeIndex)
{
	SetEnumParameterSelectedOption(ParamName, SelectedOptionName, RangeIndex);
}


void UCustomizableObjectInstance::SetEnumParameterSelectedOption(const FString& ParamName, const FString& SelectedOptionName, int32 RangeIndex)
{
	Descriptor.SetIntParameterSelectedOption(ParamName, SelectedOptionName, RangeIndex);
}


float UCustomizableObjectInstance::GetFloatParameterSelectedOption(const FString& FloatParamName, const int32 RangeIndex) const
{
	return Descriptor.GetFloatParameterSelectedOption(FloatParamName, RangeIndex);
}


void UCustomizableObjectInstance::SetFloatParameterSelectedOption(const FString& FloatParamName, const float FloatValue, const int32 RangeIndex)
{
	return Descriptor.SetFloatParameterSelectedOption(FloatParamName, FloatValue, RangeIndex);
}


UTexture* UCustomizableObjectInstance::GetTextureParameterSelectedOption(const FString& TextureParamName, const int32 RangeIndex) const
{
	return Descriptor.GetTextureParameterSelectedOption(TextureParamName, RangeIndex);
}


void UCustomizableObjectInstance::SetTextureParameterSelectedOption(const FString& TextureParamName, UTexture* TextureValue, const int32 RangeIndex)
{
	Descriptor.SetTextureParameterSelectedOption(TextureParamName, TextureValue, RangeIndex);
}


USkeletalMesh* UCustomizableObjectInstance::GetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, int32 RangeIndex)
{
	return Descriptor.GetSkeletalMeshParameterSelectedOption(SkeletalMeshParamName, RangeIndex);
}


void UCustomizableObjectInstance::SetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, USkeletalMesh* SkeletalMeshValue, int32 RangeIndex)
{
	Descriptor.SetSkeletalMeshParameterSelectedOption(SkeletalMeshParamName, SkeletalMeshValue, RangeIndex);
}


UMaterialInterface* UCustomizableObjectInstance::GetMaterialParameterSelectedOption(const FString& MaterialParamName, int32 RangeIndex)
{
	return Descriptor.GetMaterialParameterSelectedOption(MaterialParamName, RangeIndex);
}


void UCustomizableObjectInstance::SetMaterialParameterSelectedOption(const FString& MaterialParamName, UMaterialInterface* MaterialValue, int32 RangeIndex)
{
	Descriptor.SetMaterialParameterSelectedOption(MaterialParamName, MaterialValue, RangeIndex);
}


FInstancedStruct UCustomizableObjectInstance::GetExternalTypeParameterSelectedOption(const FString& ExternalTypeParamName, int32 RangeIndex) const
{
	return Descriptor.GetExternalTypeParameterSelectedOption(ExternalTypeParamName, RangeIndex);
}


void UCustomizableObjectInstance::SetExternalTypeParameterSelectedOption(const FString& ExternalTypeParamName, const FInstancedStruct& ExternalTypeValue, int32 RangeIndex)
{
	Descriptor.SetExternalTypeParameterSelectedOption(ExternalTypeParamName, ExternalTypeValue, RangeIndex);
}


FLinearColor UCustomizableObjectInstance::GetColorParameterSelectedOption(const FString& ColorParamName) const
{
	return Descriptor.GetVectorParameterSelectedOption(ColorParamName);
}


void UCustomizableObjectInstance::SetColorParameterSelectedOption(const FString & ColorParamName, const FLinearColor& ColorValue)
{
	Descriptor.SetVectorParameterSelectedOption(ColorParamName, ColorValue);
}


bool UCustomizableObjectInstance::GetBoolParameterSelectedOption(const FString& BoolParamName) const
{
	return Descriptor.GetBoolParameterSelectedOption(BoolParamName);
}


void UCustomizableObjectInstance::SetBoolParameterSelectedOption(const FString& BoolParamName, const bool BoolValue)
{
	Descriptor.SetBoolParameterSelectedOption(BoolParamName, BoolValue);
}


FVector4f UCustomizableObjectInstance::GetVectorParameterSelectedOption(const FString& VectorParamName)
{
	return Descriptor.GetVectorParameterSelectedOption(VectorParamName);
}


void UCustomizableObjectInstance::SetVectorParameterSelectedOption(const FString& VectorParamName, const FVector4f& VectorValue)
{
	Descriptor.SetVectorParameterSelectedOption(VectorParamName, VectorValue);
}

FTransform UCustomizableObjectInstance::GetTransformParameterSelectedOption(const FString& TransformParamName) const
{
	return Descriptor.GetTransformParameterSelectedOption(TransformParamName);
}

void UCustomizableObjectInstance::SetTransformParameterSelectedOption(const FString& TransformParamName, const FTransform& TransformValue)
{
	Descriptor.SetTransformParameterSelectedOption(TransformParamName, TransformValue);
}


void UCustomizableObjectInstance::SetProjectorValue(const FString& ProjectorParamName,
                                                    const FVector& Pos, const FVector& Direction, const FVector& Up, const FVector& Scale,
                                                    const float Angle,
                                                    const int32 RangeIndex)
{
	Descriptor.SetProjectorValue(ProjectorParamName, Pos, Direction, Up, Scale, Angle, RangeIndex);
}


void UCustomizableObjectInstance::SetProjectorPosition(const FString& ProjectorParamName, const FVector& Pos, const int32 RangeIndex)
{
	Descriptor.SetProjectorPosition(ProjectorParamName, Pos, RangeIndex);
}


void UCustomizableObjectInstance::SetProjectorDirection(const FString& ProjectorParamName, const FVector& Direction, int32 RangeIndex)
{
	Descriptor.SetProjectorDirection(ProjectorParamName, Direction, RangeIndex);
}


void UCustomizableObjectInstance::SetProjectorUp(const FString& ProjectorParamName, const FVector& Up, int32 RangeIndex)
{
	Descriptor.SetProjectorUp(ProjectorParamName, Up, RangeIndex);	
}


void UCustomizableObjectInstance::SetProjectorScale(const FString& ProjectorParamName, const FVector& Scale, int32 RangeIndex)
{
	Descriptor.SetProjectorScale(ProjectorParamName, Scale, RangeIndex);	
}


void UCustomizableObjectInstance::SetProjectorAngle(const FString& ProjectorParamName, float Angle, int32 RangeIndex)
{
	Descriptor.SetProjectorAngle(ProjectorParamName, Angle, RangeIndex);
}


void UCustomizableObjectInstance::SetProjectorParameterSelectedOption(const FString& ProjectorParamName,
	const FCustomizableObjectProjector& ProjectorValue, const int32 RangeIndex)
{
	Descriptor.SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorValue, RangeIndex);
}


void UCustomizableObjectInstance::GetProjectorValue(const FString& ProjectorParamName,
                                                    FVector& OutPos, FVector& OutDir, FVector& OutUp, FVector& OutScale,
                                                    float& OutAngle, ECustomizableObjectProjectorType& OutType,
                                                    const int32 RangeIndex) const
{
	Descriptor.GetProjectorValue(ProjectorParamName, OutPos, OutDir, OutUp, OutScale, OutAngle, OutType, RangeIndex);
}


void UCustomizableObjectInstance::GetProjectorValueF(const FString& ProjectorParamName,
	FVector3f& OutPos, FVector3f& OutDir, FVector3f& OutUp, FVector3f& OutScale,
	float& OutAngle, ECustomizableObjectProjectorType& OutType,
	int32 RangeIndex) const
{
	Descriptor.GetProjectorValueF(ProjectorParamName, OutPos, OutDir, OutUp, OutScale, OutAngle, OutType, RangeIndex);
}


FVector UCustomizableObjectInstance::GetProjectorPosition(const FString& ParamName, const int32 RangeIndex) const
{
	return Descriptor.GetProjectorPosition(ParamName, RangeIndex);
}


FVector UCustomizableObjectInstance::GetProjectorDirection(const FString& ParamName, const int32 RangeIndex) const
{
	return Descriptor.GetProjectorDirection(ParamName, RangeIndex);
}


FVector UCustomizableObjectInstance::GetProjectorUp(const FString& ParamName, const int32 RangeIndex) const
{
	return Descriptor.GetProjectorUp(ParamName, RangeIndex);
}


FVector UCustomizableObjectInstance::GetProjectorScale(const FString& ParamName, const int32 RangeIndex) const
{
	return Descriptor.GetProjectorScale(ParamName, RangeIndex);
}


float UCustomizableObjectInstance::GetProjectorAngle(const FString& ParamName, int32 RangeIndex) const
{
	return Descriptor.GetProjectorAngle(ParamName, RangeIndex);
}


ECustomizableObjectProjectorType UCustomizableObjectInstance::GetProjectorParameterType(const FString& ParamName, const int32 RangeIndex) const
{
	return Descriptor.GetProjectorParameterType(ParamName, RangeIndex);
}


FCustomizableObjectProjector UCustomizableObjectInstance::GetProjector(const FString& ParamName, const int32 RangeIndex) const
{
	return GetProjectorParameterSelectedOption(ParamName, RangeIndex);
}


FCustomizableObjectProjector UCustomizableObjectInstance::GetProjectorParameterSelectedOption(const FString& ParamName, int32 RangeIndex) const
{
	return Descriptor.GetProjectorParameterSelectedOption(ParamName, RangeIndex);
}


bool UCustomizableObjectInstance::ContainsIntParameter(const FString& ParameterName) const
{
	return ContainsEnumParameter(ParameterName);
}


bool UCustomizableObjectInstance::ContainsEnumParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Int) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsFloatParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Float) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsTextureParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Texture) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsSkeletalMeshParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::SkeletalMesh) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsMaterialParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Material) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsExternalTypeParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::InstancedStruct) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsBoolParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Bool) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsColorParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Color) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsVectorParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Color) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsProjectorParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Projector) != INDEX_NONE;
}


bool UCustomizableObjectInstance::ContainsTransformParameter(const FString& ParameterName) const
{
	return Descriptor.FindTypedParameterIndex(ParameterName, EMutableParameterType::Transform) != INDEX_NONE;
}


int32 UCustomizableInstancePrivate::FindIntParameterNameIndex(const FString& ParamName) const
{
	return GetPublic()->Descriptor.FindTypedParameterIndex(ParamName, EMutableParameterType::Int);
}


int32 UCustomizableInstancePrivate::FindFloatParameterNameIndex(const FString& ParamName) const
{
	return GetPublic()->Descriptor.FindTypedParameterIndex(ParamName, EMutableParameterType::Float);
}


int32 UCustomizableInstancePrivate::FindBoolParameterNameIndex(const FString& ParamName) const
{
	return GetPublic()->Descriptor.FindTypedParameterIndex(ParamName, EMutableParameterType::Bool);
}


int32 UCustomizableInstancePrivate::FindVectorParameterNameIndex(const FString& ParamName) const
{
	return GetPublic()->Descriptor.FindTypedParameterIndex(ParamName, EMutableParameterType::Color);
}


int32 UCustomizableInstancePrivate::FindProjectorParameterNameIndex(const FString& ParamName) const
{
	return GetPublic()->Descriptor.FindTypedParameterIndex(ParamName, EMutableParameterType::Projector);
}


void UCustomizableObjectInstance::SetRandomValues()
{
	Descriptor.SetRandomValues();
}

void UCustomizableObjectInstance::SetRandomValuesFromStream(const FRandomStream& InStream)
{
	Descriptor.SetRandomValuesFromStream(InStream);
}

void UCustomizableObjectInstance::SetDefaultValue(const FString& ParamName)
{
	UCustomizableObject* CustomizableObject = GetCustomizableObject();
	if (!CustomizableObject)
	{
		return;
	}

	Descriptor.SetDefaultValue(CustomizableObject->GetPrivate()->FindParameter(ParamName));
}

void UCustomizableObjectInstance::SetDefaultValues()
{
	Descriptor.SetDefaultValues();
}


UCustomizableObjectInstance* UCustomizableInstancePrivate::GetPublic() const
{
	UCustomizableObjectInstance* Public = StaticCast<UCustomizableObjectInstance*>(GetOuter());
	check(Public);

	return Public;
}


void UCustomizableInstancePrivate::ForceSetReferenceSkeletalMesh() const
{
	UCustomizableObjectInstance* Instance = Cast<UCustomizableObjectInstance>(GetOuter());
	if (!Instance)
	{
		return;
	}

	UCustomizableObject* CustomizableObject = Instance->GetCustomizableObject();
	if (!CustomizableObject)
	{
		return;
	}
	
	const UModelResources* ModelResources = CustomizableObject->GetPrivate()->GetModelResources();
	if (!ModelResources)
	{
		return;
	}
	
	for (UCustomizableObjectInstanceUsage* CustomizableObjectInstanceUsage : InstanceUsages)
	{
#if WITH_EDITOR
		if (CustomizableObjectInstanceUsage->GetPrivate()->bIsNetModeDedicatedServer)
		{
			continue;
		}
#endif
		
		USkeletalMeshComponent* Parent = CustomizableObjectInstanceUsage->GetAttachParent();
		if (!Parent)
		{
			continue;
		}
		
		const FName& ComponentName = CustomizableObjectInstanceUsage->GetComponentName();
		const UE::Mutable::Private::FComponentId ComponentId = ModelResources->ComponentNamesPerObjectComponent.IndexOfByKey(ComponentName);
		if (!ModelResources->ReferenceSkeletalMeshesData.IsValidIndex(ComponentId))
		{
			continue;
		}
		
		Parent->EmptyOverrideMaterials();

		// TODO Due to the adding support of multiple components with the same name this is no longer correct. A single name may point to multiple FComponentId.
		TSoftObjectPtr<USkeletalMesh> SoftObjectPtr = ModelResources->ReferenceSkeletalMeshesData[ComponentId].SoftSkeletalMesh;
		USkeletalMesh* SkeletalMesh = UE::Mutable::Private::LoadObject(SoftObjectPtr);
		
		Parent->SetSkeletalMesh(SkeletalMesh);
	}
}


bool MutableTextureUsesOfflineProcessedData()
{
#if PLATFORM_DESKTOP || PLATFORM_ANDROID || PLATFORM_IOS
	return true;
#else
	return false;
#endif
}

void SetTexturePropertiesFromMutableImageProps(UTexture2D* Texture, const FMutableModelImageProperties& Props, bool bNeverStream)
{
	Texture->NeverStream = bNeverStream;
	Texture->bNotOfflineProcessed = !MutableTextureUsesOfflineProcessedData();

	Texture->SRGB = Props.SRGB;
	Texture->Filter = Props.Filter;
	Texture->LODBias = Props.LODBias;

	if (Props.MipGenSettings == TextureMipGenSettings::TMGS_NoMipmaps)
	{
		Texture->NeverStream = true;
	}

#if WITH_EDITORONLY_DATA
	Texture->MipGenSettings = Props.MipGenSettings;

	Texture->bFlipGreenChannel = Props.FlipGreenChannel;
#endif

	Texture->LODGroup = Props.LODGroup;
	Texture->AddressX = Props.AddressX;
	Texture->AddressY = Props.AddressY;
}


UCustomizableInstancePrivate* UCustomizableObjectInstance::GetPrivate() const
{ 
	check(PrivateData); // Currently this is initialized in the constructor so we expect it always to exist.
	return PrivateData; 
}


// The memory allocated in the function and pointed by the returned pointer is owned by the caller and must be freed. 
// If assigned to a UTexture2D, it will be freed by that UTexture2D
TUniquePtr<FTexturePlatformData> MutableCreateImagePlatformData(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage, int32 OnlyLOD, uint16 FullSizeX, uint16 FullSizeY)
{
	int32 SizeX = FMath::Max(MutableImage->GetSize()[0], FullSizeX);
	int32 SizeY = FMath::Max(MutableImage->GetSize()[1], FullSizeY);

	if (SizeX <= 0 || SizeY <= 0)
	{
		UE_LOGF(LogMutable, Warning, "Invalid parameters specified for UCustomizableInstancePrivate::MutableCreateImagePlatformData()");
		return nullptr;
	}

	int32 FirstLOD = 0;
	for (int32 l = 0; l < OnlyLOD; ++l)
	{
		if (SizeX <= 4 || SizeY <= 4)
		{
			break;
		}
		SizeX = FMath::Max(SizeX / 2, 1);
		SizeY = FMath::Max(SizeY / 2, 1);
		++FirstLOD;
	}

	int32 MaxSize = FMath::Max(SizeX, SizeY);
	int32 FullLODCount = 1;
	int32 MipsToSkip = 0;
	
	if (OnlyLOD < 0)
	{
		FullLODCount = FMath::CeilLogTwo(MaxSize) + 1;
		MipsToSkip = FullLODCount - MutableImage->GetLODCount();
		check(MipsToSkip >= 0);
	}

	// Reduce final texture size if we surpass the max size we can generate.
	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
	UCustomizableObjectSystemPrivate* SystemPrivate = System ? System->GetPrivate() : nullptr;

	int32 MaxTextureSizeToGenerate = SystemPrivate ? SystemPrivate->MaxTextureSizeToGenerate : 0;

	if (MaxTextureSizeToGenerate > 0)
	{
		// Skip mips only if texture streaming is disabled 
		const bool bIsStreamingEnabled = MipsToSkip > 0;

		// Skip mips if the texture surpasses a certain size
		if (MaxSize > MaxTextureSizeToGenerate && !bIsStreamingEnabled && OnlyLOD < 0)
		{
			// Skip mips until MaxSize is equal or less than MaxTextureSizeToGenerate or there aren't more mips to skip
			while (MaxSize > MaxTextureSizeToGenerate && FirstLOD < (FullLODCount - 1))
			{
				MaxSize = MaxSize >> 1;
				FirstLOD++;
			}

			// Update SizeX and SizeY
			SizeX = SizeX >> FirstLOD;
			SizeY = SizeY >> FirstLOD;
		}
	}

	if (MutableImage->GetLODCount() == 1)
	{
		MipsToSkip = 0;
		FullLODCount = 1;
		FirstLOD = 0;
	}

	int32 EndLOD = OnlyLOD < 0 ? FullLODCount : FirstLOD + 1;
	
	UE::Mutable::Private::EImageFormat MutableFormat = MutableImage->GetFormat();

	int32 MaxPossibleSize = 0;
		
	if (MaxTextureSizeToGenerate > 0)
	{
		MaxPossibleSize = int32(FMath::Pow(2.f, float(FullLODCount - FirstLOD - 1)));
	}
	else
	{
		MaxPossibleSize = int32(FMath::Pow(2.f, float(FullLODCount - 1)));
	}

	// This could happen with non-power-of-two images.
	//check(SizeX == MaxPossibleSize || SizeY == MaxPossibleSize || FullLODCount == 1);
	if (!(SizeX == MaxPossibleSize || SizeY == MaxPossibleSize || FullLODCount == 1))
	{
		UE_LOGF(LogMutable, Warning, "Building instance: unsupported texture size %d x %d.", SizeX, SizeY);
		//return nullptr;
	}

	UE::Mutable::Private::FImageOperator ImOp = UE::Mutable::Private::FImageOperator::GetDefault(UE::Mutable::Private::FImageOperator::FImagePixelFormatFunc());

	EPixelFormat PlatformFormat = PF_Unknown;
	switch (MutableFormat)
	{
	case UE::Mutable::Private::EImageFormat::RGB_UByte:
		// performance penalty. can happen in states that remove compression.
		PlatformFormat = PF_R8G8B8A8;	
		UE_LOGF(LogMutable, Display, "Building instance: a texture was generated in a format not supported by the hardware (RGB), this results in an additional conversion, so a performance penalty.");
		break; 

	case UE::Mutable::Private::EImageFormat::BGRA_UByte:			
		// performance penalty. can happen with texture parameter images.
		PlatformFormat = PF_R8G8B8A8;	
		UE_LOGF(LogMutable, Display, "Building instance: a texture was generated in a format not supported by the hardware (BGRA), this results in an additional conversion, so a performance penalty.");
		break;

	// Good cases:
	case UE::Mutable::Private::EImageFormat::RGBA_UByte:		PlatformFormat = PF_R8G8B8A8;	break;
	case UE::Mutable::Private::EImageFormat::BC1:				PlatformFormat = PF_DXT1;		break;
	case UE::Mutable::Private::EImageFormat::BC2:				PlatformFormat = PF_DXT3;		break;
	case UE::Mutable::Private::EImageFormat::BC3:				PlatformFormat = PF_DXT5;		break;
	case UE::Mutable::Private::EImageFormat::BC4:				PlatformFormat = PF_BC4;		break;
	case UE::Mutable::Private::EImageFormat::BC5:				PlatformFormat = PF_BC5;		break;
	case UE::Mutable::Private::EImageFormat::L_UByte:			PlatformFormat = PF_G8;			break;
	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR:	PlatformFormat = PF_ASTC_4x4;	break;
	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR:PlatformFormat = PF_ASTC_4x4;	break;
	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR:	PlatformFormat = PF_ASTC_4x4;	break;
	default:
		// Cannot prepare texture if it's not in the right format, this can happen if mutable is in debug mode or in case of bugs
		UE_LOGF(LogMutable, Warning, "Building instance: a texture was generated in an unsupported format, it will be converted to Unreal with a performance penalty.");

		switch (UE::Mutable::Private::GetImageFormatData(MutableFormat).Channels)
		{
		case 1:
			PlatformFormat = PF_R8;
			MutableImage = ImOp.ImagePixelFormat(0, MutableImage.Get(), UE::Mutable::Private::EImageFormat::L_UByte);
			break;
		case 2:
		case 3:
		case 4:
			PlatformFormat = PF_R8G8B8A8;
			MutableImage = ImOp.ImagePixelFormat(0, MutableImage.Get(), UE::Mutable::Private::EImageFormat::RGBA_UByte);
			break;
		default: 
			// Absolutely worst case
			return nullptr;
		}		
	}

	TUniquePtr<FTexturePlatformData> PlatformData = MakeUnique<FTexturePlatformData>();
	PlatformData->SizeX = SizeX;
	PlatformData->SizeY = SizeY;
	PlatformData->PixelFormat = PlatformFormat;

	// Allocate mipmaps.

	if (!FMath::IsPowerOfTwo(SizeX) || !FMath::IsPowerOfTwo(SizeY))
	{
		EndLOD = FirstLOD + 1;
		MipsToSkip = 0;
		FullLODCount = 1;
	}

	for (int32 MipLevelUE = FirstLOD; MipLevelUE < EndLOD; ++MipLevelUE)
	{
		int32 MipLevelMutable = MipLevelUE - MipsToSkip;		

		// Unlike Mutable, UE expects MIPs sizes to be at least the size of the compression block.
		// For example, a 8x8 PF_DXT1 texture will have the following MIPs:
		// Mutable    Unreal Engine
		// 8x8        8x8
		// 4x4        4x4
		// 2x2        4x4
		// 1x1        4x4
		//
		// Notice that even though Mutable reports MIP smaller than the block size, the actual data contains at least a block.
		FTexture2DMipMap* Mip = new FTexture2DMipMap( FMath::Max(SizeX, GPixelFormats[PlatformFormat].BlockSizeX)
													, FMath::Max(SizeY, GPixelFormats[PlatformFormat].BlockSizeY));

		PlatformData->Mips.Add(Mip);
		if(MipLevelUE >= MipsToSkip || OnlyLOD>=0)
		{
			check(MipLevelMutable >= 0);
			check(MipLevelMutable < MutableImage->GetLODCount());

			Mip->BulkData.Lock(LOCK_READ_WRITE);
			Mip->BulkData.ClearBulkDataFlags(BULKDATA_SingleUse);

			const uint8* MutableData = MutableImage->GetLODData(MipLevelMutable);
			const uint32 SourceDataSize = MutableImage->GetLODDataSize(MipLevelMutable);

			uint32 DestDataSize = (MutableFormat == UE::Mutable::Private::EImageFormat::RGB_UByte)
					? (SourceDataSize/3) * 4
					: SourceDataSize;
			void* pData = Mip->BulkData.Realloc(DestDataSize);

			// Special inefficient cases
			if (MutableFormat== UE::Mutable::Private::EImageFormat::BGRA_UByte)
			{
				check(SourceDataSize==DestDataSize);

				MUTABLE_CPUPROFILER_SCOPE(Innefficent_BGRA_Format_Conversion);

				uint8_t* pDest = reinterpret_cast<uint8_t*>(pData);
				for (size_t p = 0; p < SourceDataSize / 4; ++p)
				{
					pDest[p * 4 + 0] = MutableData[p * 4 + 2];
					pDest[p * 4 + 1] = MutableData[p * 4 + 1];
					pDest[p * 4 + 2] = MutableData[p * 4 + 0];
					pDest[p * 4 + 3] = MutableData[p * 4 + 3];
				}
			}

			else if (MutableFormat == UE::Mutable::Private::EImageFormat::RGB_UByte)
			{
				MUTABLE_CPUPROFILER_SCOPE(Innefficent_RGB_Format_Conversion);

				uint8_t* pDest = reinterpret_cast<uint8_t*>(pData);
				for (size_t p = 0; p < SourceDataSize / 3; ++p)
				{
					pDest[p * 4 + 0] = MutableData[p * 3 + 0];
					pDest[p * 4 + 1] = MutableData[p * 3 + 1];
					pDest[p * 4 + 2] = MutableData[p * 3 + 2];
					pDest[p * 4 + 3] = 255;
				}
			}

			// Normal case
			else
			{
				check(SourceDataSize == DestDataSize);
				FMemory::Memcpy(pData, MutableData, SourceDataSize);
			}

			Mip->BulkData.Unlock();
		}
		else
		{
			Mip->BulkData.SetBulkDataFlags(BULKDATA_PayloadInSeparateFile);
			Mip->BulkData.ClearBulkDataFlags(BULKDATA_PayloadAtEndOfFile);
		}

		SizeX /= 2;
		SizeY /= 2;

		SizeX = SizeX > 0 ? SizeX : 1;
		SizeY = SizeY > 0 ? SizeY : 1;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Some consistency checks for dev builds
	int32 BulkDataCount = 0;

	for (int32 i = 0; i < PlatformData->Mips.Num(); ++i)
	{
		if (i > 0)
		{
			check(PlatformData->Mips[i].SizeX == PlatformData->Mips[i - 1].SizeX / 2 || PlatformData->Mips[i].SizeX == GPixelFormats[PlatformFormat].BlockSizeX);
			check(PlatformData->Mips[i].SizeY == PlatformData->Mips[i - 1].SizeY / 2 || PlatformData->Mips[i].SizeY == GPixelFormats[PlatformFormat].BlockSizeY);
		}

		if (PlatformData->Mips[i].BulkData.GetBulkDataSize() > 0)
		{
			BulkDataCount++;
		}
	}

	if (MaxTextureSizeToGenerate > 0)
	{
		check(FullLODCount == 1 || OnlyLOD >= 0 || (BulkDataCount == (MutableImage->GetLODCount() - FirstLOD)));
	}
	else
	{
		check(FullLODCount == 1 || OnlyLOD >= 0 || (BulkDataCount == MutableImage->GetLODCount()));
	}
#endif

	return PlatformData;
}


void ConvertImage(UTexture2D* Texture, UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage, const FMutableModelImageProperties& Props, int OnlyLOD, int32 ExtractChannel)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::ConvertImage);

	SetTexturePropertiesFromMutableImageProps(Texture, Props, false);

	UE::Mutable::Private::EImageFormat MutableFormat = MutableImage->GetFormat();

	// Extract a single channel, if requested.
	if (ExtractChannel >= 0)
	{
		UE::Mutable::Private::FImageOperator ImOp = UE::Mutable::Private::FImageOperator::GetDefault(UE::Mutable::Private::FImageOperator::FImagePixelFormatFunc());

		MutableImage = ImOp.ImagePixelFormat( 4, MutableImage.Get(), UE::Mutable::Private::EImageFormat::RGBA_UByte );

		uint8_t Channel = uint8_t( FMath::Clamp(ExtractChannel,0,3) );
		MutableImage = ImOp.ImageSwizzle( UE::Mutable::Private::EImageFormat::L_UByte, &MutableImage, &Channel );
		MutableFormat = UE::Mutable::Private::EImageFormat::L_UByte;
	}

	// Hack: This format is unsupported in UE, but it shouldn't happen in production.
	if (MutableFormat == UE::Mutable::Private::EImageFormat::RGB_UByte)
	{
		UE_LOGF(LogMutable, Warning, "Building instance: a texture was generated in RGB format, which is slow to convert to Unreal.");

		// Expand the image.
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Converted = 
				UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(MutableImage->GetSizeX(), MutableImage->GetSizeY(), MutableImage->GetLODCount(), UE::Mutable::Private::EImageFormat::RGBA_UByte, UE::Mutable::Private::EInitializationType::NotInitialized);

		for (int32 LODIndex = 0; LODIndex < Converted->GetLODCount(); ++LODIndex)
		{
			int32 PixelCount = MutableImage->GetLODDataSize(LODIndex)/3;
			const uint8* pSource = MutableImage->GetMipData(LODIndex);
			uint8* pTarget = Converted->GetMipData(LODIndex);
			for (int32 p = 0; p < PixelCount; ++p)
			{
				pTarget[4 * p + 0] = pSource[3 * p + 0];
				pTarget[4 * p + 1] = pSource[3 * p + 1];
				pTarget[4 * p + 2] = pSource[3 * p + 2];
				pTarget[4 * p + 3] = 255;
			}
		}

		MutableImage = Converted;
	}
	else if (MutableFormat == UE::Mutable::Private::EImageFormat::BGRA_UByte)
	{
		UE_LOGF(LogMutable, Warning, "Building instance: a texture was generated in BGRA format, which is slow to convert to Unreal.");

		MUTABLE_CPUPROFILER_SCOPE(Swizzle);
		// Swizzle the image.
		// \TODO: Raise a warning?
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Converted = 
				UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(MutableImage->GetSizeX(), MutableImage->GetSizeY(), 1, UE::Mutable::Private::EImageFormat::RGBA_UByte, UE::Mutable::Private::EInitializationType::NotInitialized);
		int32 PixelCount = MutableImage->GetSizeX() * MutableImage->GetSizeY();

		const uint8* pSource = MutableImage->GetLODData(0);
		uint8* pTarget = Converted->GetLODData(0);
		for (int32 p = 0; p < PixelCount; ++p)
		{
			pTarget[4 * p + 0] = pSource[4 * p + 2];
			pTarget[4 * p + 1] = pSource[4 * p + 1];
			pTarget[4 * p + 2] = pSource[4 * p + 0];
			pTarget[4 * p + 3] = pSource[4 * p + 3];
		}

		MutableImage = Converted;
	}

	if (OnlyLOD >= 0)
	{
		OnlyLOD = FMath::Min( OnlyLOD, MutableImage->GetLODCount()-1 );
	}

	Texture->SetPlatformData(MutableCreateImagePlatformData(MutableImage, OnlyLOD,0,0).Release());
}


static int32 EnableRayTracingFix = 0;
FAutoConsoleVariableRef CVarMutableEnableRayTracingFix(
	TEXT("mutable.EnableRayTracingFix"),
	EnableRayTracingFix,
	TEXT("If 0, Disabled. Generated meshes will have ray tracing enabled.")
	TEXT("If 1, Enable fix for meshes with mesh LOD streaming. Meshes will have ray tracing disabled.")
	TEXT("If 2, Enable fix for all generated meshes. Meshes will have ray tracing disabled.")
	);

void InitSkeletalMeshData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, const FMutableRefSkeletalMeshData& RefSkeletalMeshData, const UCustomizableObject& CustomizableObject, const TSharedPtr<FInstanceUpdateData::FComponent>& Component)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::InitSkeletalMesh);

	check(SkeletalMesh);

	UCustomizableObjectInstance* Instance = Context->Instance.Get();
	check(Instance);
	
	const FName ComponentName = Context->ComponentNames[Component->Id];

	SkeletalMesh->NeverStream = Context->FirstLODAvailable[ComponentName] == Context->FirstResidentLOD[ComponentName];

	SkeletalMesh->SetImportedBounds(RefSkeletalMeshData.Bounds);
	SkeletalMesh->SetPostProcessAnimBlueprint(RefSkeletalMeshData.PostProcessAnimInst.Get());
	SkeletalMesh->SetShadowPhysicsAsset(RefSkeletalMeshData.ShadowPhysicsAsset.Get());
	
	const bool bEnableRayTracingFix = EnableRayTracingFix == 2 || (EnableRayTracingFix == 1 && !SkeletalMesh->NeverStream);
	if (bEnableRayTracingFix)
	{
		SkeletalMesh->SetSupportRayTracing(false);
	}
	
	SkeletalMesh->SetHasVertexColors(false);

	// Set the default Physics Assets
	SkeletalMesh->SetPhysicsAsset(RefSkeletalMeshData.PhysicsAsset.Get());
	SkeletalMesh->SetEnablePerPolyCollision(RefSkeletalMeshData.Settings.bEnablePerPolyCollision);
	
	// Asset User Data
	{
		TMap<const UClass*, TArray<UAssetUserData*>> AssetUserDataByType; 
		
		// Hack. Get the Anim Slot data from the first LOD.
		for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
		{
			const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];

			if (!LOD->Mesh)
			{
				continue;
			}
			
			if (LOD->Mesh->GameplayTags.Num() || LOD->Mesh->AnimationSlots.Num())
			{
				UCustomizableObjectInstanceUserData* AssetUserData = NewObject<UCustomizableObjectInstanceUserData>(SkeletalMesh, NAME_None, RF_Public | RF_Transactional);
					
				for (const FName& GameplayTag : LOD->Mesh->GameplayTags)
				{
					AssetUserData->AnimationGameplayTag.AddTag(FGameplayTag::RequestGameplayTag(GameplayTag, false));
				}

				for (const TPair<FName, UE::Mutable::Private::TPassthroughObjectPtr<UClass>>& Pair : LOD->Mesh->AnimationSlots)
				{
					FCustomizableObjectAnimationSlot Slot;
					Slot.Name = Pair.Key;
					Slot.AnimInstance = Pair.Value.Get();

					AssetUserData->AnimationSlots.Add(Slot);
				}	
				
				TArray<UAssetUserData*>& Result = AssetUserDataByType.FindOrAdd(AssetUserData->GetClass(), {});
				Result.AddUnique(AssetUserData);
			}

			break;
		}
		
		// Hack. Get the asset user data from the first LOD.
		for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
		{
			const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];

			if (LOD->Mesh)
			{
				for (const UE::Mutable::Private::TPassthroughObjectPtr<UAssetUserData>& PassthroughAssetUserData : LOD->Mesh->AssetUserData)
				{
					if (UAssetUserData* AssetUserData = PassthroughAssetUserData.Get())
					{
						TArray<UAssetUserData*>& Result = AssetUserDataByType.FindOrAdd(AssetUserData->GetClass(), {});
						Result.AddUnique(AssetUserData);
					}
				}

				break;
			}
		}
	
		for (const TTuple<const UClass*, TArray<UAssetUserData*>>& Pair : AssetUserDataByType)
		{
			if (Pair.Value.Num() == 1)
			{
				SkeletalMesh->AddAssetUserData(Pair.Value.Top());
			}
			else if (Pair.Value.Num() > 1)
			{
				UAssetUserData* NewAssetUserData = DuplicateObject(Pair.Value[0], SkeletalMesh);

				bool bSuccess = true;
				for (int32 Index = 1; Index < Pair.Value.Num(); ++Index)
				{
					bSuccess = NewAssetUserData->Merge(Pair.Value[Index]);
					if (!bSuccess)
					{
						//UE_LOGF(LogMutable, Error, "Unable to merge Asset User Data %ls", *Pair.Key->GetName());
						break;
					}
				}
				
				if (bSuccess)
				{
					SkeletalMesh->AddAssetUserData(NewAssetUserData);
				}
			}
		}
	}
	
	// Allocate resources for rendering and add LOD Info
	{
		MUTABLE_CPUPROFILER_SCOPE(InitSkeletalMesh_AddLODData);
		SkeletalMesh->AllocateResourceForRendering();
		
		FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		int32 NumLODsAvailablePerComponent = Context->NumLODsAvailable[ComponentName];
		RenderData->NumInlinedLODs = NumLODsAvailablePerComponent - Context->FirstResidentLOD[ComponentName];
		RenderData->NumNonOptionalLODs = NumLODsAvailablePerComponent - Context->FirstLODAvailable[ComponentName];
		RenderData->CurrentFirstLODIdx = Context->FirstResidentLOD[ComponentName];
		RenderData->PendingFirstLODIdx = RenderData->CurrentFirstLODIdx;
		RenderData->LODBiasModifier = Context->FirstLODAvailable[ComponentName];		
		
		if (bEnableRayTracingFix)
		{
			RenderData->bSupportRayTracing = false;
		}

		for (int32 LODIndex = 0; LODIndex < NumLODsAvailablePerComponent; ++LODIndex)
		{
			RenderData->LODRenderData.Add(new FSkeletalMeshLODRenderData());
			
			FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];
			LODRenderData.bIsLODOptional = LODIndex < Context->FirstLODAvailable[ComponentName];
			LODRenderData.bStreamedDataInlined = LODIndex >= Context->FirstResidentLOD[ComponentName];

			FSkeletalMeshLODInfo& LODInfo = SkeletalMesh->AddLODInfo();
			LODInfo.ScreenSize = Component->SkeletalMesh->ScreenSize[LODIndex];
			LODInfo.LODHysteresis = Component->SkeletalMesh->LODHysteresis[LODIndex];
			LODInfo.bSupportUniformlyDistributedSampling = Component->SkeletalMesh->bSupportUniformlyDistributedSampling[LODIndex];
			LODInfo.bAllowCPUAccess = Component->SkeletalMesh->bAllowCPUAccess[LODIndex];

			if (bEnableRayTracingFix)
			{
				LODInfo.SkinCacheUsage = ESkinCacheUsage::Disabled;
			}

			// Disable LOD simplification when baking instances
			LODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.f;
			LODInfo.ReductionSettings.NumOfVertPercentage = 1.f;
			LODInfo.ReductionSettings.MaxNumOfTriangles = TNumericLimits<uint32>::Max();
			LODInfo.ReductionSettings.MaxNumOfVerts = TNumericLimits<uint32>::Max();
			LODInfo.ReductionSettings.bRecalcNormals = 0;
			LODInfo.ReductionSettings.WeldingThreshold = TNumericLimits<float>::Min();
			LODInfo.ReductionSettings.bMergeCoincidentVertBones = 0;
			LODInfo.ReductionSettings.bImproveTrianglesForCloth = 0;

#if WITH_EDITORONLY_DATA
			LODInfo.ReductionSettings.MaxNumOfTrianglesPercentage = TNumericLimits<uint32>::Max();
			LODInfo.ReductionSettings.MaxNumOfVertsPercentage = TNumericLimits<uint32>::Max();

			LODInfo.BuildSettings.bRecomputeNormals = false;
			LODInfo.BuildSettings.bRecomputeTangents = false;
			LODInfo.BuildSettings.bUseMikkTSpace = false;
			LODInfo.BuildSettings.bComputeWeightedNormals = false;
			LODInfo.BuildSettings.bRemoveDegenerates = false;
			LODInfo.BuildSettings.bUseHighPrecisionTangentBasis = false;
			LODInfo.BuildSettings.bUseHighPrecisionSkinWeights = false;
			LODInfo.BuildSettings.bUseFullPrecisionUVs = true;
			LODInfo.BuildSettings.bUseBackwardsCompatibleF16TruncUVs = false;
			LODInfo.BuildSettings.ThresholdPosition = TNumericLimits<float>::Min();
			LODInfo.BuildSettings.ThresholdTangentNormal = TNumericLimits<float>::Min();
			LODInfo.BuildSettings.ThresholdUV = TNumericLimits<float>::Min();
			LODInfo.BuildSettings.MorphThresholdPosition = TNumericLimits<float>::Min();
			LODInfo.BuildSettings.BoneInfluenceLimit = 0;
#endif
			LODInfo.LODMaterialMap.SetNumZeroed(1);
		}
	}

	if (RefSkeletalMeshData.SkeletalMeshLODSettings)
	{
#if WITH_EDITORONLY_DATA
		SkeletalMesh->SetLODSettings(RefSkeletalMeshData.SkeletalMeshLODSettings);
#else
		// This is the part from the above SkeletalMesh->SetLODSettings that's available in-game
		RefSkeletalMeshData.SkeletalMeshLODSettings->SetLODSettingsToMesh(SkeletalMesh);
#endif
	}

	// Set Min LOD (Override the Reference Skeletal Mesh LOD Settings)
	SkeletalMesh->SetMinLod(FMath::Max(Component->SkeletalMesh->MinLODs.GetDefault(), static_cast<int32>(Context->FirstLODAvailable[ComponentName])));
	
	SkeletalMesh->SetQualityLevelMinLod(Component->SkeletalMesh->MinQualityLevelLODs);

	// Set up unreal's default material, will be replaced when building materials
	{
		MUTABLE_CPUPROFILER_SCOPE(InitSkeletalMesh_AddDefaultMaterial);
		UMaterialInterface* UnrealMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		SkeletalMesh->GetMaterials().SetNum(1);
		SkeletalMesh->GetMaterials()[0] = UnrealMaterial;

		// Default density
		SetMeshUVChannelDensity(SkeletalMesh->GetMaterials()[0].UVChannelData);
	}
}


bool BuildSkeletonData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh& SkeletalMesh, int32 ComponentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildSkeletonData);

	UCustomizableObject* CustomizableObject = Context->Object.Get();
	check(CustomizableObject);

	const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];
	if (!Component)
	{
		return false;
	}


	const FName ComponentName = Context->ComponentNames[Component->Id];

	TArray<TObjectPtr<USkeleton>> SkeletonsToMerge;

	// Add Reference Skeleton to the list of SkeletonsToMerge. TODO: Make this optional
	const FMutableRefSkeletalMeshData& RefSkeletalMeshData = CustomizableObject->GetPrivate()->GetModelResourcesChecked().ReferenceSkeletalMeshesData[Component->Id];
	SkeletonsToMerge.Add(RefSkeletalMeshData.Skeleton);

	for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
	{
		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];
		check(LOD->Mesh)
				
		// Add Skeletons to merge
		for (const UE::Mutable::Private::TPassthroughObjectPtr<USkeleton>& SkeletonObject : LOD->Mesh->SkeletonObjects)
		{
			if (ensure(SkeletonObject.IsResolved()))
			{
				SkeletonsToMerge.AddUnique(SkeletonObject.Get());
			}
		}
	}

	bool bCreatedNewSkeleton = false;
	const TObjectPtr<USkeleton> Skeleton = MergeSkeletons(SkeletonsToMerge, *CustomizableObject, bCreatedNewSkeleton);
	if (!Skeleton)
	{
		return false;
	}

	SkeletalMesh.SetSkeleton(Skeleton);

	SkeletalMesh.SetRefSkeleton(Skeleton->GetReferenceSkeleton());
	FReferenceSkeleton& ReferenceSkeleton = SkeletalMesh.GetRefSkeleton();

	const TArray<FMeshBoneInfo>& RawRefBoneInfo = ReferenceSkeleton.GetRawRefBoneInfo();
	const int32 RawRefBoneCount = ReferenceSkeleton.GetRawBoneNum();


	{
		MUTABLE_CPUPROFILER_SCOPE(BuildSkeletonData_ApplyPose);
		
		const TArray<FInstanceUpdateData::FBone>& BonePose = Component->BonePose;
		
		TArray<FMatrix44f>& RefBasesInvMatrix = SkeletalMesh.GetRefBasesInvMatrix();
		RefBasesInvMatrix.Empty(RawRefBoneCount);

		// Calculate the InvRefMatrices to ensure all transforms are there for the second step 
		SkeletalMesh.CalculateInvRefMatrices();

		// First step is to update the RefBasesInvMatrix for the bones.
		for (const FInstanceUpdateData::FBone& Bone : BonePose)
		{
			const int32 BoneIndex = ReferenceSkeleton.FindBoneIndex(Bone.BoneName);
			if (RefBasesInvMatrix.IsValidIndex(BoneIndex))
			{
				RefBasesInvMatrix[BoneIndex] = Bone.MatrixWithScale;
			}
			else 
			{
				return false;
			}
		}

		// The second step is to update the pose transforms in the ref skeleton from the BasesInvMatrix
		FReferenceSkeletonModifier SkeletonModifier(ReferenceSkeleton, Skeleton);
		for (int32 RefSkelBoneIndex = 0; RefSkelBoneIndex < RawRefBoneCount; ++RefSkelBoneIndex)
		{
			int32 ParentBoneIndex = ReferenceSkeleton.GetParentIndex(RefSkelBoneIndex);
			if (ParentBoneIndex >= 0)
			{
				const FTransform3f BonePoseTransform(
						RefBasesInvMatrix[RefSkelBoneIndex].Inverse() * RefBasesInvMatrix[ParentBoneIndex]);

				SkeletonModifier.UpdateRefPoseTransform(RefSkelBoneIndex, (FTransform)BonePoseTransform);
			}
		}

		// Force a CalculateInvRefMatrices
		RefBasesInvMatrix.Empty(RawRefBoneCount);
	}

	{
		MUTABLE_CPUPROFILER_SCOPE(BuildSkeletonData_CalcInvRefMatrices);
		SkeletalMesh.CalculateInvRefMatrices();
	}

	USkeleton* GeneratedSkeleton = SkeletalMesh.GetSkeleton();

	if (GeneratedSkeleton && bCreatedNewSkeleton)
	{
		// If the skeleton is new, it means it has just been merged and the retargeting modes need merging too as the
		// MergeSkeletons function doesn't do it. Only do it for newly generated ones, not for cached or non-transient ones.
		GeneratedSkeleton->RecreateBoneTree(&SkeletalMesh);

		TMap<FName, EBoneTranslationRetargetingMode::Type> BoneNamesToRetargetingMode;

		const int32 NumberOfSkeletons = SkeletonsToMerge.Num();
		check(NumberOfSkeletons > 1);

		for (int32 SkeletonIndex = 0; SkeletonIndex < NumberOfSkeletons; ++SkeletonIndex)
		{
			const USkeleton* ToMergeSkeleton = SkeletonsToMerge[SkeletonIndex];
			const FReferenceSkeleton& ToMergeReferenceSkeleton = ToMergeSkeleton->GetReferenceSkeleton();
			const TArray<FMeshBoneInfo>& Bones = ToMergeReferenceSkeleton.GetRawRefBoneInfo();

			const int32 NumBones = Bones.Num();
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const FMeshBoneInfo& Bone = Bones[BoneIndex];

				EBoneTranslationRetargetingMode::Type RetargetingMode = ToMergeSkeleton->GetBoneTranslationRetargetingMode(BoneIndex, false);
				BoneNamesToRetargetingMode.Add(Bone.Name, RetargetingMode);
			}
		}

		for (const auto& Pair : BoneNamesToRetargetingMode)
		{
			const FName& BoneName = Pair.Key;
			const EBoneTranslationRetargetingMode::Type& RetargetingMode = Pair.Value;

			const int32 BoneIndex = GeneratedSkeleton->GetReferenceSkeleton().FindRawBoneIndex(BoneName);

			if (BoneIndex >= 0)
			{
				GeneratedSkeleton->SetBoneTranslationRetargetingMode(BoneIndex, RetargetingMode);
			}
		}
	}

	return true;
}


void BuildMeshSockets(USkeletalMesh* SkeletalMesh, const UModelResources& ModelResources, const FMutableRefSkeletalMeshData& RefSkeletalMeshData, UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& MutableMesh)
{
	// Build mesh sockets.
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildMeshSockets);

	check(SkeletalMesh);
	check(MutableMesh);

	TArray<TObjectPtr<USkeletalMeshSocket>>& Sockets = SkeletalMesh->GetMeshOnlySocketList();
	TArray<FName> SocketNames;

	const int32 NumSockets = FMath::Max(RefSkeletalMeshData.Sockets.Num(), MutableMesh->Sockets.Num());
	Sockets.Reserve(NumSockets);
	SocketNames.Reserve(NumSockets);

	// Add Mutable mesh sockets.
	for (const UE::Mutable::Private::FMeshSocket& MutableSocket : MutableMesh->Sockets)
	{
		USkeletalMeshSocket* Socket = NewObject<USkeletalMeshSocket>(SkeletalMesh, MutableSocket.SocketName);
		Socket->SocketName = MutableSocket.SocketName;
		Socket->BoneName = MutableSocket.BoneName;

		Socket->RelativeLocation = MutableSocket.RelativeLocation;
		Socket->RelativeRotation = MutableSocket.RelativeRotation;
		Socket->RelativeScale = MutableSocket.RelativeScale;

		Socket->bForceAlwaysAnimated = MutableSocket.bForceAlwaysAnimated;

		Sockets.Add(Socket);
		SocketNames.Add(MutableSocket.SocketName);
	}

	// Add sockets used by the SkeletalMesh of reference.
	for (const FMutableRefSocket& RefSocket : RefSkeletalMeshData.Sockets)
	{
		if (SocketNames.Find(RefSocket.SocketName) != INDEX_NONE)
		{
			continue;
		}

		USkeletalMeshSocket* Socket = NewObject<USkeletalMeshSocket>(SkeletalMesh, RefSocket.SocketName);
		Socket->SocketName = RefSocket.SocketName;
		Socket->BoneName = RefSocket.BoneName;

		Socket->RelativeLocation = RefSocket.RelativeLocation;
		Socket->RelativeRotation = RefSocket.RelativeRotation;
		Socket->RelativeScale = RefSocket.RelativeScale;

		Socket->bForceAlwaysAnimated = RefSocket.bForceAlwaysAnimated;

		Sockets.Add(Socket);
		SocketNames.Add(RefSocket.SocketName);
	}
	
#if !WITH_EDITOR
	SkeletalMesh->RebuildSocketMap();
#endif // !WITH_EDITOR
}


void BuildOrCopyElementData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, int32 ComponentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildOrCopyElementData);

	const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];
	if (!Component)
	{
		return;
	}

	const FName ComponentName = Context->ComponentNames[Component->Id];

	for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
	{
		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD+LODIndex];

		if (!LOD->SurfaceCount)
		{
			continue;
		}

		for (int32 SurfaceIndex = 0; SurfaceIndex < LOD->SurfaceCount; ++SurfaceIndex)
		{
			new(SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections) FSkelMeshRenderSection();
		}
	}
}

void BuildOrCopyMorphTargetsData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, int32 ComponentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildOrCopyMorphTargetsData);

	// This is a bit redundant as ComponentMorphTargets should not be generated.
	if (!CVarEnableRealTimeMorphTargets.GetValueOnAnyThread())
	{
		return;
	}

	if (!SkeletalMesh)
	{
		return;
	}

	const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];

	const FName ComponentName = Context->ComponentNames[Component->Id];

	FSkeletalMeshMorphTargets* ComponentMorphTargets = Context->InstanceUpdateData.RealTimeMorphTargets.Find(Component->Id);
	if (!ComponentMorphTargets)
	{
		return;
	}

	const int32 NumMorphTargets = ComponentMorphTargets->RealTimeMorphTargetNames.Num();
	
	TArray<TObjectPtr<UMorphTarget>>& MorphTargets = SkeletalMesh->GetMorphTargets();
	MorphTargets.Empty(NumMorphTargets);
	
	for (int32 MorphTargetIndex = 0; MorphTargetIndex < NumMorphTargets; ++MorphTargetIndex)
	{
		TArray<FMorphTargetLODModel>& MorphTargetData = ComponentMorphTargets->RealTimeMorphsLODData[MorphTargetIndex];

		if (MorphTargetData.IsEmpty())
		{
			continue;
		}
		
		const FName& MorphTargetName = ComponentMorphTargets->RealTimeMorphTargetNames[MorphTargetIndex];

		UMorphTarget* NewMorphTarget = NewObject<UMorphTarget>(SkeletalMesh, MorphTargetName);
		NewMorphTarget->BaseSkelMesh = SkeletalMesh;

		TArray<FMorphTargetLODModel>& MorphLODModels = NewMorphTarget->GetMorphLODModels();

		if (!SkeletalMesh->NeverStream)
		{
			const int32 FirstResidentLOD = Context->FirstResidentLOD[ComponentName];
			MorphLODModels.SetNum(ComponentMorphTargets->RealTimeMorphsLODData[MorphTargetIndex].Num());
#if !WITH_EDITOR
			// Allocate compressed LODModels storage. This will be filled by the streaming LOD update if CPU Morphs are needed. 
			// Only allocate for streamed LODs, the resident will use the non-compressed generated here.
			NewMorphTarget->GetCompressedLODModels().SetNum(FirstResidentLOD);
#endif
			// Streamed LODs
			const int32 FirstLODAvailable = Context->FirstLODAvailable[ComponentName];
			for (int32 LODIndex = FirstLODAvailable; LODIndex < FirstResidentLOD; ++LODIndex)
			{
				// Copy data required for streaming
				MorphLODModels[LODIndex].NumVertices = 1; // Trick the engine
				MorphLODModels[LODIndex].SectionIndices = MoveTemp(MorphTargetData[LODIndex].SectionIndices);
			}
			
			// Residents LODs
			for (int32 LODIndex = Context->FirstResidentLOD[ComponentName]; LODIndex < Context->NumLODsAvailable[ComponentName]; ++LODIndex)
			{
				MorphLODModels[LODIndex] = ComponentMorphTargets->RealTimeMorphsLODData[MorphTargetIndex][LODIndex];
			}
		}
		else
		{
			MorphLODModels = MoveTemp(ComponentMorphTargets->RealTimeMorphsLODData[MorphTargetIndex]);
		}

		MorphTargets.Add(NewMorphTarget);
	}

	const bool bInKeepEmptyMorphTargets = !SkeletalMesh->NeverStream;
	SkeletalMesh->InitMorphTargets(bInKeepEmptyMorphTargets); // True to avoid removing streamed Morph Targets.
}


void BuildOrCopyClothingData(
		const TSharedRef<FUpdateContextPrivate>& Context, 
		USkeletalMesh* SkeletalMesh, 
		const UModelResources& ModelResources,
		int32 ComponentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildOrCopyClothingData);

	TArray<UClothingAssetBase*> ClothingAssets;

	const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];
	if (!Component)
	{
		return;
	}

	const FName& ComponentName = Context->ComponentNames[Component->Id];
	
	// Streamed
	for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Context->FirstResidentLOD[ComponentName]; ++LODIndex)
	{
		FSkeletalMeshLODRenderData& RenderLOD = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];

		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];
		
		if (!LOD->Mesh->HasClothing())
		{
			continue;
		}
		
		TArray<FClothBufferIndexMapping> Mappings;

		constexpr int32 Stride = sizeof(FMeshToMeshVertData);
		
		int32 NumVertices = 0; // Upper bound
			
		const int32 NumSections = LOD->Mesh->Surfaces.Num();
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			FSkelMeshRenderSection& RenderSection = RenderLOD.RenderSections[SectionIndex];
					
			FClothBufferIndexMapping Mapping;
			Mapping.BaseVertexIndex = RenderSection.BaseVertexIndex;
			Mapping.MappingOffset = NumVertices;
			Mapping.LODBiasStride = RenderSection.NumVertices;
					
			Mappings.Add(Mapping);
						
			NumVertices += RenderSection.NumVertices;
			
			// Cloth hack. Initialize to something dummy (size is an upper bound). During streaming this data will be set correctly.
			const UE::Mutable::Private::FCloth& Cloth = LOD->Mesh->ClothSections[SectionIndex];
			if (Cloth.IsValid())
			{
				RenderSection.ClothMappingDataLODs.SetNum(1);
				RenderSection.ClothMappingDataLODs[0].SetNumUninitialized(RenderSection.NumVertices);
			}
		}
			
		RenderLOD.ClothVertexBuffer.SetMetadata(Mappings, Stride, NumVertices);
	}
	
	// Residents
	for (int32 LODIndex = Context->FirstResidentLOD[ComponentName]; LODIndex < Context->NumLODsAvailable[ComponentName]; ++LODIndex)
	{
		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];

		if (!LOD->Mesh->HasClothing())
		{
			continue;
		}

		FSkeletalMeshLODRenderData& RenderLOD = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];
		UnrealConversionUtils::ClothVertexBuffers(RenderLOD, *LOD->Mesh);
	}
	
	// Other Data
	for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
	{
		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];
		
		if (!LOD->Mesh->HasClothing())
		{
			continue;
		}

		FSkeletalMeshLODRenderData& RenderLOD = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];
		
		const int32 NumSections = LOD->Mesh->Surfaces.Num();
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			const UE::Mutable::Private::FCloth& Cloth = LOD->Mesh->ClothSections[SectionIndex];
			
			if (!Cloth.IsValid())
			{
				continue;
			}
			
			FSkelMeshRenderSection& RenderSection = RenderLOD.RenderSections[SectionIndex];
			
			UClothingAssetBase* Asset = Cloth.ClothingAsset.Get();
			RenderSection.CorrespondClothAssetIndex = ClothingAssets.AddUnique(Asset);
			
			FClothingSectionData& ClothingData = RenderSection.ClothingData;
			ClothingData.AssetGuid = Asset->GetAssetGuid();
			ClothingData.AssetLodIndex = Cloth.AssetLODIndex;
		}
	}

	TArray<TNotNull<UClothSharedConfigCommon*>, TInlineAllocator<8>> SkeletalMeshSharedConfigs;
	{
		MUTABLE_CPUPROFILER_SCOPE(ClothingAssetsDuplicate)
		for (int32 Index = 0; Index < ClothingAssets.Num(); ++Index)
		{
			UClothingAssetBase* Asset = CastChecked<UClothingAssetBase>(ClothingAssets[Index]);

			Asset = DuplicateObject<UClothingAssetBase>(Asset, SkeletalMesh);

			if (UClothingAssetCommon* AssetCommon = Cast<UClothingAssetCommon>(Asset))
			{
				int32 ReferenceBoneNameIndex = AssetCommon->UsedBoneIndices.Find(AssetCommon->ReferenceBoneIndex);
				
				FName ReferenceBoneName;
				if (AssetCommon->UsedBoneNames.IsValidIndex(ReferenceBoneNameIndex))
				{
					ReferenceBoneName = AssetCommon->UsedBoneNames[ReferenceBoneNameIndex];
				}

				AssetCommon->RefreshBoneMapping(SkeletalMesh);
				
				AssetCommon->ReferenceBoneIndex = INDEX_NONE;
				if (ReferenceBoneName != NAME_None)
				{
					AssetCommon->ReferenceBoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(ReferenceBoneName);
				}
	
				if (AssetCommon->ReferenceBoneIndex == INDEX_NONE)
				{
					AssetCommon->CalculateReferenceBoneIndex();
				}
		
				// Only one cloth shared config of the same type is allowed for the same SkeletalMesh. Keep the
				// first found and replace the others of the same type with the first found.
				
				for (TPair<FName, TObjectPtr<UClothConfigBase>>& Entry : AssetCommon->ClothConfigs)
				{
					UClothSharedConfigCommon* SharedConfig = Cast<UClothSharedConfigCommon>(Entry.Value);
					if (!SharedConfig)
					{
						continue;
					}

					TNotNull<UClothSharedConfigCommon*>* Found = SkeletalMeshSharedConfigs.FindByPredicate(
					[Class = SharedConfig->GetClass()](TNotNull<UClothSharedConfigCommon*> Elem) 
					{ 
						return Elem->GetClass() == Class;
					});

					if (!Found)
					{
						SkeletalMeshSharedConfigs.Add(SharedConfig);
					}
					else
					{
						// Should we LOG we are not using the original shared config?
						Entry.Value = *Found;
					}
				}
				
			}
			else
			{
				Asset->RefreshBoneMapping(SkeletalMesh);
			}

			ClothingAssets[Index] = Asset;
		}
	}

	SkeletalMesh->SetMeshClothingAssets(ClothingAssets);
	SkeletalMesh->SetHasActiveClothingAssets(!ClothingAssets.IsEmpty());
}


bool BuildOrCopyRenderData(const TSharedRef<FUpdateContextPrivate>& Context, USkeletalMesh* SkeletalMesh, 
	UCustomizableObjectInstance* Public, int32 ComponentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildOrCopyRenderData);

	FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
	check(RenderData);

	UCustomizableObject* CustomizableObject = Context->Object.Get();
	
	const TSharedPtr<FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];
	if (!Component)
	{
		return false;
	}
	
	const UModelResources& ModelResources = CustomizableObject->GetPrivate()->GetModelResourcesChecked();
	const FName ComponentName = ModelResources.ComponentNamesPerObjectComponent[Component->Id];

	for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; ++LODIndex)
	{
		MUTABLE_CPUPROFILER_SCOPE(BuildRenderData);
		
		const TSharedRef<FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];

		// There could be components without a mesh in LODs
		if (!LOD->Mesh || LOD->SurfaceCount == 0)
		{
			UE_LOGF(LogMutable, Warning, "Building instance: generated mesh [%ls] has LOD [%d] of object component index [%d] with no mesh."
				, *SkeletalMesh->GetName()
				, LODIndex
				, Component->Id);

			// End with failure
			return false;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("BuildRenderData: Component index %d, LOD %d"), Component->Id, LODIndex));
		
		FSkeletalMeshLODRenderData& LODResource = RenderData->LODRenderData[LODIndex];

		// Set active and required bones
		{
			const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
			const TArray<FName>& ActiveBones = Context->InstanceUpdateData.ActiveBones;
			LODResource.ActiveBoneIndices.Reserve(LOD->ActiveBoneCount);

			for (uint32 Index = 0; Index < LOD->ActiveBoneCount; ++Index)
			{
				const uint16 ActiveBoneIndex = RefSkeleton.FindBoneIndex(ActiveBones[LOD->FirstActiveBone + Index]);
				LODResource.ActiveBoneIndices.Add(ActiveBoneIndex);
			}

			LODResource.RequiredBones = LODResource.ActiveBoneIndices;
			LODResource.RequiredBones.Sort();
		}
		
		// Set RenderSections
		const bool bWasRenderSectionsSetupSuccessful = UnrealConversionUtils::SetupRenderSections(
			LODResource,
			LOD->Mesh.Get(),
			SkeletalMesh->GetRefSkeleton(),
			Context->InstanceUpdateData.BoneMaps,
			LOD->FirstBoneMap);

		if (!bWasRenderSectionsSetupSuccessful)
		{
			return false;
		}
		
		// Set SkinWeightProfiles
		LODResource.SkinWeightProfilesData.Init(&LODResource.SkinWeightVertexBuffer);
		
		for (const UE::Mutable::Private::FSkinWeightProfile& SkinWeightProfile : LOD->Mesh->SkinWeightProfiles)
		{
			const FName ProfileName = SkinWeightProfile.Name;

			const FSkinWeightProfileInfo* ExistingProfile = SkeletalMesh->GetSkinWeightProfiles().FindByPredicate(
				[ProfileName](const FSkinWeightProfileInfo& P) { return P.Name == ProfileName; });

			if (!ExistingProfile)
			{
				SkeletalMesh->AddSkinWeightProfile({ ProfileName, SkinWeightProfile.bDefaultProfile, SkinWeightProfile.DefaultProfileFromLODIndex });
			}
		}
		
		if (LODResource.bStreamedDataInlined) // Non-streamable LOD
		{
			// Copy Vertices
			UnrealConversionUtils::CopyMutableVertexBuffers(
				LODResource,
				LOD->Mesh.Get(),
				SkeletalMesh->GetLODInfo(LODIndex)->bAllowCPUAccess);

			// SurfaceIDs. Required to copy index buffers with padding
			TArray<uint32> SurfaceIDs;
			SurfaceIDs.SetNum(LOD->SurfaceCount);

			for (int32 SurfaceIndex = 0; SurfaceIndex < LOD->SurfaceCount; ++SurfaceIndex)
			{
				SurfaceIDs[SurfaceIndex] = LOD->Mesh->GetSurfaceId(SurfaceIndex);
			}

			// Copy indices.
			bool bMarkRenderStateDirty = false;
			if (!UnrealConversionUtils::CopyMutableIndexBuffers(LODResource, LOD->Mesh.Get(), bMarkRenderStateDirty))
			{
				// End with failure
				return false;
			}

			// Copy SkinWeightProfiles
			UnrealConversionUtils::CopyMutableSkinWeightProfilesBuffers(
				LODResource,
				*SkeletalMesh,
				LODIndex,
				LOD->Mesh.Get());
		}
		else // Streamable LOD 
		{
			// Init VertexBuffers for streaming
			UnrealConversionUtils::InitVertexBuffersWithDummyData(
				LODResource,
				LOD->Mesh.Get(),
				SkeletalMesh->GetLODInfo(LODIndex)->bAllowCPUAccess);

			// Init IndexBuffers for streaming
			UnrealConversionUtils::InitIndexBuffersWithDummyData(LODResource, LOD->Mesh.Get());
		}

		if (LODResource.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices())
		{
			SkeletalMesh->SetHasVertexColors(true);
		}
	}
	
	return true;
}


UE::Tasks::FTask LoadAdditionalAssetsAndData(const TSharedRef<FUpdateContextPrivate>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::LoadAdditionalAssetsAndDataAsync);

	UCustomizableObjectInstance* Instance = Context->Instance.Get();
	UCustomizableObject* CustomizableObject = Context->Object.Get();

	check(Instance);
	check(CustomizableObject);
	
	UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();
	const UModelResources& ModelResources = CustomizableObject->GetPrivate()->GetModelResourcesChecked();

	TArray<FSoftObjectPath> AssetsToStream;

	InstancePrivate->AnimBpPhysicsAssets.Reset();

	Context->PassthroughObjectLoader->Load(ModelResources.PassthroughObjects, AssetsToStream);

	TArray<UE::Tasks::FTask, TInlineAllocator<2>> Prerequisites;
	
	if (AssetsToStream.Num() > 0)
	{
#if WITH_EDITOR
		// TODO: Remove with UE-217665 when the underlying bug in the ColorPicker is solved
		// Disable the Slate throttling, otherwise the AsyncLoad may not complete until the editor window is clicked on due to a bug in
		// some widgets such as the ColorPicker's throttling handling
		FSlateThrottleManager::Get().DisableThrottle(true);
#endif
		

		UE::Tasks::FTaskEvent Event(TEXT("AssetsStreamed"));

		UCustomizableObjectSystemPrivate* PrivateSystem = UCustomizableObjectSystem::GetInstance()->GetPrivate();
		TSharedPtr<FStreamableHandle> Handle = PrivateSystem->StreamableManager->RequestAsyncLoad(
			AssetsToStream,
			FStreamableDelegate::CreateStatic(&AdditionalAssetsAsyncLoaded, Context, Event),
			CVarMutableHighPriorityLoading.GetValueOnAnyThread() ? FStreamableManager::AsyncLoadHighPriority : FStreamableManager::DefaultAsyncLoadPriority);

		if (Handle)
		{
			Prerequisites.Add(Event);
		}
		else
		{
			Event.Trigger();
		}
	}

	return UE::Tasks::Launch(TEXT("CaptureContext"), [Context]() {}, // Keep a reference to make sure allocated memory is always alive. TODO Probably not necessary since AdditionalAssetsAsyncLoaded already captures Context.
	Prerequisites,
	UE::Tasks::ETaskPriority::Inherit);
}


FCustomizableObjectInstanceDescriptor& UCustomizableInstancePrivate::GetDescriptor() const
{
	return GetPublic()->Descriptor;
}


TArray<UMaterialInterface*> UCustomizableObjectInstance::GetSkeletalMeshComponentOverrideMaterials(const FName& ComponentName) const
{
	FCustomizableInstanceComponentData* ComponentData = PrivateData->GetComponentData(ComponentName);
	if (!ComponentData)
	{
		return {};
	}
	
	TArray<UMaterialInterface*> Result;

	for (const TObjectPtr<UMaterialInterface>& OverrideMaterial : ComponentData->OverrideMaterials)
	{
		Result.Add(OverrideMaterial);
	}
	
	return Result;
}


void AdditionalAssetsAsyncLoaded(const TSharedRef<FUpdateContextPrivate> Context, UE::Tasks::FTaskEvent Event)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::AdditionalAssetsAsyncLoaded);

	check(IsInGameThread());
	
	ON_SCOPE_EXIT
	{
		Event.Trigger();
	};

	UCustomizableObject* CustomizableObject = Context->Object.Get();
	if (!IsValid(CustomizableObject))
	{
		Context->bIsUpdateAborted = true;
		return;
	}

	UModelResources& ModelResources = *CustomizableObject->GetPrivate()->GetModelResources();

	Context->PassthroughObjectLoader->Resolve(ModelResources.PassthroughObjects);

#if WITH_EDITOR
	// TODO: Remove with UE-217665 when the underlying bug in the ColorPicker is solved
	// Reenable the throttling which disabled when launching the Async Load
	FSlateThrottleManager::Get().DisableThrottle(false);
#endif
}


void ReuseTexture(UTexture2D* Texture, TUniquePtr<FTexturePlatformData> PlatformData)
{
	int32 NumMips = PlatformData->Mips.Num();

	TSharedPtr<FTexturePlatformData> SharedPlatformData = MakeShareable(PlatformData.Release()); 
	
	for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
	{
		FTexture2DMipMap& Mip = SharedPlatformData->Mips[MipIndex];

		if (Mip.BulkData.GetElementCount() > 0)
		{
			FUpdateTextureRegion2D Region;

			Region.DestX = 0;
			Region.DestY = 0;
			Region.SrcX = 0;
			Region.SrcY = 0;
			Region.Width = Mip.SizeX;
			Region.Height = Mip.SizeY;

			check(int32(Region.Width) <= Texture->GetSizeX());
			check(int32(Region.Height) <= Texture->GetSizeY());

			uint32 SrcPitch = Mip.SizeX * sizeof(uint8) * 4; 
			
			FTexture2DResource* Texture2DResource = static_cast<FTexture2DResource*>(Texture->GetResource());
			if (Texture2DResource)
			{
				ENQUEUE_RENDER_COMMAND(UpdateTextureRegionsMutable)(
				[MipIndex, NumMips, Region, SrcPitch, Texture2DResource, SharedPlatformData](FRHICommandList& CmdList)
				{
					FTexture2DMipMap& Mip = SharedPlatformData->Mips[MipIndex];
					const FByteBulkData* BulkData = &Mip.BulkData;

					check(NumMips >= Texture2DResource->GetCurrentMipCount());
					int32 MipDifference = NumMips - Texture2DResource->GetCurrentMipCount();
					check(MipDifference >= 0);
					int32 CurrentFirstMip = Texture2DResource->GetCurrentFirstMip();
					uint8* SrcData = (uint8*)BulkData->LockReadOnly();

					if (MipIndex >= CurrentFirstMip + MipDifference)
					{
						CmdList.UpdateTexture2D(
							Texture2DResource->GetTexture2DRHI(),
							MipIndex - CurrentFirstMip - MipDifference,
							Region,
							SrcPitch,
							SrcData);
					}

					BulkData->Unlock();
				});
			}
		}
	}
}



FName TextureName(const FInstanceUpdateData::FImage& Image, bool bBake)
{
	FString MutableTextureName;
	{
		const uint32 MutableImageHash = UE::Mutable::Private::GetTypeHashPersistent(Image.ImageID);
		MutableTextureName = FString::Printf(TEXT("T_%s_h%u"), *Image.ParameterName.ToString(), MutableImageHash);
		MutableTextureName.ReplaceInline(TEXT(" "), TEXT("_"));

#if WITH_EDITOR
		if (bBake)
		{
			// If this is a bake operation just add the baked prefix to make the name unique for all baked instances of this resource
			MutableTextureName = FBakingConfiguration::BakedResourcePrefix + MutableTextureName;
		}
#endif
	}
		
	return MakeUniqueObjectName(GetTransientPackage(), UTexture2D::StaticClass(), *MutableTextureName);
}

UTexture* BuildTexture(const TSharedRef<FUpdateContextPrivate>& Context, const int32 ImageInex, bool bCanReuseTexture)
{
	const TSharedRef<const FInstanceUpdateData::FImage>& Image = Context->InstanceUpdateData.Images[ImageInex];
	
	const FTextureCache::FId TextureCacheKey(Image->ImageID, Context->MipsToSkip, Context->bBake);
	
	TStrongObjectPtr<UTexture>* CachedTexture = Context->TextureUpdateCache.Find(TextureCacheKey);
	if (CachedTexture)
	{
		return CachedTexture->Get();
	}
	
	if (Image->Image && Image->Image->PassthroughObject)
	{
		UTexture* Texture = Image->Image->PassthroughObject.Get(); // TODO GMT Why not also cache Passthrough?

		if (Texture)
		{
			Context->Instance->GetPrivate()->Textures.AddUnique(Texture);
		}

		return Texture;
	}

	const UModelResources& ModelResources = *Context->Object->GetPrivate()->GetModelResources();

	// Get the cache of resources of all live instances of this object
	FTextureCache& TextureCache = Context->Object->GetPrivate()->TextureCache;

	FString CurrentState = Context->Instance->GetCurrentState();
	bool bNeverStream = Context->bNeverStreamMips;

	check((bNeverStream && Context->MipsToSkip == 0) ||
		(!bNeverStream && Context->MipsToSkip >= 0));
	
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage = Image->Image;

	UTexture2D* Texture = nullptr; // Texture generated by mutable

	TMap<FTextureReuseCacheKey, TStrongObjectPtr<UTexture2D>>& LastTextureReuseCache = Context->Instance->GetPrivate()->LastTextureReuseCache; 
	TMap<FTextureReuseCacheKey, TStrongObjectPtr<UTexture2D>>& CurrentTextureReuseCache = Context->Instance->GetPrivate()->CurrentTextureReuseCache;
	
	const FTextureReuseCacheKey TextureReuseCacheKey(Image->ImageID.GetKey()->Address);
	
	// If the mutable image is null, it must be in the cache
	if (!MutableImage)
	{
		if (Context->bUseCaches)
		{
			bCanReuseTexture = false; // TODO GMT If a Texture was already in the CO Cache, we can not modify it.
			
			const TStrongObjectPtr<UTexture2D> FoundTexture = TextureCache.Get(TextureCacheKey);
			Texture = FoundTexture.Get();
			check(Texture);
		}
	}
	else
	{
		// Find the additional information for this image
    	int32 ImageKey = Image->ImagePropertiesIndex;
    	check (ImageKey >= 0 && ImageKey < ModelResources.ImageProperties.Num());
    	
    	const FMutableModelImageProperties& Props = ModelResources.ImageProperties[ImageKey];
    	
		UTexture2D* ReusedTexture = nullptr;
		if (Context->bUseCaches &&  Context->bUseResueTextureCache && bCanReuseTexture)
		{
			if (TStrongObjectPtr<UTexture2D>* ReuseTexture = LastTextureReuseCache.Find(TextureReuseCacheKey))
			{
				ReusedTexture = ReuseTexture->Get();
				LastTextureReuseCache.Remove(TextureReuseCacheKey);
			}
		}

		if (ReusedTexture)
		{
			// Only uncompressed textures can be reused. This also fixes an issue in the editor where textures supposedly 
			// uncompressed by their state, are still compressed because the CO has not been compiled at maximum settings
			// and the uncompressed setting cannot be applied to them.
			EPixelFormat PixelFormat = ReusedTexture->GetPixelFormat();
			if (PixelFormat == EPixelFormat::PF_R8G8B8A8)
			{
				Texture = ReusedTexture;
			}
			else
			{
				ReusedTexture = nullptr;
				
				const FName Name = TextureName(Image.Get(), Context->bBake);
				Texture = CreateTexture(Name);
#if WITH_EDITOR
				UE_LOGF(LogMutable, Warning,
					"Tried to reuse an uncompressed texture with name %ls. Make sure the selected Mutable state disables texture compression/streaming, that one of the state's runtime parameters affects the texture and that the CO is compiled with max. optimization settings.",
					*Texture->GetName());
#endif
			}
		}
		else
		{
			const FName Name = TextureName(Image.Get(), Context->bBake);
			Texture = CreateTexture(Name);
		}

		if (!Context->ImageToPlatformDataMap.Contains(Image->ImageID))
		{
			UE_LOGF(LogMutable, Error, "Required image [%ls] with ID [%ls] was not generated in the mutable thread, and it is not cached.",
			*Props.TextureParameterName,
			*Image->ImageID.ToString());
			
			return nullptr;
		}
		
		SetTexturePropertiesFromMutableImageProps(Texture, Props, bNeverStream);

		TUniquePtr<FTexturePlatformData> PlatformData;
		Context->ImageToPlatformDataMap.RemoveAndCopyValue(Image->ImageID, PlatformData);

		if (ReusedTexture)
		{
			check(PlatformData->Mips.Num() == ReusedTexture->GetPlatformData()->Mips.Num());
			check(PlatformData->Mips[0].SizeX == ReusedTexture->GetPlatformData()->Mips[0].SizeX);
			check(PlatformData->Mips[0].SizeY == ReusedTexture->GetPlatformData()->Mips[0].SizeY);

			// Now the ReusedTexturePlatformData shared ptr owns the platform data
			// This shared ptr will hold the reused texture platform data (mips) until the reused texture is updated 
			// and delete it automatically
			ReuseTexture(ReusedTexture, MoveTemp(PlatformData));
		}
		else
		{
			// Now the MutableTexture owns the platform data
			Texture->SetPlatformData(PlatformData.Release());

			MUTABLE_CPUPROFILER_SCOPE(UpdateResource);
#if REQUIRES_SINGLEUSE_FLAG_FOR_RUNTIME_TEXTURES
			for (int32 i = 0; i < Texture->GetPlatformData()->Mips.Num(); ++i)
			{
				uint32 DataFlags = Texture->GetPlatformData()->Mips[i].BulkData.GetBulkDataFlags();
				Texture->GetPlatformData()->Mips[i].BulkData.SetBulkDataFlags(DataFlags | BULKDATA_SingleUse);
			}
#endif

			if (bNeverStream)
			{
				// To prevent LogTexture Error "Loading non-streamed mips from an external bulk file."
				for (int32 i = 0; i < Texture->GetPlatformData()->Mips.Num(); ++i)
				{
					Texture->GetPlatformData()->Mips[i].BulkData.ClearBulkDataFlags(BULKDATA_PayloadInSeparateFile);
				}
			}
				
			//if (!bNeverStreamMips) // No need to check bNeverStreamMips. In that case, the texture won't use 
			// the MutableMipDataProviderFactory anyway and it's needed for detecting Mutable textures elsewhere
			UMutableTextureMipDataProviderFactory* MutableMipDataProviderFactory = Cast<UMutableTextureMipDataProviderFactory>(Texture->GetAssetUserDataOfClass(UMutableTextureMipDataProviderFactory::StaticClass()));
			if (!MutableMipDataProviderFactory)
			{
				MutableMipDataProviderFactory = NewObject<UMutableTextureMipDataProviderFactory>();

				if (MutableMipDataProviderFactory)
				{
					MutableMipDataProviderFactory->CustomizableObjectInstance = Context->Instance.Get();
					MutableMipDataProviderFactory->ImageRef.ImageID = Image->ImageID;
					MutableMipDataProviderFactory->ImageRef.BaseMip = uint8(Image->BaseMip);
					MutableMipDataProviderFactory->ImageRef.ConstantImagesNeededToGenerate = Image->ConstantImagesNeededToGenerate;
					MutableMipDataProviderFactory->UpdateContext = Context->UpdateImageContext;
					Texture->AddAssetUserData(MutableMipDataProviderFactory);
				}
			}

			Texture->UpdateResource();
		}
	}

	if (Texture)
	{
		Context->Instance->GetPrivate()->Textures.Add(Texture);
	}
		
	if (Context->bUseCaches)
	{
		if (Context->bUseResueTextureCache && bCanReuseTexture)
		{
			CurrentTextureReuseCache.Add(TextureReuseCacheKey, TStrongObjectPtr(Texture));
		}
		else
		{
			TextureCache.Add(TextureCacheKey, Texture);
		}
	}

	Context->TextureUpdateCache.Add(TextureCacheKey, TStrongObjectPtr(Texture));
	
	return Texture;
}


struct FBuildMaterialOut
{
	UMaterialInterface* Material = nullptr;
	bool bNotifyPrimitiveUpdated = false;
};


FBuildMaterialOut BuildMaterial(const TSharedRef<FUpdateContextPrivate>& Context, const int32 MaterialIndex, const bool bCanReuseMaterial)
{
	FBuildMaterialOut Out;

	if (MaterialIndex == INDEX_NONE)
	{
		return Out;
	}

	const TSharedRef<const FInstanceUpdateData::FMaterial>& Material = Context->InstanceUpdateData.Materials[MaterialIndex];

	if (!Material->MaterialId)
	{
		Out.Material = Material->Material->PassthroughObject.Get();
		return Out;
	}

	TStrongObjectPtr<UMaterialInterface>* CachedMaterial = Context->MaterialUpdateCache.Find(Material->MaterialId);
	if (CachedMaterial)
	{
		Out.Material = CachedMaterial->Get();
		
		return Out;
	}
	
	if (!Material->Material->PassthroughObject.Get())
	{
		return Out;
	}

	UMaterialInterface* MaterialTemplate = Material->Material->PassthroughObject.Get();

	TMap<FMaterialReuseCacheKey, TStrongObjectPtr<UMaterialInterface>>& LastMaterialReuseCache = Context->Instance->GetPrivate()->LastMaterialsReuseCache; 
	TMap<FMaterialReuseCacheKey, TStrongObjectPtr<UMaterialInterface>>& CurrentMaterialReuseCache = Context->Instance->GetPrivate()->CurrentMaterialsReuseCache;
	
	const FMaterialReuseCacheKey MaterialReuseCacheKey(TStrongObjectPtr(MaterialTemplate), Material->MaterialId.GetKey()->Address);

	UMaterialInterface* MaterialInterface = nullptr;
	
	if (Context->bUseCaches && bCanReuseMaterial)
	{
		if (const TStrongObjectPtr<UMaterialInterface>* ReuseMaterial = LastMaterialReuseCache.Find(MaterialReuseCacheKey))
		{
			MaterialInterface = ReuseMaterial->Get();	
			LastMaterialReuseCache.Remove(MaterialReuseCacheKey);
		}
	}

	const bool bHasParameters = Material->Material->ColorParameters.Num() || Material->Material->ScalarParameters.Num() || Material->ImageCount;
	
	if (!MaterialInterface &&
		bHasParameters)
	{
		FName Name = NAME_None;

		if (Context->bBake)
		{
			// Add a "tag" to better identify this asset during the bake
			FString BakeName = FString::Printf(TEXT("%sMID_h%lu"), *FBakingConfiguration::BakedResourcePrefix, UE::Mutable::Private::GetTypeHashPersistent(Material->MaterialId));
			Name = MakeUniqueObjectName(GetTransientPackage(), UMaterialInstanceDynamic::StaticClass(), *BakeName);
		}

		MaterialInterface = UMaterialInstanceDynamic::Create(MaterialTemplate, GetTransientPackage(), Name);
	}
	
	if (!MaterialInterface)
	{
		MaterialInterface = MaterialTemplate;
	}
	
	if (bHasParameters)
	{
		UMaterialInstanceDynamic* MaterialInstance = CastChecked<UMaterialInstanceDynamic>(MaterialInterface);
		
		for (const TTuple<UE::Mutable::Private::FParameterKey, UE::Math::TVector4<float>>& Parameter : Material->Material->ColorParameters)
		{
			if (Parameter.Key.LayerIndex == INDEX_NONE)
			{
				FLinearColor Color = Parameter.Value;

				// HACK: We encode an invalid value (Nan) for table option "None.
				// Decoding "None" color parameters that use the material color
				if (FMath::IsNaN(Color.R))
				{
					FMaterialParameterInfo ParameterInfo(Parameter.Key.ParameterName);
					MaterialTemplate->GetVectorParameterValue(ParameterInfo, Color);
				}

				MaterialInstance->SetVectorParameterValue(Parameter.Key.ParameterName, Color);
			}
			else
			{
				FMaterialParameterInfo ParameterInfo = FMaterialParameterInfo(Parameter.Key.ParameterName, EMaterialParameterAssociation::LayerParameter, Parameter.Key.LayerIndex);
				MaterialInstance->SetVectorParameterValueByInfo(ParameterInfo, Parameter.Value);
			}
		}

		for (const TTuple<UE::Mutable::Private::FParameterKey, float>& Parameter : Material->Material->ScalarParameters)
		{
			if (Parameter.Key.LayerIndex == INDEX_NONE)
			{
				MaterialInstance->SetScalarParameterValue(Parameter.Key.ParameterName, Parameter.Value);
			}
			else
			{
				FMaterialParameterInfo ParameterInfo = FMaterialParameterInfo(Parameter.Key.ParameterName, EMaterialParameterAssociation::LayerParameter, Parameter.Key.LayerIndex);
				MaterialInstance->SetScalarParameterValueByInfo(ParameterInfo, Parameter.Value);
			}
		}

		for (int32 ImageIndex = 0; ImageIndex < Material->ImageCount; ++ImageIndex)
		{
			const TSharedRef<const FInstanceUpdateData::FImage>& Image = Context->InstanceUpdateData.Images[Material->FirstImage + ImageIndex];
				
			UTexture* Texture = BuildTexture(Context, Material->FirstImage + ImageIndex, bCanReuseMaterial);
				
			if (Image->MaterialLayer < 0)
			{
				MaterialInstance->SetTextureParameterValue(Image->ParameterName, Texture);
			}
			else
			{
				FMaterialParameterInfo ParameterInfo = FMaterialParameterInfo(Image->ParameterName, EMaterialParameterAssociation::LayerParameter, Image->MaterialLayer);
				MaterialInstance->SetTextureParameterValueByInfo(ParameterInfo, Texture);
			}

			if (!bDisableNotifyComponentsOfTextureUpdates)
			{
				Out.bNotifyPrimitiveUpdated = true;
			}
		}
	}

	if (MaterialInterface)
	{
		Context->Instance->GetPrivate()->Materials.AddUnique(MaterialInterface);
	}
		
	if (Context->bUseCaches)
	{
		if (bCanReuseMaterial)
		{
			CurrentMaterialReuseCache.Add(MaterialReuseCacheKey, TStrongObjectPtr(MaterialInterface));
		}
		else
		{
			// Add it to the CO cache. Currently, we do not have one for Materials.
		}
	}

	Context->MaterialUpdateCache.Add(Material->MaterialId, TStrongObjectPtr(MaterialInterface));

	Out.Material = MaterialInterface;
	
	return Out;
}


void UCustomizableInstancePrivate::BuildResources(const TSharedRef<FUpdateContextPrivate>& Context, UCustomizableObjectInstance* Public)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableInstancePrivate::BuildResources)

	UCustomizableObject* CustomizableObject = Context->Object.Get();

	const UModelResources& ModelResources = *CustomizableObject->GetPrivate()->GetModelResources();
	
	// Prepare the data to store in order to regenerate resources for this instance (usually texture mips).
	check(Context->LiveInstance)
	TSharedRef<FMutableUpdateImageContext> UpdateImageContext = MakeShared<FMutableUpdateImageContext>(Context->LiveInstance.ToSharedRef());
	UpdateImageContext->System = UCustomizableObjectSystem::GetInstance()->GetPrivate()->MutableSystem;
	UpdateImageContext->CustomizableObjectPathName = CustomizableObject->GetPathName();
	UpdateImageContext->InstancePathName = Public->GetPathName();
	UpdateImageContext->ModelStreamableBulkData = CustomizableObject->GetPrivate()->GetModelStreamableBulkData();

	// Cache the descriptor as a string if we want to later report it using our benchmark utility. 
	if (FLogBenchmarkUtil::IsBenchmarkingReportingEnabled())
	{
		UpdateImageContext->CapturedDescriptor = Context->GetCapturedDescriptor().ToString();
		if (GWorld)
		{
			UpdateImageContext->bLevelBegunPlay = GWorld->GetBegunPlay();
		}
	}

	Context->UpdateImageContext = UpdateImageContext;
	
	Materials.Empty();
	Textures.Empty();

	const int32 NumComponents = Context->InstanceUpdateData.Components.Num();
	
	TArray<bool> RecreateRenderStateOnInstanceComponent;
	RecreateRenderStateOnInstanceComponent.Init(false, NumComponents);

	TArray<bool> NotifyUpdateOnInstanceComponent;
	NotifyUpdateOnInstanceComponent.Init(false, NumComponents);

	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
	{
		const TSharedRef<const FInstanceUpdateData::FComponent>& Component = Context->InstanceUpdateData.Components[ComponentIndex];

		if (!ModelResources.ComponentNamesPerObjectComponent.IsValidIndex(Component->Id))
		{
			continue;
		}
		const FName& ComponentName = ModelResources.ComponentNamesPerObjectComponent[Component->Id];
		
		TObjectPtr<USkeletalMesh> SkeletalMesh = SkeletalMeshes.Contains(ComponentName) ? SkeletalMeshes[ComponentName] : nullptr;
		if (!SkeletalMesh)
		{
			continue;
		}

		const bool bUVChanged = Context->MeshChangedPerInstanceComponent[ComponentIndex];
		
		// If the mesh is not transient, it means it's pass-through so it should use material overrides and not be modified in any way
		const bool bIsTransientMesh = SkeletalMesh->HasAllFlags(EObjectFlags::RF_Transient);

		// It is not safe to replace the materials of a SkeletalMesh whose resources are initialized. Use overrides instead.
		const bool bUseOverrideMaterialsOnly = !bIsTransientMesh || (Context->bUseSkeletalMeshCache && SkeletalMesh->GetResourceForRendering()->IsInitialized());
		
		FCustomizableInstanceComponentData& ComponentData = GetComponentData(Component->Id);
		ComponentData.OverrideMaterials.Reset();

		TArray<FSkeletalMaterial> SkeletalMaterials;
		
		// SurfaceId per MaterialSlotIndex
		TArray<int32> SurfaceIdToMaterialIndex;

		MUTABLE_CPUPROFILER_SCOPE(BuildResources_LODLoop);

		for (int32 LODIndex = Context->FirstLODAvailable[ComponentName]; LODIndex < Component->LODCount; LODIndex++)
		{
			const TSharedRef<const FInstanceUpdateData::FLOD>& LOD = Context->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];
			
			if (!bUseOverrideMaterialsOnly && LODIndex < SkeletalMesh->GetLODNum())
			{
				SkeletalMesh->GetLODInfo(LODIndex)->LODMaterialMap.Reset();
			}

			// Pass-through components will not have a reference mesh.
			const FMutableRefSkeletalMeshData* RefSkeletalMeshData = nullptr;
			if (ModelResources.ReferenceSkeletalMeshesData.IsValidIndex(Component->Id))
			{
				RefSkeletalMeshData = &ModelResources.ReferenceSkeletalMeshesData[Component->Id];
			}

			for (int32 SurfaceIndex = 0; SurfaceIndex < LOD->SurfaceCount; ++SurfaceIndex)
			{
				const TSharedRef<const FInstanceUpdateData::FSurface>& Surface = Context->InstanceUpdateData.Surfaces[LOD->FirstSurface + SurfaceIndex];

				// Reuse MaterialSlot from the previous LOD.
				if (const int32 MaterialIndex = SurfaceIdToMaterialIndex.Find(Surface->SurfaceId); MaterialIndex != INDEX_NONE)
				{
					if (!bUseOverrideMaterialsOnly)
					{
						const int32 LODMaterialIndex = SkeletalMesh->GetLODInfo(LODIndex)->LODMaterialMap.Add(MaterialIndex);
						SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SurfaceIndex].MaterialIndex = LODMaterialIndex;
					}

					continue;
				}
				
				// This section will require a new slot
				SurfaceIdToMaterialIndex.Add(Surface->SurfaceId);

				// Add and set up the material data for this slot
				const int32 MaterialSlotIndex = SkeletalMaterials.Num();
				FSkeletalMaterial& MaterialSlot = SkeletalMaterials.AddDefaulted_GetRef();
				
				MaterialSlot.MaterialSlotName = Surface->MaterialSlotName;

#if WITH_EDITOR
				// Unique to avoid automatic merge of sections using same name when building the mesh in the editor.
				MaterialSlot.ImportedMaterialSlotName = FName(FString::Printf(TEXT("%s%i"), *MaterialSlot.MaterialSlotName.ToString(), MaterialSlotIndex));
#endif

				if (RefSkeletalMeshData)
				{
					SetMeshUVChannelDensity(MaterialSlot.UVChannelData, RefSkeletalMeshData->Settings.DefaultUVChannelDensity);
				}

				if (!bUseOverrideMaterialsOnly)
				{
					if (SkeletalMesh->GetResourceForRendering()->LODRenderData.IsValidIndex(LODIndex) &&
						SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.IsValidIndex(SurfaceIndex))
					{
						const int32 LODMaterialIndex = SkeletalMesh->GetLODInfo(LODIndex)->LODMaterialMap.Add(MaterialSlotIndex);
						SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SurfaceIndex].MaterialIndex = LODMaterialIndex;
					}
					else
					{
						ensure(false);
					}
				}

				FBuildMaterialOut Out = BuildMaterial(Context, Surface->MaterialIndex, !bUVChanged);
					
				if (!Out.Material)
				{
					Out.Material = UMaterial::GetDefaultMaterial(MD_Surface);
				}
					
				MaterialSlot.MaterialInterface = Out.Material;
				ComponentData.OverrideMaterials.Add(Out.Material);
				NotifyUpdateOnInstanceComponent[ComponentIndex] = Out.bNotifyPrimitiveUpdated;
			}
		}

		if (!bUseOverrideMaterialsOnly)
		{
			// Force recreate render state after replacing the materials to avoid a crash in the render pipeline if the old materials are GCed while in use.
			RecreateRenderStateOnInstanceComponent[ComponentIndex] |= SkeletalMesh->GetResourceForRendering()->IsInitialized() && SkeletalMesh->GetMaterials() != SkeletalMaterials;

			SkeletalMesh->SetMaterials(SkeletalMaterials);

#if WITH_EDITOR
			if (GEditor && RecreateRenderStateOnInstanceComponent[ComponentIndex])
			{
				// Close all open editors for this mesh to invalidate viewports.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(SkeletalMesh);
			}
#endif
		}

		// Ensure the number of materials is the same on both sides when using overrides. 
		//check(SkeletalMesh->GetMaterials().Num() == Materials.Num());

		// Component overlay material
		{
			ComponentData.OverlayMaterial = nullptr;
			if (Component->OverlayMaterialIndex != INDEX_NONE)
			{
				const FBuildMaterialOut Out = BuildMaterial(Context, Component->OverlayMaterialIndex, !bUVChanged);
				ComponentData.OverlayMaterial = Out.Material;
				
				NotifyUpdateOnInstanceComponent[ComponentIndex] = Out.bNotifyPrimitiveUpdated;
			}
		
			if (!bUseOverrideMaterialsOnly)
			{
				RecreateRenderStateOnInstanceComponent[ComponentIndex] |= SkeletalMesh->GetOverlayMaterial() != ComponentData.OverlayMaterial;
				SkeletalMesh->SetOverlayMaterial(ComponentData.OverlayMaterial);
			}
		}
		
		// MAT SLOT OVERLAY MATERIALS
		ComponentData.OverlayMaterials.Reset();
		for (int32 OverlayMaterialIndex = 0; OverlayMaterialIndex < Component->OverlayMaterialCount; ++OverlayMaterialIndex)
		{
			const FInstanceUpdateData::FOverlayMaterial& OverlayMaterial = Context->InstanceUpdateData.OverlayMaterials[Component->FirstOverlayMaterial + OverlayMaterialIndex];
				
			const int32 SlotIndex = SkeletalMesh->GetMaterials().IndexOfByPredicate([&OverlayMaterial](const FSkeletalMaterial& Material)
			{
				return Material.MaterialSlotName == OverlayMaterial.SlotName;
			});
				
			if (SlotIndex != INDEX_NONE)
			{
				const FBuildMaterialOut NewMaterial = BuildMaterial(Context, OverlayMaterial.MaterialIndex, !bUVChanged);
				
				if (!ComponentData.OverlayMaterials.IsValidIndex(SlotIndex))
				{
					ComponentData.OverlayMaterials.SetNum(SlotIndex + 1);
				}
				
				ComponentData.OverlayMaterials[SlotIndex] = NewMaterial.Material;
					
				NotifyUpdateOnInstanceComponent[ComponentIndex] = NewMaterial.bNotifyPrimitiveUpdated;
			}
		}
		
		// MAT SLOT OVERRIDE MATERIALS
		for (int32 MaterialIndex = 0; MaterialIndex < Component->OverrideMaterialCount; ++MaterialIndex)
		{
			const FInstanceUpdateData::FOverrideMaterial& OverrideMaterial = Context->InstanceUpdateData.OverrideMaterials[Component->FirstOverrideMaterial + MaterialIndex];
				
			const int32 SlotIndex = SkeletalMesh->GetMaterials().IndexOfByPredicate([&OverrideMaterial](const FSkeletalMaterial& Material)
			{
				return Material.MaterialSlotName == OverrideMaterial.SlotName;
			});
				
			if (SlotIndex != INDEX_NONE)
			{
				const FBuildMaterialOut Out = BuildMaterial(Context, OverrideMaterial.MaterialIndex, !bUVChanged);
				
				if (!ComponentData.OverrideMaterials.IsValidIndex(SlotIndex))
				{
					ComponentData.OverrideMaterials.SetNum(SlotIndex + 1);
				}

				ComponentData.OverrideMaterials[SlotIndex] = Out.Material;
					
				NotifyUpdateOnInstanceComponent[ComponentIndex] = Out.bNotifyPrimitiveUpdated;
			}
		}
	
	}

	// Force recreate render state if the mesh is reused and the materials have changed.
	// TODO: MTBL-1697 Remove after merging ConvertResources and Callbacks.
	if (RecreateRenderStateOnInstanceComponent.Find(true) != INDEX_NONE || NotifyUpdateOnInstanceComponent.Find(true) != INDEX_NONE)
	{
		MUTABLE_CPUPROFILER_SCOPE(BuildResources_RecreateRenderState);

		for (UCustomizableObjectInstanceUsage* CustomizableObjectInstanceUsage : InstanceUsages)
		{
#if WITH_EDITOR
			if (CustomizableObjectInstanceUsage->GetPrivate()->bIsNetModeDedicatedServer)
			{
				continue;
			}
#endif

			const FName& ComponentName = CustomizableObjectInstanceUsage->GetComponentName();
			FCustomizableInstanceComponentData* ComponentData = GetComponentData(ComponentName);
			if (!ComponentData)
			{
				continue;
			}
			
			int32 ComponentIndex = -1;
			for (int32 CurrentInstanceIndex = 0; CurrentInstanceIndex<Context->InstanceUpdateData.Components.Num(); ++CurrentInstanceIndex)
			{
				if (Context->InstanceUpdateData.Components[CurrentInstanceIndex]->Id == ComponentData->ComponentId)
				{
					ComponentIndex = CurrentInstanceIndex;
					break;
				}
			}

			bool bDoRecreateRenderStateOnComponent = RecreateRenderStateOnInstanceComponent.IsValidIndex(ComponentIndex) && RecreateRenderStateOnInstanceComponent[ComponentIndex];
			bool bDoNotifyUpdateOnComponent = NotifyUpdateOnInstanceComponent.IsValidIndex(ComponentIndex) && NotifyUpdateOnInstanceComponent[ComponentIndex];

			if (!bDoRecreateRenderStateOnComponent && !bDoNotifyUpdateOnComponent)
			{
				continue;
			}

			USkeletalMeshComponent* AttachedParent = CustomizableObjectInstanceUsage->GetAttachParent();
			TObjectPtr<USkeletalMesh>* SkeletalMesh = SkeletalMeshes.Find(ComponentName);
			if (!AttachedParent || (SkeletalMesh && AttachedParent->GetSkeletalMeshAsset() != *SkeletalMesh))
			{
				continue;
			}

			if (bDoRecreateRenderStateOnComponent)
			{
				AttachedParent->RecreateRenderState_Concurrent();
			}
			else if (bDoNotifyUpdateOnComponent)
			{
				IStreamingManager::Get().NotifyPrimitiveUpdated(AttachedParent);
			}
		}
	}

	{
		MUTABLE_CPUPROFILER_SCOPE(BuildResources_Exchange);

		Context->Instance->GetPrivate()->LastMaterialsReuseCache = MoveTemp(Context->Instance->GetPrivate()->CurrentMaterialsReuseCache);
		Context->Instance->GetPrivate()->LastTextureReuseCache = MoveTemp(Context->Instance->GetPrivate()->CurrentTextureReuseCache);
	}
}


void UCustomizableObjectInstance::SetReplacePhysicsAssets(bool bReplaceEnabled)
{
	bReplaceEnabled ? GetPrivate()->SetCOInstanceFlags(ReplacePhysicsAssets) : GetPrivate()->ClearCOInstanceFlags(ReplacePhysicsAssets);
}


void UCustomizableObjectInstance::SetKeepOwnershipOfGeneratedResources(bool bEnabled)
{
	GetPrivate()->bKeepOwnershipOfGeneratedResources = bEnabled;
}


#if WITH_EDITOR
void UCustomizableObjectInstance::Bake(const FBakingConfiguration& InBakingConfiguration)
{
	if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		Module->BakeCustomizableObjectInstance(this, InBakingConfiguration);
	}
	else
	{
		// Notify of the error
		UE_LOGF(LogMutable, Error, "The module \" ICustomizableObjectEditorModule \" could not be loaded and therefore the baking operation could not be started.");
		if (InBakingConfiguration.OnBakeOperationCompletedCallback.IsBound())
		{
			FCustomizableObjectInstanceBakeOutput Output;
			Output.bWasBakeSuccessful = false;
			Output.SavedPackages.Empty();
			InBakingConfiguration.OnBakeOperationCompletedCallback.Execute(Output);
		}
	}
}
#endif


USkeletalMesh* UCustomizableObjectInstance::GetComponentMeshSkeletalMesh(const FName& ComponentName) const
{
	TObjectPtr<USkeletalMesh>* Result = GetPrivate()->SkeletalMeshes.Find(ComponentName);
	return Result ? *Result : nullptr;
}


USkeletalMesh* UCustomizableObjectInstance::GetSkeletalMeshComponentSkeletalMesh(const FName& ComponentName) const
{
	return GetComponentMeshSkeletalMesh(ComponentName);
}


bool UCustomizableObjectInstance::HasAnySkeletalMesh() const
{
	return !GetPrivate()->SkeletalMeshes.IsEmpty();
}


bool UCustomizableObjectInstance::HasAnyParameters() const
{
	return Descriptor.HasAnyParameters();	
}


TArray<FName> UCustomizableObjectInstance::GetComponentNames() const
{
	TArray<FName> GeneratedComponents;

	// For now, the instances don't really hold a direct array of generated components FNames. 
	// They can be identified with the ones having a valid SkeletalMesh in the SkeletalMeshes array, but this will 
	// not longer work when we have components that don't have a SkeletalMesh, like grooms, or panel clothing. (TODO)
	for ( const TPair<FName, TObjectPtr<USkeletalMesh>>& Entry : GetPrivate()->SkeletalMeshes )
	{
		if (Entry.Value)
		{
			GeneratedComponents.Add(Entry.Key);
		}
	}

	return GeneratedComponents;
}


TSubclassOf<UAnimInstance> UCustomizableObjectInstance::GetAnimBP(FName ComponentName, const FName& SlotName) const
{
	TObjectPtr<USkeletalMesh>* SkeletalMesh = GetPrivate()->SkeletalMeshes.Find(ComponentName);
	if (!SkeletalMesh)
	{
		return nullptr;
	}
	
	UCustomizableObjectInstanceUserData* AssetUserData = Cast<UCustomizableObjectInstanceUserData>((*SkeletalMesh)->GetAssetUserDataOfClass(UCustomizableObjectInstanceUserData::StaticClass()));
	if (!AssetUserData)
	{
		return nullptr;
	}
	
	FCustomizableObjectAnimationSlot* Slot = AssetUserData->AnimationSlots.FindByPredicate([&](const FCustomizableObjectAnimationSlot& Element)
	{
		return Element.Name == SlotName;
	});

	if (!Slot)
	{
		return nullptr;	
	}
	
	return Slot->AnimInstance.Get();
}


FGameplayTagContainer UCustomizableObjectInstance::GetAnimationGameplayTags() const
{
	FGameplayTagContainer Container;
	
	for (const TTuple<FName, TObjectPtr<USkeletalMesh>>& Pair : GetPrivate()->SkeletalMeshes)
	{
		if (UCustomizableObjectInstanceUserData* AssetUserData = Cast<UCustomizableObjectInstanceUserData>(Pair.Value->GetAssetUserDataOfClass(UCustomizableObjectInstanceUserData::StaticClass())))
		{
			Container.AppendTags(AssetUserData->AnimationGameplayTag);
		}
	}
	
	return Container;
}

namespace UE::Mutable::Private
{
	template<typename DELEGATE>
	void InternalForEachAnimInstance(UCustomizableInstancePrivate* Private, FName ComponentName, DELEGATE Delegate)
	{
		// allow us to log out both bad states with one pass
		bool bAnyErrors = false;

		if (!Delegate.IsBound())
		{
			FString ErrorMsg = FString::Printf(TEXT("Attempting to iterate over AnimInstances with an unbound delegate for component [%s]."), *ComponentName.ToString());
			UE_LOGF(LogMutable, Warning, "%ls", *ErrorMsg);
#if WITH_EDITOR
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
			MessageLogModule.RegisterLogListing(FName("Mutable"), FText::FromString(FString("Mutable")));
			FMessageLog MessageLog("Mutable");

			MessageLog.Notify(FText::FromString(ErrorMsg), EMessageSeverity::Warning, true);
#endif
			bAnyErrors = true;
		}

		if (bAnyErrors)
		{
			return;
		}
		
		const TObjectPtr<USkeletalMesh>* SkeletalMesh = Private->SkeletalMeshes.Find(ComponentName);
		if (!SkeletalMesh)
		{
			return;
		}
		
		UCustomizableObjectInstanceUserData* AssetUserData = Cast<UCustomizableObjectInstanceUserData>((*SkeletalMesh)->GetAssetUserDataOfClass(UCustomizableObjectInstanceUserData::StaticClass()));
		if (!AssetUserData)
		{
			return;
		}
		
		for (const FCustomizableObjectAnimationSlot& Slot :  AssetUserData->AnimationSlots)
		{
			const TSoftClassPtr<UAnimInstance>& AnimBP = Slot.AnimInstance;

			// if this _can_ resolve to a real AnimBP
			if (!AnimBP.IsNull())
			{
				// force a load right now - we don't know whether we would have loaded already - this could be called in editor
				const TSubclassOf<UAnimInstance> LiveAnimBP = Private::LoadClass(AnimBP);
				if (LiveAnimBP)
				{
					Delegate.Execute(Slot.Name, LiveAnimBP);
				}
			}
		}
	}
}


void UCustomizableObjectInstance::ForEachComponentAnimInstance(FName ComponentName, FEachComponentAnimInstanceClassDelegate Delegate) const
{
	UE::Mutable::Private::InternalForEachAnimInstance<>(GetPrivate(), ComponentName, Delegate);
}


void UCustomizableObjectInstance::ForEachComponentAnimInstance(FName ComponentName, FEachComponentAnimInstanceClassNativeDelegate Delegate) const
{
	UE::Mutable::Private::InternalForEachAnimInstance<>(GetPrivate(), ComponentName, Delegate);
}


bool UCustomizableObjectInstance::AnimInstanceNeedsFixup(TSubclassOf<UAnimInstance> AnimInstanceClass) const
{
	return PrivateData->AnimBpPhysicsAssets.Contains(AnimInstanceClass);
}


void UCustomizableObjectInstance::AnimInstanceFixup(UAnimInstance* InAnimInstance) const
{
	if (!InAnimInstance)
	{
		return;
	}

	TSubclassOf<UAnimInstance> AnimInstanceClass = InAnimInstance->GetClass();

	const TArray<FAnimInstanceOverridePhysicsAsset>* AnimInstanceOverridePhysicsAssets = 
			PrivateData->GetGeneratedPhysicsAssetsForAnimInstance(AnimInstanceClass);
	
	if (!AnimInstanceOverridePhysicsAssets)
	{
		return;
	}

	// Swap RigidBody anim nodes override physics assets with mutable generated ones.
	if (UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstanceClass))
	{
		bool bPropertyMismatchFound = false;
		const int32 AnimNodePropertiesNum = AnimClass->AnimNodeProperties.Num();

		for (const FAnimInstanceOverridePhysicsAsset& PropIndexAndAsset : *AnimInstanceOverridePhysicsAssets)
		{
			check(PropIndexAndAsset.PropertyIndex >= 0);
			if (PropIndexAndAsset.PropertyIndex >= AnimNodePropertiesNum)
			{
				bPropertyMismatchFound = true;
				continue;
			}

			const int32 AnimNodePropIndex = PropIndexAndAsset.PropertyIndex;

			FStructProperty* StructProperty = AnimClass->AnimNodeProperties[AnimNodePropIndex];

			if (!ensure(StructProperty))
			{
				bPropertyMismatchFound = true;
				continue;
			}

			const bool bIsRigidBodyNode = StructProperty->Struct->IsChildOf(FAnimNode_RigidBody::StaticStruct());

			if (!bIsRigidBodyNode)
			{
				bPropertyMismatchFound = true;
				continue;
			}

			FAnimNode_RigidBody* RbanNode = StructProperty->ContainerPtrToValuePtr<FAnimNode_RigidBody>(InAnimInstance);

			if (!ensure(RbanNode))
			{
				bPropertyMismatchFound = true;
				continue;
			}

			RbanNode->OverridePhysicsAsset = PropIndexAndAsset.PhysicsAsset;
		}
#if WITH_EDITOR
		if (bPropertyMismatchFound)
		{
			UE_LOGF(LogMutable, Warning, "AnimBp %ls is not in sync with the data stored in the CO %ls. A CO recompilation may be needed.",
				*AnimInstanceClass.Get()->GetName(), 
				*GetCustomizableObject()->GetName());
		}
#endif
	}
}

const TArray<FAnimInstanceOverridePhysicsAsset>* UCustomizableInstancePrivate::GetGeneratedPhysicsAssetsForAnimInstance(TSubclassOf<UAnimInstance> AnimInstanceClass) const
{
	const FAnimBpGeneratedPhysicsAssets* Found = AnimBpPhysicsAssets.Find(AnimInstanceClass);

	if (!Found)
	{
		return nullptr;
	}

	return &Found->AnimInstancePropertyIndexAndPhysicsAssets;
}


TSet<UAssetUserData*> UCustomizableObjectInstance::GetMergedAssetUserData(const FName& ComponentName) const
{
	TObjectPtr<USkeletalMesh>* SkeletalMesh = GetPrivate()->SkeletalMeshes.Find(ComponentName);
	if (!SkeletalMesh)
	{
		return {};
	}
	
	TSet<UAssetUserData*> Set;
		
	for (UAssetUserData* Elem : *(*SkeletalMesh)->GetAssetUserDataArray())
	{
		// Have to convert to UAssetUserData* because BP functions don't support TObjectPtr
		Set.Add(Elem);
	}

	return Set;
}


#if WITH_EDITORONLY_DATA
void CalculateBonesToRemove(const FSkeletalMeshLODRenderData& LODResource, const FReferenceSkeleton& RefSkeleton, TArray<FBoneReference>& OutBonesToRemove)
{
	const int32 NumBones = RefSkeleton.GetNum();
	OutBonesToRemove.Empty(NumBones);

	TArray<bool> RemovedBones;
	RemovedBones.Init(true, NumBones);

	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		if (LODResource.RequiredBones.Find((uint16)BoneIndex) != INDEX_NONE)
		{
			RemovedBones[BoneIndex] = false;
			continue;
		}

		const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
		if (RemovedBones.IsValidIndex(ParentIndex) && !RemovedBones[ParentIndex])
		{
			OutBonesToRemove.Add(RefSkeleton.GetBoneName(BoneIndex));
		}
	}
}

void UCustomizableInstancePrivate::RegenerateImportedModels(const TSharedRef<FUpdateContextPrivate>& OperationData)
{
	MUTABLE_EDITOR_CPUPROFILER_SCOPE(RegenerateEditorImportedModels);

	struct FMeshDataConvertJob
	{
		int32 NumIndices = 0;
		int32 IndicesOffset = 0;
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = nullptr;
		uint32* DestIndexBuffer = nullptr;

		int32 NumVertices = 0;
		int32 VerticesOffset = 0;
		const FStaticMeshVertexBuffers*	StaticVertexBuffers = nullptr;
		const FSkinWeightVertexBuffer* SkinWeightVertexBuffer = nullptr;
		FSoftSkinVertex* DestVertexBuffer = nullptr;
	};
	
	constexpr int32 MaxJobCost = 1 << 18;
	constexpr int32 MaxVerticesPerJob = FMath::Max<int32>(1, MaxJobCost / sizeof(FSoftSkinVertex));
	constexpr int32 MaxIndicesPerJob =  FMath::Max<int32>(1, MaxJobCost / sizeof(int32));

	TArray<FMeshDataConvertJob, TInlineAllocator<64>> Jobs;   
	TArray<int32, TInlineAllocator<64>> JobRanges;
	JobRanges.Add(0);
	
	for (const TTuple<FName, TObjectPtr<USkeletalMesh>>& Tuple : SkeletalMeshes)
	{
		USkeletalMesh* SkeletalMesh = Tuple.Get<1>();
		
		if (!SkeletalMesh)
		{
			continue;
		}

		const bool bIsTransientMesh = static_cast<bool>(SkeletalMesh->HasAllFlags(EObjectFlags::RF_Transient));

		if (!bIsTransientMesh)
		{
			// This must be a pass-through referenced mesh so don't do anything to it
			continue;
		}

		FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		if (!RenderData || RenderData->IsInitialized())
		{
			continue;
		}

		for (UClothingAssetBase* ClothingAssetBase : SkeletalMesh->GetMeshClothingAssets())
		{
			if (!ClothingAssetBase)
			{
				continue;
			}

			UClothingAssetCommon* ClothAsset = Cast<UClothingAssetCommon>(ClothingAssetBase);

			if (!ClothAsset)
			{
				continue;
			}

			if (!ClothAsset->LodData.Num())
			{
				continue;
			}

			for (FClothLODDataCommon& ClothLodData : ClothAsset->LodData)
			{
				ClothLodData.PointWeightMaps.Empty(16);
				for (TPair<uint32, FPointWeightMap>& WeightMap : ClothLodData.PhysicalMeshData.WeightMaps)
				{
					if (WeightMap.Value.Num())
					{
						FPointWeightMap& PointWeightMap = ClothLodData.PointWeightMaps.AddDefaulted_GetRef();
						PointWeightMap.Initialize(WeightMap.Value, WeightMap.Key);
					}
				}
			}
		}

		FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		ImportedModel->bGuidIsHash = false;
		ImportedModel->SkeletalMeshModelGUID = FGuid::NewGuid();

		ImportedModel->LODModels.Empty();
	
		int32 OriginalIndex = 0;
		for (int32 LODIndex = 0; LODIndex < RenderData->LODRenderData.Num(); ++LODIndex)
		{
			ImportedModel->LODModels.Add(new FSkeletalMeshLODModel());
			FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];

			FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIndex];

			LODModel.ActiveBoneIndices = LODRenderData.ActiveBoneIndices;
			LODModel.NumTexCoords = LODRenderData.GetNumTexCoords();
			LODModel.RequiredBones = LODRenderData.RequiredBones;
			LODModel.NumVertices = LODRenderData.GetNumVertices();

			// Indices
			if (LODRenderData.MultiSizeIndexContainer.IsIndexBufferValid())
			{
				const FRawStaticIndexBuffer16or32Interface* IndexBuffer =
						LODRenderData.MultiSizeIndexContainer.GetIndexBuffer();

				const int32 NumIndices = IndexBuffer->Num();
				LODModel.IndexBuffer.SetNum(NumIndices);

				uint32* BaseDestIndexBuffer = LODModel.IndexBuffer.GetData();
				
				const int32 NumIndicesJobs = FMath::DivideAndRoundUp(NumIndices, MaxIndicesPerJob);

				int32 CurrentJobIndex = Jobs.Num();
				Jobs.SetNum(Jobs.Num() + NumIndicesJobs);
		
				for (int32 I = 0; I < NumIndicesJobs; ++I)
				{
					FMeshDataConvertJob Job;
					Job.NumIndices = FMath::Min(MaxIndicesPerJob, NumIndices - I*MaxIndicesPerJob); 
					Job.IndexBuffer = IndexBuffer;
					Job.IndicesOffset = I*MaxIndicesPerJob; 
					Job.DestIndexBuffer = BaseDestIndexBuffer + I*MaxIndicesPerJob;

					Jobs[I + CurrentJobIndex] = Job;  
				}
			}

			LODModel.Sections.SetNum(LODRenderData.RenderSections.Num());

			for (int32 SectionIndex = 0; SectionIndex < LODRenderData.RenderSections.Num(); ++SectionIndex)
			{
				check(!LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis());
				
				const FSkelMeshRenderSection& RenderSection = LODRenderData.RenderSections[SectionIndex];
				FSkelMeshSection& ImportedSection = ImportedModel->LODModels[LODIndex].Sections[SectionIndex];

				ImportedSection.CorrespondClothAssetIndex = RenderSection.CorrespondClothAssetIndex;
				ImportedSection.ClothingData = RenderSection.ClothingData;

				if (RenderSection.ClothMappingDataLODs.Num())
				{
					TArray<FMeshToMeshVertData>& ImportedClothMappingData = ImportedSection.ClothMappingDataLODs.AddDefaulted_GetRef();
					
					const int32 NumClothVerts = LODRenderData.ClothVertexBuffer.GetNumVertices();
					ImportedClothMappingData.SetNumUninitialized(NumClothVerts);
				
					for (int32 ClothVertDataIndex = 0; ClothVertDataIndex < NumClothVerts; ++ClothVertDataIndex)
					{
						ImportedClothMappingData[ClothVertDataIndex] = LODRenderData.ClothVertexBuffer.MappingData(ClothVertDataIndex);
					}
				}

				// Vertices
				ImportedSection.NumVertices = RenderSection.NumVertices;
				ImportedSection.SoftVertices.Empty(RenderSection.NumVertices);
				ImportedSection.SoftVertices.AddUninitialized(RenderSection.NumVertices);
				ImportedSection.bUse16BitBoneIndex = LODRenderData.DoesVertexBufferUse16BitBoneIndex();

				const int32 SectionNumVertices = RenderSection.NumVertices;
				const int32 SectionBaseVertexIndex = RenderSection.BaseVertexIndex;
				const FStaticMeshVertexBuffers*	StaticVertexBuffers = &LODRenderData.StaticVertexBuffers;
				const FSkinWeightVertexBuffer* SkinWeightVertexBuffer = &LODRenderData.SkinWeightVertexBuffer;

				FSoftSkinVertex* BaseSectionSoftVertex = ImportedSection.SoftVertices.GetData();

				int32 NumSectionJobs = FMath::DivideAndRoundUp<int32>(RenderSection.NumVertices, MaxVerticesPerJob);

				int32 FirstSectionJobIndex = Jobs.Num();
				Jobs.SetNum(Jobs.Num() + NumSectionJobs);

				for (int32 I = 0; I < NumSectionJobs; ++I)
				{
					FMeshDataConvertJob Job;
					Job.NumVertices = FMath::Min(MaxVerticesPerJob, SectionNumVertices - I*MaxVerticesPerJob); 
					Job.StaticVertexBuffers = StaticVertexBuffers;
					Job.SkinWeightVertexBuffer = SkinWeightVertexBuffer;
					Job.VerticesOffset = SectionBaseVertexIndex + I*MaxVerticesPerJob; 
					Job.DestVertexBuffer = BaseSectionSoftVertex + I*MaxVerticesPerJob;

					Jobs[I + FirstSectionJobIndex] = Job;  
				}

				// Triangles
				ImportedSection.NumTriangles = RenderSection.NumTriangles;
				ImportedSection.BaseIndex = RenderSection.BaseIndex;
				ImportedSection.BaseVertexIndex = RenderSection.BaseVertexIndex;
				ImportedSection.BoneMap = RenderSection.BoneMap;

				// Add bones to remove
				CalculateBonesToRemove(LODRenderData, SkeletalMesh->GetRefSkeleton(), SkeletalMesh->GetLODInfo(LODIndex)->BonesToRemove);

				const TArray<int32>& LODMaterialMap = SkeletalMesh->GetLODInfo(LODIndex)->LODMaterialMap;

				if (LODMaterialMap.IsValidIndex(RenderSection.MaterialIndex))
				{
					ImportedSection.MaterialIndex = LODMaterialMap[RenderSection.MaterialIndex];
				}
				else
				{
					// The material should have been in the LODMaterialMap
					ensureMsgf(false, TEXT("Unexpected material index in UCustomizableInstancePrivate::RegenerateImportedModel"));

					// Fallback index, may shift materials around sections
					if (SkeletalMesh->GetMaterials().IsValidIndex(RenderSection.MaterialIndex))
					{
						ImportedSection.MaterialIndex = RenderSection.MaterialIndex;
					}
					else
					{
						ImportedSection.MaterialIndex = 0;
					}
				}

				ImportedSection.MaxBoneInfluences = RenderSection.MaxBoneInfluences;
				ImportedSection.OriginalDataSectionIndex = OriginalIndex++;

				FSkelMeshSourceSectionUserData& SectionUserData = LODModel.UserSectionsData.FindOrAdd(ImportedSection.OriginalDataSectionIndex);
				SectionUserData.bCastShadow = RenderSection.bCastShadow;
				SectionUserData.bRecomputeTangent = RenderSection.bRecomputeTangent;
				SectionUserData.bDisabled = RenderSection.bDisabled;

				SectionUserData.CorrespondClothAssetIndex = RenderSection.CorrespondClothAssetIndex;
				SectionUserData.ClothingData.AssetGuid = RenderSection.ClothingData.AssetGuid;
				SectionUserData.ClothingData.AssetLodIndex = RenderSection.ClothingData.AssetLodIndex;
				
				LODModel.SyncronizeUserSectionsDataArray();

				// DDC keys
				const USkeletalMeshLODSettings* LODSettings = SkeletalMesh->GetLODSettings();
				const bool bValidLODSettings = LODSettings && LODSettings->GetNumberOfSettings() > LODIndex;
				const FSkeletalMeshLODGroupSettings* SkeletalMeshLODGroupSettings = bValidLODSettings ? &LODSettings->GetSettingsForLODLevel(LODIndex) : nullptr;

				FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
				LODInfo->BuildGUID = LODInfo->ComputeDeriveDataCacheKey(SkeletalMeshLODGroupSettings);

				LODModel.BuildStringID = LODModel.GetLODModelDeriveDataKey();
			}

			// Skin weight profiles: inverse of FMeshUtilities::GenerateRuntimeSkinWeightData.
			// Rebuild FImportedSkinWeightProfileData (one FRawSkinWeight per LOD vertex) from the
			// runtime override data, falling back to the base skin weight buffer for vertices
			// that have no override.
			{
				LODModel.SkinWeightProfiles.Empty();

				const TArray<FSkinWeightProfileInfo>& MeshSkinWeightProfiles = SkeletalMesh->GetSkinWeightProfiles();
				if (MeshSkinWeightProfiles.Num() > 0)
				{
					const FSkinWeightVertexBuffer& BaseSkinWeightBuffer = LODRenderData.SkinWeightVertexBuffer;
					const int32 NumLODVertices = LODRenderData.GetNumVertices();
					const int32 MaxBoneInfluences = BaseSkinWeightBuffer.GetMaxBoneInfluences();
					const bool bUseHighPrecisionWeights = BaseSkinWeightBuffer.Use16BitBoneWeight();

					for (const FSkinWeightProfileInfo& ProfileInfo : MeshSkinWeightProfiles)
					{
						const FRuntimeSkinWeightProfileData* RuntimeProfileData = LODRenderData.SkinWeightProfilesData.GetOverrideData(ProfileInfo.Name);
						if (!RuntimeProfileData)
						{
							continue;
						}

						FImportedSkinWeightProfileData& ImportedProfileData = LODModel.SkinWeightProfiles.Add(ProfileInfo.Name);
						ImportedProfileData.SkinWeights.SetNumZeroed(NumLODVertices);

						const uint32 NumWeightsPerVertex = RuntimeProfileData->NumWeightsPerVertex;
						const bool b16BitBoneIndices = RuntimeProfileData->b16BitBoneIndices;
						const uint32 BoneIndexByteSize = b16BitBoneIndices ? sizeof(FBoneIndex16) : sizeof(FBoneIndex8);
						const uint32 BoneWeightByteSize = bUseHighPrecisionWeights ? sizeof(uint16) : sizeof(uint8);

						for (int32 VertexIndex = 0; VertexIndex < NumLODVertices; ++VertexIndex)
						{
							FRawSkinWeight& RawSkinWeight = ImportedProfileData.SkinWeights[VertexIndex];

							if (const uint32* InfluenceOffsetPtr = RuntimeProfileData->VertexIndexToInfluenceOffset.Find(VertexIndex))
							{
								const uint32 BoneIDsByteOffset = (*InfluenceOffsetPtr) * NumWeightsPerVertex * BoneIndexByteSize;
								const uint32 BoneWeightsByteOffset = (*InfluenceOffsetPtr) * NumWeightsPerVertex * BoneWeightByteSize;

								if (b16BitBoneIndices)
								{
									FMemory::Memcpy(&RawSkinWeight.InfluenceBones[0], &RuntimeProfileData->BoneIDs[BoneIDsByteOffset], NumWeightsPerVertex* BoneIndexByteSize);
								}
								else
								{
									for (uint32 InfluenceIndex = 0; InfluenceIndex < NumWeightsPerVertex; ++InfluenceIndex)
									{
										RawSkinWeight.InfluenceBones[InfluenceIndex] = RuntimeProfileData->BoneIDs[BoneIDsByteOffset + InfluenceIndex];
									}
								}

								if (bUseHighPrecisionWeights)
								{
									FMemory::Memcpy(&RawSkinWeight.InfluenceWeights[0], &RuntimeProfileData->BoneWeights[BoneWeightsByteOffset], NumWeightsPerVertex* BoneWeightByteSize);
								}
								else
								{
									for (uint32 InfluenceIndex = 0; InfluenceIndex < NumWeightsPerVertex; ++InfluenceIndex)
									{
										RawSkinWeight.InfluenceWeights[InfluenceIndex] = RuntimeProfileData->BoneWeights[BoneWeightsByteOffset + InfluenceIndex];
									}
								}
							}
							else
							{
								for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; ++InfluenceIndex)
								{
									RawSkinWeight.InfluenceBones[InfluenceIndex] = BaseSkinWeightBuffer.GetBoneIndex(VertexIndex, InfluenceIndex);
									RawSkinWeight.InfluenceWeights[InfluenceIndex] = BaseSkinWeightBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
								}
							}
						}
					}
				}
			}
		}

		// Try to bundle Jobs so all cost roughly the same. Large Jobs are already split so they cost about MaxJobCost.
		// It uses a gready approach and assumes in general Jobs are sorted by cost.
		const int32 NumJobs = Jobs.Num();
		for (int32 JobIndex = 0; JobIndex < NumJobs;)
		{
			int32 RangeJobCost = 0;
			for (; JobIndex < NumJobs;)
			{
				int32 CurrentJobCost = Jobs[JobIndex].NumVertices*sizeof(FSoftSkinVertex) + Jobs[JobIndex].NumIndices*sizeof(int32);

				RangeJobCost += CurrentJobCost;
				if (RangeJobCost >= MaxJobCost)
				{
					// Go to the next Job if the current job alone cost is larger than MaxJobCost
					// and no other job has been processed for the range.
					JobIndex += static_cast<int32>(CurrentJobCost == RangeJobCost);
					break;
				}

				++JobIndex;
			}

			JobRanges.Add(JobIndex);
		}
	}

	{
		MUTABLE_CPUPROFILER_SCOPE(DoImportedModelMeshDataConversion)

		ParallelFor(JobRanges.Num() - 1, [&JobRanges, &Jobs](int32 JobId)
		{
			int32 JobRangeBegin = JobRanges[JobId];
			int32 JobRangeEnd = JobRanges[JobId + 1];
			for (int32 J = JobRangeBegin; J < JobRangeEnd; ++J)
			{
				FMeshDataConvertJob Job = Jobs[J];
		
				if (Job.NumIndices > 0)
				{
					MUTABLE_CPUPROFILER_SCOPE(DoImportedModelMeshDataConversion_Indices)

					for (int32 Index = 0; Index < Job.NumIndices; ++Index)
					{
						Job.DestIndexBuffer[Index] = Job.IndexBuffer->Get(Job.IndicesOffset + Index);
					}
				}

				if (Job.NumVertices > 0)
				{
					MUTABLE_CPUPROFILER_SCOPE(DoImportedModelMeshDataConversion_Vertices)

					check(Job.StaticVertexBuffers);
					check(Job.SkinWeightVertexBuffer);
					check(Job.DestVertexBuffer);

					const FPositionVertex* PositionBuffer = 
							static_cast<const FPositionVertex*>(Job.StaticVertexBuffers->PositionVertexBuffer.GetVertexData()) + Job.VerticesOffset;

					const FPackedNormal* TangentBuffer = 
							static_cast<const FPackedNormal*>(Job.StaticVertexBuffers->StaticMeshVertexBuffer.GetTangentData()) + Job.VerticesOffset*2;

					const int32 NumTexCoords = Job.StaticVertexBuffers->StaticMeshVertexBuffer.GetNumTexCoords();
					const int32 UVSize = Job.StaticVertexBuffers->StaticMeshVertexBuffer.GetUseFullPrecisionUVs() ? 2 * sizeof(float) : 2 * sizeof(FFloat16);
					const uint8* TexCoordBuffer = 
							static_cast<const uint8*>(Job.StaticVertexBuffers->StaticMeshVertexBuffer.GetTexCoordData()) + Job.VerticesOffset*NumTexCoords*UVSize;
					
					const FColor* ColorBuffer = 
							static_cast<const FColor*>(Job.StaticVertexBuffers->ColorVertexBuffer.GetVertexData());
			
					const bool bHasColor = !!ColorBuffer;
					ColorBuffer += Job.VerticesOffset;

					const FSkinWeightVertexBuffer* SkinWeightBuffer = Job.SkinWeightVertexBuffer;

					const int32 MaxBoneInfluences = Job.SkinWeightVertexBuffer->GetMaxBoneInfluences();

					for (int32 JobVertexIndex = 0; JobVertexIndex < Job.NumVertices; ++JobVertexIndex)
					{
						FSoftSkinVertex* Vertex = Job.DestVertexBuffer + JobVertexIndex;
						FMemory::Memzero(Vertex, sizeof(FSoftSkinVertex));

						Vertex->Position = PositionBuffer[JobVertexIndex].Position;

						const FPackedNormal* Tangent = TangentBuffer + JobVertexIndex*2;
					
						Vertex->TangentX = Tangent[0].ToFVector3f();
						Vertex->TangentZ = Tangent[1].ToFVector3f();
						float TangentSign = Tangent[1].Vector.W < 0 ? -1.0f : 1.0f;
						Vertex->TangentY = FVector3f::CrossProduct(Vertex->TangentZ, Vertex->TangentX) * TangentSign;

						const void* TexCoord = TexCoordBuffer + JobVertexIndex*NumTexCoords*UVSize;

						// Switch based jumptable.
						if (UVSize == 4)
						{
							const FFloat16* TypedSource = reinterpret_cast<const FFloat16*>(TexCoord);
							switch (NumTexCoords)
							{
							case 4: Vertex->UVs[3] = { TypedSource[6], TypedSource[7] }; // Fall through
							case 3: Vertex->UVs[2] = { TypedSource[4], TypedSource[5] }; // Fall through
							case 2: Vertex->UVs[1] = { TypedSource[2], TypedSource[3] }; // Fall through
							case 1: Vertex->UVs[0] = { TypedSource[0], TypedSource[1] }; // Fall through
							default: break;
							}
						}
						else
						{
							const FVector2f* TypedSource = reinterpret_cast<const FVector2f*>(TexCoord);
							switch (NumTexCoords)
							{
							case 4: Vertex->UVs[3] = TypedSource[3]; // Fall through
							case 3: Vertex->UVs[2] = TypedSource[2]; // Fall through
							case 2: Vertex->UVs[1] = TypedSource[1]; // Fall through
							case 1: Vertex->UVs[0] = TypedSource[0]; // Fall through
							default: break;
							}
						}

						Vertex->Color = bHasColor ? ColorBuffer[JobVertexIndex] : FColor::White;

						const int32 SourceVertexIndex = (JobVertexIndex + Job.VerticesOffset);

						for (int32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; ++InfluenceIndex)
						{
							Vertex->InfluenceBones[InfluenceIndex] = SkinWeightBuffer->GetBoneIndex(SourceVertexIndex, InfluenceIndex);
							Vertex->InfluenceWeights[InfluenceIndex] = SkinWeightBuffer->GetBoneWeight(SourceVertexIndex, InfluenceIndex);
						}
					}
				}
			}
		});
	}

	TArray<UE::Tasks::FTask> CommitMeshDescriptionTasks;

	for (const TTuple<FName, TObjectPtr<USkeletalMesh>>& Tuple : SkeletalMeshes)
	{
		USkeletalMesh* SkeletalMesh = Tuple.Get<1>();

		if (!SkeletalMesh)
		{
			continue;
		}

		const bool bIsTransientMesh = SkeletalMesh->HasAllFlags(EObjectFlags::RF_Transient);

		if (!bIsTransientMesh)
		{
			// This must be a pass-through referenced mesh so don't do anything to it
			continue;
		}

		FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
		if (!RenderData || RenderData->IsInitialized())
		{
			continue;
		}

		const int32 LODCount = SkeletalMesh->GetLODNum();

		TArray<UE::Tasks::FTask> MeshDescriptionGenerationTasks;
		MeshDescriptionGenerationTasks.Reserve(LODCount);

		TStrongObjectPtr<USkeletalMesh> StrongMesh(SkeletalMesh);

		for (int32 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
		{
			if (ensure(!SkeletalMesh->HasMeshDescription(LODIndex)))
			{
				MeshDescriptionGenerationTasks.Add(UE::Tasks::Launch(TEXT("Generate Mesh Description"),
					[StrongMesh, LODIndex]()
					{
						USkeletalMesh* SkeletalMesh = StrongMesh.Get();
						const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];

						FMeshDescription MeshDescription;
						LODModel.GetMeshDescription(SkeletalMesh, LODIndex, MeshDescription);
						SkeletalMesh->CreateMeshDescription(LODIndex, MoveTemp(MeshDescription));
					}));
			}
		}

		CommitMeshDescriptionTasks.Add(UE::Tasks::Launch(TEXT("Commit Mesh Descriptions Task"),
			[StrongMesh]()
			{
				USkeletalMesh* SkeletalMesh = StrongMesh.Get();

				for (int32 LODIndex = 0; LODIndex < SkeletalMesh->GetLODNum(); ++LODIndex)
				{
					SkeletalMesh->CommitMeshDescription(LODIndex);

					// Ensure normals aren't automatically computed when we rebuild.
					FSkeletalMeshLODInfo* MeshLODInfo = SkeletalMesh->GetLODInfo(LODIndex);
					FSkeletalMeshBuildSettings& BuildSettings = MeshLODInfo->BuildSettings;
					BuildSettings.bRecomputeNormals = false;

					// Reset the reduction settings so that we don't re-reduce the mesh and possibly lose morph targets
					// in the process.
					FSkeletalMeshOptimizationSettings& ReductionSettings = MeshLODInfo->ReductionSettings;

					//Remove the reduction settings
					ReductionSettings.NumOfTrianglesPercentage = 1.0f;
					ReductionSettings.NumOfVertPercentage = 1.0f;
					ReductionSettings.MaxNumOfTrianglesPercentage = MAX_uint32;
					ReductionSettings.MaxNumOfVertsPercentage = MAX_uint32;
					ReductionSettings.TerminationCriterion = SMTC_NumOfTriangles;
					MeshLODInfo->bHasBeenSimplified = false;

					const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
					LODModel.BuildStringID = LODModel.GetLODModelDeriveDataKey();

				}
			}, MeshDescriptionGenerationTasks));
	}

	if (OperationData->bBake)
	{
		UE::Tasks::Wait(CommitMeshDescriptionTasks);
	}
}

#endif
