// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectInstanceAssetUserData.h"

#include "MuCO/CustomizableObject.h"

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#endif


const FGameplayTagContainer& UCustomizableObjectInstanceUserData::GetAnimationGameplayTags() const
{
	return AnimationGameplayTag;
}


void UCustomizableObjectInstanceUserData::SetAnimationGameplayTags(const FGameplayTagContainer& InstanceTags)
{
	AnimationGameplayTag = InstanceTags;
}


bool UCustomizableObjectInstanceUserData::Merge(const TNotNull<UAssetUserData*> Other)
{
	UCustomizableObjectInstanceUserData* TypedOther = Cast<UCustomizableObjectInstanceUserData>(Other);
	if (!TypedOther)
	{
		return false;
	}

	AnimationGameplayTag.AppendTags(TypedOther->AnimationGameplayTag);
    
	for (const FCustomizableObjectAnimationSlot& AnimationSlot : TypedOther->AnimationSlots)
	{
		FCustomizableObjectAnimationSlot* FindResult = AnimationSlots.FindByPredicate([&](const FCustomizableObjectAnimationSlot& Slot)
		{
			return Slot.Name == AnimationSlot.Name;
		});
		
		if (!FindResult)
		{
			AnimationSlots.Add(AnimationSlot);
		}
		else if (FindResult->AnimInstance != AnimationSlot.AnimInstance)
		{
			UE_LOGF(LogMutable, Warning, "Unable to merge AnimBP Slots. AnimBP Slot Name [%ls] already exists", *AnimationSlot.Name.ToString());
		}
	}
		
	return true;
}

