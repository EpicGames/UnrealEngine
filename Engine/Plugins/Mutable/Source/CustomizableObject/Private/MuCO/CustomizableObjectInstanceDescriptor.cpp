// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "MuCO/CustomizableObjectInstanceDescriptor.h"
 
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/UnrealMutableImageProvider.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/MutableProjectorTypeUtils.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "Engine/Texture2D.h"
#include "Engine/SkeletalMesh.h"
 
#if WITH_EDITOR
#include "Editor.h"
#endif
 
#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectInstanceDescriptor)
 
 
void CustomizableObjectNullErrorMessage()
{
	UE_LOGF(LogMutable, Error,
		"Tried to perform actions on a CustomizableObjectInstance with no CustomizableObject set. Please set the CustomizableObject of the Instance before doing anything with it.");
}
 
FString GetAvailableOptionsString(const UCustomizableObject& CustomizableObject, const int32 ParameterIndexInObject)
{
	FString OptionsString;
	const int32 NumOptions = CustomizableObject.GetPrivate()->GetEnumParameterNumValues(ParameterIndexInObject);
 
	for (int32 k = 0; k < NumOptions; ++k)
	{
		OptionsString += CustomizableObject.GetPrivate()->GetIntParameterAvailableOption(ParameterIndexInObject, k);
 
		if (k < NumOptions - 1)
		{
			OptionsString += FString(", ");
		}
	}
 
	return OptionsString;
}
 
 
FCustomizableObjectInstanceDescriptor::FCustomizableObjectInstanceDescriptor(UCustomizableObject& Object)
{
	SetCustomizableObject(&Object);
}
 
 
void FCustomizableObjectInstanceDescriptor::SaveDescriptor(FArchive& Ar, bool bUseCompactDescriptor)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	Ar << bUseCompactDescriptor;
 
	// Not sure if this is needed, but it is small.
	Ar << State;
 
	int32 ModelParameterCount = CustomizableObject->GetParameterCount();
 
	if (!bUseCompactDescriptor)
	{
		Ar << ModelParameterCount;
	}
 
	for (int32 ModelParameterIndex = 0; ModelParameterIndex < ModelParameterCount; ++ModelParameterIndex)
	{
		const FString& Name = CustomizableObject->GetParameterName(ModelParameterIndex);
		EMutableParameterType Type = CustomizableObject->GetParameterTypeByName(Name);
 
		if (!bUseCompactDescriptor)
		{
			check(Ar.IsSaving());
			Ar << const_cast<FString &>(Name);
			Ar << Type;
		}
 
		switch (Type)
		{
		case EMutableParameterType::Bool:
		{
			bool Value = false;
			for (const FCustomizableObjectBoolParameterValue& P: BoolParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue;
					break;
				}
			}
			Ar << Value;
			break;
		}
 
		case EMutableParameterType::Float:
		{
			float Value = 0.0f;
			TArray<float> RangeValues;
			for (const FCustomizableObjectFloatParameterValue& P : FloatParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue;
					RangeValues = P.ParameterRangeValues;
					break;
				}
			}
			Ar << Value;
			Ar << RangeValues;
			break;
		}
 
		case EMutableParameterType::Int:
		{
			int32 Value = 0;
			FString ValueName;
 
			TArray<int32> Values;
			TArray<FString> ValueNames;
 
			bool bIsParamMultidimensional = false;
 
			const int32 IntParameterIndex = FindTypedParameterIndex(Name, EMutableParameterType::Int);
			if (IntParameterIndex >= 0)
			{					
				const FCustomizableObjectIntParameterValue & P = IntParameters[IntParameterIndex];
				Value = CustomizableObject->GetPrivate()->FindIntParameterValue(ModelParameterIndex, P.ParameterValueName);
 
				int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(IntParameters[IntParameterIndex].ParameterName);
				bIsParamMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
 
				if (bIsParamMultidimensional)
				{
					for (int32 i = 0; i < P.ParameterRangeValueNames.Num(); ++i)
					{
						ValueNames.Add(P.ParameterRangeValueNames[i]);
						Values.Add(CustomizableObject->GetPrivate()->FindIntParameterValue(ModelParameterIndex, P.ParameterRangeValueNames[i]));
					}
				}
 
				if (!bUseCompactDescriptor)
				{
					ValueName = P.ParameterValueName;
				}
			}
 
			if (bUseCompactDescriptor)
			{
				Ar << Value;
 
				if (bIsParamMultidimensional)
				{
					Ar << Values;
				}
			}
			else
			{
				Ar << ValueName;
				Ar << bIsParamMultidimensional;
				
				if (bIsParamMultidimensional)
				{
					Ar << ValueNames;
				}
			}
 
			break;
		}
 
		case EMutableParameterType::Color:
		{
			FLinearColor Value(FLinearColor::Black);
			for (const FCustomizableObjectVectorParameterValue& P : VectorParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue;
					break;
				}
			}
			Ar << Value;
 
			break;
		}
 
		case EMutableParameterType::Transform:
		{
			FTransform Value(FTransform::Identity);
			for (const FCustomizableObjectTransformParameterValue& P : TransformParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue;
					break;
				}
			}
			Ar << Value;
 
			break;
		}
 
		case EMutableParameterType::Texture:
		{
			FString Value;
			TArray<FString> RangeValues;
 
			for (const FCustomizableObjectTextureParameterValue& P : TextureParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue.GetPathName();
 
					RangeValues.Empty(P.ParameterRangeValues.Num());	
					Algo::Transform(P.ParameterRangeValues, RangeValues, [](const TObjectPtr<UTexture>& Element)
					{
						return Element.GetPathName();
					});
 
					break;
				}
			}
			Ar << Value;
			Ar << RangeValues;
 
			break;
		}
 
		case EMutableParameterType::SkeletalMesh:
		{
			FString Value;
			TArray<FString> RangeValues;
 
			for (const FCustomizableObjectSkeletalMeshParameterValue& P : SkeletalMeshParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue.GetPathName();
					
					RangeValues.Empty(P.ParameterRangeValues.Num());	
					Algo::Transform(P.ParameterRangeValues, RangeValues, [](const TObjectPtr<USkeletalMesh>& Element)
					{
						return Element.GetPathName();
					});
					
					break;
				}
			}
 
			Ar << Value;
			Ar << RangeValues;
				
			break;
		}
 
		case EMutableParameterType::Material:
		{
			FString Value;
			TArray<FString> RangeValues;

			for (const FCustomizableObjectMaterialParameterValue& P : MaterialParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.ParameterValue.GetPathName();
				
					RangeValues.Empty(P.ParameterRangeValues.Num());	
					Algo::Transform(P.ParameterRangeValues, RangeValues, [](const TObjectPtr<UMaterialInterface>& Element)
					{
						return Element.GetPathName();
					});
				
					break;
				}
			}

			Ar << Value;
			Ar << RangeValues;
			
			break;
		}
			
		case EMutableParameterType::Projector:
		{
			FCustomizableObjectProjector Value;
			TArray<FCustomizableObjectProjector> RangeValues;
 
			for (const FCustomizableObjectProjectorParameterValue& P : ProjectorParameters)
			{
				if (P.ParameterName == Name)
				{
					Value = P.Value;
					RangeValues = P.RangeValues;
					break;
				}
			}
			Ar << Value;
			Ar << RangeValues;
 
			break;
		}
			
		case EMutableParameterType::None:
		{
			UE_LOGF(LogMutable, Warning, "Found unknown parameter [%ls] when saving a Customizable Object Instance Descriptor", *Name);
			break;
		}
 
		default:
			// Parameter type replication not implemented.
			check(false);
		}
	}
}
 
 
void FCustomizableObjectInstanceDescriptor::LoadDescriptor(FArchive& Ar)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	bool bUseCompactDescriptor;
	Ar << bUseCompactDescriptor;
 
	// Not sure if this is needed, but it is small.
	Ar << State;
	
	int32 ModelParameterCount = CustomizableObject->GetParameterCount();
 
	if (!bUseCompactDescriptor)
	{
		Ar << ModelParameterCount;
	}
 
	for (int32 ParameterIndex = 0; ParameterIndex < ModelParameterCount; ++ParameterIndex)
	{
		FString Name;
		EMutableParameterType Type;
		int32 ModelParameterIndex = -1;
 
		if (bUseCompactDescriptor)
		{
			ModelParameterIndex = ParameterIndex;
			Name = CustomizableObject->GetParameterName(ModelParameterIndex);
			Type = CustomizableObject->GetPrivate()->GetParameterType(ModelParameterIndex);
		}
		else
		{
			Ar << Name;
			Ar << Type;
		}
 
		switch (Type)
		{
		case EMutableParameterType::Bool:
		{
			bool Value = false;
			Ar << Value;
			for (FCustomizableObjectBoolParameterValue& P : BoolParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = Value;
					break;
				}
			}
			break;
		}
 
		case EMutableParameterType::Float:
		{
			float Value = 0.0f;
			TArray<float> RangeValues;
			Ar << Value;
			Ar << RangeValues;
			for (FCustomizableObjectFloatParameterValue& P : FloatParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = Value;
					P.ParameterRangeValues = RangeValues;
					break;
				}
			}
			break;
		}
 
		case EMutableParameterType::Int:
		{
			int32 Value = 0;
			FString ValueName;
 
			TArray<int32> Values;
			TArray<FString> ValueNames;
 
 
			if (bUseCompactDescriptor)
			{
				const int32 IntParameterIndex = FindTypedParameterIndex(Name, EMutableParameterType::Int);
				const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(IntParameters[IntParameterIndex].ParameterName);
				const bool bIsParamMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);

				Ar << Value;
 
				if (bIsParamMultidimensional)
				{
					Ar << Values;
				}
			}
			else
			{
				bool bIsParamMultidimensional;

				Ar << ValueName;
				Ar << bIsParamMultidimensional;
				
				if (bIsParamMultidimensional)
				{
					Ar << ValueNames;
				}
			}
 
			for (FCustomizableObjectIntParameterValue& P : IntParameters)
			{
				if (P.ParameterName == Name)
				{
					if (bUseCompactDescriptor)
					{
						P.ParameterValueName = CustomizableObject->GetPrivate()->FindIntParameterValueName(ModelParameterIndex, Value);
						//check((P.ParameterValueName.IsEmpty() && ValueName.Equals(FString("None"))) || P.ParameterValueName.Equals(ValueName));
						P.ParameterRangeValueNames.SetNum(Values.Num());
 
						for (int32 ParamIndex = 0; ParamIndex < Values.Num(); ++ParamIndex)
						{
							P.ParameterRangeValueNames[ParamIndex] = CustomizableObject->GetPrivate()->FindIntParameterValueName(ModelParameterIndex, Values[ParamIndex]);
						}
					}
					else
					{
						P.ParameterValueName = ValueName;
						P.ParameterRangeValueNames = ValueNames;
					}
 
					break;
				}
			}
 
			break;
		}
 
		case EMutableParameterType::Color:
		{
			FLinearColor Value(FLinearColor::Black);
			Ar << Value;
			for (FCustomizableObjectVectorParameterValue& P : VectorParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = Value;
					break;
				}
			}
 
			break;
		}
 
		case EMutableParameterType::Transform:
		{
			FTransform Value(FTransform::Identity);
			Ar << Value;
			for (FCustomizableObjectTransformParameterValue& P : TransformParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = Value;
					break;
				}
			}
 
			break;
		}
 
		case EMutableParameterType::Texture:
		{
			FString Value;
			TArray<FString> RangeValues;
			Ar << Value;
			Ar << RangeValues;
 
			for (FCustomizableObjectTextureParameterValue& P : TextureParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = ToObject<UTexture>(Value);
 
					P.ParameterRangeValues.Empty(RangeValues.Num());
					Algo::Transform(RangeValues, P.ParameterRangeValues, [](const FString& Element)
					{
						return ToObject<UTexture>(Element);
					});
					
					break;
				}
			}
 
			break;
		}
 
		case EMutableParameterType::SkeletalMesh:
		{
			FString Value;
			TArray<FString> RangeValues;
			Ar << Value;
			Ar << RangeValues;
 
			for (FCustomizableObjectSkeletalMeshParameterValue& P : SkeletalMeshParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = ToObject<USkeletalMesh>(Value);
 
					P.ParameterRangeValues.Empty(RangeValues.Num());
					Algo::Transform( RangeValues, P.ParameterRangeValues, [](const FString& Element)
					{
						return ToObject<USkeletalMesh>(Element);
					});
					
					break;
				}
			}
 
			break;
		}
 
		case EMutableParameterType::Projector:
		{
			FCustomizableObjectProjector Value;
			TArray<FCustomizableObjectProjector> RangeValues;
			Ar << Value;
			Ar << RangeValues;
 
			for (FCustomizableObjectProjectorParameterValue& P : ProjectorParameters)
			{
				if (P.ParameterName == Name)
				{
					P.Value = Value;
					P.RangeValues = RangeValues;
					break;
				}
			}
 
			break;
		}
			
		case EMutableParameterType::Material:
		{
			FString Value;
			TArray<FString> RangeValues;
			Ar << Value;
			Ar << RangeValues;
			for (FCustomizableObjectMaterialParameterValue& P : MaterialParameters)
			{
				if (P.ParameterName == Name)
				{
					P.ParameterValue = ToObject<UMaterialInterface>(Value);
					
					P.ParameterRangeValues.Empty(RangeValues.Num());
					Algo::Transform(RangeValues, P.ParameterRangeValues, [](const FString& Element)
					{
						return ToObject<UMaterialInterface>(Element);
					});
		
					break;
				}
			}
				
			break;
		}
			
		case EMutableParameterType::None:
		{
			UE_LOGF(LogMutable, Warning, "Found unknown parameter [%ls] when loading a Customizable Object Instance Descriptor", *Name);
			break;
		}
 
		default:
			// Parameter type replication not implemented.
			check(false);
		}
	}
}
 
 
UCustomizableObject* FCustomizableObjectInstanceDescriptor::GetCustomizableObject() const
{
	return CustomizableObject;
}
 
 
void FCustomizableObjectInstanceDescriptor::SetCustomizableObject(UCustomizableObject* InCustomizableObject)
{
	CustomizableObject = InCustomizableObject;
	ReloadParameters();
}
 
 
bool FCustomizableObjectInstanceDescriptor::GetBuildParameterRelevancy() const
{
#if WITH_EDITOR
	// In editor, calculate the parameter relevancy by default.
	bool bResultBuildRelevancy = true;
	// However if we are in a PIE session, do it only if requested to simulate a more game-like performance.
	if (GIsEditor)
	{
		FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
		if (PIEWorldContext)
		{
			bResultBuildRelevancy = bBuildParameterRelevancy;
		}
	}
	return bResultBuildRelevancy;
#else
	return bBuildParameterRelevancy;
#endif
}
 
 
void FCustomizableObjectInstanceDescriptor::SetBuildParameterRelevancy(const bool Value)
{
	bBuildParameterRelevancy = Value;
}
 
 
TSharedPtr<UE::Mutable::Private::FParameters> FCustomizableObjectInstanceDescriptor::GetParameters() const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return nullptr;
	}
 
	if (!CustomizableObject->IsCompiled())
	{	
		return nullptr;
	}
	
	if (!CustomizableObject->GetPrivate()->GetModel())
	{
		return nullptr;
	}
 
	const UModelResources& ModelResources = CustomizableObject->GetPrivate()->GetModelResourcesChecked();
	
	TSharedPtr<UE::Mutable::Private::FParameters> MutableParameters = UE::Mutable::Private::FModel::NewParameters(CustomizableObject->GetPrivate()->GetModel(), nullptr);
 
	const int32 ParamCount = MutableParameters->GetCount();
	for (int32 ParamIndex = 0; ParamIndex < ParamCount; ++ParamIndex)
	{
		const FString& Name = MutableParameters->GetName(ParamIndex);
		const FGuid& Uid = MutableParameters->GetUid(ParamIndex);
		const UE::Mutable::Private::EParameterType MutableType = MutableParameters->GetType(ParamIndex);
 
		switch (MutableType)
		{
		case UE::Mutable::Private::EParameterType::Bool:
		{
			for (const FCustomizableObjectBoolParameterValue& BoolParameter : BoolParameters)
			{
				if (BoolParameter.ParameterName == Name || (Uid.IsValid() && BoolParameter.Id == Uid))
				{
					MutableParameters->SetBoolValue(ParamIndex,  BoolParameter.ParameterValue);
					break;
				}
			}
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Int:
		{
			for (const FCustomizableObjectIntParameterValue& IntParameter : IntParameters)
			{
				if (IntParameter.ParameterName.Equals(Name, ESearchCase::CaseSensitive) || (Uid.IsValid() && IntParameter.Id == Uid))
				{
					if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < IntParameter.ParameterRangeValueNames.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
 
							const FString& RangeValueName = IntParameter.ParameterRangeValueNames[RangeIndex];
							const int32 Value = CustomizableObject->GetPrivate()->FindIntParameterValue(ParamIndex, RangeValueName);
							MutableParameters->SetIntValue(ParamIndex, Value, RangeIdxPtr.Get());
						}
					}
					else
					{
						const int32 Value = CustomizableObject->GetPrivate()->FindIntParameterValue(ParamIndex, IntParameter.ParameterValueName);
						MutableParameters->SetIntValue(ParamIndex, Value);
					}
 
					break;
				}
			}
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Float:
		{
			for (const FCustomizableObjectFloatParameterValue& FloatParameter : FloatParameters)
			{
				if (FloatParameter.ParameterName == Name || (Uid.IsValid() && FloatParameter.Id == Uid))
				{
					if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < FloatParameter.ParameterRangeValues.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
							MutableParameters->SetFloatValue(ParamIndex, FloatParameter.ParameterRangeValues[RangeIndex], RangeIdxPtr.Get());
						}
					}
					else
					{
						MutableParameters->SetFloatValue(ParamIndex, FloatParameter.ParameterValue);
					}
 
					break;
				}
			}
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Color:
		{
			for (const FCustomizableObjectVectorParameterValue& VectorParameter : VectorParameters)
			{
				if (VectorParameter.ParameterName == Name || (Uid.IsValid() && VectorParameter.Id == Uid))
				{
					MutableParameters->SetColorValue(ParamIndex, VectorParameter.ParameterValue);
 
					break;
				}
			}
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Matrix:
			{
				for (const FCustomizableObjectTransformParameterValue& TransformParameter : TransformParameters)
				{
					if (TransformParameter.ParameterName == Name || (Uid.IsValid() && TransformParameter.Id == Uid))
					{
						MutableParameters->SetMatrixValue(ParamIndex, FMatrix44f(TransformParameter.ParameterValue.ToMatrixWithScale()));
 
						break;
					}
				}
 
				break;
			}
			
		case UE::Mutable::Private::EParameterType::Projector:
		{
			for (const auto& ProjectorParameter : ProjectorParameters)
			{
				if (ProjectorParameter.ParameterName == Name || (Uid.IsValid() && ProjectorParameter.Id == Uid))
				{
					auto CopyProjector = [&MutableParameters, ParamIndex](const FCustomizableObjectProjector& Value, const UE::Mutable::Private::FRangeIndex* RangeIdxPtr = nullptr)
					{
						switch (Value.ProjectionType)
						{
						case ECustomizableObjectProjectorType::Planar:
						case ECustomizableObjectProjectorType::Wrapping:
							MutableParameters->SetProjectorValue(ParamIndex,
								Value.Position,
								Value.Direction,
								Value.Up,
								Value.Scale,
								Value.Angle,
								RangeIdxPtr);
							break;
 
						case ECustomizableObjectProjectorType::Cylindrical:
						{
							// Apply strange swizzle for scales
							// TODO: try to avoid this
							const float Radius = FMath::Abs(Value.Scale[0] / 2.0f);
							const float Height = Value.Scale[2];
							// TODO: try to avoid this
							MutableParameters->SetProjectorValue(ParamIndex,
									Value.Position,
									-Value.Direction,
									-Value.Up,
									FVector3f(-Height, Radius, Radius),
									Value.Angle,
									RangeIdxPtr);
							break;
						}
 
						default:
							check(false); // Not implemented.
						}
					};
 
					CopyProjector(ProjectorParameter.Value);
 
					if (const TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < ProjectorParameter.RangeValues.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
							CopyProjector(ProjectorParameter.RangeValues[RangeIndex], RangeIdxPtr.Get());
						}
					}
				}
			}
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Texture:
		{
			for (const FCustomizableObjectTextureParameterValue& TextureParameter : TextureParameters)
			{
				if (TextureParameter.ParameterName == Name || (Uid.IsValid() && TextureParameter.Id == Uid))
				{
					if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < TextureParameter.ParameterRangeValues.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
							MutableParameters->SetTextureValue(ParamIndex, TextureParameter.ParameterRangeValues[RangeIndex], RangeIdxPtr.Get());
						}
					}
					else
					{
						MutableParameters->SetTextureValue(ParamIndex, TextureParameter.ParameterValue);
					}
 
					break;
				}
			}
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::SkeletalMesh:
		{
			for (const FCustomizableObjectSkeletalMeshParameterValue& MeshParameter : SkeletalMeshParameters)
			{
				if (MeshParameter.ParameterName == Name || (Uid.IsValid() && MeshParameter.Id == Uid))
				{
					if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < MeshParameter.ParameterRangeValues.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
							MutableParameters->SetSkeletalMeshValue(ParamIndex, MeshParameter.ParameterRangeValues[RangeIndex], RangeIdxPtr.Get());
						}
					}
					else
					{
						MutableParameters->SetSkeletalMeshValue(ParamIndex, MeshParameter.ParameterValue);
					}
 
					break;
				}
			}
 
			break;
		}

		case UE::Mutable::Private::EParameterType::Material:
		{
			for (const FCustomizableObjectMaterialParameterValue& MaterialParameter : MaterialParameters)
			{
				if (MaterialParameter.ParameterName == Name || (Uid.IsValid() && MaterialParameter.Id == Uid))
				{
					if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < MaterialParameter.ParameterRangeValues.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
							MutableParameters->SetMaterialValue(ParamIndex, MaterialParameter.ParameterRangeValues[RangeIndex], RangeIdxPtr.Get());
						}
					}
					else
					{
						MutableParameters->SetMaterialValue(ParamIndex, MaterialParameter.ParameterValue);
					}

					break;
				}
			}

			break;
		}

		case UE::Mutable::Private::EParameterType::InstancedStruct:
		{
			for (const FCustomizableObjectExternalTypeParameterValue& ExternalTypeParameter : ExternalTypeParameters)
			{
				if (ExternalTypeParameter.ParameterName == Name || (Uid.IsValid() && ExternalTypeParameter.Id == Uid))
				{
					if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
					{
						for (int32 RangeIndex = 0; RangeIndex < ExternalTypeParameter.ParameterRangeValues.Num(); ++RangeIndex)
						{
							RangeIdxPtr->SetPosition(0, RangeIndex);
							MutableParameters->SetInstancedStructValue(ParamIndex, UE::Mutable::Private::MakeManaged<const FInstancedStruct>(ExternalTypeParameter.ParameterRangeValues[RangeIndex]), RangeIdxPtr.Get());
						}
					}
					else
					{
						MutableParameters->SetInstancedStructValue(ParamIndex, UE::Mutable::Private::MakeManaged<const FInstancedStruct>(ExternalTypeParameter.ParameterValue));
					}

					break;
				}
			}

			break;
		}
 
		default:
			check(false); // Missing case.
			break;
		}
	}
	
	return MutableParameters;
}
 
 
FString FCustomizableObjectInstanceDescriptor::ToString() const
{
	const UScriptStruct* Struct = StaticStruct();
	FString ExportedText;
 
	Struct->ExportText(ExportedText, this, nullptr, nullptr, (PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited | PPF_IncludeTransient), nullptr);
 
	return ExportedText;
}
 
 
void FCustomizableObjectInstanceDescriptor::ReloadParameters()
{
	if (IsRunningCookCommandlet())
	{
		return;
	}
 
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	if (!CustomizableObject->IsCompiled())
	{
		return;
	}
 
	SetState(FMath::Clamp(GetState(), 0, CustomizableObject->GetStateCount() - 1));
 
	TArray<FCustomizableObjectBoolParameterValue> OldBoolParameters = BoolParameters;
	TArray<FCustomizableObjectIntParameterValue> OldIntParameters = IntParameters;
	TArray<FCustomizableObjectFloatParameterValue> OldFloatParameters = FloatParameters;
	TArray<FCustomizableObjectTextureParameterValue> OldTextureParameters = TextureParameters;
	TArray<FCustomizableObjectSkeletalMeshParameterValue> OldSkeletalMeshParameters = SkeletalMeshParameters;
	TArray<FCustomizableObjectMaterialParameterValue> OldMaterialParameters = MaterialParameters;
	TArray<FCustomizableObjectExternalTypeParameterValue> OldExternalTypeParameters = ExternalTypeParameters;
	TArray<FCustomizableObjectVectorParameterValue> OldVectorParameters = VectorParameters;
	TArray<FCustomizableObjectProjectorParameterValue> OldProjectorParameters = ProjectorParameters;
	TArray<FCustomizableObjectTransformParameterValue> OldTransformParameters = TransformParameters;
 
	BoolParameters.Reset();
	IntParameters.Reset();
	FloatParameters.Reset();
	TextureParameters.Reset();
	SkeletalMeshParameters.Reset();
	MaterialParameters.Reset();
	ExternalTypeParameters.Reset();
	VectorParameters.Reset();
	ProjectorParameters.Reset();
	TransformParameters.Reset();
	
	if (!CustomizableObject->GetPrivate()->GetModel())
	{
		UE_LOGF(LogMutable, Warning, "[ReloadParametersFromObject] No model in object [%ls], generated empty parameters for [%ls] ", *CustomizableObject->GetName(), *CustomizableObject->GetName());
		return;
	}

	const UModelResources* ModelResources = CustomizableObject->GetPrivate()->GetModelResources();
	
	if(!CustomizableObject->GetPrivate()->GetModelResources())
	{
		UE_LOGF(LogMutable, Warning, "[ReloadParametersFromObject] No model resources in object [%ls], generated empty parameters for [%ls] ", *CustomizableObject->GetName(), *CustomizableObject->GetName());
		return;
	}
	
	TSharedPtr<UE::Mutable::Private::FParameters> MutableParameters = UE::Mutable::Private::FModel::NewParameters(CustomizableObject->GetPrivate()->GetModel(), nullptr);
 
	int32 ParamCount = MutableParameters->GetCount();
	for (int32 ParamIndex = 0; ParamIndex < ParamCount; ++ParamIndex)
	{
		const FString& Name = MutableParameters->GetName(ParamIndex);
		const FGuid& Uid = MutableParameters->GetUid(ParamIndex);
		const UE::Mutable::Private::EParameterType MutableType = MutableParameters->GetType(ParamIndex);
 
		switch (MutableType)
		{
		case UE::Mutable::Private::EParameterType::Bool:
		{
			FCustomizableObjectBoolParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			auto FindByNameAndUid = [&](const FCustomizableObjectBoolParameterValue& P)
			{
				return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
			};
 
			if (FCustomizableObjectBoolParameterValue* Result = OldBoolParameters.FindByPredicate(FindByNameAndUid))
			{	
				Param.ParameterValue = Result->ParameterValue;
			} 
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				Param.ParameterValue = MutableParameters->GetBoolValue(ParamIndex);
			}
 
			BoolParameters.Add(Param);
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Int:
		{
			FCustomizableObjectIntParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			auto FindByNameAndUid = [&](const FCustomizableObjectIntParameterValue& P)
			{
				return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
			};
			
			if (FCustomizableObjectIntParameterValue* Result = OldIntParameters.FindByPredicate(FindByNameAndUid))
			{
				const int32 NumValueIndex = MutableParameters->GetIntPossibleValueCount(ParamIndex);
 
				auto ValueExists = [&](const FString& ValueName)
				{
					for (int32 ValueIndex = 0; ValueIndex < NumValueIndex; ++ValueIndex)
					{
						if (ValueName == MutableParameters->GetIntPossibleValueName(ParamIndex, ValueIndex))
						{
							return true;
						}
					}
 
					return false;
				};
				
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex)) // Is multidimensional
				{
					// Get num of ranges (layers) from the instance
					int32 ValueCount = Result->ParameterRangeValueNames.Num();
					Param.ParameterRangeValueNames.Reserve(ValueCount);
 
					for (int32 RangeIndex = 0; RangeIndex < ValueCount; ++RangeIndex)
					{
						// Checking if the selected value still exists as option in the parameter
						if (const FString& OldValue = Result->ParameterRangeValueNames[RangeIndex]; ValueExists(OldValue))
						{
							Param.ParameterRangeValueNames.Add(OldValue);
						}
						else
						{
							const int32 Value = MutableParameters->GetIntValue(ParamIndex, RangeIdxPtr.Get());
							const FString AuxParameterValueName = CustomizableObject->GetPrivate()->FindIntParameterValueName(ParamIndex, Value);
							Param.ParameterRangeValueNames.Add(AuxParameterValueName);
						}
					}
				}
				else
				{
					if (ValueExists(Result->ParameterValueName))
					{
						Param.ParameterValueName = Result->ParameterValueName;
					}
					else
					{
						const int32 ParamValue = MutableParameters->GetIntValue(ParamIndex);
						Param.ParameterValueName = CustomizableObject->GetPrivate()->FindIntParameterValueName(ParamIndex, ParamValue);
					}
 
					// Multilayer ints with one option are not multidimensional parameters. However, we need to preserve the layer 
					// information in case that we add a new option to the parameter, and it is converted to multidimensional.
					for (int32 RangeIndex = 0; RangeIndex < Result->ParameterRangeValueNames.Num(); ++RangeIndex)
					{
						const int32 Value = MutableParameters->GetIntValue(ParamIndex);
						const FString AuxParameterValueName = CustomizableObject->GetPrivate()->FindIntParameterValueName(ParamIndex, Value);
						Param.ParameterRangeValueNames.Add(AuxParameterValueName);
					}
				}
			} 
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);
 
					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{	
						const TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						const int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);
 
						if (!Param.ParameterRangeValueNames.IsValidIndex(RangeIndex))
						{
							Param.ParameterRangeValueNames.AddDefaulted(RangeIndex + 1 - Param.ParameterRangeValueNames.Num());
						}
 
						const int32 Value = MutableParameters->GetIntValue(ParamIndex, RangeValueIdxPtr.Get());
						const FString AuxParameterValueName = CustomizableObject->GetPrivate()->FindIntParameterValueName(ParamIndex, Value);
						Param.ParameterRangeValueNames[RangeIndex] = AuxParameterValueName;
					}
				}
				else
				{
					const int32 ParamValue = MutableParameters->GetIntValue(ParamIndex);
					Param.ParameterValueName = CustomizableObject->GetPrivate()->FindIntParameterValueName(ParamIndex, ParamValue);
				}
			}
 
			IntParameters.Add(Param);
 
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Float:
		{
			FCustomizableObjectFloatParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			auto FindByNameAndUid = [&](const FCustomizableObjectFloatParameterValue& P)
			{
				return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
			};
			
			if (FCustomizableObjectFloatParameterValue* Result = OldFloatParameters.FindByPredicate(FindByNameAndUid))
			{	
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					Param.ParameterRangeValues = Result->ParameterRangeValues;
				}
				else
				{
					Param.ParameterValue = Result->ParameterValue;
				}
			} 
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);
 
					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{	
						TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);
 
						if (!Param.ParameterRangeValues.IsValidIndex(RangeIndex))
						{
							Param.ParameterRangeValues.AddDefaulted(RangeIndex + 1 - Param.ParameterRangeValues.Num());
						}
 
						Param.ParameterRangeValues[RangeIndex] = MutableParameters->GetFloatValue(ParamIndex, RangeValueIdxPtr.Get());
					}
				}
				else
				{
					Param.ParameterValue = MutableParameters->GetFloatValue(ParamIndex);
				}
			}
 
			FloatParameters.Add(Param);
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Color:
		{
			FCustomizableObjectVectorParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			auto FindByNameAndUid = [&](const FCustomizableObjectVectorParameterValue& P)
			{
				return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
			};
 
			if (FCustomizableObjectVectorParameterValue* Result = OldVectorParameters.FindByPredicate(FindByNameAndUid))
			{	
				Param.ParameterValue = Result->ParameterValue;
			}
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				FVector4f V;
				MutableParameters->GetColorValue(ParamIndex, V);
				Param.ParameterValue = V;
			}
 
			VectorParameters.Add(Param);
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Matrix:
			{
				FCustomizableObjectTransformParameterValue Param;
				Param.ParameterName = Name;
				Param.Id = Uid;
 
				auto FindByNameAndUid = [&](const FCustomizableObjectTransformParameterValue& P)
				{
					return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
				};
 
				if (FCustomizableObjectTransformParameterValue* Result = OldTransformParameters.FindByPredicate(FindByNameAndUid))
				{	
					Param.ParameterValue = Result->ParameterValue;
				}
				else // Not found in Instance Parameters. Use Mutable Parameters.
				{
					FMatrix44f Matrix;
					MutableParameters->GetMatrixValue(ParamIndex, Matrix);
					Param.ParameterValue = FTransform(FMatrix(Matrix));
				}
 
				TransformParameters.Add(Param);
				break;
			}
			
		case UE::Mutable::Private::EParameterType::Projector:
		{
			FCustomizableObjectProjectorParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			// Projector to check if the porojector's type has changed
			FCustomizableObjectProjector DefaultProjectorValue = CustomizableObject->GetProjectorParameterDefaultValue(Name);
 
			auto FindByNameAndUid = [&](const FCustomizableObjectProjectorParameterValue& P)
			{
				return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
			};
			
			if (FCustomizableObjectProjectorParameterValue* Result = OldProjectorParameters.FindByPredicate(FindByNameAndUid))
			{	
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					Param.RangeValues = Result->RangeValues;
					Param.Value.ProjectionType = DefaultProjectorValue.ProjectionType;
 
					for (FCustomizableObjectProjector& Projector : Param.RangeValues)
					{
						Projector.ProjectionType = DefaultProjectorValue.ProjectionType;
					}
				}
				else
				{
					Param.Value = Result->Value;
					Param.Value.ProjectionType = DefaultProjectorValue.ProjectionType;
				}
			} 
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				auto GetProjector = [&MutableParameters, ParamIndex](FCustomizableObjectProjector &Value, const TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIndex = nullptr)
				{
					UE::Mutable::Private::EProjectorType Type;
					MutableParameters->GetProjectorValue(ParamIndex,
						&Type,
						&Value.Position,
						&Value.Direction,
						&Value.Up,
						&Value.Scale,
						&Value.Angle,
						RangeIndex.Get());
					
					Value.ProjectionType = ProjectorUtils::GetEquivalentProjectorType(Type);
					if (Value.ProjectionType == ECustomizableObjectProjectorType::Cylindrical)
					{
						// Unapply strange swizzle for scales.
						// TODO: try to avoid this
						Value.Direction = -Value.Direction;
						Value.Up = -Value.Up;
						Value.Scale[2] = -Value.Scale[0];
						Value.Scale[0] = Value.Scale[1] = Value.Scale[1] * 2.0f;
					}
				};
 
				GetProjector(Param.Value);
 
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);
 
					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{	
						TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);
 
						if (!Param.RangeValues.IsValidIndex(RangeIndex))
						{
							Param.RangeValues.AddDefaulted(RangeIndex + 1 - Param.RangeValues.Num());
						}
 
						GetProjector(Param.RangeValues[RangeIndex], RangeValueIdxPtr);
					}
				}
			}
 
			ProjectorParameters.Add(Param);
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Texture:
		{
			FCustomizableObjectTextureParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			auto FindByNameAndUid = [&](const FCustomizableObjectTextureParameterValue& P)
			{
				return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
			};
			
			if (FCustomizableObjectTextureParameterValue* Result = OldTextureParameters.FindByPredicate(FindByNameAndUid))
			{	
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					Param.ParameterRangeValues = Result->ParameterRangeValues;
				}
				else
				{
					Param.ParameterValue = Result->ParameterValue;
				}
			} 
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);
 
					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{	
						TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);
 
						if (!Param.ParameterRangeValues.IsValidIndex(RangeIndex))
						{
							const int32 PreviousNum = Param.ParameterRangeValues.Num();
							Param.ParameterRangeValues.AddUninitialized(RangeIndex + 1 - Param.ParameterRangeValues.Num());
 
							for (int32 Index = PreviousNum; Index < Param.ParameterRangeValues.Num(); ++Index)
							{
								Param.ParameterRangeValues[Index] = nullptr;
							}
						}
 
						Param.ParameterRangeValues[RangeIndex] = MutableParameters->GetTextureValue(ParamIndex, RangeValueIdxPtr.Get());
					}
				}
				else
				{
					Param.ParameterValue = MutableParameters->GetTextureValue(ParamIndex);
				}
			}
 
			TextureParameters.Add(Param);
			break;
		}
 
 
		case UE::Mutable::Private::EParameterType::SkeletalMesh:
		{
			FCustomizableObjectSkeletalMeshParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;
 
			auto FindByNameAndUid = [&](const FCustomizableObjectSkeletalMeshParameterValue& P)
				{
					return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
				};
 
			if (FCustomizableObjectSkeletalMeshParameterValue* Result = OldSkeletalMeshParameters.FindByPredicate(FindByNameAndUid))
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					Param.ParameterRangeValues = Result->ParameterRangeValues;
				}
				else
				{
					Param.ParameterValue = Result->ParameterValue;
				}
			}
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);
 
					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{
						TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);
 
						if (!Param.ParameterRangeValues.IsValidIndex(RangeIndex))
						{
							const int32 PreviousNum = Param.ParameterRangeValues.Num();
							Param.ParameterRangeValues.AddUninitialized(RangeIndex + 1 - Param.ParameterRangeValues.Num());
 
							for (int32 Index = PreviousNum; Index < Param.ParameterRangeValues.Num(); ++Index)
							{
								Param.ParameterRangeValues[Index] = nullptr;
							}
						}
 
						Param.ParameterRangeValues[RangeIndex] = MutableParameters->GetSkeletalMeshValue(ParamIndex, RangeValueIdxPtr.Get());
					}
				}
				else
				{
					Param.ParameterValue = MutableParameters->GetSkeletalMeshValue(ParamIndex);
				}
			}
 
			SkeletalMeshParameters.Add(Param);
			break;
		}
 
		case UE::Mutable::Private::EParameterType::Material:
		{
			FCustomizableObjectMaterialParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;

			auto FindByNameAndUid = [&](const FCustomizableObjectMaterialParameterValue& P)
				{
					return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
				};

			if (FCustomizableObjectMaterialParameterValue* Result = OldMaterialParameters.FindByPredicate(FindByNameAndUid))
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					Param.ParameterRangeValues = Result->ParameterRangeValues;
				}
				else
				{
					Param.ParameterValue = Result->ParameterValue;
				}
			}
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);

					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{
						TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);

						if (!Param.ParameterRangeValues.IsValidIndex(RangeIndex))
						{
							const int32 PreviousNum = Param.ParameterRangeValues.Num();
							Param.ParameterRangeValues.AddUninitialized(RangeIndex + 1 - Param.ParameterRangeValues.Num());

							for (int32 Index = PreviousNum; Index < Param.ParameterRangeValues.Num(); ++Index)
							{
								Param.ParameterRangeValues[Index] = nullptr;
							}
						}

						Param.ParameterRangeValues[RangeIndex] = MutableParameters->GetMaterialValue(ParamIndex, RangeValueIdxPtr.Get());
					}
				}
				else
				{
					Param.ParameterValue = MutableParameters->GetMaterialValue(ParamIndex);
				}
			}

			MaterialParameters.Add(Param);

			break;
		}

		case UE::Mutable::Private::EParameterType::InstancedStruct:
		{
			FCustomizableObjectExternalTypeParameterValue Param;
			Param.ParameterName = Name;
			Param.Id = Uid;

			auto FindByNameAndUid = [&](const FCustomizableObjectExternalTypeParameterValue& P)
				{
					return P.ParameterName == Name || (Uid.IsValid() && P.Id == Uid);
				};

			if (FCustomizableObjectExternalTypeParameterValue* Result = OldExternalTypeParameters.FindByPredicate(FindByNameAndUid))
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					Param.ParameterRangeValues = Result->ParameterRangeValues;
				}
				else
				{
					Param.ParameterValue = Result->ParameterValue;
				}
			}
			else // Not found in Instance Parameters. Use Mutable Parameters.
			{
				if (TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeIdxPtr = MutableParameters->NewRangeIndex(ParamIndex))
				{
					const int32 ValueCount = MutableParameters->GetValueCount(ParamIndex);

					for (int32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
					{
						TSharedPtr<UE::Mutable::Private::FRangeIndex> RangeValueIdxPtr = MutableParameters->GetValueIndex(ParamIndex, ValueIndex);
						int32 RangeIndex = RangeValueIdxPtr->GetPosition(0);

						if (!Param.ParameterRangeValues.IsValidIndex(RangeIndex))
						{
							const int32 PreviousNum = Param.ParameterRangeValues.Num();
							Param.ParameterRangeValues.AddUninitialized(RangeIndex + 1 - Param.ParameterRangeValues.Num());

							for (int32 Index = PreviousNum; Index < Param.ParameterRangeValues.Num(); ++Index)
							{
								Param.ParameterRangeValues[Index] = FInstancedStruct();
							}
						}

						Param.ParameterRangeValues[RangeIndex] = *MutableParameters->GetInstancedStructValue(ParamIndex, RangeValueIdxPtr.Get());
					}
				}
				else
				{
					Param.ParameterValue = *MutableParameters->GetInstancedStructValue(ParamIndex);
				}
			}

			ExternalTypeParameters.Add(Param);

			break;
		}
			
		default:
			check(false); // Missing case.
			break;
		}
	}
}
 
 
bool FCustomizableObjectInstanceDescriptor::HasAnyParameters() const
{
	return !BoolParameters.IsEmpty() ||
		!IntParameters.IsEmpty() ||
		!FloatParameters.IsEmpty() || 
		!TextureParameters.IsEmpty() ||
		!ProjectorParameters.IsEmpty() ||
		!TransformParameters.IsEmpty() ||
		!VectorParameters.IsEmpty();
}
 
#if WITH_EDITOR 
 
#define RETURN_ON_UNCOMPILED_CO(CustomizableObject, ErrorMessage) \
	if (!CustomizableObject->IsCompiled()) \
	{ \
		FString AdditionalLoggingInfo = FString::Printf(TEXT("Calling function: %hs.  %s"), __FUNCTION__, ErrorMessage); \
		CustomizableObject->GetPrivate()->AddUncompiledCOWarning(AdditionalLoggingInfo);\
		return; \
	} \
 
#else
 
#define RETURN_ON_UNCOMPILED_CO(CustomizableObject, ErrorMessage) \
	if (!ensureMsgf(CustomizableObject->IsCompiled(), TEXT("Customizable Object (%s) was not compiled."), *GetNameSafe(CustomizableObject))) \
	{ \
		FString AdditionalLoggingInfo = FString::Printf(TEXT("Calling function: %hs.  %s"), __FUNCTION__, ErrorMessage); \
		CustomizableObject->GetPrivate()->AddUncompiledCOWarning(AdditionalLoggingInfo);\
		return; \
	} \
 
#endif
	
 
 
void LogParameterNotFoundWarning(const FString& ParameterName, const int32 ObjectParameterIndex, const int32 InstanceParameterIndex, const UCustomizableObject* CustomizableObject, char const* CallingFunction)
{
	UE_LOGF(LogMutable, Warning,
		"%s: Failed to find parameter (%ls) on CO (%ls). CO parameter index: (%d). COI parameter index: (%d)s",
		CallingFunction, *ParameterName, *GetNameSafe(CustomizableObject), ObjectParameterIndex, InstanceParameterIndex
	);
}
 
 
const FString& FCustomizableObjectInstanceDescriptor::GetIntParameterSelectedOption(const FString& ParamName, const int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectBoolParameterValue::DEFAULT_PARAMETER_VALUE_NAME;
	}
	
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(ParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(ParamName, EMutableParameterType::Int);
 
	if (ParameterIndexInObject >= 0 && IntParameters.IsValidIndex(ParameterIndexInInstance))
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return IntParameters[ParameterIndexInInstance].ParameterValueName;
		}
		else
		{
			if (IntParameters[ParameterIndexInInstance].ParameterRangeValueNames.IsValidIndex(RangeIndex))
			{
				return IntParameters[ParameterIndexInInstance].ParameterRangeValueNames[RangeIndex];
			}
		}
	}
 
	LogParameterNotFoundWarning(ParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
 
	return FCustomizableObjectBoolParameterValue::DEFAULT_PARAMETER_VALUE_NAME;
}
 
 
void FCustomizableObjectInstanceDescriptor::SetIntParameterSelectedOption(const int32 ParameterIndexInInstance, const FString& SelectedOption, const int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Int parameter "));
 
	const int32 ParameterIndexInObject = IntParameters.IsValidIndex(ParameterIndexInInstance) ? CustomizableObject->GetPrivate()->FindParameter(IntParameters[ParameterIndexInInstance].ParameterName) : INDEX_NONE; //-V781
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Warn and early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(TEXT("Unknown Int Parmeter"), ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	const bool bValid = SelectedOption == TEXT("None") || CustomizableObject->GetPrivate()->FindIntParameterValue(ParameterIndexInObject, SelectedOption) >= 0;
	if (!bValid)
	{
#if !UE_BUILD_SHIPPING
		const FString Message = FString::Printf(
			TEXT("Tried to set the invalid value [%s] to parameter [%d, %s]! Value index=[%d]. Correct values=[%s]."),
			*SelectedOption, ParameterIndexInObject,
			*IntParameters[ParameterIndexInInstance].ParameterName,
			CustomizableObject->GetPrivate()->FindIntParameterValue(ParameterIndexInObject, SelectedOption),
			*GetAvailableOptionsString(*CustomizableObject, ParameterIndexInObject)
		);
		UE_LOGF(LogMutable, Error, "%ls", *Message);
#endif
		return;
	}
 
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		IntParameters[ParameterIndexInInstance].ParameterValueName = SelectedOption;
	}
	else
	{
		// Contingency in case the RangeIndex can not be used as an index of the "ParameterRangeValueNames" array
		if (!IntParameters[ParameterIndexInInstance].ParameterRangeValueNames.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = IntParameters[ParameterIndexInInstance].ParameterRangeValueNames.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - InsertionIndex;
			IntParameters[ParameterIndexInInstance].ParameterRangeValueNames.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(IntParameters[ParameterIndexInInstance].ParameterRangeValueNames.IsValidIndex(RangeIndex));
		IntParameters[ParameterIndexInInstance].ParameterRangeValueNames[RangeIndex] = SelectedOption;
	}
}
 
 
void FCustomizableObjectInstanceDescriptor::SetIntParameterSelectedOption(const FString& ParamName, const FString& SelectedOptionName, const int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Int parameter "));
 
	const int32 ParamIndexInInstance = FindTypedParameterIndex(ParamName, EMutableParameterType::Int);
	if (ParamIndexInInstance == INDEX_NONE)
	{
		// Warn and early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(ParamName, ParamIndexInInstance, ParamIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	SetIntParameterSelectedOption(ParamIndexInInstance, SelectedOptionName, RangeIndex);
}
 
 
float FCustomizableObjectInstanceDescriptor::GetFloatParameterSelectedOption(const FString& FloatParamName, const int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectFloatParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(FloatParamName);
	const int32 FloatParamIndex = FindTypedParameterIndex(FloatParamName, EMutableParameterType::Float);
 
	if (ParameterIndexInObject >= 0 && FloatParameters.IsValidIndex(FloatParamIndex))
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return FloatParameters[FloatParamIndex].ParameterValue;
		}
		else
		{
			if (FloatParameters[FloatParamIndex].ParameterRangeValues.IsValidIndex(RangeIndex))
			{
				return FloatParameters[FloatParamIndex].ParameterRangeValues[RangeIndex];
			}
		}
	}
 
	LogParameterNotFoundWarning(FloatParamName, ParameterIndexInObject, FloatParamIndex, CustomizableObject, __FUNCTION__);
	
	return FCustomizableObjectFloatParameterValue::DEFAULT_PARAMETER_VALUE; 
}
 
 
void FCustomizableObjectInstanceDescriptor::SetFloatParameterSelectedOption(const FString& FloatParamName, const float FloatValue, const int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Int parameter "));
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(FloatParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(FloatParamName, EMutableParameterType::Float);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Warn and early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(FloatParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		FloatParameters[ParameterIndexInInstance].ParameterValue = FloatValue;
	}
	else
	{
		if (!FloatParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = FloatParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - FloatParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			FloatParameters[ParameterIndexInInstance].ParameterRangeValues.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(FloatParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex));
		FloatParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex] = FloatValue;
	}
}
 
 
UTexture* FCustomizableObjectInstanceDescriptor::GetTextureParameterSelectedOption(const FString& TextureParamName, const int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return nullptr;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(TextureParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(TextureParamName, EMutableParameterType::Texture);
 
	if (ParameterIndexInObject >= 0 && ParameterIndexInInstance >= 0)
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return TextureParameters[ParameterIndexInInstance].ParameterValue;
		}
		else
		{
			if (TextureParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
			{
				return TextureParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex];
			}
		}
	}
	
	LogParameterNotFoundWarning(TextureParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
 
	return nullptr;
}
 
 
void FCustomizableObjectInstanceDescriptor::SetTextureParameterSelectedOption(const FString& TextureParamName, UTexture* TextureValue, const int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Texture parameter "));
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(TextureParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(TextureParamName, EMutableParameterType::Texture);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(TextureParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		TextureParameters[ParameterIndexInInstance].ParameterValue = TextureValue;
	}
	else
	{
		if (!TextureParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = TextureParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - TextureParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			TextureParameters[ParameterIndexInInstance].ParameterRangeValues.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(TextureParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex));
		TextureParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex] = TextureValue;
	}
}
 
 
USkeletalMesh* FCustomizableObjectInstanceDescriptor::GetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return nullptr;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(SkeletalMeshParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(SkeletalMeshParamName, EMutableParameterType::SkeletalMesh);
 
	if (ParameterIndexInObject >= 0 && ParameterIndexInInstance >= 0)
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return SkeletalMeshParameters[ParameterIndexInInstance].ParameterValue;
		}
		else
		{
			if (SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
			{
				return SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex];
			}
		}
	}
	
	LogParameterNotFoundWarning(SkeletalMeshParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
 
	return nullptr; 
}
 
 
void FCustomizableObjectInstanceDescriptor::SetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, USkeletalMesh* SkeletalMeshValue, int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Skeletal Mesh parameter "));
	
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(SkeletalMeshParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(SkeletalMeshParamName, EMutableParameterType::SkeletalMesh);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(SkeletalMeshParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		SkeletalMeshParameters[ParameterIndexInInstance].ParameterValue = SkeletalMeshValue;
	}
	else
	{
		if (!SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex));
		SkeletalMeshParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex] = SkeletalMeshValue;
	}
}


UMaterialInterface* FCustomizableObjectInstanceDescriptor::GetMaterialParameterSelectedOption(const FString& MaterialParamName,	int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return nullptr;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(MaterialParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(MaterialParamName, EMutableParameterType::Material);
 
	if (ParameterIndexInObject >= 0 && ParameterIndexInInstance >= 0)
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return MaterialParameters[ParameterIndexInInstance].ParameterValue;
		}
		else
		{
			if (MaterialParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
			{
				return MaterialParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex];
			}
		}
	}
	
	LogParameterNotFoundWarning(MaterialParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
 
	return nullptr; 
}


void FCustomizableObjectInstanceDescriptor::SetMaterialParameterSelectedOption(const FString& MaterialParamName, UMaterialInterface* MaterialValue,	int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Material parameter "));
	
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(MaterialParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(MaterialParamName, EMutableParameterType::Material);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(MaterialParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		MaterialParameters[ParameterIndexInInstance].ParameterValue = MaterialValue;
	}
	else
	{
		if (!MaterialParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = MaterialParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - MaterialParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			MaterialParameters[ParameterIndexInInstance].ParameterRangeValues.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(MaterialParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex));
		MaterialParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex] = MaterialValue;
	}
}


FInstancedStruct FCustomizableObjectInstanceDescriptor::GetExternalTypeParameterSelectedOption(const FString& ExternalTypeParameterName, int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectExternalTypeParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(ExternalTypeParameterName);
	const int32 ExternalTypeParamIndex = FindTypedParameterIndex(ExternalTypeParameterName, EMutableParameterType::InstancedStruct);
 
	if (ParameterIndexInObject >= 0 && ExternalTypeParameters.IsValidIndex(ExternalTypeParamIndex))
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return ExternalTypeParameters[ExternalTypeParamIndex].ParameterValue;
		}
		else
		{
			if (ExternalTypeParameters[ExternalTypeParamIndex].ParameterRangeValues.IsValidIndex(RangeIndex))
			{
				return ExternalTypeParameters[ExternalTypeParamIndex].ParameterRangeValues[RangeIndex];
			}
		}
	}
 
	LogParameterNotFoundWarning(ExternalTypeParameterName, ParameterIndexInObject, ExternalTypeParamIndex, CustomizableObject, __FUNCTION__);
	
	return FCustomizableObjectExternalTypeParameterValue::DEFAULT_PARAMETER_VALUE; 
}


void FCustomizableObjectInstanceDescriptor::SetExternalTypeParameterSelectedOption(const FString& ExternalTypeParamName, const FInstancedStruct& ExternalTypeValue, int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Int parameter "));
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(ExternalTypeParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(ExternalTypeParamName, EMutableParameterType::InstancedStruct);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Warn and early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(ExternalTypeParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}

	const UModelResources& ModelResources = CustomizableObject->GetPrivate()->GetModelResourcesChecked();
	if (!ExternalTypeValue.GetScriptStruct()->IsChildOf(ModelResources.ExternalTypeParameterDefaultValues[ExternalTypeParamName].GetScriptStruct()))
	{
		UE_LOGF(LogMutable, Error, "External Parameter %ls incorrect type.", *ExternalTypeParamName);
		return;
	}
	
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		ExternalTypeParameters[ParameterIndexInInstance].ParameterValue = ExternalTypeValue;
	}
	else
	{
		if (!ExternalTypeParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = ExternalTypeParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - ExternalTypeParameters[ParameterIndexInInstance].ParameterRangeValues.Num();
			ExternalTypeParameters[ParameterIndexInInstance].ParameterRangeValues.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(ExternalTypeParameters[ParameterIndexInInstance].ParameterRangeValues.IsValidIndex(RangeIndex));
		ExternalTypeParameters[ParameterIndexInInstance].ParameterRangeValues[RangeIndex] = ExternalTypeValue;
	}
}


FVector4f FCustomizableObjectInstanceDescriptor::GetVectorParameterSelectedOption(const FString& VectorParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectVectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(VectorParamName);
	const int32 VectorParamIndex = FindTypedParameterIndex(VectorParamName, EMutableParameterType::Color);
 
	if (VectorParamIndex == INDEX_NONE)
	{
		LogParameterNotFoundWarning(VectorParamName, ParameterIndexInObject, VectorParamIndex, CustomizableObject, __FUNCTION__);
		return FCustomizableObjectVectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	// TODO: Multidimensional parameter support
	return VectorParameters.IsValidIndex(VectorParamIndex) ? VectorParameters[VectorParamIndex].ParameterValue : FCustomizableObjectVectorParameterValue::DEFAULT_PARAMETER_VALUE;
}


void FCustomizableObjectInstanceDescriptor::SetVectorParameterSelectedOption(const FString& VectorParamName, const FVector4f& VectorValue)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Int parameter "));
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(VectorParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(VectorParamName, EMutableParameterType::Color);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(VectorParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	VectorParameters[ParameterIndexInInstance].ParameterValue = VectorValue;
}


FTransform FCustomizableObjectInstanceDescriptor::GetTransformParameterSelectedOption(const FString& TransformParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectTransformParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(TransformParamName);
	const int32 TransformParamIndex = FindTypedParameterIndex(TransformParamName, EMutableParameterType::Transform);
 
	if (TransformParamIndex == INDEX_NONE)
	{
		LogParameterNotFoundWarning(TransformParamName, ParameterIndexInObject, TransformParamIndex, CustomizableObject, __FUNCTION__);
		return FCustomizableObjectTransformParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	// TODO: Multidimensional parameter support
	return TransformParameters.IsValidIndex(TransformParamIndex) ? TransformParameters[TransformParamIndex].ParameterValue : FTransform::Identity;
}
 
void FCustomizableObjectInstanceDescriptor::SetTransformParameterSelectedOption(const FString& TransformParamName, const FTransform& TransformValue)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Transform parameter "));
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(TransformParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(TransformParamName, EMutableParameterType::Transform);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(TransformParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	// TODO: Multidimensional parameter support
	TransformParameters[ParameterIndexInInstance].ParameterValue = TransformValue;
}
 
 
bool FCustomizableObjectInstanceDescriptor::GetBoolParameterSelectedOption(const FString& BoolParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectBoolParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(BoolParamName);
	const int32 BoolParamIndex = FindTypedParameterIndex(BoolParamName, EMutableParameterType::Bool);
 
	if (BoolParamIndex == INDEX_NONE)
	{
		LogParameterNotFoundWarning(BoolParamName, ParameterIndexInObject, BoolParamIndex, CustomizableObject, __FUNCTION__);
		return FCustomizableObjectBoolParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	return BoolParameters[BoolParamIndex].ParameterValue;
}
 
 
void FCustomizableObjectInstanceDescriptor::SetBoolParameterSelectedOption(const FString& BoolParamName, const bool BoolValue)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Int parameter "));
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(BoolParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(BoolParamName, EMutableParameterType::Bool);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(BoolParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	BoolParameters[ParameterIndexInInstance].ParameterValue = BoolValue;
}

 
void FCustomizableObjectInstanceDescriptor::SetProjectorParameterSelectedOption(const FString& ProjectorParamName, const FCustomizableObjectProjector& ProjectorValue, const int32 RangeIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set Projector parameter "))
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(ProjectorParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(ProjectorParamName, EMutableParameterType::Projector);
 
	if (ParameterIndexInObject < 0 || ParameterIndexInInstance < 0)
	{
		// Early out since we could not find the parameter to set.
		LogParameterNotFoundWarning(ProjectorParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
		return;
	}
 
	// Parameter to modify
	FCustomizableObjectProjectorParameterValue& ProjectorParameter = ProjectorParameters[ParameterIndexInInstance];
	if (ProjectorValue.ProjectionType != ProjectorParameter.Value.ProjectionType)
	{
		UE_LOGF(LogMutable, Warning, "Changing the Projection type is not allowed once the CO has been compiled. In order to change it update the CO Projector Parameter settings and recompile the CO.");
	}
 
	// New Value
	FCustomizableObjectProjector ProjectorData = ProjectorValue;
	ProjectorData.ProjectionType = ProjectorParameter.Value.ProjectionType;
 
	// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
	// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
	bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
	if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
	{
		ProjectorParameter.Value = ProjectorData;
	}
	else
	{
		if (!ProjectorParameter.RangeValues.IsValidIndex(RangeIndex))
		{
			const int32 InsertionIndex = ProjectorParameter.RangeValues.Num();
			const int32 NumInsertedElements = RangeIndex + 1 - ProjectorParameter.RangeValues.Num();
			ProjectorParameter.RangeValues.InsertDefaulted(InsertionIndex, NumInsertedElements);
		}
 
		check(ProjectorParameter.RangeValues.IsValidIndex(RangeIndex));
		ProjectorParameter.RangeValues[RangeIndex] = ProjectorData;
	}
}
 
 
void FCustomizableObjectInstanceDescriptor::SetProjectorValue(const FString& ProjectorParamName,
	const FVector& Pos, const FVector& Direction, const FVector& Up, const FVector& Scale,
	const float Angle,
	const int32 RangeIndex)
{
	FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
	ProjectorData.Position =  static_cast<FVector3f>(Pos);
	ProjectorData.Direction = static_cast<FVector3f>(Direction);
	ProjectorData.Up = static_cast<FVector3f>(Up);
	ProjectorData.Scale = static_cast<FVector3f>(Scale);
	ProjectorData.Angle = Angle;
	
	SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorData, RangeIndex);
}


void FCustomizableObjectInstanceDescriptor::SetProjectorPosition(const FString& ProjectorParamName, const FVector& Pos, const int32 RangeIndex)
{
	FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
	ProjectorData.Position = static_cast<FVector3f>(Pos);
	
	SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorData, RangeIndex);
}
 
 
void FCustomizableObjectInstanceDescriptor::SetProjectorDirection(const FString& ProjectorParamName, const FVector& Direction, int32 RangeIndex)
{
	FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
	ProjectorData.Direction = static_cast<FVector3f>(Direction);
		
	SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorData, RangeIndex);
}
 
 
void FCustomizableObjectInstanceDescriptor::SetProjectorUp(const FString& ProjectorParamName, const FVector& Up, int32 RangeIndex)
{
	FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
	ProjectorData.Up = static_cast<FVector3f>(Up);
 
	SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorData, RangeIndex);
}
 
 
void FCustomizableObjectInstanceDescriptor::SetProjectorScale(const FString& ProjectorParamName, const FVector& Scale, int32 RangeIndex)
{
	FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
	ProjectorData.Scale = static_cast<FVector3f>(Scale);
	
	SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorData, RangeIndex);
}
 
 
void FCustomizableObjectInstanceDescriptor::SetProjectorAngle(const FString& ProjectorParamName, float Angle, int32 RangeIndex)
{
	FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
	ProjectorData.Angle = Angle;
	
	SetProjectorParameterSelectedOption(ProjectorParamName, ProjectorData, RangeIndex);
}
 
 
void FCustomizableObjectInstanceDescriptor::GetProjectorValue(const FString& ProjectorParamName,
                                                              FVector& OutPos, FVector& OutDirection, FVector& OutUp, FVector& OutScale,
                                                              float& OutAngle, ECustomizableObjectProjectorType& OutType,
                                                              const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
 
	OutPos = static_cast<FVector>(ProjectorData.Position);
	OutDirection = static_cast<FVector>(ProjectorData.Direction);
	OutUp = static_cast<FVector>(ProjectorData.Up);
	OutScale = static_cast<FVector>(ProjectorData.Scale);
	OutAngle = ProjectorData.Angle;
	OutType = ProjectorData.ProjectionType;
}
 
 
void FCustomizableObjectInstanceDescriptor::GetProjectorValueF(const FString& ProjectorParamName,
	FVector3f& OutPos, FVector3f& OutDirection, FVector3f& OutUp, FVector3f& OutScale,
	float& OutAngle, ECustomizableObjectProjectorType& OutType,
	const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ProjectorParamName, RangeIndex);
 
	OutPos = ProjectorData.Position;
	OutDirection = ProjectorData.Direction;
	OutUp = ProjectorData.Up;
	OutScale = ProjectorData.Scale;
	OutAngle = ProjectorData.Angle;
	OutType = ProjectorData.ProjectionType;
}
 
 
FVector FCustomizableObjectInstanceDescriptor::GetProjectorPosition(const FString& ParamName, const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ParamName, RangeIndex);
 
	return static_cast<FVector>(ProjectorData.Position);
}
 
 
FVector FCustomizableObjectInstanceDescriptor::GetProjectorDirection(const FString& ParamName, const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ParamName, RangeIndex);
 
	return static_cast<FVector>(ProjectorData.Direction);
}
	
 
FVector FCustomizableObjectInstanceDescriptor::GetProjectorUp(const FString& ParamName, const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ParamName, RangeIndex);
 
	return static_cast<FVector>(ProjectorData.Up);
}
 
 
FVector FCustomizableObjectInstanceDescriptor::GetProjectorScale(const FString& ParamName, const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ParamName, RangeIndex);
 
	return static_cast<FVector>(ProjectorData.Scale);
}
	
 
float FCustomizableObjectInstanceDescriptor::GetProjectorAngle(const FString& ParamName, const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ParamName, RangeIndex);

	return ProjectorData.Angle;
}
 
 
ECustomizableObjectProjectorType FCustomizableObjectInstanceDescriptor::GetProjectorParameterType(const FString& ParamName, const int32 RangeIndex) const
{
	const FCustomizableObjectProjector ProjectorData = GetProjectorParameterSelectedOption(ParamName, RangeIndex);
 
	return ProjectorData.ProjectionType;
}
 
 
FCustomizableObjectProjector FCustomizableObjectInstanceDescriptor::GetProjectorParameterSelectedOption(const FString& ParamName, const int32 RangeIndex) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FCustomizableObjectProjectorParameterValue::DEFAULT_PARAMETER_VALUE;
	}
 
	const int32 ParameterIndexInObject = CustomizableObject->GetPrivate()->FindParameter(ParamName);
	const int32 ParameterIndexInInstance = FindTypedParameterIndex(ParamName, EMutableParameterType::Projector);
 
	if (ParameterIndexInObject >= 0 && ParameterIndexInInstance >= 0)
	{
		// Due to optimizations a parameter may lose its multidimensionality (if it becomes constant). 
		// In that case it means there is only one possible option so it is ok to set it as if wasn't multidimensional.
		bool bIsMultidimensional = CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParameterIndexInObject);
		if (!bIsMultidimensional || RangeIndex == INDEX_NONE)
		{
			return ProjectorParameters[ParameterIndexInInstance].Value;
		}
		else if (ProjectorParameters[ParameterIndexInInstance].RangeValues.IsValidIndex(RangeIndex))
		{
			return ProjectorParameters[ParameterIndexInInstance].RangeValues[RangeIndex];
		}
	}
 
	LogParameterNotFoundWarning(ParamName, ParameterIndexInObject, ParameterIndexInInstance, CustomizableObject, __FUNCTION__);
 
	return FCustomizableObjectProjectorParameterValue::DEFAULT_PARAMETER_VALUE;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::FindTypedParameterIndex(const FString& ParamName, EMutableParameterType Type) const
{
	return CustomizableObject ? CustomizableObject->GetPrivate()->FindParameterTyped(ParamName, Type) : INDEX_NONE;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::GetProjectorValueRange(const FString& ParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return -1;
	}
 
	const int32 ProjectorParamIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Projector);
	if (ProjectorParamIndex < 0)
	{
		return -1;
	}
 
	return ProjectorParameters[ProjectorParamIndex].RangeValues.Num();
}
 
int32 FCustomizableObjectInstanceDescriptor::GetIntValueRange(const FString& ParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return -1;
	}
 
	const int32 IntParamIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Int);
	if (IntParamIndex < 0)
	{
		return -1;
	}
 
	return IntParameters[IntParamIndex].ParameterRangeValueNames.Num();
}
 
 
int32 FCustomizableObjectInstanceDescriptor::GetFloatValueRange(const FString& ParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return -1;
	}
 
	const int32 FloatParamIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Float);
	if (FloatParamIndex < 0)
	{
		return -1;
	}
 
	return FloatParameters[FloatParamIndex].ParameterRangeValues.Num();
}
 
 
int32 FCustomizableObjectInstanceDescriptor::GetTextureValueRange(const FString& ParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return -1;
	}
 
	const int32 TextureParamIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Texture);
	if (TextureParamIndex < 0)
	{
		return -1;
	}
 
	return TextureParameters[TextureParamIndex].ParameterRangeValues.Num();
}
 
 
int32 FCustomizableObjectInstanceDescriptor::AddValueToIntRange(const FString& ParamName)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return -1;
	}
 
	const int32 IntParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Int);
	if (IntParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectIntParameterValue& IntParameter = IntParameters[IntParameterIndex];
		const int32 ParamIndexInObject = CustomizableObject->GetPrivate()->FindParameter(IntParameter.ParameterName);
		// TODO: Define the default option in the editor instead of taking the first available, like it's currently defined for GetProjectorDefaultValue()
		const FString DefaultValue = CustomizableObject->GetPrivate()->GetIntParameterAvailableOption(ParamIndexInObject, 0);
		return IntParameter.ParameterRangeValueNames.Add(DefaultValue);
	}
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::AddValueToFloatRange(const FString& ParamName)
{
	const int32 FloatParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Float);
	if (FloatParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectFloatParameterValue& FloatParameter = FloatParameters[FloatParameterIndex];
		// TODO: Define the default float in the editor instead of [0.5f], like it's currently defined for GetProjectorDefaultValue()
		return FloatParameter.ParameterRangeValues.Add(0.5f);
	}
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::AddValueToTextureRange(const FString& ParamName)
{
	const int32 TextureParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Texture);
	if (TextureParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectTextureParameterValue& TextureParameter = TextureParameters[TextureParameterIndex];
		return TextureParameter.ParameterRangeValues.Add(nullptr);
	}
	
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::AddValueToProjectorRange(const FString& ParamName)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return -1;
	}
 
	const int32 ProjectorParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Projector);
	if (ProjectorParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectProjectorParameterValue& ProjectorParameter = ProjectorParameters[ProjectorParameterIndex];
		const FCustomizableObjectProjector Projector = GetCustomizableObject()->GetProjectorParameterDefaultValue(ParamName);
		return ProjectorParameter.RangeValues.Add(Projector);
	}
 
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::RemoveValueFromIntRange(const FString& ParamName, const int32 RangeIndex)
{
	const int32 IntParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Int);
	if (IntParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectIntParameterValue& IntParameter = IntParameters[IntParameterIndex];
		if (IntParameter.ParameterRangeValueNames.Num() > 0)
		{
			int32 IndexToRemove = IntParameter.ParameterRangeValueNames.IsValidIndex(RangeIndex) ? RangeIndex : IntParameter.ParameterRangeValueNames.Num() - 1;
			IntParameter.ParameterRangeValueNames.RemoveAt(IndexToRemove);
			return IntParameter.ParameterRangeValueNames.Num() - 1;
		}
	}
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::RemoveValueFromFloatRange(const FString& ParamName, const int32 RangeIndex)
{
	const int32 FloatParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Float);
	if (FloatParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectFloatParameterValue& FloatParameter = FloatParameters[FloatParameterIndex];
		if (FloatParameter.ParameterRangeValues.Num() > 0)
		{
			int32 IndexToRemove = FloatParameter.ParameterRangeValues.IsValidIndex(RangeIndex) ? RangeIndex : FloatParameter.ParameterRangeValues.Num() - 1;
			FloatParameter.ParameterRangeValues.RemoveAt(IndexToRemove);
			return FloatParameter.ParameterRangeValues.Num() - 1;
		}
	}
	return -1;
}

 
int32 FCustomizableObjectInstanceDescriptor::RemoveValueFromTextureRange(const FString& ParamName)
{
	const int32 TextureParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Texture);
	if (TextureParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectTextureParameterValue& TextureParameter = TextureParameters[TextureParameterIndex];
		if (TextureParameter.ParameterRangeValues.Num() > 0)
		{
			TextureParameter.ParameterRangeValues.Pop();
			return TextureParameter.ParameterRangeValues.Num() - 1;
		}
	}
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::RemoveValueFromTextureRange(const FString& ParamName, const int32 RangeIndex)
{
	const int32 TextureParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Texture);
	if (TextureParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectTextureParameterValue& TextureParameter = TextureParameters[TextureParameterIndex];
		if (TextureParameter.ParameterRangeValues.Num() > 0)
		{
			TextureParameter.ParameterRangeValues.RemoveAt(RangeIndex);
			return TextureParameter.ParameterRangeValues.Num() - 1;
		}
	}
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::RemoveValueFromProjectorRange(const FString& ParamName, const int32 RangeIndex)
{
	const int32 ProjectorParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Projector);
	if (ProjectorParameterIndex != INDEX_NONE)
	{
		FCustomizableObjectProjectorParameterValue& ProjectorParameter = ProjectorParameters[ProjectorParameterIndex];
 
		if (ProjectorParameter.RangeValues.Num() > 0)
		{
			int32 IndexToRemove = ProjectorParameter.RangeValues.IsValidIndex(RangeIndex) ? RangeIndex : ProjectorParameter.RangeValues.Num() - 1;
			ProjectorParameter.RangeValues.RemoveAt(IndexToRemove);
			
			return ProjectorParameter.RangeValues.Num() - 1;
		}
	}
	return -1;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::GetState() const
{
	return State;
}
 
 
FString FCustomizableObjectInstanceDescriptor::GetCurrentState() const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return FString();
	}
 
	return CustomizableObject->GetPrivate()->GetStateName(GetState());
}
 
 
void FCustomizableObjectInstanceDescriptor::SetState(const int32 InState)
{
	State = InState;
}
 
 
void FCustomizableObjectInstanceDescriptor::SetCurrentState(const FString& StateName)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set state"))
 
	const int32 Result = CustomizableObject->GetPrivate()->FindState(StateName);
#if WITH_EDITOR
	if (Result != INDEX_NONE)
#else
	if (ensureMsgf(Result != INDEX_NONE, TEXT("Unknown %s state."), *StateName))
#endif
	{
		SetState(Result);
	}
	else
	{
		UE_LOGF(LogMutable, Error, "%hs: Unknown %ls state.", __FUNCTION__, *StateName);
	}
}
 
 
void FCustomizableObjectInstanceDescriptor::SetRandomValues()
{
	const int32 RandomSeed = FMath::SRand() * TNumericLimits<int32>::Max();
	FRandomStream RandomStream{RandomSeed};
	SetRandomValuesFromStream(RandomSeed);
}
 
 
void FCustomizableObjectInstanceDescriptor::SetRandomValuesFromStream(const FRandomStream& InStream)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
	
	RETURN_ON_UNCOMPILED_CO(CustomizableObject, TEXT("Error: Cannot set random values"))
	
	for (FCustomizableObjectFloatParameterValue& FloatParameter : FloatParameters)
	{
		FloatParameter.ParameterValue = InStream.GetFraction();
 
		for (float& RangeValue : FloatParameter.ParameterRangeValues)
		{
			RangeValue = InStream.GetFraction();
		}
	}
 
	for (FCustomizableObjectBoolParameterValue& BoolParameter : BoolParameters)
	{
		BoolParameter.ParameterValue = static_cast<bool>(InStream.RandRange(0, 1));
	}
	
	for (FCustomizableObjectIntParameterValue& IntParameter : IntParameters)
	{
		const int32 NumValues = CustomizableObject->GetEnumParameterNumValues(IntParameter.ParameterName);
 
		if (NumValues)
		{
			IntParameter.ParameterValueName = CustomizableObject->GetEnumParameterValue(IntParameter.ParameterName, NumValues * InStream.GetFraction());
 
			for (FString& RangeValue : IntParameter.ParameterRangeValueNames)
			{
				RangeValue = CustomizableObject->GetEnumParameterValue(IntParameter.ParameterName, NumValues * InStream.GetFraction());
			}
		}
	}
 
	for (FCustomizableObjectVectorParameterValue& VectorParameter : VectorParameters)
	{
		VectorParameter.ParameterValue.R = InStream.GetFraction();
		VectorParameter.ParameterValue.G = InStream.GetFraction();
		VectorParameter.ParameterValue.B = InStream.GetFraction();
		VectorParameter.ParameterValue.A = InStream.GetFraction();
	}

	// Currently we are not randomizing the projectors since we do not know the valid range of values.
}


void FCustomizableObjectInstanceDescriptor::SetDefaultValue(int32 ParamIndex)
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	if (ParamIndex >= CustomizableObject->GetParameterCount())
	{
		return;
	}
 
	const FString ParamName = CustomizableObject->GetParameterName(ParamIndex);
	const EMutableParameterType ParamType = CustomizableObject->GetPrivate()->GetParameterType(ParamIndex);
	
	const int32 TypedIndex = FindTypedParameterIndex(ParamName, ParamType);
 
	switch (ParamType)
	{
	case EMutableParameterType::Bool:
	{
		const bool DefaultValue = CustomizableObject->GetBoolParameterDefaultValue(ParamName);
		BoolParameters[TypedIndex].ParameterValue = DefaultValue;
		break;
	}
 
	case EMutableParameterType::Int:
	{
		const FString DefaultValue = CustomizableObject->GetPrivate()->FindIntParameterValueName(ParamIndex, CustomizableObject->GetEnumParameterDefaultValue(ParamName));
 
		FCustomizableObjectIntParameterValue& IntParameter = IntParameters[TypedIndex];
		IntParameter.ParameterValueName = DefaultValue;
		IntParameter.ParameterRangeValueNames.Empty();
		break;
	}
 
	case EMutableParameterType::Float:
	{
		const float DefaultValue = CustomizableObject->GetFloatParameterDefaultValue(ParamName);
 
		FCustomizableObjectFloatParameterValue& FloatParameter = FloatParameters[TypedIndex];
		FloatParameter.ParameterValue = DefaultValue;
		FloatParameter.ParameterRangeValues.Empty();
		break;
	}
 
	case EMutableParameterType::Color:
	{
		const FLinearColor DefaultValue = CustomizableObject->GetVectorParameterDefaultValue(ParamName);
		VectorParameters[TypedIndex].ParameterValue = DefaultValue;
		break;
	}
 
	case EMutableParameterType::Transform:
	{
		const FTransform DefaultValue = CustomizableObject->GetTransformParameterDefaultValue(ParamName);
		TransformParameters[TypedIndex].ParameterValue = DefaultValue;
		break;
	}
		
	case EMutableParameterType::Projector:
	{
		const FCustomizableObjectProjector DefaultValue = CustomizableObject->GetProjectorParameterDefaultValue(ParamName);
 
		FCustomizableObjectProjectorParameterValue& ProjectorParameter = ProjectorParameters[TypedIndex];
		ProjectorParameter.Value = DefaultValue;
		ProjectorParameter.RangeValues.Empty();
		break;
	}
 
	case EMutableParameterType::Texture:
	{
		UTexture* DefaultValue = CustomizableObject->GetTextureParameterDefaultValue(ParamName);
		
		if (ensure(TextureParameters.IsValidIndex(TypedIndex)))
		{
			FCustomizableObjectTextureParameterValue& TextureParameter = TextureParameters[TypedIndex];
			TextureParameter.ParameterValue = DefaultValue;
			TextureParameter.ParameterRangeValues.Empty();
		}
		break;
	}
		
	case EMutableParameterType::SkeletalMesh:
	{
		USkeletalMesh* DefaultValue = CustomizableObject->GetSkeletalMeshParameterDefaultValue(ParamName);
	
		if (ensure(SkeletalMeshParameters.IsValidIndex(TypedIndex)))
		{
			FCustomizableObjectSkeletalMeshParameterValue& SkeletalMeshParameter = SkeletalMeshParameters[TypedIndex];
			SkeletalMeshParameter.ParameterValue = DefaultValue;
			SkeletalMeshParameter.ParameterRangeValues.Empty();
		}
		break;
	}

	case EMutableParameterType::Material:
	{
		UMaterialInterface* DefaultValue = CustomizableObject->GetMaterialParameterDefaultValue(ParamName);

		if (ensure(MaterialParameters.IsValidIndex(TypedIndex)))
		{
			FCustomizableObjectMaterialParameterValue& MaterialParameter = MaterialParameters[TypedIndex];
			MaterialParameter.ParameterValue = DefaultValue;
			MaterialParameter.ParameterRangeValues.Empty();
		}
		break;
	}

	case EMutableParameterType::InstancedStruct:
	{
		FInstancedStruct DefaultValue = CustomizableObject->GetExternalTypeParameterDefaultValue(ParamName);

		if (ensure(ExternalTypeParameters.IsValidIndex(TypedIndex)))
		{
			FCustomizableObjectExternalTypeParameterValue& ExternalTypeParameter = ExternalTypeParameters[TypedIndex];
			ExternalTypeParameter.ParameterValue = DefaultValue;
			ExternalTypeParameter.ParameterRangeValues.Empty();
		}
		break;
	}
		
	default:
		// Parameter type replication not implemented.
		unimplemented();
	}
}
 
 
void FCustomizableObjectInstanceDescriptor::SetDefaultValues()
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return;
	}
 
	const int32 NumParameters = CustomizableObject->GetParameterCount();
	for (int32 ParamIndex = 0; ParamIndex < NumParameters; ++ParamIndex)
	{
		SetDefaultValue(ParamIndex);
	}
}
 
bool FCustomizableObjectInstanceDescriptor::IsMultilayerProjector(const FString& ParamName) const
{
	if (!CustomizableObject)
	{
		CustomizableObjectNullErrorMessage();
		return false;
	}
 
	// Projector.
	if (const int32 ProjectorParameterIndex = FindTypedParameterIndex(ParamName, EMutableParameterType::Projector);
		ProjectorParameterIndex == INDEX_NONE)
	{
		ensureAlwaysMsgf(false, TEXT("%s"), *MULTILAYER_PROJECTOR_PARAMETERS_INVALID);
		return false;
	}
 
	// Num layers.
	if (const int32 FloatParameterIndex = FindTypedParameterIndex(ParamName + NUM_LAYERS_PARAMETER_POSTFIX, EMutableParameterType::Float);
		FloatParameterIndex == INDEX_NONE)
	{
		ensureAlwaysMsgf(false, TEXT("%s"), *MULTILAYER_PROJECTOR_PARAMETERS_INVALID);
		return false;
	}
 
	// Selected Image.
	if (const int32 IntParameterIndex = FindTypedParameterIndex(ParamName + IMAGE_PARAMETER_POSTFIX, EMutableParameterType::Int);
		IntParameterIndex == INDEX_NONE)
	{
		ensureAlwaysMsgf(false, TEXT("%s"), *MULTILAYER_PROJECTOR_PARAMETERS_INVALID);
		return false;
	}
 
	// Opacity.	
	if (const int32 FloatParameterIndex = FindTypedParameterIndex(ParamName + OPACITY_PARAMETER_POSTFIX, EMutableParameterType::Float);
		FloatParameterIndex == INDEX_NONE)
	{
		ensureAlwaysMsgf(false, TEXT("%s"), *MULTILAYER_PROJECTOR_PARAMETERS_INVALID);
		return false;
	}
 
	return true;
}
 
 
int32 FCustomizableObjectInstanceDescriptor::NumProjectorLayers(const FName& ProjectorParamName) const
{
	const FString ParamName = ProjectorParamName.ToString();
 
	if (IsMultilayerProjector(ParamName))
	{
		return GetFloatParameterSelectedOption(ParamName + NUM_LAYERS_PARAMETER_POSTFIX);
	}
 
	return INDEX_NONE;
}
 
 
void FCustomizableObjectInstanceDescriptor::CreateLayer(const FName& ProjectorParamName, int32 RangeIndex)
{
	const FString ParamName = ProjectorParamName.ToString();
 
	if (!IsMultilayerProjector(ParamName))
	{
		return;
	}
 
	// Num Layers.
	SetFloatParameterSelectedOption(ParamName + NUM_LAYERS_PARAMETER_POSTFIX, NumProjectorLayers(ProjectorParamName) + 1);
 
	// Projector Range. New value is defaulted.
	AddValueToProjectorRange(ParamName);
 
	// Selected Image Range.
	{
		const FString ImageParamName = ParamName + IMAGE_PARAMETER_POSTFIX;
		AddValueToTextureRange(ImageParamName);
 
		const int32 DefaultValueIndex = CustomizableObject->GetEnumParameterDefaultValue(ImageParamName);
		const FString DefaultValueName = CustomizableObject->GetEnumParameterValue(ImageParamName, DefaultValueIndex);
		SetIntParameterSelectedOption(ImageParamName, DefaultValueName, RangeIndex);
	}
 
	// Opacity Range.
	{
		const FString OpacityParamName = ParamName + OPACITY_PARAMETER_POSTFIX;
		AddValueToFloatRange(OpacityParamName);
 
		const float DefaultValue = CustomizableObject->GetFloatParameterDefaultValue(OpacityParamName);
		SetFloatParameterSelectedOption(OpacityParamName, DefaultValue, RangeIndex);
	}
}
 
 
void FCustomizableObjectInstanceDescriptor::RemoveLayerAt(const FName& ProjectorParamName, int32 RangeIndex)
{
	const FString ParamName = ProjectorParamName.ToString();
	const int32 NumLayers = NumProjectorLayers(ProjectorParamName);
	check(RangeIndex >= 0 && RangeIndex < NumLayers); // Layer out of range.
 
	if (!IsMultilayerProjector(ParamName))
	{
		return;
	}
 
	// Num Layers.
	SetFloatParameterSelectedOption(ParamName + NUM_LAYERS_PARAMETER_POSTFIX, NumLayers - 1);
 
	// Projector Range.
	RemoveValueFromProjectorRange(ParamName, RangeIndex);
 
	// Selected Image Range.
	RemoveValueFromIntRange(ParamName + IMAGE_PARAMETER_POSTFIX, RangeIndex);
 
	// Opacity Range.
	RemoveValueFromFloatRange(ParamName + OPACITY_PARAMETER_POSTFIX, RangeIndex);
}
 
 
FMultilayerProjectorLayer FCustomizableObjectInstanceDescriptor::GetLayer(const FName& ProjectorParamName, int32 Index) const
{
	const FString ParamName = ProjectorParamName.ToString();
 
	FMultilayerProjectorLayer ProjectorLayer;
 
	if (IsMultilayerProjector(ParamName))
	{
		ProjectorLayer.Read(*this, ParamName, Index);
	}
 
	return ProjectorLayer;
}
 
 
void FCustomizableObjectInstanceDescriptor::UpdateLayer(const FName& ProjectorParamName, int32 Index, const FMultilayerProjectorLayer& Layer)
{
	const FString& ParamName = ProjectorParamName.ToString();
 
	if (!IsMultilayerProjector(ParamName))
	{
		return;
	}
 
	Layer.Write(*this, ParamName, Index);
}
 
 
#undef RETURN_ON_CO_UNCOMPILED
