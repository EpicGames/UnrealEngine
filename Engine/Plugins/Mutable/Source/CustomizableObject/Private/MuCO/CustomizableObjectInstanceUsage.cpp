// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/CustomizableObjectInstanceUsagePrivate.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectInstanceUsage)


void UCustomizableObjectInstanceUsagePrivate::Callbacks()
{
	GetPublic()->UpdatedDelegate.ExecuteIfBound();
}


UCustomizableObjectInstanceUsage::UCustomizableObjectInstanceUsage()
{
	Private = CreateDefaultSubobject<UCustomizableObjectInstanceUsagePrivate>(FName("Private"));
}


void UCustomizableObjectInstanceUsage::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (UsedCustomizableObjectInstance)
	{
		UsedCustomizableObjectInstance->GetPrivate()->RegisterCustomizableObjectInstanceUsage(*this);
	}
}

void UCustomizableObjectInstanceUsage::PostLoad()
{
	Super::PostLoad();

	if (UsedCustomizableObjectInstance)
	{
		UsedCustomizableObjectInstance->GetPrivate()->RegisterCustomizableObjectInstanceUsage(*this);
	}
}

void UCustomizableObjectInstanceUsage::BeginDestroy()
{
	if (UsedCustomizableObjectInstance)
	{
		UsedCustomizableObjectInstance->GetPrivate()->UnregisterCustomizableObjectInstanceUsage(*this);
	}

	Super::BeginDestroy();
}

#if WITH_EDITOR
void UCustomizableObjectInstanceUsage::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectInstanceUsage, UsedCustomizableObjectInstance))
	{
		UCustomizableObjectInstance* Instance = UsedCustomizableObjectInstance;
		UsedCustomizableObjectInstance = OldCustomizableObjectInstance.Get(true);
		SetCustomizableObjectInstance(Instance);
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectInstanceUsage, UsedComponentName))
	{
		SetComponentName(UsedComponentName);
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectInstanceUsage, bUsedSkipSetReferenceSkeletalMesh))
	{
		SetSkipSetReferenceSkeletalMesh(bUsedSkipSetReferenceSkeletalMesh);
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectInstanceUsage, bUsedSkipSetSkeletalMeshOnAttach))
	{
		SetSkipSetSkeletalMeshOnAttach(bUsedSkipSetSkeletalMeshOnAttach);
	}
}
#endif

void UCustomizableObjectInstanceUsage::SetCustomizableObjectInstance(UCustomizableObjectInstance* InCustomizableObjectInstance)
{
	if (UsedCustomizableObjectInstance != InCustomizableObjectInstance)
	{
		if (UsedCustomizableObjectInstance)
		{
			UsedCustomizableObjectInstance->GetPrivate()->UnregisterCustomizableObjectInstanceUsage(*this);
		}

		UsedCustomizableObjectInstance = InCustomizableObjectInstance;

		if (IsValid(UsedCustomizableObjectInstance))
		{
			UsedCustomizableObjectInstance->GetPrivate()->RegisterCustomizableObjectInstanceUsage(*this);
		}

#if WITH_EDITOR
		OldCustomizableObjectInstance = MakeWeakObjectPtr(UsedCustomizableObjectInstance);
#endif
	}
}


UCustomizableObjectInstance* UCustomizableObjectInstanceUsage::GetCustomizableObjectInstance() const
{
	return UsedCustomizableObjectInstance;
}


void UCustomizableObjectInstanceUsage::SetComponentName(const FName& Name)
{
	UsedComponentName = Name;
}


FName UCustomizableObjectInstanceUsage::GetComponentName() const
{
	return UsedComponentName;
}


UCustomizableObjectInstanceUsage* UCustomizableObjectInstanceUsagePrivate::GetPublic()
{
	UCustomizableObjectInstanceUsage* Public = StaticCast<UCustomizableObjectInstanceUsage*>(GetOuter());
	check(Public);

	return Public;
}


const UCustomizableObjectInstanceUsage* UCustomizableObjectInstanceUsagePrivate::GetPublic() const
{
	UCustomizableObjectInstanceUsage* Public = StaticCast<UCustomizableObjectInstanceUsage*>(GetOuter());
	check(Public);

	return Public;
}


void UCustomizableObjectInstanceUsage::SetSkipSetReferenceSkeletalMesh(bool bSkip)
{
	bUsedSkipSetReferenceSkeletalMesh = bSkip;
}


bool UCustomizableObjectInstanceUsage::GetSkipSetReferenceSkeletalMesh() const
{
	return bUsedSkipSetReferenceSkeletalMesh;
}


void UCustomizableObjectInstanceUsage::SetSkipSetSkeletalMeshOnAttach(bool bSkip)
{
	bUsedSkipSetSkeletalMeshOnAttach = bSkip;
}


void UCustomizableObjectInstanceUsage::AttachTo(USkeletalMeshComponent* SkeletalMeshComponent)
{
	if (IsValid(SkeletalMeshComponent))
	{
		UsedSkeletalMeshComponent = SkeletalMeshComponent;
		GetPrivate()->bIsNetModeDedicatedServer = SkeletalMeshComponent->IsNetMode(NM_DedicatedServer);
	}
	else
	{
		UsedSkeletalMeshComponent = nullptr;
		GetPrivate()->bIsNetModeDedicatedServer = false;
	}

	if (!bUsedSkipSetSkeletalMeshOnAttach)
	{
		GetPrivate()->bPendingSetReferenceSkeletalMesh = true;
	}
}


USkeletalMeshComponent* UCustomizableObjectInstanceUsage::GetAttachParent() const
{
	if(UsedSkeletalMeshComponent.IsValid())
	{
		return UsedSkeletalMeshComponent.Get();
	}

	return nullptr;
}


USkeletalMesh* UCustomizableObjectInstanceUsagePrivate::GetSkeletalMesh() const
{
	UCustomizableObjectInstance* CustomizableObjectInstance = GetPublic()->GetCustomizableObjectInstance();

	return CustomizableObjectInstance ? CustomizableObjectInstance->GetComponentMeshSkeletalMesh(GetPublic()->GetComponentName()) : nullptr;
}


USkeletalMesh* UCustomizableObjectInstanceUsagePrivate::GetAttachedSkeletalMesh() const
{
	USkeletalMeshComponent* Parent = Cast<USkeletalMeshComponent>(GetPublic()->GetAttachParent());

	if (Parent)
	{
		return Parent->GetSkeletalMeshAsset();
	}

	return nullptr;
}


void UCustomizableObjectInstanceUsage::UpdateSkeletalMeshAsync(bool bNeverSkipUpdate_DEPRECATED, bool bIgnoreCloseDist_DEPRECATED, bool bForceHighPriority)
{
	if (UsedCustomizableObjectInstance)
	{
		UsedCustomizableObjectInstance->UpdateSkeletalMeshAsync(bIgnoreCloseDist_DEPRECATED, bForceHighPriority);
	}
}


void UCustomizableObjectInstanceUsage::UpdateSkeletalMeshAsyncResult(FInstanceUpdateDelegate Callback, bool bIgnoreCloseDist_DEPRECATED, bool bForceHighPriority)
{
	if (UsedCustomizableObjectInstance)
	{
		UsedCustomizableObjectInstance->UpdateSkeletalMeshAsyncResult(Callback, bIgnoreCloseDist_DEPRECATED, bForceHighPriority);
	}
}


UCustomizableObjectInstanceUsagePrivate* UCustomizableObjectInstanceUsage::GetPrivate()
{
	return Private;
}


const UCustomizableObjectInstanceUsagePrivate* UCustomizableObjectInstanceUsage::GetPrivate() const
{
	return Private;
}
