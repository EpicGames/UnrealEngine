// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
CustomizableObjectMeshUpdate.cpp: Helpers to stream in CustomizableObject skeletal mesh LODs.
=============================================================================*/


#include "MuCO/CustomizableObjectMeshUpdate.h"

#include "Components/SkinnedMeshComponent.h"

#include "MutableStreamRequest.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/CustomizableObjectSkeletalMesh.h"
#include "MuCO/UnrealConversionUtils.h"

#include "MuR/Model.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/ProgramCache.h"

#include "Engine/SkeletalMesh.h"
#include "MuR/LOD.h"
#include "Streaming/RenderAssetUpdate.inl"
#include "Rendering/SkeletalMeshRenderData.h"

template class TRenderAssetUpdate<FSkelMeshUpdateContext>;

#define UE_MUTABLE_UPDATE_MESH_REGION		TEXT("Task_Mutable_UpdateMesh")


static bool bEnableCPUMorphTargetStreaming = true;
FAutoConsoleVariableRef CVarMutableEnableCPUMorphTargetStreaming(
	TEXT("mutable.EnableCPUMorphTargetStreaming"),
	bEnableCPUMorphTargetStreaming,
	TEXT("Enable CPU morph targets when mesh streaming is enabled.")
	TEXT("If true, when streaming meshes the CPU morphs targets will be generated when needed.")
);

FCustomizableObjectMeshStreamIn::FCustomizableObjectMeshStreamIn(
	const UCustomizableObjectSkeletalMesh* InMesh,
	EThreadType CreateResourcesThread) :
	FSkeletalMeshStreamIn(InMesh, CreateResourcesThread)
{
	check(InMesh->MeshContext);

	UpdateContext = MakeShared<FMutableMeshUpdateContext>();
	
	UpdateContext->MeshContext = InMesh->MeshContext;
	UpdateContext->CurrentFirstLODIdx = CurrentFirstLODIdx;
	UpdateContext->PendingFirstLODIdx = PendingFirstLODIdx;

	PushTask(FContext(InMesh, TT_None), TT_Async, SRA_UPDATE_CALLBACK(DoInitiate), TT_None, nullptr);
}


void FCustomizableObjectMeshStreamIn::OnUpdateMeshFinished()
{
	if (!IsCancelled())
	{
		check(TaskSynchronization.GetValue() > 0)

		// At this point task synchronization would hold the number of pending requests.
		TaskSynchronization.Decrement();

		// The tick here is intended to schedule the success or cancel callback.
		// Using TT_None ensure gets which could create a dead lock.
		Tick(FSkeletalMeshUpdate::TT_None);
	}
}


void FCustomizableObjectMeshStreamIn::Abort()
{
	if (!IsCancelled() && !IsCompleted())
	{
		FSkeletalMeshStreamIn::Abort();

		// At this point task synchronization might hold the number of pending requests.
		TaskSynchronization.Set(0);

		if (MutableTaskId != FMutableTaskGraph::INVALID_ID && UCustomizableObjectSystem::IsCreated())
		{
			if (UCustomizableObjectSystemPrivate* CustomizableObjectSystem = UCustomizableObjectSystem::GetInstance()->GetPrivate())
			{
				// Cancel task if not launched yet.
				CustomizableObjectSystem->MutableTaskGraph.CancelMutableThreadTaskLowPriority(MutableTaskId);
			}
		}
	}
	else
	{
		FSkeletalMeshStreamIn::Abort();
	}
}


void FCustomizableObjectMeshStreamIn::DoInitiate(const FContext& Context)
{
	check(Context.CurrentThread == TT_Async);

	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectMeshStreamIn::DoInitiate)
	
	// Launch MutableTask
	RequestMeshUpdate(Context);

	PushTask(Context, TT_Async, SRA_UPDATE_CALLBACK(DoConvertResources), TT_Async, SRA_UPDATE_CALLBACK(DoCancel));
}


void FCustomizableObjectMeshStreamIn::DoConvertResources(const FContext& Context)
{
	check(Context.CurrentThread == TT_Async);

	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectMeshStreamIn::DoConvertResources)

	bool bMarkRenderStateDirty = false;
	ConvertMesh(Context, bMarkRenderStateDirty);

	if (bMarkRenderStateDirty)
	{
		PushTask(Context, TT_GameThread, SRA_UPDATE_CALLBACK(MarkRenderStateDirty), (EThreadType)Context.CurrentThread, SRA_UPDATE_CALLBACK(DoCancel));
	}
	else
	{
		PushTask(Context, CreateResourcesThread, SRA_UPDATE_CALLBACK(DoCreateBuffers), static_cast<EThreadType>(Context.CurrentThread), SRA_UPDATE_CALLBACK(DoCancel));
	}
}


void FCustomizableObjectMeshStreamIn::DoCreateBuffers(const FContext& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectMeshStreamIn::DoCreateBuffers)

	CreateBuffers(Context);

	check(!TaskSynchronization.GetValue());

	// We cannot cancel once DoCreateBuffers has started executing, as there's an RHICmdList that must be submitted.
	// Pass the same callback for both task and cancel.
	PushTask(Context, 
		TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate), 
		TT_Render, SRA_UPDATE_CALLBACK(DoFinishUpdate));
}


namespace impl
{
	void Task_Mutable_UpdateMesh_End(const TSharedPtr<FMutableMeshUpdateContext>& UpdateContext, TRefCountPtr<FCustomizableObjectMeshStreamIn>& StreamingTask)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_UpdateMesh_End);
		
		// End update
		const bool bClearRoms = UCustomizableObjectSystem::ShouldClearWorkingMemoryOnUpdateEnd();
		const bool bFreeCache = true;
		UpdateContext->MeshContext->System->EndUpdate(UpdateContext->MeshContext->LiveInstance, bClearRoms, bFreeCache);

		StreamingTask->OnUpdateMeshFinished();
		
		TRACE_END_REGION(UE_MUTABLE_UPDATE_MESH_REGION);
	}

	
	void Task_Mutable_UpdateMesh_Loop(
		const TSharedPtr<FMutableMeshUpdateContext>& UpdateContext,
		TRefCountPtr<FCustomizableObjectMeshStreamIn>& Task,
		int32 LODIndex)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_UpdateMesh_Loop);

		if (Task->IsCancelled() || LODIndex == UpdateContext->CurrentFirstLODIdx + UpdateContext->AssetLODBias)
		{
			Task_Mutable_UpdateMesh_End(UpdateContext, Task);
			return;
		}

		const uint16 ExecutionOptions = UE::Mutable::Private::SkeletalMeshObjectOptionsPack(false, true, LODIndex);
		UE::Tasks::TTask<UE::Mutable::Private::FGetSkeletalMeshResult> GetMeshTask = UpdateContext->MeshContext->System->GetSkeletalMesh({}, UpdateContext->MeshContext->LiveInstance, UpdateContext->MeshContext->SkeletalMeshId, ExecutionOptions);

		UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("Task_MutableGetMeshes_GetSkeletalMesh_Post"), [=]() mutable
			{
				const UE::Mutable::Private::FGetSkeletalMeshResult& Result = GetMeshTask.GetResult();
				check(Result.MutableSkeletalMesh);
				
				UpdateContext->LODs[LODIndex] = Result.MutableSkeletalMesh->LODs[LODIndex];

				Task_Mutable_UpdateMesh_Loop(UpdateContext, Task, LODIndex + 1);
			},
			GetMeshTask,
			LowLevelTasks::ETaskPriority::Inherit));
	}

	
	void Task_Mutable_UpdateMesh(const TSharedPtr<FMutableMeshUpdateContext> UpdateContext, TRefCountPtr<FCustomizableObjectMeshStreamIn>& Task)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_UpdateMesh);

		if (Task->IsCancelled())
		{
			return;
		}

		TRACE_BEGIN_REGION(UE_MUTABLE_UPDATE_MESH_REGION);

		TSharedPtr<UE::Mutable::Private::FSystem> System = UpdateContext->MeshContext->System;
		TSharedRef<UE::Mutable::Private::FLiveInstance> LiveInstance = UpdateContext->MeshContext->LiveInstance;

#if WITH_EDITOR
		const TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model = LiveInstance->Model;
		// Recompiling a CO in the editor will invalidate the previously generated Model. Check that it is valid before accessing the streamed data.
		if (!Model || !Model->IsValid())
		{
			TRACE_END_REGION(UE_MUTABLE_UPDATE_MESH_REGION);
			Task->Abort();
			return;
		}
#endif
		
		// Prevent the usage of the already cached elements by fully clearing the cache.
		if (!CVarRollbackReuseProgramCacheBetweenUpdates.GetValueOnAnyThread())
		{
			LiveInstance->Cache->Clear(UE::Mutable::Private::FProgramCache::EClearFlags::Full);
		}

		UpdateContext->LODs.SetNum(UpdateContext->CurrentFirstLODIdx + UpdateContext->AssetLODBias);

		Task_Mutable_UpdateMesh_Loop(UpdateContext, Task, UpdateContext->PendingFirstLODIdx + UpdateContext->AssetLODBias);
	}
	
} // namespace


void FCustomizableObjectMeshStreamIn::RequestMeshUpdate(const FContext& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectMeshStreamIn::RequestMeshUpdate)
	
	FSoftObjectPath Path(Cast<UCustomizableObjectSkeletalMesh>(Context.Mesh)->CustomizableObjectPathName);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Path.GetAssetName())

	if (IsCancelled())
	{
		return;
	}

	if (!UCustomizableObjectSystem::IsActive())
	{
		Abort();
		return;
	}

	check(UpdateContext.IsValid());

#if WITH_EDITOR
	// Recompiling a CO in the editor will invalidate the previously generated Model. Check that it is valid before accessing the streamed data.
	const TSharedPtr<const UE::Mutable::Private::FModel> Model = UpdateContext->MeshContext->LiveInstance->Model;
	if (!Model || !Model->IsValid())
	{
		Abort();
		return;
	}
#endif

	UpdateContext->AssetLODBias = Context.AssetLODBias;

	UCustomizableObjectSystemPrivate* CustomizableObjectSystem = UCustomizableObjectSystem::GetInstance()->GetPrivate();
	
	TaskSynchronization.Increment();

	if (CVarEnableParallelUpdates.GetValueOnAnyThread())
	{
		CustomizableObjectSystem->MutableTaskGraph.AddMeshStreamingTask(
			TEXT("Mutable_MeshUpdate"),
			[SharedUpdateContext = UpdateContext, RefThis = TRefCountPtr<FCustomizableObjectMeshStreamIn>(this)]() mutable
			{
				impl::Task_Mutable_UpdateMesh(SharedUpdateContext, RefThis);
			});
	}
	else
	{
		MutableTaskId = CustomizableObjectSystem->MutableTaskGraph.AddMutableThreadTaskLowPriority(
			TEXT("Mutable_MeshUpdate"),
			[SharedUpdateContext = UpdateContext, RefThis = TRefCountPtr<FCustomizableObjectMeshStreamIn>(this)]() mutable
			{
				impl::Task_Mutable_UpdateMesh(SharedUpdateContext, RefThis);
			});
	}

	
	if (IsCancelled() && TaskSynchronization.GetValue() > 0)
	{
		TaskSynchronization.Set(0);
	}
}


void FCustomizableObjectMeshStreamIn::ConvertMesh(const FContext& Context, bool& bOutMarkRenderStateDirty)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectMeshStreamIn::ConvertMesh);

	check(!TaskSynchronization.GetValue());

	const UCustomizableObjectSkeletalMesh* Mesh = Cast<UCustomizableObjectSkeletalMesh>(Context.Mesh);
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	if (IsCancelled() || !Mesh || !RenderData)
	{
		return;
	}

	for (int32 LODIndex = PendingFirstLODIdx; LODIndex < CurrentFirstLODIdx; ++LODIndex)
	{
		const int32 LODIndexAbsolute = LODIndex + Context.AssetLODBias;

		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLOD> LOD = UpdateContext->LODs[LODIndexAbsolute];

		if (!LOD || LOD->Meshes.IsEmpty())
		{
			//check(false);
			Abort();
			return;
		}

		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> MutableMesh = LOD->Meshes[0];

		if (!MutableMesh || MutableMesh->GetVertexCount() == 0 || MutableMesh->GetSurfaceCount() == 0 || MutableMesh->GetVertexBuffers().IsDescriptor())
		{
			check(false);
			Abort();
			return;
		}


		const bool bNeedsCPUAccess = Mesh->NeedCPUData(LODIndexAbsolute);
		const bool bGenerateCPUMorphTargetsIfNeeded = bEnableCPUMorphTargetStreaming;
		
		FSkeletalMeshLODRenderData& LODResource = *Context.LODResourcesView[LODIndex];

		TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap;

		TArray<int32, TInlineAllocator<16>> RenderToMutableSectionIndexMapStorage;
		if (LODResource.RenderSections.Num() != MutableMesh->Surfaces.Num())
		{
			const int32 NumRenderSections = LODResource.RenderSections.Num();
			RenderToMutableSectionIndexMapStorage.Init(INDEX_NONE, NumRenderSections);
			
			TArray<uint32>& SurfaceIDs = UpdateContext->MeshContext->SurfaceIDs[LODIndexAbsolute];

			check(SurfaceIDs.Num() == NumRenderSections);
			for (int32 SectionIndex = 0; SectionIndex < NumRenderSections; ++SectionIndex)
			{
				const int32 SurfaceID = SurfaceIDs[SectionIndex];
				
				RenderToMutableSectionIndexMapStorage[SectionIndex] = MutableMesh->Surfaces.IndexOfByPredicate(
				[SurfaceID](const UE::Mutable::Private::FMeshSurface& Surface)
				{
					return Surface.Id == SurfaceID;
				});
			}

			RenderToMutableSectionIndexMap.Emplace(RenderToMutableSectionIndexMapStorage);
		}

		UnrealConversionUtils::CopyMutableVertexBuffers(LODResource, MutableMesh.Get(), bNeedsCPUAccess);
		UnrealConversionUtils::CopyMutableIndexBuffers(LODResource, MutableMesh.Get(), bOutMarkRenderStateDirty, RenderToMutableSectionIndexMap);
		UnrealConversionUtils::CopyMutableSkinWeightProfilesBuffers(LODResource, *const_cast<UCustomizableObjectSkeletalMesh*>(Mesh), LODIndexAbsolute, MutableMesh.Get());
		UnrealConversionUtils::MorphTargetVertexInfoBuffers(LODResource, *Mesh, *MutableMesh.Get(), LODIndexAbsolute, bGenerateCPUMorphTargetsIfNeeded, RenderToMutableSectionIndexMap);
		UnrealConversionUtils::ClothVertexBuffers(LODResource, *MutableMesh.Get(), RenderToMutableSectionIndexMap);
		
		UnrealConversionUtils::UpdateSkeletalMeshLODRenderDataBuffersSize(LODResource);

	}

	// Clear MeshUpdate data
	UpdateContext = nullptr;
}


void FCustomizableObjectMeshStreamIn::MarkRenderStateDirty(const FContext& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectMeshStreamIn::ModifyRenderData);

	check(Context.CurrentThread == TT_GameThread);
	
	const USkeletalMesh* Mesh = Context.Mesh;
	FSkeletalMeshRenderData* RenderData = Context.RenderData;
	
	if (!IsCancelled() && Mesh && RenderData)
	{
		TArray<const IPrimitiveComponent*> Components;
		IStreamingManager::Get().GetRenderAssetStreamingManager().GetAssetComponents(Mesh, Components);

		for (const IPrimitiveComponent* ConstComponent : Components)
		{
			IPrimitiveComponent* Component = const_cast<IPrimitiveComponent*>(ConstComponent);
			Component->MarkRenderStateDirty();
		}
	}
	else
	{
		Abort();
	}
	
	PushTask(Context, CreateResourcesThread, SRA_UPDATE_CALLBACK(DoCreateBuffers), (EThreadType)TT_Async, SRA_UPDATE_CALLBACK(DoCancel));
}
