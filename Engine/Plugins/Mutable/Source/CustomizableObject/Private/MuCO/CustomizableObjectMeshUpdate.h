// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
CustomizableObjectMeshUpdate.h: Helpers to stream in skeletal mesh LODs.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "Streaming/SkeletalMeshUpdate.h"

#include "MuR/System.h"
#include "MuR/Mesh.h"

struct FModelStreamableBulkData;
struct FMutableMeshContext;
class UCustomizableObjectSkeletalMesh;

namespace UE::Mutable::Private 
{ 
	class FParameters;
	class FLOD;
}


/** Runtime data used during a mutable mesh LOD update */
struct FMutableMeshUpdateContext
{
	/** Mesh update context */
	TSharedPtr<FMutableMeshContext> MeshContext;

	// Meshes generated per LOD
	TArray<UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLOD>> LODs;

	/** SkeletalMesh */
	/** The resident first LOD resource index.With domain = [0, ResourceState.NumLODs[.NOT THE ASSET LOD INDEX! */
	int32 CurrentFirstLODIdx = INDEX_NONE;

	/** The requested first LOD resource index. With domain = [0, ResourceState.NumLODs[. NOT THE ASSET LOD INDEX! */
	int32 PendingFirstLODIdx = INDEX_NONE;

	int32 AssetLODBias = 0;
};

extern template class TRenderAssetUpdate<FSkelMeshUpdateContext>;

/** 
* This class provides a framework for loading the LODs of CustomizableObject skeletal meshes.
* Each thread essentially calls Tick() until the job is done.
* The object can be safely deleted when IsCompleted() returns true.
*/
class FCustomizableObjectMeshStreamIn : public FSkeletalMeshStreamIn
{
public:
	FCustomizableObjectMeshStreamIn(
		const UCustomizableObjectSkeletalMesh* InMesh,
		EThreadType CreateResourcesThread);

	void OnUpdateMeshFinished();

	// Decrement TaskSynchronization counter and cancel pending low-priority tasks.
	virtual void Abort() override;

private:

	void DoInitiate(const FContext& Context);

	void DoConvertResources(const FContext& Context);

	void DoCreateBuffers(const FContext& Context);

	/** Creates a MeshUpdate task to generate the meshes for the LODs to stream in. */
	void RequestMeshUpdate(const FContext& Context);

	/** Converts from UE::Mutable::Private::FMesh to FSkeletalMeshLODRenderData. */
	void ConvertMesh(const FContext& Context, bool& bOutMarkRenderStateDirty);

	void MarkRenderStateDirty(const FContext& Context);

	/** Data of the MeshUpdate. */
	TSharedPtr<FMutableMeshUpdateContext> UpdateContext;

	/** MeshUpdate task Id to cancel the task if the stream in task is aborted. */
	uint32 MutableTaskId = 0;
};
