// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectMipDataProvider.h"

#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/MutableStreamRequest.h"
#include "MuCO/UnrealMutableImageProvider.h"
#include "MuR/Model.h"
#include "MuR/Operations.h"
#include "MuR/ProgramCache.h"
#include "TextureResource.h"

#include "Containers/Ticker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectMipDataProvider)


#define UE_MUTABLE_MIPDATA_PROVIDER_UPDATE_IMAGE_REGION		TEXT("Task_Mutable_UpdateImage")

bool MutableTextureUsesOfflineProcessedData();
bool UMutableTextureMipDataProviderFactory::ShouldAllowPlatformTiling(const UTexture* Owner) const
{
	return MutableTextureUsesOfflineProcessedData();
}

UMutableTextureMipDataProviderFactory::UMutableTextureMipDataProviderFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

bool bPrefetchHighQualityMips = true;
static FAutoConsoleVariableRef CVarEnablePrefetchHighQualityMips(
	TEXT("mutable.EnablePrefetchHighQualityMips"),
	bPrefetchHighQualityMips,
	TEXT("If true, prefetch the data of high-quality mips to ensure it is available during the Mip generation task."),
	ECVF_Default);


FMutableTextureMipDataProvider::FMutableTextureMipDataProvider(const UTexture* Texture, UCustomizableObjectInstance* InCustomizableObjectInstance, const FMutableImageReference& InImageRef)
	: FTextureMipDataProvider(Texture, ETickState::Init, ETickThread::Async),
	CustomizableObjectInstance(InCustomizableObjectInstance), ImageRef(InImageRef)
{
	check(ImageRef.ImageID);
}


void FMutableTextureMipDataProvider::PrintWarningAndAdvanceToCleanup()
{
	UE_LOGF(LogMutable, Warning, "Tried to update a mip from a Customizable Object being compiled, cancelling mip update.");
	AdvanceTo(ETickState::CleanUp, ETickThread::Async);
}


void FMutableTextureMipDataProvider::Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
#if WITH_EDITOR
	check(Context.Texture->HasPendingInitOrStreaming());
	check(CustomizableObjectInstance->GetCustomizableObject());
	if (CustomizableObjectInstance->GetCustomizableObject()->GetPrivate()->IsLocked())
	{
		PrintWarningAndAdvanceToCleanup();

		return;
	}
#endif

	AdvanceTo(ETickState::GetMips, ETickThread::Async);
}


namespace Impl
{
	void Task_Mutable_UpdateImage(TSharedPtr<FMutableImageOperationData> OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_UpdateImage)
	
		if (OperationData->bIsCancelled)
		{
			return;
		}
		
		const double StartTime = FPlatformTime::Seconds();
		
		if (FLogBenchmarkUtil::IsBenchmarkingReportingEnabled())
		{
			// Cache memory used when starting the update of the image
			OperationData->ImageUpdateStartBytes = UE::Mutable::Private::FGlobalMemoryCounter::GetAbsoluteCounter();
			UE::Mutable::Private::FGlobalMemoryCounter::Zero();
		}
		
		// Any external texture that may be needed for this update will be requested from Mutable Core's GetImage
		// which will safely access the GlobalExternalImages map, and then just get the cached image or issue a disk read

		// This runs in a worker thread.
		check(OperationData.IsValid());
		check(OperationData->UpdateContext->System);
		check(OperationData->UpdateContext->LiveInstance->Model);
		check(OperationData->UpdateContext->LiveInstance->Parameters);

		if (!OperationData.IsValid())
		{
			return;
		}

		TRACE_BEGIN_REGION(UE_MUTABLE_MIPDATA_PROVIDER_UPDATE_IMAGE_REGION);

		const TSharedRef<UE::Mutable::Private::FLiveInstance> LiveInstance = OperationData->UpdateContext->LiveInstance;
		
		TSharedPtr<UE::Mutable::Private::FSystem> System = OperationData->UpdateContext->System;
		const TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model = LiveInstance->Model;

		auto EndUpdateImage = [](TSharedPtr<FMutableImageOperationData>& OperationData)
		{
			// The request could be canceled in parallel from CancelAsyncTasks and its value be changed
			// between reading it and actually running Decrement() and RescheduleCallback(), so lock
			FScopeLock Lock(&OperationData->CounterTaskLock);
			if (!OperationData->bIsCancelled)
			{
				// Make the FMutableTextureMipDataProvider continue
				OperationData->Counter->Decrement();
				check(OperationData->Counter->GetValue() == 0);

				OperationData->RescheduleCallback();
			}
			
			TRACE_END_REGION(UE_MUTABLE_MIPDATA_PROVIDER_UPDATE_IMAGE_REGION);
		};

#if WITH_EDITOR
		// Recompiling a CO in the editor will invalidate the previously generated Model. Check that it is valid before accessing the streamed data.
		if (!(Model && Model->IsValid()))
		{
			EndUpdateImage(OperationData);
			return;
		}
#endif

		const FMutableImageReference& ImageRef = OperationData->RequestedImage;
		
		UE::Tasks::TTask<UE::Mutable::Private::FGetImageResult> GetImageTask = System->GetImage({}, LiveInstance, ImageRef.ImageID, ImageRef.BaseMip + OperationData->MipsToSkip);

		UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("MipDataProvider_EndUpdateImagesTask"),
				[System, OperationData, StartTime, GetImageTask, EndUpdateImage]() mutable
				{
					check(GetImageTask.IsCompleted());

					const UE::Mutable::Private::FGetImageResult Result = GetImageTask.GetResult();
						
					UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> ResultImage = Result.MutableImage;
					check(ResultImage);

					const int32 FullMipCount = ResultImage->GetMipmapCount(ResultImage->GetSizeX(), ResultImage->GetSizeY());
					const int32 RealMipCount = ResultImage->GetLODCount();

					// Did we fail to generate the entire mipchain (if we have mips at all)?
					if (bool bForceMipChain = (RealMipCount != 1) && (RealMipCount != FullMipCount))
					{
						MUTABLE_CPUPROFILER_SCOPE(GetImage_MipFix);

						UE_LOGF(LogMutable, Warning, "Mutable generated an incomplete mip chain for image.");

						// Force the right number of mips. The missing data will be black.
						UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> NewImage = 
								UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(ResultImage->GetSizeX(), ResultImage->GetSizeY(), FullMipCount, ResultImage->GetFormat(), UE::Mutable::Private::EInitializationType::Black);

						// Formats with BytesPerBlock == 0 will not allocate memory. This type of images are not expected here.
						check(!NewImage->DataStorage.IsEmpty());

						for (int32 L = 0; L < RealMipCount; ++L)
						{
							TArrayView<uint8> DestView = NewImage->DataStorage.GetLOD(L);
							TArrayView<const uint8> SrcView = ResultImage->DataStorage.GetLOD(L);

							check(DestView.Num() == SrcView.Num());
							FMemory::Memcpy(DestView.GetData(), SrcView.GetData(), DestView.Num());
						}

						ResultImage = NewImage;
					}

					OperationData->Result = ResultImage;

					// End update
					{
						MUTABLE_CPUPROFILER_SCOPE(EndUpdate);
						const bool bClearRoms = UCustomizableObjectSystem::ShouldClearWorkingMemoryOnUpdateEnd();
						const bool bFreeCache = true;
						System->EndUpdate(OperationData->UpdateContext->LiveInstance, bClearRoms, bFreeCache);
					}

					if (FLogBenchmarkUtil::IsBenchmarkingReportingEnabled())
					{
						double Time = FPlatformTime::Seconds() - StartTime;
						// Report the peak memory used by the operation
						const int64 PeakMemory = UE::Mutable::Private::FGlobalMemoryCounter::GetPeak();
						// Report the peak memory used during the operation (operation + baseline)
						const int64 RealMemoryPeak = PeakMemory + OperationData->ImageUpdateStartBytes;

						const FString& CustomizableObjectPathName = OperationData->UpdateContext->CustomizableObjectPathName;
						const FString& InstancePathName = OperationData->UpdateContext->InstancePathName;

						const FString& Descriptor = OperationData->UpdateContext->CapturedDescriptor;
						const bool bDidLevelBeginPlay = OperationData->UpdateContext->bLevelBegunPlay;

						ExecuteOnGameThread(UE_SOURCE_LOCATION, 
						[CustomizableObjectPathName, InstancePathName, bDidLevelBeginPlay, Time, PeakMemory, RealMemoryPeak, Descriptor ]
						{
							if (!UCustomizableObjectSystem::IsCreated()) // We are shutting down
							{
								return;	
							}
							
							UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
							if (!System)
							{
								return;
							}

							System->GetPrivate()->LogBenchmarkUtil.FinishUpdateImage(CustomizableObjectPathName, InstancePathName, Descriptor, bDidLevelBeginPlay, Time, PeakMemory, RealMemoryPeak);
						});
					}

					EndUpdateImage(OperationData);
				},
				UE::Tasks::Prerequisites(GetImageTask),
				UE::Tasks::ETaskPriority::Inherit,
				UE::Tasks::EExtendedTaskPriority::Inline)); // MipDataProvider_EndUpdateImagesTaskGetImage.
	}
} // namespace Impl


int32 FMutableTextureMipDataProvider::GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions)
{
	MUTABLE_CPUPROFILER_SCOPE(FMutableTextureMipDataProvider::GetMips);

	FSoftObjectPath Path(UpdateContext->CustomizableObjectPathName);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Path.GetAssetName())

	if (!UCustomizableObjectSystem::IsActive())
	{
		// Mutable is disabled. Skip all mip operations and mark the update task as completed.
		AdvanceTo(ETickState::Done, ETickThread::None);
		return CurrentFirstLODIdx;
	}

#if WITH_EDITOR
	check(Context.Texture->HasPendingInitOrStreaming());
	check(CustomizableObjectInstance->GetCustomizableObject());
	if (CustomizableObjectInstance->GetCustomizableObject()->GetPrivate()->IsLocked())
	{
		PrintWarningAndAdvanceToCleanup();

		return CurrentFirstLODIdx;
	}
#endif

	const UTexture2D* Texture = Cast<UTexture2D>(Context.Texture);
	check(Texture);
	check(!Texture->NeverStream);
	const TIndirectArray<FTexture2DMipMap>& OwnerMips = Texture->GetPlatformMips();

	const int32 NumMips = OwnerMips.Num();
	check(ImageRef.ImageID);

	// Maximum value to skip, will be minimized by the first mip level requested
	int32 MipsToSkip = 256;

	uint32 AllocatedMemorySize = 0;
	uint8* AllocatedMemory = nullptr;

	for (int32 MipIndex = StartingMipIndex; MipIndex < CurrentFirstLODIdx; ++MipIndex)
	{
		check(Context.MipsView.IsValidIndex(MipIndex) && MipInfos.IsValidIndex(MipIndex));

		const FTexture2DMipMap& MipMap = *Context.MipsView[MipIndex];
		FByteBulkData* AuxBulkData = const_cast<FByteBulkData*>(&MipMap.BulkData);

		const FTextureMipInfo& MipInfo = MipInfos[MipIndex];
		void* Dest = MipInfo.DestData;

		if (AuxBulkData->GetBulkDataSize() > 0)
		{
			// Mips are already generated, no need for Mutable progressive mip streaming, just normal CPU->GPU streaming
			AuxBulkData->GetCopy(&Dest, false);
		}
		else // Generate a Mip Request to Mutable
		{
			if (!OperationData.IsValid())
			{
				OperationData = MakeShared<FMutableImageOperationData>();
			}

			const int32 LODBias = static_cast<int32>(Context.MipsView.GetData() - Texture->GetPlatformData()->Mips.GetData());
			check(LODBias >= 0);
			check(NumMips == Context.MipsView.Num() + LODBias);

			const uint64 MipDataSize = MipInfo.DataSize > 0 ? MipInfo.DataSize : GPixelFormats[MipInfo.Format].Get2DTextureMipSizeInBytes(MipInfo.SizeX, MipInfo.SizeY, 0);

			// Allocated memory used to prefetch
			if (MipDataSize > AllocatedMemorySize)
			{
				AllocatedMemorySize = MipDataSize;
				AllocatedMemory = reinterpret_cast<uint8*>(MipInfo.DestData);
			}

			OperationData->Levels.Add({ MipIndex + LODBias, Dest, (int32)MipInfo.SizeX, (int32)MipInfo.SizeY, (int32)MipDataSize, MipInfo.Format });
			MipsToSkip = FMath::Min(MipsToSkip, MipIndex + LODBias);
		}
	}

	if (OperationData.IsValid())
	{
		OperationData->RequestedImage = ImageRef;
		OperationData->UpdateContext = UpdateContext;
		OperationData->MipsToSkip = MipsToSkip;
		OperationData->Counter = SyncOptions.Counter;
		OperationData->RescheduleCallback = SyncOptions.RescheduleCallback;

		// Increment to stop PollMips from running until the Mutable request task finishes. 
		// If a request completes immediately, then it will call the callback
		// but that won't do anything because the tick would not try to acquire the lock since it is already locked.
		check(OperationData->Counter);
		check(OperationData->Counter->GetValue() == 0);
		OperationData->Counter->Increment(); // Prefetch task 
		OperationData->Counter->Increment(); // MipUpdate task

		MUTABLE_CPUPROFILER_SCOPE(ImagePrefetch);
		const TSharedPtr<FModelStreamableBulkData>& ModelStreamableBulkData = OperationData->UpdateContext->ModelStreamableBulkData;
		
		PrefetchRequest = MakeUnique<FMutableStreamRequest>(ModelStreamableBulkData);

		if (bPrefetchHighQualityMips)
		{
			TArray<int32> RomsToPrefetch;

			uint32 MaxBlockSize = 0;

			TSharedPtr<const UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model = UpdateContext->LiveInstance->Model;
			check(Model);

			if (!OperationData->Levels.IsEmpty())
			{
				const int32 LODIndex = OperationData->Levels[0].MipLevel;

				for (int32 ImageToLoadIndex : ImageRef.ConstantImagesNeededToGenerate)
				{
					const int32 RomId = Model->GetProgram().GetConstantImageRomId(ImageToLoadIndex, LODIndex);

					if (RomId >= 0 && Model->GetProgram().IsRomHighRes(RomId))
					{
						FMutableStreamableBlock& Block = ModelStreamableBulkData->ModelStreamables[RomId];
						if (!Block.IsPrefetched)
						{
							Block.IsPrefetched = true;

							MaxBlockSize = FMath::Max(MaxBlockSize, Model->GetProgram().GetRomSize(RomId));
							RomsToPrefetch.Add(RomId);
						}
					}
				}
			}

			if (AllocatedMemorySize < MaxBlockSize)
			{
				OperationData->AllocatedMemory.SetNum(MaxBlockSize);
				AllocatedMemory = OperationData->AllocatedMemory.GetData();
			}

			for (int32 RomId : RomsToPrefetch)
			{
				const FMutableStreamableBlock& Block = ModelStreamableBulkData->ModelStreamables[RomId];
				PrefetchRequest->AddBlock(
					ModelStreamableBulkData->ModelStreamables[RomId],
					UE::Mutable::Private::EStreamableDataType::Model,
					(uint16)UE::Mutable::Private::EDataType::Image,
					TArrayView<uint8>(AllocatedMemory, Model->GetProgram().GetRomSize(RomId)));
			}
		}

		UE::Tasks::FTask StreamTask = PrefetchRequest->Stream();

		UE::Tasks::TTask PrefetchTask = UE::Tasks::Launch(TEXT("MutableImagePrefetchTask"),
			[OperationData = this->OperationData]()
			{
				MUTABLE_CPUPROFILER_SCOPE(ImagePrefetchTask);

				OperationData->AllocatedMemory.Empty();

				FScopeLock Lock(&OperationData->CounterTaskLock);

				// Decrement counter after completing the prefetch task.
				OperationData->Counter->Decrement();

				if (OperationData->bIsCancelled)
				{
					check(OperationData->Counter->GetValue() == 0);
					return;
				}

				UCustomizableObjectSystemPrivate* CustomizableObjectSystem = UCustomizableObjectSystem::GetInstance()->GetPrivate();
				if (CustomizableObjectSystem)
				{
					if (CVarEnableParallelUpdates.GetValueOnAnyThread())
					{
						CustomizableObjectSystem->MutableTaskGraph.AddImageStreamingTask(
							TEXT("Mutable_MipUpdate"),
							[OperationData = OperationData]()
							{
								Impl::Task_Mutable_UpdateImage(OperationData);
							});
					}
					else
					{
						OperationData->MutableTaskId = CustomizableObjectSystem->MutableTaskGraph.AddMutableThreadTaskLowPriority(
							TEXT("Mutable_MipUpdate"),
							[OperationData = OperationData]()
							{
								Impl::Task_Mutable_UpdateImage(OperationData);
							});
					}
				}
			},
			StreamTask,
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);

		AdvanceTo(ETickState::PollMips, ETickThread::Async);
	}
	else
	{
		AdvanceTo(ETickState::CleanUp, ETickThread::Async);
	}

	return CurrentFirstLODIdx;
}


bool FMutableTextureMipDataProvider::PollMips(const FTextureUpdateSyncOptions& SyncOptions)
{
	MUTABLE_CPUPROFILER_SCOPE(FMutableTextureMipDataProvider::PollMips)

	// Once this point is reached, even if the task has not been completed, we know that all the work we need from it has been completed.
	// Furthermore, checking if the task is completed is incorrect since PollMips could have been called by RescheduleCallback (before completing the task).
	
#if WITH_EDITOR
	check(CustomizableObjectInstance->GetCustomizableObject());
	if (CustomizableObjectInstance->GetCustomizableObject()->GetPrivate()->IsLocked())
	{
		PrintWarningAndAdvanceToCleanup();

		return false;
	}
#endif

	if (bRequestAborted)
	{
		OperationData = nullptr;
		AdvanceTo(ETickState::CleanUp, ETickThread::Async);
		return false;
	}
	
	if (OperationData && OperationData->Levels.Num())
	{
		// The counter must be zero meaning the Mutable image operation has finished
		check(SyncOptions.Counter->GetValue() == 0);

		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> Image = OperationData->Result;
		
		int32 ImageLODCount = 0;
		if (Image)
		{
			ImageLODCount = Image->GetLODCount();
			// check(Image->GetLODCount() == OperationData->Levels.Num()); TODO PRP
			
			// No longer true, since missing data may mean we generate smaller images.
			//check(Image->GetSizeX() == OperationData->Levels[0].SizeX);
			//check(Image->GetSizeY() == OperationData->Levels[0].SizeY);
			if (Image->GetSizeX() != OperationData->Levels[0].SizeX
				||
				Image->GetSizeY() != OperationData->Levels[0].SizeY)
			{
				OperationData = nullptr;
				AdvanceTo(ETickState::CleanUp, ETickThread::Async);
				return false;
			}
		}

		int32 MipIndex = 0;
		for (FMutableMipUpdateLevel& Level : OperationData->Levels)
		{
			void* Dest = Level.Dest;

			if (MipIndex < ImageLODCount)
			{
				int32 MipDataSize = Image->GetLODDataSize(MipIndex);

				// Check Mip DataSize for consistency, but skip if 0 because it's optional and might be zero in cooked mips
				bool bCorrectDataSize = (Level.DataSize == 0 || MipDataSize == Level.DataSize);
				if (bCorrectDataSize)
				{
					FMemory::Memcpy(Dest, Image->GetMipData(MipIndex), MipDataSize);
				}
				else
				{
					UE_LOGF(LogMutable, Warning, "Mip data has incorrect size.");
					FMemory::Memzero(Dest, Level.DataSize);
				}
			}
			else
			{
				// Mutable didn't generate all the expected mips
				UE_LOGF(LogMutable, Warning, "Mutable image is missing mips.");
				FMemory::Memzero(Dest, Level.DataSize);
			}
			++MipIndex;
		}

		// Force the immediate release of the image memory to reduce the transient memory usage
		Image = nullptr;
		OperationData->Result = nullptr;
	}

	OperationData = nullptr;
	AdvanceTo(ETickState::Done, ETickThread::None);
	return true;
}


void FMutableTextureMipDataProvider::CleanUp(const FTextureUpdateSyncOptions& SyncOptions)
{
	check(!SyncOptions.Counter || SyncOptions.Counter->GetValue() == 0);
	AdvanceTo(ETickState::Done, ETickThread::None);
}


void FMutableTextureMipDataProvider::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	bRequestAborted = true;

	CancelAsyncTasks();
}


FTextureMipDataProvider::ETickThread FMutableTextureMipDataProvider::GetCancelThread() const
{
	//return OperationData ? FTextureMipDataProvider::ETickThread::Async : FTextureMipDataProvider::ETickThread::None;
	return FTextureMipDataProvider::ETickThread::None;
}


void FMutableTextureMipDataProvider::AbortPollMips()
{
	bRequestAborted = true;

	CancelAsyncTasks();
}


void FMutableTextureMipDataProvider::CancelAsyncTasks()
{
	if (PrefetchRequest)
	{
		PrefetchRequest->Cancel();
	}

	if (OperationData.IsValid())
	{
		// The Counter could be read in parallel from Task_Mutable_UpdateImage, so lock
		FScopeLock Lock(&OperationData->CounterTaskLock);
		if (!OperationData->bIsCancelled)
		{
			OperationData->bIsCancelled = true;

			// Decrement counter. Do not set the value to zero since we must wait for the PrefetchTask to complete
			if (OperationData->Counter->GetValue() > 0)
			{
				OperationData->Counter->Decrement();
			}
		}
		if (UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance())
		{
			System->GetPrivate()->MutableTaskGraph.CancelMutableThreadTaskLowPriority(OperationData->MutableTaskId);
		}
	}
}
