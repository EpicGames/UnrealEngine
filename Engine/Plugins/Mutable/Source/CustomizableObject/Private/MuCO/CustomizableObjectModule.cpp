// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/ICustomizableObjectModulePrivate.h"

#include "MuCO/CustomizableObjectInstanceUsagePrivate.h"
#include "Algo/Find.h"
#include "Engine/Engine.h"
#include "Interfaces/IPluginManager.h"
#include "GameFramework/Pawn.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/ConfigCacheIni.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/CustomizableObjectDGGUI.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/UObjectIterator.h"
#include "GPUSkinPublicDefs.h"
#include "Components/SkeletalMeshComponent.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuR/External/Operation.h"

/**
 * Customizable Object module implementation (private)
 */

class FCustomizableObjectModule : public ICustomizableObjectModulePrivate
{
public:

	// IModuleInterface 
	void StartupModule() override;
	void ShutdownModule() override;

	// ICustomizableObjectModule interface
	FString GetPluginVersion() const override;
	ECustomizableObjectNumBoneInfluences GetNumBoneInfluences() const override;
	void RegisterExtension(const UCustomizableObjectExtension* Extension) override;
	void UnregisterExtension(const UCustomizableObjectExtension* Extension) override;
	TArrayView<const UCustomizableObjectExtension* const> GetRegisteredExtensions() const override;
	TArrayView<const FRegisteredCustomizableObjectPinType> GetExtendedPinTypes() const override;
	TArrayView<const FRegisteredObjectNodeInputPin> GetAdditionalObjectNodePins() const override;
	virtual void RegisterExternalOperation(const UScriptStruct* Operation) override;
	virtual void UnregisterExternalOperation(const UScriptStruct* Operation) override;

	// ICustomizableObjectModulePrivate interface
	virtual TSet<TStrongObjectPtr<const UScriptStruct>> GetRegisteredExternalOperations() const override;

private:
	void RefreshExtensionData();

	static void InitializeSystem();

	// Command to look for Customizable Object Instance in the player pawn of the current world and open a DGGUI to edit its parameters
	IConsoleCommand* LaunchDGGUICommand = nullptr;
	static void ToggleDGGUI(const TArray<FString>& Arguments);

	// Ensure extensions aren't garbage collected
	TArray<TStrongObjectPtr<const UCustomizableObjectExtension>> StrongExtensions;
	// For returning from GetRegisteredExtensions
	TArray<const UCustomizableObjectExtension*> Extensions;

	TArray<FRegisteredCustomizableObjectPinType> ExtendedPinTypes;
	TArray<FRegisteredObjectNodeInputPin> AdditionalObjectNodePins;

	TSet<TStrongObjectPtr<const UScriptStruct>> RegisteredOperations;
};


IMPLEMENT_MODULE( FCustomizableObjectModule, CustomizableObject );

void FCustomizableObjectModule::StartupModule()
{
	LaunchDGGUICommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("mutable.ToggleDGGUI"),
		TEXT("Looks for a Customizable Object Instance within the player pawn and opens a UI to modify its parameters, or closes it if it's open. Specify slot ID to control which component is modified."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&FCustomizableObjectModule::ToggleDGGUI));

	FCoreDelegates::GetOnPostEngineInit().AddStatic(&FCustomizableObjectModule::InitializeSystem);
}


void FCustomizableObjectModule::ShutdownModule()
{
}


FString FCustomizableObjectModule::GetPluginVersion() const
{
	FString Version = "x.x";
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin("Mutable");
	if (Plugin.IsValid() && Plugin->IsEnabled())
	{
		Version = Plugin->GetDescriptor().VersionName;
	}
	return Version;
}


ECustomizableObjectNumBoneInfluences FCustomizableObjectModule::GetNumBoneInfluences() const
{
	bool bAreExtraBoneInfluencesEnabled = false;

#if WITH_EDITOR
	ensure((int32)ECustomizableObjectNumBoneInfluences::Eight == EXTRA_BONE_INFLUENCES);
	ensure((int32)ECustomizableObjectNumBoneInfluences::Twelve == MAX_TOTAL_INFLUENCES);
#endif

	FConfigFile* PluginConfig = GConfig->FindConfigFileWithBaseName("Mutable");
	if (PluginConfig)
	{
		FString Value;

		if (PluginConfig->GetString(TEXT("Features"), TEXT("CustomizableObjectNumBoneInfluences"), Value))
		{
			int32 NumInfluences = Value.IsNumeric() ? FCString::Atoi(*Value) : -1;

			if (NumInfluences == 4 || Value.Equals(FString("Four"), ESearchCase::IgnoreCase))
			{
				return ECustomizableObjectNumBoneInfluences::Four;
			}
			else if (NumInfluences == 8 || Value.Equals(FString("Eight"), ESearchCase::IgnoreCase))
			{
				return ECustomizableObjectNumBoneInfluences::Eight;
			}
			else if (NumInfluences == 12 || Value.Equals(FString("Twelve"), ESearchCase::IgnoreCase))
			{
				return ECustomizableObjectNumBoneInfluences::Twelve;
			}

			UE_LOGF(LogMutable, Warning, "The Mutable Plugin config. variable CustomizableObjectNumBoneInfluences has the invalid value [%ls]."
				"Only 4, 8, 12, Four, Eight, Twelve are valid values.", *Value);
		}

		bool bValue = false;
		if (PluginConfig->GetBool(TEXT("Features"), TEXT("bExtraBoneInfluencesEnabled"), bValue))
		{
			if (bValue)
			{
				return ECustomizableObjectNumBoneInfluences::Eight;
			}
		}
	}

	return ECustomizableObjectNumBoneInfluences::Four;
}

void FCustomizableObjectModule::RegisterExtension(const UCustomizableObjectExtension* Extension)
{
	check(IsInGameThread());
	
	StrongExtensions.Add(TStrongObjectPtr<const UCustomizableObjectExtension>(Extension));
	Extensions.Add(Extension);
	
	RefreshExtensionData();
}

void FCustomizableObjectModule::UnregisterExtension(const UCustomizableObjectExtension* Extension)
{
	check(IsInGameThread());
	
	StrongExtensions.Remove(TStrongObjectPtr<const UCustomizableObjectExtension>(Extension));
	Extensions.Remove(Extension);
	
	RefreshExtensionData();
}

TArrayView<const UCustomizableObjectExtension* const> FCustomizableObjectModule::GetRegisteredExtensions() const
{
	check(IsInGameThread());
	return MakeArrayView(Extensions);
}

TArrayView<const FRegisteredCustomizableObjectPinType> FCustomizableObjectModule::GetExtendedPinTypes() const
{
	check(IsInGameThread());
	return MakeArrayView(ExtendedPinTypes);
}

TArrayView<const FRegisteredObjectNodeInputPin> FCustomizableObjectModule::GetAdditionalObjectNodePins() const
{
	check(IsInGameThread());
	return MakeArrayView(AdditionalObjectNodePins);
}


bool ContainsUObject(const UScriptStruct* ScriptStruct)
{
	if (!ScriptStruct)
	{
		return false;
	}

	for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
	{
		const FProperty* Prop = *It;

		if (Prop->HasAnyPropertyFlags(CPF_TObjectPtr | CPF_UObjectWrapper))
		{
			return true;
		}

		if (const FStructProperty* StructProp = CastField<FStructProperty>(Prop))
		{
			if (ContainsUObject(StructProp->Struct))
			{
				return true;
			}
		}
	}

	return false;
}


void FCustomizableObjectModule::RegisterExternalOperation(const UScriptStruct* Operation)
{
	if (!Operation)
	{
		return;
	}

	if (!Operation->IsChildOf(UE::Mutable::FExternalOperation::StaticStruct()))
	{
		UE_LOGF(LogMutable, Error, "Unable to register operation. Operation that does not inherit from FMutableOperation: %ls", *Operation->GetName());
		return;
	}

	RegisteredOperations.Add(TStrongObjectPtr(Operation));
	
	if (ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
	{
		EditorModule->RegisterExternalOperation(Operation);
	}
}


void FCustomizableObjectModule::UnregisterExternalOperation(const UScriptStruct* Operation)
{
	if (IsEngineExitRequested())
	{
		return;
	}
	
	if (!Operation)
	{
		return;
	}

	RegisteredOperations.Remove(TStrongObjectPtr(Operation));
	
	if (ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
	{
		EditorModule->UnregisterExternalOperation(Operation);
	}
}


TSet<TStrongObjectPtr<const UScriptStruct>> FCustomizableObjectModule::GetRegisteredExternalOperations() const
{
	return RegisteredOperations;
}


void FCustomizableObjectModule::RefreshExtensionData()
{
	ExtendedPinTypes.Reset();
	AdditionalObjectNodePins.Reset();

	for (const UCustomizableObjectExtension* Extension : Extensions)
	{
		for (const FCustomizableObjectPinType& PinType : Extension->GetPinTypes())
		{
			FRegisteredCustomizableObjectPinType& RegisteredPinType = ExtendedPinTypes.AddDefaulted_GetRef();

			RegisteredPinType.Extension = TWeakObjectPtr<const UCustomizableObjectExtension>(Extension);
			RegisteredPinType.PinType = PinType;
		}

		for (const FObjectNodeInputPin& Pin : Extension->GetAdditionalObjectNodePins())
		{
			FRegisteredObjectNodeInputPin RegisteredPin;
			RegisteredPin.Extension = TWeakObjectPtr<const UCustomizableObjectExtension>(Extension);
			// Generate a name that will be unique across extensions, to prevent extensions from
			// unintentionally interfering with each other.
			RegisteredPin.GlobalPinName = FName(Extension->GetPathName() + TEXT("__") + Pin.PinName.ToString());
			RegisteredPin.InputPin = Pin;

			const FRegisteredObjectNodeInputPin* MatchingPin =
				Algo::FindBy(AdditionalObjectNodePins, RegisteredPin.GlobalPinName, &FRegisteredObjectNodeInputPin::GlobalPinName);

			if (MatchingPin)
			{
				// The pin should only be in the list if its extension is valid
				check(MatchingPin->Extension.Get());

				UE_LOGF(LogMutable, Error,
					"Object node pin %ls from extension %ls has the same name as pin %ls from extension %ls. Please rename one of the two.",
					*Pin.PinName.ToString(),
					*Extension->GetPathName(),
					*MatchingPin->InputPin.PinName.ToString(),
					*MatchingPin->Extension.Get()->GetPathName());

				// Don't register the clashing pin
				continue;
			}

			AdditionalObjectNodePins.Add(RegisteredPin);
		}
	}
}


void FCustomizableObjectModule::InitializeSystem()
{
	UCustomizableObjectSystem::GetInstance();
}


UCustomizableObjectInstanceUsage* GetPlayerCustomizableObjectInstanceUsage(const int32 SlotID, const UWorld* CurrentWorld, const int32 PlayerIndex)
{
	// Get customizable skeletal component attached to player pawn
	UCustomizableObjectInstanceUsage* SelectedCustomizableObjectInstanceUsage = nullptr;
	{
		AActor* PlayerPawn = Cast<AActor>(UGameplayStatics::GetPlayerPawn(CurrentWorld, PlayerIndex));
		int32 IndexFound = INDEX_NONE;
		for (TObjectIterator<UCustomizableObjectInstanceUsage> CustomizableObjectInstanceUsage; CustomizableObjectInstanceUsage; ++CustomizableObjectInstanceUsage)
		{
#if WITH_EDITOR
			if (IsValid(*CustomizableObjectInstanceUsage) && CustomizableObjectInstanceUsage->GetPrivate()->bIsNetModeDedicatedServer)
			{
				continue;
			}
#endif

			if (IsValid(*CustomizableObjectInstanceUsage) && !CustomizableObjectInstanceUsage->IsTemplate()
				&& CustomizableObjectInstanceUsage->GetAttachParent())
			{
				AActor* CustomizableActor = CustomizableObjectInstanceUsage->GetAttachParent()->GetAttachmentRootActor();
				if (CustomizableActor && PlayerPawn == CustomizableActor)
				{
					++IndexFound;
					SelectedCustomizableObjectInstanceUsage = *CustomizableObjectInstanceUsage;
					if (IndexFound == SlotID)
					{
						break;
					}
				}
			}
		}
	}


	// If none found, try getting a component without caring about the actor
	if (!SelectedCustomizableObjectInstanceUsage)
	{
		AActor* PlayerPawn = Cast<AActor>(UGameplayStatics::GetPlayerPawn(CurrentWorld, PlayerIndex));
		int32 IndexFound = INDEX_NONE;
		for (TObjectIterator<UCustomizableObjectInstanceUsage> CustomizableObjectInstanceUsage; CustomizableObjectInstanceUsage; ++CustomizableObjectInstanceUsage)
		{
#if WITH_EDITOR
			if (IsValid(*CustomizableObjectInstanceUsage) && CustomizableObjectInstanceUsage->GetPrivate()->bIsNetModeDedicatedServer)
			{
				continue;
			}
#endif

			if (IsValid(*CustomizableObjectInstanceUsage) && !CustomizableObjectInstanceUsage->IsTemplate())
			{
				++IndexFound;
				SelectedCustomizableObjectInstanceUsage = *CustomizableObjectInstanceUsage;
				if (IndexFound == SlotID)
				{
					break;
				}
			}
		}
	}

	return SelectedCustomizableObjectInstanceUsage;
}


void FCustomizableObjectModule::ToggleDGGUI(const TArray<FString>& Arguments)
{
	int32 SlotID = INDEX_NONE;
	if (Arguments.Num() >= 1)
	{
		SlotID = FCString::Atoi(*Arguments[0]);
	}

	const UWorld* CurrentWorld = []() -> const UWorld*
	{
		UWorld* WorldForCurrentCOI = nullptr;
		if (GEngine)
		{
			const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
			for (const FWorldContext& Context : WorldContexts)
			{
				if ((Context.WorldType == EWorldType::Game) && (Context.World() != NULL))
				{
					WorldForCurrentCOI = Context.World();
				}
			}
			// Fall back to GWorld if we don't actually have a world.
			if (WorldForCurrentCOI == nullptr)
			{
				WorldForCurrentCOI = GWorld;
			}
		}
		return WorldForCurrentCOI;
	}();

	const int32 PlayerIndex = 0;
	if (UDGGUI::CloseExistingDGGUI(CurrentWorld))
	{
		return;
	}
	else if (UCustomizableObjectInstanceUsage* SelectedCustomizableObjectInstanceUsage = GetPlayerCustomizableObjectInstanceUsage(SlotID, CurrentWorld, PlayerIndex))
	{
		UDGGUI::OpenDGGUI(SlotID, SelectedCustomizableObjectInstanceUsage, CurrentWorld, PlayerIndex);
	}
}


void PrintParticipatingPackagesDiff(const TArray<FName>& OutOfDatePackages, const TArray<FName>& AddedPackages, const TArray<FName>& RemovedPackages, bool bReleaseVersion)
{
	constexpr int32 MaxLogLines = 10;

	if (bReleaseVersion)
	{
		UE_LOGF(LogMutable, Display, "Release Version changed.");
	}
	
	if (OutOfDatePackages.Num())
	{
		UE_LOGF(LogMutable, Display, "Packages out of date (%i):", OutOfDatePackages.Num());
	}
				
	for (int32 Index = 0; Index < FMath::Min(MaxLogLines, OutOfDatePackages.Num()); ++Index)
	{
		UE_LOGF(LogMutable, Display, "%ls", *OutOfDatePackages[Index].ToString());
	}

	if (AddedPackages.Num())
	{
		UE_LOGF(LogMutable, Display, "Added packages (%i):", AddedPackages.Num());
	}
				
	for (int32 Index = 0; Index < FMath::Min(MaxLogLines, AddedPackages.Num()); ++Index)
	{
		UE_LOGF(LogMutable, Display, "%ls", *AddedPackages[Index].ToString());
	}

	if (RemovedPackages.Num())
	{
		UE_LOGF(LogMutable, Display, "Removed packages (%i):", RemovedPackages.Num());
	}
				
	for (int32 Index = 0; Index < FMath::Min(MaxLogLines, RemovedPackages.Num()); ++Index)
	{
		UE_LOGF(LogMutable, Display, "%ls", *RemovedPackages[Index].ToString());
	}
}

