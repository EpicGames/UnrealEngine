// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"

#include "MuR/Types.h"


uint32 GetTypeHash(const FCustomizableObjectExternalTypeParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	
	Hash = HashCombine(Hash, UE::Mutable::Private::GetTypeHash(Key.ParameterValue));
	
	for (const FInstancedStruct& Value : Key.ParameterRangeValues)
	{
		Hash = HashCombine(Hash, UE::Mutable::Private::GetTypeHash(Key.ParameterValue));
	}
	
	return Hash;
}
