// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectSkeletalMesh.h"

#include "MuCO/CustomizableObjectSystemPrivate.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectMeshUpdate.h"

#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectSkeletalMesh)


void UCustomizableObjectSkeletalMesh::InitMutableStreamingData(const TSharedRef<FUpdateContextPrivate>& InOperationData, const int32 ComponentIndex, const int32 InstanceUpdateFirstLOD, const int32 LODCount)
{
	UCustomizableObject& CustomizableObject = *InOperationData->Object;
	
	// Debug info
	CustomizableObjectPathName = GetNameSafe(&CustomizableObject);

	// Init properties
	check(InOperationData->LiveInstance.IsValid());
	MeshContext = MakeShared<FMutableMeshContext>(InOperationData->LiveInstance.ToSharedRef());

	// This must run in the mutable thread.
	check(UCustomizableObjectSystem::IsCreated());
	MeshContext->System = UCustomizableObjectSystem::GetInstance()->GetPrivate()->MutableSystem;
	MeshContext->ModelStreamableBulkData = CustomizableObject.GetPrivate()->GetModelStreamableBulkData();

	const TSharedRef<FInstanceUpdateData::FComponent>& Component = InOperationData->InstanceUpdateData.Components[ComponentIndex];
	
	MeshContext->SkeletalMeshId = InOperationData->InstanceUpdateData.Components[ComponentIndex]->SkeletalMeshId;
	MeshContext->SurfaceIDs.SetNum(LODCount);

	const FName ComponentName = InOperationData->ComponentNames[Component->Id];

	for (int32 LODIndex = InOperationData->FirstLODAvailable[ComponentName]; LODIndex < LODCount; ++LODIndex)
	{
		const TSharedRef<const FInstanceUpdateData::FLOD>& LOD = InOperationData->InstanceUpdateData.LODs[InstanceUpdateFirstLOD + LODIndex];

		MeshContext->SurfaceIDs[LODIndex].SetNum(LOD->SurfaceCount);
		check(LOD->Mesh && LOD->Mesh->GetSurfaceCount() == LOD->SurfaceCount);

		for (int32 SurfaceIndex = 0; SurfaceIndex < LOD->SurfaceCount; ++SurfaceIndex)
		{
			MeshContext->SurfaceIDs[LODIndex][SurfaceIndex] = LOD->Mesh->GetSurfaceId(SurfaceIndex);
		}
	}
}


bool UCustomizableObjectSkeletalMesh::StreamIn(int32 NewMipCount, bool bHighPrio)
{
	check(IsInGameThread());

	FSkeletalMeshRenderData* RenderData = GetResourceForRendering();
	if (!RenderData || !RenderData->IsInitialized())
	{
		return false;
	}

	if (!HasPendingInitOrStreaming() && CachedSRRState.StreamIn(NewMipCount))
	{
		FRenderAssetUpdate::EThreadType CreateResourcesThread = GRHISupportsAsyncTextureCreation
			? FRenderAssetUpdate::TT_Async
			: FRenderAssetUpdate::TT_Render;

		PendingUpdate = new FCustomizableObjectMeshStreamIn(this, CreateResourcesThread);

		return !PendingUpdate->IsCancelled();
	}
	return false;
}
