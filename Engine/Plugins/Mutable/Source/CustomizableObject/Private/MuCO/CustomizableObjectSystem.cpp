// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectSystem.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAsset.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectInstanceUsagePrivate.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCO/LogBenchmarkUtil.h"
#include "MuCO/UnrealMutableImageProvider.h"
#include "MuCO/UnrealMutableModelDiskStreamer.h"
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuR/Model.h"
#include "MuR/Settings.h"
#include "MuR/Material.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "ContentStreaming.h"
#include "SkinnedAssetCompiler.h"
#include "Components/SkeletalMeshComponent.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuR/Parameters.h"
#include "MuR/System.h"
#include "MuR/PassthroughObject.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "MuR/LOD.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Logging/MessageLog.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/World.h"
#include "LevelEditorViewport.h"
#else
#include "Engine/Engine.h"
#endif

#include "MuCO/CustomizableObjectSkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectSystem)

class AActor;
class UAnimInstance;


DECLARE_CYCLE_STAT(TEXT("MutablePendingRelease Time"), STAT_MutablePendingRelease, STATGROUP_Game);
DECLARE_CYCLE_STAT(TEXT("MutableTask"), STAT_MutableTask, STATGROUP_Game);

#define UE_MUTABLE_UPDATE_REGION TEXT("Mutable Update")


UCustomizableObjectSystem* UCustomizableObjectSystemPrivate::SSystem = nullptr;

bool bIsMutableEnabled = true;

static FAutoConsoleVariableRef CVarMutableEnabled(
	TEXT("Mutable.Enabled"),
	bIsMutableEnabled,
	TEXT("true/false - Disabling Mutable will turn off CO compilation, mesh generation, and texture streaming and will remove the system ticker. "),
	FConsoleVariableDelegate::CreateStatic(&UCustomizableObjectSystemPrivate::OnMutableEnabledChanged));

int32 WorkingMemoryKB =
#if !PLATFORM_DESKTOP
(10 * 1024);
#else
(50 * 1024);
#endif

static FAutoConsoleVariableRef CVarWorkingMemoryKB(
	TEXT("mutable.WorkingMemory"),
	WorkingMemoryKB,
	TEXT("Limit the amount of memory (in KB) to use as working memory when building characters. More memory reduces the object construction time. 0 means no restriction. Defaults: Desktop = 50,000 KB, Others = 10,000 KB"),
	ECVF_Scalability);

TAutoConsoleVariable<bool> CVarClearWorkingMemoryOnUpdateEnd(
	TEXT("mutable.ClearWorkingMemoryOnUpdateEnd"),
	true,
	TEXT("Clear the working memory and cache after every Mutable operation."),
	ECVF_Scalability);

TAutoConsoleVariable<bool> CVarReuseImagesBetweenInstances(
	TEXT("mutable.ReuseImagesBetweenInstances"),
	true,
	TEXT("Enables or disables the reuse of images between instances."),
	ECVF_Scalability);

TAutoConsoleVariable<bool> CVarEnableMeshCache(
	TEXT("mutable.EnableMeshCache"),
	true,
	TEXT("Enables or disables the reuse of meshes."),
	ECVF_Scalability);

TAutoConsoleVariable<bool> CVarEnableUpdateOptimization(
	TEXT("mutable.EnableUpdateOptimization"),
	true,
	TEXT("Enable or disable update optimization when no changes are made to the parent component."));

TAutoConsoleVariable<bool> CVarEnableRealTimeMorphTargets(
	TEXT("mutable.EnableRealTimeMorphTargets"),
	true,
	TEXT("Enable or disable generation of realtime morph targets."));

TAutoConsoleVariable<bool> CVarIgnoreFirstAvailableLODCalculation(
	TEXT("mutable.IgnoreFirstAvalilableLODCalculation"),
	false,
	TEXT("If set to true, ignores the first available LOD calculation to set the generated tetxure size."));

TAutoConsoleVariable<bool> CVarRequiresReinitCompareBoneNames(
	TEXT("mutable.RequiresReinitCompareBoneNames"),
	true,
	TEXT("If set to true, instead of comparing the indices of the RequiredBones the names will get compared when checking if the bone Pose should re reinitialized or not."));

TAutoConsoleVariable<bool> CVarEnableCaches(
	TEXT("mutable.EnableCaches"),
	true,
	TEXT("Enable or disable all caches."),
	ECVF_Scalability);

#if WITH_EDITOR
bool bEnableLODManagmentInEditor = false;

static FAutoConsoleVariableRef CVarMutableEnableLODManagmentInEditor(
	TEXT("Mutable.EnableLODManagmentInEditor"),
	bEnableLODManagmentInEditor,
	TEXT("true/false - If true, enables custom LODManagment in the editor. "),
	ECVF_Default);

TAutoConsoleVariable<bool> CVarMutableLogObjectMemoryOnUpdate(
	TEXT("mutable.LogObjectMemoryOnUpdate"),
	false,
	TEXT("Log the memory used for a CO on every update."),
	ECVF_Scalability);
#endif

TAutoConsoleVariable<bool> CVarFixLowPriorityTasksOverlap(
	TEXT("mutable.rollback.FixLowPriorityTasksOverlap"),
	true,
	TEXT("If true, use code that fixes the Low Priority Tasks overlap."));

TAutoConsoleVariable<bool> CVarMutableHighPriorityLoading(
	TEXT("Mutable.EnableLoadingAssetsWithHighPriority"),
	true,
	TEXT("If enabled, the request to load additional assets will have high priority."));

TAutoConsoleVariable<bool> CVarRollbackReuseProgramCacheBetweenUpdates (
	TEXT("Mutable.Rollback.ReuseProgramCacheBetweenUpdates"),
	true,
	TEXT("If set to true, the program cache will be kept untouched between calls to the BeginUpdate_mutableThread. If set to false, then the LiveInstance program cache will be fully cleared between calls to the mentioned method."),	
	ECVF_Default
);

/** How often update the on screen warnings (seconds). */
constexpr float OnScreenWarningsTickerTime = 5.0f;

/** Duration of the on screen warning messages (seconds). */
constexpr float WarningDisplayTime = OnScreenWarningsTickerTime * 2.0f;


int64 GetOnScreenMessageKey(const TWeakObjectPtr<const UCustomizableObject>& Object, TMap<TWeakObjectPtr<const UCustomizableObject>, uint64>& KeyMap)
{
	uint64 Key;
	if (uint64* Result = KeyMap.Find(Object))
	{
		Key = *Result;
	}
	else
	{
		Key = 0;
		while (GEngine->OnScreenDebugMessageExists(Key))
		{
			++Key;
		}
		
		KeyMap.Add(Object, Key);
	}

	return Key;
}


void RemoveUnusedOnScreenMessages(TMap<TWeakObjectPtr<const UCustomizableObject>, uint64>& KeyMap)
{
	for (TMap<TWeakObjectPtr<const UCustomizableObject>, uint64>::TIterator It = KeyMap.CreateIterator(); It; ++It)
	{
		if (!It->Key.IsValid())
		{
			GEngine->RemoveOnScreenDebugMessage(It->Value);
			It.RemoveCurrent();
		}
	}
}


bool CanUpdate(UCustomizableObjectInstance* Instance)
{
	if (!IsValid(Instance))
	{
		return false;
	}
	
	UCustomizableObject* Object = Instance->GetCustomizableObject();
	if (!IsValid(Object))
	{
		return false;
	}

	if (!Object->IsCompiled())
	{
		return false;
	}
	
#if WITH_EDITOR
	if (Object->GetPrivate()->IsLocked())
	{
		return false;
	}
#endif
	
	return true;
}


void FUpdateContextPrivate::Init(bool bUseCommitedDescriptor)
{
	check(IsInGameThread());
	
	UCustomizableObjectInstance* StrongInstance = Instance.Get();
	check(IsValid(StrongInstance));
	
	UCustomizableObject* StrongObject = StrongInstance->GetCustomizableObject();
	check(IsValid(StrongObject));

	Object = StrongObject;

	const FDescriptorHash& CommittedHash = StrongInstance->GetPrivate()->CommittedDescriptorHash;
	if (bUseCommitedDescriptor && CommittedHash != FDescriptorHash())
	{
		CapturedDescriptor = StrongInstance->GetPrivate()->CommittedDescriptor;
		CapturedDescriptorHash = CommittedHash;
	}
	else
	{
		CapturedDescriptor = StrongInstance->GetPrivate()->GetDescriptor();
		StrongObject->GetPrivate()->ApplyStateForcedValuesToParameters(CapturedDescriptor);
		CapturedDescriptorHash = FDescriptorHash(CapturedDescriptor);
	}

	if (const UModelResources* ModelResources = StrongObject->GetPrivate()->GetModelResources())
	{
		ComponentNames = ModelResources->ComponentNamesPerObjectComponent;

	}
	
	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();
	UCustomizableObjectSystemPrivate* SystemPrivate = System->GetPrivate();

	MutableSystem = SystemPrivate->MutableSystem;	
	check(MutableSystem);
	
	bKeepOwnershipOfGeneratedResources = StrongInstance->GetPrivate()->bKeepOwnershipOfGeneratedResources;
}


#if WITH_EDITOR
struct FOutOfDateWarningContext
{
	TArray<TWeakObjectPtr<const UCustomizableObject>> Objects;

	int32 IndexObject = 0;

	double StartTime = 0.0f;
};


/** If true, the warning is being executed asynchronously. */
bool bOutOfDateAsync  = false;


/** Async because work is split in between ticks. */
void OutOfDateWarning_Async(const TSharedRef<FOutOfDateWarningContext>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(OutOfDateWarning_Async)

	check(IsInGameThread());
	
	static TMap<TWeakObjectPtr<const UCustomizableObject>, uint64> KeysOutOfDate;

	constexpr float MaxTime = 1.0f / 1000.0f * 2.0f; // 2ms

	if (FPlatformTime::Seconds() - Context->StartTime >= MaxTime)
	{
		// Time limit reached. Reschedule itself.
		if (GEditor)
		{
			GEditor->GetTimerManager()->SetTimerForNextTick([=]()
			{
				Context->StartTime = FPlatformTime::Seconds();
				OutOfDateWarning_Async(Context);
			});
		}
		
		return;
	}

	const ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get();	
	if (!Module)
	{
		bOutOfDateAsync = false; // End async task.
		return;	
	}
	
	// Find the next Customizable Object still alive.
	const UCustomizableObject* Object = nullptr;
	while (Context->IndexObject < Context->Objects.Num())
	{
		Object = Context->Objects[Context->IndexObject].Get();

		if (Object)
		{
			break;
		}
		
		++Context->IndexObject;
	}
	
	// If all Customizable Objects processed, end async task.
	if (Context->IndexObject == Context->Objects.Num())
	{
		RemoveUnusedOnScreenMessages(KeysOutOfDate); // Clean old message keys.

		bOutOfDateAsync = false; // End async task.
		return;
	}
	
	ICustomizableObjectEditorModule::IsCompilationOutOfDateCallback Callback = [Context](bool bOutOfDate, bool bVersionDiff, const TArray<FName>& OutOfDatePackages,
		const TArray<FName>& AddedPackages, const TArray<FName>& RemovedPackages)
	{
		check(IsInGameThread());

		const TWeakObjectPtr<const UCustomizableObject>& WeakObject = Context->Objects[Context->IndexObject];

		if (const UCustomizableObject* Object = WeakObject.Get())
		{
			if (bOutOfDate)
			{
				const uint64 Key = GetOnScreenMessageKey(WeakObject, KeysOutOfDate);
			
				if (!GEngine->OnScreenDebugMessageExists(Key))
				{
					UE_LOGF(LogMutable, Display, "Customizable Object [%ls] compilation out of date. Changes since last compilation:", *Object->GetName());
					PrintParticipatingPackagesDiff(OutOfDatePackages, AddedPackages, RemovedPackages, bVersionDiff);
				}
			
				FString Msg = FString::Printf(TEXT("Customizable Object [%s] compilation out of date. See the Output Log for more information."), *Object->GetName());
				GEngine->AddOnScreenDebugMessage(Key, WarningDisplayTime, FColor::Yellow, Msg);
			}
			else if (uint64* Key = KeysOutOfDate.Find(WeakObject))
			{
				GEngine->RemoveOnScreenDebugMessage(*Key);
			}
		}

		// Process the next Customizable Object.
		++Context->IndexObject;
		OutOfDateWarning_Async(Context);
	};

	Module->IsCompilationOutOfDate(*Object, true, MaxTime, Callback);
}
#endif


bool TickWarnings(float DeltaTime)
{
	MUTABLE_CPUPROFILER_SCOPE(TickWarnings);

	const double StartTime = FPlatformTime::Seconds();
	
	static TMap<TWeakObjectPtr<const UCustomizableObject>, uint64> KeysNotCompiled;
	static TMap<TWeakObjectPtr<const UCustomizableObject>, uint64> KeysNotOptimized;
	
	TSet<const UCustomizableObject*> Objects;
	
	for (UCustomizableObjectInstance* Instance : UCustomizableObjectSystemPrivate::GetCustomizableObjectInstances())
	{
		if (!IsValid(Instance))
		{
			continue;
		}

		if (Instance->GetPrivate()->InstanceUsages.IsEmpty())
		{
			continue;
		}

		const UCustomizableObject* Object = Cast<UCustomizableObject>(Instance->GetCustomizableObject());
		if (!Object)
		{
			continue;
		}

		if (Object->GetPrivate()->Status.Get() != FCustomizableObjectStatus::EState::ModelLoaded)
		{
			continue;
		}

		Objects.Add(Object);
	}

	// Not compiled warning.
	{
		for (const UCustomizableObject* Object : Objects)
		{
			TWeakObjectPtr<const UCustomizableObject> WeakObject = TWeakObjectPtr(Object);
		
			if (!Object->IsLoading() && !Object->IsCompiled())
			{
				const uint64 Key = GetOnScreenMessageKey(WeakObject, KeysNotCompiled);
				FString Msg = FString::Printf(TEXT("Customizable Object [%s] not compiled."), *Object->GetName());
				GEngine->AddOnScreenDebugMessage(Key, WarningDisplayTime, FColor::Red, Msg);
			}
			else if (uint64* Key = KeysNotCompiled.Find(TWeakObjectPtr(Object)))
			{
				GEngine->RemoveOnScreenDebugMessage(*Key);
			}
		}
		
		RemoveUnusedOnScreenMessages(KeysNotCompiled);
	}

#if WITH_EDITOR
	// Compiled without optimizations warning.
	for (const UCustomizableObject* Object : Objects)
	{
		TWeakObjectPtr<const UCustomizableObject> WeakObject = TWeakObjectPtr(Object);
		
		if (!Object->GetPrivate()->GetModelResourcesChecked().bIsCompiledWithOptimization)
		{
			const uint64 Key = GetOnScreenMessageKey(WeakObject, KeysNotOptimized);
			FString Msg = FString::Printf(TEXT("Customizable Object [%s] was compiled without optimization."), *Object->GetName());
			GEngine->AddOnScreenDebugMessage(Key, WarningDisplayTime, FColor::Yellow, Msg);
		}
		else if (uint64* Key = KeysNotOptimized.Find(TWeakObjectPtr(Object)))
		{
			GEngine->RemoveOnScreenDebugMessage(*Key);
		}
	}

	RemoveUnusedOnScreenMessages(KeysNotOptimized);

	// Is compilation out of date warning.
	if (!bOutOfDateAsync)
	{
		bOutOfDateAsync = true;

		TSharedRef<FOutOfDateWarningContext> Context = MakeShareable(new FOutOfDateWarningContext);
		Context->StartTime = StartTime;
		
		for (const UCustomizableObject* Object : Objects)
		{
			Context->Objects.Add(Object);
		}
		
		OutOfDateWarning_Async(Context);
	}
#endif
	
	return true;
}


const FCustomizableObjectInstanceDescriptor& FUpdateContextPrivate::GetCapturedDescriptor() const
{
	return CapturedDescriptor;
}


const FDescriptorHash& FUpdateContextPrivate::GetCapturedDescriptorHash() const
{
	return CapturedDescriptorHash;
}


const FCustomizableObjectInstanceDescriptor&& FUpdateContextPrivate::MoveCommittedDescriptor()
{
	return MoveTemp(CapturedDescriptor);
}


UCustomizableObjectSystem* UCustomizableObjectSystem::GetInstance()
{
	if (!UCustomizableObjectSystemPrivate::SSystem)
	{
		UE_LOGF(LogMutable, Log, "Creating Mutable Customizable Object System.");

		check(IsInGameThread());

		UCustomizableObjectSystemPrivate::SSystem = NewObject<UCustomizableObjectSystem>(UCustomizableObjectSystem::StaticClass());
		check(UCustomizableObjectSystemPrivate::SSystem != nullptr);
		checkf(!GUObjectArray.IsDisregardForGC(UCustomizableObjectSystemPrivate::SSystem), TEXT("Mutable was initialized too early in the UE init process, for instance, in the constructor of a default UObject."));
		UCustomizableObjectSystemPrivate::SSystem->AddToRoot();
		checkf(!GUObjectArray.IsDisregardForGC(UCustomizableObjectSystemPrivate::SSystem), TEXT("Mutable was initialized too early in the UE init process, for instance, in the constructor of a default UObject."));
		UCustomizableObjectSystemPrivate::SSystem->InitSystem();
	}

	return UCustomizableObjectSystemPrivate::SSystem;
}


UCustomizableObjectSystem* UCustomizableObjectSystem::GetInstanceChecked()
{
	UCustomizableObjectSystem* System = GetInstance();
	check(System);
	
	return System;
}


bool UCustomizableObjectSystem::IsUpdateResultValid(const EUpdateResult UpdateResult)
{
	return UpdateResult == EUpdateResult::Success || UpdateResult == EUpdateResult::Warning;
}


FString UCustomizableObjectSystem::GetPluginVersion() const
{
	// Bridge the call from the module. This implementation is available from blueprint.
	return ICustomizableObjectModule::Get().GetPluginVersion();
}


UCustomizableObjectSystemPrivate* UCustomizableObjectSystem::GetPrivate()
{
	check(Private);
	return Private;
}


const UCustomizableObjectSystemPrivate* UCustomizableObjectSystem::GetPrivate() const
{
	check(Private);
	return Private;
}


bool UCustomizableObjectSystem::IsCreated()
{
	return UCustomizableObjectSystemPrivate::SSystem != 0;
}

bool UCustomizableObjectSystem::IsActive()
{
	return IsCreated() && bIsMutableEnabled;
}


void UCustomizableObjectSystem::InitSystem()
{
	// Everything initialized in Init() instead of constructor to prevent the default UCustomizableObjectSystem from registering a tick function
	Private = NewObject<UCustomizableObjectSystemPrivate>(this, FName("Private"));
	check(Private != nullptr);

	Private->CurrentMutableOperation = nullptr;

	Private->LastWorkingMemoryBytes = CVarWorkingMemoryKB->GetInt() * 1024;

	UE::Mutable::Private::FSettings Settings;
	Settings.SetProfile(false);
	Settings.SetWorkingMemoryBytes(Private->LastWorkingMemoryBytes);
	Private->MutableSystem = MakeShared<UE::Mutable::Private::FSystem>(Settings);
	check(Private->MutableSystem);

	Private->Streamer = MakeShared<FUnrealMutableModelBulkReader>();
	check(Private->Streamer != nullptr);
	Private->MutableSystem->SetStreamingInterface(Private->Streamer);
	Private->OnMutableEnabledChanged();
}


void UCustomizableObjectSystem::BeginDestroy()
{
	// It could be null, for the default object.
	if (Private)
	{
#if WITH_EDITOR
		if (ICustomizableObjectEditorModule* EditorModule = FModuleManager::GetModulePtr<ICustomizableObjectEditorModule>("CustomizableObjectEditor"))
		{
			EditorModule->CancelCompileRequests();
		}
#endif

#if !UE_SERVER
		FStreamingManagerCollection::Get().RemoveStreamingManager(GetPrivate());
		
		FTSTicker::RemoveTicker(Private->TickWarningsDelegateHandle);
#endif // !UE_SERVER
		
		// Complete pending task graph tasks
		Private->MutableTaskGraph.AllowLaunchingMutableTaskLowPriority(false, false);
		check(Private->Streamer);
		Private->MutableTaskGraph.AddMutableThreadTask(TEXT("EndStream"), [Streamer = Private->Streamer]()
			{
				Streamer->EndStreaming();
			});
		Private->MutableTaskGraph.WaitForMutableTasks();

		// Clear the ongoing operation
		Private->CurrentMutableOperation = nullptr;

		UCustomizableObjectSystemPrivate::SSystem = nullptr;

		Private = nullptr;
	}

	Super::BeginDestroy();
}


FString UCustomizableObjectSystem::GetDesc()
{
	return TEXT("Customizable Object System Singleton");
}


UCustomizableObjectSystem* UCustomizableObjectSystemPrivate::GetPublic() const
{
	UCustomizableObjectSystem* Public = StaticCast<UCustomizableObjectSystem*>(GetOuter());
	check(Public);

	return Public;
}


bool bForceStreamMeshLODs = false;

static FAutoConsoleVariableRef CVarMutableForceStreamMeshLODs(
	TEXT("Mutable.ForceStreamMeshLODs"),
	bForceStreamMeshLODs,
	TEXT("Experimental - true/false - If true, and bStreamMeshLODs is enabled, all COs will stream mesh LODs. "),
	ECVF_Default);

bool bStreamMeshLODs = true;

static FAutoConsoleVariableRef CVarMutableStreamMeshLODsEnabled(
	TEXT("Mutable.StreamMeshLODsEnabled"),
	bStreamMeshLODs,
	TEXT("Experimental - true/false - If true, enable generated meshes to stream mesh LODs. "),
	ECVF_Default);

int32 UCustomizableObjectSystemPrivate::EnableMutableProgressiveMipStreaming = 1;

// Warning! If this is enabled, do not get references to the textures generated by Mutable! They are owned by Mutable and could become invalid at any moment
static FAutoConsoleVariableRef CVarEnableMutableProgressiveMipStreaming(
	TEXT("mutable.EnableMutableProgressiveMipStreaming"), UCustomizableObjectSystemPrivate::EnableMutableProgressiveMipStreaming,
	TEXT("If set to 1 or greater use progressive Mutable Mip streaming for Mutable textures. If disabled, all mips will always be generated and spending memory. In that case, on Desktop platforms they will be stored in CPU memory, on other platforms textures will be non-streaming."),
	ECVF_Default);


int32 UCustomizableObjectSystemPrivate::EnableMutableLiveUpdate = 1;

static FAutoConsoleVariableRef CVarEnableMutableLiveUpdate(
	TEXT("mutable.EnableMutableLiveUpdate"), UCustomizableObjectSystemPrivate::EnableMutableLiveUpdate,
	TEXT("If set to 1 or greater Mutable can use the live update mode if set in the current Mutable state. If disabled, it will never use live update mode even if set in the current Mutable state."),
	ECVF_Default);


static TAutoConsoleVariable<bool> CVarEnableMutableReuseInstanceTextures(
	TEXT("mutable.EnableReuseInstanceTextures"), 
	true,
	TEXT("If set to true set in the corresponding setting in the current Mutable state, Mutable can reuse instance UTextures (only uncompressed and not streaming, so set the options in the state) and their resources between updates when they are modified. If geometry or state is changed they cannot be reused."),
	ECVF_Default);

bool UCustomizableObjectSystemPrivate::bGenerateInstancesWithinRange = false;

static FAutoConsoleVariableRef CVarGenerateInstancesWithinRange(
	TEXT("mutable.GenerateInstancesWithinRange"),
	UCustomizableObjectSystemPrivate::bGenerateInstancesWithinRange,
	TEXT("If true, only instances within a predefined distance to the view centers will be generated"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarMaxNumInstancesToDiscardPerTick(
	TEXT("mutable.MaxNumInstancesToDiscardPerTick"),
	1,
	TEXT("The maximum number of stale instances that will be discarded per tick by Mutable."),
	ECVF_Scalability);

int32 UCustomizableObjectSystemPrivate::MaxTextureSizeToGenerate = 0;

FAutoConsoleVariableRef CVarMaxTextureSizeToGenerate(
	TEXT("Mutable.MaxTextureSizeToGenerate"),
	UCustomizableObjectSystemPrivate::MaxTextureSizeToGenerate,
	TEXT("Max texture size on Mutable textures. Mip 0 will be the first mip with max size equal or less than MaxTextureSizeToGenerate."
		"If a texture doesn't have small enough mips, mip 0 will be the last mip available."));

static FAutoConsoleVariable CVarDescriptorDebugPrint(
	TEXT("mutable.DescriptorDebugPrint"),
	false,
	TEXT("If true, each time an update is enqueued, print its captured parameters."),
	ECVF_Default);


namespace impl
{
	void Task_Mutable_EndUpdate(TSharedPtr<UE::Mutable::Private::FLiveInstance> LiveInstance, TSharedPtr<UE::Mutable::Private::FSystem> MutableSystem);
}


void FinishUpdateGlobal(const TSharedRef<FUpdateContextPrivate>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(FinishUpdateGlobal);
	check(IsInGameThread())

	UCustomizableObjectInstance* Instance = Context->Instance.Get();
	UCustomizableObject* Object = IsValid(Instance) ? Instance->GetCustomizableObject() : nullptr;

	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
	UCustomizableObjectSystemPrivate* SystemPrivate = System ? System->GetPrivate() : nullptr;

	if (System &&
		Context->UpdateStarted)
	{
		SystemPrivate->CurrentMutableOperation = nullptr;
		
		SystemPrivate->LastUpdateMutableTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
			TEXT("Task_Mutable_EndUpdate"),
			[LiveInstance = Context->LiveInstance, MutableSystem = SystemPrivate->MutableSystem]()
			{
				impl::Task_Mutable_EndUpdate(LiveInstance, MutableSystem);
			});
	}
	
	if (IsValid(Instance))
	{
		UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();

		switch (Context->UpdateResult)
		{
		case EUpdateResult::Success:
		case EUpdateResult::Warning:
			InstancePrivate->SkeletalMeshStatus = ESkeletalMeshStatus::Success;
			
			InstancePrivate->CommittedDescriptor = Context->MoveCommittedDescriptor();
			InstancePrivate->CommittedDescriptorHash = Context->GetCapturedDescriptorHash();
			
			// Delegates must be called only after updating the Instance flags.
			Instance->UpdatedDelegate.Broadcast(Instance);
			Instance->UpdatedNativeDelegate.Broadcast(Instance);

			UCustomizableObjectSystemPrivate::RegisterInstanceToGeneratedList(*Instance);
			break;

		case EUpdateResult::ErrorOptimized:
			break; // Skeletal Mesh not changed.
			
		case EUpdateResult::ErrorDiscarded:
			break; // Status will be updated once the discard is performed.

		case EUpdateResult::ErrorUncompiled:
			break; // Object not compiled.

		case EUpdateResult::Error: 
		case EUpdateResult::Error16BitBoneIndex:
			{
				InstancePrivate->InvalidateGeneratedData();
				InstancePrivate->SkeletalMeshStatus = ESkeletalMeshStatus::Error;
			}
			break;
			
		case EUpdateResult::ErrorReplaced:
			break; // Skeletal Mesh not changed.
			
		default:
			unimplemented();
		}


		if (UCustomizableObjectSystem::IsUpdateResultValid(Context->UpdateResult))
		{
			MUTABLE_CPUPROFILER_SCOPE(FinishUpdateGlobal_Callbacks);
			TArrayView<UCustomizableObjectInstanceUsage*> InstanceUsagesView(InstancePrivate->InstanceUsages);

			if (Context->bOptimizedUpdate)
			{
				InstanceUsagesView = TArrayView<UCustomizableObjectInstanceUsage*>(Context->AttachedParentUpdated);
			}

			for (UCustomizableObjectInstanceUsage* InstanceUsage : InstanceUsagesView)
			{
				if (!IsValid(InstanceUsage))
				{
					continue;
				}
				
#if WITH_EDITOR
				if (InstanceUsage->GetPrivate()->bIsNetModeDedicatedServer)
				{
					continue;
				}
#endif
				
				InstanceUsage->GetPrivate()->Callbacks();
			}
		}
	}

	FUpdateContext ContextPublic;
	ContextPublic.UpdateResult = Context->UpdateResult;
	ContextPublic.Instance = Instance;
	
	Context->UpdateCallback.ExecuteIfBound(ContextPublic);
	Context->UpdateNativeCallback.Broadcast(ContextPublic);
	
	if (CVarFixLowPriorityTasksOverlap.GetValueOnGameThread())
	{
		if (SystemPrivate && Context->bLowPriorityTasksBlocked)
		{
			SystemPrivate->MutableTaskGraph.AllowLaunchingMutableTaskLowPriority(true, false);
		}
	}
	else
	{
		if (SystemPrivate)
		{
			SystemPrivate->MutableTaskGraph.AllowLaunchingMutableTaskLowPriority(true, false);
		}
	}
	
	if (Context->StartUpdateTime != 0.0) // Update started.
	{
		Context->UpdateTime = FPlatformTime::Seconds() - Context->StartUpdateTime;
	}
	
	if (SystemPrivate)
	{
		const FName ObjectName = GetFNameSafe(Object);
		const FName InstanceName = GetFNameSafe(Instance);

		FString Message = FString::Printf(TEXT("Finished Update Skeletal Mesh Async. CustomizableObject=%s Instance=%s, Frame=%d  QueueTime=%f, UpdateTime=%f"), *ObjectName.ToString(), *InstanceName.ToString(), GFrameNumber, Context->QueueTime, Context->UpdateTime);
		SystemPrivate->LogUpdateMessage(Message, ELogVerbosity::Verbose, Instance);
	}

	if (SystemPrivate && FLogBenchmarkUtil::IsBenchmarkingReportingEnabled())	
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady( // Calling Benchmark in a task so we make sure we exited all scopes.
		[Context]()
		{
			if (!UCustomizableObjectSystem::IsCreated()) // We are shutting down
			{
				return;	
			}
			
			UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
			if (!System)
			{
				return;
			}

			System->GetPrivate()->LogBenchmarkUtil.FinishUpdateMesh(Context);
		},
		TStatId{},
		nullptr,
		ENamedThreads::GameThread);
	}
	
	if (Context->UpdateStarted)
	{
		TRACE_END_REGION(UE_MUTABLE_UPDATE_REGION);		
	}
}


bool RequiresReinitPose(USkeletalMesh* CurrentSkeletalMesh, USkeletalMesh* NewSkeletalMesh)
{
	MUTABLE_CPUPROFILER_SCOPE(RequiresReinitPose);
	
	if (CurrentSkeletalMesh == NewSkeletalMesh)
	{
		return false;
	}

	if (!CurrentSkeletalMesh || !NewSkeletalMesh)
	{
		return NewSkeletalMesh != nullptr;
	}

	if (CurrentSkeletalMesh->GetLODNum() != NewSkeletalMesh->GetLODNum())
	{
		return true;
	}

	const FSkeletalMeshRenderData* CurrentRenderData = CurrentSkeletalMesh->GetResourceForRendering();
	const FSkeletalMeshRenderData* NewRenderData = NewSkeletalMesh->GetResourceForRendering();
	if (!CurrentRenderData || !NewRenderData)
	{
		return false;
	}

	// Instead of comparing the bone indices compare the name of the bones
	if (CVarRequiresReinitCompareBoneNames.GetValueOnAnyThread())
	{
		const FReferenceSkeleton& NewSkeletalMeshRefSkeleton = NewSkeletalMesh->GetRefSkeleton();
		const FReferenceSkeleton& CurrentSkeletalMeshRefSkeleton = CurrentSkeletalMesh->GetRefSkeleton();
	
		const int32 NumLODs = NewSkeletalMesh->GetLODNum();
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			const TArrayView<const FBoneIndexType> CurrentRequiredBones (CurrentRenderData->LODRenderData[LODIndex].RequiredBones);
			const TArrayView<const FBoneIndexType> NewRequiredBones (NewRenderData->LODRenderData[LODIndex].RequiredBones);

			if (CurrentRequiredBones.Num() != NewRequiredBones.Num())
			{
				return true;
			}

			// Iterate over the ref skeletons comparing the names of each of the bones found there by using the value for each one of the LODs
			// If the names match we should assume that it is safe to not re-init the pose.
		
			const TArray<FMeshBoneInfo>& CurrentBoneInfo = CurrentSkeletalMeshRefSkeleton.GetRefBoneInfo();
			const TArray<FMeshBoneInfo>& NewBoneInfo = NewSkeletalMeshRefSkeleton.GetRefBoneInfo();

			const int32 RequiredBoneCount = CurrentRequiredBones.Num();
			for (int32 Index = 0; Index < RequiredBoneCount; ++Index)
			{
				const FBoneIndexType& NewBoneIndex = NewRequiredBones[Index];
				const FBoneIndexType& CurrentBoneIndex = CurrentRequiredBones[Index];

				if (NewBoneIndex == CurrentBoneIndex)
				{
					continue;
				}
			
				if (NewBoneInfo[NewBoneIndex].Name != CurrentBoneInfo[CurrentBoneIndex].Name)
				{
					return true;
				}
			}
		}
	}
	else
	{
		const int32 NumLODs = NewSkeletalMesh->GetLODNum();
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			if (CurrentRenderData->LODRenderData[LODIndex].RequiredBones != NewRenderData->LODRenderData[LODIndex].RequiredBones)
			{
				return true;
			}
		}
	}

	return false;
}


/** Update the given Instance Skeletal Meshes */
void UpdateSkeletalMesh(const TSharedRef<FUpdateContextPrivate>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(UpdateSkeletalMesh);

	check(IsInGameThread());

	UCustomizableObjectInstance* CustomizableObjectInstance = Context->Instance.Get();
	UCustomizableObject* CustomizableObject = Context->Object.Get();

	if (!CustomizableObjectInstance || !CustomizableObject)
	{
		return;
	}
	
	UCustomizableInstancePrivate* InstancePrivate = CustomizableObjectInstance->GetPrivate();
	check(InstancePrivate != nullptr);

	for (const TTuple<FName, TObjectPtr<USkeletalMesh>>& Pair : InstancePrivate->SkeletalMeshes)
	{
		FPreSetSkeletalMeshParams Params;
		Params.Instance = CustomizableObjectInstance;
		Params.SkeletalMesh = Pair.Value;
		
		CustomizableObjectInstance->PreSetSkeletalMeshDelegate.Broadcast(Params);
		CustomizableObjectInstance->PreSetSkeletalMeshNativeDelegate.Broadcast(Params);
	}
	
	uint32 ParentsWithOverrideMaterialsUpdatedCount = 0;

	for (UCustomizableObjectInstanceUsage* InstanceUsage : InstancePrivate->InstanceUsages)
	{
		if (!IsValid(InstanceUsage))
		{
			continue;
		}

#if WITH_EDITOR
		if (InstanceUsage->GetPrivate()->bIsNetModeDedicatedServer)
		{
			continue;
		}
#endif
		USkeletalMeshComponent* Parent = Cast<USkeletalMeshComponent>(InstanceUsage->GetAttachParent());
		if (!Parent)
		{
			continue;
		}

		MUTABLE_CPUPROFILER_SCOPE(UpdateSkeletalMesh_SetSkeletalMesh);

		bool bAttachedParentUpdated = false;

		InstanceUsage->GetPrivate()->bPendingSetReferenceSkeletalMesh = false;

		USkeletalMesh* SkeletalMesh = CustomizableObjectInstance->GetComponentMeshSkeletalMesh(InstanceUsage->GetComponentName());
		if (SkeletalMesh != Parent->GetSkeletalMeshAsset())
		{
			Parent->SetSkeletalMesh(SkeletalMesh, RequiresReinitPose(Parent->GetSkeletalMeshAsset(), SkeletalMesh));
			bAttachedParentUpdated = true;
		}

		// Skip further checks since the instance did not change and the component has the correct SkeletalMesh.
		if (!bAttachedParentUpdated && Context->bOptimizedUpdate)
		{
			continue;
		}

		TArray<TObjectPtr<UMaterialInterface>> NewOverridenMaterials;

		FCustomizableInstanceComponentData* ComponentData = CustomizableObjectInstance->GetPrivate()->GetComponentData(InstanceUsage->GetComponentName());
		
		if (ComponentData)
		{
			NewOverridenMaterials = ComponentData->OverrideMaterials;
		}

		if (Parent->OverrideMaterials != NewOverridenMaterials)
		{
			// Before setting the new override materials clear the old ones set in the Skeletal Mesh Component
			// Note : this also resets the array of overlay materials used for each one of the override materials.
			Parent->EmptyOverrideMaterials();

			// Set the new override materials
			for (int32 Index = 0; Index < NewOverridenMaterials.Num(); ++Index)
			{
				Parent->SetMaterial(Index, NewOverridenMaterials[Index]);
			}

			ParentsWithOverrideMaterialsUpdatedCount++;
			
			bAttachedParentUpdated = true;
		}


		{
			UMaterialInterface* OverlayMaterial = nullptr;
			if (ComponentData)
			{
				OverlayMaterial = ComponentData->OverlayMaterial;
			}

			bAttachedParentUpdated |= Parent->GetOverlayMaterial() != OverlayMaterial;
			Parent->SetOverlayMaterial(OverlayMaterial);
		}

		if (ComponentData)
		{
			for (int32 OverlayMaterialIndex = 0; OverlayMaterialIndex < ComponentData->OverlayMaterials.Num(); ++OverlayMaterialIndex)
			{
				UMaterialInterface* OverlayMaterial = ComponentData->OverlayMaterials[OverlayMaterialIndex]; 
					
				bAttachedParentUpdated |= Parent->GetOverlayMaterial(true, OverlayMaterialIndex) != OverlayMaterial;
				Parent->SetOverlayMaterial(OverlayMaterial, true, OverlayMaterialIndex);
			}
		}
		
		if (InstancePrivate->HasCOInstanceFlags(ReplacePhysicsAssets) &&
			SkeletalMesh &&
			Parent->GetWorld())
		{
			UPhysicsAsset* PhysicsAsset = SkeletalMesh->GetPhysicsAsset();
			if (PhysicsAsset != Parent->GetPhysicsAsset())
			{
				Parent->SetPhysicsAsset(PhysicsAsset, true);
				bAttachedParentUpdated = true;
			}
		}

		if (bAttachedParentUpdated)
		{
			Context->AttachedParentUpdated.Add(InstanceUsage);
		}
		
		for (const UCustomizableObjectExtension* Extension : ICustomizableObjectModule::Get().GetRegisteredExtensions())
		{
			Extension->OnCustomizableObjectInstanceUsageUpdated(*InstanceUsage, InstancePrivate->ExtensionData);
		}
	}

	if (ParentsWithOverrideMaterialsUpdatedCount > 0)
	{
		UE_LOGF(LogMutable, Verbose, "A total of %u Skeletal Mesh Components got their override materials updated.", ParentsWithOverrideMaterialsUpdatedCount);
	}
}


void UCustomizableObjectSystemPrivate::UpdateViewLocations()
{
	if (!bGenerateInstancesWithinRange)
	{
		return;
	}

	MUTABLE_CPUPROFILER_SCOPE(UpdateViewLocations);
	ViewLocations.Reset(ViewCenters.Num());
	const int32 NumViewCenters = ViewCenters.Num();

	for (const TWeakObjectPtr<const AActor> ViewCenter : ViewCenters)
	{
		if (ViewCenter.IsValid())
		{
			ViewLocations.Add(ViewCenter->GetActorLocation());
		}
	}
}


bool UCustomizableObjectSystemPrivate::IsDiscardedByDistance(const UCustomizableObjectInstance& Instance) const
{
	if (!bGenerateInstancesWithinRange)
	{
		return false;
	}

#if !WITH_EDITOR
	if (ViewLocations.IsEmpty())
	{
		return false;
	}
#endif

	MUTABLE_CPUPROFILER_SCOPE(IsDiscardedByDistance);

	float MinSquareDistFromComponentToPlayer = FLT_MAX;
	for (UCustomizableObjectInstanceUsage* InstanceUsage : Instance.GetPrivate()->InstanceUsages)
	{
		if (!IsValid(InstanceUsage))
		{
			continue;
		}

		USkeletalMeshComponent* Parent = InstanceUsage->GetAttachParent();
		if (!Parent)
		{
			continue;
		}
		const AActor* ParentActor = Parent->GetAttachmentRootActor();
		if (!IsValid(ParentActor))
		{
			continue;
		}
		const APawn* Pawn = Cast<const APawn>(ParentActor);
		if (Pawn && Pawn->IsPlayerControlled())
		{
			return false;
		}
		const FVector ActorLocation = ParentActor->GetActorLocation();
		for (const FVector& ViewLocation : ViewLocations)
		{
			MinSquareDistFromComponentToPlayer = FMath::Min(FVector::DistSquared(ViewLocation, ActorLocation), MinSquareDistFromComponentToPlayer);
		}
#if WITH_EDITOR
		UWorld* LocalWorld = Parent->GetWorld();
		if (LocalWorld && LocalWorld->WorldType == EWorldType::Editor)
		{
			for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
			{
				if (LevelVC && LevelVC->IsPerspective() && LevelVC->GetWorld() == LocalWorld)
				{
					MinSquareDistFromComponentToPlayer = FMath::Min(FVector::DistSquared(LevelVC->GetViewLocation(), ActorLocation), MinSquareDistFromComponentToPlayer);
					break;
				}
			}
		}
#endif
		if (MinSquareDistFromComponentToPlayer <= GenerationRangeSquare)
		{
			return false;
		}
	}
	return true;
}


void UCustomizableObjectSystemPrivate::DiscardInstance(UCustomizableObjectInstance& Instance)
{
	MUTABLE_CPUPROFILER_SCOPE(DiscardInstance);

	UCustomizableInstancePrivate* InstancePrivate = Instance.GetPrivate();
	InstancePrivate->InvalidateGeneratedData();

	UCustomizableObject* Object = Instance.GetCustomizableObject();
	check(Object);

	for (UCustomizableObjectInstanceUsage* InstanceUsage : InstancePrivate->InstanceUsages)
	{
		if (!IsValid(InstanceUsage))
		{
			continue;
		}

		USkeletalMeshComponent* Parent = InstanceUsage->GetAttachParent();
		if (!Parent)
		{
			continue;
		}

		Parent->EmptyOverrideMaterials();
		USkeletalMesh* SkeletalMesh = nullptr;

		if (!InstanceUsage->GetSkipSetReferenceSkeletalMesh() &&
			Object->bEnableUseRefSkeletalMeshAsPlaceholder)
		{
			const FName& ComponentName = InstanceUsage->GetComponentName();
			SkeletalMesh = Object->GetComponentMeshReferenceSkeletalMesh(ComponentName);
		}

		Parent->SetSkeletalMesh(SkeletalMesh);

		for (const UCustomizableObjectExtension* Extension : ICustomizableObjectModule::Get().GetRegisteredExtensions())
		{
			Extension->OnCustomizableObjectInstanceUsageDiscarded(*InstanceUsage);
		}
	}
}


void TryAutomaticCompilation(UCustomizableObjectInstance* Instance)
{
#if WITH_EDITOR
	if (!IsValid(Instance))
	{
		return;
	}

	UCustomizableObject* Object = Instance->GetCustomizableObject();
	if (!IsValid(Object))
	{
		return;
	}

	if (!Object->IsCompiled() &&
		Object->GetPrivate()->CompilationResult != ECompilationResultPrivate::Errors) // Avoid constantly retry failed compilations.
	{
		if (ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get())
		{
			FCompileParams CompileParams;
			CompileParams.bSkipIfCompiled = true;

			EditorModule->CompileCustomizableObject(*Object, &CompileParams, true, false);
		}
	}
#endif
}


void UCustomizableObjectSystemPrivate::EnqueueUpdateSkeletalMesh(const TSharedRef<FUpdateContextPrivate>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectSystemPrivate::EnqueueUpdateSkeletalMesh);
	check(IsInGameThread());
	
	UCustomizableObjectInstance* Instance = Context->Instance.Get();

	TryAutomaticCompilation(Instance);

	if (!CanUpdate(Instance))
	{
		Context->UpdateResult = EUpdateResult::Error;
		FinishUpdateGlobal(Context);
		return;
	}
	
	// TODO GMT Move this further down
	Context->Init(false);

	UCustomizableObject* Object = Context->Object.Get();
	UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();

	UE_LOGF(LogMutable, Verbose, "Enqueue Update Skeletal Mesh Async. CustomizableObject=%ls Instance=%ls, Frame=%d", *Object->GetFName().ToString(), *Instance->GetFName().ToString(), GFrameNumber);
		
	if (!bIsMutableEnabled)
	{
		// Mutable is disabled. Set the reference SkeletalMesh and finish the update with success to avoid breaking too many things.
		Context->UpdateResult = EUpdateResult::Success;
		InstancePrivate->ForceSetReferenceSkeletalMesh();
		FinishUpdateGlobal(Context);
		return;
	}
	
	if (!Context->bForce)
	{
		if (CurrentMutableOperation &&
			Instance == CurrentMutableOperation->Instance &&
			Context->GetCapturedDescriptorHash() == CurrentMutableOperation->GetCapturedDescriptorHash())
		{
			Context->bOptimizedUpdate = true;
			Context->UpdateResult = EUpdateResult::ErrorOptimized;
			FinishUpdateGlobal(Context);
			return; // The requested update is equal to the running update.
		}

		if (Context->GetCapturedDescriptorHash() == InstancePrivate->CommittedDescriptorHash &&
			!(CurrentMutableOperation &&
				Instance == CurrentMutableOperation->Instance)) // This condition is necessary because even if the descriptor is a subset, it will be replaced by the CurrentMutableOperation
		{
			if (CVarEnableUpdateOptimization.GetValueOnGameThread()) // TODO Remove hotfix: UE-218957 
			{
				Context->bOptimizedUpdate = true;

				// The user may have changed the AttachParent and we need to re-customize it.
				// In case nothing need to be re-customized, the update will be considered ErrorOptimized.
				UpdateSkeletalMesh(Context);
				Context->UpdateResult = Context->AttachedParentUpdated.IsEmpty() ? EUpdateResult::ErrorOptimized : EUpdateResult::Success;

				FinishUpdateGlobal(Context);
				return;
			}
			else
			{
				// The user may have changed the AttachParent and we need to re-customize it.
				// In case nothing need to be re-customized, the update will be considered ErrorOptimized.
				UpdateSkeletalMesh(Context);
				FinishUpdateGlobal(Context);
				return;
			}
		}
	}

	if (CVarDescriptorDebugPrint->GetBool())
	{
		FString String = TEXT("DESCRIPTOR DEBUG PRINT\n");
		String += "================================\n";
		String += FString::Printf(TEXT("=== DESCRIPTOR HASH ===\n%s\n"), *Context->GetCapturedDescriptorHash().ToString());
		String += FString::Printf(TEXT("=== DESCRIPTOR ===\n%s"), *Instance->GetPrivate()->GetDescriptor().ToString());
		String += "================================";

		UE_LOGF(LogMutable, Log, "%ls", *String);
	}

	Context->StartQueueTime = FPlatformTime::Seconds();

	auto EnqueueUpdate = [&]()
		{
			InstancePrivate->ClearCOInstanceFlags(ECOInstanceFlags::HasPendingHighPriorityUpdate | ECOInstanceFlags::HasPendingLowPriorityUpdate);

			if (Context->bIsHighPriority)
			{
				HighPriorityInstanceUpdates.PushLast(Context);
				InstancePrivate->SetCOInstanceFlags(ECOInstanceFlags::HasPendingHighPriorityUpdate);
			}
			else
			{
				LowPriorityInstanceUpdates.PushLast(Context);
				InstancePrivate->SetCOInstanceFlags(ECOInstanceFlags::HasPendingLowPriorityUpdate);
			}

			++PendingInstanceUpdates;
		};

	if (InstancePrivate->HasCOInstanceFlags(ECOInstanceFlags::HasPendingHighPriorityUpdate))
	{
		for (FMutableInstanceUpdate& HighPriorityUpdate : HighPriorityInstanceUpdates)
		{
			if (HighPriorityUpdate && HighPriorityUpdate->Instance == Context->Instance)
			{
				HighPriorityUpdate->bOptimizedUpdate = true;
				HighPriorityUpdate->UpdateResult = EUpdateResult::ErrorOptimized;
				FinishUpdateGlobal(HighPriorityUpdate.ToSharedRef());

				if (Context->bIsHighPriority)
				{
					HighPriorityUpdate = Context;
				}
				else
				{
					HighPriorityUpdate.Reset();
					EnqueueUpdate();
				}

				break;
			}
		}
	}
	else if (InstancePrivate->HasCOInstanceFlags(ECOInstanceFlags::HasPendingLowPriorityUpdate))
	{
		for (FMutableInstanceUpdate& LowPriorityUpdate : LowPriorityInstanceUpdates)
		{
			if (LowPriorityUpdate && LowPriorityUpdate->Instance == Context->Instance)
			{
				LowPriorityUpdate->bOptimizedUpdate = true;
				LowPriorityUpdate->UpdateResult = EUpdateResult::ErrorOptimized;
				FinishUpdateGlobal(LowPriorityUpdate.ToSharedRef());

				if (!Context->bIsHighPriority)
				{
					LowPriorityUpdate = Context;
				}
				else
				{
					LowPriorityUpdate.Reset();
					EnqueueUpdate();
				}

				break;
			}
		}
	}
	else
	{
		EnqueueUpdate();
	}
}


#if WITH_EDITOR
bool UCustomizableObjectSystem::LockObject(UCustomizableObject* InObject)
{
	check(InObject != nullptr);
	check(InObject->GetPrivate());
	check(!InObject->GetPrivate()->bLocked);
	check(IsInGameThread() && !IsInParallelGameThread());

	if (Private)
	{
		// If the current instance is for this object, make the lock fail by returning false
		if (Private->CurrentMutableOperation &&
			Private->CurrentMutableOperation->Object.Get() == InObject)
		{
			UE_LOGF(LogMutable, Warning, "---- failed to lock object %ls", *InObject->GetName());

			return false;
		}

		FString Message = FString::Printf(TEXT("Customizable Object %s has pending texture streaming operations. Please wait a few seconds and try again."),
			*InObject->GetName());

		// Pre-check pending operations before locking. This check is redundant and incomplete because it's checked again after locking 
		// and some operations may start between here and the actual lock. But in the CO Editor preview it will prevent some 
		// textures getting stuck at low resolution when they try to update mips and are cancelled when the user presses 
		// the compile button but the compilation quits anyway because there are pending operations
		if (CheckIfDiskOrMipUpdateOperationsPending(*InObject))
		{
			UE_LOGF(LogMutable, Warning, "%ls", *Message);

			return false;
		}

		// Lock the object, no new file or mip streaming operations should start from this point
		InObject->GetPrivate()->bLocked = true;

		// Invalidate the current model to avoid further disk or mip updates.
		if (InObject->GetPrivate()->GetModel())
		{
			InObject->GetPrivate()->GetModel()->Invalidate();
		}

		// But some could have started between the first CheckIfDiskOrMipUpdateOperationsPending and the lock a few lines back, so check again
		if (CheckIfDiskOrMipUpdateOperationsPending(*InObject))
		{
			UE_LOGF(LogMutable, Warning, "%ls", *Message);

			// Unlock and return because the pending operations cannot be easily stopped now, the compilation hasn't started and the CO
			// hasn't changed state yet. It's simpler to quit the compilation, unlock and let the user try to compile again
			InObject->GetPrivate()->bLocked = false;

			return false;
		}

		// Ensure that we don't try to handle any further streaming operations for this object
		check(GetPrivate() != nullptr);
		if (GetPrivate()->Streamer)
		{
			UE::Tasks::FTask Task = Private->MutableTaskGraph.AddMutableThreadTask(TEXT("EndStream"), [InObject, Streamer = GetPrivate()->Streamer]()
				{
					Streamer->CancelStreamingForObject(InObject);
				});

			
			Task.Wait();
		}

		check(InObject->GetPrivate()->bLocked);

		return true;
	}
	else
	{
		FString ObjectName = InObject ? InObject->GetName() : FString("null");
		UE_LOGF(LogMutable, Warning, "Failed to lock the object [%ls] because it was null or the system was null or partially destroyed.", *ObjectName);

		return false;
	}
}


void UCustomizableObjectSystem::UnlockObject(UCustomizableObject* Obj)
{
	check(Obj != nullptr);
	check(Obj->GetPrivate());
	check(Obj->GetPrivate()->bLocked);
	check(IsInGameThread() && !IsInParallelGameThread());
	
	Obj->GetPrivate()->bLocked = false;
}


bool UCustomizableObjectSystem::CheckIfDiskOrMipUpdateOperationsPending(const UCustomizableObject& Object) const
{
	for (TObjectIterator<UCustomizableObjectInstance> CustomizableObjectInstance; CustomizableObjectInstance; ++CustomizableObjectInstance)
	{
		if (IsValid(*CustomizableObjectInstance) && CustomizableObjectInstance->GetCustomizableObject() == &Object)
		{
			for (const TObjectPtr<UTexture>& GeneratedTexture : CustomizableObjectInstance->GetPrivate()->Textures) // TODO GMT This approach is incorrect. An update may clear the generated Textures array but these may live longer.
			{
				if (GeneratedTexture && GeneratedTexture->HasPendingInitOrStreaming())
				{
					return true;
				}
			}
		}
	}

	// Ensure that we don't try to handle any further streaming operations for this object
	check(GetPrivate());
	if (const FUnrealMutableModelBulkReader* Streamer = GetPrivate()->Streamer.Get())
	{
		if (Streamer->AreTherePendingStreamingOperationsForObject(&Object))
		{
			return true;
		}
	}

	return false;
}


void UCustomizableObjectSystem::EditorSettingsChanged(const FEditorCompileSettings& InEditorSettings)
{
	GetPrivate()->EditorSettings = InEditorSettings;

	CVarMutableEnabled->Set(InEditorSettings.bIsMutableEnabled);
}

bool UCustomizableObjectSystem::IsAutoCompileEnabled() const
{
	return GetPrivate()->EditorSettings.bEnableAutomaticCompilation;
}


bool UCustomizableObjectSystem::IsAutoCompileCommandletEnabled() const
{
	return GetPrivate()->bAutoCompileCommandletEnabled;
}


void UCustomizableObjectSystem::SetAutoCompileCommandletEnabled(bool bValue)
{
	GetPrivate()->bAutoCompileCommandletEnabled = bValue;
}


bool UCustomizableObjectSystem::IsAutoCompilationSync() const
{
	return GetPrivate()->EditorSettings.bCompileObjectsSynchronously;
}

#endif

void UCustomizableObjectSystemPrivate::UpdateMemoryLimit()
{
	// This must run on game thread, and when the mutable thread is not running
	check(IsInGameThread());

	const uint64 MemoryBytes = uint64(CVarWorkingMemoryKB->GetInt()) * 1024;
	if (MemoryBytes != LastWorkingMemoryBytes)
	{
		LastWorkingMemoryBytes = MemoryBytes;
		check(MutableSystem);
		MutableSystem->SetWorkingMemoryBytes(MemoryBytes);
	}
}


namespace impl
{
	void Task_Mutable_UpdateParameterRelevancy(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Subtask_Mutable_UpdateParameterRelevancy)

		if (OperationData->bIsUpdateAborted)
		{
			return;	
		}
		
		check(OperationData->Parameters);
		
		// This must run in the mutable thread.
		check(UCustomizableObjectSystem::GetInstance() != nullptr);
		check(UCustomizableObjectSystem::GetInstance()->GetPrivate() != nullptr);

		// Update the parameter relevancy.
		{
			MUTABLE_CPUPROFILER_SCOPE(ParameterRelevancy)
			OperationData->MutableSystem->GetParameterRelevancy(OperationData->LiveInstance.ToSharedRef(), OperationData->RelevantParameters);
		}

		OperationData->TaskGetMeshesTime = FPlatformTime::Seconds() - OperationData->TaskGetMeshesTimeStart;
	}
	
	
	// This runs in a worker thread
	void Task_Mutable_PrepareSkeletonData(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Subtask_Mutable_PrepareSkeletonData);

		if (OperationData->bIsUpdateAborted)
		{
			return;	
		}
		
		for (const TSharedRef<FInstanceUpdateData::FComponent>& Component : OperationData->InstanceUpdateData.Components)
		{
			for (int32 LODIndex = 0; LODIndex < Component->LODCount; ++LODIndex)
			{
				TSharedRef<FInstanceUpdateData::FLOD>& LOD = OperationData->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];

				UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh = LOD->Mesh;
				if (!Mesh || Mesh->IsReference())
				{
					continue;
				}

				if (!Mesh->GetSkeleton())
				{
					continue;
				}

				const TArray<FName>& BoneNames = Mesh->GetSkeleton()->BoneNames;

				// Append BoneMap to the array of BoneMaps
				const TArray<UE::Mutable::Private::FBoneIdOrIndex>& BoneMap = Mesh->GetBoneMap();
				LOD->FirstBoneMap = OperationData->InstanceUpdateData.BoneMaps.Num();
				LOD->BoneMapCount = BoneMap.Num();
				
				for (UE::Mutable::Private::FBoneIdOrIndex Bone : BoneMap)
				{
					OperationData->InstanceUpdateData.BoneMaps.Add(BoneNames[Bone.Index]);
				}

				// Add active bone indices and poses
				LOD->FirstActiveBone = OperationData->InstanceUpdateData.ActiveBones.Num();
				LOD->ActiveBoneCount = Mesh->GetBonePoseCount();
				for (uint32 BoneIndex = 0; BoneIndex < LOD->ActiveBoneCount; ++BoneIndex)
				{
					FName BoneName = BoneNames[Mesh->GetBonePoseId(BoneIndex).Index];

					OperationData->InstanceUpdateData.ActiveBones.Add(BoneName);

					if (!Component->BonePose.FindByKey(BoneName))
					{
						FTransform3f Transform;
						Mesh->GetBonePoseTransform(BoneIndex, Transform);
						Component->BonePose.Add({ BoneName,Transform.Inverse().ToMatrixWithScale() });
					}
				}
			}
		}
	}

	
	void Subtask_Mutable_PrepareRealTimeMorphData(const TSharedRef<FUpdateContextPrivate>& OperationData) 
	{
		MUTABLE_CPUPROFILER_SCOPE(BuildMorphTargetsData);

		FInstanceUpdateData& UpdateData = OperationData->InstanceUpdateData;

		const int32 NumComponents = OperationData->InstanceUpdateData.Components.Num();
		
		UpdateData.RealTimeMorphTargets.Reserve(NumComponents);
		for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			TSharedRef<FInstanceUpdateData::FComponent>& Component = OperationData->InstanceUpdateData.Components[ComponentIndex];
			check(Component->Id != INDEX_NONE);

			const FName ComponentName = OperationData->ComponentNames[Component->Id];
			
			FSkeletalMeshMorphTargets& ComponentMorphTargets = UpdateData.RealTimeMorphTargets.Add(Component->Id);

			// Allocate Morph data for used morphs.
			const int32 NumLODsAvailable = OperationData->NumLODsAvailable[ComponentName];
	
			for (int32 LODIndex = OperationData->FirstResidentLOD[ComponentName]; LODIndex < NumLODsAvailable; ++LODIndex)
			{
				const TSharedRef<FInstanceUpdateData::FLOD>& LOD = UpdateData.LODs[Component->FirstLOD + LODIndex];

				if (!(LOD->Mesh && LOD->Mesh->HasMorphs()))
				{
					continue;
				}

				// At this point, if the mesh HasMorph then we must have something in morph data.
				check(LOD->Mesh->MorphDataBuffer.Num());

				TArray<FMorphTargetLODModel> MorphTargets;
				ReconstructMorphTargetsFromMeshCompressedData(*LOD->Mesh, MorphTargets, UE::Mutable::Private::EMorphUsageFlags::RealTime);

				const UE::Mutable::Private::FMeshMorph& Morph = LOD->Mesh->Morph;

				for (int32 LODMorphIndex = 0, NumMorphs = Morph.Names.Num(); LODMorphIndex < NumMorphs; ++LODMorphIndex)
				{
					if (MorphTargets[LODMorphIndex].Vertices.Num())
					{
						const FName MorphName = Morph.Names[LODMorphIndex];
						int32 MorphIndex = ComponentMorphTargets.RealTimeMorphTargetNames.IndexOfByKey(MorphName);
						
						if (MorphIndex == INDEX_NONE)
						{
							ComponentMorphTargets.RealTimeMorphsLODData.AddDefaulted_GetRef().SetNum(NumLODsAvailable);
							MorphIndex = ComponentMorphTargets.RealTimeMorphTargetNames.Add(MorphName);
						}
						
						ComponentMorphTargets.RealTimeMorphsLODData[MorphIndex][LODIndex] = MoveTemp(MorphTargets[LODMorphIndex]);
					}
				}
			}
		}
	}
	

	void Task_Mutable_GetSurfaces(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_GetSurfaces)
		
		if (OperationData->bIsUpdateAborted)
		{
			return;
		}
		
		const UE::Mutable::Private::FInstance* MutableInstance = OperationData->MutableInstance.Get();

		TArray<uint32> GeneratedSurfaceIds;
		
		UE::Tasks::FTask LastTask = UE::Tasks::MakeCompletedTask<void>();
		
		for (int32 ComponentIndex = 0; ComponentIndex < OperationData->InstanceUpdateData.Components.Num(); ++ComponentIndex)
		{
			TSharedRef<FInstanceUpdateData::FComponent>& Component = OperationData->InstanceUpdateData.Components[ComponentIndex];
			const FName& ComponentName = OperationData->ComponentNames[Component->Id];

			const int32 ComponentFirstSurfaceIndex = OperationData->InstanceUpdateData.Surfaces.Num();

			for (int32 MutableLODIndex = OperationData->FirstLODAvailable[ComponentName]; MutableLODIndex < Component->LODCount; ++MutableLODIndex)
			{
				TSharedRef<FInstanceUpdateData::FLOD>& LOD = OperationData->InstanceUpdateData.LODs[Component->FirstLOD + MutableLODIndex];

				LOD->FirstSurface = OperationData->InstanceUpdateData.Surfaces.Num();
				LOD->SurfaceCount = 0;

				if (!LOD->Mesh)
				{
					continue;
				}
				
				// Materials and images
				const int32 SurfaceCount = LOD->Mesh->GetSurfaceCount();
				for (int32 MeshSurfaceIndex = 0; MeshSurfaceIndex < SurfaceCount; ++MeshSurfaceIndex)
				{
					const uint32 SurfaceId = LOD->Mesh->GetSurfaceId(MeshSurfaceIndex);

					const int32 MaterialSlotIndex = Component->SkeletalMesh->MaterialSlotIds.IndexOfByKey(SurfaceId);
					check(LOD->Mesh->GetVertexCount() > 0 || MaterialSlotIndex >= 0);

					if (MaterialSlotIndex < 0)
					{
						continue;
					}

					++LOD->SurfaceCount;
					
					const int32 ComponentSurfaceIndex = GeneratedSurfaceIds.Find(SurfaceId); // SurfaceIndex relative to the component
					GeneratedSurfaceIds.Add(SurfaceId);

					if (ComponentSurfaceIndex != INDEX_NONE)
					{
						const int32 SurfaceIndex = ComponentFirstSurfaceIndex + ComponentSurfaceIndex;
						TSharedRef<FInstanceUpdateData::FSurface> GeneratedSurface = OperationData->InstanceUpdateData.Surfaces[SurfaceIndex];
						OperationData->InstanceUpdateData.Surfaces.Add(GeneratedSurface);
						continue;
					}

					TSharedRef<FInstanceUpdateData::FSurface>& Surface = OperationData->InstanceUpdateData.Surfaces.Add_GetRef(MakeShared<FInstanceUpdateData::FSurface>());

					Surface->SurfaceId = SurfaceId;
					Surface->MaterialSlotName = Component->SkeletalMesh->MaterialSlotNames[MaterialSlotIndex];
					Surface->MaterialIndex = INDEX_NONE;

					const TVariant<UE::Mutable::Private::FOperation::ADDRESS, UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>>& MaterialOrAddress = 
							Component->SkeletalMesh->MaterialSlotMaterials[MaterialSlotIndex];

					UE::Mutable::Private::FMaterialId MaterialId;

					if (MaterialOrAddress.IsType<UE::Mutable::Private::FOperation::ADDRESS>())
					{
						const UE::Mutable::Private::FOperation::ADDRESS Address = Component->SkeletalMesh->MaterialSlotMaterials[MaterialSlotIndex].Get<UE::Mutable::Private::FOperation::ADDRESS>();
						MaterialId = UE::Mutable::Private::FSystem::GetMaterialId(OperationData->LiveInstance->MaterialIdRegistry, *OperationData->Model.Get(), *OperationData->Parameters.Get(), Address);
					}

					if (MaterialId || MaterialOrAddress.IsType<UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>>())
					{	
						Surface->MaterialIndex = OperationData->InstanceUpdateData.Materials.Add(MakeShared<FInstanceUpdateData::FMaterial>());

						if (MaterialOrAddress.IsType<UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>>())
						{
							OperationData->InstanceUpdateData.Materials[Surface->MaterialIndex]->Material = 
									MaterialOrAddress.Get<UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>>();

							OperationData->InstanceUpdateData.Materials[Surface->MaterialIndex]->MaterialId = {};
						}
						else
						{
							OperationData->InstanceUpdateData.Materials[Surface->MaterialIndex]->MaterialId = MaterialId;
						}
					}
				}
			}

			GeneratedSurfaceIds.SetNum(0, EAllowShrinking::No);
		}
	}


	void Task_Mutable_GetMaterials(const TSharedRef<FUpdateContextPrivate>& Context)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_GetMaterials)

		if (Context->bIsUpdateAborted)
		{
			return;
		}
		
		// TODO We should not access UObjects outside GT
		const TStrongObjectPtr<UCustomizableObject> CustomizableObject = Context->Object.Pin();
		if (!IsValid(CustomizableObject.Get()))
		{
			Context->bIsUpdateAborted = true;
			return;
		}

		// TODO We should not access UObjects outside GT
		const TStrongObjectPtr<UModelResources> ModelResources = TStrongObjectPtr(CustomizableObject->GetPrivate()->GetModelResources());
		if (!IsValid(ModelResources.Get()))
		{
			Context->bIsUpdateAborted = true;
			return;
		}
		
		UE::Tasks::FTask LastTask = UE::Tasks::MakeCompletedTask<void>();
		
		for (int32 MaterialIndex = 0; MaterialIndex < Context->InstanceUpdateData.Materials.Num(); ++MaterialIndex)
		{
			TSharedRef<FInstanceUpdateData::FMaterial> Material = Context->InstanceUpdateData.Materials[MaterialIndex];
			
			UE::Tasks::TTask<UE::Mutable::Private::FGetMaterialResult> GetMaterialTask;
			if (Material->Material)
			{
				check(!Material->MaterialId);
				GetMaterialTask = UE::Tasks::MakeCompletedTask<UE::Mutable::Private::FGetMaterialResult>(UE::Mutable::Private::FGetMaterialResult
				{
					.MutableMaterial = Material->Material,
					.bWasOperationSuccessful = !!Material->Material,
				});
			}
			else
			{
				GetMaterialTask = Context->MutableSystem->GetMaterial(LastTask, Context->LiveInstance.ToSharedRef(), Material->MaterialId);
			}

			LastTask = UE::Tasks::Launch(TEXT("GetMaterialResult"), [Context, ModelResources, Material, GetMaterialTask]() mutable
			{
				const UE::Mutable::Private::FGetMaterialResult Result = GetMaterialTask.GetResult();
				check(Result.MutableMaterial);
					
				// The material could not be found so just skip the processing of it
				if (!Result.bWasOperationSuccessful)
				{
					Context->bIsUpdateAborted = true;
					return;
				}
					
				Material->Material = Result.MutableMaterial;

				// Images
				Material->FirstImage = Context->InstanceUpdateData.Images.Num();
				Material->ImageCount = Material->Material->ImageParameters.Num();
				for (TTuple<UE::Mutable::Private::FParameterKey, UE::Mutable::Private::FMaterial::FImageParameterData> ImageParameter : Material->Material->ImageParameters)
				{
					const UE::Mutable::Private::FOperation::ADDRESS Address = ImageParameter.Value.ImageParameter.Get<UE::Mutable::Private::FOperation::ADDRESS>();
					
					TSharedRef<FInstanceUpdateData::FImage>& Image = Context->InstanceUpdateData.Images.Add_GetRef(MakeShared<FInstanceUpdateData::FImage>());
					Image->ParameterName = ImageParameter.Key.ParameterName;
					Image->MaterialLayer = (int32)ImageParameter.Key.LayerIndex;
					Image->ImageID = UE::Mutable::Private::FSystem::GetImageId(Context->LiveInstance->ImageIdRegistry, *Context->Model.Get(), *Context->Parameters.Get(), Address);
					Image->FullImageSizeX = 0;
					Image->FullImageSizeY = 0;
					Image->BaseMip = 0;
					Image->ImagePropertiesIndex = ImageParameter.Value.ImagePropertyIndex;

					check(ModelResources->ImageProperties.IsValidIndex(Image->ImagePropertiesIndex))
					const FMutableModelImageProperties& Props = ModelResources->ImageProperties[Image->ImagePropertiesIndex];
					Image->bIsNonProgressive = Props.MipGenSettings == TMGS_NoMipmaps;
					Image->bIsPassThrough = Props.IsPassThrough;
				}
				
			},
			GetMaterialTask,
			LowLevelTasks::ETaskPriority::Inherit);

			UE::Tasks::AddNested(LastTask);
		}
	}

	
	void Task_Mutable_GetSkeletalMeshes(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_GetSkeletalMeshes)

		if (OperationData->bIsUpdateAborted)
		{
			return;
		}
		
		OperationData->TaskGetMeshesTimeStart = FPlatformTime::Seconds();

		check(OperationData->Parameters);
		OperationData->InstanceUpdateData.Clear();
		
		check(OperationData->LiveInstance);
		
		// Prevent the usage of the already cached elements by fully clearing the cache.
		if (!CVarRollbackReuseProgramCacheBetweenUpdates.GetValueOnAnyThread())
		{
			OperationData->LiveInstance->Cache->Clear(UE::Mutable::Private::FProgramCache::EClearFlags::Full);
		}
		
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FInstance> Instance = OperationData->MutableSystem->BeginUpdate_MutableThread(
			OperationData->LiveInstance.ToSharedRef(),
			OperationData->bClearCacheLayer1);
		
		OperationData->MutableInstance = Instance;
		
		OperationData->ExtensionData = Instance->ExtensionData;
		
		UE::Tasks::FTask LastTask;
		
		const int32 NumComponents = OperationData->MutableInstance->GetComponentCount();
		
		OperationData->InstanceUpdateData.Components.Reserve(NumComponents);
		for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			TSharedRef<FInstanceUpdateData::FComponent>& Component = OperationData->InstanceUpdateData.Components.Add_GetRef(MakeShared<FInstanceUpdateData::FComponent>());
			UE::Mutable::Private::FComponentId MutableComponentId = Instance->GetComponentId(ComponentIndex);
			Component->Id = MutableComponentId;
			
			Component->SkeletalMeshId = Instance->GetSkeletalMeshId(ComponentIndex);

			if (OperationData->bUseSkeletalMeshCache)
			{
				TStrongObjectPtr<UCustomizableObjectSkeletalMesh> SkeletalMesh = OperationData->Object->GetPrivate()->SkeletalMeshCache.Get(Component->SkeletalMeshId);
				if (SkeletalMesh)
				{
					OperationData->Objects.Emplace(SkeletalMesh);
				}
			}
			
			const uint16 ExecutionOptions = UE::Mutable::Private::SkeletalMeshObjectOptionsPack(true, OperationData->bStreamMeshLODs, MAX_uint8);
			UE::Tasks::TTask<UE::Mutable::Private::FGetSkeletalMeshResult> GetSkeletalMeshTask = OperationData->MutableSystem->GetSkeletalMesh(LastTask, OperationData->LiveInstance.ToSharedRef(), Component->SkeletalMeshId, ExecutionOptions);
			UE::Tasks::AddNested(GetSkeletalMeshTask);
			LastTask = GetSkeletalMeshTask;
			
			UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("GetSkeletalMeshResult"), [=]() mutable
			{
				const UE::Mutable::Private::FGetSkeletalMeshResult Result = GetSkeletalMeshTask.GetResult();
				check(Result.MutableSkeletalMesh);
				Component->SkeletalMesh = Result.MutableSkeletalMesh;
			},
			GetSkeletalMeshTask,
			LowLevelTasks::ETaskPriority::Inherit));

			if (UE::Mutable::Private::FMaterialId MaterialId = Instance->GetOverlayMaterialId(ComponentIndex))
			{
				Component->OverlayMaterialIndex = OperationData->InstanceUpdateData.Materials.Num();
				TSharedRef<FInstanceUpdateData::FMaterial> Material = OperationData->InstanceUpdateData.Materials.Add_GetRef(MakeShared<FInstanceUpdateData::FMaterial>());
				Material->MaterialId = MaterialId;
			}
			
			// OVERLAY MATERIALS
			const int32 OverlayMaterialCount = Instance->GetOverlayMaterialCount(ComponentIndex);
			Component->FirstOverlayMaterial = OperationData->InstanceUpdateData.OverlayMaterials.Num();
			for (int32 MaterialIndex = 0; MaterialIndex < OverlayMaterialCount; ++MaterialIndex)
			{
				if (UE::Mutable::Private::FMaterialId SlotMaterialId = Instance->GetOverlayMaterialId(ComponentIndex, MaterialIndex))
				{
					FInstanceUpdateData::FOverlayMaterial& OverlayMaterial = OperationData->InstanceUpdateData.OverlayMaterials.AddDefaulted_GetRef();
					++Component->OverlayMaterialCount;
					OverlayMaterial.SlotName = Instance->GetOverlayMaterialSlotSlotName(ComponentIndex, MaterialIndex);
					
					OverlayMaterial.MaterialIndex = OperationData->InstanceUpdateData.Materials.Num();
					TSharedRef<FInstanceUpdateData::FMaterial> Material = OperationData->InstanceUpdateData.Materials.Add_GetRef(MakeShared<FInstanceUpdateData::FMaterial>());
					Material->MaterialId = SlotMaterialId;
				}	
			}
			
			// OVERRIDE MATERIALS
			const int32 OverrideMaterialCount = Instance->GetOverrideMaterialCount(ComponentIndex);
			Component->FirstOverrideMaterial = OperationData->InstanceUpdateData.OverrideMaterials.Num();
			for (int32 MaterialIndex = 0; MaterialIndex < OverrideMaterialCount; ++MaterialIndex)
			{
				if (UE::Mutable::Private::FMaterialId MaterialId = Instance->GetOverrideMaterialId(ComponentIndex, MaterialIndex))
				{
					FInstanceUpdateData::FOverrideMaterial& OverrideMaterial = OperationData->InstanceUpdateData.OverrideMaterials.AddDefaulted_GetRef();
					++Component->OverrideMaterialCount;
					OverrideMaterial.SlotName = Instance->GetOverrideMaterialSlotSlotName(ComponentIndex, MaterialIndex);
					
					OverrideMaterial.MaterialIndex = OperationData->InstanceUpdateData.Materials.Num();
					TSharedRef<FInstanceUpdateData::FMaterial> Material = OperationData->InstanceUpdateData.Materials.Add_GetRef(MakeShared<FInstanceUpdateData::FMaterial>());
					Material->MaterialId = MaterialId;
				}	
			}
		}
	}
	
	
	void Task_Mutable_GetLODs(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_GetLODs)

		if (OperationData->bIsUpdateAborted)
		{
			return;	
		}
		
		UE::Tasks::FTask LastTask;

		const int32 NumComponents = OperationData->MutableInstance->GetComponentCount();
		for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
		{
			TSharedRef<FInstanceUpdateData::FComponent>& Component = OperationData->InstanceUpdateData.Components[ComponentIndex];

			const FName ComponentName = OperationData->ComponentNames[Component->Id];
			
			uint8& FirstLODAvailable = OperationData->FirstLODAvailable.FindOrAdd(ComponentName, 0);
			uint8& NumLODsAvailable = OperationData->NumLODsAvailable.FindOrAdd(ComponentName, 0);
			uint8& FirstResidentLOD = OperationData->FirstResidentLOD.FindOrAdd(ComponentName, 0);

			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeletalMesh> SkeletalMesh = Component->SkeletalMesh;
			if (!SkeletalMesh)
			{
				continue;
			}
			
			bool bNoGeometry = false;

			for (int32 LODIndex = SkeletalMesh->FirstLODResident; LODIndex < SkeletalMesh->LODs.Num(); ++LODIndex)
			{
				const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLOD>& LOD = SkeletalMesh->LODs[LODIndex];
				if (!LOD || !LOD->Meshes.Num())
				{
					bNoGeometry = true;
					break;
				}
				
				UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh = LOD->Meshes[0]; // TODO SKMPIN 
				if (!Mesh || !Mesh->GetVertexCount())
				{
					bNoGeometry = true;
					break;
				}
			}
			
			if (bNoGeometry)
			{
				continue;
			}

			FirstLODAvailable = SkeletalMesh->FirstLODAvailable;
			NumLODsAvailable = SkeletalMesh->LODs.Num();
			FirstResidentLOD = SkeletalMesh->FirstLODResident;
			
			Component->FirstLOD = OperationData->InstanceUpdateData.LODs.Num();
			Component->LODCount = NumLODsAvailable;

			// If the LOD is not generated we still add an empty one to keep indexes aligned.
			OperationData->InstanceUpdateData.LODs.Reserve(Component->FirstLOD + Component->LODCount);
			for (int32 Index = OperationData->InstanceUpdateData.LODs.Num(); Index < Component->FirstLOD + Component->LODCount; ++Index)
			{
				OperationData->InstanceUpdateData.LODs.Add(MakeShared<FInstanceUpdateData::FLOD>());
			}
			
			for (int32 LODIndex = FirstLODAvailable; LODIndex < Component->LODCount; ++LODIndex)
			{
				MUTABLE_CPUPROFILER_SCOPE(GetMesh);

				TSharedRef<FInstanceUpdateData::FLOD>& LOD = OperationData->InstanceUpdateData.LODs[Component->FirstLOD + LODIndex];
				LOD->Mesh = Component->SkeletalMesh->LODs[LODIndex]->Meshes[0];
				// TODO SKMPIN
			}
		}
	}

	
	void Task_Mutable_GetImageDescriptors(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_GetImageDescriptros)
		
		if (OperationData->bIsUpdateAborted)
		{
			return;
		}
		
		OperationData->TaskGetImagesTimeStart = FPlatformTime::Seconds();		

		const TSharedPtr<TArray<UE::Mutable::Private::FImageId>> ImagesInThisInstance = MakeShared<TArray<UE::Mutable::Private::FImageId>>();

		UE::Tasks::FTask LastTask;
		for (int32 ImageIndex = 0; ImageIndex < OperationData->InstanceUpdateData.Images.Num(); ++ImageIndex)
		{
			TSharedRef<FInstanceUpdateData::FImage> Image = OperationData->InstanceUpdateData.Images[ImageIndex];

			if (Image->bIsPassThrough)
			{
				continue;
			}

			UE::Tasks::TTask<UE::Mutable::Private::FExtendedImageDesc> GetImageDescTask = OperationData->MutableSystem->GetImageDesc(LastTask, OperationData->LiveInstance.ToSharedRef(), Image->ImageID);
			UE::Tasks::AddNested(GetImageDescTask);
			LastTask = GetImageDescTask;
			
			UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("GetImageDescResult"), [GetImageDescTask, Image]() mutable
			{
				UE::Mutable::Private::FExtendedImageDesc& ImageDesc = GetImageDescTask.GetResult();

				Image->Format = ImageDesc.m_format;
				Image->ConstantImagesNeededToGenerate = MoveTemp(ImageDesc.ConstantImagesNeededToGenerate);

				const UCustomizableObjectSystemPrivate* CustomizableObjectSystemPrivate = UCustomizableObjectSystem::GetInstanceChecked()->GetPrivate();
			
				const uint16 MaxTextureSizeToGenerate = static_cast<uint16>(CustomizableObjectSystemPrivate->MaxTextureSizeToGenerate);
				const uint16 MaxSize = FMath::Max(ImageDesc.m_size[0], ImageDesc.m_size[1]);

				Image->BaseMip = 0;
				if (MaxTextureSizeToGenerate > 0 && MaxSize > MaxTextureSizeToGenerate)
				{
					// Find the reduction factor, and the BaseMip of the texture.
					const uint32 NextPowerOfTwo = FMath::RoundUpToPowerOfTwo(FMath::DivideAndRoundUp(MaxSize, MaxTextureSizeToGenerate));
					uint16 Reduction = FMath::Max(NextPowerOfTwo, 2U); // At least divide the texture by a factor of two
					Image->BaseMip = FMath::FloorLog2(Reduction);
				}

				if (!CVarIgnoreFirstAvailableLODCalculation.GetValueOnAnyThread())
				{
					Image->BaseMip = FMath::Max<uint8>(Image->BaseMip, ImageDesc.FirstLODAvailable);
				}

				Image->FullImageSizeX = ImageDesc.m_size[0] >> Image->BaseMip;
				Image->FullImageSizeY = ImageDesc.m_size[1] >> Image->BaseMip;
			},
			GetImageDescTask,
			UE::Tasks::ETaskPriority::Inherit));
		}
	}

	
	void Task_Mutable_GetImages(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_GetImages)
		
		if (OperationData->bIsUpdateAborted)
		{
			return;
		}
		
		// TODO We should not access UObjects outside GT
		const TStrongObjectPtr<UCustomizableObject> StrongObject = OperationData->Object.Pin();
		if (!IsValid(StrongObject.Get()))
		{
			OperationData->bIsUpdateAborted = true;
			return;
		}
		
		const TSharedPtr<TArray<UE::Mutable::Private::FImageId>> RequestedGetImages = MakeShared<TArray<UE::Mutable::Private::FImageId>>();

		UE::Tasks::FTask LastTask;
		for (int32 ImageIndex = 0; ImageIndex < OperationData->InstanceUpdateData.Images.Num(); ++ImageIndex)
		{
			TSharedRef<FInstanceUpdateData::FImage> Image = OperationData->InstanceUpdateData.Images[ImageIndex];

			if (Image->bIsPassThrough)
			{
				UE::Tasks::TTask<UE::Mutable::Private::FGetImageResult> GetImageTask = OperationData->MutableSystem->GetImage(LastTask, OperationData->LiveInstance.ToSharedRef(), Image->ImageID, 0);
				UE::Tasks::AddNested(GetImageTask);
				LastTask = GetImageTask;
					
				UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("GetImageResult"), [Image, GetImageTask]() mutable
				{
					const UE::Mutable::Private::FGetImageResult Result = GetImageTask.GetResult();
					check(Result.MutableImage);
					Image->Image = Result.MutableImage;
					check(Image->Image->IsReference());
				},
				GetImageTask,
				LowLevelTasks::ETaskPriority::Inherit));
			}
			else
			{
				// Access the CO TextureCache and try to grab the texture from there if found.
				if (OperationData->bUseCaches)
				{
					FTextureCache& TextureCache = StrongObject->GetPrivate()->TextureCache;
					const FTextureCache::FId TextureCacheKey = FTextureCache::FId(Image->ImageID, OperationData->MipsToSkip, OperationData->bBake);

					if (const TStrongObjectPtr<const UTexture> CachedTexture = TextureCache.Get(TextureCacheKey))
					{
						UE_LOGF(LogMutable, VeryVerbose, "Texture resource with id [%ls] is cached.", *Image->ImageID.ToString());
						OperationData->Objects.Add(CachedTexture);
						continue;
					}
				}
				
				// Check if we have already found this ImageID during the runtime of this method
				if (RequestedGetImages->Contains(Image->ImageID))
				{
					continue;
				}

				RequestedGetImages->Add(Image->ImageID);
			
				const int32 MaxSize = FMath::Max(Image->FullImageSizeX, Image->FullImageSizeY);
				const int32 FullLODCount = FMath::CeilLogTwo(MaxSize) + 1;
				const int32 MinMipsInImage = FMath::Min(FullLODCount, UTexture::GetStaticMinTextureResidentMipCount());
				const int32 MaxMipsToSkip = FullLODCount - MinMipsInImage;
				int32 MipsToSkip = FMath::Min(MaxMipsToSkip, OperationData->MipsToSkip);

				if (Image->bIsNonProgressive || !FMath::IsPowerOfTwo(Image->FullImageSizeX) || !FMath::IsPowerOfTwo(Image->FullImageSizeY))
				{
					// It doesn't make sense to skip mips as non-power-of-two size textures cannot be streamed anyway
					MipsToSkip = 0;
				}

				const int32 MipSizeX = FMath::Max(Image->FullImageSizeX >> MipsToSkip, 1);
				const int32 MipSizeY = FMath::Max(Image->FullImageSizeY >> MipsToSkip, 1);

				UE::Tasks::TTask<UE::Mutable::Private::FGetImageResult> GetImageTask = OperationData->MutableSystem->GetImage(LastTask, OperationData->LiveInstance.ToSharedRef(), Image->ImageID, Image->BaseMip + MipsToSkip);
				UE::Tasks::AddNested(GetImageTask);
				LastTask = GetImageTask;

				UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("GetImageResult"), [Image, GetImageTask, MipSizeX, MipSizeY, FullLODCount, MipsToSkip]() mutable
				{
					const UE::Mutable::Private::FGetImageResult Result = GetImageTask.GetResult();
					Image->Image = Result.MutableImage;
					check(Image->Image);

					// We should have generated exactly this size.
					if (const bool bSizeMissmatch = Image->Image->GetSizeX() != MipSizeX || Image->Image->GetSizeY() != MipSizeY)
					{
						// Generate a correctly-sized but empty image instead, to avoid crashes.
						UE_LOGF(LogMutable, Warning, "Mutable generated a wrongly-sized image %ls.", *Image->ImageID.ToString());
						Image->Image = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(MipSizeX, MipSizeY, FullLODCount - MipsToSkip, Image->Image->GetFormat(), UE::Mutable::Private::EInitializationType::Black);
					}

					// We need one mip or the complete chain. Otherwise, there was a bug.
					const int32 FullMipCount = Image->Image->GetMipmapCount(Image->Image->GetSizeX(), Image->Image->GetSizeY());
					const int32 RealMipCount = Image->Image->GetLODCount();

					bool bForceMipchain = 
						// Did we fail to generate the entire mipchain (if we have mips at all)?
						(RealMipCount != 1) && (RealMipCount != FullMipCount);

					if (bForceMipchain)
					{
						MUTABLE_CPUPROFILER_SCOPE(GetImage_MipFix);

						UE_LOGF(LogMutable, Warning, "Mutable generated an incomplete mip chain for image %ls.", *Image->ImageID.ToString());

						// Force the right number of mips. The missing data will be black.
						const UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> NewImage = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(Image->Image->GetSizeX(), Image->Image->GetSizeY(), FullMipCount, Image->Image->GetFormat(), UE::Mutable::Private::EInitializationType::Black);
						check(NewImage);	
						// Formats with BytesPerBlock == 0 will not allocate memory. This type of images are not expected here.
						check(!NewImage->DataStorage.IsEmpty());

						for (int32 L = 0; L < RealMipCount; ++L)
						{
							TArrayView<uint8> DestView = NewImage->DataStorage.GetLOD(L);
							TArrayView<const uint8> SrcView = Image->Image->DataStorage.GetLOD(L);

							check(DestView.Num() == SrcView.Num());
							FMemory::Memcpy(DestView.GetData(), SrcView.GetData(), DestView.Num());
						}
						Image->Image = NewImage;
					}
				},
				GetImageTask,
				UE::Tasks::ETaskPriority::Inherit));
			}
		}
	}


	void Task_Mutable_PrepareTextures(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Subtask_Mutable_PrepareTextures)

		if (OperationData->bIsUpdateAborted)
		{
			return;
		}
		
		for (const TSharedRef<FInstanceUpdateData::FMaterial>& Material : OperationData->InstanceUpdateData.Materials)
		{
			for (int32 ImageIndex = 0; ImageIndex < Material->ImageCount; ++ImageIndex)
			{
				const TSharedRef<FInstanceUpdateData::FImage>& Image = OperationData->InstanceUpdateData.Images[Material->FirstImage + ImageIndex];

				if (Image->Image)
				{
					// Image references are just references to texture assets and require no work at all
					if (!Image->Image->IsReference())
					{
						if (!OperationData->ImageToPlatformDataMap.Contains(Image->ImageID))
						{
							OperationData->ImageToPlatformDataMap.Add(Image->ImageID, MutableCreateImagePlatformData(Image->Image, -1, Image->FullImageSizeX, Image->FullImageSizeY));
						}
						else
						{
							// The ImageID already exists in the ImageToPlatformDataMap, that means the equivalent surface in a lower
							// LOD already created the PlatformData for that ImageID and added it to the ImageToPlatformDataMap.
						}
					}
				}
			}
		}

		OperationData->TaskGetImagesTime = FPlatformTime::Seconds() - OperationData->TaskGetImagesTimeStart;
	}
	

	// This runs in a worker thread.
	void Task_Mutable_EndUpdate(TSharedPtr<UE::Mutable::Private::FLiveInstance> LiveInstance, TSharedPtr<UE::Mutable::Private::FSystem> MutableSystem)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Mutable_ReleaseInstance)
		
		if (LiveInstance && MutableSystem)
		{
			const bool bClearRoms = UCustomizableObjectSystem::ShouldClearWorkingMemoryOnUpdateEnd();
			const bool bFreeCache = true;
			MutableSystem->EndUpdate(LiveInstance.ToSharedRef(), bClearRoms, bFreeCache);
		}
		
		UCustomizableObjectSystem::GetInstance()->GetPrivate()->MutableTaskGraph.AllowLaunchingMutableTaskLowPriority(true, true);
	}
	
	
	void Task_Game_Callbacks(const TSharedRef<FUpdateContextPrivate>& Context)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Game_Callbacks)
		FMutableScopeTimer Timer(Context->TaskCallbacksTime);

		check(IsInGameThread());

		if (Context->bIsUpdateAborted)
		{
			Context->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Context);
			return;
		}
		
		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
		if (!IsValid(System))
		{
			Context->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Context);
			return;
		}
		
		UCustomizableObjectInstance* CustomizableObjectInstance = Context->Instance.Get();
		if (!IsValid(CustomizableObjectInstance))
		{
			Context->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Context);
			return;
		}

		UCustomizableObject* CustomizableObject = Context->Object.Get();
		if (!IsValid(CustomizableObject))
		{
			Context->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Context);
			return;
		}
		
		CustomizableObjectInstance->GetPrivate()->RelevantParameters = MoveTemp(Context->RelevantParameters);
		
		// Actual work
		// TODO MTBL-391: Review This hotfix
		UpdateSkeletalMesh(Context);

		// End Update
		FinishUpdateGlobal(Context);
	}


	void Task_Game_ConvertResources(const TSharedRef<FUpdateContextPrivate>& OperationData)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Game_ConvertResources)
		FMutableScopeTimer Timer(OperationData->TaskConvertResourcesTime);

		check(IsInGameThread());

		if (OperationData->bIsUpdateAborted)
		{
			OperationData->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(OperationData);
			return;
		}
		
		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
		if (!IsValid(System))
		{
			OperationData->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(OperationData);
			return;
		}

		UCustomizableObjectSystemPrivate* SystemPrivate = System->GetPrivate();

		UCustomizableObject* Object = OperationData->Object.Get();
		if (!IsValid(Object))
		{
			OperationData->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(OperationData);
			return;
		}
		
		UCustomizableObjectInstance* Instance = OperationData->Instance.Get();
        if (!IsValid(Instance))
        {
        	OperationData->UpdateResult = EUpdateResult::Error;
        	FinishUpdateGlobal(OperationData);
        	return;
        }
		
		UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();
		
		if (CVarEnableRealTimeMorphTargets.GetValueOnAnyThread())
		{
			// TODO: This subtask should execute before Convert resources in a worker thread but after 
			// Loading resources. For now keep it here.
			Subtask_Mutable_PrepareRealTimeMorphData(OperationData);
		}
		
		// Extension data
		{
			InstancePrivate->ExtensionData.Empty(OperationData->ExtensionData.Num());
			for (UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FExtensionData>& Data : OperationData->ExtensionData)
			{
				if (UObject* PassthroughObject = Data->PassthroughObject.Get())
				{
					InstancePrivate->ExtensionData.Add(PassthroughObject);
				}
			}
		}
		
		// Prepare the component data
		//-------------------------------------------------------------
		{		
			const UModelResources& ModelResources = Object->GetPrivate()->GetModelResourcesChecked();
			
			const int32 NumComponents = OperationData->MutableInstance->GetComponentCount();
			InstancePrivate->ComponentsData.SetNum(NumComponents);

			TArray<FName> ComponentNames;
			ComponentNames.Reserve(NumComponents);

			for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
			{
				const UE::Mutable::Private::FComponentId ComponentId = OperationData->MutableInstance->GetComponentId(ComponentIndex);
				const FName& ComponentName = ModelResources.ComponentNamesPerObjectComponent[ComponentId];

				if (ComponentNames.Contains(ComponentName)) // Components with the same name. Abort.
				{
					UE_LOGF(LogMutable, Error, "Update error. Trying to generate multiple components with the same name: %ls",
						*ComponentName.ToString());
					
					OperationData->UpdateResult = EUpdateResult::Error;
					FinishUpdateGlobal(OperationData);
					return;
				}

				ComponentNames.Add(ComponentName);

				InstancePrivate->ComponentsData[ComponentIndex].ComponentId = ComponentId;
				InstancePrivate->ComponentsData[ComponentIndex].ComponentName = ComponentName;
			}
		}
		
		const bool bSuccess = InstancePrivate->UpdateSkeletalMesh_PostBeginUpdate0(Instance, OperationData);
		if (!bSuccess)
		{
			OperationData->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(OperationData);
			return;
		}
		
		InstancePrivate->BuildResources(OperationData, Instance);

#if WITH_EDITORONLY_DATA
		InstancePrivate->RegenerateImportedModels(OperationData);
#endif
			
		InstancePrivate->PostEditChangePropertyWithoutEditor();
		
		if (FLogBenchmarkUtil::IsBenchmarkingReportingEnabled())
		{
			// Memory used in the context of this the update of mesh
			OperationData->UpdateEndPeakBytes = UE::Mutable::Private::FGlobalMemoryCounter::GetPeak();
			// Memory used in the context of the mesh update + the baseline memory used by mutable when starting the update
			OperationData->UpdateEndRealPeakBytes = OperationData->UpdateEndPeakBytes + OperationData->UpdateStartBytes;
		}
		
		// If they are still in this container it means they have not been used. No longer required. Free some memory now.
		OperationData->ImageToPlatformDataMap.Empty();
		
		// Next Task: Callbacks
		//-------------------------------------------------------------
		SystemPrivate->MutableTaskGraph.AddGameThreadTask(
			TEXT("Task_Game_Callbacks"),
			[OperationData](float RemainingTime)
			{
				Task_Game_Callbacks(OperationData);
			});
	}
	
	
	/** "Start Update" */
	void Task_Game_StartUpdate(const TSharedRef<FUpdateContextPrivate>& Operation)
	{
		MUTABLE_CPUPROFILER_SCOPE(Task_Game_StartUpdate)

		check(IsInGameThread());

		Operation->StartUpdateTime = FPlatformTime::Seconds();

		Operation->UpdateStarted = true;
		TRACE_BEGIN_REGION(UE_MUTABLE_UPDATE_REGION)
		
		// Check if a level has been loaded
		if (FLogBenchmarkUtil::IsBenchmarkingReportingEnabled() && GWorld)
		{
			Operation->bLevelBegunPlay = GWorld->GetBegunPlay();
		}

		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
		if (!IsValid(System))
		{
			Operation->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Operation);
			return;
		}

		UCustomizableObjectSystemPrivate* SystemPrivate = System->GetPrivate();
		
		UCustomizableObject* Object = Operation->Object.Get();
		if (!IsValid(Object))
		{
			Operation->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Operation);
			return;
		}
		
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*Object->GetName()); // So that we know what we are currently updating.
		
		UCustomizableObjectPrivate* ObjectPrivate = Object->GetPrivate();
		
		// If the object is locked (for instance, compiling) we skip any instance update.
		if (ObjectPrivate->bLocked)
		{
			Operation->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Operation);
			return;
		}

		UCustomizableObjectInstance* Instance = Operation->Instance.Get();
		if (!IsValid(Instance)) // Only start if it hasn't been already destroyed (i.e. GC after finish PIE)
		{
			Operation->UpdateResult = EUpdateResult::Error;
			FinishUpdateGlobal(Operation);
			return;
		}

#if WITH_EDITOR
		if (CVarMutableLogObjectMemoryOnUpdate.GetValueOnAnyThread())
		{
			ObjectPrivate->LogMemory();
		}
#endif
		
		const UModelResources& ModelResources = Operation->Object->GetPrivate()->GetModelResourcesChecked();

		UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();
		check(InstancePrivate); // Already checked in GetPrivate. Make static analyzer happy.

		const int32 State = InstancePrivate->GetState();
		const FString& StateName = ObjectPrivate->GetStateName(State);
		const FMutableStateData* StateData = ModelResources.StateUIDataMap.Find(StateName);
		
		// Streaming disabled from platform settings or from platform CustomizableObjectSystem properties?
#if PLATFORM_SUPPORTS_TEXTURE_STREAMING
		if (!System->IsProgressiveMipStreamingEnabled() ||
			!IStreamingManager::Get().IsTextureStreamingEnabled() ||
#if WITH_EDITORONLY_DATA
			ModelResources.bIsTextureStreamingDisabled || // Was streaming disabled at object-compilation time? 
#endif
			Operation->bBake)
		{
			Operation->bNeverStreamMips = true;
		}
		else
		{
			Operation->bNeverStreamMips = StateData ? StateData->bLiveUpdateMode || StateData->bDisableTextureStreaming : false;
		}
#else
		Operation->bNeverStreamMips = true;
#endif
	
		Operation->MipsToSkip = !Operation->bNeverStreamMips ? 255 : 0; // // 0 means generate all mips. 255 means skip all possible mips until only UTexture::GetStaticMinTextureResidentMipCount() are left
		
		Operation->bUseCaches = Operation->bKeepOwnershipOfGeneratedResources && !Operation->bBake && !FLogBenchmarkUtil::IsBenchmarkingReportingEnabled() && CVarEnableCaches.GetValueOnGameThread();
		Operation->bUseSkeletalMeshCache = Operation->bUseCaches && Object->bEnableMeshCache && !Operation->bLiveUpdateMode && CVarEnableMeshCache.GetValueOnGameThread();
		Operation->bUseResueTextureCache = Operation->bUseCaches && Operation->bNeverStreamMips && CVarEnableMutableReuseInstanceTextures.GetValueOnGameThread();
		
		Operation->Model = ObjectPrivate->GetModel().ToSharedRef();

		UE::Tasks::FTask CacheRuntimeTexturesEvent = UE::Tasks::MakeCompletedTask<void>();

#if WITH_EDITOR
		// Async load all Runtime Referenced Textures.
		const TArray<TSoftObjectPtr<UTexture2D>>& RuntimeReferencedTextures = ModelResources.RuntimeReferencedTextures;
		if (!RuntimeReferencedTextures.IsEmpty())
		{
			UE::Tasks::FTaskEvent Event = UE::Tasks::FTaskEvent(TEXT("Texture"));
			CacheRuntimeTexturesEvent = Event;

			TArray<FSoftObjectPath> Textures;
			Textures.Reserve(RuntimeReferencedTextures.Num());
			for (const TSoftObjectPtr<UTexture2D>& Texture : RuntimeReferencedTextures)
			{
				Textures.Add(Texture.ToSoftObjectPath());
			}
		
			SystemPrivate->StreamableManager->RequestAsyncLoad(Textures, FStreamableDelegate::CreateLambda([Operation, Event]() mutable
			{
				UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
				if (!System)
				{
					Event.Trigger();
					return;
				}
				
				UCustomizableObject* Object = Operation->Object.Get();
				if (!Object)
				{
					Event.Trigger();
					return;
				}
			
				const UModelResources& ModelResources = Object->GetPrivate()->GetModelResourcesChecked();
				Operation->ExternalResourceProvider->CacheRuntimeReferencedImages(ModelResources.RuntimeReferencedTextures);
				Event.Trigger();
			}));
		}
#endif
		
		// CreateMutableInstance
		{
			if (FLogBenchmarkUtil::IsBenchmarkingReportingEnabled())
			{
				// Get the amount of mutable memory in use now
				Operation->UpdateStartBytes = UE::Mutable::Private::FGlobalMemoryCounter::GetAbsoluteCounter();
				// Reset the counter to later get the peak during the updated
				UE::Mutable::Private::FGlobalMemoryCounter::Zero();													
			}

			// Prepare streaming for the current customizable object
			check(SystemPrivate->Streamer != nullptr);
			SystemPrivate->Streamer->PrepareStreamingForObject(Operation->Object.Get());			

			Operation->bLowPriorityTasksBlocked = true;
			SystemPrivate->MutableTaskGraph.AllowLaunchingMutableTaskLowPriority(false, false);

			const TSharedPtr<UE::Mutable::Private::FSystem> MutableSystem = SystemPrivate->MutableSystem;
			
			Operation->ExternalResourceProvider = MakeShared<FUnrealMutableResourceProvider>();
			Operation->PassthroughObjectLoader = MakeShared<UE::Mutable::Private::FPassthroughObjectLoader>();

			Operation->bLiveUpdateMode = SystemPrivate->EnableMutableLiveUpdate && (StateData ? StateData->bLiveUpdateMode : false);
		
			Operation->bStreamMeshLODs = IsStreamingEnabled(*Object, State) && !Operation->bLiveUpdateMode && !Operation->bBake;

			TSharedPtr<UE::Mutable::Private::FLiveInstanceLogger> UpdateInstanceLogger = MakeShared<UE::Mutable::Private::FLiveInstanceLogger>();
			UpdateInstanceLogger->SetLogger(InstancePrivate->UpdateLogger);

			Operation->Parameters = Operation->GetCapturedDescriptor().GetParameters();

			// Ensure that in the case of running with the live update enabled we have mesh and mip streaming disabled
			check(!Operation->bLiveUpdateMode || Operation->bNeverStreamMips);
			check(!Operation->bLiveUpdateMode || Operation->MipsToSkip == 0);
			check(!Operation->bLiveUpdateMode || !Operation->bStreamMeshLODs);
			
			Operation->LiveInstance = MutableSystem->NewInstance(
				Operation->Model,
				ObjectPrivate->ImageIdRegistry,
				ObjectPrivate->MaterialIdRegistry,
				ObjectPrivate->SkeletalMeshIdRegistry,
				Operation->ExternalResourceProvider,
				Operation->PassthroughObjectLoader,
				Operation->PixelFormatOverride,
				UpdateInstanceLogger);

			Operation->LiveInstance->Parameters = Operation->Parameters;
			Operation->LiveInstance->State = Operation->GetCapturedDescriptor().GetState();
			
			if (Operation->bLiveUpdateMode)
			{
				if (Instance->GetPrivate()->LiveInstance.IsValid())
				{
					// If we allow the cache to be reused then grab the one from the old instance and prepare it for this one
					// Reuse the cache of the live instance that was added during a previous update
					Operation->LiveInstance->Cache = Instance->GetPrivate()->LiveInstance->Cache;
					
					const TSharedPtr<const UE::Mutable::Private::FParameters> PreviousParameters = Instance->GetPrivate()->LiveInstance->Parameters;
					const int32 PreviousState = Instance->GetPrivate()->LiveInstance->State;
					
					Operation->bClearCacheLayer1 = PreviousState != Operation->LiveInstance->State || 
						UE::Mutable::Private::FSystem::CheckForUpdatedParameters(Operation->LiveInstance.ToSharedRef(), PreviousParameters);
				}
				
				Instance->GetPrivate()->LiveInstance = Operation->LiveInstance;
			}
			else
			{
				Instance->GetPrivate()->LiveInstance = nullptr;
			}
			
			MutableSystem->BeginUpdate_GameThread(
				Operation->LiveInstance.ToSharedRef(),
				UE::Mutable::Private::FSystem::AllLODs);
		}
		
		TArray<	UE::Tasks::FTask> Prerequisites;
		Prerequisites.Add(CacheRuntimeTexturesEvent);
		
#if WITH_EDITOR
		// Add to the prerequisites of the operation the compilation of all the Skeletal Mesh Parameters
		// Note : We are accessing the internal parameters (no the ones on the COI) as those do change in a later stage and may not yet have the SKM parameters in them
		// at this point.
		{
			const int32 ParameterCount = Operation->Parameters->Values.Num();
			for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
			{
				auto AddToPrerequisites = [SystemPrivate, &Prerequisites](USkeletalMesh* SkeletalMesh) -> void
				{
					// If the skeletal mesh is compiling wait for it to be done before proceeding
					if (SkeletalMesh && SkeletalMesh->IsCompiling())
					{
						UE::Tasks::FTaskEvent Event { UE_SOURCE_LOCATION };
						Prerequisites.Add(Event);

						SystemPrivate->MutableTaskGraph.AddGameThreadTask(TEXT("FinishMeshCompilation"), 
						[
							Event, 
							SkeletalMesh = TStrongObjectPtr(SkeletalMesh)
						](float) mutable
						{
							FSkinnedAssetCompilingManager::Get().FinishCompilation({ SkeletalMesh.Get() });
							Event.Trigger();
						});
					}
				};

				if (!Operation->Parameters->HasMultipleValues(ParameterIndex))
				{
					// Add the non ranged value
					UE::Mutable::Private::FParameterValue& ParameterValue = Operation->Parameters->Values[ParameterIndex];
					if (TStrongObjectPtr<USkeletalMesh>* SkeletalMeshParameterValue = ParameterValue.TryGet<UE::Mutable::Private::FParamSkeletalMeshType>())
					{
						AddToPrerequisites(SkeletalMeshParameterValue->Get());
					}
				}
				else
				{
					const TMap<TArray<int32>, UE::Mutable::Private::FParameterValue>& MultiValueParameterData = Operation->Parameters->MultiValues[ParameterIndex];

					TArray<UE::Mutable::Private::FParameterValue> MultiDimensionalParameterValues;
					MultiValueParameterData.GenerateValueArray(MultiDimensionalParameterValues);
				
					for (UE::Mutable::Private::FParameterValue& MultiDimensionalParameterValue : MultiDimensionalParameterValues)
					{
						if (TStrongObjectPtr<USkeletalMesh>* MultiDimensionalSkeletalMeshParameterValue = MultiDimensionalParameterValue.TryGet<UE::Mutable::Private::FParamSkeletalMeshType>())
						{
							AddToPrerequisites(MultiDimensionalSkeletalMeshParameterValue->Get());
						}
					}
				}
			}
		}
#endif
				
		UE::Tasks::FTask Mutable_GetSkeletalMeshesTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_GetSkeletalMeshes"),
		[Operation]()
		{
			Task_Mutable_GetSkeletalMeshes(Operation);
		},
		{ Prerequisites});
		
		UE::Tasks::FTask Mutable_GetLODsTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_GetLODs"),
		[Operation]()
		{
			Task_Mutable_GetLODs(Operation);
		},
		{ Mutable_GetSkeletalMeshesTask });

		UE::Tasks::FTask Mutable_GetSurfacesTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_GetSurfaces"),
		[Operation]()
		{
			Task_Mutable_GetSurfaces(Operation);
		},
		{ Mutable_GetLODsTask });
		
		UE::Tasks::FTask Mutable_GetMaterialsTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_GetMaterials"),
		[Operation]()
		{
			Task_Mutable_GetMaterials(Operation);
		},
		{ Mutable_GetSurfacesTask });

		UE::Tasks::FTask Mutable_PrepareSkeletonDataTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_PrepareSkeletonData"),
		[Operation]()
		{
			Task_Mutable_PrepareSkeletonData(Operation);
		},
		{ Mutable_GetMaterialsTask });

		UE::Tasks::FTask Mutable_UpdateParameterRelevancy = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_UpdateParameterRelevancy"),
		[Operation]()
		{
			Task_Mutable_UpdateParameterRelevancy(Operation);
		},
		{Mutable_PrepareSkeletonDataTask});
		
		UE::Tasks::FTask Mutable_GetImageDescriptors_Task = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_GetImageDescriptors"),
		[Operation]()
		{
			Task_Mutable_GetImageDescriptors(Operation);
		},
		{Mutable_UpdateParameterRelevancy});	

		UE::Tasks::FTask Mutable_GetImagesTask = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_GetImages"),
		[Operation]()
		{
			Task_Mutable_GetImages(Operation);
		},
		{Mutable_GetImageDescriptors_Task});

		UE::Tasks::FTask Mutable_PrepareTextures = SystemPrivate->MutableTaskGraph.AddMutableThreadTask(
		TEXT("Task_Mutable_PrepareTextures"),
		[Operation]()
		{
			Task_Mutable_PrepareTextures(Operation);
		},
		{Mutable_GetImagesTask});

		// Next Task: Load Unreal Assets
		//-------------------------------------------------------------
		UE::Tasks::FTask Game_LoadUnrealAssets = LoadAdditionalAssetsAndData(Operation);

		// Next-next Task: Convert Resources
		//-------------------------------------------------------------
		SystemPrivate->MutableTaskGraph.AddGameThreadTask(
		TEXT("Task_Game_ConvertResources"),
		[Operation](float RemainingTime)
		{
			Task_Game_ConvertResources(Operation);
		},
		{Game_LoadUnrealAssets, Mutable_PrepareTextures});
	}
} // namespace impl


void UCustomizableObjectSystemPrivate::UpdateTick(const bool bBlocking)
{
	TickPendingSetReferenceSkeletalMesh();

	bool bPendingCompilation = false;
#if WITH_EDITOR
	ICustomizableObjectEditorModule* EditorModule = ICustomizableObjectEditorModule::Get();
	bPendingCompilation = EditorModule && EditorModule->GetNumCompileRequests() > 0;
#endif
	
	TickDiscardInstances();

	// Get a new operation if we aren't working on one
	if (!CurrentMutableOperation && bIsMutableEnabled && !bPendingCompilation)
	{
		bool bUpdatingInstance = false;
		{
			MUTABLE_CPUPROFILER_SCOPE(UpdateTick_Queue);

			while (!bUpdatingInstance)
			{
				FMutableInstanceUpdate InstanceUpdate;
				if (!HighPriorityInstanceUpdates.TryPopFirst(InstanceUpdate))
				{
					break;
				}

				if (InstanceUpdate)
				{
					bUpdatingInstance = TryStartUpdate(InstanceUpdate.ToSharedRef());
				}
				--PendingInstanceUpdates;
			}

			while (!bUpdatingInstance)
			{
				FMutableInstanceUpdate InstanceUpdate;
				if (!LowPriorityInstanceUpdates.TryPopFirst(InstanceUpdate))
				{
					break;
				}

				if (InstanceUpdate)
				{
					bUpdatingInstance = TryStartUpdate(InstanceUpdate.ToSharedRef());
				}
				--PendingInstanceUpdates;
			}
		}

		if (!bUpdatingInstance)
		{
			TickAutomaticUpdateInstances();
		}

		// Update the streaming limit if it has changed. It is safe to do this now.
		UpdateMemoryLimit();
	}
	
	LogBenchmarkUtil.UpdateStats(); // Must be the last thing to perform

	if (!bIsMutableEnabled && !CurrentMutableOperation)
	{
		FStreamingManagerCollection::Get().RemoveStreamingManager(this);
	}
}


void UCustomizableObjectSystemPrivate::MainTick(const bool bBlocking)
{
	MUTABLE_CPUPROFILER_SCOPE(UCustomizableObjectSystem::Tick)

	check(IsInGameThread());
	
	// Building instances is not enabled in servers. If at some point relevant collision or animation data is necessary for server logic this will need to be changed.
#if UE_SERVER
	return;
#else // !UE_SERVER

	if (!IsValid(GetPublic()))
	{
		return;
	}

	if (IsEngineExitRequested())
	{	
		// TODO BEGIN. Remove once UE-281921  
		MutableTaskGraph.AllowLaunchingMutableTaskLowPriority(true, false);
		// TODO END

		return;
	}
	
	// Do not tick if the CookCommandlet is running.
	if (!IsRunningCookCommandlet())
	{
		UpdateTick(bBlocking);
	}

	MutableTaskGraph.Tick();

	if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		Module->Tick(bBlocking);
	}
	
	StreamableManager->Tick(bBlocking);
#endif // !UE_SERVER
}


int32 UCustomizableObjectSystemPrivate::GetRemainingWork() const
{
#if UE_SERVER
	return 0;
#else // !UE_SERVER
	
	if (!IsValid(GetPublic()))
	{
		return 0;
	}

	if (IsEngineExitRequested())
	{
		return 0;
	}

	int32 RemainingWork = 0;

	// Do not tick if the CookCommandlet is running.
	if (!IsRunningCookCommandlet())
	{
		RemainingWork += CurrentMutableOperation.IsValid() + PendingInstanceUpdates;
	}
	
	RemainingWork += MutableTaskGraph.GetRemainingWork();

	if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		RemainingWork += Module->GetRemainingWork();
	}
	
	RemainingWork += StreamableManager->GetRemainingWork();
	
	return RemainingWork;
#endif // !UE_SERVER
}


bool UCustomizableObjectSystem::IsUpdating(const UCustomizableObjectInstance* Instance) const
{
	if (!Instance)
	{
		return false;
	}
	
	return GetPrivate()->IsUpdating(*Instance);
}

TArray<UCustomizableObjectInstance*> UCustomizableObjectSystemPrivate::CustomizableObjectInstances;

void UCustomizableObjectSystemPrivate::RegisterCustomizableObjectInstance(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		check(!UCustomizableObjectSystemPrivate::CustomizableObjectInstances.Contains(&Obj));
		UCustomizableObjectSystemPrivate::CustomizableObjectInstances.Add(&Obj);
	}
}


void UCustomizableObjectSystemPrivate::UnregisterCustomizableObjectInstance(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		UCustomizableObjectSystemPrivate::CustomizableObjectInstances.RemoveSingleSwap(&Obj);
		UCustomizableObjectSystemPrivate::PendingSetReferenceSkeletalMesh.RemoveSingleSwap(&Obj);
		UCustomizableObjectSystemPrivate::GeneratedInstances.RemoveSingleSwap(&Obj);
		UCustomizableObjectSystemPrivate::AutomaticUpdateInstances.RemoveSingleSwap(&Obj);
	}
}

TArray<UCustomizableObjectInstance*> UCustomizableObjectSystemPrivate::PendingSetReferenceSkeletalMesh;
TArray<UCustomizableObjectInstance*> UCustomizableObjectSystemPrivate::GeneratedInstances;
TArray<UCustomizableObjectInstance*> UCustomizableObjectSystemPrivate::AutomaticUpdateInstances;

void UCustomizableObjectSystemPrivate::RegisterInstanceToPendinSetReferenceSkeletalMeshList(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		UCustomizableObjectSystemPrivate::PendingSetReferenceSkeletalMesh.AddUnique(&Obj);
	}
}


void UCustomizableObjectSystemPrivate::RegisterInstanceToGeneratedList(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		UCustomizableObjectSystemPrivate::GeneratedInstances.AddUnique(&Obj);
	}
}


void UCustomizableObjectSystemPrivate::UnregisterInstancefromGeneratedList(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		UCustomizableObjectSystemPrivate::GeneratedInstances.RemoveSingleSwap(&Obj);
	}
}


void UCustomizableObjectSystemPrivate::RegisterInstanceToAutomaticUpdateList(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		UCustomizableObjectSystemPrivate::AutomaticUpdateInstances.AddUnique(&Obj);
	}
}


void UCustomizableObjectSystemPrivate::UnregisterInstanceFromAutomaticUpdateList(UCustomizableObjectInstance& Obj)
{
	if (!Obj.IsTemplate())
	{
		UCustomizableObjectSystemPrivate::AutomaticUpdateInstances.RemoveSingleSwap(&Obj);
	}
}


void UCustomizableObjectSystemPrivate::TickPendingSetReferenceSkeletalMesh()
{
	MUTABLE_CPUPROFILER_SCOPE(TickPendingSetReferenceSkeletalMesh);

	for (int32 Index = PendingSetReferenceSkeletalMesh.Num() - 1; Index >= 0; --Index)
	{
		UCustomizableObjectInstance* Instance = PendingSetReferenceSkeletalMesh.Pop(EAllowShrinking::No);
		if (!IsValid(Instance))
		{
			continue;
		}

		UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();

		UCustomizableObject* Object = Instance->GetCustomizableObject();
		if (!Object || !Object->bEnableUseRefSkeletalMeshAsPlaceholder)
		{
			continue;
		}

		for (UCustomizableObjectInstanceUsage* InstanceUsage : InstancePrivate->InstanceUsages)
		{
			if (!IsValid(InstanceUsage))
			{
				continue;
			}

			UCustomizableObjectInstanceUsagePrivate* InstanceUsagePrivate = InstanceUsage->GetPrivate();

			if (!InstanceUsagePrivate->bPendingSetReferenceSkeletalMesh)
			{
				continue;
			}

			if (InstanceUsage->GetSkipSetReferenceSkeletalMesh())
			{
				continue;
			}

			USkeletalMeshComponent* Parent = Cast<USkeletalMeshComponent>(InstanceUsage->GetAttachParent());
			if (!Parent)
			{
				continue;
			}

			const FName& ComponentName = InstanceUsage->GetComponentName();
			if (USkeletalMesh* ReferenceSkeletalMesh = Object->GetComponentMeshReferenceSkeletalMesh(ComponentName))
			{
				Parent->EmptyOverrideMaterials();
				Parent->SetSkeletalMesh(ReferenceSkeletalMesh);
			}

			InstanceUsagePrivate->bPendingSetReferenceSkeletalMesh = false;
		}
	}

	PendingSetReferenceSkeletalMesh.Shrink();
}


void UCustomizableObjectSystemPrivate::TickDiscardInstances()
{
	if (!bGenerateInstancesWithinRange)
	{
		return;
	}

	MUTABLE_CPUPROFILER_SCOPE(TickDiscardInstances);

	UpdateViewLocations();

	int32 MaxNumInstancesToDiscard = CVarMaxNumInstancesToDiscardPerTick.GetValueOnGameThread();

	constexpr int32 NumInstancesPerTick = 25;
	const int32 NumInstances = GeneratedInstances.Num();
	int32 NumRemainingInstances = FMath::Min(NumInstances, NumInstancesPerTick);

	while (NumRemainingInstances > 0)
	{
		--NumRemainingInstances;

		DiscardIndex = DiscardIndex < GeneratedInstances.Num() ? DiscardIndex : 0;

		UCustomizableObjectInstance* Instance = GeneratedInstances[DiscardIndex];
		++DiscardIndex;

		if (!IsValid(Instance))
		{
			continue;
		}

		if (IsDiscardedByDistance(*Instance))
		{
			DiscardInstance(*Instance);

			GeneratedInstances.RemoveSingleSwap(Instance, EAllowShrinking::No);
			--DiscardIndex;

			AutomaticUpdateInstances.AddUnique(Instance);

			if (--MaxNumInstancesToDiscard == 0)
			{
				break;
			}
		}
	}

	if (GeneratedInstances.Num() < (GeneratedInstances.GetAllocatedSize() / 2) / GeneratedInstances.GetTypeSize())
	{
		GeneratedInstances.Shrink();
	}
}


void UCustomizableObjectSystemPrivate::TickAutomaticUpdateInstances()
{
	MUTABLE_CPUPROFILER_SCOPE(TickAutomaticUpdateInstances);

	constexpr int32 NumInstancesPerTick = 50;
	
	const int32 NumInstances = AutomaticUpdateInstances.Num();
	int32 NumRemainingInstances = FMath::Min(NumInstances, NumInstancesPerTick);
	
	while (NumRemainingInstances > 0)
	{
		--NumRemainingInstances;
		AutomaticUpdateIndex = AutomaticUpdateIndex < AutomaticUpdateInstances.Num() ? AutomaticUpdateIndex : 0;

		UCustomizableObjectInstance* Instance = AutomaticUpdateInstances[AutomaticUpdateIndex];
		++AutomaticUpdateIndex;

		if (!IsValid(Instance))
		{
			continue;
		}

		UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();

		const bool bNeedsAutomaticUpdate = InstancePrivate->SkeletalMeshStatus != ESkeletalMeshStatus::Error &&
			!IsDiscardedByDistance(*Instance);

		if (bNeedsAutomaticUpdate && TryStartAutomaticUpdateForInstance(*Instance))
		{
			AutomaticUpdateInstances.RemoveSingleSwap(Instance, EAllowShrinking::No);
			--AutomaticUpdateIndex;
			break;
		}
	}

	if (AutomaticUpdateInstances.Num() < (AutomaticUpdateInstances.GetAllocatedSize() / 2) / AutomaticUpdateInstances.GetTypeSize())
	{
		AutomaticUpdateInstances.Shrink();
	}
}


bool UCustomizableObjectSystemPrivate::IsUsingBenchmarkingSettings()
{
	return bUseBenchmarkingSettings;
}


void UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(bool bUseBenchmarkingOptimizedSettings)
{
	bUseBenchmarkingSettings = bUseBenchmarkingOptimizedSettings;
}



int32 UCustomizableObjectSystem::GetNumInstances() const
{
	int32 NumInstances;
	int32 NumBuiltInstances;
	int32 NumAllocatedSkeletalMeshes;
	GetPrivate()->LogBenchmarkUtil.GetInstancesStats(NumInstances, NumBuiltInstances, NumAllocatedSkeletalMeshes);

	return NumBuiltInstances;
}


int32 UCustomizableObjectSystem::GetNumPendingInstances() const
{
	return GetPrivate()->PendingInstanceUpdates;
}


int32 UCustomizableObjectSystem::GetTotalInstances() const
{
	return UCustomizableObjectSystemPrivate::GetCustomizableObjectInstances().Num();
}


int64 UCustomizableObjectSystem::GetTextureMemoryUsed() const
{
	return GetPrivate()->LogBenchmarkUtil.TextureGPUSize.GetValue();
}


int32 UCustomizableObjectSystem::GetAverageBuildTime() const
{
	return GetPrivate()->LogBenchmarkUtil.InstanceBuildTimeAvrg.GetValue() * 1000;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool UCustomizableObjectSystem::IsSupport16BitBoneIndexEnabled() const
{
	return true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


bool UCustomizableObjectSystem::IsProgressiveMipStreamingEnabled() const
{
	return GetPrivate()->EnableMutableProgressiveMipStreaming != 0;
}


void UCustomizableObjectSystem::SetProgressiveMipStreamingEnabled(bool bIsEnabled)
{
	GetPrivate()->EnableMutableProgressiveMipStreaming = bIsEnabled ? 1 : 0;
}


void UCustomizableObjectSystem::SetGenerateInstancesWithinRange(bool bActive)
{
	GetPrivate()->bGenerateInstancesWithinRange = bActive;
}


void UCustomizableObjectSystem::SetInstanceGenerationRange(float GenerationRange)
{
	GetPrivate()->GenerationRangeSquare = GenerationRange * GenerationRange;
}


void UCustomizableObjectSystem::AddViewCenter(const AActor* const InViewCenter)
{
	if (IsValid(InViewCenter))
	{
		GetPrivate()->ViewCenters.Add(InViewCenter);
	}
}


void UCustomizableObjectSystem::RemoveViewCenter(const AActor* const InViewCenter)
{
	if (IsValid(InViewCenter))
	{
		GetPrivate()->ViewCenters.Remove(InViewCenter);
	}
}


void UCustomizableObjectSystem::ClearViewCenters()
{
	GetPrivate()->ViewCenters.Empty();
}


void UCustomizableObjectSystem::AddUncompiledCOWarning(const UCustomizableObject& InObject, FString const* OptionalLogInfo)
{
	FString Msg;
	Msg += FString::Printf(TEXT("Warning: Customizable Object [%s] not loaded or compiled."), *InObject.GetName());

#if WITH_EDITOR
	// Mutable will spam these warnings constantly due to the tick and LOD manager checking for instances to update with every tick. Send only one message per CO in the editor.
	if (GetPrivate()->UncompiledCustomizableObjectIds.Find(InObject.GetPrivate()->GetVersionId()) != INDEX_NONE)
	{
		return;
	}
	
	// Add notification
	GetPrivate()->UncompiledCustomizableObjectIds.Add(InObject.GetPrivate()->GetVersionId());

	FMessageLog MessageLog("Mutable");
	MessageLog.Warning(FText::FromString(Msg));

	if (!GetPrivate()->UncompiledCustomizableObjectsNotificationPtr.IsValid())
	{
		FNotificationInfo Info(FText::FromString("Customizable Object/s not loaded or compiled. Please, check the Message Log - Mutable for more information."));
		Info.bFireAndForget = true;
		Info.bUseThrobber = true;
		Info.FadeOutDuration = 1.0f;
		Info.ExpireDuration = 5.0f;

		GetPrivate()->UncompiledCustomizableObjectsNotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	}

	const FString ErrorString = FString::Printf(
		TEXT("Customizable Object [%s] not loaded or not compiled. Compile via the editor or via code before instancing.  %s"),
		*InObject.GetName(), OptionalLogInfo ? **OptionalLogInfo : TEXT(""));

#else // !WITH_EDITOR
	const FString ErrorString = FString::Printf(
		TEXT("Customizable Object [%s] not loaded or compiled. This is not an Editor build, so this is an unrecoverable bad state; could be due to code or a cook failure.  %s"),
		*InObject.GetName(), OptionalLogInfo ? **OptionalLogInfo : TEXT(""));
#endif

	// Also log an error so if this happens as part of a bug report we'll have this info.
	UE_LOGF(LogMutable, Error, "%ls", *ErrorString);
}


void UCustomizableObjectSystem::EnableBenchmark()
{
	// Start reporting benchmarking data (log and .csv file)
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(true);
}


void UCustomizableObjectSystem::EndBenchmark()
{
	// Stop the reporting of benchmarking data
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(false);
}


bool UCustomizableObjectSystem::ShouldClearWorkingMemoryOnUpdateEnd()
{
	return UCustomizableObjectSystemPrivate::IsUsingBenchmarkingSettings() ? true : CVarClearWorkingMemoryOnUpdateEnd.GetValueOnAnyThread();
}


void UCustomizableObjectSystem::SetWorkingMemory(int32 KBytes)
{
	CVarWorkingMemoryKB->Set( KBytes );
	UE_LOGF(LogMutable, Log, "Working Memory set to %i kilobytes.", KBytes);
}


int32 UCustomizableObjectSystem::GetWorkingMemory() const
{
	return UCustomizableObjectSystemPrivate::IsUsingBenchmarkingSettings() ? 16384 : CVarWorkingMemoryKB->GetInt();
}


void UCustomizableObjectSystemPrivate::OnMutableEnabledChanged(IConsoleVariable* MutableEnabled)
{
	if (!UCustomizableObjectSystem::IsCreated())
	{
		return;
	}

	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
	UCustomizableObjectSystemPrivate* SystemPrivate = System->GetPrivate();

	if (bIsMutableEnabled)
	{
#if !UE_SERVER
		FStreamingManagerCollection::Get().RemoveStreamingManager(SystemPrivate); // Avoid being added twice
        FStreamingManagerCollection::Get().AddStreamingManager(SystemPrivate);

		if (!SystemPrivate->TickWarningsDelegateHandle.IsValid())
		{
			SystemPrivate->TickWarningsDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&TickWarnings), OnScreenWarningsTickerTime);
		}
#endif // !UE_SERVER
	}
}


bool UCustomizableObjectSystemPrivate::TryStartUpdate(const TSharedRef<FUpdateContextPrivate>& Context)
{
	MUTABLE_CPUPROFILER_SCOPE(TryStartUpdate);

	check(!CurrentMutableOperation); // Can not start an update if there is already another in progress

	UCustomizableObjectInstance* Instance = Context->Instance.Get();
	if (!CanUpdate(Instance))
	{
		Context->UpdateResult = EUpdateResult::Error;
		FinishUpdateGlobal(Context);
		return false;
	}

	UCustomizableInstancePrivate* InstancePrivate = Instance->GetPrivate();
	InstancePrivate->ClearCOInstanceFlags(ECOInstanceFlags::HasPendingHighPriorityUpdate | ECOInstanceFlags::HasPendingLowPriorityUpdate);

	// Only update resources if the instance is in range (it could have got far from the player since the task was queued)
	if (IsDiscardedByDistance(*Instance))
	{
		Context->UpdateResult = EUpdateResult::ErrorDiscarded;
		FinishUpdateGlobal(Context);
		return false;
	}

	// Skip update, the requested update is equal to the running update.
	if (Context->GetCapturedDescriptorHash() == InstancePrivate->CommittedDescriptorHash)
	{
		if (CVarEnableUpdateOptimization.GetValueOnGameThread()) // TODO Remove hotfix: UE-218957 
		{
			Context->bOptimizedUpdate = true;

			// The user may have changed the AttachParent and we need to re-customize it.
			// In case nothing need to be re-customized, the update will be considered ErrorOptimized.
			UpdateSkeletalMesh(Context);
			Context->UpdateResult = Context->AttachedParentUpdated.IsEmpty() ? EUpdateResult::ErrorOptimized : EUpdateResult::Success;

			FinishUpdateGlobal(Context);
			return true;
		}
		else
		{
			UpdateSkeletalMesh(Context);
			FinishUpdateGlobal(Context);
			return true;
		}
	}
	
	StartUpdate(Context);

	return true;
}


bool UCustomizableObjectSystemPrivate::TryStartAutomaticUpdateForInstance(UCustomizableObjectInstance& Instance)
{
	MUTABLE_CPUPROFILER_SCOPE(TryStartAutomaticUpdateForInstance);

	TryAutomaticCompilation(&Instance);
	
	check(!CurrentMutableOperation); // Can not start an update if there is already another in progress

	if (!CanUpdate(&Instance))
	{
		return false;
	}
	
	const TSharedRef<FUpdateContextPrivate> Context = MakeShared<FUpdateContextPrivate>();
	Context->Instance = &Instance;
	Context->Init(true);
	
	if (Context->GetCapturedDescriptorHash() == Instance.GetPrivate()->CommittedDescriptorHash)
	{
		Context->bOptimizedUpdate = true;

		// The user may have changed the AttachParent and we need to re-customize it.
		// In case nothing need to be re-customized, the update will be considered ErrorOptimized.
		UpdateSkeletalMesh(Context);
		Context->UpdateResult = Context->AttachedParentUpdated.IsEmpty() ? EUpdateResult::ErrorOptimized : EUpdateResult::Success;

		FinishUpdateGlobal(Context);
		return true;
	}
	
	StartUpdate(Context);

	return true;
}


void UCustomizableObjectSystemPrivate::StartUpdate(const TSharedRef<FUpdateContextPrivate>& Context)
{
	if (FPlatformTime::Seconds() > LogStartedUpdateUnmute)
	{
		MUTABLE_CPUPROFILER_SCOPE(LogStartedUpdate);

		UCustomizableObjectInstance* Instance = Context->Instance.Get();
		const UCustomizableObject* Object = Context->Object.Get();
		const FName AssetName = !Instance->HasAnyFlags(RF_Transient) ? Instance->GetFName() : Object->GetFName();

		FString Message = FString::Printf(TEXT("Started Update Skeletal Mesh Async. CO/COI = %s"), *AssetName.ToString());
		LogUpdateMessage(Message, ELogVerbosity::Log, Instance);

		const float CurrentTime = FPlatformTime::Seconds();
		constexpr float LogInterval = 1.0 / 2.0; // Allow maximum 2 logs per second.
		const bool bMute = CurrentTime - LogStartedUpdateLast < LogInterval;
		LogStartedUpdateLast = CurrentTime;

		if (bMute)
		{
			UE_LOGF(LogMutable, Log, "Disabling \"Started Update Skeletal Mesh Async\" log during 5 seconds due to spam");
			LogStartedUpdateUnmute = CurrentTime + 5.0;
		}
	}

	// It is safe to do this now.
	UpdateMemoryLimit();

	CurrentMutableOperation = Context;

	MutableTaskGraph.AddGameThreadTask(
		TEXT("Task_Game_StartUpdate"),
		[Context](float RemainingTime)
		{
			impl::Task_Game_StartUpdate(Context);
		},
		{ LastUpdateMutableTask });
}


bool UCustomizableObjectSystemPrivate::IsUpdating(const UCustomizableObjectInstance& Instance) const
{
	if (CurrentMutableOperation && CurrentMutableOperation->Instance.Get() == &Instance)
	{
		return true;
	}
	
	return false;
}


void UCustomizableObjectSystemPrivate::UpdateResourceStreaming(float DeltaTime, bool bProcessEverything)
{
	MainTick(false);
}


int32 UCustomizableObjectSystemPrivate::BlockTillAllRequestsFinished(float TimeLimit, bool bLogResults)
{
	const double BlockEndTime = FPlatformTime::Seconds() + TimeLimit;

	int32 LocalRemainingWork = TNumericLimits<int32>::Max();
	
	if (TimeLimit == 0.0f)
	{
		while (LocalRemainingWork > 0)
		{
			MainTick(true);
			LocalRemainingWork = GetRemainingWork();
		}
	}
	else
	{
		while (LocalRemainingWork > 0)
		{			
			if (FPlatformTime::Seconds() > BlockEndTime)
			{
				return LocalRemainingWork;
			}
			
			MainTick(true);
			LocalRemainingWork = GetRemainingWork();
		}
	}

	return 0;
}


void UCustomizableObjectSystemPrivate::LogUpdateMessage(const FString& Message, ELogVerbosity::Type Verbosity, UCustomizableObjectInstance* Instance, bool bClearMessageList)
{
	switch (Verbosity)
	{
	case ELogVerbosity::Error:
		UE_LOGF(LogMutable, Error, "%ls", *Message);
		break;
	case ELogVerbosity::Warning:
		UE_LOGF(LogMutable, Warning, "%ls", *Message);
		break;
	case ELogVerbosity::Verbose:
		UE_LOGF(LogMutable, Verbose, "%ls", *Message);
		break;
	case ELogVerbosity::Display:
		UE_LOGF(LogMutable, Verbose, "%ls", *Message);
		break;
	case ELogVerbosity::Log:
	default:
		UE_LOGF(LogMutable, Log, "%ls", *Message);
		break;
	}

	if (Instance && Instance->GetPrivate()->UpdateLogger)
	{
		if (bClearMessageList)
		{
			Instance->GetPrivate()->UpdateLogger->ClearLogMessageList();
		}

		// Translation from ELogVerbosity to EMessageSeverity
		EMessageSeverity::Type MessageSeverity;

		switch(Verbosity)
		{
		case ELogVerbosity::Error:
			MessageSeverity = EMessageSeverity::Error;
			break;
		case ELogVerbosity::Warning:
			MessageSeverity = EMessageSeverity::Warning;
			break;
		case ELogVerbosity::Verbose:
		case ELogVerbosity::Log:
		case ELogVerbosity::Display:
		default:
			MessageSeverity = EMessageSeverity::Info;
			break;
		}

		TSharedRef<FTokenizedMessage> LogMessage = FTokenizedMessage::Create(MessageSeverity, FText::FromString(Message));
		LogMessage->SetIndentationLevel(MessageSeverity == EMessageSeverity::Info ? 0 : 1);
		Instance->GetPrivate()->UpdateLogger->LogMessage(LogMessage);
	}
}

int32 CalculateMorphNumVertices(TConstArrayView<uint32> MorphHeadersDataView)
{
	using namespace UE::MorphTargetVertexCodec;

	if (MorphHeadersDataView.Num() == 0)
	{
		return 0;
	}

	int32 NumBatches = MorphHeadersDataView.Num() / UE::MorphTargetVertexCodec::NumBatchHeaderDwords;

	int32 NumMorphVertices = 0;
	for (int32 BatchHeaderIndex = 0; BatchHeaderIndex < NumBatches; ++BatchHeaderIndex)
	{
		TConstArrayView<uint32> BatchHeaderData(
			MorphHeadersDataView.GetData() + BatchHeaderIndex * UE::MorphTargetVertexCodec::NumBatchHeaderDwords,
			UE::MorphTargetVertexCodec::NumBatchHeaderDwords);

		FDeltaBatchHeader BatchHeader;
		ReadHeader(BatchHeader, BatchHeaderData);

		NumMorphVertices += BatchHeader.NumElements;
	}

	return NumMorphVertices;
}

#if !WITH_EDITOR
void ReconstructMorphTargetsFromMeshCompressedData(const UE::Mutable::Private::FMesh& Mesh, TArray<FMorphTargetCompressedLODModel>& OutMorphTargets, UE::Mutable::Private::EMorphUsageFlags UsageFilter)
{
	MUTABLE_CPUPROFILER_SCOPE(ReconstructMorphTargetsFromMeshCompressedData_ToCompressed);
	
	using namespace UE::MorphTargetVertexCodec;

	const int32 NumMorphs = Mesh.Morph.Names.Num();
	OutMorphTargets.SetNum(NumMorphs);

	for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
	{
		if (EnumHasAnyFlags(Mesh.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
		{
			const float PositionPrecision = Mesh.Morph.PositionPrecision;
			const float TangentZPrecision = Mesh.Morph.TangentZPrecision;
			
			OutMorphTargets[MorphIndex].PositionPrecision = PositionPrecision;
			OutMorphTargets[MorphIndex].TangentPrecision = TangentZPrecision;
			
			const uint32 BatchStartOffset = Mesh.Morph.BatchStartOffsetPerMorph[MorphIndex];
			const uint32 MorphNumBatches  = Mesh.Morph.BatchesPerMorph[MorphIndex];

			OutMorphTargets[MorphIndex].PackedDeltaHeaders.SetNumUninitialized(MorphNumBatches);

			int32 MorphDeltaDataBeginInDwords = 0;
			int32 MorphDeltaDataEndInDwords = 0;

			if (MorphNumBatches > 0)
			{
				// Get the Offset value for the morph target first batch.
				MorphDeltaDataBeginInDwords = Mesh.MorphDataBuffer[BatchStartOffset * NumBatchHeaderDwords] / sizeof(uint32);

				for (uint32 BatchHeaderIndex = 0; BatchHeaderIndex < MorphNumBatches; ++BatchHeaderIndex)
				{
					TConstArrayView<uint32> BatchHeaderData(
							&Mesh.MorphDataBuffer[(BatchStartOffset + BatchHeaderIndex) * NumBatchHeaderDwords], 
							NumBatchHeaderDwords);

					FDeltaBatchHeader& Header = OutMorphTargets[MorphIndex].PackedDeltaHeaders[BatchHeaderIndex];
					ReadHeader(Header, BatchHeaderData);
				}	

				MorphDeltaDataEndInDwords = 
						OutMorphTargets[MorphIndex].PackedDeltaHeaders.Last().DataOffset / sizeof(uint32) +
						CalculateBatchDwords(OutMorphTargets[MorphIndex].PackedDeltaHeaders.Last()); 

				// Fixup header offset.
				for (uint32 BatchHeaderIndex = 0; BatchHeaderIndex < MorphNumBatches; ++BatchHeaderIndex)
				{
					FDeltaBatchHeader& Header = OutMorphTargets[MorphIndex].PackedDeltaHeaders[BatchHeaderIndex];
					Header.DataOffset -= MorphDeltaDataBeginInDwords * sizeof(uint32);
				}	
			}

			OutMorphTargets[MorphIndex].PackedDeltaData.SetNumUninitialized(MorphDeltaDataEndInDwords - MorphDeltaDataBeginInDwords);
			FMemory::Memcpy(
					OutMorphTargets[MorphIndex].PackedDeltaData.GetData(),
					Mesh.MorphDataBuffer.GetData() + MorphDeltaDataBeginInDwords,
					(MorphDeltaDataEndInDwords - MorphDeltaDataBeginInDwords) * sizeof(uint32));
		}
	}
}
#endif

void ReconstructMorphTargetsFromMeshCompressedData(const UE::Mutable::Private::FMesh& Mesh, TArray<FMorphTargetLODModel>& OutMorphTargets, UE::Mutable::Private::EMorphUsageFlags UsageFilter)
{
	MUTABLE_CPUPROFILER_SCOPE(ReconstructMorphTargetsFromMeshCompressedData_ToUncompressed);
	
	using namespace UE::MorphTargetVertexCodec;

	const int32 NumMorphs = Mesh.Morph.Names.Num();
	OutMorphTargets.SetNum(NumMorphs);

	for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
	{
		if (EnumHasAnyFlags(Mesh.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
		{
			OutMorphTargets[MorphIndex].SectionIndices = Mesh.Morph.SurfacesInUsePerMorph[MorphIndex];

			const float PositionPrecision = Mesh.Morph.PositionPrecision;
			const float TangentZPrecision = Mesh.Morph.TangentZPrecision;
			
			const int32 BatchStartOffset = Mesh.Morph.BatchStartOffsetPerMorph[MorphIndex];
			const int32 MorphNumBatches  = Mesh.Morph.BatchesPerMorph[MorphIndex];

			check((BatchStartOffset + MorphNumBatches)*NumBatchHeaderDwords <= (uint32)Mesh.MorphDataBuffer.Num());
			TConstArrayView<uint32> MorphHeadersView = TConstArrayView<uint32>(
				Mesh.MorphDataBuffer.GetData() + BatchStartOffset * NumBatchHeaderDwords,
				MorphNumBatches * NumBatchHeaderDwords);

			int32 NumMorphVertices = CalculateMorphNumVertices(MorphHeadersView);
			
			OutMorphTargets[MorphIndex].Vertices.SetNumUninitialized(NumMorphVertices);
			OutMorphTargets[MorphIndex].NumVertices = NumMorphVertices;
			
			TConstArrayView<uint32> MorphDataBufferView = Mesh.MorphDataBuffer;

			int32 MorphVertexIndex = 0;
			for (int32 BatchHeaderIndex = 0; BatchHeaderIndex < MorphNumBatches; ++BatchHeaderIndex)
			{
				check((BatchStartOffset + BatchHeaderIndex) * NumBatchHeaderDwords + NumBatchHeaderDwords <= (uint32)MorphDataBufferView.Num());
				TConstArrayView<uint32> BatchHeaderData(
						MorphDataBufferView.GetData() + (BatchStartOffset + BatchHeaderIndex) * NumBatchHeaderDwords, 
						NumBatchHeaderDwords);

				FDeltaBatchHeader BatchHeader;
				ReadHeader(BatchHeader, BatchHeaderData);

				if (BatchHeader.NumElements == 0)
				{
					continue;
				}

				TArray<FQuantizedDelta, TInlineAllocator<UE::MorphTargetVertexCodec::BatchSize>> QuantizedDeltas;
				QuantizedDeltas.SetNumUninitialized(BatchHeader.NumElements);

				const uint32 BatchSizeInDwords = CalculateBatchDwords(BatchHeader); 

				check(BatchHeader.DataOffset / sizeof(uint32) + BatchSizeInDwords <= (uint32)MorphDataBufferView.Num());
				TConstArrayView<uint32> Data(
						MorphDataBufferView.GetData() + (BatchHeader.DataOffset / sizeof(uint32)), 
						BatchSizeInDwords);

				ReadQuantizedDeltas(QuantizedDeltas, BatchHeader, Data);
				for (FQuantizedDelta& QuantizedDelta : QuantizedDeltas)
				{
					FMorphTargetDelta& Delta = OutMorphTargets[MorphIndex].Vertices[MorphVertexIndex++];
					DequantizeDelta(Delta, BatchHeader.bTangents, QuantizedDelta, Mesh.Morph.PositionPrecision, Mesh.Morph.TangentZPrecision);
				}
			}
		
			check(MorphVertexIndex == NumMorphVertices);
		}
	}
}

bool IsStreamingEnabled(const UCustomizableObject& Object, const int32 State)
{
	const UModelResources& ModelResources = Object.GetPrivate()->GetModelResourcesChecked();
	const FString& StateName = Object.GetPrivate()->GetStateName(State);
	const FMutableStateData* StateData = ModelResources.StateUIDataMap.Find(StateName);

	const bool bStateAllowsStreaming = StateData ? !StateData->bDisableMeshStreaming : true;

	return (bStateAllowsStreaming || bForceStreamMeshLODs) &&
		bStreamMeshLODs &&
		IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::SkeletalMesh);
}

