// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableObjectUIData.h"

#include "MuCO/CustomizableObjectCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectUIData)


FArchive& operator<<(FArchive& Ar, FMutableUIMetadata& Metadata)
{
	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	Ar << Metadata.ObjectFriendlyName;
	Ar << Metadata.UISectionName;
	Ar << Metadata.UIOrder;

	if (Ar.IsLoading())
	{
		FString StringRef;
		Ar << StringRef;
		Metadata.UIThumbnail = TSoftObjectPtr<UTexture2D>(FSoftObjectPath(StringRef));
	}
	else
	{
		FString StringRef = Metadata.UIThumbnail.ToSoftObjectPath().ToString();
		Ar << StringRef;
	}

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		FString StringRef;
		Ar << StringRef;
		Metadata.EditorUIThumbnailObject = TSoftObjectPtr<UObject>(FSoftObjectPath(StringRef));
	}
	else
	{
		FString StringRef = Metadata.EditorUIThumbnailObject.ToSoftObjectPath().ToString();
		Ar << StringRef;
	}
#endif // WITH_EDITORONLY_DATA

	Ar << Metadata.ExtraInformation;

	if (Ar.IsLoading())
	{
		int32 NumReferencedAssets = 0;
		Ar << NumReferencedAssets;
		Metadata.ExtraAssets.Empty(NumReferencedAssets);

		for (int32 i = 0; i < NumReferencedAssets; ++i)
		{
			FString Key, StringRef;
			Ar << Key;
			Ar << StringRef;

			Metadata.ExtraAssets.Add(Key, TSoftObjectPtr<UObject>(FSoftObjectPath(StringRef)));
		}
	}
	else
	{
		int32 NumReferencedAssets = Metadata.ExtraAssets.Num();
		Ar << NumReferencedAssets;

		for (TPair<FString, TSoftObjectPtr<UObject>>& AssetPair : Metadata.ExtraAssets)
		{
			FString StringRef = AssetPair.Value.ToSoftObjectPath().ToString();
			Ar << AssetPair.Key;
			Ar << StringRef;
		}
	}

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableParamUIMetadata& Struct)
{
	Ar << *static_cast<FMutableUIMetadata*>(&Struct);
	
	Ar << Struct.MinimumValue;
	Ar << Struct.MaximumValue;

	FString ExportString;
	if (Ar.IsSaving())
	{
		ExportString = Struct.GameplayTags.ToString();
	}

	Ar << ExportString;

	if (Ar.IsLoading())
	{
		if (!ExportString.IsEmpty())
		{
			Struct.GameplayTags.FromExportString(ExportString);
		}
	}

	return Ar;
}


FArchive& operator<<(FArchive& Ar, FMutableStateUIMetadata& Struct)
{
	Ar << *static_cast<FMutableUIMetadata*>(&Struct);

	return Ar;
}
