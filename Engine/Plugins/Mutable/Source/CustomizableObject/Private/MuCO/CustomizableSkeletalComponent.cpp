// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableSkeletalComponent.h"

#include "MuCO/CustomizableObjectInstanceUsagePrivate.h"
#include "MuCO/CustomizableSkeletalComponentPrivate.h"

#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "UObject/UObjectGlobals.h"
#include "Components/SkeletalMeshComponent.h"
#include "MuCO/CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableSkeletalComponent)


UCustomizableSkeletalComponent::UCustomizableSkeletalComponent()
{
	Private = CreateDefaultSubobject<UCustomizableSkeletalComponentPrivate>(TEXT("Private"));
}

#if WITH_EDITOR
void UCustomizableSkeletalComponent::PostLoad()
{
	Super::PostLoad();

	SetCustomizableObjectInstance(CustomizableObjectInstance);
	SetComponentName(ComponentName);
	SetSkipSetReferenceSkeletalMesh(bSkipSetReferenceSkeletalMesh);
	SetSkipSetSkeletalMeshOnAttach(bSkipSetSkeletalMeshOnAttach);
}

void UCustomizableSkeletalComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableSkeletalComponent, CustomizableObjectInstance))
	{
		SetCustomizableObjectInstance(CustomizableObjectInstance);
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableSkeletalComponent, ComponentName))
	{
		SetComponentName(ComponentName);
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableSkeletalComponent, bSkipSetReferenceSkeletalMesh))
	{
		SetSkipSetReferenceSkeletalMesh(bSkipSetReferenceSkeletalMesh);
	}
	else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableSkeletalComponent, bSkipSetSkeletalMeshOnAttach))
	{
		SetSkipSetSkeletalMeshOnAttach(bSkipSetSkeletalMeshOnAttach);
	}
}
#endif

UCustomizableSkeletalComponentPrivate::UCustomizableSkeletalComponentPrivate()
{
	InstanceUsage = CreateDefaultSubobject<UCustomizableObjectInstanceUsage>(TEXT("InstanceUsage"));
}


UCustomizableObjectInstanceUsage* UCustomizableSkeletalComponent::GetInstanceUsage()
{
	return GetPrivate()->InstanceUsage;
}


void UCustomizableSkeletalComponent::SetComponentName(const FName& Name)
{
	ComponentName = Name;
	GetPrivate()->InstanceUsage->SetComponentName(Name);
}


FName UCustomizableSkeletalComponent::GetComponentName() const
{
	return ComponentName;
}


UCustomizableObjectInstance* UCustomizableSkeletalComponent::GetCustomizableObjectInstance() const
{
	return CustomizableObjectInstance;
}


void UCustomizableSkeletalComponent::SetCustomizableObjectInstance(UCustomizableObjectInstance* Instance)
{
	CustomizableObjectInstance = Instance;
	GetPrivate()->InstanceUsage->SetCustomizableObjectInstance(Instance);
}


void UCustomizableSkeletalComponent::SetSkipSetReferenceSkeletalMesh(bool bSkip)
{
	bSkipSetReferenceSkeletalMesh = bSkip;
	GetPrivate()->InstanceUsage->SetSkipSetReferenceSkeletalMesh(bSkip);
}


void UCustomizableSkeletalComponent::SetSkipSetSkeletalMeshOnAttach(bool bSkip)
{
	bSkipSetSkeletalMeshOnAttach = bSkip;
	GetPrivate()->InstanceUsage->SetSkipSetSkeletalMeshOnAttach(bSkip);
}


void UCustomizableSkeletalComponent::UpdateSkeletalMeshAsync(bool bNeverSkipUpdate)
{
	GetPrivate()->InstanceUsage->UpdateSkeletalMeshAsync(bNeverSkipUpdate);
}


void UCustomizableSkeletalComponent::UpdateSkeletalMeshAsyncResult(FInstanceUpdateDelegate Callback, bool bIgnoreCloseDist, bool bForceHighPriority)
{
	GetPrivate()->InstanceUsage->UpdateSkeletalMeshAsyncResult(Callback, bIgnoreCloseDist, bForceHighPriority);
}


UCustomizableSkeletalComponentPrivate* UCustomizableSkeletalComponent::GetPrivate()
{
	check(Private);
	return Private;
}


const UCustomizableSkeletalComponentPrivate* UCustomizableSkeletalComponent::GetPrivate() const
{
	check(Private);
	return Private;
}


void UCustomizableSkeletalComponent::OnAttachmentChanged()
{
	Super::OnAttachmentChanged();

	GetPrivate()->InstanceUsage->AttachTo(Cast<USkeletalMeshComponent>(GetAttachParent()));
}

