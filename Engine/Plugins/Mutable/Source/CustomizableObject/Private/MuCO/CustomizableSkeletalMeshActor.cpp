// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/CustomizableSkeletalMeshActor.h"
#include "MuCO/CustomizableSkeletalMeshActorPrivate.h"

#include "Components/SkeletalMeshComponent.h"
#include "MuCO/CustomizableSkeletalComponent.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableSkeletalComponentPrivate.h"
#include "Engine/SkeletalMesh.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/UnrealPortabilityHelpers.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableSkeletalMeshActor)

#define LOCTEXT_NAMESPACE "CustomizableObject"


UCustomizableObjectInstance* ACustomizableSkeletalMeshActor::GetCustomizableObjectInstance()
{
	for (const UCustomizableSkeletalComponent* Component : CustomizableSkeletalComponents)
	{
		if (UCustomizableObjectInstance* COInstance = Component->GetCustomizableObjectInstance())
		{
			return COInstance;
		}
	}

	return nullptr;
}


ACustomizableSkeletalMeshActor::ACustomizableSkeletalMeshActor(const FObjectInitializer& Initializer)
{
	Private = CreateDefaultSubobject<UCustomizableSkeletalMeshActorPrivate>(TEXT("Private"));

	// Old assets used to create the first UCustomizableSkeletalComponent as a Subobject. To be able to deserialize them we need to create it.
	// Creating instead an Object instead of a Suobject will not work. Only Component UCustomizableSkeletalComponent 0 is a Subobject.
	{
		UCustomizableSkeletalComponent* CustomizableSkeletalComponent = CreateDefaultSubobject<UCustomizableSkeletalComponent>(TEXT("CustomizableSkeletalComponent0"));
		CustomizableSkeletalComponents.Add(CustomizableSkeletalComponent);
		if (USkeletalMeshComponent* SkeletalMeshComp = Super::GetSkeletalMeshComponent()) 
		{
			SkeletalMeshComponents.Add(SkeletalMeshComp);
			CustomizableSkeletalComponents[0]->AttachToComponent(SkeletalMeshComp, FAttachmentTransformRules::KeepRelativeTransform);
		}
	}
}


USkeletalMeshComponent* ACustomizableSkeletalMeshActor::GetSkeletalMeshComponent(const FName& ComponentName)
{
	for (int32 ComponentIndex = 0; ComponentIndex < CustomizableSkeletalComponents.Num(); ComponentIndex++)
	{
		const UCustomizableSkeletalComponent* Component = CustomizableSkeletalComponents[ComponentIndex];
		
		if (Component->GetComponentName() == ComponentName)
		{
			return SkeletalMeshComponents[ComponentIndex];
		}
	}

	return nullptr; 
}


void UCustomizableSkeletalMeshActorPrivate::Init(UCustomizableObjectInstance* Instance)
{
	UCustomizableObject* Object = Instance->GetCustomizableObject();
	if (!Object)
	{
		return;
	}

	// The current Actor transform will be lost when the component hierarchy is rebuild. 
	// Cache it here and set it back after the component hierarchy recreation.
	const FTransform ActorTransform = GetPublic()->GetActorTransform(); 

	const int32 NumComponents = Object->GetComponentCount();
	
	for (TObjectPtr<USkeletalMeshComponent> Component : GetPublic()->SkeletalMeshComponents)
	{
		Component->DestroyComponent();
	}

	GetPublic()->SkeletalMeshComponents.Empty(NumComponents);

	for (TObjectPtr<UCustomizableSkeletalComponent> Component : GetPublic()->CustomizableSkeletalComponents)
	{
		Component->DestroyComponent();
	}

	GetPublic()->CustomizableSkeletalComponents.Empty(NumComponents);
	
	for (UE::Mutable::Private::FComponentId ComponentId = 0; ComponentId < NumComponents; ++ComponentId)
	{
		const FName& ComponentName = Object->GetComponentName(ComponentId);
		const FString ComponentNameString = ComponentName.ToString();

		const FString SkeletalMeshComponentName = FString::Printf(TEXT("SkeletalMeshComponent %s"), *ComponentNameString);
		USkeletalMeshComponent* Component = NewObject<USkeletalMeshComponent>(GetPublic(), USkeletalMeshComponent::StaticClass(), FName(*SkeletalMeshComponentName));
		Component->CreationMethod = EComponentCreationMethod::Native;
		
		if (ComponentId == 0)
		{
			GetPublic()->SetRootComponent(Component);
		}
		else
		{
			Component->AttachToComponent(GetPublic()->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
		}
		
		Component->RegisterComponent();
		
		UCustomizableSkeletalComponent* CustomizableComponent;
		const FString CustomizableComponentName = FString::Printf(TEXT("CustomizableSkeletalComponent %s"), *ComponentNameString);
		CustomizableComponent = NewObject<UCustomizableSkeletalComponent>(GetPublic(), UCustomizableSkeletalComponent::StaticClass(), FName(CustomizableComponentName));

		CustomizableComponent->AttachToComponent(Component, FAttachmentTransformRules::KeepRelativeTransform);
		CustomizableComponent->SetCustomizableObjectInstance(Instance);
		CustomizableComponent->SetComponentName(ComponentName);
		CustomizableComponent->RegisterComponent();
		
		GetPublic()->SkeletalMeshComponents.Add(Component);
		GetPublic()->CustomizableSkeletalComponents.Add(CustomizableComponent);
	}

	GetPublic()->SetActorTransform(ActorTransform);
	Instance->UpdateSkeletalMeshAsync();
}


void ACustomizableSkeletalMeshActor::SetDebugMaterial(UMaterialInterface* InDebugMaterial)
{
	if (!InDebugMaterial)
	{
		return;
	}

	DebugMaterial = InDebugMaterial;
}


void ACustomizableSkeletalMeshActor::EnableDebugMaterial(bool bEnableDebugMaterial)
{
	bRemoveDebugMaterial = bDebugMaterialEnabled && !bEnableDebugMaterial;
	bDebugMaterialEnabled = bEnableDebugMaterial;

	if (UCustomizableObjectInstance* COInstance = GetCustomizableObjectInstance())
	{
		//Bind Instance Update delegate to Actor
		COInstance->UpdatedDelegate.AddUniqueDynamic(this, &ACustomizableSkeletalMeshActor::SwitchComponentsMaterials);
		SwitchComponentsMaterials(COInstance);
	}
}


UCustomizableSkeletalMeshActorPrivate* ACustomizableSkeletalMeshActor::GetPrivate()
{
	check(Private);
	return Private;
}


const UCustomizableSkeletalMeshActorPrivate* ACustomizableSkeletalMeshActor::GetPrivate() const
{
	check(Private);
	return Private;
}


void ACustomizableSkeletalMeshActor::SwitchComponentsMaterials(UCustomizableObjectInstance* Instance)
{
	if (!DebugMaterial)
	{
		return;
	}

	if (bDebugMaterialEnabled || bRemoveDebugMaterial)
	{
		UCustomizableObjectInstance* COInstance = GetCustomizableObjectInstance();

		if (!COInstance)
		{
			return;
		}

		for (UE::Mutable::Private::FComponentId ComponentId = 0; ComponentId < SkeletalMeshComponents.Num(); ++ComponentId)
		{
			int32 NumMaterials = SkeletalMeshComponents[ComponentId]->GetNumMaterials();

			if (bDebugMaterialEnabled)
			{
				for (int32 MatIndex = 0; MatIndex < NumMaterials; ++MatIndex)
				{
					SkeletalMeshComponents[ComponentId]->SetMaterial(MatIndex, DebugMaterial);
				}
			}
			else // Remove debugmaterial
			{
				const FName ComponentName = CustomizableSkeletalComponents[ComponentId]->GetComponentName();
				
				// check if original materials already overriden
				const TArray<UMaterialInterface*> OverrideMaterials = COInstance->GetSkeletalMeshComponentOverrideMaterials(ComponentName);
				const bool bUseOverrideMaterials = COInstance->GetCustomizableObject()->bEnableMeshCache && CVarEnableMeshCache.GetValueOnAnyThread();

				if (bUseOverrideMaterials && OverrideMaterials.Num() > 0)
				{
					for (int32 MatIndex = 0; MatIndex < OverrideMaterials.Num(); ++MatIndex)
					{
						SkeletalMeshComponents[ComponentId]->SetMaterial(MatIndex, OverrideMaterials[MatIndex]);
					}
				}
				else
				{
					SkeletalMeshComponents[ComponentId]->EmptyOverrideMaterials();
				}

				bRemoveDebugMaterial = false;
			}
		}
	}
}

ACustomizableSkeletalMeshActor* UCustomizableSkeletalMeshActorPrivate::GetPublic()
{
	return CastChecked<ACustomizableSkeletalMeshActor>(GetOuter());
}


TArray<TObjectPtr<UCustomizableSkeletalComponent>>& UCustomizableSkeletalMeshActorPrivate::GetComponents()
{
	return GetPublic()->CustomizableSkeletalComponents;
}


#undef LOCTEXT_NAMESPACE
