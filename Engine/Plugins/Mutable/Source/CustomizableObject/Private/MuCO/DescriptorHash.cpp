// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/DescriptorHash.h"

#include "MuCO/CustomizableObject.h"


FDescriptorHash::FDescriptorHash(const FCustomizableObjectInstanceDescriptor& Descriptor)
{
#if WITH_EDITORONLY_DATA
	if (Descriptor.CustomizableObject)
	{
		Hash = HashCombine(Hash, GetTypeHash(Descriptor.CustomizableObject->GetPathName()));
	}
#endif

	for (const FCustomizableObjectBoolParameterValue& Value : Descriptor.BoolParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}	

	for (const FCustomizableObjectIntParameterValue& Value : Descriptor.IntParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}
	
	for (const FCustomizableObjectFloatParameterValue& Value : Descriptor.FloatParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	for (const FCustomizableObjectTextureParameterValue& Value : Descriptor.TextureParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	for (const FCustomizableObjectSkeletalMeshParameterValue& Value : Descriptor.SkeletalMeshParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	for (const FCustomizableObjectVectorParameterValue& Value : Descriptor.VectorParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	for (const FCustomizableObjectProjectorParameterValue& Value : Descriptor.ProjectorParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}
	
	for (const FCustomizableObjectTransformParameterValue& Value : Descriptor.TransformParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}
	
	for (const FCustomizableObjectExternalTypeParameterValue& Value : Descriptor.ExternalTypeParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	for (const FCustomizableObjectMaterialParameterValue& Value : Descriptor.MaterialParameters)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	Hash = HashCombine(Hash, GetTypeHash(Descriptor.State));
	Hash = HashCombine(Hash, GetTypeHash(Descriptor.GetBuildParameterRelevancy()));
}


FString FDescriptorHash::ToString() const
{
	return FString::Printf(TEXT("(Hash=%u,"), Hash);
}
