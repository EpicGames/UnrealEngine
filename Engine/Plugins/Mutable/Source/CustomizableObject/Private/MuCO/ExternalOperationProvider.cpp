// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/ExternalOperationProvider.h"

#include "CustomizableObject/Internal/MuCO/CustomizableObjectPrivate.h"


namespace UE::Mutable::Private
{
	FExternalOperationProvider::FExternalOperationProvider(UModelResources& InModelResources)
	{
		ExternalOperations = InModelResources.ExternalOperations;
	}


	void FExternalOperationProvider::AddReferencedObjects(FReferenceCollector& Collector)
	{
		for (TPair<FOperation::ADDRESS, FInstancedStruct>& Pair : ExternalOperations)
		{
			if (const UScriptStruct* ScriptStruct = Pair.Value.GetScriptStruct())
			{
				Collector.AddPropertyReferences(ScriptStruct, Pair.Value.GetMutableMemory());
			}
		}
	}


	const FInstancedStruct& FExternalOperationProvider::Get(FOperation::ADDRESS Address) const
	{
		return ExternalOperations[Address];
	}
}
