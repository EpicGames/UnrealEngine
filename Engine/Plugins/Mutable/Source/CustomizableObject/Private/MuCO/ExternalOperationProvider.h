// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ExternalOperationProvider.h"
#include "UObject/GCObject.h"

#include "MuR/Operations.h"

struct FInstancedStruct;
class UModelResources;


namespace UE::Mutable::Private
{
	class FExternalOperationProvider : public IExternalOperationProvider, public FGCObject
	{
	public:
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override; 
		virtual FString GetReferencerName() const override
		{
			return TEXT("FExternalOperationProvider");
		}

		// IExternalOperationProvider interface
		virtual const FInstancedStruct& Get(FOperation::ADDRESS Address) const override;
		
		// Own interface
		FExternalOperationProvider(UModelResources& InModelResources);
	
	private:
		TMap<FOperation::ADDRESS, FInstancedStruct> ExternalOperations;
	};
}

