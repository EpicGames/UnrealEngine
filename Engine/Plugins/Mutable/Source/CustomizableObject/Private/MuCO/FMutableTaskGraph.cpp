// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/FMutableTaskGraph.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuR/MutableTrace.h"


#define UE_MUTABLE_THREAD_REQUEST_LOCK TEXT("Mutable Thread Request Lock")
#define UE_MUTABLE_THREAD_LOCK TEXT("Mutable Thread Lock")


constexpr LowLevelTasks::ETaskPriority TASKGRAPH_PRIORITY = LowLevelTasks::ETaskPriority::BackgroundHigh;

static TAutoConsoleVariable<float> CVarMutableTaskLowPriorityMaxWaitTime(
	TEXT("mutable.MutableTaskLowPriorityMaxWaitTime"),
	3.0f,
	TEXT("Max time a Mutable Task with Low priority can wait. Once this time has passed it will be launched unconditionally."),
	ECVF_Scalability);

static TAutoConsoleVariable<float> CVarGameThreadTaskMaxTime(
	TEXT("mutable.GameThreadTaskMaxTime"),
	0.5f / 1000.0f, // 0.5ms
	TEXT("Max time Mutable can execute Game Thread Task each tick. A single task may take longer than the specified time."),
	ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarEnableMutableTaskLowPriority(
	TEXT("mutable.EnableMutableTaskLowPriority"),
	true,
	TEXT("Enable or disable Mutable Tasks with Low priority. If disabled, all task will have the same priority. "),
	ECVF_Scalability);

TAutoConsoleVariable<bool> CVarEnableParallelUpdates(
	TEXT("mutable.EnableParallelUpdates"),
	true,
	TEXT("Allow initial generation, Mips and LODs to be run in parallel."),
	ECVF_Scalability);


bool FMutableTaskGraph::FGameThreadTask::AreDependenciesComplete() const
{
	for (const UE::Tasks::FTask& Dependency : Prerequisites)
	{
		if (!Dependency.IsCompleted())
		{
			return false;
		}
	}

	return true;
}


FMutableTaskGraph::~FMutableTaskGraph()
{
	GameThreadTasks.Empty();
}


void FMutableTaskGraph::AddGameThreadTask(const TCHAR* DebugName, TUniqueFunction<void(float)>&& TaskBody, const TArray<UE::Tasks::FTask>& Prerequisites)
{
	GameThreadTasks.Enqueue({ DebugName, Forward<TUniqueFunction<void(float)>>(TaskBody), Prerequisites });
}


UE::Tasks::FTask FMutableTaskGraph::AddMutableThreadTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& InPrerequisites)
{
	if (CVarEnableParallelUpdates.GetValueOnAnyThread())
	{
		return AddGenerationTask(DebugName, Forward<TUniqueFunction<void()>>(TaskBody), InPrerequisites);
	}
	else
	{
		FScopeLock Lock(&MutableTaskLock);

		TArray<UE::Tasks::FTask, TInlineAllocator<1>> Prerequisites = LastMutableTask.IsValid() ?
			TArray<UE::Tasks::FTask, TInlineAllocator<1>>{ LastMutableTask } :
			TArray<UE::Tasks::FTask, TInlineAllocator<1>>{};

		Prerequisites.Append(InPrerequisites);
	
		LastMutableTask = UE::Tasks::Launch(DebugName, Forward<TUniqueFunction<void()>>(TaskBody), Prerequisites, TASKGRAPH_PRIORITY);
	
		return LastMutableTask;
	}
}


UE::Tasks::FTask FMutableTaskGraph::AddGenerationTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& InPrerequisites)
{
	FScopeLock Lock(&CriticalSectionLastGenerationTask);
	
	TArray<UE::Tasks::FTask, TInlineAllocator<1>> Prerequisites = LastMutableTask.IsValid() 
		? TArray<UE::Tasks::FTask, TInlineAllocator<1>>{ LastMutableTask } 
		: TArray<UE::Tasks::FTask, TInlineAllocator<1>>{};

	Prerequisites.Append(InPrerequisites);
	
	LastGenerationTask = UE::Tasks::Launch(DebugName, Forward<TUniqueFunction<void()>>(TaskBody), Prerequisites, TASKGRAPH_PRIORITY);
	return LastGenerationTask;
}


UE::Tasks::FTask FMutableTaskGraph::AddImageStreamingTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& InPrerequisites)
{
	FScopeLock Lock(&CriticalSectionLastImageStreaming);
	
	TArray<UE::Tasks::FTask, TInlineAllocator<1>> Prerequisites = LastImageStreamingTask.IsValid() 
		? TArray<UE::Tasks::FTask, TInlineAllocator<1>>{ LastImageStreamingTask } 
		: TArray<UE::Tasks::FTask, TInlineAllocator<1>>{};

	Prerequisites.Append(InPrerequisites);
  
	LastImageStreamingTask = UE::Tasks::Launch(DebugName, Forward<TUniqueFunction<void()>>(TaskBody), Prerequisites, TASKGRAPH_PRIORITY);
	return LastImageStreamingTask;
}


UE::Tasks::FTask FMutableTaskGraph::AddMeshStreamingTask(const TCHAR* DebugName, TUniqueFunction<void()>&& TaskBody, const TArray<UE::Tasks::FTask>& InPrerequisites)
{
	FScopeLock Lock(&CriticalSectionLastMeshStreamingTask);
	
	TArray<UE::Tasks::FTask, TInlineAllocator<1>> Prerequisites = LastMeshStreamingTask.IsValid() 
		? TArray<UE::Tasks::FTask, TInlineAllocator<1>>{ LastMeshStreamingTask } 
		: TArray<UE::Tasks::FTask, TInlineAllocator<1>>{};

	Prerequisites.Append(InPrerequisites);
	
	LastMeshStreamingTask = UE::Tasks::Launch(DebugName, Forward<TUniqueFunction<void()>>(TaskBody), Prerequisites, TASKGRAPH_PRIORITY);
	return LastMeshStreamingTask;
}


uint32 FMutableTaskGraph::AddMutableThreadTaskLowPriority(const TCHAR* DebugName, TFunction<void()>&& TaskBody)
{
	FScopeLock Lock(&MutableTaskLock);

	if (CVarEnableMutableTaskLowPriority.GetValueOnAnyThread())
	{
		const uint32 Id = ++TaskIdGenerator;
		const FMutableThreadLowPriorityTask Task { Id, DebugName, MoveTemp(TaskBody)};
		QueueMutableTasksLowPriority.Add(Task);		

		TryLaunchMutableTaskLowPriority(false);

		return Id;
	}
	else
	{
		AddMutableThreadTask(DebugName, TaskBody);
		return INVALID_ID;	
	}
}


bool FMutableTaskGraph::CancelMutableThreadTaskLowPriority(uint32 Id)
{
	FScopeLock Lock(&MutableTaskLock);

	for (int32 Index = 0; Index < QueueMutableTasksLowPriority.Num(); ++Index)
	{
		if (QueueMutableTasksLowPriority[Index].Id == Id)
		{
			QueueMutableTasksLowPriority.RemoveAt(Index);
			return true;
		}
	}

	return false;
}


void FMutableTaskGraph::WaitForMutableTasks()
{
	FScopeLock Lock(&MutableTaskLock);

	if (LastMutableTask.IsValid())
	{
		LastMutableTask.Wait();
		LastMutableTask = {};
	}
}


void FMutableTaskGraph::WaitForLaunchedLowPriorityTask(uint32 TaskID)
{
	UE::Tasks::FTask Task;

	{
		FScopeLock Lock(&MutableTaskLock);

		if (LastMutableTaskLowPriorityID == TaskID)
		{
			Task = LastMutableTaskLowPriority;
		}
	}

	if (Task.IsValid())
	{
		Task.Wait();
	}
}


void FMutableTaskGraph::TryLaunchMutableTaskLowPriority(bool bFromMutableTask)
{
	MUTABLE_CPUPROFILER_SCOPE(TryLaunchMutableTaskLowPriority)

	if (QueueMutableTasksLowPriority.IsEmpty())
	{
		return;		
	}
	
	if (!IsTaskCompleted(LastMutableTaskLowPriority)) // At any time only a single Low Priority task can be launched.
	{
		return;
	}

	double TimeLimit;
	double TimeElapsed;
	
	bool bTimeLimit;
	
	{
		FScopeLock Lock(&MutableTaskLock);

		if (QueueMutableTasksLowPriority.IsEmpty()) // Also checked inside the lock since we will be writing it
		{
			return;		
		}
		
		if (!IsTaskCompleted(LastMutableTaskLowPriority)) // Check #1 // Also check inside the lock since we will be writing it
		{
			return;
		}

		FMutableThreadLowPriorityTask& NextTask = QueueMutableTasksLowPriority[0];

		TimeLimit = CVarMutableTaskLowPriorityMaxWaitTime.GetValueOnAnyThread();
		TimeElapsed = FPlatformTime::Seconds() - NextTask.CreationTime;

		bTimeLimit = TimeElapsed >= TimeLimit;
		if (!bAllowLaunchMutableTaskLowPriority || // Check #2
			(bFromMutableTask && IsTaskCompleted(LastMutableTask) && !bTimeLimit)) // Check #3
		{			
			return;
		}

		if (CVarFixLowPriorityTasksOverlap.GetValueOnAnyThread())
		{
			LastMutableTaskLowPriorityID = NextTask.Id;
			LastMutableTaskLowPriority = AddMutableThreadTask(*NextTask.DebugName, [this, Task = MoveTemp(NextTask)]() // Moves the task, not the pointer.
			{
				MUTABLE_CPUPROFILER_SCOPE(LowPriorityTaskBody)
				Task.Body();
			});

			const FString TaskName = NextTask.DebugName + TEXT(" End");
			AddMutableThreadTask(*TaskName, [this]()
			{
				MUTABLE_CPUPROFILER_SCOPE(LowPriorityTaskBodyEnd)

				FScopeLock Lock(&MutableTaskLock);
				LastMutableTaskLowPriorityID = INVALID_ID;
				LastMutableTaskLowPriority = {};

				TryLaunchMutableTaskLowPriority(true);
			},
			{ LastMutableTaskLowPriority });

		}
		else
		{
			LastMutableTaskLowPriorityID = NextTask.Id;
			LastMutableTaskLowPriority = AddMutableThreadTask(*NextTask.DebugName, [this, Task = MoveTemp(NextTask)]() // Moves the task, not the pointer.
			{
				MUTABLE_CPUPROFILER_SCOPE(LowPriorityTaskBody)
				Task.Body();

				{
					FScopeLock Lock(&MutableTaskLock);
					LastMutableTaskLowPriorityID = INVALID_ID;
					LastMutableTaskLowPriority = {};
				
					TryLaunchMutableTaskLowPriority(true);
				}
			});
		}

		QueueMutableTasksLowPriority.RemoveAt(0);
	}

	if (bTimeLimit)
	{
		UE_LOGF(LogMutable, Verbose, "Low Priority Mutable Task launched due to time limit (%f)! Waited for: %f", TimeLimit, TimeElapsed)
	}
}


bool FMutableTaskGraph::IsTaskCompleted(const UE::Tasks::FTask& Task) const
{
	return Task.IsCompleted();
}


void FMutableTaskGraph::AllowLaunchingMutableTaskLowPriority(bool bAllow, bool bFromMutableTask)
{
	FScopeLock Lock(&MutableTaskLock);

	bAllowLaunchMutableTaskLowPriority = bAllow;

	if (bAllowLaunchMutableTaskLowPriority)
	{
		TryLaunchMutableTaskLowPriority(bFromMutableTask);
	}
}


void FMutableTaskGraph::TryLaunchGameThreadTask()
{
	MUTABLE_CPUPROFILER_SCOPE(AdvanceCurrentOperation);

	const double TickTime = CVarGameThreadTaskMaxTime.GetValueOnGameThread();
	
	const double StartTime = FPlatformTime::Seconds();
	double RemainingTime = TickTime;
	bool bLoop = true;

	while (bLoop)
	{
		const FGameThreadTask* PendingTask = GameThreadTasks.Peek();
		if (!PendingTask)
		{
			break;
		}
		
		if (!PendingTask->AreDependenciesComplete())
		{
			break;
		}
		
		FGameThreadTask CurrentTask;
		GameThreadTasks.Dequeue(CurrentTask); // Dequeue now since current task can enqueue new tasks.

		// Careful! From this point, PendingTask is no longer valid.

		CurrentTask.Body(RemainingTime);

		RemainingTime = TickTime - (FPlatformTime::Seconds() - StartTime);
		bLoop = RemainingTime > 0.0;
	}
}


void FMutableTaskGraph::Tick()
{
	check(IsInGameThread())

	TryLaunchGameThreadTask();
	
	TryLaunchMutableTaskLowPriority(false);
}


int32 FMutableTaskGraph::GetRemainingWork() const
{
	return !IsTaskCompleted(LastMutableTask) + !IsTaskCompleted(LastMutableTaskLowPriority) + QueueMutableTasksLowPriority.Num() + !GameThreadTasks.IsEmpty();
}

