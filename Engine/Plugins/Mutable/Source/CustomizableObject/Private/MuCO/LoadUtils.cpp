// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/LoadUtils.h"

#include "AssetRegistry/AssetData.h"


namespace UE::Mutable::Private
{
	UObject* LoadObject(const FAssetData& DataAsset)
	{
#if	WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif
	
		return DataAsset.GetAsset();
	}


	UObject* LoadObject(const FSoftObjectPath& Path)
	{
#if	WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif
	
		return Path.TryLoad();
	}
	

	UObject* LoadObject(const FSoftObjectPtr& SoftObject)
	{
#if	WITH_EDITOR
		FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
#endif
	
		return SoftObject.LoadSynchronous();
	}
}
