// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/LogBenchmarkUtil.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/SkeletalMesh.h"
#include "TextureResource.h"
#include "GameFramework/Pawn.h"
#include "Components/SkeletalMeshComponent.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif

#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/CustomizableObjectInstanceUsagePrivate.h"


extern ENGINE_API UEngine* GEngine;


TAutoConsoleVariable<bool> CVarEnableBenchmark(
	TEXT("mutable.EnableBenchmark"),
	false,
	TEXT("Enable or disable the benchmarking."));


namespace LogBenchmarkUtil
{
	void Write(FArchive& Archive, const FString& Text )
	{
		const FString LogUID = TEXT("MUTABLE_BENCHMARK");
		UE_LOGF(LogMutable, Display, "%ls : %ls",*LogUID, *Text);
		
		const FString ComposedString =  FString::Printf(TEXT("%s\n"),*Text);
		const FStringView StringView = ComposedString;
		Archive.Serialize(const_cast<ANSICHAR*>(StringCast<ANSICHAR>(StringView.GetData(), StringView.Len()).Get()), StringView.Len() * sizeof(ANSICHAR));
	}
}


TSharedPtr<FArchive> CreateFile()
{
	const FString Directory = FPaths::ProfilingDir() + TEXT("Mutable/Benchmark");
	IFileManager::Get().MakeDirectory(*Directory, true);

	const FDateTime FileDate = FDateTime::Now();
	const FString Filename = FString::Printf(TEXT("%s/%s.csv"), *Directory,  *FileDate.ToString());

	TSharedPtr<FArchive> Archive = MakeShareable(IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead | FILEWRITE_NoFail));
	check(Archive);

	const FString HeaderRow = TEXT(
		"ID_CO;ID_COI;ID_UpdateType;ID_Descriptor;ID_UpdateResult;"											// Basic identifying data
		"Context_LevelBegunPlay;TriangleCount;"															// The context of the update
		"Time_Queue;Time_Update;Time_TaskGetMesh;"														// Time and memory data ...
		"Time_TaskGetImages;Time_TaskConvertResources;Time_TaskCallbacks;Memory_Update;Memory_Update_Real;"
		"Time_TaskUpdateImage;Memory_TaskUpdateImage;Memory_TaskUpdateImage_Real");

	LogBenchmarkUtil::Write(*Archive, HeaderRow);

	return Archive;
}


FLogBenchmarkUtil::~FLogBenchmarkUtil()
{
	if (Archive)
	{
		Archive->Close();
	}
}


void FLogBenchmarkUtil::GetInstancesStats(int32& OutNumInstances, int32& OutNumBuiltInstances, int32& OutNumAllocatedSkeletalMeshes) const
{
	OutNumInstances = 0;
	OutNumBuiltInstances = 0;
	OutNumAllocatedSkeletalMeshes = 0;
	
	for (TObjectIterator<UCustomizableObjectInstance> Instance; Instance; ++Instance)
	{
		if (!IsValid(*Instance) ||
			Instance->HasAnyFlags(RF_ClassDefaultObject))
		{
			continue;
		}
		
		++OutNumInstances;
		
		if (Instance->GetPrivate()->SkeletalMeshStatus == ESkeletalMeshStatus::Success)
		{
			++OutNumBuiltInstances;
		}

		UCustomizableObject* CO = Instance->GetCustomizableObject();
		if (CO)
		{
			TArray<FName> InstanceComponents = Instance->GetComponentNames();
			for (FName ComponentName: InstanceComponents)
			{
				if (Instance->GetComponentMeshSkeletalMesh(ComponentName))
				{
					++OutNumAllocatedSkeletalMeshes;
				}
			}
		}
	}
}


void FLogBenchmarkUtil::AddTexture(UTexture2D& Texture)
{
	check(IsInGameThread());

	if (!IsBenchmarkingReportingEnabled())
	{
		return;
	}

	TextureTrackerArray.Add(&Texture);
}


void FLogBenchmarkUtil::UpdateStats()
{
	check(IsInGameThread());
	
	if (!IsBenchmarkingReportingEnabled())
	{
		return;
	}

	const UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();
	
	// Instances and Skeletal Mesh
	int32 LocalNumInstances = 0;
	int32 LocalNumBuiltInstances = 0;
	int32 LocalNumAllocatedSkeletalMeshes = 0;

	GetInstancesStats(LocalNumInstances, LocalNumBuiltInstances, LocalNumAllocatedSkeletalMeshes);

	NumInstances = LocalNumInstances;
	NumBuiltInstances = LocalNumBuiltInstances;
	NumAllocatedSkeletalMeshes = LocalNumAllocatedSkeletalMeshes;

	NumPendingInstanceUpdates = System->GetNumPendingInstances();

	// Textures
	uint32 LocalNumAllocatedTextures = 0;
	uint64 LocalTextureGPUSize = 0;

	for (auto Iterator = TextureTrackerArray.CreateIterator(); Iterator; ++Iterator)
	{
		if (Iterator->IsStale())
		{
			Iterator.RemoveCurrent();
		}
		else if (Iterator->IsValid())
		{
			++LocalNumAllocatedTextures;

			UTexture2D* ActualTexture = Iterator->Get();

			if (ActualTexture && ActualTexture->GetResource())
			{
				LocalTextureGPUSize += (*Iterator)->CalcTextureMemorySizeEnum(ETextureMipCount::TMC_ResidentMips);
			}
		}
	}

	NumAllocatedTextures = LocalNumAllocatedTextures;
	TextureGPUSize = LocalTextureGPUSize;
}


void FLogBenchmarkUtil::FinishUpdateMesh(const TSharedRef<FUpdateContextPrivate>& Context)
{
	check(IsInGameThread());

	if (!IsBenchmarkingReportingEnabled())
	{
		return;
	}
	
	TotalUpdateTime += Context->UpdateTime;
	++NumUpdates;

	InstanceBuildTimeAvrg = TotalUpdateTime / NumUpdates;

	const UCustomizableObjectInstance* Instance = Context->Instance.Get();
	if (!Instance)
	{
		return;
	}
	
	const UCustomizableObject* Object = Instance->GetCustomizableObject();
	if (!Object)
	{
		return;
	}

	if (!Archive) // Create file if it does not exist.
	{
		Archive = CreateFile();
	}

	// Cache the amount of triangles of this instance
	uint32 TriangleCount = 0;	
	for (const FName& ComponentName : Instance->GetComponentNames())
	{
		// Process the generated components (not null)
		if (const USkeletalMesh* InstanceSkeletalMesh = Instance->GetComponentMeshSkeletalMesh(ComponentName))
		{
			const FSkeletalMeshRenderData* RenderData = InstanceSkeletalMesh->GetResourceForRendering();
			check(RenderData);

			// Add the amount of triangles for all LODs and all sections
			for (const FSkeletalMeshLODRenderData& RenderDataObject : RenderData->LODRenderData)
			{
				for	(const FSkelMeshRenderSection& Section : RenderDataObject.RenderSections)
				{
					TriangleCount += Section.NumTriangles;
				}
			}
		}
	}
	
	// Identifying data
	FInstanceUpdateStats UpdateData;
	UpdateData.UpdateType = TEXT("Mesh");
	UpdateData.CustomizableObjectPathName = Context->Instance->GetCustomizableObject()->GetPathName();
	UpdateData.CustomizableObjectInstancePathName = Instance->GetPathName();
	UpdateData.Descriptor = Context->GetCapturedDescriptor().ToString();
	UpdateData.UpdateResult = Context->UpdateResult;
	UpdateData.bLevelBegunPlay = Context->bLevelBegunPlay;
	UpdateData.TriangleCount = TriangleCount;
	UpdateData.QueueTime = Context->QueueTime * 1000;
	UpdateData.UpdateTime = Context->UpdateTime * 1000;
	UpdateData.TaskGetMeshesTime = Context->TaskGetMeshesTime * 1000;
	UpdateData.TaskGetImagesTime = Context->TaskGetImagesTime * 1000;
	UpdateData.TaskConvertResourcesTime = Context->TaskConvertResourcesTime * 1000;
	UpdateData.TaskCallbacksTime = Context->TaskCallbacksTime * 1000;
	UpdateData.UpdatePeakMemory = (Context->UpdateEndPeakBytes / 1024.0) / 1024.0;
	UpdateData.UpdateRealPeakMemory = (Context->UpdateEndRealPeakBytes / 1024.0) / 1024.0;
	
	// todo: Find a better way of handling the construction of this string so we know the symmetry with the header row will not get lost when adding new elements
	const FString UpdateString = FString::Printf(TEXT("%s;%s;%s;%s;%s;%s;%u;%f;%f;%f;%f;%f;%f;%f;%f"),
		*UpdateData.CustomizableObjectPathName,
		*UpdateData.CustomizableObjectInstancePathName,
		*UpdateData.UpdateType,
		*UpdateData.Descriptor,
		*StaticEnum<EUpdateResult>()->GetValueAsString(UpdateData.UpdateResult),
		(UpdateData.bLevelBegunPlay ? TEXT("true") : TEXT("false")),
		UpdateData.TriangleCount,
		UpdateData.QueueTime,
		UpdateData.UpdateTime,
		UpdateData.TaskGetMeshesTime,
		UpdateData.TaskGetImagesTime,
		UpdateData.TaskConvertResourcesTime,
		UpdateData.TaskCallbacksTime,
		UpdateData.UpdatePeakMemory,
		UpdateData.UpdateRealPeakMemory);
	LogBenchmarkUtil::Write(*Archive, UpdateString);
	Archive->Flush();

	OnMeshUpdateReported.Broadcast(Context, UpdateData);
}


void FLogBenchmarkUtil::FinishUpdateImage(const FString& CustomizableObjectPathName, const FString& InstancePathName, const FString& InstanceDescriptor, const bool bDidLevelBeginPlay, const double TaskUpdateImageTime, const int64 TaskUpdateImageMemoryPeak, const int64 TaskUpdateImageRealMemoryPeak)
{
	check(IsInGameThread());

	if (!IsBenchmarkingReportingEnabled())
	{
		return;
	}

	if (!Archive) // Create file if it does not exist.
	{
		Archive = CreateFile();
	}
	
	FInstanceUpdateStats MipsUpdateData;
	MipsUpdateData.UpdateType = TEXT("Image");
	MipsUpdateData.CustomizableObjectPathName = CustomizableObjectPathName;
	MipsUpdateData.CustomizableObjectInstancePathName = InstancePathName;
	MipsUpdateData.Descriptor = InstanceDescriptor;
	MipsUpdateData.bLevelBegunPlay = bDidLevelBeginPlay;
	MipsUpdateData.TaskUpdateImageTime = TaskUpdateImageTime * 1000;
	MipsUpdateData.TaskUpdateImagePeakMemory = (TaskUpdateImageMemoryPeak / 1024.0) / 1024.0;
	MipsUpdateData.TaskUpdateImageRealPeakMemory = (TaskUpdateImageRealMemoryPeak / 1024.0) / 1024.0;
	

	// todo: Find a better way of handling the construction of this string so we know the symmetry with the header row will not get lost when adding new elements
	const FString UpdateString = FString::Printf(TEXT("%s;%s;%s;%s;;%s;;;;;;;;;;%f;%f;%f"),
		*MipsUpdateData.CustomizableObjectPathName,
		*MipsUpdateData.CustomizableObjectInstancePathName,
		*MipsUpdateData.UpdateType,
		*MipsUpdateData.Descriptor,
		(MipsUpdateData.bLevelBegunPlay ? TEXT("true") : TEXT("false")),
		MipsUpdateData.TaskUpdateImageTime,
		MipsUpdateData.TaskUpdateImagePeakMemory,
		MipsUpdateData.TaskUpdateImageRealPeakMemory);
	LogBenchmarkUtil::Write(*Archive, UpdateString);
	Archive->Flush();

	OnImageUpdateReported.Broadcast(MipsUpdateData);
}

void FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(bool bIsEnabled)
{
	bIsEnabledOverride = bIsEnabled;
}

bool FLogBenchmarkUtil::IsBenchmarkingReportingEnabled()
{
	return CVarEnableBenchmark.GetValueOnAnyThread() || bIsEnabledOverride;
}

