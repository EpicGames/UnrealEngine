// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/MultilayerProjector.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstanceDescriptor.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultilayerProjector)


void FMultilayerProjectorLayer::Read(const FCustomizableObjectInstanceDescriptor& Descriptor, const FString& ParamName, const int32 LayerIndex)
{
	if (!Descriptor.IsMultilayerProjector(ParamName))
	{
		ensureAlwaysMsgf(false, TEXT("%s"), *MULTILAYER_PROJECTOR_PARAMETERS_INVALID);
		return;
	}

	check(LayerIndex >= 0 && LayerIndex < Descriptor.NumProjectorLayers(*ParamName)); // Layer out of range.

	{
		ECustomizableObjectProjectorType DummyType;
		Descriptor.GetProjectorValue(ParamName, Position, Direction, Up, Scale, Angle, DummyType, LayerIndex);
	}

	{
		const int32 ImageParamIndex = Descriptor.FindTypedParameterIndex(ParamName + IMAGE_PARAMETER_POSTFIX, EMutableParameterType::Int);
		Image = Descriptor.IntParameters[ImageParamIndex].ParameterRangeValueNames[LayerIndex];
	}

	{
		const int32 OpacityParamIndex = Descriptor.FindTypedParameterIndex(ParamName + OPACITY_PARAMETER_POSTFIX, EMutableParameterType::Float);
		Opacity = Descriptor.FloatParameters[OpacityParamIndex].ParameterRangeValues[LayerIndex];
	}
}


void FMultilayerProjectorLayer::Write(FCustomizableObjectInstanceDescriptor& Descriptor, const FString& ParamName, int32 LayerIndex) const
{
	if (!Descriptor.IsMultilayerProjector(ParamName))
	{
		ensureAlwaysMsgf(false, TEXT("%s"), *MULTILAYER_PROJECTOR_PARAMETERS_INVALID);
		return;
	}

	check(LayerIndex >= 0 && LayerIndex < Descriptor.NumProjectorLayers(*ParamName)); // Layer out of range.


	Descriptor.SetProjectorValue(ParamName, Position, Direction, Up, Scale, Angle, LayerIndex);
	Descriptor.SetIntParameterSelectedOption(ParamName + IMAGE_PARAMETER_POSTFIX, Image, LayerIndex);
	Descriptor.SetFloatParameterSelectedOption(ParamName + OPACITY_PARAMETER_POSTFIX, Opacity, LayerIndex);

}


uint32 GetTypeHash(const FMultilayerProjectorLayer& Key)
{
	uint32 Hash = GetTypeHash(Key.Position);

	Hash = HashCombine(Hash, GetTypeHash(Key.Direction));
	Hash = HashCombine(Hash, GetTypeHash(Key.Up));
	Hash = HashCombine(Hash, GetTypeHash(Key.Scale));
	Hash = HashCombine(Hash, GetTypeHash(Key.Angle));
	Hash = HashCombine(Hash, GetTypeHash(Key.Image));
	Hash = HashCombine(Hash, GetTypeHash(Key.Opacity));

	return Hash;
}

