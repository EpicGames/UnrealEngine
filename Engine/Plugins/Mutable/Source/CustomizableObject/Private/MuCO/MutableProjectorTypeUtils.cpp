// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/MutableProjectorTypeUtils.h"

#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"
#include "MuR/Parameters.h"

ECustomizableObjectProjectorType ProjectorUtils::GetEquivalentProjectorType (UE::Mutable::Private::EProjectorType ProjectorType)
{
	// Translate projector type from Mutable Core enum type to CO enum type
	switch (ProjectorType)
	{
	case UE::Mutable::Private::EProjectorType::Planar:
		return ECustomizableObjectProjectorType::Planar;
		
	case UE::Mutable::Private::EProjectorType::Cylindrical:
		return ECustomizableObjectProjectorType::Cylindrical;
		
	case UE::Mutable::Private::EProjectorType::Wrapping:
		return ECustomizableObjectProjectorType::Wrapping;
		
	case UE::Mutable::Private::EProjectorType::Count:
	default:
		checkNoEntry();
		return ECustomizableObjectProjectorType::Planar;
	}
}


UE::Mutable::Private::EProjectorType ProjectorUtils::GetEquivalentProjectorType (ECustomizableObjectProjectorType ProjectorType)
{
	if (GetEquivalentProjectorType(UE::Mutable::Private::EProjectorType::Planar) == ProjectorType)
	{
		return UE::Mutable::Private::EProjectorType::Planar;
	}
	if (GetEquivalentProjectorType(UE::Mutable::Private::EProjectorType::Wrapping) == ProjectorType)
	{
		return UE::Mutable::Private::EProjectorType::Wrapping;
	}
	if (GetEquivalentProjectorType(UE::Mutable::Private::EProjectorType::Cylindrical) == ProjectorType)
	{
		return UE::Mutable::Private::EProjectorType::Cylindrical;
	}

	checkNoEntry();
	return UE::Mutable::Private::EProjectorType::Count;		// Invalid 
}
