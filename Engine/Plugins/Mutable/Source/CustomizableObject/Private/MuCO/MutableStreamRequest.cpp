// Copyright Epic Games, Inc. All Rights Reserved.

#include "MutableStreamRequest.h"

#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "HAL/PlatformFileManager.h"

#if WITH_EDITOR
#include "DerivedDataCache.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataRequestOwner.h"
#endif

FMutableStreamRequest::FMutableStreamRequest(const TSharedPtr<FModelStreamableBulkData>& InModelStreamableBulkData) :
	ModelStreamableBulkData(InModelStreamableBulkData)
{
}


const TSharedPtr<FModelStreamableBulkData>& FMutableStreamRequest::GetModelStreamableBulkData() const
{
	return ModelStreamableBulkData;
}


void FMutableStreamRequest::AddBlock(const FMutableStreamableBlock& Block, UE::Mutable::Private::EStreamableDataType DataType, uint16 ResourceType, TArrayView<uint8> AllocatedMemoryView)
{
	if (bIsStreaming)
	{
		check(false);
		return;
	}

	IAsyncReadFileHandle* FileHandle = nullptr;

#if WITH_EDITOR
	if (!ModelStreamableBulkData->bIsStoredInDDC)
	{
		int32 FileHandleIndex = OpenFilesIds.Find(Block.FileId);
		if (FileHandleIndex == INDEX_NONE)
		{
			const FString& FullFileName = ModelStreamableBulkData->FullFilePath + GetDataTypeExtension(DataType);
			IAsyncReadFileHandle* ReadFileHandle = FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*FullFileName);
			OpenFileHandles.Emplace(ReadFileHandle);

			FileHandleIndex = OpenFilesIds.Add(Block.FileId);
			check(OpenFileHandles.Num() == OpenFilesIds.Num());
		}

		if (OpenFileHandles.IsValidIndex(FileHandleIndex))
		{
			FileHandle = OpenFileHandles[FileHandleIndex].Get();
		}
	}
#endif
	
	BlockReadInfos.Emplace(Block.Offset, FileHandle, AllocatedMemoryView, Block.FileId, DataType, ResourceType, Block.Flags);
}


UE::Tasks::FTask FMutableStreamRequest::Stream()
{
	if (bIsStreaming)
	{
		check(false);
		return UE::Tasks::MakeCompletedTask<void>();
	}

	bIsStreaming = true;

	if (BlockReadInfos.IsEmpty())
	{
		return UE::Tasks::MakeCompletedTask<void>();
	}

	for (const FBlockReadInfo& Block : BlockReadInfos)
	{
		HeapMemory->CompletionEvents.Emplace(TEXT("AsyncReadDataReadyEvent"));
	}

	UE::Tasks::Launch(TEXT("CustomizableObjectReadRequestTask"),
		[
			ModelStreamableBulkData = ModelStreamableBulkData,
			BlockReadInfos = MoveTemp(BlockReadInfos),
			HeapMemory = this->HeapMemory
		]() mutable
		{
			MUTABLE_CPUPROFILER_SCOPE(CustomizableInstanceLoadBlocksAsyncRead_Request);
			FScopeLock Lock(&HeapMemory->ReadRequestLock);

			// Task cancelled, early out.
			if (HeapMemory->bIsCancelled)
			{
				// Trigger preallocated events to complete GatherStreamingRequestsCompletionTask prerequisites
				for (UE::Tasks::FTaskEvent& CompletionEvent : HeapMemory->CompletionEvents)
				{
					CompletionEvent.Trigger();
				}

				return;
			}

#if WITH_EDITOR
			if (ModelStreamableBulkData->bIsStoredInDDC)
			{
				using namespace UE::DerivedData;

				UE::FSharedString SharedName = UE::FSharedString(TEXT("MutableStreamRequest"));

				TArray<FCacheGetValueRequest> Requests;
				Requests.Reserve(BlockReadInfos.Num());

				TArray<uint32> UniqueFileIds;
				for (const FBlockReadInfo& Block : BlockReadInfos)
				{
					if (UniqueFileIds.Find(Block.FileId) == INDEX_NONE)
					{
						FCacheGetValueRequest Request;
						Request.Name = SharedName;
						Request.Key = ModelStreamableBulkData->DDCKey;
						Request.Key.Hash = ModelStreamableBulkData->DDCValues[Block.FileId];
						Request.UserData = (uint32)Block.FileId;
						Request.Policy = ModelStreamableBulkData->DDCDefaultPolicy;
						Requests.Add(MoveTemp(Request));

						UniqueFileIds.Add(Block.FileId);
					}
				}

				HeapMemory->DDCRequestOwner = MakeShared<FRequestOwner>(EPriority::High);
				HeapMemory->DDCRequestOwner->KeepAlive();
				
				GetCache().GetValue(Requests, *HeapMemory->DDCRequestOwner,
					[HeapMemory, BlockReadInfos = MoveTemp(BlockReadInfos)](FCacheGetValueResponse&& Response) mutable
					{
						MUTABLE_CPUPROFILER_SCOPE(MutableStreamRequest_DDC_Callback);
						bool bSuccess = Response.Status == EStatus::Ok;

						if (!bSuccess)
						{
							for (UE::Tasks::FTaskEvent& CompletionEvent : HeapMemory->CompletionEvents)
							{
								CompletionEvent.Trigger();
							}

							return;
						}
						
						for (int32 Index = 0; Index < BlockReadInfos.Num(); ++Index)
						{
							const FBlockReadInfo& Block = BlockReadInfos[Index];
							if (Block.FileId == Response.UserData)
							{
								const FCompressedBuffer& CompressedBuffer = Response.Value.GetData();
								if (!CompressedBuffer.IsNull())
								{
									const uint64 Size = Block.AllocatedMemoryView.Num();
									if (Size == CompressedBuffer.GetRawSize())
									{
										bSuccess = CompressedBuffer.TryDecompressTo(MakeMemoryView(Block.AllocatedMemoryView.GetData(), Size));
									}
									else if (Block.Offset + Size <= CompressedBuffer.GetRawSize())
									{
										FCompressedBufferReader BufferReader(CompressedBuffer);
										bSuccess = BufferReader.TryDecompressTo(MakeMemoryView(Block.AllocatedMemoryView.GetData(), Size), Block.Offset);
									}
								}

								HeapMemory->CompletionEvents[Index].Trigger();
							}
						}
					});

				return;
			}
#endif

			const bool bUseFBulkData = !ModelStreamableBulkData->StreamableBulkData.IsEmpty();
			const EAsyncIOPriorityAndFlags Priority = CVarMutableHighPriorityLoading.GetValueOnAnyThread() ? AIOP_High : AIOP_Normal;

			const int32 NumBlocks = BlockReadInfos.Num();
			for (int32 Index = 0; Index < NumBlocks; ++Index)
			{
				const FBlockReadInfo& Block = BlockReadInfos[Index];
				UE::Tasks::FTaskEvent& CompletionEvent = HeapMemory->CompletionEvents[Index];

				if (bUseFBulkData)
				{
					FBulkDataIORequestCallBack IOCallback =
						[CompletionEvent, FileId = Block.FileId](bool bWasCancelled, IBulkDataIORequest*) mutable
						{
							CompletionEvent.Trigger();
						};

					check(ModelStreamableBulkData->StreamableBulkData.IsValidIndex(Block.FileId));
					FByteBulkData& ByteBulkData = ModelStreamableBulkData->StreamableBulkData[Block.FileId];

					HeapMemory->BulkReadRequests.Add(TSharedPtr<IBulkDataIORequest>(ByteBulkData.CreateStreamingRequest(
						BulkDataFileOffset + Block.Offset,
						(int64)Block.AllocatedMemoryView.Num(),
						Priority,
						&IOCallback,
						Block.AllocatedMemoryView.GetData())));
				}
				else if (Block.FileHandle)
				{
					FAsyncFileCallBack ReadRequestCallBack =
						[CompletionEvent, FileId = Block.FileId](bool bWasCancelled, IAsyncReadRequest*) mutable
						{
							CompletionEvent.Trigger();
						};

					HeapMemory->ReadRequests.Add(TSharedPtr<IAsyncReadRequest>(Block.FileHandle->ReadRequest(
						BulkDataFileOffset + Block.Offset,
						(int64)Block.AllocatedMemoryView.Num(),
						Priority,
						&ReadRequestCallBack,
						Block.AllocatedMemoryView.GetData())));
				}
				else
				{
					ensure(false);
					CompletionEvent.Trigger();
				}
			}
		},
		UE::Tasks::ETaskPriority::High);
	
		
	return UE::Tasks::Launch(TEXT("GatherStreamingRequestsCompletionTask"),
        [
            OpenFileHandles = OpenFileHandles,
			HeapMemory = this->HeapMemory
        ]() mutable 
        {
			{
				FScopeLock Lock(&HeapMemory->ReadRequestLock);

				for (TSharedPtr<IAsyncReadRequest>& ReadRequest : HeapMemory->ReadRequests)
				{
					if (ReadRequest)
					{
						ReadRequest->WaitCompletion();
					}
				}

				for (TSharedPtr<IBulkDataIORequest>& BulkReadRequest : HeapMemory->BulkReadRequests)
				{
					if (BulkReadRequest)
					{
						BulkReadRequest->WaitCompletion();
					}
				}

				HeapMemory->BulkReadRequests.Empty();
				HeapMemory->ReadRequests.Empty();
			}
			
			OpenFileHandles.Empty();
        },
		HeapMemory->CompletionEvents,
        UE::Tasks::ETaskPriority::High);
}


void FMutableStreamRequest::Cancel()
{
	FScopeLock Lock(&HeapMemory->ReadRequestLock);

	if (!HeapMemory->bIsCancelled)
	{
		HeapMemory->bIsCancelled = true;

		for (TSharedPtr<IAsyncReadRequest>& ReadRequest : HeapMemory->ReadRequests)
		{
			if (ReadRequest)
			{
				ReadRequest->Cancel();
			}
		}

		for (TSharedPtr<IBulkDataIORequest>& BulkReadRequest : HeapMemory->BulkReadRequests)
		{
			if (BulkReadRequest)
			{
				BulkReadRequest->Cancel();
			}
		}
	}
}


FMutableStreamRequest::FBlockReadInfo::FBlockReadInfo(uint64 InOffset, IAsyncReadFileHandle* InFileHandle, TArrayView<uint8>& InAllocatedMemoryView,
	uint32 InFileId, UE::Mutable::Private::EStreamableDataType InDataType, uint16 InResourceType, uint16 InResourceFlags)
	: Offset(InOffset)
	, FileHandle(InFileHandle)
	, AllocatedMemoryView(InAllocatedMemoryView)
	, FileId(InFileId)
	, ResourceType(InResourceType)
	, ResourceFlags(InResourceFlags)
	, DataType(InDataType)
{
}
