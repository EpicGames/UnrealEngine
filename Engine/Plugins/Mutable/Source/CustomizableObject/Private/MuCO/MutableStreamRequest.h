// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MuCO/CustomizableObjectPrivate.h"

struct FBlockReadInfo;
struct FMutableStreamableBlock;
struct FModelStreamableBulkData;

#if WITH_EDITOR
#include "DerivedDataRequestOwner.h"
#endif

constexpr int32 BulkDataFileOffset =
#if WITH_EDITOR
	sizeof(MutableCompiledDataStreamHeader);
#else
		0;
#endif


/** Stream data from .mut or Bulk files.
 * 
 * Can be destroyed at any time, even when the stream request is in progress. */
class FMutableStreamRequest
{
public:
	FMutableStreamRequest(const TSharedPtr<FModelStreamableBulkData>& InModelStreamableBulkData);
	
	const TSharedPtr<FModelStreamableBulkData>& GetModelStreamableBulkData() const;

	/** Add a block to stream. */
	void AddBlock(const FMutableStreamableBlock& Block, UE::Mutable::Private::EStreamableDataType DataType, uint16 InResourceType, TArrayView<uint8> AllocatedMemoryView);

	/** Stream the previously requested data. */
	UE::Tasks::FTask Stream();

	/** Cancel pending requests */
	void Cancel();

private:
	struct FBlockReadInfo
	{
		FBlockReadInfo() = default;
		FBlockReadInfo(uint64 InOffset, IAsyncReadFileHandle* InFileHandle, TArrayView<uint8>& InAllocatedMemoryView, uint32 InFileId, UE::Mutable::Private::EStreamableDataType InDataType, uint16 InResourceType, uint16 InResourceFlags);

		uint64 Offset;
		IAsyncReadFileHandle* FileHandle;
		TArrayView<uint8> AllocatedMemoryView;
		uint32 FileId;
		uint16 ResourceType;
		uint16 ResourceFlags;

		UE::Mutable::Private::EStreamableDataType DataType;
	};
	
	const TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData;

	TArray<uint32> OpenFilesIds;
	TArray<TSharedPtr<IAsyncReadFileHandle>> OpenFileHandles;

	struct FHeapMemory
	{
		TArray<UE::Tasks::FTaskEvent> CompletionEvents;
		TArray<TSharedPtr<IAsyncReadRequest>> ReadRequests;
		TArray<TSharedPtr<IBulkDataIORequest>> BulkReadRequests;

#if WITH_EDITORONLY_DATA
		TSharedPtr<UE::DerivedData::FRequestOwner> DDCRequestOwner;
#endif

		bool bIsCancelled = false;
		FCriticalSection ReadRequestLock;
	};

	TSharedRef<FHeapMemory> HeapMemory = MakeShared<FHeapMemory>();

	TArray<FBlockReadInfo> BlockReadInfos;

	bool bIsStreaming = false;
};
