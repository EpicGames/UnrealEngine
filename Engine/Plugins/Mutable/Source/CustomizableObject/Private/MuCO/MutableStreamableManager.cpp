// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/MutableStreamableManager.h"


void FMutableStreamableManager::Tick(bool bBlocking)
{
	if (bBlocking)
	{
		for (const TSharedRef<FMutableStreamableHandle>& MutableStreamableHandle : StreamableHandles)
		{
			if (!MutableStreamableHandle->bCompleted)
			{
				TArray<FSoftObjectPath> RequestedAssets;
				MutableStreamableHandle->Handle->GetRequestedAssets(RequestedAssets);

				FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
				StreamableManager.RequestSyncLoad(RequestedAssets);

				MutableStreamableHandle->Delegate.ExecuteIfBound();
				MutableStreamableHandle->Handle->CancelHandle();
			}			
		}

		StreamableHandles.Empty();
	}
}


int32 FMutableStreamableManager::GetRemainingWork() const
{
	return StreamableHandles.Num();
}
