// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/UnrealConversionUtils.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuR/Skeleton.h"
#include "MuR/Mesh.h"
#include "MuR/SkeletalMesh.h"
#include "MuR/LOD.h"
#include "MuR/Material.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/MeshBufferUtils.h"
#include "MuR/MutableTrace.h"
#include "Animation/Skeleton.h"
#include "Containers/StaticArray.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "UObject/StrongObjectPtr.h"
#include "GPUSkinVertexFactory.h"
#include "SkinnedAssetCompiler.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/MorphTargetVertexCodec.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"


class USkeleton;

namespace UnrealConversionUtils
{
	// Hidden functions only used internally to aid other functions
	namespace
	{
		/**
		 * Initializes the static mesh vertex buffers object provided with the data found on the mutable buffers
		 * @param OutVertexBuffers - The Unreal's vertex buffers container to be updated with the mutable data.
		 * @param NumVertices - The amount of vertices on the buffer
		 * @param NumTexCoords - The amount of texture coordinates
		 * @param bUseFullPrecisionUVs - Determines if we want to use or not full precision UVs
		 * @param bUseFullPrecisionTangentBasis - Determines if we want to use or not full precision Tangent Basis
		 * @param bNeedCPUAccess - Determines if CPU access is required
		 * @param InMutablePositionData - Mutable position data buffer
		 * @param InMutableTangentData - Mutable tangent data buffer
		 * @param InMutableTextureData - Mutable texture data buffer
		 */
		void FStaticMeshVertexBuffers_InitWithMutableData(
			FStaticMeshVertexBuffers& OutVertexBuffers,
			const int32 NumVertices,
			const int32 NumTexCoords,
			const bool bUseFullPrecisionUVs,
			const bool bUseFullPrecisionTangentBasis,
			const bool bNeedCPUAccess,
			const void* InMutablePositionData,
			const void* InMutableTangentData,
			const void* InMutableTextureData)
		{
			// positions
			OutVertexBuffers.PositionVertexBuffer.Init(NumVertices, bNeedCPUAccess);
			FMemory::Memcpy(OutVertexBuffers.PositionVertexBuffer.GetVertexData(), InMutablePositionData, NumVertices * OutVertexBuffers.PositionVertexBuffer.GetStride());

			// tangent and texture coords
			
			check(!OutVertexBuffers.StaticMeshVertexBuffer.IsInitialized() || OutVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs() == bUseFullPrecisionUVs); // If already initialized to full precision, it must not change.
			OutVertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(bUseFullPrecisionUVs);
			
			check(!OutVertexBuffers.StaticMeshVertexBuffer.IsInitialized() || OutVertexBuffers.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis() == bUseFullPrecisionTangentBasis); // If already initialized to full precision, it must not change.
			OutVertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(bUseFullPrecisionTangentBasis);
			
			OutVertexBuffers.StaticMeshVertexBuffer.Init(NumVertices, NumTexCoords, bNeedCPUAccess);
			FMemory::Memcpy(OutVertexBuffers.StaticMeshVertexBuffer.GetTangentData(), InMutableTangentData, OutVertexBuffers.StaticMeshVertexBuffer.GetTangentSize());
			FMemory::Memcpy(OutVertexBuffers.StaticMeshVertexBuffer.GetTexCoordData(), InMutableTextureData, OutVertexBuffers.StaticMeshVertexBuffer.GetTexCoordSize());
		}


		/**
		 * Initializes the color vertex buffers object provided with the data found on the mutable buffers
		 * @param OutVertexBuffers - The Unreal's vertex buffers container to be updated with the mutable data.
		 * @param NumVertices - The amount of vertices on the buffer
		 * @param InMutableColorData - Mutable color data buffer
		 */
		void FColorVertexBuffers_InitWithMutableData(
			FStaticMeshVertexBuffers& OutVertexBuffers,
			const int32 NumVertices,
			const void* InMutableColorData
		)
		{
			// positions
			OutVertexBuffers.ColorVertexBuffer.Init(NumVertices);
			FMemory::Memcpy(OutVertexBuffers.ColorVertexBuffer.GetVertexData(), InMutableColorData, NumVertices * OutVertexBuffers.ColorVertexBuffer.GetStride());
		}


		/**
		 * Initializes the skin vertex buffers object provided with the data found on the mutable buffers
		 * @param OutVertexWeightBuffer - The Unreal's vertex buffers container to be updated with the mutable data.
		 * @param NumVertices - The amount of vertices on the buffer
		 * @param NumBones - The amount of bones to use to init the skin weights buffer
		 * @param NumBoneInfluences - The amount of bone influences on the buffer
		 * @param bNeedCPUAccess - Determines if CPU access is required
		 * @param InMutableData - Mutable data buffer
		 */
		void FSkinWeightVertexBuffer_InitWithMutableData(
			FSkinWeightVertexBuffer& OutVertexWeightBuffer,
			const int32 NumVertices,
			const int32 NumBones,
			const int32 NumBoneInfluences,
			const bool bNeedCPUAccess,
			const void* InMutableData,
			const uint32 MutableDataSize)
		{
			OutVertexWeightBuffer.SetNeedsCPUAccess(bNeedCPUAccess);

			FSkinWeightDataVertexBuffer* VertexBuffer = OutVertexWeightBuffer.GetDataVertexBuffer();
			VertexBuffer->SetMaxBoneInfluences(NumBoneInfluences);

			if (!NumVertices)
			{
				return;
			}

			int32 MaxBoneInfluences = VertexBuffer->GetMaxBoneInfluences();
			if (MaxBoneInfluences == NumBoneInfluences)
			{
				VertexBuffer->Init(NumBones, NumVertices);
				uint32 OutVertexWeightBufferSize = VertexBuffer->GetVertexDataSize();
				ensure(MutableDataSize == OutVertexWeightBufferSize);
				void* Data = VertexBuffer->GetWeightData();
				FMemory::Memcpy(Data, InMutableData, OutVertexWeightBufferSize);
			}
			else if (NumBones>0)
			{
				// We need to expand it with blank data interleaved
				uint32 MutableVertexDataSize = MutableDataSize / NumVertices;
				uint32 FinalVertexDataSize = ( MutableDataSize / NumBones) * MaxBoneInfluences;
				VertexBuffer->Init(NumVertices*MaxBoneInfluences, NumVertices);
				uint32 OutVertexWeightBufferSize = VertexBuffer->GetVertexDataSize();
				ensure(FinalVertexDataSize*NumVertices == OutVertexWeightBufferSize);

				int32 BoneIndexSize = OutVertexWeightBuffer.GetBoneIndexByteSize();
				int32 WeightSize = OutVertexWeightBuffer.GetBoneWeightByteSize();

				const uint8* MutableData = reinterpret_cast<const uint8*>(InMutableData);
				uint8* Data = VertexBuffer->GetWeightData();
				FMemory::Memzero(Data, OutVertexWeightBufferSize);
				for (int32 V=0; V<NumVertices; ++V)
				{
					// Bone indices
					FMemory::Memcpy(Data, MutableData, NumBoneInfluences*BoneIndexSize);
					MutableData += NumBoneInfluences * BoneIndexSize;
					Data += MaxBoneInfluences * BoneIndexSize;

					// Weights
					FMemory::Memcpy(Data, MutableData, NumBoneInfluences * WeightSize);
					MutableData += NumBoneInfluences * WeightSize;
					Data += MaxBoneInfluences * WeightSize;
				}
			}

			bool bIsVariableBonesPerVertex = VertexBuffer->GetVariableBonesPerVertex();
			check(!FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(NumBoneInfluences) || bIsVariableBonesPerVertex);
			if (bIsVariableBonesPerVertex)
			{
				OutVertexWeightBuffer.RebuildLookupVertexBuffer();

				{
					MUTABLE_CPUPROFILER_SCOPE(OptimizeVertexAndLookupBuffers);

					// Everything in this scope is optional and makes extra copies, but will optimize the variable bone
					// influences buffers. Without it, the vertices are assumed to have a constant NumBoneInfluences per vertex.
					TArray<FSkinWeightInfo> TempVertices;
					OutVertexWeightBuffer.GetSkinWeights(TempVertices);

					// The assignment operator actually optimizes the DataVertexBuffer
					OutVertexWeightBuffer = TempVertices;
				}
			}
	
		}

		/** 
		 * Compute the minimum and maximum values for the morphs models.
		 *
		 * @param MorphTargets     Morph target form which to compute the minimum an maximum values.
		 * @param OutMinimumValues Per morph minimum values result. Must have the same size as MorphTargets.
		 * @param OutMaximumValues Per morph maximum values result. Must have the same size as MorphTargets. 
		 */
		void ComputeMorphTargetsMinAndMaxValues(TConstArrayView<const FMorphTargetLODModel*> MorphTargets, TArrayView<FVector4f> OutMinimumValues, TArrayView<FVector4f> OutMaximumValues)
		{
			check(OutMinimumValues.Num() == MorphTargets.Num());
			check(OutMaximumValues.Num() == MorphTargets.Num());

			for (int32 MorphIndex = 0; MorphIndex < MorphTargets.Num(); ++MorphIndex)
			{
				const FMorphTargetLODModel* MorphLODModel = MorphTargets[MorphIndex];

				FVector4f MaximumValues = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
				FVector4f MinimumValues = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);

				if (MorphLODModel && MorphLODModel->Vertices.Num())
				{
					MaximumValues = FVector4f(-FLT_MAX, -FLT_MAX, -FLT_MAX, -FLT_MAX);
					MinimumValues = FVector4f( FLT_MAX,  FLT_MAX,  FLT_MAX,  FLT_MAX);

					for (const FMorphTargetDelta& MorphDelta : MorphLODModel->Vertices)
					{
						const FVector3f& PositionDelta = MorphDelta.PositionDelta;
						const FVector3f& TangentZDelta = MorphDelta.TangentZDelta;

						MaximumValues[0] = FMath::Max(MaximumValues[0], PositionDelta.X);
						MaximumValues[1] = FMath::Max(MaximumValues[1], PositionDelta.Y);
						MaximumValues[2] = FMath::Max(MaximumValues[2], PositionDelta.Z);
						MaximumValues[3] = FMath::Max(MaximumValues[3], FMath::Max(TangentZDelta.X, FMath::Max(TangentZDelta.Y, TangentZDelta.Z)));

						MinimumValues[0] = FMath::Min(MinimumValues[0], PositionDelta.X);
						MinimumValues[1] = FMath::Min(MinimumValues[1], PositionDelta.Y);
						MinimumValues[2] = FMath::Min(MinimumValues[2], PositionDelta.Z);
						MinimumValues[3] = FMath::Min(MinimumValues[3], FMath::Min(TangentZDelta.X, FMath::Min(TangentZDelta.Y, TangentZDelta.Z)));

					}
				}

				OutMinimumValues[MorphIndex] = MinimumValues;
				OutMaximumValues[MorphIndex] = MaximumValues;
			}
		}
	}

	bool SetupRenderSections(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		const FReferenceSkeleton& InRefSkeleton,
		const TArray<FName>& InBoneMap,
		const int32 InFirstBoneMapIndex)
	{
		check(InMutableMesh);

		const int32 SurfaceCount = InMutableMesh->GetSurfaceCount();
		if (SurfaceCount != LODResource.RenderSections.Num())
		{
			UE_LOGF(LogMutable, Error, "The amount of surfaces of the Mutable Mesh differs with the amount of RenderSections of the provided LODResource.");
			checkNoEntry();
			return false;
		}
		
		const UE::Mutable::Private::FMeshBufferSet& MutableMeshVertexBuffers = InMutableMesh->GetVertexBuffers();

		// Find the number of influences from this mesh
		int32 NumBoneInfluences = 0;
		int32 boneIndexBuffer = -1;
		int32 boneIndexChannel = -1;
		MutableMeshVertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::BoneIndices, 0, &boneIndexBuffer, &boneIndexChannel);
		if (boneIndexBuffer >= 0 || boneIndexChannel >= 0)
		{
			MutableMeshVertexBuffers.GetChannel(boneIndexBuffer, boneIndexChannel,
				nullptr, nullptr, nullptr, &NumBoneInfluences, nullptr);
		}
		
		for (int32 SurfaceIndex = 0; SurfaceIndex < SurfaceCount; ++SurfaceIndex)
		{
			MUTABLE_CPUPROFILER_SCOPE(SetupRenderSections);

			int32 FirstIndex;
			int32 IndexCount;
			int32 FirstVertex;
			int32 VertexCount;
			int32 FirstBone;
			int32 BoneCount;
			InMutableMesh->GetSurface(SurfaceIndex, FirstVertex, VertexCount, FirstIndex, IndexCount, FirstBone, BoneCount);
			FSkelMeshRenderSection& Section = LODResource.RenderSections[SurfaceIndex];

			Section.DuplicatedVerticesBuffer.Init(1, TMap<int, TArray<int32>>());

			if (!InMutableMesh->Surfaces.IsValidIndex(SurfaceIndex) || VertexCount == 0 || IndexCount == 0)
			{
				Section.bDisabled = true;
				continue; // Unreal doesn't like empty meshes
			}

			Section.BaseIndex = FirstIndex;
			Section.NumTriangles = IndexCount / 3;
			Section.BaseVertexIndex = FirstVertex;
			Section.MaxBoneInfluences = NumBoneInfluences;
			Section.NumVertices = VertexCount;
			Section.bCastShadow = InMutableMesh->Surfaces[SurfaceIndex].bCastShadow;
			Section.bRecomputeTangent = InMutableMesh->Surfaces[SurfaceIndex].bRecomputeTangent;
			
			// InBoneMaps may contain BoneMaps from other sections. Copy the bones belonging to this mesh.
			FirstBone += InFirstBoneMapIndex;
			
			if (InBoneMap.IsValidIndex(FirstBone + BoneCount - 1))
			{
				Section.BoneMap.Reserve(BoneCount);
				for (int32 BoneMapIndex = 0; BoneMapIndex < BoneCount; ++BoneMapIndex, ++FirstBone)
				{
					const int32 BoneIndex = InRefSkeleton.FindBoneIndex(InBoneMap[FirstBone]);
					if (BoneIndex != INDEX_NONE)
					{
						Section.BoneMap.Add(BoneIndex);
					}
					else
					{
						Section.BoneMap.Add(0);
					}
				}
			}
			else
			{
				Section.BoneMap.Init(0, BoneCount);
			}
		}
		
		return true;
	}


	bool IsUseFullPrecisionUVs(const UE::Mutable::Private::FMesh& Mesh)
	{
		int32 TexCoordsBufferIndex = -1;
		int32 TexCoordsChannelIndex = -1;
		Mesh.VertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::TexCoords, 0, &TexCoordsBufferIndex, &TexCoordsChannelIndex);
		if (TexCoordsBufferIndex >= 0 && TexCoordsChannelIndex >= 0)
		{
			return Mesh.VertexBuffers.Buffers[TexCoordsBufferIndex].Channels[TexCoordsChannelIndex].Format == UE::Mutable::Private::EMeshBufferFormat::Float32;
		}

		return true;
	}
	
	
	bool IsUseFullPrecisionTangentBasis(const UE::Mutable::Private::FMesh& Mesh)
	{
		int32 BufferIndex = -1;
		int32 ChannelIndex = -1;
		Mesh.VertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::Tangent, 0, &BufferIndex, &ChannelIndex);
		if (BufferIndex >= 0 && ChannelIndex >= 0)
		{
			return Mesh.VertexBuffers.Buffers[BufferIndex].Channels[ChannelIndex].Format == UE::Mutable::Private::EMeshBufferFormat::Float32;
		}
		
		return false;
	}
	
	
	void InitVertexBuffersWithDummyData(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		const bool bAllowCPUAccess)
	{
		MUTABLE_CPUPROFILER_SCOPE(InitVertexBuffersWithDummyData);

		const UE::Mutable::Private::FMeshBufferSet& MutableMeshVertexBuffers = InMutableMesh->GetVertexBuffers();
		check(MutableMeshVertexBuffers.GetElementCount() > 0);

		const bool bUseFullPrecisionUVs = IsUseFullPrecisionUVs(*InMutableMesh);
		const bool bUseFullPrecisionTangentBasis = IsUseFullPrecisionTangentBasis(*InMutableMesh);

		const uint32 NumVertices = MutableMeshVertexBuffers.GetElementCount();
		const uint32 NumTexCoords = MutableMeshVertexBuffers.GetBufferChannelCount(MUTABLE_VERTEXBUFFER_TEXCOORDS);

		const uint32 DummyNumVertices = 1;

		// Static Vertex buffers
		{
			LODResource.StaticVertexBuffers.PositionVertexBuffer.Init(DummyNumVertices, bAllowCPUAccess);
			LODResource.StaticVertexBuffers.PositionVertexBuffer.SetMetaData(LODResource.StaticVertexBuffers.PositionVertexBuffer.GetStride(), NumVertices);

			// tangent and texture coords
			LODResource.StaticVertexBuffers.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(bUseFullPrecisionUVs);
			LODResource.StaticVertexBuffers.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(bUseFullPrecisionTangentBasis);
			LODResource.StaticVertexBuffers.StaticMeshVertexBuffer.Init(DummyNumVertices, NumTexCoords, bAllowCPUAccess);
			LODResource.StaticVertexBuffers.StaticMeshVertexBuffer.SetMetaData(NumTexCoords, NumVertices, bUseFullPrecisionUVs, bUseFullPrecisionTangentBasis);
		}

		UE::Mutable::Private::EMeshBufferFormat BoneIndexFormat = UE::Mutable::Private::EMeshBufferFormat::None;
		int32 NumBoneInfluences = 0;
		int32 BoneIndexBuffer = -1;
		int32 BoneIndexChannel = -1;
		MutableMeshVertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::BoneIndices, 0, &BoneIndexBuffer, &BoneIndexChannel);
		if (BoneIndexBuffer >= 0 || BoneIndexChannel >= 0)
		{
			MutableMeshVertexBuffers.GetChannel(BoneIndexBuffer, BoneIndexChannel,
				nullptr, nullptr, &BoneIndexFormat, &NumBoneInfluences, nullptr);
		}

		UE::Mutable::Private::EMeshBufferFormat BoneWeightFormat = UE::Mutable::Private::EMeshBufferFormat::None;
		int32 BoneWeightBuffer = -1;
		int32 BoneWeightChannel = -1;
		MutableMeshVertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::BoneWeights, 0, &BoneWeightBuffer, &BoneWeightChannel);
		if (BoneWeightBuffer >= 0 || BoneWeightChannel >= 0)
		{
			MutableMeshVertexBuffers.GetChannel(BoneWeightBuffer, BoneWeightChannel,
				nullptr, nullptr, &BoneWeightFormat, nullptr, nullptr);
		}

		// Skin Weights
		{
			const bool bUse16BitBoneIndex = BoneIndexFormat == UE::Mutable::Private::EMeshBufferFormat::UInt16;
			const bool bUse16BitBoneWeights = BoneWeightFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16;

			LODResource.SkinWeightVertexBuffer.SetUse16BitBoneIndex(bUse16BitBoneIndex);
			LODResource.SkinWeightVertexBuffer.SetUse16BitBoneWeight(bUse16BitBoneWeights);
			LODResource.SkinWeightVertexBuffer.SetNeedsCPUAccess(bAllowCPUAccess);

			FSkinWeightDataVertexBuffer* VertexBuffer = LODResource.SkinWeightVertexBuffer.GetDataVertexBuffer();

			// NumBoneInfluences must be equal to MaxBoneInfluences. Set and then GetMaxBoneInfluences to know the number of bone influences to use (at least MAX_INFLUENCES_PER_STREAM).
			VertexBuffer->SetMaxBoneInfluences(NumBoneInfluences);
			NumBoneInfluences = VertexBuffer->GetMaxBoneInfluences();
			
			VertexBuffer->Init(NumBoneInfluences, DummyNumVertices);
			VertexBuffer->SetMetaData(NumVertices, NumBoneInfluences, bUse16BitBoneIndex, bUse16BitBoneWeights);

			if (VertexBuffer->GetVariableBonesPerVertex())
			{
				FSkinWeightLookupVertexBuffer* LookUpVertexBuffer = const_cast<FSkinWeightLookupVertexBuffer*>(LODResource.SkinWeightVertexBuffer.GetLookupVertexBuffer());
				LookUpVertexBuffer->SetMetaData(NumVertices);
			}
		}

		// Optional buffers
		for (int32 Buffer = MUTABLE_VERTEXBUFFER_TEXCOORDS + 1; Buffer < MutableMeshVertexBuffers.GetBufferCount(); ++Buffer)
		{
			if (MutableMeshVertexBuffers.GetBufferChannelCount(Buffer) > 0)
			{
				UE::Mutable::Private::EMeshBufferSemantic Semantic;
				UE::Mutable::Private::EMeshBufferFormat Format;
				int32 SemanticIndex;
				int32 ComponentCount;
				int32 Offset;
				MutableMeshVertexBuffers.GetChannel(Buffer, 0, &Semantic, &SemanticIndex, &Format, &ComponentCount, &Offset);

				// color buffer?
				if (Semantic == UE::Mutable::Private::EMeshBufferSemantic::Color)
				{
					LODResource.StaticVertexBuffers.ColorVertexBuffer.Init(1, bAllowCPUAccess);
					LODResource.StaticVertexBuffers.ColorVertexBuffer.SetMetaData(LODResource.StaticVertexBuffers.ColorVertexBuffer.GetStride(), NumVertices);
					
					check(LODResource.StaticVertexBuffers.ColorVertexBuffer.GetStride() == MutableMeshVertexBuffers.GetElementSize(Buffer));
				}
			}
		}
	}


	void CopyMutableVertexBuffers(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* MutableMesh,
		const bool bAllowCPUAccess)

	{
		MUTABLE_CPUPROFILER_SCOPE(CopyMutableVertexBuffers);

		const UE::Mutable::Private::FMeshBufferSet& MutableMeshVertexBuffers = MutableMesh->GetVertexBuffers();
		const int32 NumVertices = MutableMeshVertexBuffers.IsDescriptor()
				? 0 
				: MutableMeshVertexBuffers.GetElementCount();
		
		const bool bUseFullPrecisionUVs = IsUseFullPrecisionUVs(*MutableMesh);
		const bool bUseFullPrecisionTangentBasis = IsUseFullPrecisionTangentBasis(*MutableMesh);
		
		const int NumTexCoords = MutableMeshVertexBuffers.GetBufferChannelCount(MUTABLE_VERTEXBUFFER_TEXCOORDS);
		
		FStaticMeshVertexBuffers_InitWithMutableData(
			LODResource.StaticVertexBuffers,
			NumVertices,
			NumTexCoords,
			bUseFullPrecisionUVs,
			bUseFullPrecisionTangentBasis,
			bAllowCPUAccess,
			MutableMeshVertexBuffers.GetBufferData(MUTABLE_VERTEXBUFFER_POSITION),
			MutableMeshVertexBuffers.GetBufferData(MUTABLE_VERTEXBUFFER_TANGENT),
			MutableMeshVertexBuffers.GetBufferData(MUTABLE_VERTEXBUFFER_TEXCOORDS)
		);

		UE::Mutable::Private::EMeshBufferFormat BoneIndexFormat = UE::Mutable::Private::EMeshBufferFormat::None;
		int32 NumBoneInfluences = 0;
		int32 BoneIndexBuffer = -1;
		int32 BoneIndexChannel = -1;
		MutableMeshVertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::BoneIndices, 0, &BoneIndexBuffer, &BoneIndexChannel);
		if (BoneIndexBuffer >= 0 || BoneIndexChannel >= 0)
		{
			MutableMeshVertexBuffers.GetChannel(BoneIndexBuffer, BoneIndexChannel,
				nullptr, nullptr, &BoneIndexFormat, &NumBoneInfluences, nullptr);
		}

		UE::Mutable::Private::EMeshBufferFormat BoneWeightFormat = UE::Mutable::Private::EMeshBufferFormat::None;
		int32 BoneWeightBuffer = -1;
		int32 BoneWeightChannel = -1;
		MutableMeshVertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::BoneWeights, 0, &BoneWeightBuffer, &BoneWeightChannel);
		if (BoneWeightBuffer >= 0 || BoneWeightChannel >= 0)
		{
			MutableMeshVertexBuffers.GetChannel(BoneWeightBuffer, BoneWeightChannel,
				nullptr, nullptr, &BoneWeightFormat, nullptr, nullptr);
		}

		if (BoneIndexFormat == UE::Mutable::Private::EMeshBufferFormat::UInt16)
		{
			LODResource.SkinWeightVertexBuffer.SetUse16BitBoneIndex(true);
		}

		if (BoneWeightFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16)
		{
			LODResource.SkinWeightVertexBuffer.SetUse16BitBoneWeight(true);
		}

		// Init skin weight buffer
		FSkinWeightVertexBuffer_InitWithMutableData(
			LODResource.SkinWeightVertexBuffer,
			NumVertices,
			NumBoneInfluences * NumVertices,
			NumBoneInfluences,
			bAllowCPUAccess,
			MutableMeshVertexBuffers.GetBufferData(BoneIndexBuffer),
			MutableMeshVertexBuffers.GetBufferDataSize(BoneIndexBuffer)
		);

		// Optional buffers
		for (int32 Buffer = MUTABLE_VERTEXBUFFER_TEXCOORDS + 1; Buffer < MutableMeshVertexBuffers.GetBufferCount(); ++Buffer)
		{
			if (MutableMeshVertexBuffers.GetBufferChannelCount(Buffer) > 0)
			{
				UE::Mutable::Private::EMeshBufferSemantic Semantic;
				UE::Mutable::Private::EMeshBufferFormat Format;
				int32 SemanticIndex;
				int32 ComponentCount;
				int32 Offset;
				MutableMeshVertexBuffers.GetChannel(Buffer, 0, &Semantic, &SemanticIndex, &Format, &ComponentCount, &Offset);

				// color buffer?
				if (Semantic == UE::Mutable::Private::EMeshBufferSemantic::Color)
				{
					const void* DataPtr = MutableMeshVertexBuffers.GetBufferData(Buffer);
					FColorVertexBuffers_InitWithMutableData(LODResource.StaticVertexBuffers, NumVertices, DataPtr);
					check(LODResource.StaticVertexBuffers.ColorVertexBuffer.GetStride() == MutableMeshVertexBuffers.GetElementSize(Buffer));
				}
			}
		}
	}

	void InitIndexBuffersWithDummyData(FSkeletalMeshLODRenderData& LODResource, const UE::Mutable::Private::FMesh* InMutableMesh)
	{
		MUTABLE_CPUPROFILER_SCOPE(InitIndexBuffersWithDummyData);

		check(InMutableMesh->GetIndexBuffers().GetElementCount() > 0);
		
		const int32 NumIndices = InMutableMesh->GetIndexBuffers().GetElementCount();
		const int32 ElementSize = InMutableMesh->GetIndexBuffers().GetElementSize(0);

		LODResource.MultiSizeIndexContainer.CreateIndexBuffer(ElementSize);
		LODResource.MultiSizeIndexContainer.GetIndexBuffer()->SetMetaData(NumIndices);
	}

	bool CopyMutableIndexBuffers(
		FSkeletalMeshLODRenderData& LODResource,
		const UE::Mutable::Private::FMesh* InMutableMesh,
		bool& bOutMarkRenderStateDirty,
		TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap)
	{
		MUTABLE_CPUPROFILER_SCOPE(CopyMutableIndexBuffers);

		const int32 MutableIndexCount = InMutableMesh->GetIndexBuffers().GetElementCount();

		if (MutableIndexCount == 0)
		{
			// Copy indices from an empty buffer
			UE_LOGF(LogMutable, Error, "UCustomizableInstancePrivateData::BuildSkeletalMeshRenderData is converting an empty mesh.");
			return false;
		}

		if (InMutableMesh->GetIndexBuffers().IsDescriptor())
		{
			UE_LOGF(LogMutable, Error, "UCustomizableInstancePrivateData::BuildSkeletalMeshRenderData is converting a mesh descriptor.");
			return false;
		}

		const uint8* DataPtr = InMutableMesh->GetIndexBuffers().GetBufferData(0);
		const int32 ElementSize = InMutableMesh->GetIndexBuffers().GetElementSize(0);

		if (!LODResource.MultiSizeIndexContainer.IsIndexBufferValid())
		{
			LODResource.MultiSizeIndexContainer.CreateIndexBuffer(ElementSize);
		}
		
		check(LODResource.MultiSizeIndexContainer.GetDataTypeSize() == ElementSize);
		
		// UE-363228: Add a collapsed triangle whenever instead of disabling missing sections
		const bool bAddCollapsedTriangle = LODResource.RenderSections.Num() != InMutableMesh->Surfaces.Num();

		// Don't assume the buffer is empty, otherwise we may add extra elements using Insert(). 
		{
			const int32 IndexCount = MutableIndexCount + 3*(int32)bAddCollapsedTriangle;
			LODResource.MultiSizeIndexContainer.GetIndexBuffer()->Empty(IndexCount);
			LODResource.MultiSizeIndexContainer.GetIndexBuffer()->Insert(0, IndexCount);
		}

		FMemory::Memcpy(LODResource.MultiSizeIndexContainer.GetIndexBuffer()->GetPointerTo(0), DataPtr, MutableIndexCount * ElementSize);
	
		check(!RenderToMutableSectionIndexMap || RenderToMutableSectionIndexMap->Num() == LODResource.RenderSections.Num());
		for (int32 SectionIndex = 0; SectionIndex < LODResource.RenderSections.Num(); ++SectionIndex)
		{
			FSkelMeshRenderSection& Section = LODResource.RenderSections[SectionIndex];
			
			int32 MutableSectionIndex = SectionIndex;
			if (RenderToMutableSectionIndexMap)
			{
				MutableSectionIndex = (*RenderToMutableSectionIndexMap)[SectionIndex];
			}
			
			const UE::Mutable::Private::FMeshSurface* Surface = nullptr;
			if (MutableSectionIndex != INDEX_NONE)
			{
				Surface = &InMutableMesh->Surfaces[MutableSectionIndex];
			}

			if (Surface)
			{
				const int32 NumVertices = Surface->SubMeshes.Last().VertexEnd - Surface->SubMeshes[0].VertexBegin;

				bOutMarkRenderStateDirty |= Section.NumVertices != NumVertices;
				Section.NumVertices = NumVertices;

				const int32 IndexBegin = Surface->SubMeshes[0].IndexBegin; 
				const int32 IndexEnd = Surface->SubMeshes.Last().IndexEnd; 
			
				check(IndexBegin <= IndexEnd);
				check(IndexEnd*ElementSize <= LODResource.MultiSizeIndexContainer.GetIndexBuffer()->GetResourceDataSize())

				check((IndexEnd - IndexBegin) % 3 == 0);
				const int32 NumTriangles = (IndexEnd - IndexBegin) / 3;

				bOutMarkRenderStateDirty |= Section.NumTriangles != NumTriangles;
				Section.NumTriangles = NumTriangles;

				bOutMarkRenderStateDirty |= Section.BaseIndex != IndexBegin;
				Section.BaseIndex = IndexBegin;

				const int32 VertexBegin = Surface->SubMeshes[0].VertexBegin;

				bOutMarkRenderStateDirty |= Section.BaseVertexIndex != VertexBegin;
				Section.BaseVertexIndex = VertexBegin;
			}
			else
			{
				// Section.bDisabled = true; // UE-363228: Disabling sections after the mesh is generated produces visual artifacts.
				Section.NumTriangles = 1; // UE-363228: Add collapsed triangle to avoid visual artifacts
				Section.NumVertices = 0;
				Section.BaseIndex = MutableIndexCount;
				Section.BaseVertexIndex = 0;

				bOutMarkRenderStateDirty = true;	
			}
		}
		
		return true;
	}

	
	void CopyMutableSkinWeightProfilesBuffers(
		FSkeletalMeshLODRenderData& LODResource,
		USkeletalMesh& SkeletalMesh,
		int32 LODIndex,
		const UE::Mutable::Private::FMesh* InMutableMesh)
	{
		MUTABLE_CPUPROFILER_SCOPE(CopyMutableSkinWeightProfilesBuffers);

		const uint8 NumInfluences = LODResource.GetVertexBufferMaxBoneInfluences();
		const bool b16BitBoneIndices = LODResource.SkinWeightVertexBuffer.Use16BitBoneIndex();
		const bool b16BitBoneWeights = LODResource.SkinWeightVertexBuffer.Use16BitBoneWeight();

		const int32 BoneIndexByteSize = LODResource.SkinWeightVertexBuffer.GetBoneIndexByteSize();
		const int32 BoneWeightByteSize = LODResource.SkinWeightVertexBuffer.GetBoneWeightByteSize();

		const uint8 BoneIndicesStride = NumInfluences * BoneIndexByteSize;
		const uint8 BoneWeightsStride = NumInfluences * BoneWeightByteSize;

		for (const UE::Mutable::Private::FSkinWeightProfile& Profile : InMutableMesh->SkinWeightProfiles)
		{
			if (Profile.VertexIndexToInfluenceOffset.IsEmpty())
			{
				continue;
			}

			if (Profile.NumBoneInfluences == 0)
			{
				check(false);
				continue;
			}

			// Basic Buffer override settings
			FRuntimeSkinWeightProfileData& Override = LODResource.SkinWeightProfilesData.AddOverrideData(Profile.Name);
			Override.NumWeightsPerVertex = NumInfluences;
			Override.b16BitBoneIndices = b16BitBoneIndices;

			// BoneIndices channel info
			const uint8 SourceBoneIndexByteSize = Profile.bUse16BitBoneIndex ? 2 : 1;
			const uint8 SourceBoneIndicesStride = SourceBoneIndexByteSize * Profile.NumBoneInfluences;

			// BoneWeights channel info
			const uint8 SourceBoneWeightByteSize = Profile.bUse16BitBoneWeight ? 2 : 1;
			const uint8 SourceBoneWeightsStride = SourceBoneWeightByteSize * Profile.NumBoneInfluences;

			check(b16BitBoneIndices == Profile.bUse16BitBoneIndex);
			check(b16BitBoneWeights == Profile.bUse16BitBoneWeight);
			check(NumInfluences >= Profile.NumBoneInfluences);

			const int32 NumBoneWeights = Profile.BoneIDs.Num() / SourceBoneIndicesStride;

			// Copy Bone IDs. May add padding if source data has less influences.
			if (BoneIndexByteSize == SourceBoneIndexByteSize)
			{
				if (NumInfluences == Profile.NumBoneInfluences)
				{
					Override.BoneIDs = Profile.BoneIDs;
				}
				else
				{
					Override.BoneIDs.SetNumZeroed(NumBoneWeights * BoneIndicesStride);

					uint8* BoneIDsData = Override.BoneIDs.GetData();
					const uint8* SourceBoneIDsData = Profile.BoneIDs.GetData();

					for (int32 WeightIndex = 0; WeightIndex < NumBoneWeights; ++WeightIndex)
					{
						FMemory::Memcpy(BoneIDsData, SourceBoneIDsData, SourceBoneIndicesStride);

						BoneIDsData += BoneIndicesStride;
						SourceBoneIDsData += SourceBoneIndicesStride;
					}
				}
			}
			else
			{
				Override.BoneIDs.SetNumZeroed(NumBoneWeights * BoneIndicesStride);

				uint16* TypedDestBoneIDsData = reinterpret_cast<uint16*>(Override.BoneIDs.GetData());
				const uint8* SourceBoneIDsData = Profile.BoneIDs.GetData();

				for (int32 WeightIndex = 0; WeightIndex < NumBoneWeights; ++WeightIndex)
				{
					for (int32 InfluenceIndex = 0; InfluenceIndex < Profile.NumBoneInfluences; ++InfluenceIndex)
					{
						*(TypedDestBoneIDsData + InfluenceIndex) = *(SourceBoneIDsData + InfluenceIndex);
					}

					TypedDestBoneIDsData += NumInfluences;
					SourceBoneIDsData += SourceBoneIndicesStride;
				}
			}

			// Copy Bone Weights. May add padding if source data has less influences.
			if (BoneWeightByteSize == SourceBoneWeightByteSize)
			{
				if (NumInfluences == Profile.NumBoneInfluences)
				{
					Override.BoneWeights = Profile.BoneWeights;
				}
				else
				{
					Override.BoneWeights.SetNumZeroed(NumBoneWeights * BoneWeightsStride);

					uint8* DestBoneWeights = Override.BoneWeights.GetData();
					const uint8* SourceBoneWeights = Profile.BoneWeights.GetData();

					for (int32 WeightIndex = 0; WeightIndex < NumBoneWeights; ++WeightIndex)
					{
						FMemory::Memzero(DestBoneWeights, BoneWeightsStride);
						FMemory::Memcpy(DestBoneWeights, SourceBoneWeights, SourceBoneWeightsStride);

						DestBoneWeights += BoneWeightsStride;
						SourceBoneWeights += SourceBoneWeightsStride;
					}
				}
			}
			else
			{
				Override.BoneWeights.SetNumZeroed(NumBoneWeights * BoneWeightsStride);

				uint16* TypedDestBoneWeights = reinterpret_cast<uint16*>(Override.BoneWeights.GetData());
				const uint8* SourceBoneWeights = Profile.BoneWeights.GetData();

				for (int32 SourceWeightIndex = 0; SourceWeightIndex < NumBoneWeights; ++SourceWeightIndex)
				{
					FMemory::Memzero(TypedDestBoneWeights, BoneWeightsStride);

					for (int32 InfluenceIndex = 0; InfluenceIndex < Profile.NumBoneInfluences; ++InfluenceIndex)
					{
						*(TypedDestBoneWeights + InfluenceIndex) = static_cast<uint16>(*(SourceBoneWeights + InfluenceIndex)) * 257;
					}

					TypedDestBoneWeights += NumInfluences;
					SourceBoneWeights += SourceBoneWeightsStride;
				}
			}

			// Add VertexIndex to Influence Offset 
			const int32 NumVertices = Profile.VertexIndexToInfluenceOffset.Num();
			Override.VertexIndexToInfluenceOffset.Reserve(NumVertices);

			for (const UE::Mutable::Private::FSkinWeightProfile::FVertexInfo& VertexInfo : Profile.VertexIndexToInfluenceOffset)
			{
				Override.VertexIndexToInfluenceOffset.Add(VertexInfo.VertexIndex, VertexInfo.InfluenceOffset);
			}
		}
			
		LODResource.SkinWeightProfilesData.Init(&LODResource.SkinWeightVertexBuffer);
		SkeletalMesh.SetSkinWeightProfilesData(LODIndex, LODResource.SkinWeightProfilesData);
	}

	
	 void CopySkeletalMeshLODRenderData(
		 FSkeletalMeshLODRenderData& LODResource,
		 FSkeletalMeshLODRenderData& SourceLODResource,
		 USkeletalMesh& SkeletalMesh,
		 int32 LODIndex,
		 const bool bAllowCPUAccess
	 )
	 {
		 MUTABLE_CPUPROFILER_SCOPE(CopySkeletalMeshLODRenderData);

		 // Copying render sections
		 {
			 const int32 SurfaceCount = SourceLODResource.RenderSections.Num();
			 for (int32 SurfaceIndex = 0; SurfaceIndex < SurfaceCount; ++SurfaceIndex)
			 {
				 const FSkelMeshRenderSection& SrcSection = SourceLODResource.RenderSections[SurfaceIndex];
				 FSkelMeshRenderSection* DestSection = new(LODResource.RenderSections) FSkelMeshRenderSection();

				 DestSection->DuplicatedVerticesBuffer.Init(1, TMap<int, TArray<int32>>());
				 DestSection->bDisabled = SrcSection.bDisabled;

				 if (!DestSection->bDisabled)
				 {
					 DestSection->BaseIndex = SrcSection.BaseIndex;
					 DestSection->NumTriangles = SrcSection.NumTriangles;
					 DestSection->BaseVertexIndex = SrcSection.BaseVertexIndex;
					 DestSection->MaxBoneInfluences = SrcSection.MaxBoneInfluences;
					 DestSection->NumVertices = SrcSection.NumVertices;
					 DestSection->BoneMap = SrcSection.BoneMap;
					 DestSection->bCastShadow = SrcSection.bCastShadow;
					 DestSection->bRecomputeTangent = SrcSection.bRecomputeTangent;
				 }
			 }
		 }

		 const FStaticMeshVertexBuffers& SrcStaticVertexBuffer = SourceLODResource.StaticVertexBuffers;
		 FStaticMeshVertexBuffers& DestStaticVertexBuffer = LODResource.StaticVertexBuffers;

		 const int32 NumVertices = SrcStaticVertexBuffer.PositionVertexBuffer.GetNumVertices();
		 const int32 NumTexCoords = SrcStaticVertexBuffer.StaticMeshVertexBuffer.GetNumTexCoords();

		 // Copying Static Vertex Buffers
		 {
			 // Position buffer
			 DestStaticVertexBuffer.PositionVertexBuffer.Init(NumVertices, bAllowCPUAccess);
			 FMemory::Memcpy(DestStaticVertexBuffer.PositionVertexBuffer.GetVertexData(), SrcStaticVertexBuffer.PositionVertexBuffer.GetVertexData(), NumVertices * DestStaticVertexBuffer.PositionVertexBuffer.GetStride());

			 // Tangent and Texture coords buffers
	     	 const bool bUseFullPrecisionUVs = SrcStaticVertexBuffer.StaticMeshVertexBuffer.GetUseFullPrecisionUVs();
			 DestStaticVertexBuffer.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(bUseFullPrecisionUVs);
		 	
	         const bool bUseHighPrecisionTangentBasis =  SrcStaticVertexBuffer.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis();
			 DestStaticVertexBuffer.StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(bUseHighPrecisionTangentBasis);
			 
		 	 DestStaticVertexBuffer.StaticMeshVertexBuffer.Init(NumVertices, NumTexCoords, bAllowCPUAccess);
			 FMemory::Memcpy(DestStaticVertexBuffer.StaticMeshVertexBuffer.GetTangentData(), SrcStaticVertexBuffer.StaticMeshVertexBuffer.GetTangentData(), DestStaticVertexBuffer.StaticMeshVertexBuffer.GetTangentSize());
			 FMemory::Memcpy(DestStaticVertexBuffer.StaticMeshVertexBuffer.GetTexCoordData(), SrcStaticVertexBuffer.StaticMeshVertexBuffer.GetTexCoordData(), DestStaticVertexBuffer.StaticMeshVertexBuffer.GetTexCoordSize());

			 // Color buffer
			 if (LODResource.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
			 {
				 DestStaticVertexBuffer.ColorVertexBuffer.Init(NumVertices);
				 FMemory::Memcpy(DestStaticVertexBuffer.ColorVertexBuffer.GetVertexData(), SrcStaticVertexBuffer.ColorVertexBuffer.GetVertexData(), NumVertices * DestStaticVertexBuffer.ColorVertexBuffer.GetStride());
			 }
		 }

		 // Copying Skin Buffers
		 {
			 const FSkinWeightVertexBuffer& SrcSkinWeightBuffer = SourceLODResource.SkinWeightVertexBuffer;
			 FSkinWeightVertexBuffer& DestSkinWeightBuffer = LODResource.SkinWeightVertexBuffer;

			 int32 NumBoneInfluences = SrcSkinWeightBuffer.GetDataVertexBuffer()->GetMaxBoneInfluences();
			 int32 NumBones = SrcSkinWeightBuffer.GetDataVertexBuffer()->GetNumBoneWeights();

			 DestSkinWeightBuffer.SetUse16BitBoneIndex(SrcSkinWeightBuffer.Use16BitBoneIndex());
			 DestSkinWeightBuffer.SetNeedsCPUAccess(bAllowCPUAccess);

			 FSkinWeightDataVertexBuffer* SkinWeightDataVertexBuffer = DestSkinWeightBuffer.GetDataVertexBuffer();
			 SkinWeightDataVertexBuffer->SetMaxBoneInfluences(NumBoneInfluences);
			 SkinWeightDataVertexBuffer->Init(NumBones, NumVertices);

			 if (NumVertices)
			 {

				 const void* SrcData = SrcSkinWeightBuffer.GetDataVertexBuffer()->GetWeightData();
				 void* Data = SkinWeightDataVertexBuffer->GetWeightData();
				 check(SrcData);
				 check(Data);

				 FMemory::Memcpy(Data, SrcData, DestSkinWeightBuffer.GetVertexDataSize());
			 }
		 }

		 // Copying Skin Weight Profiles Buffers
		 {
			 const int32 NumSkinWeightProfiles = SkeletalMesh.GetSkinWeightProfiles().Num();
			 for (int32 ProfileIndex = 0; ProfileIndex < NumSkinWeightProfiles; ++ProfileIndex)
			 {
				 const FName& ProfileName = SkeletalMesh.GetSkinWeightProfiles()[ProfileIndex].Name;
				 
				 const FRuntimeSkinWeightProfileData* SourceProfile = SourceLODResource.SkinWeightProfilesData.GetOverrideData(ProfileName);
				 FRuntimeSkinWeightProfileData& DestProfile = LODResource.SkinWeightProfilesData.AddOverrideData(ProfileName);
				 
				 DestProfile = *SourceProfile;
			 }

			 LODResource.SkinWeightProfilesData.Init(&LODResource.SkinWeightVertexBuffer);
			 SkeletalMesh.SetSkinWeightProfilesData(LODIndex, LODResource.SkinWeightProfilesData);
		 }

		 // Copying Indices
		 {
			 if (SourceLODResource.MultiSizeIndexContainer.IsIndexBufferValid())
			 {
				 int32 IndexCount = SourceLODResource.MultiSizeIndexContainer.GetIndexBuffer()->Num();
				 int32 ElementSize = SourceLODResource.MultiSizeIndexContainer.GetDataTypeSize();

				 const void* Data = SourceLODResource.MultiSizeIndexContainer.GetIndexBuffer()->GetPointerTo(0);

				 LODResource.MultiSizeIndexContainer.CreateIndexBuffer(ElementSize);
				 LODResource.MultiSizeIndexContainer.GetIndexBuffer()->Insert(0, IndexCount);
				 FMemory::Memcpy(LODResource.MultiSizeIndexContainer.GetIndexBuffer()->GetPointerTo(0), Data, IndexCount * ElementSize);
			 }
		 }

		 LODResource.ActiveBoneIndices.Append(SourceLODResource.ActiveBoneIndices);
		 LODResource.RequiredBones.Append(SourceLODResource.RequiredBones);
		 LODResource.bIsLODOptional = SourceLODResource.bIsLODOptional;
		 LODResource.bStreamedDataInlined = SourceLODResource.bStreamedDataInlined;
		 LODResource.BuffersSize = SourceLODResource.BuffersSize;
	}


	void UpdateSkeletalMeshLODRenderDataBuffersSize(FSkeletalMeshLODRenderData& LODResource)
	{
		LODResource.BuffersSize = 0;
		
		// Add VertexBuffers' size
		LODResource.BuffersSize += LODResource.StaticVertexBuffers.PositionVertexBuffer.GetAllocatedSize();
		LODResource.BuffersSize += LODResource.StaticVertexBuffers.StaticMeshVertexBuffer.GetResourceSize();
		LODResource.BuffersSize += LODResource.StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize();
		LODResource.BuffersSize += LODResource.SkinWeightVertexBuffer.GetVertexDataSize();

		// Add Optional VertexBuffers' size
		LODResource.BuffersSize += LODResource.ClothVertexBuffer.GetVertexDataSize();
		LODResource.BuffersSize += LODResource.SkinWeightProfilesData.GetResourcesSize();
		LODResource.BuffersSize += LODResource.MorphTargetVertexInfoBuffers.GetMorphDataSizeInBytes();

		// Add IndexBuffer's size
		if (LODResource.MultiSizeIndexContainer.IsIndexBufferValid())
		{
			LODResource.BuffersSize += LODResource.MultiSizeIndexContainer.GetIndexBuffer()->GetResourceDataSize();
		}
	}

	void MorphTargetVertexInfoBuffers(
			FSkeletalMeshLODRenderData& LODResource, 
			const USkeletalMesh& Owner, 
			const UE::Mutable::Private::FMesh& MutableMesh, 
			int32 LODIndex, 
			bool bGenerateCPUMorphTargetsIfNeeded,
			TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap
	)
	{
		if (Owner.GetMorphTargets().IsEmpty())
		{
			return;
		}
	
		if (!MutableMesh.HasMorphs())
		{
			return;
		}

		TOptional<TArray<int32, TInlineAllocator<16>>> ReverseSectionIndexMap;
		if (RenderToMutableSectionIndexMap)
		{
			ReverseSectionIndexMap.Emplace();
			ReverseSectionIndexMap->Init(INDEX_NONE, MutableMesh.Surfaces.Num());

			for (int32 I = 0; I < RenderToMutableSectionIndexMap->Num(); ++I)
			{
				int32 SectionIndex = (*RenderToMutableSectionIndexMap)[I];
				if (SectionIndex != INDEX_NONE)
				{
					(*ReverseSectionIndexMap)[SectionIndex] = I;
				}
			}
		}

		// Get the global names map.
		const TMap<FName, int32>& IndexMap = Owner.GetMorphTargetIndexMap();
	
		// Reconstruct the final Morph Targets using the global names.

		const bool bGenerateCPUMorphTargets = 
				bGenerateCPUMorphTargetsIfNeeded && !FMorphTargetVertexInfoBuffers::IsPlatformShaderSupported(GMaxRHIShaderPlatform);

#if WITH_EDITOR
		TArray<FMorphTargetLODModel> MorphTargetLODs;
		ReconstructMorphTargetsFromMeshCompressedData(MutableMesh, MorphTargetLODs, UE::Mutable::Private::EMorphUsageFlags::RealTime);
#else
		TArray<FMorphTargetCompressedLODModel> CompressedMorphs;
		ReconstructMorphTargetsFromMeshCompressedData(MutableMesh, CompressedMorphs, UE::Mutable::Private::EMorphUsageFlags::RealTime);
#endif
		
#if WITH_EDITOR
		TArray<FMorphTargetLODModel*> MorphsToInit;
		MorphsToInit.Init(nullptr, Owner.GetMorphTargets().Num());

		for (int32 MorphIndex = MutableMesh.Morph.Names.Num() - 1; MorphIndex >= 0; --MorphIndex)
		{
			const int32* FoundIndex = IndexMap.Find(MutableMesh.Morph.Names[MorphIndex]);

			if (FoundIndex && MorphTargetLODs[MorphIndex].NumVertices)
			{
				MorphsToInit[*FoundIndex] = &MorphTargetLODs[MorphIndex];
			}
		}

		const float ErrorTolerance = Owner.GetLODInfo(LODIndex)->MorphTargetPositionErrorTolerance;
		const int32 NumVertices = LODResource.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices();

		const TBitArray UsesBuiltinMorphTargetCompression(true, MorphTargetLODs.Num());
		LODResource.MorphTargetVertexInfoBuffers.InitMorphResourcesStreaming(GMaxRHIShaderPlatform, LODResource.RenderSections, MorphsToInit, NumVertices, ErrorTolerance);
#else
		TArray<FVector4f> MorphTargetMinimumValues;
		TArray<FVector4f> MorphTargetMaximumValues;

		MorphTargetMinimumValues.SetNum(Owner.GetMorphTargets().Num());
		MorphTargetMaximumValues.SetNum(Owner.GetMorphTargets().Num());

		TArray<FMorphTargetCompressedLODModel*> CompressedMorphsToInit;
		CompressedMorphsToInit.Init(nullptr, Owner.GetMorphTargets().Num());

		for (int32 MorphIndex = MutableMesh.Morph.Names.Num() - 1; MorphIndex >= 0; --MorphIndex)
		{
			const int32* FoundIndex = IndexMap.Find(MutableMesh.Morph.Names[MorphIndex]);

			if (FoundIndex)
			{
				CompressedMorphsToInit[*FoundIndex] = &CompressedMorphs[MorphIndex];
				MorphTargetMinimumValues[*FoundIndex] = MutableMesh.Morph.MinimumValuePerMorph[MorphIndex]; 
				MorphTargetMaximumValues[*FoundIndex] = MutableMesh.Morph.MaximumValuePerMorph[MorphIndex];

				check(Owner.GetMorphTargets().IsValidIndex(*FoundIndex));
				UMorphTarget* MorphLOD = Owner.GetMorphTargets()[*FoundIndex];

				check(MorphLOD->GetMorphLODModels().IsValidIndex(LODIndex));

				MorphLOD->GetMorphLODModels()[LODIndex].SectionIndices = MutableMesh.Morph.SurfacesInUsePerMorph[MorphIndex];
				if (ReverseSectionIndexMap)
				{
					for (int32& SectionIndex : MorphLOD->GetMorphLODModels()[LODIndex].SectionIndices)
					{
						SectionIndex = (*ReverseSectionIndexMap)[SectionIndex];
					}
				}
			}
		}

		LODResource.MorphTargetVertexInfoBuffers.InitMorphResourcesStreaming(GMaxRHIShaderPlatform, CompressedMorphsToInit, MorphTargetMinimumValues, MorphTargetMaximumValues);
#endif
		if (bGenerateCPUMorphTargets)
		{
#if !WITH_EDITOR
			for (int32 Index = 0, NumMorphs = Owner.GetMorphTargets().Num(); Index < NumMorphs; ++Index)
			{
				if (CompressedMorphsToInit[Index])
				{
					UMorphTarget* MorphLOD = Owner.GetMorphTargets()[Index];
					check(MorphLOD->GetCompressedLODModels().IsValidIndex(LODIndex));
					MorphLOD->GetCompressedLODModels()[LODIndex] = MoveTemp(*CompressedMorphsToInit[Index]);
				}
			}
#else
			for (int32 Index = 0, NumMorphs = Owner.GetMorphTargets().Num(); Index < NumMorphs; ++Index)
			{
				if (MorphsToInit[Index])
				{
					UMorphTarget* MorphLOD = Owner.GetMorphTargets()[Index];

					check(MorphLOD->GetMorphLODModels().IsValidIndex(LODIndex));
					MorphLOD->GetMorphLODModels()[LODIndex].NumVertices = MorphsToInit[Index]->Vertices.Num();
					MorphLOD->GetMorphLODModels()[LODIndex].Vertices    = MoveTemp(MorphsToInit[Index]->Vertices);
				}
			}
#endif
		}
	}

	void SetMorphData(
			TNotNull<UE::Mutable::Private::FMesh*> Result, 
			TNotNull<const USkeletalMesh*> SkeletalMesh, 
			int32 LODIndex, int32 SectionIndex, 
			uint32 SourceSectionBaseVertexIndex, uint32 SourceSectionNumVertices, uint32 SourceMeshNumVertices, 
			FMorphTargetVertexInfoBuffers* OptionalMorphTargetVertexBuffer)
	{
		using namespace UE::MorphTargetVertexCodec;
		const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = SkeletalMesh->GetLODInfo(LODIndex);

		const TArray<TObjectPtr<UMorphTarget>>& MorphTargets = SkeletalMesh->GetMorphTargets();
		for (UMorphTarget* MorphTarget : MorphTargets)
		{
			if (!MorphTarget)
			{
				continue;
			}
			
			TArray<FMorphTargetLODModel>& MorphLODModels = MorphTarget->GetMorphLODModels();

			if (MorphLODModels.IsValidIndex(LODIndex) && MorphLODModels[LODIndex].SectionIndices.Contains(SectionIndex))
			{
				Result->Morph.Names.Add(MorphTarget->GetFName());
				Result->Morph.UsageFlagsPerMorph.Add(UE::Mutable::Private::EMorphUsageFlags::None);
			
				// Init with one surface. 
				Result->Morph.SurfacesInUsePerMorph.Emplace_GetRef().Init(0, 1);
			}
		}

		TArray<uint32> RuntimeCompressedMorphData;
		if (!OptionalMorphTargetVertexBuffer)
		{
			MUTABLE_CPUPROFILER_SCOPE(CompressMorphs)

			// In Editor morph data is not compressed and is located in FMorphTargetLODModel.	
			TArray<const FMorphTargetLODModel*> MorphTargetLODs;
			MorphTargetLODs.Reserve(MorphTargets.Num());

			for (UMorphTarget* MorphTarget : MorphTargets)
			{
				if (!MorphTarget)
				{
					continue;
				}

				TArray<FMorphTargetLODModel>& MorphLODModels = MorphTarget->GetMorphLODModels();
				if (MorphLODModels.IsValidIndex(LODIndex) && MorphLODModels[LODIndex].SectionIndices.Contains(SectionIndex))
				{
					MorphTargetLODs.Add(&MorphLODModels[LODIndex]);
				}
			}

			check(MorphTargetLODs.Num() == Result->Morph.Names.Num());
			
			Result->Morph.BatchStartOffsetPerMorph.Empty(MorphTargetLODs.Num());
			Result->Morph.BatchesPerMorph.Empty(MorphTargetLODs.Num());
			Result->Morph.MaximumValuePerMorph.Empty(MorphTargetLODs.Num());
			Result->Morph.MinimumValuePerMorph.Empty(MorphTargetLODs.Num());

			// NOTE: Here we are compressing the morphs for all sections. This could be avoided pre-filtering
			// the morph vertices in the section. This is done this way so the post-compress filtering is applied
			// in editor to mimic what will happen with cooked data.	
			
			const float PositionPrecision = ComputePositionPrecision(SkeletalMeshLODInfo->MorphTargetPositionErrorTolerance);
			const float TangentZPrecision = ComputeTangentPrecision();

			TArray<FDeltaBatchHeader> BatchHeaders;
			TArray<uint32> BitstreamData;
		
			for (const FMorphTargetLODModel* MorphModel : MorphTargetLODs)
			{
				if (!MorphModel)
				{
					continue;
				}
				
				uint32 BatchStartOffset = BatchHeaders.Num();
				
				FVector4f MinimumValues;
				FVector4f MaximumValues;

				ComputeMorphTargetsMinAndMaxValues(
						TConstArrayView<const FMorphTargetLODModel*>(&MorphModel, 1),
						TArrayView<FVector4f>(&MinimumValues, 1),
						TArrayView<FVector4f>(&MaximumValues, 1));

				// Encode the actual morph vertex info into the quantized bitstream. NeedsTangents set to null 
				// as we always need tangents.
				Encode(MorphModel->Vertices, nullptr, PositionPrecision, TangentZPrecision, BatchHeaders, BitstreamData);
			
				const uint32 MorphNumBatches = BatchHeaders.Num() - BatchStartOffset;
				Result->Morph.BatchStartOffsetPerMorph.Add(BatchStartOffset);
				Result->Morph.BatchesPerMorph.Add(MorphNumBatches);
				Result->Morph.MaximumValuePerMorph.Add(MaximumValues);
				Result->Morph.MinimumValuePerMorph.Add(MinimumValues);
			}
		
			Result->Morph.PositionPrecision = PositionPrecision;
			Result->Morph.TangentZPrecision = TangentZPrecision;
			Result->Morph.NumTotalBatches = BatchHeaders.Num();
			
			// Fix batch headers and write them packed.
			for (FDeltaBatchHeader& BatchHeader : BatchHeaders)
			{
				BatchHeader.DataOffset += BatchHeaders.Num() * NumBatchHeaderDwords*sizeof(uint32);

				TStaticArray<uint32, NumBatchHeaderDwords> HeaderData;
				WriteHeader(BatchHeader, HeaderData);
				RuntimeCompressedMorphData.Append(HeaderData);
			}

			RuntimeCompressedMorphData.Append(BitstreamData);
		}

		const TConstArrayView<uint32> SourceMorphDataView = !OptionalMorphTargetVertexBuffer 
				? TConstArrayView<uint32>(RuntimeCompressedMorphData)
				: TConstArrayView<uint32>(OptionalMorphTargetVertexBuffer->GetData(), OptionalMorphTargetVertexBuffer->GetMorphDataSizeInBytes()/sizeof(uint32));

		if (SourceMorphDataView.GetData())
		{	
			MUTABLE_CPUPROFILER_SCOPE(ExtractSectionMorphs)
			UE::Mutable::Private::FMeshMorph& Morph = Result->Morph;
		
			if (OptionalMorphTargetVertexBuffer)
			{
				check(OptionalMorphTargetVertexBuffer->IsMorphCPUDataValid());

				const int32 NumMorphs = OptionalMorphTargetVertexBuffer->GetNumMorphs();
				Morph.MaximumValuePerMorph.SetNumUninitialized(NumMorphs);
				Morph.MinimumValuePerMorph.SetNumUninitialized(NumMorphs);
				Morph.BatchStartOffsetPerMorph.SetNumUninitialized(NumMorphs);
				Morph.BatchesPerMorph.SetNumUninitialized(NumMorphs);

				for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
				{
					Morph.MaximumValuePerMorph[MorphIndex] = OptionalMorphTargetVertexBuffer->GetMaximumMorphScale(MorphIndex);
					Morph.MinimumValuePerMorph[MorphIndex] = OptionalMorphTargetVertexBuffer->GetMinimumMorphScale(MorphIndex);
					Morph.BatchStartOffsetPerMorph[MorphIndex] = OptionalMorphTargetVertexBuffer->GetBatchStartOffset(MorphIndex);
					Morph.BatchesPerMorph[MorphIndex] = OptionalMorphTargetVertexBuffer->GetNumBatches(MorphIndex);
					Morph.NumTotalBatches = OptionalMorphTargetVertexBuffer->GetNumBatches();
					Morph.PositionPrecision = OptionalMorphTargetVertexBuffer->GetPositionPrecision();
					Morph.TangentZPrecision = OptionalMorphTargetVertexBuffer->GetTangentZPrecision();
				}
			}

			// Strip all vertices not in this section.
			const int32 SectionVertexRangeBegin = SourceSectionBaseVertexIndex;
			const int32 SectionVertexRangeEnd   = SourceSectionBaseVertexIndex + SourceSectionNumVertices;

			uint32 TotalNumBatches = 0;
			uint32 CumulativeDataOffsetInDwords = 0;

			int32 NumMorphs = Morph.BatchStartOffsetPerMorph.Num();
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{	
				const int32 SourceMorphNumBatches = Morph.BatchesPerMorph[MorphIndex];
				const int32 SourceBatchOffset = Morph.BatchStartOffsetPerMorph[MorphIndex]; 

				if (SourceMorphNumBatches == 0)
				{
					continue;
				}

				int32 BatchIndex = 0;
				for ( ; BatchIndex < SourceMorphNumBatches; ++BatchIndex)
				{
					FDeltaBatchHeader BatchHeader;
					TConstArrayView<uint32> BatchHeaderDataView(
							SourceMorphDataView.GetData() + (SourceBatchOffset + BatchIndex) * NumBatchHeaderDwords, 
							NumBatchHeaderDwords); 

					ReadHeader(BatchHeader, BatchHeaderDataView);

					if (BatchHeader.IndexMin >= (uint32)SectionVertexRangeBegin)
					{
						break;
					}
				}

				// The previous batch could have deltas in the section range.
				int32 FirstSectionBatchIndex = FMath::Max(0, BatchIndex - 1);

				for ( ; BatchIndex < SourceMorphNumBatches; ++BatchIndex)
				{
					FDeltaBatchHeader BatchHeader;
					TConstArrayView<uint32> BatchHeaderDataView(
							SourceMorphDataView.GetData() + (SourceBatchOffset + BatchIndex) * NumBatchHeaderDwords, 
							NumBatchHeaderDwords); 

					ReadHeader(BatchHeader, BatchHeaderDataView);

					if (BatchHeader.IndexMin >= (uint32)SectionVertexRangeEnd)
					{
						break;
					}
				}
				
				int32 LastSectionBatchIndex = FMath::Max(0, BatchIndex - 1);

				uint32 NumSectionMorphBatches = LastSectionBatchIndex + 1 - FirstSectionBatchIndex;
				
				TConstArrayView<uint32> SectionMorphHeadersView(
							SourceMorphDataView.GetData() + (SourceBatchOffset + FirstSectionBatchIndex) * NumBatchHeaderDwords, 
							NumSectionMorphBatches * NumBatchHeaderDwords);

				Result->MorphDataBuffer.Append(SectionMorphHeadersView);

				Result->Morph.BatchStartOffsetPerMorph[MorphIndex] = CumulativeDataOffsetInDwords / NumBatchHeaderDwords;
				Result->Morph.BatchesPerMorph[MorphIndex] = NumSectionMorphBatches;
				TotalNumBatches += NumSectionMorphBatches;
				CumulativeDataOffsetInDwords += SectionMorphHeadersView.Num();
			}
			
			Result->Morph.NumTotalBatches = TotalNumBatches;

			auto TrimBatchDeltasNotInRange = [](
					int32 VertexRangeBegin, int32 VertexRangeEnd,
					FDeltaBatchHeader& InOutBatchHeader,
					TConstArrayView<uint32> DeltasData,
					TArrayView<uint32> OutBatchDeltasData)
			{
				TConstArrayView<uint32> BatchDeltasData(
						DeltasData.GetData() + (InOutBatchHeader.DataOffset / sizeof(uint32)), 
						CalculateBatchDwords(InOutBatchHeader));
			
				TArray<FQuantizedDelta, TInlineAllocator<UE::MorphTargetVertexCodec::BatchSize>> QuantizedDeltas;
				QuantizedDeltas.SetNumUninitialized(UE::MorphTargetVertexCodec::BatchSize);

				ReadQuantizedDeltas(QuantizedDeltas, InOutBatchHeader, BatchDeltasData);

				int32 NewIndexMin = InOutBatchHeader.IndexMin;
				uint32 DeltaIndex = 0;
				for ( ; DeltaIndex < InOutBatchHeader.NumElements; ++DeltaIndex)
				{
					int32 DeltaVertexIdx = QuantizedDeltas[DeltaIndex].Index;

					if (DeltaVertexIdx >= VertexRangeBegin)
					{
						NewIndexMin = DeltaVertexIdx;
						break;
					}
				}

				int32 DeltaIndexBegin = DeltaIndex;

				for ( ; DeltaIndex < InOutBatchHeader.NumElements; ++DeltaIndex)
				{
					int32 DeltaVertexIdx = QuantizedDeltas[DeltaIndex].Index;
					
					if (DeltaVertexIdx >= VertexRangeEnd)
					{
						break;
					}
				}

				InOutBatchHeader.IndexMin    = NewIndexMin;
				InOutBatchHeader.NumElements = DeltaIndex - DeltaIndexBegin;

				check(InOutBatchHeader.NumElements * sizeof(FQuantizedDelta) <= QuantizedDeltas.NumBytes());
				FMemory::Memmove(
						QuantizedDeltas.GetData(), 
						QuantizedDeltas.GetData() + DeltaIndexBegin,
						InOutBatchHeader.NumElements * sizeof(FQuantizedDelta));
				
				WriteQuantizedDeltas(QuantizedDeltas, InOutBatchHeader, OutBatchDeltasData);
			};

			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				const int32 MorphNumBatches = Result->Morph.BatchesPerMorph[MorphIndex];
				const int32 BatchOffset = Result->Morph.BatchStartOffsetPerMorph[MorphIndex]; 
		
				if (MorphNumBatches == 0)
				{
					continue;
				}

				TArrayView<uint32> MorphBatchHeadersView(
						Result->MorphDataBuffer.GetData() + BatchOffset * NumBatchHeaderDwords,
						MorphNumBatches * NumBatchHeaderDwords);

				// Batches still point to the source data.
				FDeltaBatchHeader FirstBatchHeader;
				TArrayView<uint32> FirstBatchHeaderDataView(
						MorphBatchHeadersView.GetData(), NumBatchHeaderDwords);
				ReadHeader(FirstBatchHeader, FirstBatchHeaderDataView);

				FDeltaBatchHeader LastBatchHeader;
				TArrayView<uint32> LastBatchHeaderDataView(
						MorphBatchHeadersView.GetData() + (MorphNumBatches - 1)*NumBatchHeaderDwords, NumBatchHeaderDwords);
				ReadHeader(LastBatchHeader, LastBatchHeaderDataView);

				FDeltaBatchHeader FirstFullBatchHeader;
				FDeltaBatchHeader LastFullBatchHeader;
				
				TConstArrayView<uint32> SourceFullBatchesDataView;
				if (MorphNumBatches > 2)
				{
					TArrayView<uint32> FirstFullBatchHeaderDataView(
							MorphBatchHeadersView.GetData() + 1*NumBatchHeaderDwords, NumBatchHeaderDwords);
					ReadHeader(FirstFullBatchHeader, FirstFullBatchHeaderDataView);

					TArrayView<uint32> LastFullBatchHeaderDataView(
							MorphBatchHeadersView.GetData() + (MorphNumBatches - 2)*NumBatchHeaderDwords, NumBatchHeaderDwords);
					ReadHeader(LastFullBatchHeader, LastFullBatchHeaderDataView);

					SourceFullBatchesDataView = TConstArrayView<uint32>(
						SourceMorphDataView.GetData() + (FirstFullBatchHeader.DataOffset / sizeof(uint32)),
						((LastFullBatchHeader.DataOffset - FirstFullBatchHeader.DataOffset) / sizeof(uint32)) + CalculateBatchDwords(LastFullBatchHeader));
				}

				TArray<uint32, TInlineAllocator<BatchSize*sizeof(FQuantizedDelta)/sizeof(uint32)>> PartialBatchDeltaStorage;
				uint32 PartialBatchDeltaStorageDwords = FMath::Max(CalculateBatchDwords(FirstBatchHeader), CalculateBatchDwords(LastBatchHeader));
				PartialBatchDeltaStorage.SetNumUninitialized(PartialBatchDeltaStorageDwords);

				// We are potentially reallocating, views will be invalidated after the reserve or any append.
				// TODO: We can know an accurate upper-bound of the final size beforehand only looking at source 
				// headers, pre-compute the size and only make one allocation.
				Result->MorphDataBuffer.Reserve(
						MorphBatchHeadersView.Num() + SourceFullBatchesDataView.Num() + 
						CalculateBatchDwords(FirstBatchHeader) + CalculateBatchDwords(LastBatchHeader));

				TrimBatchDeltasNotInRange(
						SectionVertexRangeBegin, SectionVertexRangeEnd, FirstBatchHeader, SourceMorphDataView, PartialBatchDeltaStorage);
				
				FirstBatchHeaderDataView = TArrayView<uint32>(
						Result->MorphDataBuffer.GetData() + BatchOffset*NumBatchHeaderDwords, NumBatchHeaderDwords);
				WriteHeader(FirstBatchHeader, FirstBatchHeaderDataView);			

				// Batch size may have changed, CalculateBatchDwords value cannot be cached from previous query. 
				Result->MorphDataBuffer.Append(
						TArrayView<uint32>(PartialBatchDeltaStorage.GetData(), CalculateBatchDwords(FirstBatchHeader)));
			
				// SourceFullBatchesDataView is expected to be empty in some cases.
				Result->MorphDataBuffer.Append(SourceFullBatchesDataView);

				if (MorphNumBatches > 1)
				{
					PartialBatchDeltaStorage.SetNumUninitialized(CalculateBatchDwords(LastBatchHeader), EAllowShrinking::No);

					TrimBatchDeltasNotInRange(
							SectionVertexRangeBegin, SectionVertexRangeEnd, LastBatchHeader, SourceMorphDataView, PartialBatchDeltaStorage);
					
					LastBatchHeaderDataView = TArrayView<uint32>(
							Result->MorphDataBuffer.GetData() + (BatchOffset + MorphNumBatches - 1)*NumBatchHeaderDwords, NumBatchHeaderDwords);
					WriteHeader(LastBatchHeader, LastBatchHeaderDataView);					

					// Batch size may have changed, CalculateBatchDwords value cannot be cached from previous query. 
					Result->MorphDataBuffer.Append(
							TArrayView<uint32>(PartialBatchDeltaStorage.GetData(), CalculateBatchDwords(LastBatchHeader)));
				}

				int32 NumMorphBatches = Result->Morph.BatchesPerMorph[MorphIndex]; 

				// Batch header fixup.
				TArrayView<uint32> BatchHeadersData(
						Result->MorphDataBuffer.GetData() + Result->Morph.BatchStartOffsetPerMorph[MorphIndex] * NumBatchHeaderDwords, 
						NumMorphBatches * NumBatchHeaderDwords);
				
				for (int32 BatchIndex = 0; BatchIndex < NumMorphBatches; ++BatchIndex)
				{
					TArrayView<uint32> BatchHeaderData(
							BatchHeadersData.GetData() + BatchIndex*NumBatchHeaderDwords, NumBatchHeaderDwords);

					FDeltaBatchHeader BatchHeader;
					ReadHeader(BatchHeader, BatchHeaderData);

					if (BatchHeader.NumElements != 0)
					{
						check(BatchHeader.IndexMin >= (uint32)SectionVertexRangeBegin);
						BatchHeader.IndexMin = BatchHeader.IndexMin - SectionVertexRangeBegin;
						BatchHeader.DataOffset = CumulativeDataOffsetInDwords * sizeof(uint32);
						CumulativeDataOffsetInDwords += CalculateBatchDwords(BatchHeader);
					}
					else
					{
						BatchHeader.IndexMin   = 0;
						BatchHeader.DataOffset = CumulativeDataOffsetInDwords * sizeof(uint32);
					}

					WriteHeader(BatchHeader, BatchHeaderData);
				}
			}
		}
	}

	struct FMeshConversionContext
	{
		TStrongObjectPtr<USkeletalMesh> SkeletalMesh;
		int32 LODIndex = 0;

		FSkeletalMeshLODRenderData StreamedLOD { /* AddRef= */ false };
		FSkeletalMeshLODRenderData* OriginalLOD = nullptr;
		FSkeletalMeshLODRenderData* DataLOD = nullptr;
		TUniquePtr<IBulkDataIORequest> Request;
	};

	void ExtractSkeletalMeshSection(TNotNull<FMeshConversionContext*> Context, int32 SectionIndex, EMutableMeshConversionFlags ConversionFlags, TNotNull<UE::Mutable::Private::FMesh*> OutResult)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshConversion);

		using namespace UE::Mutable::Private;

		const int32 LODIndex = Context->LODIndex;
		USkeletalMesh *const SkeletalMesh = Context->SkeletalMesh.Get();

		if (!Context->OriginalLOD->RenderSections.IsValidIndex(SectionIndex) ||
			Context->OriginalLOD->RenderSections[SectionIndex].bDisabled)
		{
			return;
		}
	
		const FSkelMeshRenderSection& Section = Context->OriginalLOD->RenderSections[SectionIndex];
		
		const bool bIgnoreSkinning = EnumHasAnyFlags(ConversionFlags, EMutableMeshConversionFlags::IgnoreSkinning);
		const bool bIgnorePhysics = bIgnoreSkinning || EnumHasAnyFlags(ConversionFlags, EMutableMeshConversionFlags::IgnorePhysics);
		const bool bIgnoreMorphs = EnumHasAnyFlags(ConversionFlags, EMutableMeshConversionFlags::IgnoreMorphs);
		const bool bIgnoreTexCoords = EnumHasAnyFlags(ConversionFlags, EMutableMeshConversionFlags::IgnoreTexCoords);

		OutResult->MeshIDPrefix = UINT32_MAX;
		
		if (!bIgnoreSkinning)
		{
			MUTABLE_CPUPROFILER_SCOPE(MeshConversion_Skeleton)
			USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
			if (!Skeleton)
			{
				ensure(false);
				return;
			}

			OutResult->SkeletonObjects.Add(TPassthroughObjectPtr(Skeleton));

			// Create the skeleton, poses, and BoneMap for this mesh
			TManagedPtr<FSkeleton> MutableSkeleton = 
					MakeManaged<FSkeleton>();
			OutResult->SetSkeleton(MutableSkeleton);

			const TArray<uint16>& RequiredBones = Context->OriginalLOD->RequiredBones;
			const int32 NumRequiredBones = RequiredBones.Num();
			OutResult->SetBonePoseCount(NumRequiredBones);

			MutableSkeleton->BoneNames.Reserve(NumRequiredBones);
			MutableSkeleton->BoneParents.Reserve(NumRequiredBones);

			const TArray<uint16>& BoneMap = Section.BoneMap;
			const int32 NumBonesInBoneMap = BoneMap.Num();

			TArray<FBoneIdOrIndex> MutableBoneMap;
			MutableBoneMap.SetNum(NumBonesInBoneMap);

			TArray<FMatrix> ComposedRefPoseMatrices;
			ComposedRefPoseMatrices.SetNum(NumRequiredBones);

			const TArray<FMeshBoneInfo>& RefBoneInfo = SkeletalMesh->GetRefSkeleton().GetRefBoneInfo();
			for (int32 BoneIndex = 0; BoneIndex < NumRequiredBones; ++BoneIndex)
			{
				const int32 RefSkeletonBoneIndex = RequiredBones[BoneIndex];

				const FMeshBoneInfo& BoneInfo = RefBoneInfo[RefSkeletonBoneIndex];
				const int32 ParentBoneIndex = RequiredBones.Find(BoneInfo.ParentIndex);

				FBoneIdOrIndex MutableBoneIndex;
				MutableBoneIndex.Index = MutableSkeleton->AddBone(BoneInfo.Name, ParentBoneIndex);

				// BoneMap: Convert RefSkeletonBoneIndex to BoneId
				const int32 BoneMapIndex = BoneMap.Find(RefSkeletonBoneIndex);
				if (BoneMapIndex != INDEX_NONE)
				{
					MutableBoneMap[BoneMapIndex] = MutableBoneIndex;
				}

				if (ParentBoneIndex >= 0)
				{
					ComposedRefPoseMatrices[BoneIndex] = SkeletalMesh->GetRefPoseMatrix(RefSkeletonBoneIndex) * ComposedRefPoseMatrices[ParentBoneIndex];
				}
				else
				{
					ComposedRefPoseMatrices[BoneIndex] = SkeletalMesh->GetRefPoseMatrix(RefSkeletonBoneIndex);
				}

				// Set bone pose
				FTransform3f BoneTransform;
				BoneTransform.SetFromMatrix(FMatrix44f(ComposedRefPoseMatrices[BoneIndex]));

				EBoneUsageFlags BoneUsageFlags = EBoneUsageFlags::None;
				EnumAddFlags(BoneUsageFlags, BoneMapIndex != INDEX_NONE ? EBoneUsageFlags::Skinning : EBoneUsageFlags::None);
				EnumAddFlags(BoneUsageFlags, ParentBoneIndex == INDEX_NONE ? EBoneUsageFlags::Root : EBoneUsageFlags::None);

				OutResult->SetBonePose(BoneIndex, MutableBoneIndex, BoneTransform, BoneUsageFlags);
			}

			OutResult->SetBoneMap(MutableBoneMap);
		}

		if (!bIgnoreSkinning)
		{
			for (int32 SocketIndex = 0; SocketIndex < SkeletalMesh->NumSockets(); ++SocketIndex)
			{
				const USkeletalMeshSocket* Socket = SkeletalMesh->GetSocketByIndex(SocketIndex);
				check(Socket);

				FMeshSocket MutableSocket;
				MutableSocket.SocketName = Socket->SocketName;
				MutableSocket.BoneName = Socket->BoneName;
				MutableSocket.RelativeLocation = Socket->RelativeLocation;
				MutableSocket.RelativeRotation = Socket->RelativeRotation;
				MutableSocket.RelativeScale = Socket->RelativeScale;
				MutableSocket.bForceAlwaysAnimated = Socket->bForceAlwaysAnimated;

				OutResult->Sockets.Add(MutableSocket);
			}
		}

		// Physics
		TObjectPtr<UPhysicsAsset> PhysicsAsset = SkeletalMesh->GetPhysicsAsset();
		if (!bIgnorePhysics && PhysicsAsset && OutResult->GetSkeleton())
		{
			TManagedPtr<FPhysicsBody> PhysicsBody = UnrealConversionUtils::CreatePhysicsBodyForMesh(*PhysicsAsset, *OutResult);

			if (PhysicsBody)
			{
				OutResult->SetPhysicsBody(PhysicsBody);
				OutResult->PhysicsAssets.Emplace(TPassthroughObjectPtr(PhysicsAsset.Get()));

				// Set Physics usage flag
				const int32 NumBodySetups = PhysicsBody->GetBodyCount();
				for (int32 BodyIndex = 0; BodyIndex < NumBodySetups; ++BodyIndex)
				{
					const int32 PoseIndex = OutResult->Skeleton->BoneNames.Find(PhysicsBody->BodiesBoneNames[BodyIndex]); // Bone Poses are 1 : 1 with the skeleton bones
					check(PoseIndex != INDEX_NONE);
					EnumAddFlags(OutResult->BonePoses[PoseIndex].BoneUsageFlags, EBoneUsageFlags::Physics);
				}
			}
		}
		
		{
			// Mesh data
			int32 FirstVertexIndex = Section.BaseVertexIndex;
			int32 VertexCount = Section.GetNumVertices();

			FMeshBufferSet& Vertices = OutResult->GetVertexBuffers();
			if (!Context->DataLOD)
			{
				EnumAddFlags(Vertices.Flags, EMeshBufferSetFlags::IsDescriptor);
			}

			Vertices.SetElementCount(VertexCount, EMemoryInitPolicy::Zeroed);
			Vertices.SetBufferCount(5);

			int32 CurrentVertexBuffer = 0;

			// Position buffer
			{	
				MUTABLE_CPUPROFILER_SCOPE(MeshConversion_Positions)
				MeshBufferUtils::SetupVertexPositionsBuffer(CurrentVertexBuffer, Vertices);

				if (Context->DataLOD)
				{
					int32 ElementSize = Vertices.Buffers[CurrentVertexBuffer].ElementSize;
					const uint8* SourceVertexData = ((const uint8*)Context->DataLOD->StaticVertexBuffers.PositionVertexBuffer.GetVertexData()) + FirstVertexIndex * ElementSize;

					check(ElementSize * (FirstVertexIndex + VertexCount) <= Context->DataLOD->StaticVertexBuffers.PositionVertexBuffer.GetAllocatedSize());
					FMemory::Memcpy(Vertices.GetBufferData(CurrentVertexBuffer), SourceVertexData, ElementSize * VertexCount);

				}

				++CurrentVertexBuffer;
			}

			// Tangent buffer
			{
				MUTABLE_CPUPROFILER_SCOPE(MeshConversion_Tangent)

				bool bIsHighPrecision = Context->OriginalLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis();

				const EMeshBufferSemantic Semantics[2] = { EMeshBufferSemantic::Tangent, EMeshBufferSemantic::Normal };
				const int32 SemanticIndices[2] = { 0, 0 };
				const int32 Components[2] = { 4, 4 };
				EMeshBufferFormat Formats[2] = { EMeshBufferFormat::PackedDirS8, EMeshBufferFormat::PackedDirS8_W_TangentSign };
				int32 Offsets[2] = { 0, 4 };
				int32 ElementSize = 8;

				if (bIsHighPrecision)
				{
					// Not really supported
					ensure(false);
					Formats[0] = EMeshBufferFormat::Int16;
					Formats[1] = EMeshBufferFormat::Int16;
					Offsets[1] = 8;
					ElementSize = 16;
				}
				Vertices.SetBuffer(CurrentVertexBuffer, ElementSize, 2, Semantics, SemanticIndices, Formats, Components, Offsets);

				if (Context->DataLOD)
				{
					const uint8* SourceVertexData = ((const uint8*)Context->DataLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentData()) + FirstVertexIndex * ElementSize;

					check((int32)ElementSize * (FirstVertexIndex + VertexCount) <= Context->DataLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetTangentSize());
					FMemory::Memcpy(Vertices.GetBufferData(CurrentVertexBuffer), SourceVertexData, ElementSize * VertexCount);
				}

				++CurrentVertexBuffer;
			}

			// Texture coords buffer
			if (!bIgnoreTexCoords)
			{
				MUTABLE_CPUPROFILER_SCOPE(MeshConversion_TexCoords)

				int32 NumTexCoords = Context->OriginalLOD->GetNumTexCoords();

				constexpr int32 MaxChannelCount = 4;
				EMeshBufferSemantic Semantics[MaxChannelCount];
				int32 SemanticIndices[MaxChannelCount];
				EMeshBufferFormat Formats[MaxChannelCount];
				int32 Components[MaxChannelCount];
				int32 Offsets[MaxChannelCount];

				int32 UVSize = Context->OriginalLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs() ? 8 : 4;
				const int32 ElementSize = UVSize * NumTexCoords;

				for (int32 UV = 0; UV < NumTexCoords; ++UV)
				{
					Semantics[UV] = EMeshBufferSemantic::TexCoords;
					SemanticIndices[UV] = UV;
					Formats[UV] = (UVSize == 8) ? EMeshBufferFormat::Float32 : EMeshBufferFormat::Float16;
					Components[UV] = 2;
					Offsets[UV] = UVSize * UV;
				}
				Vertices.SetBuffer(CurrentVertexBuffer, ElementSize, NumTexCoords, Semantics, SemanticIndices, Formats, Components, Offsets);

				if (Context->DataLOD)
				{
					check(Context->OriginalLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs() == Context->DataLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs());
					const uint8* SourceVertexData = ((const uint8*)Context->DataLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetTexCoordData()) + FirstVertexIndex * UVSize * NumTexCoords;
					check(UVSize * NumTexCoords * (FirstVertexIndex + VertexCount) <= Context->DataLOD->StaticVertexBuffers.StaticMeshVertexBuffer.GetTexCoordSize());

					FMemory::Memcpy(Vertices.GetBufferData(CurrentVertexBuffer), SourceVertexData, Vertices.Buffers[CurrentVertexBuffer].ElementSize * VertexCount);
				}

				++CurrentVertexBuffer;
			}

			// Skin buffer
			if (!bIgnoreSkinning)
			{
				if (Section.MaxBoneInfluences > 0)
				{
					MUTABLE_CPUPROFILER_SCOPE(MeshConversion_Skin)

					const FSkinWeightVertexBuffer* OriginalSkinBuffer = Context->OriginalLOD->GetSkinWeightVertexBuffer();
					const int32 MaxBoneIndexTypeSizeBytes = OriginalSkinBuffer->GetBoneIndexByteSize();
					const int32 MaxBoneWeightTypeSizeBytes = OriginalSkinBuffer->GetBoneWeightByteSize();
					const int32 MaxBonesPerVertex = OriginalSkinBuffer->GetMaxBoneInfluences();
					MeshBufferUtils::SetupSkinBuffer(CurrentVertexBuffer, MaxBoneIndexTypeSizeBytes, MaxBoneWeightTypeSizeBytes, MaxBonesPerVertex, Vertices);

					if (Context->DataLOD)
					{
						const FSkinWeightVertexBuffer* SkinBuffer = Context->DataLOD->GetSkinWeightVertexBuffer();
						switch (SkinBuffer->GetBoneInfluenceType())
						{
						case GPUSkinBoneInfluenceType::DefaultBoneInfluence:
						{
							int32 ElementSize = Vertices.Buffers[CurrentVertexBuffer].ElementSize;
							const uint8* SourceVertexData = ((const uint8*)SkinBuffer->GetDataVertexBuffer()->GetWeightData()) + FirstVertexIndex * ElementSize;

							check(ElementSize * (FirstVertexIndex + VertexCount) <= (int32)SkinBuffer->GetVertexDataSize());

							FMemory::Memcpy(Vertices.GetBufferData(CurrentVertexBuffer), SourceVertexData, ElementSize * VertexCount);
							break;
						}
						case GPUSkinBoneInfluenceType::UnlimitedBoneInfluence:
						{
							check(SkinBuffer->GetVariableBonesPerVertex());

							int32 ElementSize = Vertices.Buffers[CurrentVertexBuffer].ElementSize;
							uint8* DestVertexData = Vertices.GetBufferData(CurrentVertexBuffer);
							FMemory::Memzero(DestVertexData, VertexCount * ElementSize);

							const uint8* SourceVertexData = (const uint8*)SkinBuffer->GetDataVertexBuffer()->GetWeightData();
							const FSkinWeightLookupVertexBuffer* LookUpVertexBuffer = SkinBuffer->GetLookupVertexBuffer();
							for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
							{
								uint32 WeightOffset = 0;
								uint32 InfluenceCount = 0;
								SkinBuffer->GetVertexInfluenceOffsetCount(FirstVertexIndex + VertexIndex, WeightOffset, InfluenceCount);

								// Bone indices
								int32 SourceIndicesDataSize = InfluenceCount * MaxBoneIndexTypeSizeBytes;
								FMemory::Memcpy(DestVertexData, SourceVertexData + WeightOffset, SourceIndicesDataSize);
								DestVertexData += MaxBonesPerVertex * MaxBoneIndexTypeSizeBytes;


								// Weights
								int32 SourceWeightDataSize = InfluenceCount * MaxBoneWeightTypeSizeBytes;
								FMemory::Memcpy(DestVertexData, SourceVertexData + WeightOffset + SourceIndicesDataSize, SourceWeightDataSize);
								DestVertexData += MaxBonesPerVertex * MaxBoneWeightTypeSizeBytes;
							}

							check((int32)(DestVertexData - Vertices.GetBufferData(CurrentVertexBuffer)) == (int32)(VertexCount * ElementSize));
							break;
						}
						default:
							unimplemented();
							break;
						}
					}

					++CurrentVertexBuffer;
				}
			}

			// Color buffer
			if (Context->OriginalLOD->StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0)
			{
				MUTABLE_CPUPROFILER_SCOPE(MeshConversion_Color)

				MeshBufferUtils::SetupVertexColorBuffer(CurrentVertexBuffer, Vertices);

				if (Context->DataLOD)
				{
					const FColorVertexBuffer& ColorBuffer = Context->DataLOD->StaticVertexBuffers.ColorVertexBuffer;
					
					int32 ElementSize = Vertices.Buffers[CurrentVertexBuffer].ElementSize;
					const uint8* SourceVertexData = ((const uint8*)ColorBuffer.GetVertexData()) + FirstVertexIndex * ElementSize;

					check(ElementSize * (FirstVertexIndex + VertexCount) <= (int32)ColorBuffer.GetAllocatedSize());
					check(ElementSize == ColorBuffer.GetStride());

					FMemory::Memcpy(Vertices.GetBufferData(CurrentVertexBuffer), SourceVertexData, ElementSize * VertexCount);
				}

				++CurrentVertexBuffer;
			}


			// Indices
			{
				MUTABLE_CPUPROFILER_SCOPE(MeshConversion_Indices)

				int32 FirstIndexIndex = Section.BaseIndex;
				int32 IndexCount = Section.NumTriangles * 3;
				int32 ElementSize = Context->OriginalLOD->MultiSizeIndexContainer.GetDataTypeSize();

				FMeshBufferSet& Indices = OutResult->GetIndexBuffers();
				
				if (!Context->DataLOD)
				{
					EnumAddFlags(Indices.Flags, EMeshBufferSetFlags::IsDescriptor);
				}

				Indices.SetBufferCount(1);
				Indices.SetElementCount(IndexCount);
				constexpr int32 ChannelCount = 1;
				const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::VertexIndex };
				const int32 SemanticIndices[ChannelCount] = { 0 };
				EMeshBufferFormat Formats[ChannelCount] = { ElementSize == 4 ? EMeshBufferFormat::UInt32 : EMeshBufferFormat::UInt16 };
				const int32 Components[ChannelCount] = { 1 };
				const int32 Offsets[ChannelCount] = { 0 };
				Indices.SetBuffer(0, ElementSize, ChannelCount, Semantics, SemanticIndices, Formats, Components, Offsets);


				if (Context->DataLOD)
				{
					const uint8* SourceData = (const uint8*)Context->DataLOD->MultiSizeIndexContainer.GetIndexBuffer()->GetPointerTo(FirstIndexIndex);
					uint8* TargetData = Indices.GetBufferData(0);

					if (FirstVertexIndex == 0)
					{
						FMemory::Memcpy(TargetData, SourceData, IndexCount * ElementSize);
					}
					else
					{
						// Apply vertex offset
						switch (ElementSize)
						{
						case 2:
							for (int32 Index = 0; Index < IndexCount; ++Index)
							{
								const uint16* Source = ((const uint16*)SourceData) + Index;
								uint16* Target = ((uint16*)TargetData) + Index;
								check(*Source >= FirstVertexIndex);
								*Target = *Source - FirstVertexIndex;
							}
							break;
						case 4:
							for (int32 Index = 0; Index < IndexCount; ++Index)
							{
								const uint32* Source = ((const uint32*)SourceData) + Index;
								uint32* Target = ((uint32*)TargetData) + Index;
								check(*Source >= uint32(FirstVertexIndex));
								*Target = *Source - FirstVertexIndex;
							}
							break;
						default:
							// Index size not implemented
							break;
						}
					}
				}
			}

			if (!bIgnoreMorphs)
			{
				FMorphTargetVertexInfoBuffers* MorphTargetBuffers = nullptr;
				if (Context->DataLOD && Context->DataLOD->MorphTargetVertexInfoBuffers.IsMorphCPUDataValid())
				{
					MorphTargetBuffers = &Context->DataLOD->MorphTargetVertexInfoBuffers;
				}

				SetMorphData(
					OutResult, SkeletalMesh,
					LODIndex, SectionIndex,
					(int32)Section.BaseVertexIndex, (int32)Section.NumVertices, (int32)Context->OriginalLOD->GetNumVertices(),
					MorphTargetBuffers);

				if (EnumHasAnyFlags(ConversionFlags, EMutableMeshConversionFlags::AddMorphsAsRealTime))
				{
					for (EMorphUsageFlags& UsageFlags : OutResult->Morph.UsageFlagsPerMorph)
					{
						EnumAddFlags(UsageFlags, EMorphUsageFlags::RealTime);
					}
				}
			}

			OutResult->EnsureSurfaceData();
			if (OutResult->Surfaces.Num())
			{
				int32 MaterialIndex = Section.MaterialIndex;
				const FSkeletalMeshLODInfo* SourceLODInfo = SkeletalMesh->GetLODInfo(Context->LODIndex);
				if (SourceLODInfo && SourceLODInfo->LODMaterialMap.IsValidIndex(SectionIndex) && SourceLODInfo->LODMaterialMap[SectionIndex] != INDEX_NONE)
				{
					MaterialIndex = SourceLODInfo->LODMaterialMap[SectionIndex];
				}

				check(OutResult->Surfaces.Num() == 1);
				OutResult->Surfaces[0].Id = MaterialIndex;
				OutResult->Surfaces[0].bCastShadow = Section.bCastShadow;
				OutResult->Surfaces[0].bRecomputeTangent = Section.bRecomputeTangent;
			}

			// Cloth
			if (!bIgnoreSkinning)
			{
				FSkeletalMeshVertexClothBuffer& ClothVertexBuffer = Context->OriginalLOD->ClothVertexBuffer;

				const TArray<FClothBufferIndexMapping>& SectionClothIndexMappings = ClothVertexBuffer.GetClothIndexMapping();

				const bool bHasClothData = SectionClothIndexMappings.IsValidIndex(SectionIndex);
				
				UClothingAssetBase* ClothingAsset = SkeletalMesh->GetClothingAsset(Section.ClothingData.AssetGuid);
				if (bHasClothData && ClothingAsset)
				{
					OutResult->ClothSections[0].ClothingAsset = TPassthroughObjectPtr(ClothingAsset);
					OutResult->ClothSections[0].AssetLODIndex = Section.ClothingData.AssetLodIndex;

					const FClothBufferIndexMapping& SectionClothIndexMapping = ClothVertexBuffer.GetClothIndexMapping()[SectionIndex];
					
					uint32 FirstClothVertexNextSection = ClothVertexBuffer.GetNumVertices();
					for (int32 MappingIndex = SectionIndex + 1; MappingIndex < SectionClothIndexMappings.Num(); ++MappingIndex)
					{
						const int32 MappingOffset = SectionClothIndexMappings[MappingIndex].MappingOffset;
						if (MappingOffset)
						{
							FirstClothVertexNextSection = MappingOffset;
							break;
						}
					}

					const uint32 NumClothVertex = FirstClothVertexNextSection - SectionClothIndexMapping.MappingOffset;
					const uint32 NumInfluences = SectionClothIndexMapping.LODBiasStride / Section.NumVertices;

					uint32 NumClothLOD = NumClothVertex / (NumInfluences * Section.NumVertices);
					if (NumClothLOD > 1)
					{
						NumClothLOD = 1;
						UE_LOGF(LogMutable, Error, "\"Mappings to Same LOD\" only supported in Ray Tracing \"Cloth LODBias Mode\" option for [%ls] Skeletal Mesh", *SkeletalMesh->GetName());
					}

					if (Context->DataLOD)
					{
						uint32 NumVertices = SectionClothIndexMapping.LODBiasStride * NumClothLOD;

						OutResult->ClothSections[0].Data.SetNumUninitialized(NumVertices);
						FMemory::Memcpy(OutResult->ClothSections[0].Data.GetData(), &Context->DataLOD->ClothVertexBuffer.MappingData(SectionClothIndexMapping.MappingOffset), NumVertices * sizeof(FMeshToMeshVertData));
					}
				}
			}
		}

		// Asset User Data
		for (UAssetUserData* AssetUserData : *SkeletalMesh->GetAssetUserDataArray())
		{
			OutResult->AssetUserData.Add(AssetUserData);
		}
	}

	UE::Tasks::FTask LaunchLoadSkeletalMeshLODRenderDataTask(TSharedPtr<FMeshConversionContext> Context)
	{
		USkeletalMesh* SkeletalMesh = Context->SkeletalMesh.Get();

#if WITH_EDITOR
		if (!ensure(!SkeletalMesh->IsCompiling()))
		{
			return UE::Tasks::MakeCompletedTask<void>();
		}
#endif

		return UE::Tasks::Launch(TEXT("SkeletalMeshLoadTask"),
		[
			Context
		]()
		{
			const int32 LODIndex = Context->LODIndex;
			USkeletalMesh *const SkeletalMesh = Context->SkeletalMesh.Get();

#if WITH_EDITOR
			const bool bNeedsCPUData = true;
#else
			const bool bNeedsCPUData = SkeletalMesh->NeedCPUData(LODIndex);
#endif

			if (Context->OriginalLOD->bStreamedDataInlined && bNeedsCPUData)
			{
				Context->DataLOD = Context->OriginalLOD;
			}
			else if (Context->OriginalLOD->StreamingBulkData.CanLoadFromDisk())
			{
				MUTABLE_CPUPROFILER_SCOPE(MeshStreaming);

				UE::Tasks::FTaskEvent LoadedEvent(TEXT("MutableMeshParamLoadCompleted"));
				UE::Tasks::AddNested(LoadedEvent);

				const int32 NumRenderSections = Context->OriginalLOD->RenderSections.Num();
				Context->StreamedLOD.RenderSections.SetNum(NumRenderSections);

				for (int32 SectionIndex = 0; SectionIndex < NumRenderSections; ++SectionIndex)
				{
					const FSkelMeshRenderSection& OriginalSection = Context->OriginalLOD->RenderSections[SectionIndex];
					FSkelMeshRenderSection& StreamedSection = Context->StreamedLOD.RenderSections[SectionIndex];

					if (OriginalSection.HasClothingData())
					{
						StreamedSection.ClothMappingDataLODs.SetNum(1);
						StreamedSection.ClothMappingDataLODs[0].SetNum(1);
					}
				}

				FBulkDataIORequestCallBack RequestCallback =
				[
					LoadedEvent, 
					SkeletalMesh, 
					WeakContext = Context.ToWeakPtr(), 
					LODIndex
				](bool bWasCancelled, IBulkDataIORequest* IORequest) mutable
				{
					uint8* BulkData = IORequest->GetReadResults();

					ON_SCOPE_EXIT
					{
						LoadedEvent.Trigger();
						FMemory::Free(BulkData);
					};

					if (!bWasCancelled)
					{
						check(BulkData != nullptr);	

						TSharedPtr<FMeshConversionContext> Context = WeakContext.Pin();
						if (Context)
						{
							MUTABLE_CPUPROFILER_SCOPE(MeshStreamingSerialization);

							FMemoryReaderView Ar(TArrayView<uint8>(BulkData, IORequest->GetSize()), true);

							constexpr uint8 DummyStripFlags = 0;
							const bool bForceKeepCPUResources = false;
							const bool bNeedsCPUAccess = false;
							Context->StreamedLOD.SerializeStreamedData(Ar, SkeletalMesh, LODIndex, DummyStripFlags, bNeedsCPUAccess, bForceKeepCPUResources);

							if (!Ar.IsError() && !Ar.IsCriticalError())
							{
								Context->DataLOD = &Context->StreamedLOD;
							}
							else
							{
								UE_LOGF(LogMutable, Error, "Error in SerializeStreamedData LOD %d Data from Skeletal Mesh %ls.", LODIndex, *SkeletalMesh->GetName());
							}
						}
						else
						{
							check(false); // Context is owned by the conversion task. It should be always valid.
						}
					}
					else
					{
						UE_LOGF(LogMutable, Error, "Request for LOD %d Data from Skeletal Mesh %ls was cancelled.", LODIndex, *SkeletalMesh->GetName());
					}

				};

				Context->Request.Reset(Context->OriginalLOD->StreamingBulkData.CreateStreamingRequest(EAsyncIOPriorityAndFlags::AIOP_High, &RequestCallback, nullptr) );
			}
		});
	}

	int32 FindSectionAtLOD(USkeletalMesh& SkeletalMesh, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias)
	{
		const int32 TargetLODIndex = BaseLODIndex + LODBias;
		if (TargetLODIndex < 0 || TargetLODIndex >= SkeletalMesh.GetLODNum())
		{
			return INDEX_NONE;
		}

		const FSkeletalMeshRenderData* RenderData = SkeletalMesh.GetResourceForRendering();
		if (!RenderData || !RenderData->LODRenderData.IsValidIndex(BaseLODIndex) ||
			!RenderData->LODRenderData[BaseLODIndex].RenderSections.IsValidIndex(BaseSectionIndex))
		{
			return INDEX_NONE;
		}

		if (LODBias == 0)
		{
			return BaseSectionIndex;
		}

		const FSkeletalMeshLODInfo* SourceLODInfo = SkeletalMesh.GetLODInfo(BaseLODIndex);
		const FSkeletalMeshLODInfo* TargetLODInfo = SkeletalMesh.GetLODInfo(BaseLODIndex + LODBias);
		if (!SourceLODInfo || !TargetLODInfo)
		{
			return INDEX_NONE;
		}

		const int32 SourceMaterialIndex = SourceLODInfo->LODMaterialMap.IsValidIndex(BaseSectionIndex) && SourceLODInfo->LODMaterialMap[BaseSectionIndex] != INDEX_NONE ?
			SourceLODInfo->LODMaterialMap[BaseSectionIndex] :
			RenderData->LODRenderData[BaseLODIndex].RenderSections[BaseSectionIndex].MaterialIndex;

		const FSkeletalMeshLODRenderData& TargetLODRenderData = RenderData->LODRenderData[TargetLODIndex];

		const int32 NumSections = TargetLODRenderData.RenderSections.Num();
		for (int32 Index = 0; Index < NumSections; ++Index)
		{
			const int32 MaterialIndex = TargetLODInfo->LODMaterialMap.IsValidIndex(Index) && TargetLODInfo->LODMaterialMap[Index] != INDEX_NONE ?
				TargetLODInfo->LODMaterialMap[Index] :
				TargetLODRenderData.RenderSections[Index].MaterialIndex;

			// Note: This maps mesh sections across LODs by MaterialSlot. If two mesh sections within a LOD use the same MaterialSlot it will return the first one.
			if (MaterialIndex == SourceMaterialIndex)
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}

	UE::Tasks::FTask ConvertSkeletalMeshFromRuntimeData(USkeletalMesh* SkeletalMesh, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 ConversionFlags, UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> Result)
	{
		MUTABLE_CPUPROFILER_SCOPE(ConvertSkeletalMeshFromRuntimeData);

		if (!SkeletalMesh)
		{
			return UE::Tasks::MakeCompletedTask<void>();
		}
		
		const int32 LODIndex = BaseLODIndex + LODBias;
		const int32 SectionIndex = FindSectionAtLOD(*SkeletalMesh, BaseLODIndex, BaseSectionIndex, LODBias);
		if (SectionIndex == INDEX_NONE)
		{
			return UE::Tasks::MakeCompletedTask<void>();
		}

		TSharedPtr<FMeshConversionContext> Context = MakeShared<FMeshConversionContext>();
		Context->SkeletalMesh = TStrongObjectPtr(SkeletalMesh);
		Context->LODIndex = LODIndex;
		Context->OriginalLOD = &SkeletalMesh->GetResourceForRendering()->LODRenderData[Context->LODIndex];

		UE::Tasks::FTask SkeletalMeshLoadRenderDataTask = LaunchLoadSkeletalMeshLODRenderDataTask(Context);
		
		return UE::Tasks::Launch( TEXT("MutableRuntimeMeshConversionTask"),
		[Context, SectionIndex, ConversionFlags = (EMutableMeshConversionFlags)ConversionFlags, Result]()
		{
			MUTABLE_CPUPROFILER_SCOPE(MeshConversion);

			const int32 LODIndex = Context->LODIndex;
			USkeletalMesh *const SkeletalMesh = Context->SkeletalMesh.Get();
			
			if (!Context->DataLOD)
			{
				if (Context->OriginalLOD->bStreamedDataInlined && !SkeletalMesh->NeedCPUData(LODIndex))
				{
					UE_LOGF(LogMutable, Error, "Unable to read LOD %d Data from the CPU. Enable bAllowCPUAccess on the Skeletal Mesh %ls.", LODIndex, *SkeletalMesh->GetName());
				}
				else
				{
					UE_LOGF(LogMutable, Error, "Unable to read LOD %d Data from Skeletal Mesh %ls.", LODIndex, *SkeletalMesh->GetName());
				}

				return;
			}

			ExtractSkeletalMeshSection(Context.Get(), SectionIndex, ConversionFlags, Result.Get());
		},
		UE::Tasks::Prerequisites(SkeletalMeshLoadRenderDataTask));
	}
	
	UE::Tasks::FTask ConvertSkeletalMeshFromRuntimeData(USkeletalMesh* SkeletalMesh, int32 LODBegin, int32 LODEnd, int32 GeometryLODBegin, int32 GeometryLODEnd, uint8 ConversionFlags, UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeletalMesh> Result)
	{
		MUTABLE_CPUPROFILER_SCOPE(ConvertSkeletalMeshFromRuntimeData_LODRange);

		using namespace UE::Mutable::Private;

		check(Result);
		if (!(SkeletalMesh && Result))
		{
			return UE::Tasks::MakeCompletedTask<void>();
		}
		
		UE::Tasks::FTask SkeletalMeshConvertTask = UE::Tasks::Launch(TEXT("SkeletalMeshConvertTask"),
		[
			SkeletalMesh = TStrongObjectPtr(SkeletalMesh), 
			LODBegin, LODEnd,
			GeometryLODBegin, GeometryLODEnd,
			ConversionFlags,
			Result
		]()
		{
			Result->LODs.SetNum(LODEnd);
			for (int32 LODIndex = LODBegin; LODIndex < LODEnd; ++LODIndex)
			{
				if (!SkeletalMesh->GetResourceForRendering()->LODRenderData.IsValidIndex(LODIndex))
				{
					continue;
				}

				TSharedPtr<FMeshConversionContext> Context = MakeShared<FMeshConversionContext>();
				Context->SkeletalMesh = SkeletalMesh;
				Context->LODIndex = LODIndex;
				Context->OriginalLOD = &SkeletalMesh->GetResourceForRendering()->LODRenderData[Context->LODIndex];

				UE::Tasks::FTask SkeletalMeshLoadRenderDataTask;
				if (LODIndex >= GeometryLODBegin && LODIndex < GeometryLODEnd)
				{
					SkeletalMeshLoadRenderDataTask = LaunchLoadSkeletalMeshLODRenderDataTask(Context);
				}
				else
				{
					SkeletalMeshLoadRenderDataTask = UE::Tasks::MakeCompletedTask<void>();
				}

				UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("MutableRuntimeMeshLODConversionTask"),
				[
					Result,
					Context,
					GeometryLODBegin,
					GeometryLODEnd,
					ConversionFlags = (EMutableMeshConversionFlags)ConversionFlags 
				]()
				{
					MUTABLE_CPUPROFILER_SCOPE(MeshConversion);

					const int32 LODIndex = Context->LODIndex;
					USkeletalMesh *const SkeletalMesh = Context->SkeletalMesh.Get();

					int32 NumRenderSections = Context->OriginalLOD->RenderSections.Num();
					if (LODIndex >= GeometryLODBegin && LODIndex < GeometryLODEnd)
					{
						if (!Context->DataLOD)
						{
							if (Context->OriginalLOD->bStreamedDataInlined && !SkeletalMesh->NeedCPUData(LODIndex))
							{
								UE_LOGF(LogMutable, Error, "Unable to read LOD %d Data from the CPU. Enable bAllowCPUAccess on the Skeletal Mesh %ls.", LODIndex, *SkeletalMesh->GetName());
							}
							else
							{
								UE_LOGF(LogMutable, Error, "Unable to read LOD %d Data from Skeletal Mesh %ls.", LODIndex, *SkeletalMesh->GetName());
							}
						}
					}

					TManagedPtr<FLOD> MutableLOD = MakeManaged<FLOD>();

					MutableLOD->Meshes.SetNum(NumRenderSections);

					for (int32 SectionIndex = 0; SectionIndex < NumRenderSections; ++SectionIndex)
					//ParallelFor(NumRenderSections,
					//[
					//	Context,
					//	ConversionFlags,
					//	&MutableLOD
					//](int32 SectionIndex)
					{
						TManagedPtr<FMesh> MutableMesh = MakeManaged<FMesh>();
						ExtractSkeletalMeshSection(Context.Get(), SectionIndex, ConversionFlags, MutableMesh.Get());

						MutableLOD->Meshes[SectionIndex] = MutableMesh;
					}
					//);

					Result->LODs[LODIndex] = MutableLOD;
				},
				UE::Tasks::Prerequisites(SkeletalMeshLoadRenderDataTask)));
			}
		});

		const TArray<FSkeletalMaterial>& Materials = SkeletalMesh->GetMaterials();

		for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
		{
			const FSkeletalMaterial& Material = Materials[MaterialIndex];

			TManagedPtr<UE::Mutable::Private::FMaterial> PassthroughMaterial = MakeManaged<UE::Mutable::Private::FMaterial>(); 
			PassthroughMaterial->PassthroughObject = TPassthroughObjectPtr<UMaterialInterface>(Material.MaterialInterface);

			Result->MaterialSlotMaterials.Emplace(TInPlaceType<TManagedPtr<const UE::Mutable::Private::FMaterial>>{}, PassthroughMaterial);
			Result->MaterialSlotNames.Add(Material.MaterialSlotName);
			Result->MaterialSlotIds.Add(MaterialIndex);
		}

		return SkeletalMeshConvertTask;
	}

	void ClothVertexBuffers(
			FSkeletalMeshLODRenderData& LODResource, 
			const UE::Mutable::Private::FMesh& MutableMesh,
			TOptional<TConstArrayView<int32>> RenderToMutableSectionIndexMap)
	{
		const int32 NumSections = MutableMesh.ClothSections.Num();
		check(MutableMesh.Surfaces.Num() == MutableMesh.ClothSections.Num());
		
		int32 DataSize = 0;
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			DataSize += MutableMesh.ClothSections[SectionIndex].Data.Num();
		}
	
		if (DataSize)
		{
			TArray<FMeshToMeshVertData> Data;
			Data.Reserve(DataSize);

			for (const UE::Mutable::Private::FCloth& Cloth : MutableMesh.ClothSections)
			{
				Data.Append(Cloth.Data);
			}
	
			int32 NumRenderSections = LODResource.RenderSections.Num();

			TArray<FClothBufferIndexMapping> Mappings;
			Mappings.SetNumZeroed(NumRenderSections);

			check(!RenderToMutableSectionIndexMap || RenderToMutableSectionIndexMap->Num() == NumRenderSections);
			uint32 VertexOffset = 0;
			for (int32 SectionIndex = 0; SectionIndex < NumRenderSections; ++SectionIndex)
			{
				int32 MutableSectionIndex = SectionIndex;
				if (RenderToMutableSectionIndexMap)
				{
					MutableSectionIndex = (*RenderToMutableSectionIndexMap)[SectionIndex];
				}

				if (MutableSectionIndex == INDEX_NONE)
				{
					continue;
				}

				const UE::Mutable::Private::FCloth& Cloth = MutableMesh.ClothSections[MutableSectionIndex];
				const uint32 BaseVertexIndex = LODResource.RenderSections[SectionIndex].BaseVertexIndex;
			
				FClothBufferIndexMapping& Mapping = Mappings[SectionIndex];
				
				const int32 NumClothVertices = Cloth.Data.Num();
				if (Cloth.IsValid() && NumClothVertices)
				{
					Mapping.BaseVertexIndex = BaseVertexIndex; 
					Mapping.MappingOffset = VertexOffset;
					Mapping.LODBiasStride = VertexOffset;
				
					VertexOffset += NumClothVertices;

					// Cloth hack. This data should not be used.
					FSkelMeshRenderSection& RenderSection = LODResource.RenderSections[SectionIndex];
					RenderSection.ClothMappingDataLODs.SetNum(1, EAllowShrinking::No);
					RenderSection.ClothMappingDataLODs[0].SetNum(NumClothVertices, EAllowShrinking::No);
					FMemory::Memcpy(&RenderSection.ClothMappingDataLODs[0][0], Cloth.Data.GetData(), NumClothVertices * sizeof(FMeshToMeshVertData));
				}
				// else we need to add zeros to the Mapping, already done by the initialization.
			}

			LODResource.ClothVertexBuffer.Init(Data, Mappings);
		}
	}
	

	void PropagateBoneUsageFlagsThroughMeshPose(UE::Mutable::Private::FMesh& Mesh)
	{
		if (!Mesh.Skeleton)
		{
			return;
		}

		const int32 NumBonePoses = Mesh.BonePoses.Num();
		check(NumBonePoses == Mesh.Skeleton->GetNumBones());

		for (int32 PoseIndex = NumBonePoses - 1; PoseIndex >= 0; --PoseIndex)
		{
			const UE::Mutable::Private::EBoneUsageFlags BoneUsageFlags = Mesh.BonePoses[PoseIndex].BoneUsageFlags;

			constexpr UE::Mutable::Private::EBoneUsageFlags FlagsToPropagate =
				UE::Mutable::Private::EBoneUsageFlags::Skinning | UE::Mutable::Private::EBoneUsageFlags::Physics | UE::Mutable::Private::EBoneUsageFlags::Deform;
			if (EnumHasAnyFlags(BoneUsageFlags, FlagsToPropagate))
			{
				const UE::Mutable::Private::EBoneUsageFlags ParentPropagationFlags =
					(EnumHasAnyFlags(BoneUsageFlags, UE::Mutable::Private::EBoneUsageFlags::Skinning)
						? UE::Mutable::Private::EBoneUsageFlags::SkinningParent : UE::Mutable::Private::EBoneUsageFlags::None) |
					(EnumHasAnyFlags(BoneUsageFlags, UE::Mutable::Private::EBoneUsageFlags::Physics)
						? UE::Mutable::Private::EBoneUsageFlags::PhysicsParent : UE::Mutable::Private::EBoneUsageFlags::None) |
					(EnumHasAnyFlags(BoneUsageFlags, UE::Mutable::Private::EBoneUsageFlags::Deform)
						? UE::Mutable::Private::EBoneUsageFlags::DeformParent : UE::Mutable::Private::EBoneUsageFlags::None);

				int32 ParentPoseIndex = Mesh.Skeleton->BoneParents[PoseIndex]; // Bone Poses are 1 : 1 with the skeleton bones
				while (ParentPoseIndex != INDEX_NONE)
				{
					if (EnumHasAllFlags(Mesh.BonePoses[ParentPoseIndex].BoneUsageFlags, ParentPropagationFlags))
					{
						break;
					}

					EnumAddFlags(Mesh.BonePoses[ParentPoseIndex].BoneUsageFlags, ParentPropagationFlags);

					ParentPoseIndex = Mesh.Skeleton->BoneParents[ParentPoseIndex];
				}
			}
		}
	}


	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FPhysicsBody> CreatePhysicsBodyForMesh(const UPhysicsAsset& PhysicsAsset, const UE::Mutable::Private::FMesh& Mesh)
	{
		const TArray<TObjectPtr<USkeletalBodySetup>>& BodySetups = PhysicsAsset.SkeletalBodySetups;
		const int32 NumBodySetups = BodySetups.Num();

		TArray<int32> ActiveBodySetups;
		ActiveBodySetups.Reserve(NumBodySetups);

		check(Mesh.Skeleton);

		for (int32 BodyIndex = 0; BodyIndex < NumBodySetups; ++BodyIndex)
		{
			const TObjectPtr<USkeletalBodySetup>& BodySetup = BodySetups[BodyIndex];
			if (ensure(BodySetup))
			{
				const int32 BoneIndex = Mesh.Skeleton->BoneNames.Find(BodySetup->BoneName);
				if (BoneIndex != INDEX_NONE)
				{
					ActiveBodySetups.Add(BodyIndex);
				}
			}
		}

		if (ActiveBodySetups.IsEmpty())
		{
			return nullptr;
		}

		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FPhysicsBody> PhysicsBody =
			UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FPhysicsBody>();

		const int32 NumActiveBodySetups = ActiveBodySetups.Num();
		PhysicsBody->SetBodyCount(NumActiveBodySetups);

		auto GetKBodyElemFlags = [](const FKShapeElem& KElem) -> uint32
			{
				uint8 ElemCollisionEnabled = static_cast<uint8>(KElem.GetCollisionEnabled());

				uint32 Flags = static_cast<uint32>(ElemCollisionEnabled);
				Flags = Flags | (static_cast<uint32>(KElem.GetContributeToMass()) << 8);

				return Flags;
			};

		for (int32 BodyIndex = 0; BodyIndex < NumActiveBodySetups; ++BodyIndex)
		{
			const TObjectPtr<USkeletalBodySetup>& BodySetup = BodySetups[ActiveBodySetups[BodyIndex]];

			PhysicsBody->SetBody(BodyIndex, BodySetup->BoneName);

			const int32 NumSpheres = BodySetup->AggGeom.SphereElems.Num();
			PhysicsBody->SetSphereCount(BodyIndex, NumSpheres);

			for (int32 I = 0; I < NumSpheres; ++I)
			{
				const FKSphereElem& SphereElem = BodySetup->AggGeom.SphereElems[I];
				PhysicsBody->SetSphere(BodyIndex, I, FVector3f(SphereElem.Center), SphereElem.Radius);

				const FString ElemName = SphereElem.GetName().ToString();
				PhysicsBody->SetSphereName(BodyIndex, I, StringCast<ANSICHAR>(*ElemName).Get());
				PhysicsBody->SetSphereFlags(BodyIndex, I, GetKBodyElemFlags(SphereElem));
			}

			const int32 NumBoxes = BodySetup->AggGeom.BoxElems.Num();
			PhysicsBody->SetBoxCount(BodyIndex, NumBoxes);

			for (int32 I = 0; I < NumBoxes; ++I)
			{
				const FKBoxElem& BoxElem = BodySetup->AggGeom.BoxElems[I];
				PhysicsBody->SetBox(BodyIndex, I,
					FVector3f(BoxElem.Center),
					FQuat4f(BoxElem.Rotation.Quaternion()),
					FVector3f(BoxElem.X, BoxElem.Y, BoxElem.Z));

				const FString KElemName = BoxElem.GetName().ToString();
				PhysicsBody->SetBoxName(BodyIndex, I, StringCast<ANSICHAR>(*KElemName).Get());
				PhysicsBody->SetBoxFlags(BodyIndex, I, GetKBodyElemFlags(BoxElem));
			}

			const int32 NumConvex = BodySetup->AggGeom.ConvexElems.Num();
			PhysicsBody->SetConvexCount(BodyIndex, NumConvex);
			for (int32 I = 0; I < NumConvex; ++I)
			{
				const FKConvexElem& ConvexElem = BodySetup->AggGeom.ConvexElems[I];

				// Convert to FVector3f
				TArray<FVector3f> VertexData;
				VertexData.SetNumUninitialized(ConvexElem.VertexData.Num());
				for (int32 Elem = VertexData.Num() - 1; Elem >= 0; --Elem)
				{
					VertexData[Elem] = FVector3f(ConvexElem.VertexData[Elem]);
				}

				PhysicsBody->SetConvexMesh(BodyIndex, I,
					TArrayView<const FVector3f>(VertexData.GetData(), ConvexElem.VertexData.Num()),
					TArrayView<const int32>(ConvexElem.IndexData.GetData(), ConvexElem.IndexData.Num()));

				PhysicsBody->SetConvexTransform(BodyIndex, I, FTransform3f(ConvexElem.GetTransform()));

				const FString KElemName = ConvexElem.GetName().ToString();
				PhysicsBody->SetConvexName(BodyIndex, I, StringCast<ANSICHAR>(*KElemName).Get());
				PhysicsBody->SetConvexFlags(BodyIndex, I, GetKBodyElemFlags(ConvexElem));
			}

			const int32 NumSphyls = BodySetup->AggGeom.SphylElems.Num();
			PhysicsBody->SetSphylCount(BodyIndex, NumSphyls);

			for (int32 I = 0; I < NumSphyls; ++I)
			{
				const FKSphylElem& SphylElem = BodySetup->AggGeom.SphylElems[I];
				PhysicsBody->SetSphyl(BodyIndex, I,
					FVector3f(SphylElem.Center),
					FQuat4f(SphylElem.Rotation.Quaternion()),
					SphylElem.Radius, SphylElem.Length);

				const FString KElemName = SphylElem.GetName().ToString();
				PhysicsBody->SetSphylName(BodyIndex, I, StringCast<ANSICHAR>(*KElemName).Get());
				PhysicsBody->SetSphylFlags(BodyIndex, I, GetKBodyElemFlags(SphylElem));
			}

			const int32 NumTaperedCapsules = BodySetup->AggGeom.TaperedCapsuleElems.Num();
			PhysicsBody->SetTaperedCapsuleCount(BodyIndex, NumTaperedCapsules);

			for (int32 I = 0; I < NumTaperedCapsules; ++I)
			{
				const FKTaperedCapsuleElem& TaperedCapsuleElem = BodySetup->AggGeom.TaperedCapsuleElems[I];
				PhysicsBody->SetTaperedCapsule(BodyIndex, I,
					FVector3f(TaperedCapsuleElem.Center),
					FQuat4f(TaperedCapsuleElem.Rotation.Quaternion()),
					TaperedCapsuleElem.Radius0, TaperedCapsuleElem.Radius1, TaperedCapsuleElem.Length);

				const FString KElemName = TaperedCapsuleElem.GetName().ToString();
				PhysicsBody->SetTaperedCapsuleName(BodyIndex, I, StringCast<ANSICHAR>(*KElemName).Get());
				PhysicsBody->SetTaperedCapsuleFlags(BodyIndex, I, GetKBodyElemFlags(TaperedCapsuleElem));
			}
		}

		return PhysicsBody;
	}
}
