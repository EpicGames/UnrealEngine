// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/UnrealMutableImageProvider.h"

#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCO/UnrealConversionUtils.h"
#include "MuCO/LoadUtils.h"
#include "MuR/Parameters.h"
#include "MuR/ManagedPointer.h"
#include "MuR/ImageTypes.h"
#include "MuR/Model.h"
#include "MuR/Mesh.h"
#include "MuR/SkeletalMesh.h"
#include "MuR/LOD.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/BoneReference.h"
#include "Animation/Skeleton.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "ReferenceSkeleton.h"
#include "ImageCoreUtils.h"
#include "TextureResource.h"
#include "Engine/Texture.h"

#include "IO/IoBuffer.h"

namespace
{
	void ConvertTextureUnrealPlatformToMutable(UE::Mutable::Private::FImage* OutResult, UTexture2D* Texture, uint8 MipmapsToSkip)
	{		
		check(Texture->GetPlatformData()->Mips[MipmapsToSkip].BulkData.IsBulkDataLoaded());

		int32 LODs = 1;
		int32 SizeX = Texture->GetSizeX() >> MipmapsToSkip;
		int32 SizeY = Texture->GetSizeY() >> MipmapsToSkip;
		check(SizeX > 0 && SizeY > 0);

		EPixelFormat Format = Texture->GetPlatformData()->PixelFormat;
		UE::Mutable::Private::EImageFormat MutableFormat = UE::Mutable::Private::EImageFormat::None;

		switch (Format)
		{
		case EPixelFormat::PF_B8G8R8A8: MutableFormat = UE::Mutable::Private::EImageFormat::BGRA_UByte; break;
			// This format is deprecated and using the enum fails to compile in some cases.
			//case ETextureSourceFormat::TSF_RGBA8: MutableFormat = UE::Mutable::Private::EImageFormat::RGBA_UByte; break;
		case EPixelFormat::PF_G8: MutableFormat = UE::Mutable::Private::EImageFormat::L_UByte; break;
		default:
			break;
		}

		// If not locked ReadOnly the Texture Source's FGuid can change, invalidating the texture's caching/shaders
		// making shader compile and cook times increase
		const void* pSource = Texture->GetPlatformData()->Mips[MipmapsToSkip].BulkData.LockReadOnly();

		if (pSource)
		{
			OutResult->Init(SizeX, SizeY, LODs, MutableFormat, UE::Mutable::Private::EInitializationType::NotInitialized);
			FMemory::Memcpy(OutResult->GetLODData(0), pSource, OutResult->GetLODDataSize(0));
			Texture->GetPlatformData()->Mips[MipmapsToSkip].BulkData.Unlock();
		}
		else
		{
			check(false);
			OutResult->Init(SizeX, SizeY, LODs, MutableFormat, UE::Mutable::Private::EInitializationType::Black);
		}
	}
}


UE::Mutable::Private::EImageFormat GetMutablePixelFormat(EPixelFormat InTextureFormat)
{
	switch (InTextureFormat)
	{
	case PF_B8G8R8A8: return UE::Mutable::Private::EImageFormat::BGRA_UByte;
	case PF_R8G8B8A8: return UE::Mutable::Private::EImageFormat::RGBA_UByte;
	case PF_DXT1: return UE::Mutable::Private::EImageFormat::BC1;
	case PF_DXT3: return UE::Mutable::Private::EImageFormat::BC2;
	case PF_DXT5: return UE::Mutable::Private::EImageFormat::BC3;
	case PF_BC4: return UE::Mutable::Private::EImageFormat::BC4;
	case PF_BC5: return UE::Mutable::Private::EImageFormat::BC5;
	case PF_G8: return UE::Mutable::Private::EImageFormat::L_UByte;
	case PF_ASTC_4x4: return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR;
	case PF_ASTC_4x4_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR;
	case PF_ASTC_6x6: return UE::Mutable::Private::EImageFormat::ASTC_6x6_RGBA_LDR;
	case PF_ASTC_6x6_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_6x6_RG_LDR;
	case PF_ASTC_8x8: return UE::Mutable::Private::EImageFormat::ASTC_8x8_RGBA_LDR;
	case PF_ASTC_8x8_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_8x8_RG_LDR;
	case PF_ASTC_10x10: return UE::Mutable::Private::EImageFormat::ASTC_10x10_RGBA_LDR;
	case PF_ASTC_10x10_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR;
	case PF_ASTC_12x12: return UE::Mutable::Private::EImageFormat::ASTC_12x12_RGBA_LDR;
	case PF_ASTC_12x12_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_12x12_RG_LDR;
	default: return UE::Mutable::Private::EImageFormat::None;
	}
}


namespace UnrealMutableImageProviderInteranl
{
	bool CheckBulkSizesAreValid(TNotNull<UTexture2D*> Texture, TNotNull<UE::Mutable::Private::FImage*> Image, int32 TextureMipIndexBegin, int32 TextureMipIndexEnd)
	{
		int32 TextureNumMips = TextureMipIndexEnd - TextureMipIndexBegin; 

		if (TextureNumMips != Image->DataStorage.GetNumLODs())
		{
			return false;
		}

		for (int32 TextureMipIndex = TextureMipIndexBegin; TextureMipIndex < TextureMipIndexEnd; ++TextureMipIndex)
		{
			FByteBulkData& BulkData = Texture->GetPlatformData()->Mips[TextureMipIndex].BulkData;
			int32 BulkDataSize = BulkData.GetBulkDataSize();
			int32 ImageDataSize = Image->DataStorage.GetLOD(TextureMipIndex - TextureMipIndexBegin).Num(); 

			if (BulkDataSize < ImageDataSize)
			{
				return false;
			}
		}

		return true;
	}
} // namespace UnrealMutableImageProviderInteranl

//-------------------------------------------------------------------------------------------------
TTuple<UE::Tasks::FTask, TFunction<void()>> FUnrealMutableResourceProvider::GetImageAsync(UTexture* Texture, uint8 MipmapsToSkip, bool bLoadMipTail, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage>)>& ResultCallback)
{
	using namespace UnrealMutableImageProviderInteranl;

	// Thread: worker
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableImageProvider::GetImageAsync);

	// Some data that may have to be copied from the GlobalExternalImages while it's locked
	EPixelFormat Format = EPixelFormat::PF_Unknown;

	UE::Mutable::Private::EImageFormat MutImageFormat = UE::Mutable::Private::EImageFormat::None;
	int32 MutImageDataSize = 0;

	auto TrivialReturn = []() -> TTuple<UE::Tasks::FTask, TFunction<void()>>
	{
		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	};

	if (!Texture)
	{
		UE_LOGF(LogMutable, Warning, "Invalid Image Parameter. Nullptr");
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}
	
	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	if (!Texture2D)
	{
		UE_LOGF(LogMutable, Warning, "Invalid Image Parameter [%ls]. Is not a UTexture2D.", *Texture->GetName());
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}
	
#if WITH_EDITOR
	FTextureSource Source = Texture->Source.CopyTornOff();
	
	const int32 MipIndex = FMath::Min(static_cast<int32>(MipmapsToSkip), Source.GetNumMips() - 1);
	check(MipIndex >= 0);
		
	// In the editor the src data can be directly accessed
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Image = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>();

	FMutableSourceTextureData TextureData(*Texture2D);
	EUnrealToMutableConversionError Error = ConvertTextureUnrealSourceToMutable(Image.Get(), TextureData, MipIndex, bLoadMipTail);
	if (Error != EUnrealToMutableConversionError::Success)
	{
		// This could happen in the editor, because some source textures may have changed while there was a background compilation.
		// We just show a warning and move on. This cannot happen during cooks, so it is fine.
		UE_LOGF(LogMutable, Warning, "Failed to load some source texture data for image [%ls]. Some materials may look corrupted.", *Texture->GetName());
	}

	ResultCallback(Image);
	return Invoke(TrivialReturn);
#else

	// It's safe to access TextureToLoad because ExternalImagesLock guarantees that the data in GlobalExternalImages is valid,
	// not being modified by the game thread at the moment and the texture cannot be GCed because of the AddReferencedObjects
	// in the FUnrealMutableImageProvider
	
	// Texture format and the equivalent mutable format
	Format = Texture2D->GetPlatformData()->PixelFormat;
	MutImageFormat = GetMutablePixelFormat(Format);

	// Check if it's a format we support
	if (MutImageFormat == UE::Mutable::Private::EImageFormat::None)
	{
		UE_LOGF(LogMutable, Warning, "Failed to get Image Parameter [%ls]. Unexpected image format. EImageFormat [%ls].", *Texture2D->GetName(), GetPixelFormatString(Format));
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}

	const int32 TextureNumMips = Texture2D->GetPlatformData()->Mips.Num();

	int32 TextureMipIndexBegin = MipmapsToSkip < TextureNumMips ? MipmapsToSkip : TextureNumMips - 1;
	check(TextureMipIndexBegin >= 0);

	int32 NumMipsToLoad = bLoadMipTail ? TextureMipIndexBegin - TextureNumMips : 1;
	int32 LastMipToLoad = bLoadMipTail ? TextureNumMips - 1 : TextureMipIndexBegin;

	// Find the smallest mip available.
	for (; LastMipToLoad > 0; --LastMipToLoad)
	{
		if (Texture2D->GetPlatformData()->Mips[LastMipToLoad].BulkData.CanLoadFromDisk())
		{
			break;
		}
	}

	TextureMipIndexBegin = FMath::Min(LastMipToLoad, TextureMipIndexBegin);
	int32 TextureMipIndexEnd = LastMipToLoad + 1;

	int32 SizeX = FMath::Max(Texture2D->GetSizeX() >> TextureMipIndexBegin, 1);
	int32 SizeY = FMath::Max(Texture2D->GetSizeY() >> TextureMipIndexBegin, 1);

	int32 NumImageLODs = TextureMipIndexEnd - TextureMipIndexBegin;
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Image = MakeManaged<UE::Mutable::Private::FImage>(SizeX, SizeY, NumImageLODs, MutImageFormat, UE::Mutable::Private::EInitializationType::NotInitialized);

	if (!CheckBulkSizesAreValid(Texture2D, Image.Get(), TextureMipIndexBegin, TextureMipIndexEnd))
	{
		UE_LOGF(LogMutable, Warning, "Failed to get Image Parameter [%ls]. Bulk data does not match expected image data sizes.", *Texture2D->GetName());
		ResultCallback(CreateDummy());

		return Invoke(TrivialReturn);
	}

	const int32 BatchCount = TextureMipIndexEnd - TextureMipIndexBegin;
	FBulkDataBatchRequest::FBatchBuilder Batch = FBulkDataBatchRequest::NewBatch(BatchCount);
	
	for (int32 TextureMipIndex = TextureMipIndexBegin; TextureMipIndex < TextureMipIndexEnd; ++TextureMipIndex)
	{
		FByteBulkData& BulkData = Texture2D->GetPlatformData()->Mips[TextureMipIndex].BulkData;
		int32 BulkDataSize = BulkData.GetBulkDataSize();
		if (BulkDataSize > 0)
		{
			TArrayView<uint8> ImageDataView = Image->DataStorage.GetLOD(TextureMipIndex - TextureMipIndexBegin);
			// check(BulkDataSize == ImageDataView.Num());

			TRACE_IOSTORE_METADATA_SCOPE_TAG(FName(Texture->GetPathName()));
			FIoBuffer Dst(FIoBuffer::Wrap, ImageDataView.GetData(), ImageDataView.Num());
			Batch.Read(BulkData, 0, ImageDataView.Num(), AIOP_High | AIOP_FLAG_DONTCACHE, Dst);
		}
	}
		
	UE::Tasks::FTaskEvent RequestCompletionEvent(TEXT("Mutable_ImageParamRequestCompletionEvent"));
	TSharedPtr<FBulkDataBatchRequest> ResultBulkDataRequest = MakeShared<FBulkDataBatchRequest>();
	Batch.Issue(
	[
		Image,
		RequestCompletionEvent,
		ResultCallback // captured by copy
	](FBulkDataBatchRequest::EStatus Status) mutable
	{
		ON_SCOPE_EXIT
		{
			RequestCompletionEvent.Trigger();
		};
		
		// Should we do someting different than returning a dummy image if cancelled?
		if (Status == FBulkDataRequest::EStatus::Cancelled)
		{
			UE_LOGF(LogMutable, Warning, "Failed to get external image. Cancelled Bulk Data Request");
			ResultCallback(CreateDummy());
			return;
		}
		
		if (Status != FBulkDataRequest::EStatus::Ok)
		{
			UE_LOGF(LogMutable, Warning, "Failed to get external image parameter.");
			ResultCallback(CreateDummy());
			return;
		}
		
		ResultCallback(Image);
	}, *ResultBulkDataRequest);

	return MakeTuple(RequestCompletionEvent, [BulkDataRequest = MoveTemp(ResultBulkDataRequest)](){});
#endif
}


//-------------------------------------------------------------------------------------------------
TTuple<UE::Tasks::FTask, TFunction<void()>> FUnrealMutableResourceProvider::GetReferencedImageAsync(int32 Id, uint8 MipmapsToSkip, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage>)>& ResultCallback)
{
	// Thread: worker
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableImageProvider::GetReferencedImageAsync);

	auto TrivialReturn = []() -> TTuple<UE::Tasks::FTask, TFunction<void()>>
	{
		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	};

#if WITH_EDITOR
	FScopeLock Lock(&RuntimeReferencedLock);

	if (!Images.IsValidIndex(Id))
	{
		// This could happen in the editor, because some source textures may have changed while there was a background compilation.
		// We just show a warning and move on. This cannot happen during cooks, so it is fine.
		UE_LOGF(LogMutable, Warning, "Failed to load Referenced Image [%i]. Invalid id.", Id);
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}

	UTexture* Texture = Images[Id].Get();
	if (!Texture)
	{
		UE_LOGF(LogMutable, Warning, "Invalid Referenced Image [%i]. Nullptr.", Id);
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}
	
	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	if (!Texture2D)
	{
		UE_LOGF(LogMutable, Warning, "Invalid Referenced Image [%i, %ls]. Is not a UTexture2D.", Id, *Texture->GetName());
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}
	
	const int32 MipIndex = FMath::Min(static_cast<int32>(MipmapsToSkip), Texture2D->Source.CopyTornOff().GetNumMips() - 1);
	check(MipIndex >= 0);

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Image = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>();
	
	FMutableSourceTextureData Tex(*Texture2D);

	constexpr bool bLoadMipTail = false;
	EUnrealToMutableConversionError Error = ConvertTextureUnrealSourceToMutable(Image.Get(), Tex, MipIndex, bLoadMipTail);
	if (Error != EUnrealToMutableConversionError::Success)
	{
		// This could happen in the editor, because some source textures may have changed while updating.
		// We just show a warning and move on. This cannot happen during cooks, so it is fine.
		UE_LOGF(LogMutable, Warning, "Failed to load some source texture data for Referenced Image [%i, %ls]. Some textures may be corrupted.", Id, *Texture->GetName());
		
		ResultCallback(CreateDummy());
		return Invoke(TrivialReturn);
	}

	ResultCallback(Image);
	return Invoke(TrivialReturn);
#else // WITH_EDITOR

	// Not supported outside editor yet.
	UE_LOGF(LogMutable, Warning, "Failed to get Reference Image. Only supported in editor.");

	ResultCallback(CreateDummy());
	return Invoke(TrivialReturn);

#endif
}


// This should mantain parity with the descriptor of the images generated by GetImageAsync 
UE::Mutable::Private::FExtendedImageDesc FUnrealMutableResourceProvider::GetImageDesc(UTexture* Texture)
{
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableImageProvider::GetImageDesc);

	if (!Texture)
	{
		return CreateDummyDesc();
	}

	UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
	if (!Texture2D)
	{
		return CreateDummyDesc();
	}
	
#if WITH_EDITOR
	const FTextureSource& Source = Texture->Source.CopyTornOff();

	const UE::Mutable::Private::FImageSize ImageSize = UE::Mutable::Private::FImageSize(Source.GetSizeX(), Source.GetSizeY());
	const uint8 LODs = 1;

	return UE::Mutable::Private::FExtendedImageDesc { UE::Mutable::Private::FImageDesc { ImageSize, UE::Mutable::Private::EImageFormat::None, UE::Mutable::Private::EImageFormat::None, LODs }, 0 };
#else
	UE::Mutable::Private::FExtendedImageDesc Result;	

	// It's safe to access TextureToLoad because ExternalImagesLock guarantees that the data in GlobalExternalImages is valid,
	// not being modified by the game thread at the moment and the texture cannot be GCed because of the AddReferencedObjects
	// in the FUnrealMutableImageProvider

	const int32 TextureToLoadNumMips = Texture2D->GetPlatformData()->Mips.Num();

	int32 FirstLODAvailable = 0;
	for (; FirstLODAvailable < TextureToLoadNumMips; ++FirstLODAvailable)
	{
		if (Texture2D->GetPlatformData()->Mips[FirstLODAvailable].BulkData.DoesExist())
		{
			break;
		}
	}
	
	// Texture format and the equivalent mutable format
	const EPixelFormat Format = Texture2D->GetPlatformData()->PixelFormat;
	const UE::Mutable::Private::EImageFormat MutableFormat = GetMutablePixelFormat(Format);

	// Check if it's a format we support
	if (MutableFormat == UE::Mutable::Private::EImageFormat::None)
	{
		UE_LOGF(LogMutable, Warning, "Failed to get Image Parameter descriptor. Unexpected image format. EImageFormat [%ls].", GetPixelFormatString(Format));
		return CreateDummyDesc();
	}

	const UE::Mutable::Private::FImageDesc ImageDesc = UE::Mutable::Private::FImageDesc 
		{ UE::Mutable::Private::FImageSize(Texture2D->GetSizeX(), Texture2D->GetSizeY()), MutableFormat, UE::Mutable::Private::EImageFormat::None, 1 };

	Result = UE::Mutable::Private::FExtendedImageDesc { ImageDesc, (uint8)FirstLODAvailable }; 

	return Result;
#endif
}


TTuple<UE::Tasks::FTask, TFunction<void()>> FUnrealMutableResourceProvider::GetMeshAsync(USkeletalMesh* SkeletalMesh, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 ConversionFlags, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh>)>& ResultCallback)
{
	// Thread: worker
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableImageProvider::GetMeshAsync);

	auto TrivialReturn = []() -> TTuple<UE::Tasks::FTask, TFunction<void()>>
	{
		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	};

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> Result = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();

	if (!SkeletalMesh)
	{
		ResultCallback(Result);
		return Invoke(TrivialReturn);
	}
	
	UE::Tasks::FTask ConversionTask = UnrealConversionUtils::ConvertSkeletalMeshFromRuntimeData(SkeletalMesh, BaseLODIndex, BaseSectionIndex, LODBias, ConversionFlags, Result);

	return MakeTuple(
		// Some post-game conversion stuff can happen here in a worker thread
		UE::Tasks::Launch(TEXT("MutableMeshParameterLoadPostGame"),
			[ResultCallback, Result]()
			{
				ResultCallback(Result);
			},
			ConversionTask),

		// Cleanup code that will be called after the result is received in calling code.
		[]()
		{
		}
	);
}


TTuple<UE::Tasks::FTask, TFunction<void()>> FUnrealMutableResourceProvider::GetSkeletalMeshAsync(
		USkeletalMesh* SkeletalMesh, int32 LODBegin, int32 LODEnd, int32 GeometryLODBegin, int32 GeometryLODEnd, uint8 ConversionFlags, TFunction<void(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeletalMesh>)>& ResultCallback)
{
	using namespace UE::Mutable::Private;

	// Thread: worker
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableImageProvider::GetSkeletalMeshAsync);

	auto TrivialReturn = []() -> TTuple<UE::Tasks::FTask, TFunction<void()>>
	{
		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	};

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeletalMesh> Result = 
			UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FSkeletalMesh>();

	if (!SkeletalMesh)
	{
		ResultCallback(Result);
		return Invoke(TrivialReturn);
	}

	UE::Tasks::FTask SkeletalMeshTask = UnrealConversionUtils::ConvertSkeletalMeshFromRuntimeData(
			SkeletalMesh, LODBegin, LODEnd, GeometryLODBegin, GeometryLODEnd, ConversionFlags,  Result);

	return MakeTuple(
		// Some post-game conversion stuff can happen here in a worker thread
		UE::Tasks::Launch(TEXT("MutableMeshParameterLoadPostGame"),
		[
			ResultCallback, 
			Result
		]()
		{
			ResultCallback(Result);
		},
		UE::Tasks::Prerequisites(SkeletalMeshTask)),

		// Cleanup code that will be called after the result is received in calling code.
		[](){}
	);
}

#if WITH_EDITOR
void FUnrealMutableResourceProvider::CacheRuntimeReferencedImages(const TArray<TSoftObjectPtr<UTexture2D>>& RuntimeReferencedTextures)
{
	check(IsInGameThread());
	
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableImageProvider::CacheRuntimeReferencedImages);
	
	FScopeLock Lock(&RuntimeReferencedLock);

	Images.Reset();
	for (const TSoftObjectPtr<UTexture2D>& RuntimeReferencedTexture : RuntimeReferencedTextures)
	{
		UTexture2D* Texture = RuntimeReferencedTexture.Get(); // Is already loaded.
		if (!Texture)
		{
			UE_LOGF(LogMutable, Warning, "Runtime Referenced Texture [%ls] was not async loaded. Forcing load sync.", *RuntimeReferencedTexture->GetPathName());
			
			Texture = UE::Mutable::Private::LoadObject(RuntimeReferencedTexture);
			if (!Texture)
			{
				UE_LOGF(LogMutable, Warning, "Failed to force load sync [%ls].", *RuntimeReferencedTexture->GetPathName());
				continue;
			}
		}

		Images.Add(TStrongObjectPtr(Texture)); // Perform a CopyTornOff. Once done, we no longer need the texture loaded.
	}
}
#endif


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> FUnrealMutableResourceProvider::CreateDummy()
{
	// Create a dummy image
	const int32 Size = DUMMY_IMAGE_DESC.m_size[0];
	const int32 CheckerSize = 4;
	constexpr int32 CheckerTileCount = 2;
	
#if !UE_BUILD_SHIPPING
	uint8 Colors[CheckerTileCount][4] = {{255, 255, 0, 255}, {0, 0, 255, 255}};
#else
	uint8 Colors[CheckerTileCount][4] = {{255, 255, 0, 0}, {0, 0, 255, 0}};
#endif

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> pResult = 
			UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(Size, Size, DUMMY_IMAGE_DESC.m_lods, DUMMY_IMAGE_DESC.m_format, UE::Mutable::Private::EInitializationType::NotInitialized);

	check(pResult->GetLODCount() == 1);
	check(pResult->GetFormat() == UE::Mutable::Private::EImageFormat::RGBA_UByte || pResult->GetFormat() == UE::Mutable::Private::EImageFormat::BGRA_UByte);
	uint8* pData = pResult->GetLODData(0);
	for (int32 X = 0; X < Size; ++X)
	{
		for (int32 Y = 0; Y < Size; ++Y)
		{
			int32 CheckerIndex = ((X / CheckerSize) + (Y / CheckerSize)) % CheckerTileCount;
			pData[0] = Colors[CheckerIndex][0];
			pData[1] = Colors[CheckerIndex][1];
			pData[2] = Colors[CheckerIndex][2];
			pData[3] = Colors[CheckerIndex][3];
			pData += 4;
		}
	}

	return pResult;
}


UE::Mutable::Private::FExtendedImageDesc FUnrealMutableResourceProvider::CreateDummyDesc()
{
	return UE::Mutable::Private::FExtendedImageDesc{ {DUMMY_IMAGE_DESC}, 0 };
}

