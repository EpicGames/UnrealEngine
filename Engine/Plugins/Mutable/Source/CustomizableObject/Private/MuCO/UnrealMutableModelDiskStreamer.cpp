// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/UnrealMutableModelDiskStreamer.h"

#include "Async/AsyncFileHandle.h"
#include "HAL/PlatformFile.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuCO/LogBenchmarkUtil.h"
#include "HAL/PlatformFileManager.h"
#include "Serialization/BulkData.h"

#if WITH_EDITOR
#include "DerivedDataCache.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataRequestOwner.h"
#endif

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Streaming Ops"), STAT_MutableStreamingOps, STATGROUP_Mutable);


//-------------------------------------------------------------------------------------------------
#if WITH_EDITOR

UnrealMutableOutputStream::UnrealMutableOutputStream(FArchive& ar)
	: m_ar(ar)
{
}


void UnrealMutableOutputStream::Write(const void* pData, uint64 size)
{
	// TODO: ugly const cast
	m_ar.Serialize(const_cast<void*>(pData), size);
}

#endif

//-------------------------------------------------------------------------------------------------
FUnrealMutableInputStream::FUnrealMutableInputStream(FArchive& ar)
	: m_ar(ar)
{
}


void FUnrealMutableInputStream::Read(void* pData, uint64 size)
{
	m_ar.Serialize(pData, size);
}


//-------------------------------------------------------------------------------------------------
FUnrealMutableModelBulkReader::~FUnrealMutableModelBulkReader()
{
}

bool FUnrealMutableModelBulkReader::PrepareStreamingForObject(UCustomizableObject* CustomizableObject)
{
	if (!CustomizableObject)
	{
		check(false);
		return false;
	}

	FScopeLock Lock(&CriticalSection);

	// See if we can free previuously allocated resources.
	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); )
	{
		if (!Objects[ObjectIndex].Model.Pin() && Objects[ObjectIndex].CurrentReadRequests.IsEmpty())
		{
			Objects.RemoveAtSwap(ObjectIndex);
		}
		else
		{
			++ObjectIndex;
		}
	}

	const TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData = CustomizableObject->GetPrivate()->GetModelStreamableBulkData();

	// There's no need to prepare the object for streaming if there's no data to stream (Dataless COs).
	if (!ModelStreamableBulkData || ModelStreamableBulkData->ModelStreamables.IsEmpty())
	{
		return true;
	}

	ModelStreamableBulkData->BulkDataFlags.SetNumZeroed(ModelStreamableBulkData->StreamableBulkData.Num());

	const TSharedPtr<const UE::Mutable::Private::FModel>& Model = CustomizableObject->GetPrivate()->GetModel();

	// Is the object already prepared for streaming?
	bool bAlreadyStreaming = Objects.FindByPredicate(
		[Model](const FObjectData& d)
		{ return d.Model.Pin().Get() == Model.Get(); })
		!=
		nullptr;

	if (!bAlreadyStreaming)
	{
		FObjectData NewData;
		NewData.Model = TWeakPtr<const UE::Mutable::Private::FModel>(Model);
		NewData.ModelStreamableBulkData = TWeakPtr<FModelStreamableBulkData>(ModelStreamableBulkData);

#if WITH_EDITOR
		FString FullFileName = CustomizableObject->GetPrivate()->GetCompiledDataFileName(nullptr, true);
		NewData.BulkFilePrefix = *FullFileName + GetDataTypeExtension(UE::Mutable::Private::EStreamableDataType::Model);
#endif
		Objects.Add(MoveTemp(NewData));
	}

	return true;
}


#if WITH_EDITOR
void FUnrealMutableModelBulkReader::CancelStreamingForObject(const UCustomizableObject* CustomizableObject)
{
	if (!CustomizableObject)
	{
		check(false);
	}

	FScopeLock Lock(&CriticalSection);

	// See if we can free previuously allocated resources
	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		if (Objects[ObjectIndex].Model.Pin() == CustomizableObject->GetPrivate()->GetModel())
		{
			check(Objects[ObjectIndex].CurrentReadRequests.IsEmpty());

			Objects.RemoveAtSwap(ObjectIndex);
			break;
		}
	}
}


bool FUnrealMutableModelBulkReader::AreTherePendingStreamingOperationsForObject(const UCustomizableObject* CustomizableObject) const
{
	// This happens in the game thread
	check(IsInGameThread());

	if (!CustomizableObject)
	{
		check(false);
		return false;
	}

	FScopeLock Lock(&CriticalSection);

	for (int32 ObjectIndex = 0; ObjectIndex < Objects.Num(); ++ObjectIndex)
	{
		if (Objects[ObjectIndex].Model.Pin() == CustomizableObject->GetPrivate()->GetModel())
		{
			if (!Objects[ObjectIndex].CurrentReadRequests.IsEmpty())
			{
				return true;
			}
		}
	}

	return false;
}

#endif // WITH_EDITOR


void FUnrealMutableModelBulkReader::EndStreaming()
{
	FScopeLock Lock(&CriticalSection);

	for (FObjectData& o : Objects)
	{
		for (TPair<FOperationID, FReadRequest>& it : o.CurrentReadRequests)
		{
			if (it.Value.FileReadRequest)
			{
				it.Value.FileReadRequest->WaitCompletion();
			}
			else if (it.Value.BulkReadRequest)
			{
				it.Value.BulkReadRequest->WaitCompletion();
			}
		}
	}
	Objects.Empty();
}


static int32 StreamPriority = 4;
FAutoConsoleVariableRef CVarStreamPriority(
	TEXT("Mutable.StreamPriority"),
	StreamPriority,
	TEXT(""));

namespace Private
{
FORCEINLINE FString MakeBlockFilePath(const FString& Prefix, const FMutableStreamableBlock& Block)
{
	FString Result;

#if WITH_EDITOR
	Result = Prefix;
#else
	Result = FString::Printf(TEXT("%s-%08x.mut"), *Prefix, Block.FileId);
	if (Block.Flags == uint16(EMutableFileFlags::HighRes))
	{
		Result += TEXT(".high");
	}
#endif

	return Result;
}
};

bool FUnrealMutableModelBulkReader::DoesBlockExist(const UE::Mutable::Private::FModel* Model, uint32 Key)
{
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkReader::DoesBlockExist)

	FScopeLock Lock(&CriticalSection);
	
	FObjectData* ObjectData = Objects.FindByPredicate(
			[Model](const FObjectData& ObjectData) { return ObjectData.Model.Pin().Get() == Model; });

	if (!ObjectData)
	{
		return false;
	}

	const TSharedPtr<FModelStreamableBulkData> PinnedModelStreamableBulkData = ObjectData->ModelStreamableBulkData.Pin();

	if (!PinnedModelStreamableBulkData)
	{
		return false;
	}

	const FMutableStreamableBlock* Block = PinnedModelStreamableBulkData->ModelStreamables.Find(Key);

	if (!Block)
	{
		return false;
	}

#if !WITH_EDITOR
	if (ensure(PinnedModelStreamableBulkData->BulkDataFlags.IsValidIndex(Block->FileId)))
	{	
		MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkReader::DoesBlockExist_Bulk);

		EMutableFileFlags BulkDataFlags = PinnedModelStreamableBulkData->BulkDataFlags[Block->FileId];
		if (EnumHasAnyFlags(BulkDataFlags, EMutableFileFlags::DoesExistCached))
		{
			return EnumHasAnyFlags(BulkDataFlags, EMutableFileFlags::DoesExist);
		}
		else
		{
			const bool bDoesExist = PinnedModelStreamableBulkData->StreamableBulkData[Block->FileId].DoesExist();
			EnumAddFlags(BulkDataFlags, bDoesExist ? EMutableFileFlags::DoesExist : EMutableFileFlags::None);
			EnumAddFlags(BulkDataFlags, EMutableFileFlags::DoesExistCached);
			PinnedModelStreamableBulkData->BulkDataFlags[Block->FileId] = BulkDataFlags;
			return bDoesExist;
		}
	}
	else
	{
		return false;
	}
#else
	if (PinnedModelStreamableBulkData->bIsStoredInDDC)
	{
		MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkReader::DoesBlockExist_DDC)
		return true;
	}
	else
	{
		MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkReader::DoesBlockExist_File)
		const FString FilePath = Private::MakeBlockFilePath(ObjectData->BulkFilePrefix, *Block);
		return FPlatformFileManager::Get().GetPlatformFile().FileExists(*FilePath);
	}
#endif
}


UE::Mutable::Private::FModelReader::FOperationID FUnrealMutableModelBulkReader::BeginReadBlock(const UE::Mutable::Private::FModel* Model, uint32 Key, void* Buffer, uint64 Size, UE::Mutable::Private::EDataType ResourceType, TFunction<void(bool bSuccess)>* CompletionCallback)
{
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkStreamer::OpenReadFile);

	FScopeLock Lock(&CriticalSection);
	
	UE_LOGF(LogMutable, VeryVerbose, "Streaming: reading data %08u.", Key);

	// Find the object we are streaming for
	FObjectData* ObjectData = Objects.FindByPredicate(
			[Model](const FObjectData& d) { return d.Model.Pin().Get() == Model; });

	if (!ObjectData)
	{
		// The object has been unloaded. Streaming is not possible. 
		// This may happen in the editor if we are recompiling an object but we still have instances from the old
		// object that have progressive mip generation.
		if (CompletionCallback)
		{
			(*CompletionCallback)(false);
		}
		return -1;
	}

	const TSharedPtr<FModelStreamableBulkData> PinnedModelStreamableBulkData = ObjectData->ModelStreamableBulkData.Pin();	
	if (!PinnedModelStreamableBulkData)
	{
		UE_LOGF(LogMutable, Error, "Streamable Bulk Data not available!");
		if (CompletionCallback)
		{
			(*CompletionCallback)(false);
		}
		return -1;
	}

	const FMutableStreamableBlock* Block = PinnedModelStreamableBulkData->ModelStreamables.Find(Key);
	if (!Block)
	{
		// File Handle not found! This shouldn't really happen.
		UE_LOGF(LogMutable, Error, "Streaming Block not found!");
		check(false);

		if (CompletionCallback)
		{
			(*CompletionCallback)(false);
		}
		return -1;
	}

	FOperationID Result = ++LastOperationID;

#if !WITH_EDITOR
	if (ensure(PinnedModelStreamableBulkData->StreamableBulkData.IsValidIndex(Block->FileId)))
	{
		FByteBulkData& BulkData = PinnedModelStreamableBulkData->StreamableBulkData[Block->FileId];

		FBulkDataIORequestCallBack IOCallback;

		if (CompletionCallback)
		{
			IOCallback = [CompletionCallbackCapture = *CompletionCallback](bool bWasCancelled, IBulkDataIORequest*)
				{
					CompletionCallbackCapture(!bWasCancelled);
				};
		}

		FReadRequest& Request = ObjectData->CurrentReadRequests.Add(Result);

		Request.BulkReadRequest = MakeShareable(BulkData.CreateStreamingRequest(
			Block->Offset,
			Size,
			(EAsyncIOPriorityAndFlags)StreamPriority,
			&IOCallback,
			reinterpret_cast<uint8*>(Buffer)));
	}
	else
	{
		return INDEX_NONE;
	}
#else
	if (PinnedModelStreamableBulkData->bIsStoredInDDC)
	{
		using namespace UE::DerivedData;

		UE::FSharedString SharedName = UE::FSharedString(TEXT("UnrealMutableModelBulkReader"));

		FCacheGetValueRequest Request;
		Request.Name = SharedName;
		Request.Key = PinnedModelStreamableBulkData->DDCKey;
		Request.Key.Hash = PinnedModelStreamableBulkData->DDCValues[Block->FileId];
		Request.Policy = PinnedModelStreamableBulkData->DDCDefaultPolicy;
		
		FReadRequest ReadRequest;
		ReadRequest.DDCReadRequest = MakeShared<FDDCReadRequest>();
		GetCache().GetValue(MakeArrayView(&Request,1), ReadRequest.DDCReadRequest->RequestOwner,
			[Request = ReadRequest.DDCReadRequest, CompletionCallbackCapture = *CompletionCallback, Block, Buffer, Size](FCacheGetValueResponse&& Response)
		{
			MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkStreamer::OpenReadFile_DDC_Callback);
			bool bSuccess = Response.Status == EStatus::Ok;

			if (bSuccess)
			{
				const FCompressedBuffer& CompressedBuffer = Response.Value.GetData();
				if (ensure(!CompressedBuffer.IsNull()))
				{
					if (Size == CompressedBuffer.GetRawSize())
					{
						bSuccess = CompressedBuffer.TryDecompressTo(MakeMemoryView(Buffer, Size));
					}
					else if (Block->Offset + Size <= CompressedBuffer.GetRawSize())
					{
						FCompressedBufferReader BufferReader(CompressedBuffer);
						bSuccess = BufferReader.TryDecompressTo(MakeMemoryView(Buffer, Size), Block->Offset);
					}
					else 
					{
						bSuccess = false;
					}
				}
			}

			Request->bSuccess = bSuccess;
			CompletionCallbackCapture(bSuccess);
		});

		ObjectData->CurrentReadRequests.Add(Result, ReadRequest);
	}
	else
	{
		FReadRequest ReadRequest;

		if (CompletionCallback)
		{
			ReadRequest.FileCallback = MakeShared<TFunction<void(bool, IAsyncReadRequest*)>>([CompletionCallbackCapture = *CompletionCallback](bool bWasCancelled, IAsyncReadRequest*) -> void
				{
					CompletionCallbackCapture(!bWasCancelled);
				});
		}

		TSharedPtr<IAsyncReadFileHandle> FileHandle;
		for (const TTuple<FOperationID, FReadRequest>& Pair : ObjectData->CurrentReadRequests)
		{
			if (Pair.Value.FileId == Block->FileId)
			{
				FileHandle = Pair.Value.FileHandle;
				break;
			}
		}
		
		if (!FileHandle)
		{
			const FString FilePath = Private::MakeBlockFilePath(ObjectData->BulkFilePrefix, *Block);
			FileHandle = MakeShareable(FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*FilePath));
			
			if (!FileHandle)
			{
				UE_LOGF(LogMutable, Error, "Failed to create AsyncReadFileHandle. File Path [%ls].", *FilePath);
				check(false);
				return -1;
			}
		}
	
		ReadRequest.FileId = Block->FileId;
		ReadRequest.FileHandle = FileHandle;
		ReadRequest.FileReadRequest = MakeShareable(FileHandle->ReadRequest(
			sizeof(MutableCompiledDataStreamHeader) + Block->Offset,
			Size,
			(EAsyncIOPriorityAndFlags)StreamPriority,
			ReadRequest.FileCallback.Get(),
			reinterpret_cast<uint8*>(Buffer)));

		ObjectData->CurrentReadRequests.Add(Result, ReadRequest);
	}
#endif
	

	INC_DWORD_STAT(STAT_MutableStreamingOps);

	return Result;
}


bool FUnrealMutableModelBulkReader::IsReadCompleted(UE::Mutable::Private::FModelReader::FOperationID OperationId)
{
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkStreamer::IsReadCompleted);

	FScopeLock Lock(&CriticalSection);
	
	for (FObjectData& o : Objects)
	{
		if (FReadRequest* ReadRequest = o.CurrentReadRequests.Find(OperationId))
		{
			if (ReadRequest->FileReadRequest)
			{
				return ReadRequest->FileReadRequest->PollCompletion();
			}
			else if (ReadRequest->BulkReadRequest)
			{
				return ReadRequest->BulkReadRequest->PollCompletion();
			}
#if WITH_EDITORONLY_DATA
			else if (ReadRequest->DDCReadRequest)
			{
				return ReadRequest->DDCReadRequest->RequestOwner.Poll(); // ReadLock
			}
#endif
		}
	}

	UE_LOGF(LogMutable, Error, "Operation not found in IsReadCompleted.");
	check(false);
	return true;
}


bool FUnrealMutableModelBulkReader::EndRead(UE::Mutable::Private::FModelReader::FOperationID OperationId)
{
	MUTABLE_CPUPROFILER_SCOPE(FUnrealMutableModelBulkStreamer::EndRead);

	FScopeLock Lock(&CriticalSection);

	bool bSuccess = true;
	bool bFound = false;
	for (FObjectData& o : Objects)
	{
		FReadRequest* ReadRequest = o.CurrentReadRequests.Find(OperationId);
		if (ReadRequest)
		{
			bool bCompleted = false;
			bool bResult = false;

			if (ReadRequest->FileReadRequest)
			{
				bCompleted = ReadRequest->FileReadRequest->WaitCompletion();
				bResult = ReadRequest->FileReadRequest->GetReadResults() != nullptr;
			}
			else if (ReadRequest->BulkReadRequest)
			{
				bCompleted = ReadRequest->BulkReadRequest->WaitCompletion();
				bResult = ReadRequest->BulkReadRequest->GetReadResults() != nullptr;
			}
#if WITH_EDITORONLY_DATA
			else if (ReadRequest->DDCReadRequest)
			{
				ReadRequest->DDCReadRequest->RequestOwner.Wait();
				
				bCompleted = true;
				bResult = ReadRequest->DDCReadRequest->bSuccess;
			}
#endif
			if (!bCompleted)
			{
				UE_LOGF(LogMutable, Error, "Operation failed to complete in EndRead.");
				check(false);
			}

			bSuccess = bCompleted && bResult;
			
			o.CurrentReadRequests.Remove(OperationId);
			bFound = true;
			break;
		}
	}

	if (!bFound)
	{
		UE_LOGF(LogMutable, Error, "Operation not found in EndRead.");
		check(false);
	}

	return bSuccess;
}


#if WITH_EDITOR

//-------------------------------------------------------------------------------------------------
FUnrealMutableModelBulkWriterEditor::FUnrealMutableModelBulkWriterEditor(FArchive* InMainDataArchive, FArchive* InStreamedDataArchive)
{
	MainDataArchive = InMainDataArchive;
	StreamedDataArchive = InStreamedDataArchive;
	CurrentWriteFile = nullptr;
}


void FUnrealMutableModelBulkWriterEditor::OpenWriteFile(uint32 BlockKey, bool bIsStreamable)
{
	if (!bIsStreamable) // Model
	{
		check(MainDataArchive);
		CurrentWriteFile = MainDataArchive;
	}
	else
	{
		check(StreamedDataArchive);
		CurrentWriteFile = StreamedDataArchive;
	}
}


void FUnrealMutableModelBulkWriterEditor::Write(const void* pBuffer, uint64 size)
{
	check(CurrentWriteFile);
	CurrentWriteFile->Serialize(const_cast<void*>(pBuffer), size);
}


void FUnrealMutableModelBulkWriterEditor::CloseWriteFile()
{
	CurrentWriteFile = nullptr;
}



//-------------------------------------------------------------------------------------------------
FUnrealMutableModelBulkWriterCook::FUnrealMutableModelBulkWriterCook(FArchive* InMainDataArchive, UE::Mutable::Private::FModelStreamableData* InStreamedData)
{
	MainDataArchive = InMainDataArchive;
	StreamedData = InStreamedData;
	CurrentKey = 0;
	CurrentIsStreamable = false;
}


void FUnrealMutableModelBulkWriterCook::OpenWriteFile(uint32 BlockKey, bool bIsStreamable)
{
	CurrentKey = BlockKey;
	CurrentIsStreamable = bIsStreamable;
}


void FUnrealMutableModelBulkWriterCook::Write(const void* pBuffer, uint64 size)
{
	if (!CurrentIsStreamable)
	{
		MainDataArchive->Serialize(const_cast<void*>(pBuffer), size);
	}
	else
	{
		StreamedData->Set(CurrentKey, reinterpret_cast<const uint8*>(pBuffer), size);
	}
}


void FUnrealMutableModelBulkWriterCook::CloseWriteFile()
{
}

FDDCReadRequest::FDDCReadRequest() :
	RequestOwner(UE::DerivedData::EPriority::High)
{
}

#endif

