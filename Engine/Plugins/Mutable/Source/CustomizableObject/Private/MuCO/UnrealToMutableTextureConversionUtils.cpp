// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCO/UnrealToMutableTextureConversionUtils.h"

#include "MuR/MutableTrace.h"
#include "MuR/Image.h"
#include "Engine/Texture2D.h"
#include "ImageCoreUtils.h"
#include "ImageUtils.h"
#include "Async/ParallelFor.h"

#if WITH_EDITOR

namespace UnrealToMutableImageConversion_Internal
{
	
FORCEINLINE ERawImageFormat::Type ConvertFormatSourceToRaw(const ETextureSourceFormat SourceFormat)
{
	return FImageCoreUtils::ConvertToRawImageFormat(SourceFormat);
}

EUnrealToMutableConversionError ApplyCompositeTexture(
        FImage& Image, 
        UTexture* CompositeTexture,
        const ECompositeTextureMode CompositeTextureMode,
        const float CompositePower)
{
    const int32 SizeX = CompositeTexture->Source.GetSizeX();
    const int32 SizeY = CompositeTexture->Source.GetSizeY();
    const ETextureSourceFormat SourceFormat = CompositeTexture->Source.GetFormat();

    const ERawImageFormat::Type RawFormat = ConvertFormatSourceToRaw(SourceFormat);

    if (RawFormat == ERawImageFormat::RGBA32F)
    {
        return EUnrealToMutableConversionError::CompositeUnsupportedFormat;
    }   

    FImage CompositeImage(SizeX, SizeY, 1, RawFormat, EGammaSpace::Linear);
    
    if(!CompositeTexture->Source.GetMipData(CompositeImage.RawData, 0))
    {
        return EUnrealToMutableConversionError::Unknown;
    }

    // Convert Composite Image to RGBA32F format and resize so both images have
    // the source image size.
    const bool bHaveSimilarAspect = FMath::IsNearlyEqual(
                float(SizeX) / float(SizeY), 
                float(Image.SizeX) / float(Image.SizeY), 
                KINDA_SMALL_NUMBER);
   
    if (SizeX < Image.SizeX || SizeY < Image.SizeY || !bHaveSimilarAspect)
    {
        return EUnrealToMutableConversionError::CompositeImageDimensionMismatch;
    }

    {
        FImage TempImage;
        CompositeImage.ResizeTo(
            TempImage, Image.SizeX, Image.SizeY, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
        
        Exchange(CompositeImage, TempImage);
    }

    TArrayView64<FLinearColor> ImageView = Image.AsRGBA32F();
    TArrayView64<FLinearColor> CompositeImageView = CompositeImage.AsRGBA32F();
   
    const size_t OutChannelOffset = [CompositeTextureMode]() -> size_t
        {
            switch(CompositeTextureMode)
            {
                case CTM_NormalRoughnessToRed:   return offsetof(FLinearColor, R);
                case CTM_NormalRoughnessToGreen: return offsetof(FLinearColor, G);
                case CTM_NormalRoughnessToBlue:  return offsetof(FLinearColor, B);
                case CTM_NormalRoughnessToAlpha: return offsetof(FLinearColor, A);
            }

            check(false);
            return 0;
        }(); 

    const int64 NumPixels = ImageView.Num();
    for (int64 I = 0; I < NumPixels; ++I)
    {
        const FVector Normal = FVector( 
                CompositeImageView[I].R * 2.0f - 1.0f,
                CompositeImageView[I].G * 2.0f - 1.0f,
                CompositeImageView[I].B * 2.0f - 1.0f); 

        // Is that C++ undefined behaviour?
        float* Value = reinterpret_cast<float*>(
                reinterpret_cast<uint8*>(&ImageView[I]) + OutChannelOffset);

        // See TextureCompressorModule.cpp:1924 for details. 
        // Toksvig estimation of variance
        float LengthN = FMath::Min( Normal.Size(), 1.0f );
        float Variance = ( 1.0f - LengthN ) / LengthN;
        Variance = FMath::Max( 0.0f, Variance - 0.00004f );

        Variance *= CompositePower;
        
        float Roughness = *Value;

        float a = Roughness * Roughness;
        float a2 = a * a;
        float B = 2.0f * Variance * (a2 - 1.0f);
        a2 = ( B - a2 ) / ( B - 1.0f );
        Roughness = FMath::Pow( a2, 0.25f );
        
        *Value = Roughness;
    }

    return EUnrealToMutableConversionError::Success;
}

void FlipGreenChannelRGBA32F(FImageView& Image)
{
	TArrayView64<FLinearColor> ImageDataView = Image.AsRGBA32F();
	ParallelFor(ImageDataView.Num(),
		[&ImageDataView](uint32 p)
		{
			ImageDataView[p].G = 1.0f - FMath::Clamp(ImageDataView[p].G, 0.0f, 1.0f);
		});
}

void FlipGreenChannelBGRA8(FImageView& Image)
{
	TArrayView64<FColor> ImageDataView = Image.AsBGRA8();
	ParallelFor(ImageDataView.Num(),
		[&ImageDataView](uint32 p)
		{
			ImageDataView[p].G = 255 - ImageDataView[p].G;
		});
}

void Normalize(FImageView& Image)
{
	TArrayView64<FLinearColor> ImageView = Image.AsRGBA32F();

	for (FLinearColor& Color : ImageView)
	{
		FVector3f Normal = (FVector3f(Color.R, Color.G, Color.B) * 2.0f - 1.0f).GetUnsafeNormal();

		Color.R = Normal.X * 0.5f + 0.5f;
		Color.G = Normal.Y * 0.5f + 0.5f;
		Color.B = Normal.Z * 0.5f + 0.5f;
	}
}

void BlurNormalForComposite(FImageView& Image)
{
	TArrayView64<FLinearColor> ImageView = Image.AsRGBA32F();

	const int32 SizeX = Image.SizeX;
	const int32 SizeY = Image.SizeY;

	for (int32 Y = 0; Y < SizeY - 1; ++Y)
	{
		for (int32 X = 0; X < SizeX - 1; ++X)
		{
			const int64 Idx0 = Y * SizeX + X;
			const int64 Idx1 = Y * SizeX + (X + 1);
			const int64 Idx2 = (Y + 1) * SizeX + X;
			const int64 Idx3 = (Y + 1) * SizeX + (X + 1);

			// Simple 2x2 box filter in place to gather info about the top mip normals variance.
			ImageView[Idx0] = (ImageView[Idx0] + ImageView[Idx1] + ImageView[Idx2] + ImageView[Idx3]) * 0.25f;
		}
	}
}

} //namespace UnrealToMutableImageConversion_Internal


FMutableSourceTextureData::FMutableSourceTextureData(const UTexture2D& Texture)
{
	Source = Texture.Source.CopyTornOff();
	bFlipGreenChannel = Texture.bFlipGreenChannel;
	bHasAlphaChannel = Texture.AdjustMinAlpha != Texture.AdjustMaxAlpha &&
		Texture.CompressionSettings != TC_Normalmap &&
		!Texture.CompressionNoAlpha;
	bCompressionForceAlpha = Texture.CompressionForceAlpha;
	bIsNormalComposite = false; // TODO?
}


FTextureSource& FMutableSourceTextureData::GetSource()
{
	return Source;	
}


bool FMutableSourceTextureData::GetFlipGreenChannel() const
{
	return bFlipGreenChannel;
}


bool FMutableSourceTextureData::HasAlphaChannel() const
{
	return bHasAlphaChannel;
}


bool FMutableSourceTextureData::GetCompressionForceAlpha() const
{
	return bCompressionForceAlpha;
}


bool FMutableSourceTextureData::IsNormalComposite() const
{
	return bIsNormalComposite;
}


EUnrealToMutableConversionError ConvertTextureUnrealSourceToMutable(UE::Mutable::Private::FImage* OutResult, FMutableSourceTextureData& Texture, uint8 MipmapsToSkip, bool bLoadMipTail)
{
	MUTABLE_CPUPROFILER_SCOPE(ConvertTextureUnrealSourceToMutable);

	using namespace UnrealToMutableImageConversion_Internal;

	FTextureSource& Source = Texture.GetSource();
	
	// Correct mips to skip to fit source data
	MipmapsToSkip = FMath::Clamp(MipmapsToSkip, 0, Source.GetNumMips()-1);

	const int32 TextureNumMips = Source.GetNumMips();

	int32 TextureMipIndexBegin = MipmapsToSkip < TextureNumMips ? MipmapsToSkip : TextureNumMips - 1;
	check(TextureMipIndexBegin >= 0);

	int32 NumMipsToLoad = bLoadMipTail ? TextureMipIndexBegin - TextureNumMips : 1;
	int32 LastMipToLoad = bLoadMipTail ? TextureNumMips - 1 : TextureMipIndexBegin;
	int32 TextureMipIndexEnd = LastMipToLoad + 1;

	const int32 SizeX = FMath::Max(Source.GetSizeX() >> MipmapsToSkip, 1);
	const int32 SizeY = FMath::Max(Source.GetSizeY() >> MipmapsToSkip, 1);

	ETextureSourceFormat Format = Source.GetFormat();
 
	ERawImageFormat::Type RawFormat = ConvertFormatSourceToRaw(Format);

	int32 ImageNumMips = TextureMipIndexEnd - TextureMipIndexBegin;
	FMipMapImage TempImageStorage0;
	FMipMapImage TempImageStorage1;

	FMipMapImage* TempImage0 = &TempImageStorage0;
	FMipMapImage* TempImage1 = &TempImageStorage1;

	// What if source data is not linear?
	TempImage0->Init(SizeX, SizeY, ImageNumMips, RawFormat, EGammaSpace::Linear);

	for (int32 MipLevel = 0; MipLevel < ImageNumMips; ++MipLevel)
	{
		FImageView MipView = TempImage0->GetMipImage(MipLevel);
		
		FImage OutMipImage(MipView.SizeX, MipView.SizeY, MipView.Format, MipView.GammaSpace);
		if (!Source.GetMipImage(OutMipImage, TextureMipIndexBegin + MipLevel))
		{
			return EUnrealToMutableConversionError::Unknown;
		}

		check(MipView.GetImageSizeBytes() == OutMipImage.GetImageSizeBytes());
		FMemory::Memcpy(MipView.RawData, OutMipImage.RawData.GetData(), MipView.GetImageSizeBytes());
	}

	bool bFlipGreenChannel = Texture.GetFlipGreenChannel();

	// If any post processes of the image is needed, convert to RGBA32F
	if (Texture.IsNormalComposite())
	{ 
		MUTABLE_CPUPROFILER_SCOPE(FlipOrComposite);

		TempImage0->CopyTo(*TempImage1, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
		RawFormat = ERawImageFormat::RGBA32F;

		for (int32 MipLevel = 0; MipLevel < TempImage1->GetMipCount(); ++MipLevel)
		{
			FImageView MipView = TempImage1->GetMipImage(MipLevel);

			if (bFlipGreenChannel)
			{
				FlipGreenChannelRGBA32F(MipView);
			}

			// Prepare texture for use as normal composite.
			Normalize(MipView);
			BlurNormalForComposite(MipView);
		}
		
		// Don't flip again below.
		bFlipGreenChannel = false;

		// TempImage0 is always used as the source of the operations, swap to be consistent.
		::Swap(TempImage0, TempImage1);
	}
   
	const ERawImageFormat::Type MutableCompatibleFormat = Format == TSF_G8 
			? ERawImageFormat::G8 
			: ERawImageFormat::BGRA8;

	if (MutableCompatibleFormat != RawFormat)
	{
		MUTABLE_CPUPROFILER_SCOPE(ToCompatibleFormat);
		TempImage0->CopyTo(*TempImage1, MutableCompatibleFormat, EGammaSpace::Linear);
		::Swap(TempImage0, TempImage1);
	}
	
	for (int32 MipLevel = 0; MipLevel < TempImage0->GetMipCount(); ++MipLevel)
	{
		FImageView MipView = TempImage0->GetMipImage(MipLevel);

		if (bFlipGreenChannel)
		{
			FlipGreenChannelBGRA8(MipView);
		}
	}

	switch (MutableCompatibleFormat)
	{
	case ERawImageFormat::G8:
	{
		MUTABLE_CPUPROFILER_SCOPE(NoConvert);

		OutResult->Init(SizeX, SizeY, ImageNumMips, UE::Mutable::Private::EImageFormat::L_UByte, UE::Mutable::Private::EInitializationType::NotInitialized);
	
		for (int32 MipLevel = 0; MipLevel < ImageNumMips; ++MipLevel)
		{
			TArrayView<uint8> MutableImageDataView = OutResult->DataStorage.GetLOD(MipLevel);
			FImageView ImageCoreDataView  = TempImage0->GetMipImage(MipLevel);

			check(MutableImageDataView.Num() == ImageCoreDataView.GetImageSizeBytes());
			FMemory::Memcpy(MutableImageDataView.GetData(), ImageCoreDataView.RawData, MutableImageDataView.Num());
		}
		break;
	}

	case ERawImageFormat::BGRA8:
	{
		// Try to find out if the texture has and actually makes use of the alpha channel
		bool bHasAlphaChannel = Texture.HasAlphaChannel()
			&& (Texture.GetCompressionForceAlpha()
				||
				FImageCore::DetectAlphaChannel(TempImage0->GetMipImage(0)));

		// TODO: If we ever manage to get Pixel Format data on cook compilation time, remove the code that sets bHasAlphaChannel and just use Texture->HasAlphaChannel() here. Currently unreliable, it always returns EPixelFormat::PF_Unknown when cooking, which returns always "false" to HasAlphaChannel().
		if (bHasAlphaChannel)
		{
			MUTABLE_CPUPROFILER_SCOPE(ToRGBA);
			OutResult->Init(SizeX, SizeY, ImageNumMips, UE::Mutable::Private::EImageFormat::RGBA_UByte, UE::Mutable::Private::EInitializationType::NotInitialized);
		
			for (int32 MipLevel = 0; MipLevel < ImageNumMips; ++MipLevel)
			{
				FImageView MipImageView = TempImage0->GetMipImage(MipLevel);
				uint8* DataDest = OutResult->GetLODData(MipLevel);

				// Convert to RGBA8 while copying
				TArrayView64<FColor> ImageDataView = MipImageView.AsBGRA8();
				ParallelFor(TEXT("MutableToRGBA"), ImageDataView.Num(), 16*1024,
				[DataDest, &ImageDataView](uint32 P)
				{
					DataDest[4 * P + 0] = ImageDataView[P].R;
					DataDest[4 * P + 1] = ImageDataView[P].G;
					DataDest[4 * P + 2] = ImageDataView[P].B;
					DataDest[4 * P + 3] = ImageDataView[P].A;
				});
			}   
		}
		else
		{
			MUTABLE_CPUPROFILER_SCOPE(ToRGB);
			// TODO: add support for a UE::Mutable::Private::IF_RGBX_UBYTE?
			OutResult->Init(SizeX, SizeY, ImageNumMips, UE::Mutable::Private::EImageFormat::RGB_UByte, UE::Mutable::Private::EInitializationType::NotInitialized);
		
			for (int32 MipLevel = 0; MipLevel < ImageNumMips; ++MipLevel)
			{
				FImageView MipImageView = TempImage0->GetMipImage(MipLevel);
				uint8* DataDest = OutResult->GetLODData(MipLevel);

				// Convert to RGBA8 while copying
				TArrayView64<FColor> ImageDataView = MipImageView.AsBGRA8();
				ParallelFor(TEXT("MutableToRGB"), ImageDataView.Num(), 16*1024,
				[DataDest, &ImageDataView](uint32 P)
				{
					DataDest[3 * P + 0] = ImageDataView[P].R;
					DataDest[3 * P + 1] = ImageDataView[P].G;
					DataDest[3 * P + 2] = ImageDataView[P].B;
				});
			}

		}

		//FString Msg = FString::Printf(TEXT("Alpha channel is %s for %s"), bHasAlphaChannel ? TEXT("enabled") : TEXT("disabled"), *Texture->GetName());
		//UE_LOGF(LogMutable, VeryVerbose, "%ls", *Msg);

		break;
	}

	default:
		// Format not supported yet?
		check(false);
		break;
	}

	return EUnrealToMutableConversionError::Success;
}


#endif // WITH_EDITOR
