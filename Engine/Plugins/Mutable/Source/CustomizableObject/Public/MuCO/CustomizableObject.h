// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "RHIDefinitions.h"
#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"
#include "MuCO/CustomizableObjectUIData.h"
#include "MuCO/CustomizableObject_Deprecated.h"
#include "Templates/TypeCompatibleBytes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SkeletalMeshTypes.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "CustomizableObject.generated.h"

class FReply;
class FObjectPreSaveContext;
class FText;
class IAsyncReadFileHandle;
class ITargetPlatform;
class UCustomizableObject;
class UEdGraph;
class USkeletalMesh;
class UCustomizableObjectPrivate;
struct FFrame;
struct FStreamableHandle;
template <typename FuncType> class TFunctionRef;


CUSTOMIZABLEOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogMutable, Log, All);


DECLARE_MULTICAST_DELEGATE(FPostCompileDelegate)


CUSTOMIZABLEOBJECT_API extern TAutoConsoleVariable<bool> CVarMutableAsyncCook;


USTRUCT()
struct FFParameterOptionsTags
{
	GENERATED_USTRUCT_BODY()

	/** List of tags of a Parameter Options */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TArray<FString> Tags;
};


USTRUCT()
struct FParameterTags
{
	GENERATED_USTRUCT_BODY()

	/** List of tags of a parameter */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TArray<FString> Tags;

	/** Map of options available for a parameter can have and their tags */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TMap<FString, FFParameterOptionsTags> ParameterOptions;
};


USTRUCT()
struct FMutableLODSettings
{
	GENERATED_BODY()

	/** Minimum LOD to render per Platform. */
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (DisplayName = "Minimum LOD"))
	FPerPlatformInt MinLOD;

	/** Minimum LOD to render per Quality level.*/
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (DisplayName = "Quality Level Minimum LOD"))
	FPerQualityLevelInt MinQualityLevelLOD;

#if WITH_EDITORONLY_DATA

	/** Override the LOD Streaming settings from the reference skeletal meshes.*/
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (DisplayName = "Override LOD Streaming Settings"))
	bool bOverrideLODStreamingSettings = true;

	/** Enabled: streaming LODs will trigger automatic updates to generate and discard LODs. Streaming may decrease the amount of memory used, but will stress the CPU and Streaming of resources.
	  *	Keep in mind that, even though updates may be faster depending on the amount of LODs to generate, there may be more updates to process.
	  * 
	  * Disabled: all LODs will be generated at once. It may increase the amount of memory used by the meshes and the generation may take longer, but less updates will be required. */
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (EditCondition = "bOverrideLODStreamingSettings", DisplayName = "Enable LOD Streaming"))
	FPerPlatformBool bEnableLODStreaming = true;

	/** Limit the number of LODs to stream. A value of 0 is the same as disabling streaming of LODs.*/
	UPROPERTY(EditAnywhere, Category = LODSettings, meta = (EditCondition = "bOverrideLODStreamingSettings"))
	FPerPlatformInt NumMaxStreamedLODs = MAX_MESH_LOD_COUNT;

#endif
};


USTRUCT(BlueprintType)
struct FCompileCallbackParams
{
	GENERATED_BODY()

	/** The compile request has failed. See output log for more details. False if the request has been skipped. */
	UPROPERTY(BlueprintReadOnly, Category = Compile)
	bool bRequestFailed = false;

	/** The compile request has finished with warnings. False if the request has been skipped. */
	UPROPERTY(BlueprintReadOnly, Category = Compile)
	bool bWarnings = false;

	/** The compile request has finished with errors. False if the request has been skipped. */
	UPROPERTY(BlueprintReadOnly, Category = Compile)
	bool bErrors = false;
	
	/** The compilation has been skipped due to Compile Parameters. */
	UPROPERTY(BlueprintReadOnly, Category = Compile)
	bool bSkipped = false;

	/** The Customizable Object is compiled. */
	UPROPERTY(BlueprintReadOnly, Category = Compile)
	bool bCompiled = false;
};


DECLARE_DYNAMIC_DELEGATE_OneParam(FCompileDelegate, const FCompileCallbackParams&, Params);

DECLARE_DELEGATE_OneParam(FCompileNativeDelegate, const FCompileCallbackParams&);


USTRUCT(BlueprintType)
struct FCompileParams
{
	GENERATED_BODY()

	/** If true, skip the compilation if the Customizable Object is already compiled. */
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	bool bSkipIfCompiled = false;

	/** If true, skip the compilation if the Customizable Object or its contents are not out of date (child Customizable Objects, Data Tables, referenced Materials...). */
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	bool bSkipIfNotOutOfDate = true;
	
	/** Gathers all asset references used in this Customizable Object. Marks the object as modified.*/
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	bool bGatherReferences = false;

	/** If true, compile the Customizable Object asynchronously. If synchronously and the Customizable Object is not ready the compilation will fail. */
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	bool bAsync = true;

	/** If assigned, compile the Customizable Object only the selected parameter of the given Instance. */
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	TObjectPtr<UCustomizableObjectInstance> CompileOnlySelectedInstance = nullptr;
	
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	ECustomizableObjectOptimizationLevel OptimizationLevel = ECustomizableObjectOptimizationLevel::FromCustomizableObject;

	UPROPERTY(BlueprintReadWrite, Category = Compile)
	ECustomizableObjectTextureCompression TextureCompression = ECustomizableObjectTextureCompression::Fast;

	/** Called once the compilation request has finished (successfully or not). */
	UPROPERTY(BlueprintReadWrite, Category = Compile)
	FCompileDelegate Callback;

	/** See Callback. */
	FCompileNativeDelegate CallbackNative;
};


UCLASS(MinimalAPI,  BlueprintType, config=Engine )
class UCustomizableObject : public UObject
{
	friend UCustomizableObjectPrivate;
	
public:
	GENERATED_BODY()

	CUSTOMIZABLEOBJECT_API UCustomizableObject();

#if WITH_EDITORONLY_DATA
private:

	UPROPERTY()
	TObjectPtr<USkeletalMesh> ReferenceSkeletalMesh_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<USkeletalMesh>> ReferenceSkeletalMeshes_DEPRECATED;

public:

	/** Optional Version Bridge asset.
	  *
	  * Used to decide which Mutable child Customizable Objects and Data Table rows must be included in a compilation/cook.
	  * An asset will be included if the struct/column version matches the game-specific system version.
	  *
	  * The provided asset must implement ICustomizableObjectVersionBridgeInterface. */
	UPROPERTY(EditAnywhere, Category = Versioning)
	TObjectPtr<UObject> VersionBridge;

	/** Optional struct.
	  *
	  * Used to define which version this child Customizable Object belongs to. It will be used during
	  * cook/compilation to decide whether this Customizable Object should be included in the final compiled Customizable Object.
	  *
	  * To work, the root Customizable Object must have a Version Bridge. */
	UPROPERTY(EditAnywhere, Category = Versioning)
	FInstancedStruct VersionStruct;
#endif

	UE_DEPRECATED(5.6, "Moved to Mesh Component Node")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage="Moved to Mesh Component Node"))
	FMutableLODSettings LODSettings;
	
	/** true to use the Reference Skeletal Mesh as a placeholder while the Generated Skeletal Mesh is not ready.
	  * If false, a null mesh will be used to replace the discarded mesh due to 'ReplaceDiscardedWithReferenceMesh' being enabled. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, DisplayName = "Use Reference Skeletal Mesh As Placeholder")
	bool bEnableUseRefSkeletalMeshAsPlaceholder = true;
	
	/** If true, reuse previously generated USkeletalMesh (if still valid and the the number of LOD have not changed)
	 * USkeletalMeshes are only reused between the same CO. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bEnableMeshCache = false;
	
#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	FCompilationOptions_DEPRECATED CompileOptions_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bEnableRealTimeMorphTargets = false;

	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bEnable16BitBoneWeights = false;

	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bEnableAltSkinWeightProfiles = false;

	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bEnablePhysicsAssetMerge = false;
	
	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bUseLegacyLayouts = false;

	/** Experimental */
	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bEnableAnimBpPhysicsAssetsManipulation = false;
	
	// Disabling the Table Materials parent material check will let the user use any material regardless of its parent when connecting a material from a table column to a material node.
	// Warning! But it will not check if the table material channels connected to the Material node exist in the actual material used in the Instance, and will fail silently at runtime 
	// when setting the value of those channels if they don't exist.
	UPROPERTY(EditAnywhere, Category = CompileOptions)
	bool bDisableTableMaterialsParentCheck = false;

	// Options when compiling this customizable object (see EMutableCompileMeshType declaration for info)
	UPROPERTY(EditAnywhere, Category = CompileOptions)
	EMutableCompileMeshType MeshCompileType = EMutableCompileMeshType::LocalAndChildren;

	// Array of elements to use with compile option CompileType = WorkingSet
	UPROPERTY(EditAnywhere, Category = CompileOptions)
	TArray<TSoftObjectPtr<UCustomizableObject>> WorkingSet;

	// Editor graph
	UPROPERTY()
	TObjectPtr<UEdGraph> Source;

	// Used to verify the derived data matches this version of the Customizable Object.
	UPROPERTY()
	FGuid VersionId;
#endif // WITH_EDITORONLY_DATA

public:
	/** Get the number of components this Customizable Object has. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API int32 GetComponentCount() const;

	/** Return the name of the component.
	 *  @param ComponentIndex [0 - GetComponentCount). Index may not represent the same component between runs. To identify them use the name.
	 *  @return NAME_None if the component does not exist. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FName GetComponentName(int32 ComponentIndex) const;
	
	/** Get the number of parameters available in instances of this object. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API int32 GetParameterCount() const;
	
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool ContainsParameter(const FString& ParameterName) const;
	
	/** Get the type of a parameter from its name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API EMutableParameterType GetParameterTypeByName(const FString& Name) const; // TODO >5.6 rename to GetParameterType (with redirector). Name -> ParameterName (create redirector)
	
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API const FString& GetParameterName(int32 ParamIndex) const;
	
	/** If the given parameter is enum int parameter, return how many possible values an enum parameter has. Otherwise, return 0. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API int32 GetEnumParameterNumValues(const FString& ParameterName) const;

	/** Gets the Name of the value at position ValueIndex in the list of available values for the int parameter. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API const FString& GetEnumParameterValue(const FString& ParameterName, int32 ValueIndex) const;

	/** Return true if the enum parameter contains this value a possible option. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool ContainsEnumParameterValue(const FString& ParameterName, const FString Value) const;

	// DEPRECATED: Use GetSkeletalMeshComponentReferenceSkeletalMesh instead.
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API USkeletalMesh* GetComponentMeshReferenceSkeletalMesh(const FName& Name) const;

	/** Given a Mesh Component name, return its reference Skeletal Mesh. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API USkeletalMesh* GetSkeletalMeshComponentReferenceSkeletalMesh(const FName& ComponentName) const;
	
	/** Get the default value of a parameter of type Float.
	  * @param ParameterName The name of the Float parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API float GetFloatParameterDefaultValue(const FString& ParameterName) const;
	
	/** Get the default value of a parameter of type Int. 
	  * @param ParameterName The name of the Int parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API int32 GetEnumParameterDefaultValue(const FString& ParameterName) const;
	
	/** Get the default value of a parameter of type Bool.
	  * @param ParameterName The name of the Bool parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool GetBoolParameterDefaultValue(const FString& ParameterName) const;

	/** Get the default value of a parameter of type Color.
	  * @param ParameterName The name of the Color parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FLinearColor GetColorParameterDefaultValue(const FString& ParameterName) const;

	/** Get the default value of a parameter of type Vector.
	  * @param ParameterName The name of the Vector parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FVector4f GetVectorParameterDefaultValue(const FString& ParameterName) const;
	
	/** Get the default value of a parameter of type Transform.
	  * @param ParameterName The name of the Transform parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FTransform GetTransformParameterDefaultValue(const FString& ParameterName) const;

	/** Get the default value of a projector with the provided name
	  * @param ParameterName The name of the parameter to get the default value of.
	  * @return A data structure containing all the default data for the targeted projector parameter. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FCustomizableObjectProjector GetProjectorParameterDefaultValue(const FString& ParameterName) const;
	
	/** Get the default value of a parameter of type Texture.
	  * @param InParameterName The name of the Texture parameter to get the default value of. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API UTexture* GetTextureParameterDefaultValue(const FString& InParameterName) const;

	/** Get the default value of a parameter of type Skeletal Mesh.
	  * @param ParameterName The name of the Mesh parameter to get the default value of. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API USkeletalMesh* GetSkeletalMeshParameterDefaultValue(const FString& ParameterName) const;
	
	/** Get the default value of a parameter of type Material.
	* @param ParameterName The name of the Material parameter to get the default value of. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API UMaterialInterface* GetMaterialParameterDefaultValue(const FString& ParameterName) const;

	/** Get the default value of a parameter of type External Type.
	  * @param ParameterName The name of the External Type parameter to get the default value of.
	  * @return The default value of the provided parameter name. */
	CUSTOMIZABLEOBJECT_API FInstancedStruct GetExternalTypeParameterDefaultValue(const FString& ParameterName) const;
	
	/** Return true if the parameter at the index provided is multidimensional.
	  * @param InParameterName The name of the parameter to check.
	  * @return True if the parameter is multidimensional and false if it is not. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool IsParameterMultidimensional(UPARAM(DisplayName = "Parameter Name") const FString& InParameterName) const;
	
	/** Compile the Customizable Object. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API void Compile(const FCompileParams& Params);
	
#if WITH_EDITOR
	/** DEPRECATED. Use Compile instead.
	  * 
	  *	Compile the object if Automatic Compilation is enabled and has not been already compiled.
	  * Automatic compilation can be enabled/disabled in the Mutable's Plugin Settings.
	  * @return true if compiled */
	CUSTOMIZABLEOBJECT_API bool ConditionalAutoCompile();
#endif

	// UObject interface
	CUSTOMIZABLEOBJECT_API virtual void PostLoad() override;
	CUSTOMIZABLEOBJECT_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	CUSTOMIZABLEOBJECT_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	CUSTOMIZABLEOBJECT_API virtual FString GetDesc() override;
	CUSTOMIZABLEOBJECT_API virtual bool IsEditorOnly() const override;
	CUSTOMIZABLEOBJECT_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	CUSTOMIZABLEOBJECT_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	CUSTOMIZABLEOBJECT_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;	
	CUSTOMIZABLEOBJECT_API virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	CUSTOMIZABLEOBJECT_API virtual bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;
#endif

	/** Return the number of object states that are defined in the CustomizableObject. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API int32 GetStateCount() const;

	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FString GetStateName(int32 StateIndex) const;
	
	/** Return the number of parameters that are editable at runtime for a specific state. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API int32 GetStateParameterCount(const FString& StateName) const;

	/** Return the name of one of the state's runtime parameters, by its index (from 0 to GetStateParameterCount - 1). */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FString GetStateParameterName(const FString& StateName, int32 ParameterIndex) const;
	
	/** Return the metadata associated to a parameter. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FMutableParamUIMetadata GetParameterUIMetadata(const FString& ParamName) const;

	/** Return the metadata associated to the given enum parameter value. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FMutableParamUIMetadata GetEnumParameterValueUIMetadata(const FString& ParamName, const FString& OptionName) const; // TODO GMT Rename "OptionName" -> "Value" using core redirects
	
	/** Returns the group type of the given integer parameter */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API ECustomizableObjectGroupType GetEnumParameterGroupType(const FString& ParamName) const;

	/** Return the metadata associated to a state. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API FMutableStateUIMetadata GetStateUIMetadata(const FString& StateName) const;

#if WITH_EDITOR
	/** Return the DataTables used by the given parameter and its value (if any). */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API TArray<TSoftObjectPtr<UDataTable>> GetEnumParameterValueDataTable(const FString& ParamName, const FString& Value);
#endif
	
private:
	// Customizable Object Population data start ------------------------------------------------------
	/** Array to store the selected Population Class tags for this Customizable Object */
	UPROPERTY()
	TArray<FString> CustomizableObjectClassTags;
	
	/** Array to store all the Population Class tags */
	UPROPERTY()
	TArray<FString> PopulationClassTags;

	/** Map of parameters available for the Customizable Object and their tags */
	UPROPERTY()
	TMap<FString, FParameterTags> CustomizableObjectParametersTags;
	// Customizable Object Population data end --------------------------------------------------------

#if WITH_EDITORONLY_DATA
	/** True if this object references a parent object. This is used basically to exclude this object
	  * from cooking. This is actually derived from the source graph object node pointing to another
	  * object or not, but it needs to be cached here because the source graph is not always available.
	  * For old objects this may be false even if they are child objects until they are resaved, but 
	  * that is the conservative case and shouldn't cause a problem. */
	UPROPERTY()
	bool bIsChildObject = false;
#endif
	
public:
#if WITH_EDITORONLY_DATA
	CUSTOMIZABLEOBJECT_API FPostCompileDelegate& GetPostCompileDelegate();
#endif

	/** Create a new instance of this object. The instance parameters will be initialized with the object default values. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API UCustomizableObjectInstance* CreateInstance();
	
	/** Check if the CustomizableObject asset has been compiled. This will always be true in a packaged game, but it could be false in the editor.
	 *  It may return false due to the Customizable Object still being loaded. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool IsCompiled() const;

	/** Return true if the Customizable Object is still being loaded. It may take a few frames to load the Customizable Object. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool IsLoading() const;
	
#if WITH_EDITOR
	/** Return true if this Customizable Object has references to a parent Customizable Object. Only root Customizable Objects will return false. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	CUSTOMIZABLEOBJECT_API bool IsChildObject() const;
#endif
	
	CUSTOMIZABLEOBJECT_API const UCustomizableObjectPrivate* GetPrivate() const;

	CUSTOMIZABLEOBJECT_API UCustomizableObjectPrivate* GetPrivate();
	
private:

	UPROPERTY(Instanced)
	TObjectPtr<UCustomizableObjectPrivate> Private;
};
