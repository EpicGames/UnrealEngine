// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "CustomizableObjectExtension.generated.h"

namespace UE::Mutable::Private
{
	class FExtensionData;
}

class UCustomizableObjectInstance;
class UCustomizableObjectInstanceUsage;
class USkeletalMesh;
class ACustomizableSkeletalMeshActor;

/** A type of pin in the Mutable graph UI */
struct FCustomizableObjectPinType
{
	/*
	 * The identifier for this type, to be used internally
	 *
	 * Note that the same pin type names may be used by different extensions, so that extensions
	 * can interoperate with each other using extension-defined pin types.
	 * 
	 * In other words, it's valid for one extension to create a new pin type and another extension
	 * to create nodes that use that type.
	 */
	FName Name;

	/** The display name for this type in the editor UI */
	FText DisplayName;

	/** The color that will be used in the editor UI for this pin and any wires connected to it */
	FLinearColor Color;
};

/** An input pin that will be added to Object nodes */
struct FObjectNodeInputPin
{
	/** This can be the name of a built-in pin type or an extension-defined FCustomizableObjectPinType */
	FName PinType;

	/**
	 * The internal name for the pin, to disambiguate it from other pins.
	 *
	 * Ensure this name is unique for object node pins created by the same extension.
	 * 
	 * The system can automatically distinguish between pins with the same name across different
	 * extensions, so this doesn't need to be a globally unique name.
	 */
	FName PinName;

	/** The name that will be displayed for the pin in the editor UI */
	FText DisplayName;

	/**
	 * Whether this pin accepts multiple inputs or not.
	 *
	 * Note that even if this is false, an Object node pin can still receive one input per Child
	 * Object node, so the extension still needs to handle receiving multiple inputs for a single
	 * pin.
	 */
	bool bIsArray = false;
};


/**
 * An extension that adds functionality to the Customizable Object system
 *
 * To create a new extension, make a subclass of this class and register it by calling
 * ICustomizableObjectModule::Get().RegisterExtension().
 */
UCLASS(MinimalAPI)
class UCustomizableObjectExtension : public UObject
{
	GENERATED_BODY() 
public:
	/** Returns any new pin types that are defined by this extension */
	virtual TArray<FCustomizableObjectPinType> GetPinTypes() const { return TArray<FCustomizableObjectPinType>(); }

	/** Returns the pins that this extension adds to Object nodes */
	virtual TArray<FObjectNodeInputPin> GetAdditionalObjectNodePins() const { return TArray<FObjectNodeInputPin>(); }

	/**
	 * Called when a Skeletal Mesh asset is created
	 *
	 * @param InputPinData - The data for only the input pins *registered by this extension*. This
	 * helps to enforce separation between the extensions, so that they don't depend on each other.
	 * 
	 * @param ComponentName - The component name of the Skeletal Mesh, for the case where the pin
	 * data is associated with a particular component.
	 * 
	 * @param SkeletalMesh - The Skeletal Mesh that was created.
	 */
	virtual void OnSkeletalMeshCreated(FName ComponentName, USkeletalMesh* SkeletalMesh) const {}
	
	/** Called when a Customizable Object Instance Usage is being updated.
	 * 
	 * @param Usage Customizable Object Instance Usage being updated. */
	virtual void OnCustomizableObjectInstanceUsageUpdated(UCustomizableObjectInstanceUsage& Usage, TArray<TObjectPtr<const UObject>>& ExtensionData) const {}

	/** Called when a Customizable Object Instance Usage is being discarded.
	 * 
	 * @param Usage Customizable Object Instance Usage being discarded. */
	virtual void OnCustomizableObjectInstanceUsageDiscarded(UCustomizableObjectInstanceUsage& Usage) const {}
};
