// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphFwd.h"
#include "Templates/SubclassOf.h"
#include "Math/RandomStream.h"
#include "MuCO/CustomizableObjectInstanceDescriptor.h"

#include "CustomizableObjectInstance.generated.h"

class UCustomizableObject;
class USkeletalMesh;
class UAnimInstance;
class UMaterialInterface;
class FProperty;
class FUpdateContextPrivate;
struct FFrame;
struct FGameplayTagContainer;


/**
 * Represents what kind of saving procedure was performed to save the package
 */
UENUM()
enum class EPackageSaveResolutionType : uint8
{
	None = 0,

	/** The package got saved as a new file. */
	NewFile,

	/** The package was already present on disk so it was used instead of creating a new one. */
	ReusedFile,
	
	/** The package was already present on disk so the old package was deleted and a new one was saved on its place */
	Overriden  UE_DEPRECATED(5.8, "This option is no longer used")		UMETA(Hidden),		// Deprecated

	/** Error type : An override was required but due to an error or lack of user permission it could not be done. */
	UnableToOverride UE_DEPRECATED(5.8, "This option is no longer used") UMETA(Hidden),		// Deprecated
};


/**
 * Data structure that exposes the path to a baked package and also what type of save was performed (an override, a standard save with a new file...)
 */
USTRUCT(BlueprintType, Blueprintable)
struct FBakedResourceData
{
	GENERATED_BODY()

	/**
	 * The way the package represented by this object was saved onto disk.
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	EPackageSaveResolutionType SaveType = EPackageSaveResolutionType::None;

	/**
	 * The path used by the saved package.
	 * EX : "/Game/baking_CO/BakeDir_Char/Part3/T_CharacterBake_Body_EyeBrown_Normal_2"
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	FString AssetPath;

	/**
	 * The section of the asset name that represents the asset type based prefix.
	 * EX : "T_", "M_", "SKEL_", "" ...
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	FString Prefix;

	/**
	 * The section of the asset name that represents the name the user has requested for all the baked resources to have.
	 * EX : "CharacterBake_", "CO_ABC_", "MyTestBake_", "" ...
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	FString UserGivenName;

	/**
	 * The section of the asset name that represents the name of the component of the source asset that was baked.
	 * It can be empty in the case the resource is shared between multiple components.
	 * EX : "Body_", "A_", "B_", "" ...
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	FString ComponentName;

	/**
	 * The section of the asset name that represents the name of the source resource that was baked.
	 * EX : "EyeBrown_Normal", "BodyTexture_4849745649878/", "AlbedoEyes_4756489*65", "BodyTexture" ...
	 * @note The substring is composed by the source object name and, in some cases, a suffix that contains the mutable resource generation hash used
	 * to be able to reuse baked mutable resources between baking runs.
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	FString ResourceName;

	/**
	 * The section of the asset name that represents the suffix added in some cases to prevent name collisions during the bake.
	 * This is most likely to be empty.
	 * EX : "_2", "_1" , "" ...
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	FString UniqueAssetNameSuffix;
};


/**
 * Structure returned as output of the baking operation. May contain a filled collection of FBakedResourceData objects and also the success end state of the
 * baking operation. 
 */
USTRUCT(BlueprintType, Blueprintable)
struct FCustomizableObjectInstanceBakeOutput
{
	GENERATED_BODY()

	/**
	 * Success state for the baking operation. True for success and false for failure.
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	bool bWasBakeSuccessful = false;

	/**
	 * Collection of FBakedResourceData representing all saved packages during the baking operation. It may be empty if the operation failed.
	 */
	UPROPERTY(BlueprintReadOnly, Category = CustomizableObjectInstanceBaker)
	TArray<FBakedResourceData> SavedPackages;
};


DECLARE_DYNAMIC_DELEGATE_OneParam(FBakeOperationCompletedDelegate, const FCustomizableObjectInstanceBakeOutput, BakeOperationOutput);


/**
 * Configuration data structure designed to serve as variable container for the customizable object instance baking methods.
 */
USTRUCT(BlueprintType, Blueprintable)
struct FBakingConfiguration
{
	GENERATED_BODY()

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FBakingConfiguration() = default;
	FBakingConfiguration(const FBakingConfiguration&) = default;
	FBakingConfiguration(FBakingConfiguration&&) = default;
	FBakingConfiguration& operator=(const FBakingConfiguration&) = default;
	FBakingConfiguration& operator=(FBakingConfiguration&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * The path where to save the baked resources. EX /Game/MyBakingTest
	 */
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = CustomizableObjectInstanceBaker)
	FString OutputPath = TEXT("/Game");
	
	/**
	 * The name to be used as base (prefix) during the naming of the exported resources
	 */
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = CustomizableObjectInstanceBaker)
	FString OutputFilesBaseName;

	/**
	 * Determines if we want a full or partial export
	 */
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = CustomizableObjectInstanceBaker)
	bool bExportAllResourcesOnBake = false;
	
	/**
	 *  Determines if we want (or not) to generate constant material instances for each of the material instances found in the mutable instance
	 */
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = CustomizableObjectInstanceBaker)
	bool bGenerateConstantMaterialInstancesOnBake = false;
	
	/**
	 *  If true, invalid characters will be removed from exported resource names
	 */
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = CustomizableObjectInstanceBaker)
	bool bReplaceRestrictedCharacters = false;

	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString SkeletalMeshAssetPrefix = TEXT("SK_");

	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString SkeletonAssetPrefix = TEXT("SKEL_");

	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString PhysicsAssetPrefix = TEXT("PHYS_");
	
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString TextureAssetPrefix = TEXT("T_");

	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString MaterialAssetPrefix = TEXT("M_");

	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString MaterialInstanceAssetPrefix = TEXT("MI_");
	
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString MaterialDynamicInstanceAssetPrefix = TEXT("MID_");

	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = "CustomizableObjectInstanceBaker|ResourceNameGeneration")
	FString MaterialConstantInstanceAssetPrefix = TEXT("MIC_");
	
	/**
	 * Callback executed once the baking operation gets completed. It will return the end success state and also some data about the assets saved.
	 */
	UPROPERTY(BlueprintReadWrite, Blueprintable, Category = CustomizableObjectInstanceBaker)
	FBakeOperationCompletedDelegate OnBakeOperationCompletedCallback;

	/**
	 * The contents of the prefix used to mark resources generated for a bake operation
	 */
	const static inline FString BakedResourcePrefix = TEXT("BAKE_");
};


/** Instance Update Result. */
UENUM(BlueprintType)
enum class EUpdateResult : uint8
{
	Success, // Update finished without issues.
	Warning, // Generic warning. Update finished but with warnings.
	
	Error, // Generic error.
	ErrorOptimized, // The update was skipped since its result would have been the same as the current customization.
	ErrorReplaced, // The update was replaced by a newer update request.
	ErrorDiscarded, // The update was not finished since due to the LOD management discarding the data.
	Error16BitBoneIndex, // The update finish unsuccessfully due to Instance not supporting 16 Bit Bone Indexing required by the Engine.

	ErrorUncompiled // The update failed due to the CO being uncompiled
};


/** Instance Update Context.
 * Used to avoid changing the delegate signature in the future.  */
USTRUCT(BlueprintType)
struct FUpdateContext
{	
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = UpdateResult)
	EUpdateResult UpdateResult = EUpdateResult::Success;

	UPROPERTY(BlueprintReadOnly, Category = UpdateResult)
	TObjectPtr<UCustomizableObjectInstance> Instance;
};


DECLARE_DYNAMIC_DELEGATE_OneParam(FInstanceUpdateDelegate, const FUpdateContext&, Result);
DECLARE_MULTICAST_DELEGATE_OneParam(FInstanceUpdateNativeDelegate, const FUpdateContext&);

/* When creating new delegates use the following conventions:
 *
 * - All delegates must be multicast.
 * - If the delegate is exposed to the API create both, dynamic and native versions (non-dynamic).
 * - Dynamic delegates should not be transient. Use the native version if you do not want it to be saved.
 * - Native delegates names should end with "NativeDelegate".
 * - Dynamic delegates broadcast before native delegates. */

/** Broadcast when an Instance update has completed.
 * Notice that Mutable internally can also start an Instance update. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FObjectInstanceUpdatedDelegate, UCustomizableObjectInstance*, Instance);
DECLARE_MULTICAST_DELEGATE_OneParam(FObjectInstanceUpdatedNativeDelegate, UCustomizableObjectInstance*);

USTRUCT()
struct FPreSetSkeletalMeshParams
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UCustomizableObjectInstance> Instance;
	
	UPROPERTY()
	TObjectPtr<USkeletalMesh> SkeletalMesh;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPreSetSkeletalMeshDelegate, const FPreSetSkeletalMeshParams&, Params);
DECLARE_MULTICAST_DELEGATE_OneParam(FPreSetSkeletalMeshNativeDelegate, const FPreSetSkeletalMeshParams&);

DECLARE_DELEGATE_OneParam(FProjectorStateChangedDelegate, FString);

DECLARE_DYNAMIC_DELEGATE_TwoParams(FEachComponentAnimInstanceClassDelegate, FName, SlotIndex, TSubclassOf<UAnimInstance>, AnimInstClass);

DECLARE_DELEGATE_TwoParams(FEachComponentAnimInstanceClassNativeDelegate, FName /*SlotIndex*/, TSubclassOf<UAnimInstance> /*AnimInstClass*/);

UCLASS(MinimalAPI,  BlueprintType, HideCategories=(CustomizableObjectInstance) )
class UCustomizableObjectInstance : public UObject
{
	GENERATED_BODY()

	// Friends
	friend UCustomizableInstancePrivate;
	friend FMutableUpdateCandidate;

public:
	CUSTOMIZABLEOBJECT_API UCustomizableObjectInstance();
	
	/** Broadcast when the Customizable Object Instance is updated. */
	UPROPERTY(Transient, BlueprintAssignable, Category = CustomizableObjectInstance)
	FObjectInstanceUpdatedDelegate UpdatedDelegate;

	/** Broadcast when the Customizable Object Instance is updated. */
	FObjectInstanceUpdatedNativeDelegate UpdatedNativeDelegate;

	/** Broadcast before setting the generated Skeletal Mesh to the Skeletal Mesh Components.
	  * Broadcast even if Skeletal Meshes are reused. */
	FPreSetSkeletalMeshDelegate PreSetSkeletalMeshDelegate;

	/** Broadcast before setting the generated Skeletal Mesh to the Skeletal Mesh Components.
	  * Broadcast even if Skeletal Meshes are reused. */
	FPreSetSkeletalMeshNativeDelegate PreSetSkeletalMeshNativeDelegate;
	
	// UObject interface.
#if WITH_EDITOR
	CUSTOMIZABLEOBJECT_API virtual bool CanEditChange( const FProperty* InProperty ) const override;
	CUSTOMIZABLEOBJECT_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif //WITH_EDITOR

	CUSTOMIZABLEOBJECT_API virtual void Serialize(FArchive& Ar) override;
	CUSTOMIZABLEOBJECT_API virtual void PostLoad() override;
	CUSTOMIZABLEOBJECT_API virtual void BeginDestroy() override;
	CUSTOMIZABLEOBJECT_API virtual FString GetDesc() override;
	CUSTOMIZABLEOBJECT_API virtual bool IsEditorOnly() const override;
	CUSTOMIZABLEOBJECT_API virtual void PostInitProperties() override;

	/** Set the CustomizableObject this instance will be generated from. 
	  * It is usually not necessary to call this since instances are already generated from a CustomizableObject. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetObject(UCustomizableObject* InObject);

	/** Get the CustomizableObject that this is an instance of. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API UCustomizableObject* GetCustomizableObject() const;

	/** Return true if the parameter relevancy will be updated when this instance is generated. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool GetBuildParameterRelevancy() const;

	/** Set the flag that controls if parameter relevancy will be updated when this instance is generated. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetBuildParameterRelevancy(bool Value);

	/** Return the name of the current CustomizableObject state this is instance is set to. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FString GetCurrentState() const;

	/** Set the CustomizableObject state that this instance will be generated into. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetCurrentState(const FString& StateName);
	
	// DEPRECATED: Use GetSkeletalMeshComponentSkeletalMesh 
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API USkeletalMesh* GetComponentMeshSkeletalMesh(const FName& ComponentName) const;

	/** Given a Mesh Component name, return its generated Skeletal Mesh. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API USkeletalMesh* GetSkeletalMeshComponentSkeletalMesh(const FName& ComponentName) const;
	
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API TArray<UMaterialInterface*> GetSkeletalMeshComponentOverrideMaterials(const FName& ComponentName) const;
	
	/** Return true if a skeletal mesh has been generated for any component of this instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool HasAnySkeletalMesh() const;
	
	/** Return true if the instance has any parameters. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool HasAnyParameters() const;
	
	/** Set random values to the parameters. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetRandomValues();

	/** Set random values to the parameters using a stream. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetRandomValuesFromStream(const FRandomStream& InStream);

	/** Sets a parameter to its default value */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetDefaultValue(const FString& ParamName);

	/** Set all parameters to their default value. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetDefaultValues();
	
	/** Returns the AssetUserData that was gathered from all the constituent mesh parts during the last update. 
	  * It requires that the CustomizableObject had the bEnableAssetUserDataMerge set to true during compilation. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API TSet<UAssetUserData*> GetMergedAssetUserData(const FName& ComponentName) const;
	
	/** Generate the instance with the current parameters and update all the components Skeletal Meshes asynchronously. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void UpdateSkeletalMeshAsync(bool bIgnoreCloseDist = false, bool bForceHighPriority = false);
		
	/** Generate the instance with the current parameters and update all the components Skeletal Meshes asynchronously.
	  * Callback will be called once the update finishes, even if it fails. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void UpdateSkeletalMeshAsyncResult(FInstanceUpdateDelegate Callback, bool bIgnoreCloseDist = false, bool bForceHighPriority = false);

	CUSTOMIZABLEOBJECT_API void UpdateSkeletalMeshAsyncResult(FInstanceUpdateNativeDelegate Callback, bool bIgnoreCloseDist = false, bool bForceHighPriority = false);

	/** Clones the instance creating a new identical transient instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API UCustomizableObjectInstance* Clone();

	/** Clones the instance creating a new identical static instance with the given Outer. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API UCustomizableObjectInstance* CloneStatic(UObject* Outer);

	/** Copy parameters from the given Instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void CopyParametersFromInstance(UCustomizableObjectInstance* Instance);

	/** Destroy the Mutable Core Live Update Instance attached to this (if exists). */
	CUSTOMIZABLEOBJECT_API void DestroyLiveUpdateInstance();
	
	/** Return true if the given parameter has any effect in the current object state, and considering the current values of the other parameters. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool IsParameterRelevant(const FString& ParamName) const;

	/** Return true if the parameter has changed but the Instance has not yet been updated. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool IsParameterDirty(const FString& ParamName, int32 RangeIndex = -1) const;
	
	/** For multidimensional parameters, return the number of dimensions that the given projector parameter supports. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 GetProjectorValueRange(const FString& ParamName) const;

	/** For multidimensional parameters, return the number of dimensions that the given int parameter supports. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 GetIntValueRange(const FString& ParamName) const;

	/** For multidimensional parameters, return the number of dimensions that the given float parameter supports. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 GetFloatValueRange(const FString& ParamName) const;

	/** For multidimensional parameters, return the number of dimensions that the given texture parameter supports. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 GetTextureValueRange(const FString& ParamName) const;

	// DEPRECATED. Use GetEnumParameterSelectedOption instead.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API const FString& GetIntParameterSelectedOption(const FString& ParamName, int32 RangeIndex = -1) const;

	/** Return the name of the option currently set in the given parameter. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API const FString& GetEnumParameterSelectedOption(const FString& ParamName, int32 RangeIndex = -1) const;
	
	// DEPRECATED. Use SetEnumParameterSelectedOption instead.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetIntParameterSelectedOption(const FString& ParamName, const FString& SelectedOptionName, int32 RangeIndex = -1);

	/** Set the currently selected option value for the given parameter, by parameter name and option name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetEnumParameterSelectedOption(const FString& ParamName, const FString& SelectedOptionName, int32 RangeIndex = -1);
	
	/** Gets the value of a float parameter with name "FloatParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API float GetFloatParameterSelectedOption(const FString& FloatParamName, int32 RangeIndex = -1) const;

	/** Sets the float value "FloatValue" of a float parameter with index "FloatParamIndex". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetFloatParameterSelectedOption(const FString& FloatParamName, float FloatValue, int32 RangeIndex = -1);
	
	/** Gets the value of a texture parameter with name "TextureParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API UTexture* GetTextureParameterSelectedOption(const FString& TextureParamName, int32 RangeIndex = -1) const;

	/** Sets the texture value "TextureValue" of a texture parameter with index "TextureParamIndex". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetTextureParameterSelectedOption(const FString& TextureParamName, UTexture* TextureValue, int32 RangeIndex = -1);

	/** Gets the value of a Skeletal Mesh parameter with name "SkeletalMeshParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API USkeletalMesh* GetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, int32 RangeIndex = -1);

	/** Sets the Skeletal Mesh value "SkeletalMeshValue" of a Skeletal Mesh parameter with index "SkeletalMeshParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, USkeletalMesh* SkeletalMeshValue, int32 RangeIndex = -1);

	/** Gets the value of a Material parameter with name "MaterialParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API UMaterialInterface* GetMaterialParameterSelectedOption(const FString& MaterialParamName, int32 RangeIndex = -1);

	/** Sets the Material value "MaterialParamName" of a Material parameter with index "MaterialParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetMaterialParameterSelectedOption(const FString& MaterialParamName, UMaterialInterface* MaterialValue, int32 RangeIndex = -1);
	
	/** Gets the value of an External Type parameter with name "ExternalTypeParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FInstancedStruct GetExternalTypeParameterSelectedOption(const FString& ExternalTypeParamName,  int32 RangeIndex = -1) const;

	/** Sets the External Type value "ExternalTypeValue" of an External Type parameter with name "ExternalTypeParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetExternalTypeParameterSelectedOption(const FString& ExternalTypeParamName, const FInstancedStruct& ExternalTypeValue, int32 RangeIndex = -1);

	/** Gets the value of a color parameter with name "ColorParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FLinearColor GetColorParameterSelectedOption(const FString& ColorParamName) const;

	/** Sets the color value "ColorValue" of a color parameter with index "ColorParamIndex". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetColorParameterSelectedOption(const FString& ColorParamName, const FLinearColor& ColorValue);

	/** Gets the bool value of a bool parameter with name "BoolParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool GetBoolParameterSelectedOption(const FString& BoolParamName) const;

	/** Sets the bool value "BoolValue" of a bool parameter with name "BoolParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetBoolParameterSelectedOption(const FString& BoolParamName, bool BoolValue);

	/** Gets the Vector value of a Vector parameter with name "VectorParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FVector4f GetVectorParameterSelectedOption(const FString& VectorParamName);
	
	/** Sets the Vector value "VectorValue" of a Vector parameter with name "VectorParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetVectorParameterSelectedOption(const FString& VectorParamName, const FVector4f& VectorValue);

	/** Gets the value of a transform parameter with name "TransformParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FTransform GetTransformParameterSelectedOption(const FString& TransformParamName) const;

	/** Sets the transform value "TransformValue" of a transform parameter with name "TransformParamName". */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetTransformParameterSelectedOption(const FString& TransformParamName, const FTransform& TransformValue);
	
	/** Sets the projector values of a projector parameter with index "ProjectorParamIndex". */
	UE_DEPRECATED(5.8, "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void SetProjectorValue(const FString& ProjectorParamName,
		const FVector& OutPos, const FVector& OutDirection, const FVector& OutUp, const FVector& OutScale,
		float OutAngle,
		int32 RangeIndex = -1);

	/** Set only the projector position keeping the rest of values. */
	UE_DEPRECATED(5.8, "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void SetProjectorPosition(const FString& ProjectorParamName, const FVector& Pos, int32 RangeIndex = -1);
	
	/** Set only the projector direction vector keeping the rest of values. */
	UE_DEPRECATED(5.8, "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void SetProjectorDirection(const FString& ProjectorParamName, const FVector& Direction, int32 RangeIndex = -1);
	
	/** Set only the projector up vector keeping the rest of values. */
	UE_DEPRECATED(5.8, "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void SetProjectorUp(const FString& ProjectorParamName, const FVector& Up, int32 RangeIndex = -1);

	/** Set only the projector scale keeping the rest of values. */
	UE_DEPRECATED(5.8, "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void SetProjectorScale(const FString& ProjectorParamName, const FVector& Scale, int32 RangeIndex = -1);

	/** Set only the cylindrical projector angle keeping the rest of values. */
	UE_DEPRECATED(5.8, "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'SetProjectorParameterSelectedOption' method that takes a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void SetProjectorAngle(const FString& ProjectorParamName, float Angle, int32 RangeIndex = -1);

	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetProjectorParameterSelectedOption(const FString& ProjectorParamName, const FCustomizableObjectProjector& ProjectorValue, const int32 RangeIndex = -1);
	
	/** Get the projector values of a projector parameter with index "ProjectorParamIndex". */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API void GetProjectorValue(const FString& ProjectorParamName,
		FVector& OutPos, FVector& OutDirection, FVector& OutUp, FVector& OutScale,
		float& OutAngle, ECustomizableObjectProjectorType& OutType,
		int32 RangeIndex = -1) const;

	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	CUSTOMIZABLEOBJECT_API void GetProjectorValueF(const FString& ProjectorParamName,
		FVector3f& Pos, FVector3f& Direction, FVector3f& Up, FVector3f& Scale,
		float& Angle, ECustomizableObjectProjectorType& Type,
		int32 RangeIndex = -1) const;

	/** Get the current projector position for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API FVector GetProjectorPosition(const FString & ParamName, int32 RangeIndex = -1) const;

	/** Get the current projector direction vector for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API FVector GetProjectorDirection(const FString & ParamName, int32 RangeIndex = -1) const;

	/** Get the current projector up vector for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API FVector GetProjectorUp(const FString & ParamName, int32 RangeIndex = -1) const;

	/** Get the current projector scale for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API FVector GetProjectorScale(const FString & ParamName, int32 RangeIndex = -1) const;

	/** Get the current cylindrical projector angle for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API float GetProjectorAngle(const FString& ParamName, int32 RangeIndex = -1) const;

	/** Get the current projector type for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance, meta = (DeprecatedFunction, DeprecationMessage = "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead."))
	CUSTOMIZABLEOBJECT_API ECustomizableObjectProjectorType GetProjectorParameterType(const FString& ParamName, int32 RangeIndex = -1) const;

	/** Get the current projector for the parameter with the given name. */
	UE_DEPRECATED(5.8, "Use the 'GetProjectorParameterSelectedOption' method that returns a 'FCustomizableObjectProjector' instead.")
	CUSTOMIZABLEOBJECT_API FCustomizableObjectProjector GetProjector(const FString& ParamName, int32 RangeIndex = -1) const;

	/** Get the current projector for the parameter with the given name. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FCustomizableObjectProjector GetProjectorParameterSelectedOption(const FString& ParamName, int32 RangeIndex = -1) const;
	
	// DEPRECATED. Use ContainsEnumParameter instead.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsIntParameter(const FString& ParameterName) const;

	/** Return true if the Int Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsEnumParameter(const FString& ParameterName) const;
	
	/** Return true if the Float Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsFloatParameter(const FString& ParameterName) const;

	/** Return true if the Texture Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsTextureParameter(const FString& ParameterName) const;
	
	/** Return true if the Skeletal Mesh Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsSkeletalMeshParameter(const FString& ParameterName) const;
	
	/** Return true if the Material Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsMaterialParameter(const FString& ParameterName) const;

	/** Return true if the External Type Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsExternalTypeParameter(const FString& ParameterName) const;
	
	/** Return true if the Bool Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsBoolParameter(const FString& ParameterName) const;

	/** Return true if the Color Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsColorParameter(const FString& ParameterName) const;
	
	/** Return true if the Vector Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsVectorParameter(const FString& ParameterName) const;

	/** Return true if the Projector Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsProjectorParameter(const FString& ParameterName) const;

	/** Return true if the Transform Parameter exists. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API bool ContainsTransformParameter(const FString& ParameterName) const;

	/** Increases the range of values of the integer with ParamName and returns the index of the new integer value, -1 otherwise.
	  * The added value is initialized with the first integer option and is the last one of the range. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 AddValueToIntRange(const FString& ParamName);

	/** Increases the range of values of the float with ParamName, returns the index of the new float value, -1 otherwise.
	  * The added value is initialized with 0.5f and is the last one of the range. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 AddValueToFloatRange(const FString& ParamName);

	/** Increases the range of values of the projector with ParamName, returns the index of the new projector value, -1 otherwise.
	  * The added value is initialized with the default projector as set up in the editor and is the last one of the range. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 AddValueToProjectorRange(const FString& ParamName);

	/** Remove the FRangeIndex element of the integer range of values from the parameter ParamName. If RangeValue is -1 removes the last of the integer range of values.
		Returns the index of the last valid integer, -1 if no values left. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 RemoveValueFromIntRange(const FString& ParamName, int32 RangeIndex = -1);

	/** Remove the FRangeIndex element of the float range of values from the parameter ParamName. If RangeValue is -1 removes the last of the float range of values.
		Returns the index of the last valid float, -1 if no values left. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 RemoveValueFromFloatRange(const FString& ParamName, int32 RangeIndex = -1);

	/** Remove the FRangeIndex element of the projector range of values from the parameter ParamName. If RangeValue is -1 removes the last of the projector range of values.
		Returns the index of the last valid projector, -1 if no values left. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 RemoveValueFromProjectorRange(const FString& ParamName, int32 RangeIndex = -1);

	// ------------------------------------------------------------
	// Multilayer Projectors
	// ------------------------------------------------------------
	
	// Layers

	/** See FMultilayerProjector::NumLayers. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API int32 MultilayerProjectorNumLayers(const FName& ProjectorParamName) const;

	/** See FMultilayerProjector::CreateLayer. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void MultilayerProjectorCreateLayer(const FName& ProjectorParamName, int32 Index);

	/** See FMultilayerProjector::RemoveLayerAt. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void MultilayerProjectorRemoveLayerAt(const FName& ProjectorParamName, int32 Index);

	/** See FMultilayerProjector::GetLayer. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FMultilayerProjectorLayer MultilayerProjectorGetLayer(const FName& ProjectorParamName, int32 Index) const;

	/** See FMultilayerProjector::UpdateLayer. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void MultilayerProjectorUpdateLayer(const FName& ProjectorParamName, int32 Index, const FMultilayerProjectorLayer& Layer);
	
	// ------------------------------------------------------------

	/** Return the list of names of components generated for this instance. 
	* This only has values when the instance has been completely generated. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API TArray<FName> GetComponentNames() const;

	// ------------------------------------------------------------

	/** Returns the animation BP for the parameter component and slot, gathered from all the meshes that compose this instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API TSubclassOf<UAnimInstance> GetAnimBP(FName ComponentName, const FName& Slot) const;

	/** Return the list of tags for this instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API FGameplayTagContainer GetAnimationGameplayTags() const;
	
	/** Execute a delegate for each animation instance involved in this customizable object instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void ForEachComponentAnimInstance(FName ComponentName, FEachComponentAnimInstanceClassDelegate Delegate) const;

	CUSTOMIZABLEOBJECT_API void ForEachComponentAnimInstance(FName ComponentName, FEachComponentAnimInstanceClassNativeDelegate Delegate) const;

	/** Check if the given UAnimInstance class requires to be fixed up. */
	CUSTOMIZABLEOBJECT_API bool AnimInstanceNeedsFixup(TSubclassOf<UAnimInstance> AnimInstance) const;

	/** Fix the given UAnimInstance instance. */
	CUSTOMIZABLEOBJECT_API void AnimInstanceFixup(UAnimInstance* AnimInstance) const;
	
	/** See FCustomizableObjectInstanceDescriptor::SaveDescriptor. */
	CUSTOMIZABLEOBJECT_API void SaveDescriptor(FArchive &CustomizableObjectDescriptor, bool bUseCompactDescriptor);

	/** See FCustomizableObjectInstanceDescriptor::LoadDescriptor. */
	CUSTOMIZABLEOBJECT_API void LoadDescriptor(FArchive &CustomizableObjectDescriptor);

	/** Enable physics asset replacement so that generated skeletal meshes have the merged physics assets of their skeletal mesh parts and reference mesh. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetReplacePhysicsAssets(bool bReplaceEnabled);

	/** If true, Mutable will keep the ownership of the generated resources. This means that Mutable may decide to modify or reuse them for performance considerations. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetKeepOwnershipOfGeneratedResources(bool bEnabled);
	
#if WITH_EDITOR
	/**
	 * Performs the baking of the instance resources in an async fashion. Bind yourself to the callback present in InBakingConfiguration to get notified in case it fails
	 * @param InBakingConfiguration The configuration to be using for the baking operation
	 */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceBaking)
	CUSTOMIZABLEOBJECT_API void Bake(const FBakingConfiguration& InBakingConfiguration);
#endif
	
	CUSTOMIZABLEOBJECT_API UCustomizableInstancePrivate* GetPrivate() const;

private:
	UPROPERTY()
	FCustomizableObjectInstanceDescriptor Descriptor;

	UPROPERTY(Instanced)
	TObjectPtr<UCustomizableInstancePrivate> PrivateData;

	// Deprecated properties	
	UPROPERTY()
	TObjectPtr<UCustomizableObject> CustomizableObject_DEPRECATED;
	
	UPROPERTY()
	TArray<FCustomizableObjectBoolParameterValue> BoolParameters_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectIntParameterValue> IntParameters_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectFloatParameterValue> FloatParameters_DEPRECATED;

	UPROPERTY()																																						
	TArray<FCustomizableObjectTextureParameterValue> TextureParameters_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectVectorParameterValue> VectorParameters_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectProjectorParameterValue> ProjectorParameters_DEPRECATED;
	
	bool bBuildParameterRelevancy_DEPRECATED = false;
};

