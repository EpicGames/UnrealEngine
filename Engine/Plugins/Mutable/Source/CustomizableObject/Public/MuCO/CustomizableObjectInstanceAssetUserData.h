// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "GameplayTagContainer.h"
#include "UObject/SoftObjectPtr.h"

#include "CustomizableObjectInstanceAssetUserData.generated.h"

class UAnimInstance;


USTRUCT(BlueprintType)
struct FCustomizableObjectAnimationSlot
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = CustomizableObjectInstance)
	FName Name;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = CustomizableObjectInstance)
	TSoftClassPtr<UAnimInstance> AnimInstance;
};


/** Additional data attached to Skeletal Meshes. */
UCLASS(MinimalAPI, BlueprintType)
class UCustomizableObjectInstanceUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	/** Return the list of tags for this instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API const FGameplayTagContainer& GetAnimationGameplayTags() const;

	/** Sets the list of tags for this instance. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	CUSTOMIZABLEOBJECT_API void SetAnimationGameplayTags(const FGameplayTagContainer& InstanceTags);

	CUSTOMIZABLEOBJECT_API virtual bool Merge(const TNotNull<UAssetUserData*> Other) override;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = CustomizableObjectInstance)
	FGameplayTagContainer AnimationGameplayTag;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = CustomizableObjectInstance)
	TArray<FCustomizableObjectAnimationSlot> AnimationSlots;
};
