// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/MultilayerProjector.h"
#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"
#include "Math/RandomStream.h"

#include "CustomizableObjectInstanceDescriptor.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UTexture;
enum class ECustomizableObjectProjectorType : uint8;

class FArchive;
class UCustomizableInstancePrivate;
class UCustomizableObject;
class UCustomizableObjectInstance;
class UMaterialInterface;
class FDescriptorHash;
class FMutableUpdateCandidate;

typedef TMap<const UCustomizableObjectInstance*, FMutableUpdateCandidate> FMutableInstanceUpdateMap;

namespace UE::Mutable::Private
{
	class FParameters;
	
	template<typename Type>
	class Ptr;
}


/** Internal use only! Currently in /Public due to UCustomizableObjectInstance::Descriptor.
 *
 * Set of parameters + state that defines a CustomizableObjectInstance.
 *
 * This object has the same parameters + state interface as UCustomizableObjectInstance.
 * UCustomizableObjectInstance must share the same interface. Any public methods added here should also end up in the Instance. */
USTRUCT()
struct FCustomizableObjectInstanceDescriptor
{
	GENERATED_BODY()

	FCustomizableObjectInstanceDescriptor() = default;
	
	UE_API explicit FCustomizableObjectInstanceDescriptor(UCustomizableObject& Object);

	/** Serialize this object. 
	 *
	 * Backwards compatibility is not guaranteed.
 	 * Multilayer Projectors not supported.
	 *
  	 * @param bUseCompactDescriptor If true it assumes the compiled objects are the same on both ends of the serialisation */
	UE_API void SaveDescriptor(FArchive &Ar, bool bUseCompactDescriptor);

	/** Deserialize this object. Does not support Multilayer Projectors! */

	/** Deserialize this object.
     *
	 * Backwards compatibility is not guaranteed.
	 * Multilayer Projectors not supported */
	UE_API void LoadDescriptor(FArchive &Ar);

	// Could return nullptr in some rare situations, so check first
	UE_API UCustomizableObject* GetCustomizableObject() const;

	UE_API void SetCustomizableObject(UCustomizableObject* InCustomizableObject);
	
	UE_API bool GetBuildParameterRelevancy() const;
	
	UE_API void SetBuildParameterRelevancy(bool Value);
	
	/** Update all parameters to be up to date with the Mutable Core parameters. */
	UE_API void ReloadParameters();

	// ------------------------------------------------------------
	// FParameters
	// ------------------------------------------------------------
	
	UE_API bool HasAnyParameters() const;

	UE_API const FString& GetIntParameterSelectedOption(const FString& ParamName, int32 RangeIndex = -1) const;

	UE_API void SetIntParameterSelectedOption(int32 IntParamIndex, const FString& SelectedOption, int32 RangeIndex = -1);

	UE_API void SetIntParameterSelectedOption(const FString& ParamName, const FString& SelectedOptionName, int32 RangeIndex = -1);

	UE_API float GetFloatParameterSelectedOption(const FString& FloatParamName, int32 RangeIndex = -1) const;

	UE_API void SetFloatParameterSelectedOption(const FString& FloatParamName, float FloatValue, int32 RangeIndex = -1);

	UE_API UTexture* GetTextureParameterSelectedOption(const FString& TextureParamName, int32 RangeIndex) const;

	UE_API void SetTextureParameterSelectedOption(const FString& TextureParamName, UTexture* TextureValue, int32 RangeIndex);
	
	UE_API USkeletalMesh* GetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, int32 RangeIndex = -1) const;

	UE_API void SetSkeletalMeshParameterSelectedOption(const FString& SkeletalMeshParamName, USkeletalMesh* SkeletalMeshValue, int32 RangeIndex = -1);

	UE_API UMaterialInterface* GetMaterialParameterSelectedOption(const FString& MaterialMeshParamName, int32 RangeIndex = -1) const;

	UE_API FInstancedStruct GetExternalTypeParameterSelectedOption(const FString& ExternalTypeParamName, int32 RangeIndex = -1) const;
	
	UE_API void SetExternalTypeParameterSelectedOption(const FString& ParamName, const FInstancedStruct& Value, int32 RangeIndex = -1);
	
	UE_API void SetMaterialParameterSelectedOption(const FString& MaterialParamName, UMaterialInterface* MaterialValue, int32 RangeIndex = -1);

	UE_API FTransform GetTransformParameterSelectedOption(const FString& TransformParamName) const;

	UE_API void SetTransformParameterSelectedOption(const FString& TransformParamName, const FTransform& TransformValue);

	UE_API bool GetBoolParameterSelectedOption(const FString& BoolParamName) const;

	UE_API void SetBoolParameterSelectedOption(const FString& BoolParamName, bool BoolValue);

	UE_API FVector4f GetVectorParameterSelectedOption(const FString& BoolParamName) const;
	
	UE_API void SetVectorParameterSelectedOption(const FString& VectorParamName, const FVector4f& VectorValue);
	
	UE_API void SetProjectorValue(const FString& ProjectorParamName,
		const FVector& Pos, const FVector& Direction, const FVector& Up, const FVector& Scale,
		float Angle,
		int32 RangeIndex = -1);

	UE_API void SetProjectorParameterSelectedOption(const FString& ProjectorParamName, const FCustomizableObjectProjector& ProjectorValue, const int32 RangeIndex = -1);
	
	UE_API void SetProjectorPosition(const FString& ProjectorParamName, const FVector& Pos, int32 RangeIndex = -1);

	UE_API void SetProjectorDirection(const FString& ProjectorParamName, const FVector& Direction, int32 RangeIndex = -1);
	
	UE_API void SetProjectorUp(const FString& ProjectorParamName, const FVector& Up, int32 RangeIndex = -1);

	UE_API void SetProjectorScale(const FString& ProjectorParamName, const FVector& Scale, int32 RangeIndex = -1);

	UE_API void SetProjectorAngle(const FString& ProjectorParamName, float Angle, int32 RangeIndex = -1);
	
	UE_API void GetProjectorValue(const FString& ProjectorParamName,
		FVector& OutPos, FVector& OutDirection, FVector& OutUp, FVector& OutScale,
		float& OutAngle, ECustomizableObjectProjectorType& OutType,
		int32 RangeIndex = -1) const;

	UE_API void GetProjectorValueF(const FString& ProjectorParamName,
		FVector3f& OutPos, FVector3f& OutDirection, FVector3f& OutUp, FVector3f& OutScale,
		float& OutAngle, ECustomizableObjectProjectorType& OutType,
		int32 RangeIndex = -1) const;

	UE_API FVector GetProjectorPosition(const FString & ParamName, int32 RangeIndex = -1) const;

	UE_API FVector GetProjectorDirection(const FString & ParamName, int32 RangeIndex = -1) const;

	UE_API FVector GetProjectorUp(const FString & ParamName, int32 RangeIndex = -1) const;

	UE_API FVector GetProjectorScale(const FString & ParamName, int32 RangeIndex = -1) const;

	UE_API float GetProjectorAngle(const FString& ParamName, int32 RangeIndex = -1) const;

	UE_API ECustomizableObjectProjectorType GetProjectorParameterType(const FString& ParamName, int32 RangeIndex = -1) const;

	UE_API FCustomizableObjectProjector GetProjectorParameterSelectedOption(const FString& ParamName, int32 RangeIndex = -1) const;
	
	UE_API int32 FindTypedParameterIndex(const FString& ParamName, EMutableParameterType Type) const;

	UE_API int32 GetProjectorValueRange(const FString& ParamName) const;

	UE_API int32 GetIntValueRange(const FString& ParamName) const;

	UE_API int32 GetFloatValueRange(const FString& ParamName) const;

	UE_API int32 GetTextureValueRange(const FString& ParamName) const;

	UE_API int32 AddValueToIntRange(const FString& ParamName);

	UE_API int32 AddValueToFloatRange(const FString& ParamName);

	UE_API int32 AddValueToTextureRange(const FString& ParamName);

	UE_API int32 AddValueToProjectorRange(const FString& ParamName);

	UE_API int32 RemoveValueFromIntRange(const FString& ParamName, int32 RangeIndex = -1);

	UE_API int32 RemoveValueFromFloatRange(const FString& ParamName, int32 RangeIndex = -1);

	UE_API int32 RemoveValueFromTextureRange(const FString& ParamName);

	UE_API int32 RemoveValueFromTextureRange(const FString& ParamName, int32 RangeIndex);
	
	UE_API int32 RemoveValueFromProjectorRange(const FString& ParamName, int32 RangeIndex = -1);

	UE_API int32 GetState() const;

	UE_API FString GetCurrentState() const;

	UE_API void SetState(int32 InState);

	UE_API void SetCurrentState(const FString& StateName);

	UE_API void SetRandomValues();
	
	UE_API void SetRandomValuesFromStream(const FRandomStream& InStream);

	UE_API void SetDefaultValue(int32 ParamIndex);

	UE_API void SetDefaultValues();
	
	UE_API bool IsMultilayerProjector(const FString& ParamName) const;

	UE_API int32 NumProjectorLayers(const FName& ParamName) const;

	UE_API void CreateLayer(const FName& ParamName, int32 Index);

	UE_API void RemoveLayerAt(const FName& ParamName, int32 Index);

	UE_API FMultilayerProjectorLayer GetLayer(const FName& ParamName, int32 Index) const;

	UE_API void UpdateLayer(const FName& ParamName, int32 Index, const FMultilayerProjectorLayer& Layer);

	/** Return a Mutable Core object containing all parameters. */
	UE_API TSharedPtr<UE::Mutable::Private::FParameters> GetParameters() const;

	UE_API FString ToString() const;
	
	UPROPERTY()
	TObjectPtr<UCustomizableObject> CustomizableObject = nullptr;

	UPROPERTY()
	TArray<FCustomizableObjectBoolParameterValue> BoolParameters;

	UPROPERTY()
	TArray<FCustomizableObjectIntParameterValue> IntParameters;
	
	UPROPERTY()
	TArray<FCustomizableObjectFloatParameterValue> FloatParameters;
	
	UPROPERTY()
	TArray<FCustomizableObjectTextureParameterValue> TextureParameters;

	UPROPERTY()
	TArray<FCustomizableObjectSkeletalMeshParameterValue> SkeletalMeshParameters;

	UPROPERTY()
	TArray<FCustomizableObjectMaterialParameterValue> MaterialParameters;
	
	UPROPERTY()
	TArray<FCustomizableObjectVectorParameterValue> VectorParameters;
	
	UPROPERTY()
	TArray<FCustomizableObjectProjectorParameterValue> ProjectorParameters;

	UPROPERTY()
	TArray<FCustomizableObjectTransformParameterValue> TransformParameters;

	UPROPERTY()
	TArray<FCustomizableObjectExternalTypeParameterValue> ExternalTypeParameters;

	UPROPERTY()
	int32 State = 0;
	
	/** If this is set to true, when updating the instance an additional step will be performed to calculate the list of instance parameters that are relevant for the current parameter values. */
	bool bBuildParameterRelevancy = false;

	// Friends
	friend FDescriptorHash;
	friend UCustomizableObjectInstance;
	friend UCustomizableInstancePrivate;
	friend FMultilayerProjector;
	friend FMutableUpdateCandidate;
};

#undef UE_API
