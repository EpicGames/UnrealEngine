// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "MuCO/CustomizableObjectInstance.h"

#include "CustomizableObjectInstanceUsage.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UObject;
class UCustomizableObjectInstanceUsagePrivate;

DECLARE_DELEGATE(FCustomizableObjectInstanceUsageUpdatedDelegate);

// This class can be used instead of a UCustomizableComponent (for example for non-BP projects) to link a 
// UCustomizableObjectInstance and a USkeletalComponent so that the CustomizableObjectSystem takes care of updating it and its LODs, 
// streaming, etc. It's a UObject, so it will be much cheaper than a UCustomizableComponent as it won't have to refresh its transforms
// every time it's moved.
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (CustomizableObject))
class UCustomizableObjectInstanceUsage : public UObject
{
	friend UCustomizableObjectInstanceUsagePrivate;
	
public:
	GENERATED_BODY()

	// Own interface
	UE_API UCustomizableObjectInstanceUsage();

	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API void SetCustomizableObjectInstance(UCustomizableObjectInstance* CustomizableObjectInstance);

	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API UCustomizableObjectInstance* GetCustomizableObjectInstance() const;

	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API void SetComponentName(const FName& Name);

	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API FName GetComponentName() const;

	/** Attach this Customizable Object Instance Usage to a Skeletal Mesh Component to be customized. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API void AttachTo(USkeletalMeshComponent* SkeletalMeshComponent);
	
	/** Get the parent Skeletal Mesh Component this Customizable Object Instance Usage is attached to. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API USkeletalMeshComponent* GetAttachParent() const;
	
	/** Update Skeletal Mesh asynchronously. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObject)
	UE_API void UpdateSkeletalMeshAsync(bool bNeverSkipUpdate_DEPRECATED = false, bool bIgnoreCloseDist_DEPRECATED = false, bool bForceHighPriority = false);

	/** Update Skeletal Mesh asynchronously. Callback will be called once the update finishes, even if it fails. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstance)
	UE_API void UpdateSkeletalMeshAsyncResult(FInstanceUpdateDelegate Callback, bool bIgnoreCloseDist = false, bool bForceHighPriority = false);
	
	/** Set to true to avoid automatically replacing the Skeletal Mesh of the parent Skeletal Mesh Component by the Reference Skeletal Mesh.
	 * If SkipSetSkeletalMeshOnAttach is true, it will not replace it. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API void SetSkipSetReferenceSkeletalMesh(bool bSkip);

	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API bool GetSkipSetReferenceSkeletalMesh() const;

	/** Set to true to avoid automatically replacing the Skeletal Mesh of the parent Skeletal Mesh Component with any mesh. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API void SetSkipSetSkeletalMeshOnAttach(bool bSkip);
	
	UE_API UCustomizableObjectInstanceUsagePrivate* GetPrivate();

	UE_API const UCustomizableObjectInstanceUsagePrivate* GetPrivate() const;

	FCustomizableObjectInstanceUsageUpdatedDelegate UpdatedDelegate;

private:
	// If the outer is no CustomizableSkeletalComponent, this SkeletalComponent will be used
	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshComponent> UsedSkeletalMeshComponent;

	// If the outer is no CustomizableSkeletalComponentd, this Instance will be used
	UPROPERTY(EditAnywhere, Category = CustomizableObjectInstanceUsage, meta = (DisplayName = "Customizable Object Instance"))
	TObjectPtr<UCustomizableObjectInstance> UsedCustomizableObjectInstance;

	UPROPERTY(EditAnywhere, Category = CustomizableObjectInstanceUsage, meta = (DisplayName = "Component Name"))
	FName UsedComponentName;

	// Used to avoid replacing the SkeletalMesh of the parent component by the ReferenceSkeletalMesh if bPendingSetSkeletalMesh is true
	UPROPERTY(EditAnywhere, Category = CustomizableObjectInstanceUsage, meta = (DisplayName = "Skip Set Reference Skeletal Mesh"))
	bool bUsedSkipSetReferenceSkeletalMesh = false;

	UPROPERTY(EditAnywhere, Category = CustomizableObjectInstanceUsage, meta = (DisplayName = "Skip Set Skeletal Mesh On Attach"))
	bool bUsedSkipSetSkeletalMeshOnAttach = false;
	
	UPROPERTY(Instanced)
	TObjectPtr<UCustomizableObjectInstanceUsagePrivate> Private;
	
#if WITH_EDITORONLY_DATA
	TWeakObjectPtr<UCustomizableObjectInstance> OldCustomizableObjectInstance;
#endif

	friend UCustomizableObjectInstanceUsagePrivate;
};

#undef UE_API
