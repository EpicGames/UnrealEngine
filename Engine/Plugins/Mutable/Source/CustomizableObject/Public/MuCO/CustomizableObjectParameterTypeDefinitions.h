// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"

#include "CustomizableObjectParameterTypeDefinitions.generated.h"

class UTexture;
class USkeletalMesh;
class UMaterialInterface;


UENUM(BlueprintType)
enum class EMutableParameterType : uint8
{
	None			UMETA(DisplayName = "Unknown"),
	Bool			UMETA(DisplayName = "Boolean"),
	Int				UMETA(DisplayName = "Integer"),
	Float			UMETA(DisplayName = "Float"),
	Color			UMETA(DisplayName = "Color"),
	Projector		UMETA(DisplayName = "Projector"),
	Texture			UMETA(DisplayName = "Texture"),
	Transform		UMETA(DisplayName = "Transform"),
	SkeletalMesh	UMETA(DisplayName = "SkeletalMesh"),
	Material		UMETA(DisplayName = "Material"),
	InstancedStruct UMETA(DisplayName = "InstancedStruct"),
};


UENUM(BlueprintType)
enum class ECustomizableObjectGroupType : uint8
{
	COGT_TOGGLE			UMETA(DisplayName = "Toggle"),
	COGT_ALL			UMETA(DisplayName = "All"),
	COGT_ONE			UMETA(DisplayName = "One"),
	COGT_ONE_OR_NONE    UMETA(DisplayName = "One or None")
};


/** Customizable object mesh compilation options */
UENUM(BlueprintType)
enum class EMutableCompileMeshType : uint8
{
	Full                     UMETA(DisplayName = "Full object"),                  // Compile this CO, and add all COs in the whole hierarchy
	Local                    UMETA(DisplayName = "Just local object"),            // Compile this CO and add all parents until whole graph root
	LocalAndChildren         UMETA(DisplayName = "Local object and children"),    // Compile this CO and add all children and parents until whole graph root
	AddWorkingSetNoChildren  UMETA(DisplayName = "Add working set, no children"), // Add to the compilation all COs in the Working Set array and all parents of this object and all parents of each element in the Working Set array until whole graph root, don't include this CO's children
	AddWorkingSetAndChildren UMETA(DisplayName = "Add working set and children")  // Add to the compilation all COs in the Working Set array and all parents of this object and all parents of each element in the Working Set array until whole graph root, include this CO's children
};


USTRUCT(BlueprintType)
struct FCustomizableObjectBoolParameterValue
{
	GENERATED_USTRUCT_BODY()

	static constexpr bool DEFAULT_PARAMETER_VALUE = false;

	inline static const FString DEFAULT_PARAMETER_VALUE_NAME;
	
	UPROPERTY(Category = CustomizableObjectBoolParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectBoolParameterValue, VisibleAnywhere)
	bool ParameterValue = true;

	UPROPERTY(Category = CustomizableObjectBoolParameterValue, VisibleAnywhere)
	FGuid Id;
};


inline uint32 GetTypeHash(const FCustomizableObjectBoolParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	return Hash;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectIntParameterValue
{
	GENERATED_USTRUCT_BODY()

	static constexpr int32 DEFAULT_PARAMETER_VALUE = INDEX_NONE;
	
	UPROPERTY(Category = CustomizableObjectIntParameterValue, VisibleAnywhere)
	FString ParameterName;

	// This is what we actually want to store.
	UPROPERTY(Category = CustomizableObjectIntParameterValue, VisibleAnywhere)
	FString ParameterValueName;

	UPROPERTY(Category = CustomizableObjectIntParameterValue, VisibleAnywhere)
	FGuid Id;

	// Same as ParameterValueName but for multidimensional params
	UPROPERTY()
	TArray<FString> ParameterRangeValueNames;

	FCustomizableObjectIntParameterValue()
	{
	}

	FCustomizableObjectIntParameterValue(const FString & InParamName, const FString & InParameterValueName, const FGuid & InUid, const TArray<FString>& InParameterRangeValueNames)
		: ParameterName(InParamName), ParameterValueName(InParameterValueName), Id(InUid)
	{
		ParameterRangeValueNames = InParameterRangeValueNames;
	}
};


inline uint32 GetTypeHash(const FCustomizableObjectIntParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValueName));

	for (const FString& Value : Key.ParameterRangeValueNames)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	return Hash;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectFloatParameterValue
{
	GENERATED_USTRUCT_BODY()

	static constexpr float DEFAULT_PARAMETER_VALUE = 0;

	UPROPERTY(Category = CustomizableObjectFloatParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectFloatParameterValue, VisibleAnywhere)
	float ParameterValue = 0.0f;

	UPROPERTY(Category = CustomizableObjectFloatParameterValue, VisibleAnywhere)
	FGuid Id;

	UPROPERTY(Category = CustomizableObjectFloatParameterValue, VisibleAnywhere)
	TArray<float> ParameterRangeValues;
};


inline uint32 GetTypeHash(const FCustomizableObjectFloatParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	for (const float Value : Key.ParameterRangeValues)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value));
	}

	return Hash;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectTextureParameterValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category = CustomizableObjectTextureParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectTextureParameterValue, VisibleAnywhere)
	TObjectPtr<UTexture> ParameterValue;

	UPROPERTY(Category = CustomizableObjectTextureParameterValue, VisibleAnywhere)
	FGuid Id;

	UPROPERTY(Category = CustomizableObjectTextureParameterValue, VisibleAnywhere)
	TArray<TObjectPtr<UTexture>> ParameterRangeValues;
};


inline uint32 GetTypeHash(const FCustomizableObjectTextureParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	for (const TObjectPtr<UTexture>& Value : Key.ParameterRangeValues)
    {
    	Hash = HashCombine(Hash, GetTypeHash(Value.GetPathName()));
    }

	return Hash;
}


USTRUCT()
struct FCustomizableObjectSkeletalMeshParameterValue
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(Category = CustomizableObjectSkeletalMeshParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectSkeletalMeshParameterValue, VisibleAnywhere)
	TObjectPtr<USkeletalMesh> ParameterValue;

	UPROPERTY(Category = CustomizableObjectSkeletalMeshParameterValue, VisibleAnywhere)
	FGuid Id;

	UPROPERTY(Category = CustomizableObjectSkeletalMeshParameterValue, VisibleAnywhere)
	TArray<TObjectPtr<USkeletalMesh>> ParameterRangeValues;
};


inline uint32 GetTypeHash(const FCustomizableObjectSkeletalMeshParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	for (const TObjectPtr<USkeletalMesh>& Value : Key.ParameterRangeValues)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value.GetPathName()));
	}

	return Hash;
}


USTRUCT()
struct FCustomizableObjectMaterialParameterValue
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY(Category = CustomizableObjectMaterialParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectMaterialParameterValue, VisibleAnywhere)
	TObjectPtr<UMaterialInterface> ParameterValue;

	UPROPERTY(Category = CustomizableObjectMaterialParameterValue, VisibleAnywhere)
	FGuid Id;

	UPROPERTY(Category = CustomizableObjectMaterialParameterValue, VisibleAnywhere)
	TArray<TObjectPtr<UMaterialInterface>> ParameterRangeValues;
};


inline uint32 GetTypeHash(const FCustomizableObjectMaterialParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	for (const TObjectPtr<UMaterialInterface>& Value : Key.ParameterRangeValues)
	{
		Hash = HashCombine(Hash, GetTypeHash(Value.GetPathName()));
	}

	return Hash;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectVectorParameterValue
{
	GENERATED_USTRUCT_BODY()

	inline static const FLinearColor DEFAULT_PARAMETER_VALUE = FLinearColor::Black;
	
	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FLinearColor ParameterValue = FLinearColor(ForceInit);

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FGuid Id;
};


inline uint32 GetTypeHash(const FCustomizableObjectVectorParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	return Hash;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectTransformParameterValue
{
	GENERATED_USTRUCT_BODY()

	inline static const FTransform DEFAULT_PARAMETER_VALUE = FTransform::Identity;
	
	UPROPERTY(Category = CustomizableObjectTransformParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectTransformParameterValue, VisibleAnywhere)
	FTransform ParameterValue = FTransform::Identity;

	UPROPERTY(Category = CustomizableObjectTransformParameterValue, VisibleAnywhere)
	FGuid Id;
};


inline uint32 GetTypeHash(const FCustomizableObjectTransformParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterValue));

	return Hash;
}



UENUM()
enum class ECustomizableObjectProjectorType : uint8
{
	Planar = 0 UMETA(DisplayName = "Planar projection"),
	Cylindrical = 1 UMETA(DisplayName = "Cylindrical projection"),
	Wrapping = 2 UMETA(DisplayName = "Wrapping projection")
};


USTRUCT(BlueprintType)
struct FCustomizableObjectProjector
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CustomizableObjectProjector)
	FVector3f Position = FVector3f(0, 0, 0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CustomizableObjectProjector)
	FVector3f Direction = FVector3f(1, 0, 0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CustomizableObjectProjector)
	FVector3f Up = FVector3f(0, 1, 0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CustomizableObjectProjector)
	FVector3f Scale = FVector3f(10, 10, 100);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = CustomizableObjectProjector)
	ECustomizableObjectProjectorType ProjectionType = ECustomizableObjectProjectorType::Planar;

	// Just for cylindrical projectors, in radians
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = CustomizableObjectProjector)
	float Angle = 2.0f * PI;

	bool operator==(const FCustomizableObjectProjector& Other) const = default;
};


inline uint32 GetTypeHash(const FCustomizableObjectProjector& Key)
{
	uint32 Hash = GetTypeHash(Key.Position);
	Hash = HashCombine(Hash, GetTypeHash(Key.Direction));
	Hash = HashCombine(Hash, GetTypeHash(Key.Up));
	Hash = HashCombine(Hash, GetTypeHash(Key.Scale));
	Hash = HashCombine(Hash, GetTypeHash(Key.ProjectionType));
	Hash = HashCombine(Hash, GetTypeHash(Key.Angle));

	return Hash;
}


inline FArchive& operator<<(FArchive& Ar, FCustomizableObjectProjector& Data)
{
	Ar << Data.Position;
	Ar << Data.Direction;
	Ar << Data.Up;
	Ar << Data.Scale;
	Ar << Data.ProjectionType;
	Ar << Data.Angle;
	return Ar;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectProjectorParameterValue
{
	GENERATED_USTRUCT_BODY()

	inline static const FCustomizableObjectProjector DEFAULT_PARAMETER_VALUE = {};

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FCustomizableObjectProjector Value;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FGuid Id;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	TArray<FCustomizableObjectProjector> RangeValues;
};


inline uint32 GetTypeHash(const FCustomizableObjectProjectorParameterValue& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterName);
	Hash = HashCombine(Hash, GetTypeHash(Key.Value));
	
	for (const FCustomizableObjectProjector& Value : Key.RangeValues)
    {
    	Hash = HashCombine(Hash, GetTypeHash(Value));
    }
	
	return Hash;
}


USTRUCT(BlueprintType)
struct FCustomizableObjectExternalTypeParameterValue
{
	GENERATED_USTRUCT_BODY()

	inline static const FInstancedStruct DEFAULT_PARAMETER_VALUE = {};

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FString ParameterName;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FInstancedStruct ParameterValue;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	FGuid Id;

	UPROPERTY(Category = CustomizableObjectVectorParameterValue, VisibleAnywhere)
	TArray<FInstancedStruct> ParameterRangeValues;
};


uint32 GetTypeHash(const FCustomizableObjectExternalTypeParameterValue& Key);
