// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "Engine/SkeletalMesh.h"
#include "CoreMinimal.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuR/ResourceID.h"

#include "CustomizableObjectSkeletalMesh.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UCustomizableObjectInstance;
class UCustomizableObject;
class UCustomizableInstancePrivate;
class FUpdateContextPrivate;
class FCustomizableObjectMeshStreamIn;
struct FModelStreamableBulkData;
class FUnrealMutableResourceProvider;

/**
 * Context required to stream in/out mesh LODs of a UCustomizableObjectSkeletalMesh
 */
struct FMutableMeshContext
{
	FMutableMeshContext(const TSharedRef<UE::Mutable::Private::FLiveInstance>& InLiveInstance)
		: LiveInstance(InLiveInstance) {}

	TSharedPtr<UE::Mutable::Private::FSystem> System;
	TSharedRef<UE::Mutable::Private::FLiveInstance> LiveInstance;

	TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData;

	UE::Mutable::Private::FSkeletalMeshId SkeletalMeshId;

	/** IDs per LOD */
	TArray<TArray<uint32>> SurfaceIDs;
};


/**
 * CustomizableObjectSkeletalMesh is a class inheriting from USkeletalMesh with the sole purpose of allowing skeletal meshes generated by Mutable
 * to stream-in Mesh LODs. 
 */
UCLASS(MinimalAPI)
class UCustomizableObjectSkeletalMesh : public USkeletalMesh
{
	GENERATED_BODY()

	friend FCustomizableObjectMeshStreamIn;
	friend UCustomizableInstancePrivate;
	friend FSkeletalMeshCache;
	
	UE_API void InitMutableStreamingData(const TSharedRef<FUpdateContextPrivate>& InOperationData, int32 ComponentIndex, const int32 FirstLOD, const int32 LODCount);

	//~ Begin UStreamableRenderAsset Interface.
	UE_API virtual bool StreamIn(int32 NewMipCount, bool bHighPrio) override;
	//~ End UStreamableRenderAsset Interface.

	FString CustomizableObjectPathName;

	TSharedPtr<FMutableMeshContext> MeshContext;

	UE::Mutable::Private::FSkeletalMeshId SkeletalMeshId;
};

#undef UE_API
