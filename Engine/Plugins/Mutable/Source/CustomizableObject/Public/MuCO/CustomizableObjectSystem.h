// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define UE_MUTABLE_MAX_OPTIMIZATION			2

#include "HAL/IConsoleManager.h"
#include "Engine/StreamableManager.h"
#include "AssetRegistry/AssetData.h"
#include "MuCO/CustomizableObjectInstance.h"

#include "CustomizableObjectSystem.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class IConsoleVariable;
class ITargetPlatform;
class UCustomizableObject;
class USkeletalMesh;
class UMaterialInterface;
class UTexture2D;
class UCustomizableObjectSystemPrivate; // This is used to hide Mutable SDK members in the public headers.
class FUpdateContextPrivate;
struct FFrame;
struct FGuid;
struct FEditorCompileSettings;


extern TAutoConsoleVariable<bool> CVarClearWorkingMemoryOnUpdateEnd;

extern TAutoConsoleVariable<bool> CVarReuseImagesBetweenInstances;

extern TAutoConsoleVariable<bool> CVarEnableMeshCache;

extern TAutoConsoleVariable<bool> CVarEnableRealTimeMorphTargets;

extern TAutoConsoleVariable<bool> CVarRollbackFixModelDiskStreamerDataRace;

extern TAutoConsoleVariable<bool> CVarEnableReleaseMeshResources;

extern TAutoConsoleVariable<bool> CVarFixLowPriorityTasksOverlap;

extern TAutoConsoleVariable<bool> CVarRollbackReuseProgramCacheBetweenUpdates;


UENUM(BlueprintType)
enum class ECustomizableObjectOptimizationLevel : uint8
{
	None = 0, 
	Minimal UE_DEPRECATED(5.6, "Converted internally to None. Use None instead.") = 1,
	Maximum = 2,
	FromCustomizableObject  = 3					// Grab the optimization settings from the CO
};


UENUM(BlueprintType)
enum class ECustomizableObjectTextureCompression : uint8
{
	// Don't use texture compression
	None = 0,
	// Use Mutable's fast low-quality compression
	Fast,
	// Use Unreal's highest quality compression (100x slower to compress)
	HighQuality
};


UENUM()
enum class ECustomizableObjectNumBoneInfluences : uint8
{
	// The enum values can be used as the real numeric value of number of bone influences
	Four = 4,
	// 
	Eight = 8,
	//
	Twelve = 12 // This is essentially the same as "Unlimited", but UE ultimately limits to 12
};


namespace EMutableProfileMetric
{
	typedef uint8 Type;

	constexpr Type BuiltInstances = 1;
	constexpr Type UpdateOperations = 2;
	constexpr Type Count = 4;

};


USTRUCT()
struct FPendingReleaseMaterialsInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	UPROPERTY()
	int32 TicksUntilRelease = 0;
};



// Opaque representation of a possible registered value for texture parameters.
struct FCustomizableObjectExternalTexture
{
	FCustomizableObjectExternalTexture() = default;

	FString Name;
	FName Value;
};


UCLASS(MinimalAPI, BlueprintType)
class UCustomizableObjectSystem : public UObject
{
	// Friends
	friend class UCustomizableObjectSystemPrivate;

public:
	GENERATED_BODY()

	UCustomizableObjectSystem() = default;
	UE_API void InitSystem();

	/** Get the singleton object. It will be created if it doesn't exist yet. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Status)
	static UE_API UCustomizableObjectSystem* GetInstance();
	
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = Status)
	static UE_API UCustomizableObjectSystem* GetInstanceChecked();

	/** Determines if the result of the instance update is valid or not.
	 * @return true if the result is successful or has warnings, false if the result is from the Error category */
	UFUNCTION(BlueprintCallable, Category = Status)
	static UE_API bool IsUpdateResultValid(const EUpdateResult UpdateResult);
	
	// Return true if the singleton has been created. It is different than GetInstance in that GetInstance will create it if it doesn't exist.
	static UE_API bool IsCreated();

	/** Returns the current status of Mutable. Only when active is it possible to compile COs, generate instances, and stream textures.
	  * @return True if Mutable is enabled. */
	static UE_API bool IsActive();

	// Begin UObject interface.
	UE_API virtual void BeginDestroy() override;
	UE_API virtual FString GetDesc() override;
	// End UObject interface.

#if WITH_EDITOR
	// Lock a CustomizableObjects, preventing the generation or update of any of its instances
	// Will return true if successful, false if it fails to lock because an update is already underway
	// This is usually only used in the editor
	UE_API bool LockObject(UCustomizableObject*);
	UE_API void UnlockObject(UCustomizableObject*);

	/** Checks if there are any outstanding disk or mip update operations in flight for the parameter Customizable Object that may
	* make it unsafe to compile at the moment.
	* @return true if there are operations in flight and it's not safe to compile */
	UE_API bool CheckIfDiskOrMipUpdateOperationsPending(const UCustomizableObject& Object) const;
	
	// Called whenever the Mutable Editor Settings change, copying the new value of the current needed settings to the Customizable Object System
	UE_API void EditorSettingsChanged(const FEditorCompileSettings& InEditorSettings);

	// If true, uncompiled Customizable Objects will be compiled whenever an instance update is required
	UE_API bool IsAutoCompileEnabled() const;

	/** Return true if inside commandlets uncompiled Customizable Objects will be compiled whenever an instance update is required. */
	UE_API bool IsAutoCompileCommandletEnabled() const;

	/** Set if inside commandlets uncompiled Customizable Objects will be compiled whenever an instance update is required. */
	UE_API void SetAutoCompileCommandletEnabled(bool bValue);
	
	// If true, uncompiled Customizable Objects will be compiled synchronously
	UE_API bool IsAutoCompilationSync() const;
#endif

	UE_DEPRECATED(5.8, "All platforms now support 16-bit bone indices. This always returns true.")
	UE_API bool IsSupport16BitBoneIndexEnabled() const;

	UE_API bool IsProgressiveMipStreamingEnabled() const;
	UE_API void SetProgressiveMipStreamingEnabled(bool bIsEnabled);
	
	// Show a warning on-screen and via a notification (if in Editor) and log an error when a CustomizableObject is
	// being used and it's not compiled.  Callers can add additional information to the error log.
	UE_API void AddUncompiledCOWarning(const UCustomizableObject& InObject, FString const* OptionalLogInfo = nullptr);

	// Give access to the internal object data.
	UE_API UCustomizableObjectSystemPrivate* GetPrivate();
	UE_API const UCustomizableObjectSystemPrivate* GetPrivate() const;

	// Find out the version of the plugin
	UFUNCTION(BlueprintCallable, Category = Status)
	UE_API FString GetPluginVersion() const;

	// Get the number of instances built and alive.
	UFUNCTION(BlueprintCallable, Category = Status)
	UE_API int32 GetNumInstances() const;

	// Get the number of instances waiting to be updated.
	UFUNCTION(BlueprintCallable, Category = Status)
	UE_API int32 GetNumPendingInstances() const;

	// Get the total number of instances including built and not built.
	UFUNCTION(BlueprintCallable, Category = Status)
	UE_API int32 GetTotalInstances() const;

	// Get the amount of GPU memory in use in bytes for textures generated by mutable.
	UFUNCTION(BlueprintCallable, Category = Status)
	UE_API int64 GetTextureMemoryUsed() const;

	// Return the average build/update time of an instance in ms.
	UFUNCTION(BlueprintCallable, Category = Status)
	UE_API int32 GetAverageBuildTime() const;

	/** Return true if the instance is being updated. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = CustomizableObjectSystem)
	UE_API bool IsUpdating(const UCustomizableObjectInstance* Instance) const;

	/** Set Mutable's working memory limit (kilobytes). Mutable will flush internal caches to try to keep its memory consumption below the WorkingMemory (i.e., it is not a hard limit).
	 * The working memory limit will especially reduce the memory required to perform Instance Updates and Texture Streaming.
 	 * Notice that Mutable does not track all its memory (e.g., UObjects memory is no tracked).
	 * This value can also be set using "mutable.WorkingMemory" CVar. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectSystem)
	UE_API void SetWorkingMemory(int32 KiloBytes);

	/** Get Mutable's working memory limit (kilobytes). See SetWorkingMemory(int32). */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = CustomizableObjectSystem)
	UE_API int32 GetWorkingMemory() const;

	/**
	 * Get if mutable should clear its working memory between instance updates.
	 * @return True if the clearing of memory will be performed, false otherwise.
	 */
	static UE_API bool ShouldClearWorkingMemoryOnUpdateEnd();

	// Instances within the generation range to the ViewCenters will be generated, 
	// When an instance falls outside the generation range the generated resources (meshes, textured, etc) will be dereferenced, and meshes on the components cleared.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectSystem)
	UE_API void SetGenerateInstancesWithinRange(bool bActive);

	// Instance generation range around view centers when mutable.GenerateInstancesWithinRange is true
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectSystem)
	UE_API void SetInstanceGenerationRange(float Distance);

	// Add ViewCenters. Used to generate instances around when generating instances within a range.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectSystem)
	UE_API void AddViewCenter(const AActor* const InCentralActor);

	// Remove ViewCenters. Used to generate instances around when generating instances within a range.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectSystem)
	UE_API void RemoveViewCenter(const AActor* const InCentralActor);
	
	// Clear ViewCenters. Used to generate instances around when generating instances within a range.
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectSystem)
	UE_API void ClearViewCenters();

private:
	UPROPERTY(Transient)
	TObjectPtr<UCustomizableObjectSystemPrivate> Private = nullptr;


public:
	
	/**
	 * Enables the collection of internal Mutable performance data. It has a performance cost.
	 */
	UE_API void EnableBenchmark();

	/**
	 * Disables the reporting of mutable instance benchmarking data.
	 */
	UE_API void EndBenchmark();
	
};

#undef UE_API
