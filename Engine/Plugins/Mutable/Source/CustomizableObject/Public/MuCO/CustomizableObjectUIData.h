// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "Engine/Texture2D.h"
#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"
#include "GameplayTagContainer.h"

#include "CustomizableObjectUIData.generated.h"

class UTexture2D;


USTRUCT(BlueprintType)
struct FMutableUIMetadata
{
	GENERATED_BODY()

	/** This is the name to be shown in UI */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	FString ObjectFriendlyName;

	/** This is the name of the section where the object will be placed in UI */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	FString UISectionName;

	/** This is the order of the object inside its section */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	int32 UIOrder = 0;

	/** Thumbnail for UI */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	TSoftObjectPtr<UTexture2D> UIThumbnail;

#if WITH_EDITORONLY_DATA
	/** Editor Only Parameter Thumbnail. In an editor parameter combobox, this option will be represented with the selected asset's thumbnail. */
	UPROPERTY(EditAnywhere, Category = UI)
	TSoftObjectPtr<UObject> EditorUIThumbnailObject;
#endif

	/** Extra information to be used in UI building, with semantics completely defined by the game/UI programmer, with a key to identify the semantic of its related value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	TMap<FString, FString> ExtraInformation;

	/** Extra assets to be used in UI building */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	TMap<FString, TSoftObjectPtr<UObject>> ExtraAssets;

	friend FArchive& operator<<(FArchive& Ar, FMutableUIMetadata& Struct);
};


USTRUCT(BlueprintType)
struct FMutableParamUIMetadata : public FMutableUIMetadata
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	float MinimumValue = 0.0f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	float MaximumValue = 1.0f;

	/** Gameplay tags to take into consideration when filtering parameters.
	  * Only applies to the options shown in drop downs. Child object parameters and data table rows. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = UI)
	FGameplayTagContainer GameplayTags;

	friend FArchive& operator<<(FArchive& Ar, FMutableParamUIMetadata& Struct);
};


USTRUCT(BlueprintType)
struct FMutableStateUIMetadata : public FMutableUIMetadata
{
	GENERATED_BODY()

	friend FArchive& operator<<(FArchive& Ar, FMutableStateUIMetadata& Struct);
};
