// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectSystem.h"

#include "CustomizableObject_Deprecated.generated.h"


USTRUCT()
struct FCompilationOptions_DEPRECATED
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	ECustomizableObjectTextureCompression TextureCompression = ECustomizableObjectTextureCompression::Fast;

	UPROPERTY()
	int32 OptimizationLevel = 2; // UE_MUTABLE_MAX_OPTIMIZATION;

	UPROPERTY()
	uint64 EmbeddedDataBytesLimit = 1024;
};