// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"

#include "MuCO/CustomizableObjectInstance.h"


#include "CustomizableSkeletalComponent.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UCustomizableSkeletalComponentPrivate;
class UCustomizableObjectInstanceUsage;

DECLARE_DELEGATE(FCustomizableSkeletalComponentUpdatedDelegate);


UCLASS(MinimalAPI, Blueprintable, BlueprintType, ClassGroup = (CustomizableObject), meta = (BlueprintSpawnableComponent))
class UCustomizableSkeletalComponent : public USceneComponent
{
	friend UCustomizableSkeletalComponentPrivate;

	GENERATED_BODY()

public:
	/** This component index refers to the object list of components
	* DEPRECATED: Use FNames instead with Get/SetComponentName
	*/
	UPROPERTY(BlueprintReadWrite, Category = CustomizableSkeletalComponent)
	int32 ComponentIndex;

private:

	UPROPERTY(EditAnywhere, Category = CustomizableSkeletalComponent)
	TObjectPtr<UCustomizableObjectInstance> CustomizableObjectInstance;

	UPROPERTY(EditAnywhere, Category = CustomizableSkeletalComponent)
	FName ComponentName;

	UPROPERTY(EditAnywhere, Category = CustomizableSkeletalComponent)
	bool bSkipSetReferenceSkeletalMesh = false;

	UPROPERTY(EditAnywhere, Category = CustomizableSkeletalComponent)
	bool bSkipSetSkeletalMeshOnAttach = false;
	
	UPROPERTY(Instanced)
	TObjectPtr<UCustomizableSkeletalComponentPrivate> Private;

public:
	
	// Own interface
	UE_API UCustomizableSkeletalComponent();

#if WITH_EDITOR
	UE_API virtual void PostLoad();
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API UCustomizableObjectInstanceUsage* GetInstanceUsage();

	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API void SetComponentName(const FName& Name);

	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API FName GetComponentName() const;
	
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API UCustomizableObjectInstance* GetCustomizableObjectInstance() const;
	
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API void SetCustomizableObjectInstance(UCustomizableObjectInstance* Instance);
	
	/** Set to true to avoid automatically replacing the Skeletal Mesh of the parent Skeletal Mesh Component by the Reference Skeletal Mesh.
	 * If SkipSetSkeletalMeshOnAttach is true, it will not replace it. */
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API void SetSkipSetReferenceSkeletalMesh(bool bSkip);

	/** Set to true to avoid automatically replacing the Skeletal Mesh of the parent Skeletal Mesh Component with any mesh. */
	UFUNCTION(BlueprintCallable, Category = CustomizableObjectInstanceUsage)
	UE_API void SetSkipSetSkeletalMeshOnAttach(bool bSkip);
	
	/** Update Skeletal Mesh asynchronously. */
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API void UpdateSkeletalMeshAsync(bool bForceUpdate = false);

	/** Update Skeletal Mesh asynchronously. Callback will be called once the update finishes, even if it fails. */
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalComponent)
	UE_API void UpdateSkeletalMeshAsyncResult(FInstanceUpdateDelegate Callback, bool bIgnoreCloseDist = false, bool bForceHighPriority = false);

protected:

	UE_API UCustomizableSkeletalComponentPrivate* GetPrivate();

	UE_API const UCustomizableSkeletalComponentPrivate* GetPrivate() const;

	// USceneComponent interface
	UE_API virtual void OnAttachmentChanged() override;
};

#undef UE_API
