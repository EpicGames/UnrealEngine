// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/SkeletalMeshActor.h"

#include "CustomizableSkeletalMeshActor.generated.h"

#define UE_API CUSTOMIZABLEOBJECT_API

class UCustomizableObjectInstance;
class UCustomizableSkeletalComponent;
class UCustomizableSkeletalMeshActorPrivate;

class UObject;

UCLASS(MinimalAPI, ClassGroup = CustomizableObject, Blueprintable, ComponentWrapperClass, ConversionRoot, meta = (ChildCanTick))
class ACustomizableSkeletalMeshActor : public ASkeletalMeshActor
{
	GENERATED_BODY()

	friend UCustomizableSkeletalMeshActorPrivate;

public:
	UE_API ACustomizableSkeletalMeshActor(const FObjectInitializer& Initializer);

	UFUNCTION(BlueprintCallable, Category = "CustomizableSkeletalMeshActor")
	UE_API UCustomizableObjectInstance* GetCustomizableObjectInstance();
	
	UFUNCTION(BlueprintCallable, Category = "CustomizableSkeletalMeshActor")
	UE_API USkeletalMeshComponent* GetSkeletalMeshComponent(const FName& ComponentName);
	
	UE_DEPRECATED(5.6, "Plase replace with custom code.")
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalMeshActor, meta=(DeprecatedFunction))
	UE_API void SetDebugMaterial(UMaterialInterface* InDebugMaterial);

	UE_DEPRECATED(5.6, "Plase replace with custom code.")
	UFUNCTION(BlueprintCallable, Category = CustomizableSkeletalMeshActor, meta=(DeprecatedFunction))
	UE_API void EnableDebugMaterial(bool bEnableDebugMaterial);

	UE_API UCustomizableSkeletalMeshActorPrivate* GetPrivate();
	UE_API const UCustomizableSkeletalMeshActorPrivate* GetPrivate() const;

private:
	// DEPRECATED
	UFUNCTION()
	UE_API void SwitchComponentsMaterials(UCustomizableObjectInstance* Instance);

	/** TODO: There are as many as components in the CO (not in the COI). */
	UPROPERTY(VisibleAnywhere, Category = CustomizableSkeletalMeshActor)
	TArray<TObjectPtr<UCustomizableSkeletalComponent>> CustomizableSkeletalComponents;

	/** TODO: There are as many as components in the CO (not in the COI). */
	UPROPERTY(VisibleAnywhere, Category = CustomizableSkeletalMeshActor)
	TArray<TObjectPtr<USkeletalMeshComponent>> SkeletalMeshComponents;

	// DEPRECATED
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DebugMaterial;

	// DEPRECATED
	bool bDebugMaterialEnabled = false;

	// DEPRECATED
	bool bRemoveDebugMaterial = false;

	UPROPERTY()
	TObjectPtr<UCustomizableSkeletalMeshActorPrivate> Private = nullptr;
};

#undef UE_API
