// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "MuCO/CustomizableObject.h"

class UCustomizableObjectInstanceUsage;

struct FRegisteredCustomizableObjectPinType
{
	TWeakObjectPtr<const UCustomizableObjectExtension> Extension;
	FCustomizableObjectPinType PinType;
};

struct FRegisteredObjectNodeInputPin
{
	TWeakObjectPtr<const UCustomizableObjectExtension> Extension;
	/** A name for this pin that should be globally unique across extensions */
	FName GlobalPinName;

	FObjectNodeInputPin InputPin;
};


/**
 * The public interface of the CustomizableObject module
 */
class ICustomizableObjectModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to ICustomizableObjectModule
	 *
	 * @return Returns CustomizableObjectModule singleton instance, loading the module on demand if needed
	 */
	static inline ICustomizableObjectModule& Get()
	{
		return FModuleManager::LoadModuleChecked< ICustomizableObjectModule >( "CustomizableObject" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "CustomizableObject" );
	}

	// Returns the number of bone influences that Mutable will use according to the plugin conf
	virtual ECustomizableObjectNumBoneInfluences GetNumBoneInfluences() const = 0;

	// Return a string representing the plugin version.
	virtual FString GetPluginVersion() const = 0;

	/**
	 * Extension functions
	 *
	 * These may only be called from the game thread
	 */
	virtual void RegisterExtension(const UCustomizableObjectExtension* Extension) = 0;
	virtual void UnregisterExtension(const UCustomizableObjectExtension* Extension) = 0;
	virtual TArrayView<const UCustomizableObjectExtension* const> GetRegisteredExtensions() const = 0;

	/**
	 * The results from these functions should only reference extensions that are still valid.
	 *
	 * If one of these functions returns data with an invalid weak pointer to a
	 * UCustomizableObjectExtension, it means that the extension was unloaded without calling
	 * UnregisterExtension.
	 */
	virtual TArrayView<const FRegisteredCustomizableObjectPinType> GetExtendedPinTypes() const = 0;
	virtual TArrayView<const FRegisteredObjectNodeInputPin> GetAdditionalObjectNodePins() const = 0;
	
	/** EXPERIMENTAL. USE WITH CAUTION!
	 *
	 * Register a new External Operation.
	 *
	 * All External Operations must inherit from UE::Mutable::FMutableOperation.
	 * It is only necessary to register operations in editor (to be able to create nodes). No need to register them at runtime. */
	virtual void RegisterExternalOperation(const UScriptStruct* Operation) = 0;

	/** EXPERIMENTAL. USE WITH CAUTION!
	 * 
	 * Unregister an External Operation. */
	virtual void UnregisterExternalOperation(const UScriptStruct* Operation) = 0;
};


/* Utility function for commands*/
CUSTOMIZABLEOBJECT_API
UCustomizableObjectInstanceUsage* GetPlayerCustomizableObjectInstanceUsage(const int32 SlotID, const UWorld* CurrentWorld, const int32 PlayerIndex = 0);


CUSTOMIZABLEOBJECT_API void PrintParticipatingPackagesDiff(const TArray<FName>& OutOfDatePackages, const TArray<FName>& AddedPackages, const TArray<FName>& RemovedPackages, bool bReleaseVersion);