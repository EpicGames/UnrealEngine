// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MultilayerProjector.generated.h"

struct FCustomizableObjectInstanceDescriptor;
class UCustomizableObjectInstance;
struct FMultilayerProjector;


/** Data structure representing a Multilayer Projector Layer.
 *
 * This struct is not actually saved, its values are obtained from the Instance Parameters. */
USTRUCT(BlueprintType)
struct FMultilayerProjectorLayer
{
	GENERATED_BODY()
	
	/** Read the Layer from the Instance Parameters.
	 *
	 * @param Descriptor Instance Descriptor.
	 * @param Index Index where to read the Layer in the Instance Parameters.
	 */
	void Read(const FCustomizableObjectInstanceDescriptor& Descriptor, const FString& ParamName, int32 Index);

	/** Write to Layer to the Instance Parameters.
	 *
 	 * @param Descriptor Instance Descriptor.
	 * @param Index Index where to write the Layer in the Instance Parameters.
	 */
	void Write(FCustomizableObjectInstanceDescriptor& Descriptor, const FString& ParamName, int32 Index) const;
	
	/** Layer position. */
	FVector Position;

	/** Layer direction vector. */
	FVector Direction;

	/** Layer up direction vector. */
	FVector Up;

	/** Layer scale. */
	FVector Scale;

	/** Layer angle. */
	float Angle;

	/** Layer selected image. */
	FString Image;	

	/** Layer image opacity. */
	float Opacity;
};


uint32 GetTypeHash(const FMultilayerProjectorLayer& Key);

