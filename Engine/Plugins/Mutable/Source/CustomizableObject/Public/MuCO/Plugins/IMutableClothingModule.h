// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "CoreMinimal.h"

#define MUTABLE_CLOTHING_MODULE_NAME "MutableClothing"

struct FMeshToMeshVertData;
class UClothingAssetCommon;

class IMutableClothingModule : public IModuleInterface
{
public:

	/**
	 * Updates in-place the InSimulationLODIndex of InOutClothingAsset physics mesh to conform 
	 * to InOutAttachedLODsRenderData. Remaps InOutAttachedLODsRenderData indices to be valid 
	 * after the modifications have been applied.
	 *
	 * NOTE: After calling UpdateClothSimulationLOD LOD transition mapping may contain invalid data 
	 * if any modification has taken place. In that case, FixLODTransitionMappings() must be called 
	 * after all InOutClothingAsset LODs have been updated.
	 *
	 * @param InSimulationLODIndex 		  Lod index in InOutClothingAsset to modify.
	 * @param InOutClothingAsset 		  Clothing asset to modify.
	 * @param InOutAttachedLODsRenderData List of mapping datas for all meshes attached to InOutClothingAsset's InSimulationLODIndex.
	 *
	 * @return true if the clothing asset has been modified, false otherwise. 
	 */
	virtual bool UpdateClothSimulationLOD(
			int32 InSimulationLODIndex,	
			UClothingAssetCommon& InOutClothingAsset,
			TConstArrayView<TArrayView<FMeshToMeshVertData>> InOutAttachedLODsRenderData) = 0;

	/**
	 * Fixes in-place LOD transition mappings for InSimulationLODIndex of InOutClothingAsset  
	 * 
	 * NOTE: FixLODTransitionMappings needs to run after all InOutClothingAsset LODs have been 
	 * updated.
	 *
	 * @param InSimulationLODIndex LOD in InOutClothingAsset to fix.
	 * @param InOutClothingAsset   Clothing asset to fix.
	 */
	virtual void FixLODTransitionMappings(
			int32 InSimulationLODIndex, 
			UClothingAssetCommon& InOutClothingAsset) = 0;

};

