// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CustomizableObjectEditor : ModuleRules
{

	public CustomizableObjectEditor(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		ShortName = "MuCOE";

		DefaultBuildSettings = BuildSettingsVersion.Latest;
		//bUseUnity = false;

		PrivateIncludePathModuleNames.AddRange(
			new string[] { 
				"AssetRegistry", 
				"MeshUtilities",
				"SkeletonEditor",
				"WorkspaceMenuStructure",
				"MessageLog",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ImageCore",
				"Engine",
				
				"Slate",
				"SlateCore",
				"EditorWidgets",
				"ApplicationCore",
				"RenderCore",
				"RHI",
				"UnrealEd",
				"TargetPlatform",
				"RawMesh",
				"LevelEditor",
				"AssetTools",
				"GraphEditor",
				"InputCore",
				"Kismet",
				"AdvancedPreviewScene",
				"AppFramework",
				"Projects",
				"ClothingSystemRuntimeCommon",
				"ClothingSystemRuntimeInterface",
				"DeveloperToolSettings",
				"ToolMenus",
				"ToolWidgets",
				"EditorFramework",
				"UMG",
				"Persona",
				"CurveEditor",
				"AnimGraphRuntime",
				"AnimGraph",

				"MutableRuntime",
				"MutableTools",
				"GameplayTags",
				
				"AssetDefinition",
				"ContentBrowser",
				
				"TextureCompressor",
				"TextureBuildUtilities",
				"ImageCore",
				"KismetWidgets",
				"MeshDescription",
				"StructUtilsEditor",
			}
		);


		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"SceneOutliner",
				"ClassViewer",
				"WorkspaceMenuStructure",
			}
		);


		PublicDependencyModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"CustomizableObject",
				"DerivedDataCache",
				"MutableTools",
			}
		);
	}
}
