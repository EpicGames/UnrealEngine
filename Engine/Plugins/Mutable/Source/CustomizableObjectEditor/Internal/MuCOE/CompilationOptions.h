// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObjectCompilerTypes.h"

class UCustomizableObject;

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


UE_API FCompilationOptions GetCompilationOptions(const UCustomizableObject& Object);


#undef UE_API