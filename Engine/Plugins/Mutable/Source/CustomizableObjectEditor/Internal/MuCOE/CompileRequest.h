// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"

#if WITH_EDITOR
#include "DerivedDataCacheKey.h"
#include "DerivedDataCachePolicy.h"
#endif

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UCustomizableObject;
class FCustomizableObjectCustomLogList;

struct FCompilationRequest
{
	UE_API FCompilationRequest(UCustomizableObject& CustomizableObject);

	UE_API UCustomizableObject* GetCustomizableObject();
	
	UE_API void SetDerivedDataCachePolicy(UE::DerivedData::ECachePolicy InCachePolicy);
	UE_API UE::DerivedData::ECachePolicy GetDerivedDataCachePolicy() const;

	UE_API void BuildDerivedDataCacheKey();
	UE_API UE::DerivedData::FCacheKey GetDerivedDataCacheKey() const;

	UE_API void SetCompilationState(ECompilationStatePrivate InState, ECompilationResultPrivate InResult);

	UE_API ECompilationStatePrivate GetCompilationState() const;
	UE_API ECompilationResultPrivate GetCompilationResult() const;
	
	UE_API bool operator==(const FCompilationRequest& Other) const;

private:
	TWeakObjectPtr<UCustomizableObject> CustomizableObject;

	ECompilationStatePrivate State = ECompilationStatePrivate::None;
	ECompilationResultPrivate Result = ECompilationResultPrivate::Unknown;
	
	UE::DerivedData::ECachePolicy DDCPolicy = UE::DerivedData::ECachePolicy::None;
	UE::DerivedData::FCacheKey DDCKey;

public:
	FCompilationOptions Options;

	TArray<FText> Warnings;
	TArray<FText> Errors;

	FCompileDelegate Callback;
	FCompileNativeDelegate CallbackNative;

	bool bAsync = true;
	bool bSkipIfCompiled = false;
	bool bSkipIfNotOutOfDate = false;
	bool bSilentCompilation = true;
};

#undef UE_API
