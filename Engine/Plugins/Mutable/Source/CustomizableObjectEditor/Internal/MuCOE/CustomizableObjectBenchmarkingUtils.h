// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/SpscQueue.h"
#include "UObject/StrongObjectPtr.h"


class UCustomizableObject;
class UCustomizableObjectInstance;
struct FCompilationOptions;
enum class ECustomizableObjectTextureCompression : uint8;

namespace CustomizableObjectBenchmarkingUtils
{
	/**
	 * Generates a set of instances. If the CO does not change the instances generated from one run to the other will be equal.
	 * @param TargetCustomizableObject The customizable object used to generate the set of deterministic instances.
	 * @param InstancesPerState The target amount of instances we want generated for each state of the CO. Ex: If the CO has 2 states and we define 4 here we will get a total of 2 * 4 instances as output.
	 * @param OutGeneratedInstances Queue with the generated instances. Based on the amount of states and the value set in InstancesPerState
	 * @param OutSuccesfullyGeneratedInstanceCount The total of instances generated successfully [0 , InstancesPerState * COStateCount]
	 * @return 
	 */
	CUSTOMIZABLEOBJECTEDITOR_API bool GenerateDeterministicSetOfInstances(UCustomizableObject& TargetCustomizableObject, const uint16 InstancesPerState, TSpscQueue<TStrongObjectPtr<UCustomizableObjectInstance>>& OutGeneratedInstances, uint32& OutSuccesfullyGeneratedInstanceCount);

	
	/**
	 * Returns the optimization level to be used in benchmarking runs
	 * @return Am integer value representing the optimization level that should be used for benchmarking runs
	 */
	CUSTOMIZABLEOBJECTEDITOR_API int32 GetOptimizationLevelForBenchmarking();
}