// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/CustomizableObjectInstance.h"


/**
 * Schedules the async compilation of a given Customizable object and, if a Customizable Object Instance is provided, tries to compile only the data required
 * for that given COI. Use the delegate to know when the operation has completed.
 * @param InTargetInstance The Customizable Object Instance generated from the provided COI that we want to use to limit the scope of the CO compilation.
 * @param InCustomizableObjectCompilationDelegate Delegate invoked once the compilation has completed.
 * @param bPerformPartialCompilation Instead of compiling all the data from the InTargetInstance Co just compile the data required for the provided instance.
 * todo: UE-315780 Make this option true by default when the compilation warnings caused by it are resolved
 */
CUSTOMIZABLEOBJECTEDITOR_API void ScheduleCOCompilationForBaking(UCustomizableObjectInstance& InTargetInstance, const FCompileNativeDelegate& InCustomizableObjectCompilationDelegate, const bool bPerformPartialCompilation = false);

/*
 * Data required by an asset to be able to be baked during the runtime of the baking operation.
 * It also serves as an interface to get the different substrings that form the actual baked package name to ease name manipulation once the baking
 * operation has completed
 */
class FResourceBakingData
{

public:
	FResourceBakingData(const FString& InPrefix, const FString& inUserGivenName, const FString& InRawResourceName, const FString& InComponentName, const FString& InAssetPath, bool bInRemoveRestrictedChars):
		Prefix(InPrefix),
		ResourceName(InRawResourceName),
		ComponentName(InComponentName),
		UserGivenName(inUserGivenName),
		AssetPath(InAssetPath),
		bReplaceRestrictedChars(bInRemoveRestrictedChars)
	{
		check(!AssetPath.IsEmpty());
	}

	/**
	 * Ensure that the data contained in this object is valid and well-formed
	 * @return true If it is ok, false otherwise
	 */
	bool Validate() const;

	/**
	 * Generate and get the Composed Resource Name to be used when duplicating (baking) the object represented by this data structure
	 * @return The string composed like so : Prefix_ObjectName_ResourceName(_UniqueAssetSuffix)
	 */
	FString GetResourceComposedName();
	
	/**
	* Get the complete path to be used when duplicating the package to decide where and how the package is named
	* @return The AssetPath/ComposedResourceName
	 */
	FString GetResourceComposedFullPath();
	
	/**
	 * Given the ComposedName looks for it in the provided array and updates this object data so the next time the ComposedName is generated is unique
	 * @param InCachedResources Array of resources we want to make sure the new name does not collide with.
	 * @note This does not check the actual path of the objects, just the name as this is intended to be used during the runtime of the bake to prevent collisions
	 * within the same bake
	 */
	void MakeComposedResourceNameUnique(const TArray<UObject*>& InCachedResources);

	/**
	 * Check if the file with the AssetPath + Composed name already exists, and based on that determines the save type to be performed
	 */
	void ComputePackageSaveResolutionType();
	
	/**
	 * Get the section of the ComposedResourceName that represents the prefix.
	 * @return The prefix string.
	 */
	FString GetPrefixSection() const;

	/**
	 * Get the section of the ComposedResourceName that represents the Component Name
	 * @return The Component name section.
	 */
	FString GetComponentNameSection() const;

	/**
	 * Get the section of the ComposedResourceName that represents the User provided name
	 * @return The User given name section. It can be empty.
	 */
	FString GetUserGivenNameSection() const;

/**
	 * Get the section of the ComposedResourceName that represents the actual Resource Name.
	 * @return The Resource name section.
	 */
	FString GetResourceNameSection() const;

	/***
	 * Get the section of the ComposedResourceName that represents the uniqueness suffix added to make the name unique.
	 * @return The suffix section.
	 */
	FString GetUniqueAssetNameSuffixSection() const;
	
private:

	/**
	 * Given a collection of UObjects return the index of the one that matches the name provided
	 * @param InName The name of the object we are looking for
	 * @param Objects The collection of objects we want to check for an element with the given name
	 * @return The index of the element whose name matches the one provided or INDEX_NONE if the element could not be found
	 */
	int32 GetObjectWithNameIndex(const FString& InName, const TArray<UObject*>& Objects ) const;

	/**
	 * Get the package our data points to. This is based on the AssetPath and the ComposedResourceName
	 * @return True if the asset is already at path
	 */
	bool DoesPackageExist(const FString& InPackageFullName, const FString& InAssetName) const;
	
	/**
	 * Get the last generated Composed Resource Name.
	 * @note As the ComposedResourceName changes over time as we update the parts that form it this method is only safe to call once the asset has been baked
	 * as no more changes to the CachedComposedResourceName are to be expected
	 * @return The last generated ComposedResourceName.
	 */
	FString GetCachedComposedResourceName() const;
	
	
public:
	
	/**
	 * The prefix (type based) of the resource. It is expected to include a "_" at the end like "T_" or "SKM_".
	 */
	FString Prefix;
	
	/**
	 * The name of the object that it is being baked. This string is expected to be externally modified during the bake runtime
	 */
	FString ResourceName;

	/**
	 * A suffix string used to make the name of the object Unique.
	 * Can be empty if no name collisions are found at the moment of bake
	 */
	FString UniqueAssetNameValueSuffix;

	/**
	 * The type of saving operation was performed. Does not affect the behaviour of the bake, but it is useful info for the caller
	 */
	EPackageSaveResolutionType SaveResolutionType = EPackageSaveResolutionType::None;

	
private:

	/**
	 * The component this object is part of.
	 */
	const FString ComponentName;
	
	/**
	 * The custom name given by the user to differentiate the objects of this bake from others.
	 * Can be empty.
	 */
	const FString UserGivenName;

	/**
	 * The path where to store the asset we are baking.
	 */
	const FString AssetPath;

	/**
	 * Cached version of the ComposedResourceName. Useful for the const consultation of the ComposedResourceName.
	 */
	FString CachedComposedResourceName;

	/**
	 * If true, restricted characters are stripped from the final composed resource name.
	 */
	bool bReplaceRestrictedChars = false;

	// String sections used to ease the removal of parts of the generated resource name once the baking has completed
	
	FString PrefixSection;
	
	FString ComponentNameSection;
	
	FString UserGivenNameSection;
	
	FString ResourceNameSection;
	
	FString UniqueAssetNameSuffixSection;
	
};


/**
 * Schedules the async update of the target instance. It will clear previous update data.
 * This method will also take care of setting and later resetting the state of the Customizable Object System.
 * @param InInstance The instance we want to update so we can later bake it's resources.
 * @param InInstanceUpdateDelegate Delegate to be called after the instance gets updated.
 */
CUSTOMIZABLEOBJECTEDITOR_API void ScheduleInstanceUpdateForBaking(UCustomizableObjectInstance& InInstance, FInstanceUpdateNativeDelegate& InInstanceUpdateDelegate);


/**
 * Serializes onto disk the resources used by the targeted Customizable Object Instance. The operation can be configured in the FInstanceBakingSettings settings object.
 * @param InInstance The mutable COI instance whose resources we want to bake onto disk
 * @param Configuration The baking configuration object with all the settings to be used for this baking operation.
 * @param bIsUnattendedExecution Determines if we want to run this method and show prompts for user interaction or if we want to automatically chose the more sensible option (without the need of user interaction)
 * @param OutSavedPackages A collection with the packages marked for save.
 * @return True if the baking operation could be completed without issues and false otherwise
 */
CUSTOMIZABLEOBJECTEDITOR_API bool BakeCustomizableObjectInstance(
	UCustomizableObjectInstance& InInstance,
	const FBakingConfiguration& Configuration,
	bool bIsUnattendedExecution,
	TMap<UPackage*,const FResourceBakingData>& OutSavedPackages);


/**
 * Serializes onto disk the resources used by the targeted Customizable Object Instance. The operation can be configured in the FInstanceBakingSettings settings object.
 * @param InInstance The mutable COI instance whose resources we want to bake onto disk
 * @param Configuration The baking configuration object with all the settings to be used for this baking operation.
 * @param bIsUnattendedExecution Determines if we want to run this method and show prompts for user interaction or if we want to automatically chose the more sensible option (without the need of user interaction)
 * @return True if the baking operation could be completed without issues and false otherwise
 */
CUSTOMIZABLEOBJECTEDITOR_API bool BakeCustomizableObjectInstance(
	UCustomizableObjectInstance& InInstance,
	const FBakingConfiguration& Configuration,
	bool bIsUnattendedExecution);