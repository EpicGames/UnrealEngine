// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"

#include "CustomizableObjectSkeletalMeshThumbnailRenderer.generated.h"

UCLASS()
class UCustomizableObjectSkeletalMeshThumbnailRenderer : public USkeletalMeshThumbnailRenderer
{
	GENERATED_BODY()
public:

	void EvictThumbnailScene(const UObject* Object)
	{
		ThumbnailSceneCache.RemoveThumbnailScene(Object);
	}
};
