// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "MuCO/ICustomizableObjectEditorModule.h"


/** Interface only accessible from the Customizable Object Editor module.
 *
 * TODO Move to private. Currently can not be moved due to MutableValidation module. */
class ICustomizableObjectEditorModulePrivate : public ICustomizableObjectEditorModule
{
public:
	static ICustomizableObjectEditorModulePrivate* Get()
	{
		return FModuleManager::LoadModulePtr<ICustomizableObjectEditorModulePrivate>(MODULE_NAME_COE);
	}
	
	static ICustomizableObjectEditorModulePrivate& GetChecked()
	{
		return FModuleManager::LoadModuleChecked<ICustomizableObjectEditorModulePrivate>(MODULE_NAME_COE);
	}
	
	virtual void EnqueueCompileRequest(const TSharedRef<FCompilationRequest>& InCompilationRequest, bool bForceRequest = false) = 0;
	
	virtual const TMap<FName, FLinearColor>& GetExternalPinTypeColors() const = 0;

	virtual const TMap<FName, FText>& GetExternalPinFriendlyName() const = 0;
	
	virtual const TMap<FName, TStrongObjectPtr<const UScriptStruct>>& GetExternalPinTypes() const = 0;
};