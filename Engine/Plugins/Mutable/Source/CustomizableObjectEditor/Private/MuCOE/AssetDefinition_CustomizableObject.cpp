// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/AssetDefinition_CustomizableObject.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "CustomizableObjectEditor.h"
#include "Editor.h"
#include "IContentBrowserSingleton.h"
#include "MuCOE/ICustomizableObjectEditorModulePrivate.h"
#include "Misc/MessageDialog.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenus.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CompileRequest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CustomizableObject)


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

EAssetCommandResult UAssetDefinition_CustomizableObject::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
	
	for (UCustomizableObject* Object : OpenArgs.LoadObjects<UCustomizableObject>())
	{
		const TSharedPtr<FCustomizableObjectEditor> Editor = MakeShared<FCustomizableObjectEditor>(*Object);
		Editor->InitCustomizableObjectEditor(Mode, OpenArgs.ToolkitHost);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CustomizableObject::GetAssetCategories() const
{
	static const auto Categories =
	{
		FAssetCategoryPath(NSLOCTEXT("AssetDefinition", "CustomizableObjectCategory", "Mutable"))
	};
	return Categories;
}


EAssetCommandResult UAssetDefinition_CustomizableObject::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	const EAssetActivationMethod ActivationType = ActivateArgs.ActivationMethod;
	const TConstArrayView<FAssetData,int> AssetsToActivate = ActivateArgs.Assets;
	
	if ( ActivationType == EAssetActivationMethod::DoubleClicked || ActivationType == EAssetActivationMethod::Opened )
	{
		if ( ActivateArgs.Assets.Num() == 1 )
		{
			const FAssetData* FirstAsset = AssetsToActivate.GetData();
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(UE::Mutable::Private::LoadObject(*FirstAsset));
		}
		else if ( ActivateArgs.Assets.Num() > 1 )
		{
			// Prepare an array with the assets
			TArray<UObject*> ObjectsToOpen;
			ObjectsToOpen.Reserve(AssetsToActivate.Num());
			for (const FAssetData& ToOpen : AssetsToActivate)
			{
				ObjectsToOpen.Add(UE::Mutable::Private::LoadObject(ToOpen));
			}
			ObjectsToOpen.Shrink();
			
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(ObjectsToOpen);
		}

		return EAssetCommandResult::Handled;
	}

	return EAssetCommandResult::Unhandled;
}

FAssetOpenSupport UAssetDefinition_CustomizableObject::GetAssetOpenSupport(
	const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(EAssetOpenMethod::Edit,true,EToolkitMode::Standalone);
}

namespace MenuExtension_CustomizableObject
{
	// Ported but may need tweaking
	void ExecuteNewInstance(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		check(Context);

		const FString DefaultSuffix = TEXT("_Inst");
		TArray<UObject*> ObjectsToSync;

		for (UCustomizableObject* Object : Context->LoadSelectedObjects<UCustomizableObject>())
		{
			// Determine an appropriate name
			FString Name;
			FString PackageName;
			// Create Unique asset name
			{
				FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);
			}
				
			UPackage* Pkg = CreatePackage(*PackageName);
			if (Pkg)
			{
				UCustomizableObjectInstance* Instance = NewObject<UCustomizableObjectInstance>(Pkg, FName(*Name), RF_Public|RF_Standalone);
			
				if (Instance)
				{
					// Mark the package dirty...
					Pkg->MarkPackageDirty();

					Instance->SetObject(Object);

					FAssetRegistryModule::AssetCreated(Instance);
					
					ObjectsToSync.Add(Instance);
				}
			}
		}

		if ( ObjectsToSync.Num() > 0 )
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToAssets( ObjectsToSync, /*bAllowLockedBrowsers=*/true );

			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ObjectsToSync[0]);
		}
	}
	
	void ExecuteEdit(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		check(Context);
		
		for (UCustomizableObject* Object : Context->LoadSelectedObjects<UCustomizableObject>())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
		}
	}

	enum class ERecompileCO
	{
		RCO_All,
		RCO_AllRootObjects,
		RCO_Selected,
		RCO_InMemory,
		RCO_SelectedGatherReferences,
	};
	
	void RecompileObjects(const FToolMenuContext& InMenuContext, ERecompileCO RecompileType)
	{
		if (!UCustomizableObjectSystem::GetInstance())
		{
			return;
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			return;
		}

		TArray<TSharedRef<FCompilationRequest>> CompilationRequests;


		if (RecompileType == ERecompileCO::RCO_Selected || RecompileType == ERecompileCO::RCO_SelectedGatherReferences)
		{
			const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InMenuContext);
			check(Context);
			
			TArray<UCustomizableObject*> Objects = Context->LoadSelectedObjects<UCustomizableObject>();
			CompilationRequests.Reserve(Objects.Num());

			for (UCustomizableObject* Object : Objects)
			{
				if (Object)
				{
					TSharedRef<FCompilationRequest> NewRequest = MakeShared<FCompilationRequest>(*Object);
					NewRequest->bSilentCompilation = true;
					
					if (RecompileType == ERecompileCO::RCO_SelectedGatherReferences)
					{
						NewRequest->Options.bGatherReferences = true;
					}

					CompilationRequests.Add(NewRequest);
				}
			}
		}
		else
		{
			FARFilter AssetRegistryFilter;
			AssetRegistryFilter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/CustomizableObject"), TEXT("CustomizableObject")));

			FText Msg;
			switch (RecompileType)
			{
			case ERecompileCO::RCO_All:
				Msg = LOCTEXT("AssetTypeActions_CustomizableObject_RecompileAll_Dialog", "Are you sure you want to recompile all Customizable Objects in this project?\n\nThis action might take a long time and leave the editor unresponsive until it finishes.");
				break;
			case ERecompileCO::RCO_AllRootObjects:
				Msg = LOCTEXT("AssetTypeActions_CustomizableObject_RecompileAllRoot_Dialog", "Are you sure you want to recompile all Root Customizable Objects in this project?\n\nThis action might take a long time and leave the editor unresponsive until it finishes.");
				AssetRegistryFilter.TagsAndValues.Add(FName("IsRoot"), FString::FromInt(1));
				break;
			case ERecompileCO::RCO_InMemory:
				Msg = LOCTEXT("AssetTypeActions_CustomizableObject_RecompileObjectsInMemory_Dialog", "Do you want to recompile all Customizable Objects in memory?\n\nThis action will compile objects referenced by the current level and other loaded objects.");
				break;
			default: check(false);
				break;
			}

			if (FMessageDialog::Open(EAppMsgType::OkCancel, Msg) == EAppReturnType::Ok)
			{
				TArray<FAssetData> OutAssets;
				AssetRegistryModule.Get().GetAssets(AssetRegistryFilter, OutAssets);

				for (const FAssetData& Asset : OutAssets)
				{
					if (RecompileType == ERecompileCO::RCO_InMemory && !Asset.IsAssetLoaded())
					{
						continue;
					}

					UCustomizableObject* Object = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(Asset));
					if (Object)
					{
						TSharedRef<FCompilationRequest> NewRequest = MakeShared<FCompilationRequest>(*Object);
						NewRequest->bSilentCompilation = true;
						CompilationRequests.Add(NewRequest);
					}
				}
			}
		}
		
		for (const TSharedRef<FCompilationRequest>& CompilationRequest : CompilationRequests)
		{
			ICustomizableObjectEditorModulePrivate::GetChecked().EnqueueCompileRequest(CompilationRequest);
		}
	}

	
	void GenerateCompilationSubmenu(UToolMenu* InMenu)
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection("CompileCOMenu");
		
		// Recompile Selected
		{
			const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileSelected", "Selected");
			const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileSelectedTooltip", "Recompile selected Customizable Object.");
			const FSlateIcon Icon = FSlateIcon();
			
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&RecompileObjects,ERecompileCO::RCO_Selected);
			Section.AddMenuEntry("CustomizableObject_ExecuteCompileSelected", Label, ToolTip, Icon, UIAction);
		}
	
		// Recompile Selected Gather References.
		{
			const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileSelectedGatherReferences", "Selected and Gather references");
			const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileSelectedGatherReferencesTooltip", "Recompile selected Customizable Object and gather references.");
			const FSlateIcon Icon = FSlateIcon();
			
			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&RecompileObjects,ERecompileCO::RCO_SelectedGatherReferences);
			Section.AddMenuEntry("CustomizableObject_ExecuteCompileSelectedGatherReferences", Label, ToolTip, Icon, UIAction);
		}


		// Recompile All
		{
			const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileAll", "All");
			const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileAllToolTip", "Recompile all Customizable Object assets in the project. Can take a really long time and freeze the editor while it is compiling..");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&RecompileObjects,ERecompileCO::RCO_All);
			Section.AddMenuEntry("CustomizableObject_ExecuteCompileAll", Label, ToolTip, Icon, UIAction);
		}

		// Recompile All Root Objects
		{
			const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileRootObjects", "All root objects");
			const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileRootObjectsTooltip", "Recompile all Root Customizable Objects in the project. Can take a long time and freeze the editor while it is compiling.");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&RecompileObjects,ERecompileCO::RCO_AllRootObjects);
			Section.AddMenuEntry("CustomizableObject_ExecuteCompileRoot", Label, ToolTip, Icon, UIAction);
		}

		// Recompile all objects in memory, either loaded by the level, and editor or a during a compilation (child objects)
		{
			const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileObjectsInMemory", "All loaded objects");
			const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject", "CustomizableObject_RecompileObjectsInMemoryTooltip", "Recompile all Customizable Objects in memory. This action will compile objects referenced by the current level, CO Editors, etc. Can take a long time and freeze the editor while it is compiling.");
			const FSlateIcon Icon = FSlateIcon();

			FToolUIAction UIAction;
			UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&RecompileObjects,ERecompileCO::RCO_InMemory);
			Section.AddMenuEntry("CustomizableObject_ExecuteCompileMem", Label, ToolTip, Icon, UIAction);
		}
	}
	

	// Method that registers the callbacks to be executed and the buttons to be displayed when right-clicking an object
	// of the CustomizableObject type.
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
	  		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
	  		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UCustomizableObject::StaticClass());

	        FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
	        Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	        {
		        // Here add the actions you want to be able to perform

		        // Edit
		        {
			        const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject",
			                                                  "CustomizableObject_Edit", "Edit");
			        const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject",
			                                                    "CustomizableObject_EditTooltip",
			                                                    "Opens the selected object in the customizable object editor.");
			        const FSlateIcon Icon = FSlateIcon();

			        FToolUIAction UIAction;
			        UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteEdit);
			        InSection.AddMenuEntry("CustomizableObject_ExecuteEdit", Label, ToolTip, Icon, UIAction);
		        }

		        // New instance
		        {
			        const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject",
			                                                  "CustomizableObject_NewInstance", "Create New Instance");
			        const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject",
			                                                    "CustomizableObject_NewInstanceTooltip",
			                                                    "Creates a new instance of this customizable object.");
			        const FSlateIcon Icon = FSlateIcon();

			        FToolUIAction UIAction;
			        UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewInstance);
			        InSection.AddMenuEntry("CustomizableObject_ExecuteNewInstance", Label, ToolTip, Icon, UIAction);
		        }

		        // Recompile submenu
		        {
			        const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObject",
			                                                  "CustomizableObject_RecompileObjects", "Recompile");
			        const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObject",
			                                                    "CustomizableObject_RecompileAllObjectsTooltip",
			                                                    "Recompile Customizable Objects actions.");
			        const FSlateIcon Icon = FSlateIcon();

			        InSection.AddSubMenu(
				        "CustomizableObject_ExecuteCompile",
				        Label,
				        ToolTip,
				        FNewToolMenuDelegate::CreateStatic(&GenerateCompilationSubmenu));
		        }
				
		 	}));
	   }));
   });
	
}



#undef LOCTEXT_NAMESPACE

