// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "MuCO/CustomizableObject.h"

#include "AssetDefinition_CustomizableObject.generated.h"

UCLASS()
class UAssetDefinition_CustomizableObject final : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CustomizableObject", "Customizable Object");};

	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(234, 255, 0));};

	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCustomizableObject::StaticClass();};

	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override {return EAssetCommandResult::Unhandled;}

	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
};
