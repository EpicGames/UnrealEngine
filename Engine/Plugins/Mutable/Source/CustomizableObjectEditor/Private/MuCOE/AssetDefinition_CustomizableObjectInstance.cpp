// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/AssetDefinition_CustomizableObjectInstance.h"

#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "CustomizableObjectInstanceEditor.h"
#include "Editor.h"
#include "ToolMenus.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCO/LoadUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CustomizableObjectInstance)

#define LOCTEXT_NAMESPACE "CustomizableObjectInstance"

EAssetCommandResult UAssetDefinition_CustomizableObjectInstance::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UCustomizableObjectInstance* Object : OpenArgs.LoadObjects<UCustomizableObjectInstance>())
	{
		const TSharedPtr<FCustomizableObjectInstanceEditor> Editor = MakeShared<FCustomizableObjectInstanceEditor>();
		Editor->InitCustomizableObjectInstanceEditor(Mode, OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_CustomizableObjectInstance::GetAssetCategories() const
{
	static const std::initializer_list<FAssetCategoryPath> Categories =
	{
		// Asset can be found inside the Mutable submenu 
		NSLOCTEXT("AssetTypeActions", "Mutable", "Mutable")
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_CustomizableObjectInstance::ActivateAssets(
	const FAssetActivateArgs& ActivateArgs) const
{
	if (ActivateArgs.ActivationMethod == EAssetActivationMethod::DoubleClicked || ActivateArgs.ActivationMethod == EAssetActivationMethod::Opened)
	{
		if (ActivateArgs.Assets.Num() == 1)
		{
			const FAssetData* FirstAsset = ActivateArgs.Assets.GetData();
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(UE::Mutable::Private::LoadObject(*FirstAsset));
		}
		else if (ActivateArgs.Assets.Num() > 1)
		{
			TArray<UObject*> ObjectsToOpen{};
			ObjectsToOpen.Reserve(ActivateArgs.Assets.Num());
			for (const FAssetData& ToOpen : ActivateArgs.Assets)
			{
				ObjectsToOpen.Add(UE::Mutable::Private::LoadObject(ToOpen));
			}
			ObjectsToOpen.Shrink();
			
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(ObjectsToOpen);
		}

		return EAssetCommandResult::Handled;
	}
	
	return EAssetCommandResult::Unhandled;
}

FAssetOpenSupport UAssetDefinition_CustomizableObjectInstance::GetAssetOpenSupport(
	const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(EAssetOpenMethod::Edit,true,EToolkitMode::Standalone);
}


namespace MenuExtension_CustomizableObjectInstance
{
	void ExecuteEdit(const FToolMenuContext& ToolMenuContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(ToolMenuContext);
		check(Context);
		
		for (UCustomizableObjectInstance* Object : Context->LoadSelectedObjects<UCustomizableObjectInstance>())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);
		}
	}
	
	// Method that registers the callbacks to be executed and the buttons to be displayed when right-clicking an object
	// of the CustomizableObjectInstance type.
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{
	   UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	   {
	  		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
	  		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UCustomizableObjectInstance::StaticClass());
	   	
		  FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		  Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		  {
			// Here add the actions you want to be able to perform when right clicking the object in the Editor

		  	// Edit
		 	{
		 		const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObjectInstance","CustomizableObjectInstance_Edit","Edit");
		 		const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObjectInstance", "CustomizableObjectInstance_EditTooltip", "Opens the selected instances in the static mesh editor.");
		 		const FSlateIcon Icon = FSlateIcon();
			
		 		FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteEdit);
				InSection.AddMenuEntry("CustomizableObject_ExecuteEdit", Label, ToolTip, Icon, UIAction);
		 	}
		  	
		  }));
	   }));
   });
}

#undef LOCTEXT_NAMESPACE

