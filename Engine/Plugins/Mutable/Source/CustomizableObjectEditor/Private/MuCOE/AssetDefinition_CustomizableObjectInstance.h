// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "MuCO/CustomizableObjectInstance.h"

#include "AssetDefinition_CustomizableObjectInstance.generated.h"


UCLASS()
class UAssetDefinition_CustomizableObjectInstance final : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CustomizableObjectInstance", "Customizable Object Instance");};

	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 82, 49));};

	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCustomizableObjectInstance::StaticClass();};

	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override {return EAssetCommandResult::Unhandled;}

	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;

	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
};
