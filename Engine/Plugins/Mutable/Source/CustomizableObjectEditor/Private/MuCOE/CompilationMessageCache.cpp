// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CompilationMessageCache.h"

FCompilationMessageCache::FCompilationMessageCache()
{
	ClearMessageCounters();
}

bool FCompilationMessageCache::AddMessage(const FText& InMessage, const TArray<const UObject*>& InContext, EMessageSeverity::Type MessageSeverity /* = EMessageSeverity::Warning */, const ELoggerSpamBin SpamBin /*= ELoggerSpamBin::ShowAll*/)
{
	// Skip message if an identical one has already been reported
	const FLoggedMessage Cached = { InMessage, InContext, MessageSeverity };
	if (LoggedMessages.Contains(Cached))
	{
		return false;
	}

	// Skip message if it has a spam bin other than all and is introduced too many times
	if (SpamBin != ELoggerSpamBin::ShowAll)
	{
		uint32 currentNumOfBinMessages = ++SpamBinCounts[static_cast<uint8>(SpamBin)];
		if (currentNumOfBinMessages > MaxSpamMessages)
		{
			IgnoredCount++;
			IgnoredMessages.Add(Cached);
			return false;
		}
	}

	LoggedMessages.Add(Cached);

	switch (MessageSeverity)
	{
	case EMessageSeverity::Error:
		ErrorCount++;
		break;
	case EMessageSeverity::PerformanceWarning:
		PerformanceWarningCount++;
		break;
	case EMessageSeverity::Warning:
		WarningCount++;
		break;
	case EMessageSeverity::Info:
		// not handled at the moment
		break;
	default:
		checkNoEntry();
		break;
	}

	return true;
}


void FCompilationMessageCache::GetMessages(TArray<FText>& OutWarningMessages, TArray<FText>& OutErrorMessages) const
{
	if (LoggedMessages.IsEmpty())
	{
		return;
	}

	for (const FLoggedMessage& Log : LoggedMessages)
	{
		switch (Log.Severity)
		{
		case EMessageSeverity::Info:
			// not handled at the moment -> not required
			break;
		
		case EMessageSeverity::Error:
			OutErrorMessages.Add(Log.Message);
			break;

			// Both performance and standard warnings get treated as warnings.
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			OutWarningMessages.Add(Log.Message);
			break;

		default:
			// ERROR : severity type not defined
			checkNoEntry();
		}
	}	
}


void FCompilationMessageCache::ClearMessagesArray()
{
	LoggedMessages.Empty();
	IgnoredMessages.Empty();
}


void FCompilationMessageCache::ClearMessageCounters()
{
	PerformanceWarningCount = 0;
	WarningCount = 0;
	ErrorCount = 0;
	IgnoredCount = 0;
	SpamBinCounts.Empty();
	SpamBinCounts.SetNumZeroed(static_cast<uint8>(ELoggerSpamBin::ShowAll), EAllowShrinking::No);
}

uint32 FCompilationMessageCache::GetWarningCount(bool bIncludePerformanceWarnings) const
{
	if (bIncludePerformanceWarnings)
	{
		return WarningCount + PerformanceWarningCount;
	}
	
	return WarningCount;
}

uint32 FCompilationMessageCache::GetErrorCount() const
{
	return ErrorCount;
}

uint32 FCompilationMessageCache::GetIgnoredCount() const
{
	return IgnoredCount;
}
