// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Logging/TokenizedMessage.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

// Forward declarations
class UCustomizableObjectNode;

/** Class designed to allow easy access to the logs produced during the compilation of a Customizable Object */
class FCompilationMessageCache
{
public:
	UE_API FCompilationMessageCache();

	/** Method designed to serve as a way to add a new message to the collection of messages the compiler has produced as
	 * part of the compilation process.
	 * @param InMessage The message's text.
	 * @param InContext The UObject that is related with the InMessage provided.
	 * @param MessageSeverity Severity of the message.
	 * @return True if the message is new and had not been added before. If it returns false it means it has not been logged again.
	 */
	 UE_API bool AddMessage(const FText& InMessage, const TArray<const UObject*>& InContext, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll);
	
	
	/** Provides the caller with separated arrays with all the warning and error messages.
	 * @param OutWarningMessages Array with messages that can be interpreted as Warnings.
	 * @param OutErrorMessages Array with messages that can be interpreted as errors.
	 * @note Performance warnings are considered warnings so they get returned
	 */
	UE_API void GetMessages(TArray<FText>& OutWarningMessages, TArray<FText>& OutErrorMessages) const;


// protected:

	/** Clear the array of logs */
	UE_API void ClearMessagesArray();

	/** Clear the counters that show how many errors and warnings the array contains */
	UE_API void ClearMessageCounters();

	/** Provides with the amount of messages cached as warnings
	* @param bIncludePerformanceWarnings If true the count will include performance and non performance related warnings.
	* If false only standard warnings will be returned to the caller.
	* @return Amount of warning messages cached.
	*/
	UE_API uint32 GetWarningCount(bool bIncludePerformanceWarnings) const;

	/** Provides the caller with the amount of messages cached as errors
	 * @return Amount of error messages cached.
	 */
	UE_API uint32 GetErrorCount() const;

	/** Provides the caller with the amount of messages ignored because they were too similar even if not identical.
	 * @return Amount of error messages ignored.
	 */
	UE_API uint32 GetIgnoredCount() const;
	
private:
	
	// List of all logged messages to ensure uniqueness
	struct FLoggedMessage
	{
		FText Message;
		TArray<const UObject*> Context;
		EMessageSeverity::Type Severity;
	
		bool operator==( const FLoggedMessage& o) const
		{
			return Message.EqualTo(o.Message) && Context == o.Context && Severity == o.Severity;
		}
	};
	const uint32 MaxSpamMessages = 10;

	/** Array with all the logged messages stored. */
	TArray<FLoggedMessage> LoggedMessages;
	TArray<FLoggedMessage> IgnoredMessages;

	// Counters for the type of messages we have on LoggedMessages
	uint32 PerformanceWarningCount;
	uint32 WarningCount;
	uint32 ErrorCount;
	uint32 IgnoredCount;
	TArray<uint8> SpamBinCounts;
};

#undef UE_API
