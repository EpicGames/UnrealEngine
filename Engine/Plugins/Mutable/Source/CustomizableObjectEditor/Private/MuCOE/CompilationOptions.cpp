// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CompilationOptions.h"

#include "GraphTraversal.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"


FCompilationOptions GetCompilationOptions(const UCustomizableObject& Object)
{
	const UCustomizableObjectPrivate* RootObjectPrivate = GraphTraversal::GetRootObject(&Object)->GetPrivate();
	
	FCompilationOptions Options;
	Options.TextureCompression = RootObjectPrivate->TextureCompression;
	Options.OptimizationLevel = RootObjectPrivate->OptimizationLevel;
	Options.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	Options.EmbeddedDataBytesLimit = RootObjectPrivate->EmbeddedDataBytesLimit;
	Options.CustomizableObjectNumBoneInfluences = ICustomizableObjectModule::Get().GetNumBoneInfluences();
	Options.bRealTimeMorphTargetsEnabled = RootObjectPrivate->IsRealTimeMorphTargetsEnabled();
	Options.b16BitBoneWeightsEnabled = RootObjectPrivate->Is16BitBoneWeightsEnabled();
	Options.bSkinWeightProfilesEnabled = RootObjectPrivate->IsAltSkinWeightProfilesEnabled();
	Options.bPhysicsAssetMergeEnabled = RootObjectPrivate->IsPhysicsAssetMergeEnabled();
	Options.bAnimBpPhysicsManipulationEnabled = RootObjectPrivate->IsEnabledAnimBpPhysicsAssetsManipulation();
	Options.bUseLegacyLayouts = RootObjectPrivate->ShouldUseLegacyLayouts();

	return Options;
}
