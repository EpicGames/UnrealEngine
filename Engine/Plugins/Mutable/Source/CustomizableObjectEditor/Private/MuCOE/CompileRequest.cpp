// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuCOE/CompileRequest.h"

#include "CustomizableObjectCompiler.h"
#include "DerivedDataCachePolicy.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/CompilationOptions.h"


FCompilationRequest::FCompilationRequest(UCustomizableObject& InCustomizableObject)
{
	CustomizableObject = &InCustomizableObject;
	Options = GetCompilationOptions(InCustomizableObject);
	DDCPolicy = UE::DerivedData::ECachePolicy::None;
}


UCustomizableObject* FCompilationRequest::GetCustomizableObject()
{
	return CustomizableObject.Get();
}


void FCompilationRequest::SetDerivedDataCachePolicy(UE::DerivedData::ECachePolicy InCachePolicy)
{
	DDCPolicy = InCachePolicy;
	Options.bQueryCompiledDatafromDDC = EnumHasAnyFlags(InCachePolicy, UE::DerivedData::ECachePolicy::Query);
	Options.bStoreCompiledDataInDDC = EnumHasAnyFlags(InCachePolicy, UE::DerivedData::ECachePolicy::Store);
}


UE::DerivedData::ECachePolicy FCompilationRequest::GetDerivedDataCachePolicy() const
{
	return DDCPolicy;
}

void FCompilationRequest::BuildDerivedDataCacheKey()
{
	if (UCustomizableObject* Object = CustomizableObject.Get())
	{
		DDCKey = Object->GetPrivate()->GetDerivedDataCacheKeyForOptions(Options);
	}
}


UE::DerivedData::FCacheKey FCompilationRequest::GetDerivedDataCacheKey() const
{
	return DDCKey;
}


void FCompilationRequest::SetCompilationState(ECompilationStatePrivate InState, ECompilationResultPrivate InResult)
{
	State = InState;
	Result = InResult;
}


ECompilationStatePrivate FCompilationRequest::GetCompilationState() const
{
	return State;
}


ECompilationResultPrivate FCompilationRequest::GetCompilationResult() const
{
	return Result;
}


bool FCompilationRequest::operator==(const FCompilationRequest& Other) const
{
	return CustomizableObject == Other.CustomizableObject && Options.TargetPlatform == Other.Options.TargetPlatform;
}

