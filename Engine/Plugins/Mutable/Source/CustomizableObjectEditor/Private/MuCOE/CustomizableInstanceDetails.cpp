// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableInstanceDetails.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "CustomizableObjectInstanceEditor.h"
#include "ContentBrowserModule.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "IDetailGroup.h"
#include "Misc/TransactionObjectEvent.h"
#include "ScopedTransaction.h"
#include "Toolkits/ToolkitManager.h"
#include "Math/Transform.h"
#include "PropertyCustomizationHelpers.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "AssetThumbnail.h"
#include "IStructureDetailsView.h"
#include "Slate/DeferredCleanupSlateBrush.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "MuCO/LoadUtils.h"
#include "Types/SlateEnums.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Engine/Texture.h"
#include "Engine/SkeletalMesh.h"
#include "Nodes/ExternalTypeCustomization.h"
#include "UObject/StructOnScope.h"

class UObject;

#define LOCTEXT_NAMESPACE "CustomizableInstanceDetails"

// Define here metadata keywords used in the properties details
namespace UIMetadataKeyWords
{
	// Key name for float slider decorators
	constexpr TCHAR const* FloatDecoratorName = TEXT("SliderImage");
}


TSharedRef<IDetailCustomization> FCustomizableInstanceDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableInstanceDetails);
}


void FCustomizableInstanceDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	LayoutBuilder = DetailBuilder;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder->GetDetailsViewSharedPtr();
	check(DetailsView.IsValid());
	check(DetailsView->GetSelectedObjects().Num());

	CustomInstance = Cast<UCustomizableObjectInstance>(DetailsView->GetSelectedObjects()[0].Get());
	check(CustomInstance.IsValid());

	TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(CustomInstance.Get());
	TSharedPtr<ICustomizableObjectInstanceEditor> EditorPtr;

	// Tab spawned in a COEInstanceEditor
	if (FoundAssetEditor.IsValid()) 
	{
		if (TSharedPtr<FCustomizableObjectInstanceEditor> InstanceEditor = StaticCastSharedPtr<FCustomizableObjectInstanceEditor>(FoundAssetEditor))
		{
			EditorPtr = StaticCastSharedPtr<ICustomizableObjectInstanceEditor>(InstanceEditor);
		}
	}

	if (!FoundAssetEditor.IsValid())
	{
		FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(CustomInstance->GetCustomizableObject());

		// Tab spawned in a COEditor
		if (FoundAssetEditor.IsValid())
		{
			if (TSharedPtr<FCustomizableObjectEditor> ObjectEditor = StaticCastSharedPtr<FCustomizableObjectEditor>(FoundAssetEditor))
			{
				EditorPtr = StaticCastSharedPtr<ICustomizableObjectInstanceEditor>(ObjectEditor);
			}
		}
	}
	check(EditorPtr.IsValid());

	WeakEditor = EditorPtr.ToWeakPtr();
	
	// Delegate to refresh the details when the instance has finished the Update
	CustomInstance->UpdatedNativeDelegate.AddSP(this, &FCustomizableInstanceDetails::InstanceUpdated);
	CustomInstance->GetPrivate()->OnInstanceTransactedDelegate.AddSP(this, &FCustomizableInstanceDetails::OnInstanceTransacted);

	// New Category that will store all properties widgets
	IDetailCategoryBuilder& ResourcesCategory = DetailBuilder->EditCategory("Generated Resources");
	IDetailCategoryBuilder& VisibilitySettingsCategory = DetailBuilder->EditCategory("ParametersVisibility");
	IDetailCategoryBuilder& ParametersCategory = DetailBuilder->EditCategory("Instance Parameters");

	// Showing warning message in case that the instance has not been compiled
	UCustomizableObject* CustomizableObject = CustomInstance->GetCustomizableObject();
	if (!CustomizableObject)
	{
		VisibilitySettingsCategory.AddCustomRow(LOCTEXT("CustomizableInstanceDetails_NoCOMessage", "Instance Parameters"))
		[
			SNew(STextBlock).Text(LOCTEXT("Model not compiled", "Model not compiled"))
		];

		return;
	}

	CustomizableObject->GetPostCompileDelegate().AddSP(this, &FCustomizableInstanceDetails::ObjectCompiled);

	TArray<UObject*> Private;
	Private.Add(CustomInstance->GetPrivate());

	FAddPropertyParams PrivatePropertyParams;
	PrivatePropertyParams.HideRootObjectNode(true);

	ResourcesCategory.InitiallyCollapsed(true);
	IDetailPropertyRow* PrivateDataRow = ResourcesCategory.AddExternalObjects(Private, EPropertyLocation::Default, PrivatePropertyParams);

	// State Selector Widget
	VisibilitySettingsCategory.AddCustomRow(LOCTEXT("CustomizableInstanceDetails_StateSelector", "State"))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("StateSelector_Text","State"))
		.ToolTipText(LOCTEXT("StateSelector_Tooltip","Select a state."))
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		GenerateStateSelector()
	];

	// Only Runtime FParameters Option
	VisibilitySettingsCategory.AddCustomRow( LOCTEXT("CustomizableInstanceDetails_RuntimeParm", "Only Runtime") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString("Only Runtime"))
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		SNew(SCheckBox)
		.IsChecked(CustomInstance->GetPrivate()->bShowOnlyRuntimeParameters ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
		.OnCheckStateChanged(this, &FCustomizableInstanceDetails::OnShowOnlyRuntimeSelectionChanged)
	];
	
	// Only Relevant FParameters Option
	VisibilitySettingsCategory.AddCustomRow( LOCTEXT("CustomizableInstanceDetails_RelevantParam", "Only Relevant") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString("Only Relevant"))
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		SNew(SCheckBox)
		.IsChecked(CustomInstance->GetPrivate()->bShowOnlyRelevantParameters ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
		.OnCheckStateChanged(this, &FCustomizableInstanceDetails::OnShowOnlyRelevantSelectionChanged)
	];
	
	// Show UI sections Option
	VisibilitySettingsCategory.AddCustomRow( LOCTEXT("CustomizableInstanceDetails_UISections", "UI Sections") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString("UI Sections"))
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		SNew(SCheckBox)
		.IsChecked(CustomInstance->GetPrivate()->bShowUISections ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
		.OnCheckStateChanged(this, &FCustomizableInstanceDetails::OnUseUISectionsSelectionChanged)
	];

	
	// Show UI thumbnails Option
	VisibilitySettingsCategory.AddCustomRow( LOCTEXT("CustomizableInstanceDetails_UIThumbnails", "UI Thumbnails") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString("UI Thumbnails"))
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	[
		SNew(SCheckBox)
		.IsChecked(CustomInstance->GetPrivate()->bShowUIThumbnails ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
		.OnCheckStateChanged(this, &FCustomizableInstanceDetails::OnUseUIThumbnailsSelectionChanged)
	];

	// Gameplay filters property
	{
		TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
		if (UCustomizableObjectEditorProperties* EditorProperties = Editor->GetEditorProperties())
		{
			IDetailPropertyRow* FilterPropertyRow = VisibilitySettingsCategory.AddExternalObjectProperty({ EditorProperties }, GET_MEMBER_NAME_CHECKED(UCustomizableObjectEditorProperties, Filter));
			check(FilterPropertyRow);
			FilterPropertyRow->GetPropertyHandle()->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableInstanceDetails::Refresh));
		
			Filter = EditorProperties->Filter.GameplayTagsFilter;
			FilterType = EditorProperties->Filter.GameplayTagsFilterType;
		}
	}


	// Copy, Paste and Reset FParameters
	ParametersCategory.AddCustomRow(LOCTEXT("CustomizableInstanceDetails_CopyPasteResetButtons", "Copy Paste Reset"))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(2.0f)
		.Padding(0.0f, 5.0f, 0.0f, 5.0f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("Copy_Parameters", "Copy Parameters"))
			.OnClicked(this, &FCustomizableInstanceDetails::OnCopyAllParameters)
			.IsEnabled(CustomInstance->HasAnyParameters())
			.ToolTipText(FText::FromString(FString("Copy the preview Instance parameters")))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(2.0f)
		.Padding(0.0f, 5.0f, 0.0f, 5.0f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("Paste_Parameters", "Paste Parameters"))
			.OnClicked(this, &FCustomizableInstanceDetails::OnPasteAllParameters)
			.IsEnabled(CustomInstance->HasAnyParameters())
			.ToolTipText(FText::FromString(FString("Paste the preview Instance parameters")))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(2.0f)
		.Padding(0.0f, 5.0f, 0.0f, 5.0f)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.Text(LOCTEXT("Reset_Integer_Parameters", "Reset parameters"))
			.OnClicked(this, &FCustomizableInstanceDetails::OnResetAllParameters)
			.IsEnabled(CustomInstance->HasAnyParameters())
			.ToolTipText(FText::FromString(FString("Clear the preview Instance parameters")))
		]
	];

	// FParameters Widgets
	if (bool bHiddenParamsRuntime = GenerateParametersView(ParametersCategory))
	{
		FText HiddenParamsRuntimeMessage = LOCTEXT("CustomizableInstanceDetails_HiddenParamsRuntime", "Parameters are hidden due to their Runtime type. \nUncheck the Only Runtime checkbox to see them.");

		ParametersCategory.AddCustomRow(LOCTEXT("CustomizableInstanceDetails_HiddenParamsRuntimeRow", "Parameters are hidden"))
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(2.0f)
				.Padding(0.0f, 15.0f, 0.0f, 5.0f)
				[
					SNew(STextBlock)
						.Text(HiddenParamsRuntimeMessage)
						.ToolTipText(HiddenParamsRuntimeMessage)
						.AutoWrapText(true)
				]
		];
	}
}


void FCustomizableInstanceDetails::Refresh() const
{
	if (IDetailLayoutBuilder* Layout = LayoutBuilder.Pin().Get()) // Raw because we don't want to keep alive the details builder when calling the force refresh details
	{
		Layout->ForceRefreshDetails();
	}
}


void FCustomizableInstanceDetails::UpdateInstance()
{
	if (CustomInstance->GetPrivate()->UpdateLogger)
	{
		// Clear the States tab before loging all the messages
		CustomInstance->GetPrivate()->UpdateLogger->ClearLogMessageList();
	}

	CustomInstance->UpdateSkeletalMeshAsync(true, true);
	CustomInstance->PostEditChange();
}


void FCustomizableInstanceDetails::InstanceUpdated(UCustomizableObjectInstance* Instance) const
{
	// Check the instance update context to aboid unecessary UI updates.
	if (!bUpdatingSlider)
	{
		Refresh();
	}
}


void FCustomizableInstanceDetails::ObjectCompiled()
{
	if (!bUpdatingSlider)
	{
		Refresh();
	}
}


// STATE SELECTOR -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateStateSelector()
{
	UCustomizableObject* CustomizableObject = CustomInstance->GetCustomizableObject();

	// States selector options
	const int NumStates = CustomizableObject->GetStateCount();
	const int CurrentState = CustomInstance->GetPrivate()->GetState();
	TSharedPtr<FString> CurrentStateName = nullptr;
	
	//I think that this is not necessary. There is always a "Default" state
	if (NumStates == 0)
	{
		StateNames.Add(MakeShareable(new FString("Default")));
		CurrentStateName = StateNames.Last();
	}

	for (int StateIndex = 0; StateIndex < NumStates; ++StateIndex)
	{
		if (StateIndex == CurrentState)
		{
			CurrentStateName = MakeShareable(new FString(CustomizableObject->GetPrivate()->GetStateName(StateIndex)));
			StateNames.Add(CurrentStateName);
		}
		else
		{
			StateNames.Add(MakeShareable(new FString(CustomizableObject->GetPrivate()->GetStateName(StateIndex))));
		}
	}

	StateNames.Sort([](const TSharedPtr<FString>& LHS, const TSharedPtr<FString>& RHS)
	{
		return *LHS < *RHS;
	});

	return SNew(STextComboBox)
		.OptionsSource(&StateNames)
		.InitiallySelectedItem((CustomInstance->GetPrivate()->GetState() != -1) ? CurrentStateName : StateNames[0])
		.OnSelectionChanged(this, &FCustomizableInstanceDetails::OnStateComboBoxSelectionChanged);
}


void FCustomizableInstanceDetails::OnStateComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		{
			FScopedTransaction LocalTransaction(LOCTEXT("OnStateSelectionChanged", "Change State"));
			CustomInstance->Modify();
			CustomInstance->SetCurrentState(*Selection);
		}

		UpdateInstance();
	}
}


void FCustomizableInstanceDetails::OnShowOnlyRuntimeSelectionChanged(ECheckBoxState InCheckboxState)
{
	CustomInstance->GetPrivate()->bShowOnlyRuntimeParameters = InCheckboxState == ECheckBoxState::Checked;
	Refresh();
}


void FCustomizableInstanceDetails::OnShowOnlyRelevantSelectionChanged(ECheckBoxState InCheckboxState)
{
	CustomInstance->GetPrivate()->bShowOnlyRelevantParameters = InCheckboxState == ECheckBoxState::Checked;
	Refresh();
}


void FCustomizableInstanceDetails::OnUseUISectionsSelectionChanged(ECheckBoxState InCheckboxState)
{
	CustomInstance->GetPrivate()->bShowUISections = InCheckboxState == ECheckBoxState::Checked;
	Refresh();
}

void FCustomizableInstanceDetails::OnUseUIThumbnailsSelectionChanged(ECheckBoxState InCheckboxState)
{
	CustomInstance->GetPrivate()->bShowUIThumbnails = InCheckboxState == ECheckBoxState::Checked;
	Refresh();
}


// PARAMETER INFO STRUCT -----------------------------------------------------------------------------------------------------------------

struct FParameterInfo
{
	int32 ParamIndexInObject = -1;
	int32 ParamUIOrder = 0;
	FString ParamName = "";

	bool operator==(const FParameterInfo& Other)
	{
		return ParamName == Other.ParamName;
	}

	friend bool operator<(const FParameterInfo& A, const FParameterInfo& B)
	{
		if (A.ParamUIOrder != B.ParamUIOrder)
		{
			return A.ParamUIOrder < B.ParamUIOrder;
		}
		else
		{
			return A.ParamName < B.ParamName;
		}
	}
};


// PARAMETERS WIDGET GENERATION -----------------------------------------------------------------------------------------------------------------

void FCustomizableInstanceDetails::SortParametersAndUISections(const UCustomizableObject& CustomizableObject, TArray<FParameterInfo>& ParametersToSort) const
{
	// Sort the parameter tree not based in the type defined sorting behaviour but using for it the name of the section that the parameters are part of
	ParametersToSort.Sort([&CustomizableObject, DefaultSectionName = this->DefaultSectionName](const FParameterInfo& A,const FParameterInfo& B)
	{
		const FMutableParamUIMetadata AParamIMetadata = CustomizableObject.GetParameterUIMetadata(A.ParamName);
		const FMutableParamUIMetadata BParamIMetadata = CustomizableObject.GetParameterUIMetadata(B.ParamName);

		const FString AParameterSectionName = !AParamIMetadata.UISectionName.IsEmpty() ? AParamIMetadata.UISectionName : DefaultSectionName;
		const FString BParameterSectionName = !BParamIMetadata.UISectionName.IsEmpty() ? BParamIMetadata.UISectionName : DefaultSectionName;
		
		return AParameterSectionName < BParameterSectionName;
	});

	// Now that the parameters are sorted by section make sure each section has its parameters sorted
	
	// Key is section name and the amount of parameters of this section
	TMap<FString, uint32> SectionParameterMap;
	for (const FParameterInfo& ParameterInfo : ParametersToSort)
	{
		FString ParamSectionName = CustomizableObject.GetParameterUIMetadata(ParameterInfo.ParamName).UISectionName;
		
		// Sections with no name should use the default Section Name.
		if (ParamSectionName.IsEmpty())
		{
			ParamSectionName = DefaultSectionName;
		}
		
		if (SectionParameterMap.Contains(ParamSectionName))
		{
			SectionParameterMap[ParamSectionName] ++;
		}
		else
		{
			SectionParameterMap.Add(ParamSectionName, 1);
		}
	}

	// Locate the section of parameters that need to be sorted from the ones found in the actual UI section
	uint32 SectionStartIndex = 0;
	for (const TPair<FString, uint32>& SectionAndParamIndex : SectionParameterMap)
	{
		const uint32 SectionParametersCount = SectionAndParamIndex.Value;

		// Capture the parameters of the section so we can sort them
		TArray<FParameterInfo> SectionParameters;
		for (uint32 ParamIndex = SectionStartIndex; ParamIndex < SectionStartIndex + SectionParametersCount; ++ParamIndex)
		{
			SectionParameters.Add(ParametersToSort[ParamIndex]);
		}

		// Sort the section parameters
		SectionParameters.Sort();

		// Replace the unsorted (ParametersToSort) parameters with the sorted ones (SectionParameters)
		int32 SectionParametersIndex = 0;
		for (uint32 ParamIndex = SectionStartIndex; ParamIndex < SectionStartIndex + SectionParametersCount; ++ParamIndex)
		{
			ParametersToSort[ParamIndex] = SectionParameters[SectionParametersIndex++];
		}
		
		// Move the starting index forward for the next section
		SectionStartIndex += SectionParametersCount;
	}
}


bool FCustomizableInstanceDetails::GenerateParametersView(IDetailCategoryBuilder& DetailsCategory)
{
	ParamChildren.Empty();
	ParamHasParent.Empty();
	GeneratedSections.Empty();
	DynamicBrushes.Empty();

	TArray<FParameterInfo> ParametersTree;
	UCustomizableObject* CustomizableObject = CustomInstance->GetCustomizableObject();

	if (!CustomizableObject)
	{
		return false;
	}

	bool bParametersHiddenRuntime = false;

	if (CustomInstance->GetPrivate()->bShowOnlyRuntimeParameters)
	{
		const int32 NumStateParameters = CustomizableObject->GetStateParameterCount(CustomInstance->GetCurrentState());

		if (NumStateParameters < CustomizableObject->GetParameterCount())
		{
			bParametersHiddenRuntime = true;
		}

		for (int32 ParamIndexInState = 0; ParamIndexInState < NumStateParameters; ++ParamIndexInState)
		{
			FParameterInfo ParameterSortInfo;
			ParameterSortInfo.ParamIndexInObject = CustomizableObject->GetPrivate()->GetStateParameterIndex(CustomInstance->GetPrivate()->GetState(), ParamIndexInState);

			if (CustomInstance->GetPrivate()->IsParameterRelevant(ParameterSortInfo.ParamIndexInObject) && IsVisible(ParameterSortInfo.ParamIndexInObject))
			{
				ParameterSortInfo.ParamName = CustomizableObject->GetParameterName(ParameterSortInfo.ParamIndexInObject);
				ParameterSortInfo.ParamUIOrder = CustomizableObject->GetParameterUIMetadata(ParameterSortInfo.ParamName).UIOrder;
				ParametersTree.Add(ParameterSortInfo);
			}
		}

		if (CustomInstance->GetPrivate()->bShowUISections)
		{
			// Sort the parameter tree using for it the name of the section that the parameters are part of
			SortParametersAndUISections(*CustomizableObject, ParametersTree);
		}
		else
		{
			// Sort the parameters using for it the UIOrder and the name of them.
			ParametersTree.Sort();
		}

		for (int32 ParamIndex = 0; ParamIndex < ParametersTree.Num(); ++ParamIndex)
		{
			const FParameterInfo& ParamInfo = ParametersTree[ParamIndex];

			if (CustomInstance->GetPrivate()->bShowUISections)
			{
				IDetailGroup* CurrentSection = GenerateParameterSection(DetailsCategory, *CustomizableObject, ParamInfo.ParamName);
				check(CurrentSection);

				if (!IsMultidimensionalProjector(ParamInfo.ParamIndexInObject))
				{
					GenerateWidgetRow(CurrentSection->AddWidgetRow(), *CustomizableObject, ParamInfo.ParamName, ParamInfo.ParamIndexInObject);
				}
				else
				{
					IDetailGroup* ProjectorGroup = &CurrentSection->AddGroup(FName(*ParamInfo.ParamName), FText::GetEmpty());

					// Call Order between the following lines maters.
					ParentsGroups.Add(ParamInfo.ParamName, ProjectorGroup);
					GenerateWidgetRow(ProjectorGroup->HeaderRow(), *CustomizableObject, ParamInfo.ParamName, ParamInfo.ParamIndexInObject);
				}
			}
			else
			{
				if (!IsMultidimensionalProjector(ParamInfo.ParamIndexInObject))
				{
					GenerateWidgetRow(DetailsCategory.AddCustomRow(FText::FromString(ParamInfo.ParamName)), *CustomizableObject, ParamInfo.ParamName, ParamInfo.ParamIndexInObject);
				}
				else
				{
					IDetailGroup* ProjectorGroup = &DetailsCategory.AddGroup(FName(*ParamInfo.ParamName), FText::GetEmpty());

					// Call Order between the following lines maters.
					ParentsGroups.Add(ParamInfo.ParamName, ProjectorGroup);
					GenerateWidgetRow(ProjectorGroup->HeaderRow(), *CustomizableObject, ParamInfo.ParamName, ParamInfo.ParamIndexInObject);
				}
			}
		}
	}
	else
	{
		const int32 NumObjectParameter = CustomizableObject->GetParameterCount();

		//TODO: get all parameters and sort, then make the next "for" use that sorted list as source of indexes
		for (int32 ParamIndexInObject = 0; ParamIndexInObject < NumObjectParameter; ++ParamIndexInObject)
		{
			FParameterInfo ParameterSortInfo;
			ParameterSortInfo.ParamIndexInObject = ParamIndexInObject;
			if ((!CustomInstance->GetPrivate()->bShowOnlyRelevantParameters || CustomInstance->GetPrivate()->IsParameterRelevant(ParameterSortInfo.ParamIndexInObject)) && IsVisible(ParameterSortInfo.ParamIndexInObject))
			{
				ParameterSortInfo.ParamName = CustomizableObject->GetParameterName(ParameterSortInfo.ParamIndexInObject);
				ParameterSortInfo.ParamUIOrder = CustomizableObject->GetParameterUIMetadata(ParameterSortInfo.ParamName).UIOrder;
				ParametersTree.Add(ParameterSortInfo);
			}
		}
		
		if (CustomInstance->GetPrivate()->bShowUISections)
		{
			// Sort the parameter tree using for it the name of the section that the parameters are part of
			SortParametersAndUISections(*CustomizableObject, ParametersTree);
		}
		else
		{
			// Sort the parameters using for it the UIOrder and the name of them.
			ParametersTree.Sort();
		}
		
		for (int32 ParamIndexInObject = 0; ParamIndexInObject < ParametersTree.Num(); ++ParamIndexInObject)
		{
			FillChildrenMap(ParametersTree[ParamIndexInObject].ParamIndexInObject);
		}

		for (int32 ParamIndexInObject = 0; ParamIndexInObject < ParametersTree.Num(); ++ParamIndexInObject)
		{
			if (!ParamHasParent.Find(ParametersTree[ParamIndexInObject].ParamIndexInObject))
			{
				RecursivelyAddParamAndChildren(*CustomizableObject, ParametersTree[ParamIndexInObject].ParamIndexInObject, "", DetailsCategory);
			}
		}
	}

	return bParametersHiddenRuntime;
}


void FCustomizableInstanceDetails::RecursivelyAddParamAndChildren(const UCustomizableObject& CustomizableObject, const int32 ParamIndexInObject, const FString ParentName, IDetailCategoryBuilder& DetailsCategory)
{
	const FString ParamName = CustomizableObject.GetParameterName(ParamIndexInObject);	
	TArray<int32> Children;
	
	ParamChildren.MultiFind(ParamName, Children, true);

	if (ParentName.IsEmpty())
	{
		if (CustomInstance->GetPrivate()->bShowUISections)
		{
			IDetailGroup* CurrentSection = GenerateParameterSection(DetailsCategory, CustomizableObject, ParamName);
			check(CurrentSection);

			if (Children.Num() == 0 && !IsMultidimensionalProjector(ParamIndexInObject))
			{
				GenerateWidgetRow(CurrentSection->AddWidgetRow(), CustomizableObject, ParamName, ParamIndexInObject);
			}
			else
			{
				IDetailGroup* ParentGroup = &CurrentSection->AddGroup(FName(*ParamName), FText::GetEmpty());
				
				// Call Order between the following lines maters.
				ParentsGroups.Add(ParamName, ParentGroup);
				GenerateWidgetRow(ParentGroup->HeaderRow(), CustomizableObject, ParamName, ParamIndexInObject);
			}
		}
		else
		{
			if (Children.Num() == 0 && !IsMultidimensionalProjector(ParamIndexInObject))
			{
				GenerateWidgetRow(DetailsCategory.AddCustomRow(FText::FromString(ParamName)), CustomizableObject, ParamName, ParamIndexInObject);
			}
			else
			{
				IDetailGroup* ParentGroup = &DetailsCategory.AddGroup(FName(*ParamName), FText::GetEmpty());
				
				// Call Order between the following lines maters.
				ParentsGroups.Add(ParamName, ParentGroup);
				GenerateWidgetRow(ParentGroup->HeaderRow(), CustomizableObject, ParamName, ParamIndexInObject);
			}
		}
	}
	else
	{
		IDetailGroup* ParentGroup = *ParentsGroups.Find(ParentName);

		if (Children.Num() == 0 && !IsMultidimensionalProjector(ParamIndexInObject))
		{
			GenerateWidgetRow(ParentGroup->AddWidgetRow(), CustomizableObject, ParamName, ParamIndexInObject);
		}
		else
		{
			IDetailGroup* ChildGroup = &ParentGroup->AddGroup(FName(*ParamName), FText::GetEmpty());

			// Call Order between the following lines maters
			ParentsGroups.Add(ParamName, ChildGroup);
			GenerateWidgetRow(ParentGroup->HeaderRow(), CustomizableObject, ParamName, ParamIndexInObject);
		}
	}

	for (const int32 ChildIndexInObject : Children)
	{
		RecursivelyAddParamAndChildren(CustomizableObject, ChildIndexInObject, ParamName, DetailsCategory);
	}
}


void FCustomizableInstanceDetails::FillChildrenMap(int32 ParamIndexInObject)
{
	const UCustomizableObject* CustomizableObject = CustomInstance->GetCustomizableObject();
	const FString& ParamName = CustomizableObject->GetParameterName(ParamIndexInObject);
	FMutableParamUIMetadata UIMetadata = CustomizableObject->GetParameterUIMetadata(ParamName);

	if (const FString* ParentName = UIMetadata.ExtraInformation.Find(FString("__ParentParamName")))
	{
		if (CustomizableObject->ContainsParameter(*ParentName))
		{
			ParamChildren.Add(*ParentName, ParamIndexInObject);
			ParamHasParent.Add(ParamIndexInObject, true);
		}
	}
}


bool FCustomizableInstanceDetails::IsVisible(int32 ParamIndexInObject)
{
	const UCustomizableObject* CustomizableObject = CustomInstance->GetCustomizableObject();
	const FString& ParamName = CustomizableObject->GetParameterName(ParamIndexInObject);
	FMutableParamUIMetadata UIMetadata = CustomizableObject->GetParameterUIMetadata(ParamName);
	const FString* ParentName = UIMetadata.ExtraInformation.Find(FString("__ParentParamName"));

	const bool bIsAProjectorParam = ParamName.EndsWith(NUM_LAYERS_PARAMETER_POSTFIX)
		|| (ParamName.EndsWith(IMAGE_PARAMETER_POSTFIX) && CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParamIndexInObject))
		|| (ParamName.EndsWith(OPACITY_PARAMETER_POSTFIX) && CustomizableObject->GetPrivate()->IsParameterMultidimensional(ParamIndexInObject))
		|| (ParamName.EndsWith(POSE_PARAMETER_POSTFIX));

	if (bIsAProjectorParam)
	{
		return false;
	}

	if (ParentName)
	{
		if (!CustomizableObject->ContainsParameter(*ParentName))
		{
			return true; // Visible since Parent parameter got optimized out (Always Enabled)
		}

		if (CustomInstance->GetPrivate()->bShowOnlyRelevantParameters)
		{
			const FString* Value = UIMetadata.ExtraInformation.Find(FString("__DisplayWhenParentValueEquals"));

			bool bParentIsBoolParam = CustomizableObject->GetParameterTypeByName(*ParentName) == EMutableParameterType::Bool;

			FString SelectedOption = bParentIsBoolParam ?
				(CustomInstance->GetBoolParameterSelectedOption(*ParentName) ? FString("1") : FString("0")) :
				CustomInstance->GetIntParameterSelectedOption(*ParentName);

			if (Value && CustomizableObject->GetPrivate()->FindParameter(*ParentName) != INDEX_NONE && SelectedOption != *Value)
			{
				return false;
			}
		}
	}

	return true;
}


bool FCustomizableInstanceDetails::IsIntParameterFilteredOut(const UCustomizableObject& CustomizableObject, const FString& ParamName, const FString& ParamOption) const
{
	if (Filter.IsEmpty())
	{
		return false;
	}

	FMutableParamUIMetadata Metadata = CustomizableObject.GetEnumParameterValueUIMetadata(ParamName, ParamOption);

	switch (FilterType)
	{
	case EGameplayContainerMatchType::Any:
	{
		return !Metadata.GameplayTags.HasAny(Filter);
	}
	case EGameplayContainerMatchType::All:
	{
		return !Metadata.GameplayTags.HasAll(Filter);
	}
	default:
		unimplemented();
		return false;
	}
}



bool FCustomizableInstanceDetails::IsMultidimensionalProjector(int32 ParamIndexInObject)
{
	const UCustomizableObject* CustomizableObject = CustomInstance->GetCustomizableObject();

	return CustomizableObject->GetPrivate()->GetParameterType(ParamIndexInObject) == EMutableParameterType::Projector && CustomInstance->GetCustomizableObject()->GetPrivate()->IsParameterMultidimensional(ParamIndexInObject);
}


IDetailGroup* FCustomizableInstanceDetails::GenerateParameterSection(IDetailCategoryBuilder& DetailsCategory, const UCustomizableObject& CustomizableObject, const FString& ParamName)
{
	const FMutableParamUIMetadata UIMetadata = CustomizableObject.GetParameterUIMetadata(ParamName);
	const FString SectionName = UIMetadata.UISectionName.IsEmpty() ? DefaultSectionName : UIMetadata.UISectionName;
	IDetailGroup* CurrentSection = nullptr;

	if (GeneratedSections.Contains(SectionName))
	{
		CurrentSection = *GeneratedSections.Find(SectionName);
	}

	if (!CurrentSection)
	{
		CurrentSection = &DetailsCategory.AddGroup(FName(SectionName), FText::FromString(SectionName));

		GeneratedSections.Add(SectionName, CurrentSection);
	}

	return CurrentSection;
}


void FCustomizableInstanceDetails::GenerateWidgetRow(FDetailWidgetRow& WidgetRow, const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject)
{
	WidgetRow.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString(ParamName))
	]
	.ValueContent()
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	.VAlign(EVerticalAlignment::VAlign_Fill)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.Padding(0.0f, 5.0f, 0.0f, 5.0f)
		[
			GenerateParameterWidget(CustomizableObject, ParamName, ParamIndexInObject)
		]
	]
	.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableInstanceDetails::OnResetParameterButtonClicked, ParamIndexInObject)))
	.FilterString(FText::FromString(ParamName));
}


TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateParameterWidget(const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject)
{
	switch (CustomizableObject.GetPrivate()->GetParameterType(ParamIndexInObject))
	{
		case EMutableParameterType::Bool:
		{
			return GenerateBoolWidget(ParamName);
		}
		case EMutableParameterType::Float:
		{
			return GenerateFloatWidget(CustomizableObject, ParamName);
		}
		case EMutableParameterType::Color:
		{
			return GenerateColorWidget(ParamName);
		}
		case EMutableParameterType::Texture:
		{
			UTexture* Texture = CustomInstance->GetTextureParameterSelectedOption(ParamName);
			const FString ObjectPath = Texture ? Texture->GetPathName() : TEXT("");

			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(UTexture2D::StaticClass())
				.OnObjectChanged(this, &FCustomizableInstanceDetails::OnTextureParameterSelected, ParamName)
				.ObjectPath(ObjectPath);
		}
		case EMutableParameterType::SkeletalMesh:
		{
			USkeletalMesh* SkeletalMesh = CustomInstance->GetSkeletalMeshParameterSelectedOption(ParamName);
			const FString ObjectPath = SkeletalMesh ? SkeletalMesh->GetPathName() : TEXT("");

			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(USkeletalMesh::StaticClass())
				.OnObjectChanged(this, &FCustomizableInstanceDetails::OnSkeletalMeshParameterSelected, ParamName)
				.ObjectPath(ObjectPath);
		}
		case EMutableParameterType::Material:
		{
			UMaterialInterface* Material = CustomInstance->GetMaterialParameterSelectedOption(ParamName);
			const FString ObjectPath = Material ? Material->GetPathName() : TEXT("");

			return SNew(SObjectPropertyEntryBox)
				.AllowedClass(UMaterialInterface::StaticClass())
				.OnObjectChanged(this, &FCustomizableInstanceDetails::OnMaterialParameterSelected, ParamName)
				.ObjectPath(ObjectPath);
		}
		case EMutableParameterType::InstancedStruct:
		{
			FInstancedStruct InstancedStruct = CustomInstance->GetExternalTypeParameterSelectedOption(ParamName);

			const UScriptStruct* ScriptStruct = InstancedStruct.GetScriptStruct();
				
			TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(ScriptStruct);

			ScriptStruct->CopyScriptStruct(StructOnScope->GetStructMemory(), InstancedStruct.GetMemory());

			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs DetailViewArgs;
			DetailViewArgs.bAllowSearch = false;
			DetailViewArgs.bShowScrollBar = true;
			DetailViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

			FStructureDetailsViewArgs StructureDetailsViewArgs;
			StructureDetailsViewArgs.bShowObjects = false;
			
			TSharedRef<IStructureDetailsView> Detailsview = PropertyEditorModule.CreateStructureDetailView(DetailViewArgs, StructureDetailsViewArgs, StructOnScope);
			Detailsview->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &FCustomizableInstanceDetails::OnExternalTypeParameterChanged, StructOnScope, ParamName, -1);

			return Detailsview->GetWidget().ToSharedRef();
		}
		case EMutableParameterType::Transform:
		{
			return GenerateTransformWidget(ParamName);
		}
		case EMutableParameterType::Projector:
		{
			bool bIsParamMultidimensional = CustomInstance->GetCustomizableObject()->GetPrivate()->IsParameterMultidimensional(ParamIndexInObject);

			if (!bIsParamMultidimensional)
			{
				return GenerateSimpleProjector(ParamName);
			}
			else 
			{
				return GenerateMultidimensionalProjector(CustomizableObject, ParamName, ParamIndexInObject);
			}
		}
		case EMutableParameterType::Int:
		{
			return GenerateIntWidget(CustomizableObject, ParamName, ParamIndexInObject);
		}
		case EMutableParameterType::None:
		{
			return SNew(STextBlock).Text(LOCTEXT("ParameterTypeNotSupported_Text", "Parameter Type not supported"));
		}
	}

	return SNew(STextBlock);
}


// INT PARAMETERS -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateIntWidget(const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject)
{
	const bool bMultidimensional = CustomInstance->GetCustomizableObject()->GetPrivate()->IsParameterMultidimensional(ParamIndexInObject);
	if (bMultidimensional)
	{
		return SNew(STextBlock).Text(LOCTEXT("MultidimensionalINTParameter_Text", "Multidimensional INT Parameter not supported"));
	}

	const int32 NumValues = CustomizableObject.GetPrivate()->GetEnumParameterNumValues(ParamIndexInObject);
	if (!NumValues)
	{
		return SNew(STextBlock).Text(LOCTEXT("NoAvailableOptions", "No Available Options"));
	}
	
	TSharedPtr<TArray<TSharedPtr<FString>>>* FoundOptions = IntParameterOptions.Find(ParamIndexInObject);
	TArray<TSharedPtr<FString>>& OptionNamesAttribute = FoundOptions && FoundOptions->IsValid() ? 
														*FoundOptions->Get() :
														*(IntParameterOptions.Add(ParamIndexInObject, MakeShared<TArray<TSharedPtr<FString>>>()).Get());

	OptionNamesAttribute.Empty();

	const FString SelectedOption = CustomInstance->GetIntParameterSelectedOption(ParamName, INDEX_NONE);

	// Tooltip for the selected option
	FString ToolTipText = FString("None");
	if (const UModelResources* ModelResources = CustomizableObject.GetPrivate()->GetModelResources())
	{
		if (const FString* Identifier = ModelResources->GroupNodeMap.FindKey(FCustomizableObjectIdPair(ParamName, SelectedOption)))
		{
			if (const FString* CustomizableObjectPath = ModelResources->CustomizableObjectPathMap.Find(*Identifier))
			{
				ToolTipText = *CustomizableObjectPath;
			}
		}
	}

	TSharedPtr<FString> SelectedOptionString;
	for (int32 i = 0; i < NumValues; ++i)
	{
		const FString PossibleValue = CustomizableObject.GetPrivate()->GetIntParameterAvailableOption(ParamIndexInObject, i);

		if (PossibleValue == SelectedOption) // Always add the selected option, even if it should be filtered.
		{
			SelectedOptionString = TSharedPtr<FString>(new FString(PossibleValue));
			OptionNamesAttribute.Add(SelectedOptionString);
		}

		else if (!IsIntParameterFilteredOut(CustomizableObject, ParamName, PossibleValue))
		{
			OptionNamesAttribute.Add(TSharedPtr<FString>(new FString(PossibleValue)));
		}
	}

	return SNew(SSearchableComboBox)
		.ToolTipText(FText::FromString(ToolTipText))
		.OptionsSource(&OptionNamesAttribute)
		.InitiallySelectedItem(SelectedOptionString)
		.Method(EPopupMethod::UseCurrentWindow)
		.OnSelectionChanged(this, &FCustomizableInstanceDetails::OnIntParameterComboBoxChanged, ParamName)
		.OnGenerateWidget(this, &FCustomizableInstanceDetails::OnGenerateWidgetIntParameter, ParamName)
		.Content()
		[
			SNew(STextBlock)
				.Text(FText::FromString(*SelectedOption))
		];
}


TSharedRef<SWidget> FCustomizableInstanceDetails::OnGenerateWidgetIntParameter(TSharedPtr<FString> OptionName, const FString ParameterName)
{
	// Final widget
	TSharedPtr<SHorizontalBox> IntWidgetBox = SNew(SHorizontalBox);

	if (CustomInstance->GetPrivate()->bShowUIThumbnails)
	{
		bool bUsesCustomThumbnail = false;

		// Asset with the thumbnail info
		FAssetData AssetData;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		// Metadata of the represented int option
		const FMutableParamUIMetadata ParameterMetadata = CustomInstance->GetCustomizableObject()->GetEnumParameterValueUIMetadata(ParameterName, *OptionName);

		// Custom thumbnail has preference
		if (!ParameterMetadata.UIThumbnail.IsNull())
		{
			// Custom Thumbnail
			AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ParameterMetadata.UIThumbnail.ToSoftObjectPath());

			if (AssetData.IsValid())
			{
				if (UTexture2D* UIThumbnail = Cast<UTexture2D>(UE::Mutable::Private::LoadObject(AssetData)))
				{
					// We need to store the generated texture.
					TSharedRef<FDeferredCleanupSlateBrush> Brush = FDeferredCleanupSlateBrush::CreateBrush(UIThumbnail, FVector2D(68.f, 68.0f), /* Texture size (64) + padding (4) */
						FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), ESlateBrushTileType::NoTile, ESlateBrushImageType::Linear);
					DynamicBrushes.Add(Brush);

					IntWidgetBox->AddSlot().AutoWidth()
					[
						SNew(SImage).Image(Brush->GetSlateBrush())
					];

					bUsesCustomThumbnail = true;
				}
			}
		}

		if (!bUsesCustomThumbnail)
		{
			// Asset thumbnail
			AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ParameterMetadata.EditorUIThumbnailObject.ToSoftObjectPath());

			// We don't need to check if the asset data is valid here. We want to use the default thumbnail if there is no asset data.
			TSharedPtr<FAssetThumbnail> Thumbnail = MakeShareable(new FAssetThumbnail(AssetData, 64, 64, UThumbnailManager::Get().GetSharedThumbnailPool()));
			FAssetThumbnailConfig ThumbnailConfig;

			ThumbnailConfig.ColorStripOrientation = EThumbnailColorStripOrientation::VerticalRightEdge;
			ThumbnailConfig.BorderPadding = FMargin(2.0f); // Prevents overlap with rounded corners; this matches what the Content Browser tiles do

			IntWidgetBox->AddSlot().AutoWidth()
			[
				Thumbnail->MakeThumbnailWidget(ThumbnailConfig)
			];

			if (!AssetData.IsValid())
			{
				IntWidgetBox->GetSlot(0).SetFillWidth(2.0f);
				IntWidgetBox->GetSlot(0).SetMaxSize(68.0f);
			}
		}
	}

	IntWidgetBox->AddSlot().VAlign(EVerticalAlignment::VAlign_Center).Padding(10.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(STextBlock).Text(FText::FromString(*OptionName.Get()))
	];

	return IntWidgetBox.ToSharedRef();
}


void FCustomizableInstanceDetails::OnIntParameterComboBoxChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, FString ParamName)
{
	if (Selection.IsValid())
	{
		{
			FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("OnIntParameterSet", "Set Int Parameter: {0}"), FText::FromString(ParamName)));
			CustomInstance->Modify();
			CustomInstance->SetIntParameterSelectedOption(ParamName, *Selection);
		}

		UpdateInstance();
	}
}


// FLOAT PARAMETERS -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateFloatWidget(const UCustomizableObject& CustomizableObject, const FString& ParamName)
{
	const FMutableParamUIMetadata& UIMetadata = CustomizableObject.GetParameterUIMetadata(ParamName);

	if (const TSoftObjectPtr<UObject>* FloatDecoratorAsset = UIMetadata.ExtraAssets.Find(UIMetadataKeyWords::FloatDecoratorName))
	{
		// Checking if there is an image decorator for the float slider:
		if (UTexture2D* DecoratorTexture = Cast<UTexture2D>(UE::Mutable::Private::LoadObject(*FloatDecoratorAsset)))
		{
			TSharedRef<FDeferredCleanupSlateBrush> Brush = FDeferredCleanupSlateBrush::CreateBrush(DecoratorTexture, FVector2D(DecoratorTexture->GetSizeX(), 1.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), ESlateBrushTileType::NoTile, ESlateBrushImageType::Linear);
			DynamicBrushes.Add(Brush);

			// Add slider with the decorator as background
			return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SBox)
				.Padding(0.0f, 8.0f) //Helps to shrinks the image from the borders
				[
					SNew(SImage)
					.Image(Brush->GetSlateBrush())
				]
			]

			+ SOverlay::Slot()
			[
				SNew(SSlider)
				.Value(this, &FCustomizableInstanceDetails::GetFloatParameterValue, ParamName, -1)
				.MinValue(UIMetadata.MinimumValue)
				.MaxValue(UIMetadata.MaximumValue)
				.OnValueChanged(this, &FCustomizableInstanceDetails::OnFloatParameterChanged, ParamName, -1)
				.OnMouseCaptureBegin(this, &FCustomizableInstanceDetails::OnFloatParameterSliderBegin, ParamName, -1)
				.OnMouseCaptureEnd(this, &FCustomizableInstanceDetails::OnFloatParameterSliderEnd, ParamName, -1)
				.Style(&FAppStyle::Get().GetWidgetStyle<FSliderStyle>("ColorPicker.Slider"))
				.IndentHandle(false)
				.SliderBarColor(FLinearColor::Transparent)
			];
		}
	}

	return SNew(SSpinBox<float>)
		.Value(this, &FCustomizableInstanceDetails::GetFloatParameterValue, ParamName, -1)
		.MinValue(UIMetadata.MinimumValue)
		.MaxValue(UIMetadata.MaximumValue)
		.OnValueChanged(this, &FCustomizableInstanceDetails::OnFloatParameterChanged, ParamName, -1)
		.OnValueCommitted(this, &FCustomizableInstanceDetails::OnFloatParameterCommited, ParamName, -1)
		.OnBeginSliderMovement(this, &FCustomizableInstanceDetails::OnFloatParameterSliderBegin, ParamName, -1);
}


float FCustomizableInstanceDetails::GetFloatParameterValue(FString ParamName, int32 RangeIndex) const
{
	if (CustomInstance->GetCustomizableObject()->GetPrivate()->IsLocked()) // TODO Move, if necessary, to GetFloatParameterSelectedOption. UE-224815
	{
		// Prevent crashing if polling the float value during CO compilation
		return -1.f;
	}

	if (RangeIndex == INDEX_NONE)
	{
		return CustomInstance->GetFloatParameterSelectedOption(ParamName, RangeIndex);
	}
	else //multidimensional
	{
		// We may have deleted a range but the Instance has not been updated yet
		if (CustomInstance->GetFloatValueRange(ParamName) > RangeIndex)
		{
			return CustomInstance->GetFloatParameterSelectedOption(ParamName, RangeIndex);
		}
	}

	return 0;
}


void FCustomizableInstanceDetails::OnFloatParameterChanged(float Value, FString ParamName, int32 RangeIndex)
{
	float OldValue = CustomInstance->GetFloatParameterSelectedOption(ParamName, RangeIndex);

	if (OldValue != Value)
	{
		CustomInstance->SetFloatParameterSelectedOption(ParamName, Value, RangeIndex);
		UpdateInstance();
	}
}


void FCustomizableInstanceDetails::OnFloatParameterSliderBegin(const FString ParamName, int32 RangeIndex)
{
	bUpdatingSlider = true;
	SliderStartValue = CustomInstance->GetFloatParameterSelectedOption(ParamName, RangeIndex);
}


void FCustomizableInstanceDetails::OnFloatParameterSliderEnd(const FString ParamName, int32 RangeIndex)
{
	float Value = CustomInstance->GetFloatParameterSelectedOption(ParamName, RangeIndex);
	
	if (bUpdatingSlider)
	{
		CustomInstance->SetFloatParameterSelectedOption(ParamName, SliderStartValue, RangeIndex); // To be able to undo before the slide movement, before doing the transaction set the value to that original slider position.
	}
	
	{
		FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("OnFloatParameterCommited", "Set Float Parameter: {0}"), FText::FromString(ParamName)));
		CustomInstance->Modify();
		CustomInstance->SetFloatParameterSelectedOption(ParamName, Value, RangeIndex);
	}

	UpdateInstance();
	
	bUpdatingSlider = false;
}


void FCustomizableInstanceDetails::OnFloatParameterCommited(float Value, ETextCommit::Type Type, const FString ParamName, int32 RangeIndex)
{
	float OldValue = CustomInstance->GetFloatParameterSelectedOption(ParamName, RangeIndex);

	if (OldValue != Value)
	{
		if (bUpdatingSlider)
		{
			CustomInstance->SetFloatParameterSelectedOption(ParamName, SliderStartValue, RangeIndex); // To be able to undo before the slide movement, before doing the transaction set the value to that original slider position.
		}
	
		{
			FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("OnFloatParameterCommited", "Set Float Parameter: {0}"), FText::FromString(ParamName)));
			CustomInstance->Modify();
			CustomInstance->SetFloatParameterSelectedOption(ParamName, Value, RangeIndex);
		}

		UpdateInstance();
	}
	
	bUpdatingSlider = false;
}

// TEXTURE PARAMETERS -----------------------------------------------------------------------------------------------------------------

void FCustomizableInstanceDetails::OnTextureParameterSelected(const FAssetData& AssetData, const FString ParamName)
{
	CustomInstance->SetTextureParameterSelectedOption(ParamName, Cast<UTexture2D>(AssetData.GetAsset()));
	UpdateInstance();
}


// SKELETAL MESH PARAMETERS -----------------------------------------------------------------------------------------------------------------

void FCustomizableInstanceDetails::OnSkeletalMeshParameterSelected(const FAssetData& AssetData, const FString ParamName)
{
	CustomInstance->SetSkeletalMeshParameterSelectedOption(ParamName, Cast<USkeletalMesh>(AssetData.GetAsset()));
	UpdateInstance();
}


// MATERIAL PARAMETERS -----------------------------------------------------------------------------------------------------------------

void FCustomizableInstanceDetails::OnMaterialParameterSelected(const FAssetData& AssetData, const FString ParamName)
{
	CustomInstance->SetMaterialParameterSelectedOption(ParamName, Cast<UMaterialInterface>(AssetData.GetAsset()));
	UpdateInstance();
}


// COLOR PARAMETERS -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateColorWidget(const FString& ParamName)
{
	return SNew(SColorBlock)
		.Color(this, &FCustomizableInstanceDetails::GetColorParameterValue, ParamName)
		.ShowBackgroundForAlpha(false)
		.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
		.UseSRGB(true)
		.OnMouseButtonDown(this, &FCustomizableInstanceDetails::OnColorBlockMouseButtonDown, ParamName)
		.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f));
}


FLinearColor FCustomizableInstanceDetails::GetColorParameterValue(FString ParamName) const
{
	return CustomInstance->GetVectorParameterSelectedOption(ParamName);
}


FReply FCustomizableInstanceDetails::OnColorBlockMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FString ParamName)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	FLinearColor Col = GetColorParameterValue(ParamName);

	FColorPickerArgs args;
	args.bIsModal = true;
	args.bUseAlpha = true;
	args.bOnlyRefreshOnMouseUp = false;
	args.InitialColor = Col;
	args.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FCustomizableInstanceDetails::OnSetColorFromColorPicker, ParamName);
	OpenColorPicker(args);

	return FReply::Handled();
}


void FCustomizableInstanceDetails::OnSetColorFromColorPicker(FLinearColor NewColor, FString PickerParamName)
{
	{
		FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("SetColorParameter", "Set Color Parameter: {0}"), FText::FromString(PickerParamName)));
		CustomInstance->Modify();
		CustomInstance->SetVectorParameterSelectedOption(PickerParamName, NewColor);
	}

	UpdateInstance();
}

// TRANSFORM PARAMETERS -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateTransformWidget(const FString& ParamName)
{
	auto OnLocationChanged = [WeakThis = SharedThis(this).ToWeakPtr(), ParamName](double Value, EAxis::Type Axis)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			FTransform Transform = SharedThis->GetTransformParameterValue(ParamName);
			FVector Location = Transform.GetLocation();
			if (!FMath::IsNearlyEqualByULP(Value, Location.GetComponentForAxis(Axis)))
			{
				Location.SetComponentForAxis(Axis, Value);
				Transform.SetLocation(Location);
				SharedThis->OnTransformParameterChanged(Transform, ParamName);
			}
		}
	};
	auto OnLocationCommitted = [WeakThis = SharedThis(this).ToWeakPtr(), ParamName](double Value, ETextCommit::Type Type, EAxis::Type Axis)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			FTransform Transform = SharedThis->GetTransformParameterValue(ParamName);
			FVector Location = Transform.GetLocation();
			if (!FMath::IsNearlyEqualByULP(Value, Location.GetComponentForAxis(Axis)))
			{
				Location.SetComponentForAxis(Axis, Value);
				Transform.SetLocation(Location);
				SharedThis->OnTransformParameterCommitted(Transform, Type, ParamName);
			}
		}
	};

	auto OnRotationChanged = [WeakThis = SharedThis(this).ToWeakPtr(), ParamName](double Value, EAxis::Type Axis)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			FTransform Transform = SharedThis->GetTransformParameterValue(ParamName);
			FRotator Rotation = Transform.Rotator();
			Rotation.SetComponentForAxis(Axis, Value);
		
			if (!Transform.Rotator().Equals(Rotation))
			{
				Transform.SetRotation(Rotation.Quaternion());
				SharedThis->OnTransformParameterChanged(Transform, ParamName);
			}
		}
	};
	auto OnRotationCommitted = [WeakThis = SharedThis(this).ToWeakPtr(), ParamName](double Value, ETextCommit::Type Type, EAxis::Type Axis)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			FTransform Transform = SharedThis->GetTransformParameterValue(ParamName);
			FRotator Rotation = Transform.Rotator();
			Rotation.SetComponentForAxis(Axis, Value);
		
			if (!Transform.Rotator().Equals(Rotation))
			{
				Transform.SetRotation(Rotation.Quaternion());
				SharedThis->OnTransformParameterCommitted(Transform, Type, ParamName);
			}
		}
	};
	
	auto OnScaleChanged = [WeakThis = SharedThis(this).ToWeakPtr(), ParamName](double Value, EAxis::Type Axis)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			FTransform Transform = SharedThis->GetTransformParameterValue(ParamName);
			FVector Scale = Transform.GetScale3D();
			if (!FMath::IsNearlyEqualByULP(Value, Scale.GetComponentForAxis(Axis)))
			{
				Scale.SetComponentForAxis(Axis, Value);
				Transform.SetScale3D(Scale);
				SharedThis->OnTransformParameterChanged(Transform, ParamName);
			}
		}
	};
	auto OnScaleCommitted = [WeakThis = SharedThis(this).ToWeakPtr(), ParamName](double Value, ETextCommit::Type Type, EAxis::Type Axis)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			FTransform Transform = SharedThis->GetTransformParameterValue(ParamName);
			FVector Scale = Transform.GetScale3D();
			if (!FMath::IsNearlyEqualByULP(Value, Scale.GetComponentForAxis(Axis)))
			{
				Scale.SetComponentForAxis(Axis, Value);
				Transform.SetScale3D(Scale);
				SharedThis->OnTransformParameterCommitted(Transform, Type, ParamName);
			}
		}
	};
	
	auto BeginSliderMovement = [WeakThis = SharedThis(this).ToWeakPtr()]()
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin()) 
		{ 
			SharedThis->bUpdatingSlider = true; 
		}
	};

	auto EndSliderMovement = [WeakThis = SharedThis(this).ToWeakPtr()](float NewValue)
	{
		if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
		{
			SharedThis->bUpdatingSlider = false;
		}
	};
	
	return SNew(SGridPanel)
		.FillColumn(1, 1)
		+ SGridPanel::Slot(0, 0)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Transform_Location", "Location"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SGridPanel::Slot(1, 0)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		.HAlign(HAlign_Fill)
		[
			SNew(SNumericVectorInputBox<double>)
			.bColorAxisLabels(true)
			.AllowSpin(true)
			.X_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).GetLocation().X;
				}

				return 0.0;
			})
			.Y_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).GetLocation().Y;
				}

				return 0.0;
			})
			.Z_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).GetLocation().Z;
				}

				return 0.0;
			})
			.OnXChanged_Lambda([OnLocationChanged](float Value) { OnLocationChanged(Value, EAxis::X); })
			.OnYChanged_Lambda([OnLocationChanged](float Value) { OnLocationChanged(Value, EAxis::Y); })
			.OnZChanged_Lambda([OnLocationChanged](float Value) { OnLocationChanged(Value, EAxis::Z); })
			.OnXCommitted_Lambda([OnLocationCommitted](float Value, ETextCommit::Type Type) { OnLocationCommitted(Value, Type, EAxis::X); })
			.OnYCommitted_Lambda([OnLocationCommitted](float Value, ETextCommit::Type Type) { OnLocationCommitted(Value, Type, EAxis::Y); })
			.OnZCommitted_Lambda([OnLocationCommitted](float Value, ETextCommit::Type Type) { OnLocationCommitted(Value, Type, EAxis::Z); })
			.Font(IDetailLayoutBuilder::GetDetailFont())

			.OnXBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnXEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
			.OnYBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnYEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
			.OnZBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnZEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
		]
		
		+ SGridPanel::Slot(0, 1)
		.HAlign(EHorizontalAlignment::HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Transform_Rotation", "Rotation"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SGridPanel::Slot(1, 1)
		.HAlign(HAlign_Fill)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SNumericRotatorInputBox<double>)
			.bColorAxisLabels(true)
			.AllowSpin(true)
			.Roll_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).Rotator().Roll;
				}
				
				return 0.0;
			})
			.Pitch_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).Rotator().Pitch;
				}
				
				return 0.0;
			})
			.Yaw_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).Rotator().Yaw;
				}

				return 0.0;
			})
			.OnRollChanged_Lambda([OnRotationChanged](float Value) { OnRotationChanged(Value, EAxis::X); })
			.OnPitchChanged_Lambda([OnRotationChanged](float Value) { OnRotationChanged(Value, EAxis::Y); })
			.OnYawChanged_Lambda([OnRotationChanged](float Value) { OnRotationChanged(Value, EAxis::Z); })
			.OnRollCommitted_Lambda([OnRotationCommitted](float Value, ETextCommit::Type Type) { OnRotationCommitted(Value, Type, EAxis::X); })
			.OnPitchCommitted_Lambda([OnRotationCommitted](float Value, ETextCommit::Type Type) { OnRotationCommitted(Value, Type, EAxis::Y); })
			.OnYawCommitted_Lambda([OnRotationCommitted](float Value, ETextCommit::Type Type) { OnRotationCommitted(Value, Type, EAxis::Z); })
			.Font(IDetailLayoutBuilder::GetDetailFont())

			.OnRollBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnRollEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
			.OnPitchBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnPitchEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
			.OnYawBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnYawEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
		]

		+ SGridPanel::Slot(0, 2)
		.HAlign(EHorizontalAlignment::HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Transform_Scale", "Scale"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SGridPanel::Slot(1, 2)
		.HAlign(HAlign_Fill)
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SNumericVectorInputBox<double>)
			.bColorAxisLabels(true)
			.AllowSpin(false)
			.X_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).GetScale3D().X;
				}
				
				return 0.0;
			})
			.Y_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).GetScale3D().Y;
				}
				
				return 0.0;
			})
			.Z_Lambda([WeakThis = SharedThis(this).ToWeakPtr(), ParamName]()
			{
				if (TSharedPtr<FCustomizableInstanceDetails> SharedThis = WeakThis.Pin())
				{
					return SharedThis->GetTransformParameterValue(ParamName).GetScale3D().Z;
				}
				
				return 0.0;
			})

			.OnXChanged_Lambda([OnScaleChanged](float Value) { OnScaleChanged(Value, EAxis::X); })
			.OnYChanged_Lambda([OnScaleChanged](float Value) { OnScaleChanged(Value, EAxis::Y); })
			.OnZChanged_Lambda([OnScaleChanged](float Value) { OnScaleChanged(Value, EAxis::Z); })
			.OnXCommitted_Lambda([OnScaleCommitted](float Value, ETextCommit::Type Type) { OnScaleCommitted(Value, Type, EAxis::X); })
			.OnYCommitted_Lambda([OnScaleCommitted](float Value, ETextCommit::Type Type) { OnScaleCommitted(Value, Type, EAxis::Y); })
			.OnZCommitted_Lambda([OnScaleCommitted](float Value, ETextCommit::Type Type) { OnScaleCommitted(Value, Type, EAxis::Z); })
			.Font(IDetailLayoutBuilder::GetDetailFont())

			.OnXBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnXEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
			.OnYBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnYEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
			.OnZBeginSliderMovement_Lambda([BeginSliderMovement]() { BeginSliderMovement(); })
			.OnZEndSliderMovement_Lambda([EndSliderMovement](float NewValue) { EndSliderMovement(NewValue); })
		];
}

FTransform FCustomizableInstanceDetails::GetTransformParameterValue(const FString ParamName) const
{
	return CustomInstance->GetTransformParameterSelectedOption(ParamName);
}

void FCustomizableInstanceDetails::OnTransformParameterChanged(FTransform NewValue, const FString ParamName)
{
	FTransform OldValue = CustomInstance->GetTransformParameterSelectedOption(ParamName);

	if (!OldValue.Equals(NewValue))
	{
		CustomInstance->SetTransformParameterSelectedOption(ParamName, NewValue);
		UpdateInstance();
	}
}

void FCustomizableInstanceDetails::OnTransformParameterCommitted(FTransform NewTransform, ETextCommit::Type Type, const FString ParamName)
{
	if (Type != ETextCommit::OnEnter)
	{
		return;
	}

	const FTransform OldValue = CustomInstance->GetTransformParameterSelectedOption(ParamName);
	if (OldValue.Equals(NewTransform))
	{
		return;
	}

	{
		FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("OnTransformParameterCommited", "Set Transform Parameter: {0}"), FText::FromString(ParamName)));
		CustomInstance->Modify();
		CustomInstance->SetTransformParameterSelectedOption(ParamName, NewTransform);
	}

	UpdateInstance();
}


// PROJECTOR PARAMETERS -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateSimpleProjector(const FString& ParamName)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	const UProjectorParameter* ProjectorParameter = Editor->GetProjectorParameter();
	const bool bSelectedProjector = ProjectorParameter->IsProjectorSelected(ParamName);

	TSharedPtr<SButton> Button;
	TSharedPtr<SHorizontalBox> SimpleProjectorBox;

	SAssignNew(SimpleProjectorBox,SHorizontalBox)
	+ SHorizontalBox::Slot()
	.FillWidth(0.25f)
	[
		SAssignNew(Button, SButton)
		.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorSelectChanged, ParamName, -1)
		.HAlign(HAlign_Center)
		.Content()
		[
			SNew(STextBlock)
			.Text(bSelectedProjector ? LOCTEXT("Unselect Projector", "Unselect Projector") : LOCTEXT("Select Projector", "Select Projector"))
			.ToolTipText(bSelectedProjector ? LOCTEXT("Unselect Projector", "Unselect Projector") : LOCTEXT("Select Projector", "Select Projector"))
			.Font(LayoutBuilder.Pin()->GetDetailFont())
		]
	]

	+ SHorizontalBox::Slot()
	.FillWidth(0.25f)
	[
		SNew(SButton)
		.HAlign(HAlign_Center)
		.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorCopyTransform, ParamName, -1)
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CopyTransform_Text", "Copy Transform"))
			.ToolTipText(LOCTEXT("CopyTransform_Tooltip", "Copy Transform"))
			.Font(LayoutBuilder.Pin()->GetDetailFont())
		]
	]

	+ SHorizontalBox::Slot()
	.FillWidth(0.25f)
	[
		SNew(SButton)
		.HAlign(HAlign_Center)
		.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorPasteTransform, ParamName, -1)
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PasteTransform_Text", "Paste Transform"))
			.ToolTipText(LOCTEXT("PasteTransform_Tooltip", "Paste Transform"))
			.Font(LayoutBuilder.Pin()->GetDetailFont())
		]
	]

	+ SHorizontalBox::Slot()
	.FillWidth(0.25f)
	[
		SNew(SButton)
		.HAlign(HAlign_Center)
		.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorResetTransform, ParamName, -1)
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ResetTransform_Text", "Reset Transform"))
			.ToolTipText(LOCTEXT("ResetTransform_Tooltip", "Reset Transform"))
			.Font(LayoutBuilder.Pin()->GetDetailFont())
		]
	];

	Button->SetBorderBackgroundColor(bSelectedProjector ? FLinearColor::Green : FLinearColor::White);

	return SimpleProjectorBox.ToSharedRef();
}


FReply FCustomizableInstanceDetails::OnProjectorSelectChanged(const FString ParamName, const int32 RangeIndex) const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	const UProjectorParameter* ProjectorParameter = Editor->GetProjectorParameter();
	if (ProjectorParameter->IsProjectorSelected(ParamName, RangeIndex))
	{
		Editor->HideGizmo();
	}
	else
	{
		Editor->ShowGizmoProjectorParameter(ParamName, RangeIndex);
	}

	return FReply::Handled();
}


FReply FCustomizableInstanceDetails::OnProjectorCopyTransform(const FString ParamName, const int32 RangeIndex) const
{
	const int32 ParameterIndexInObject = CustomInstance->GetCustomizableObject()->GetPrivate()->FindParameter(ParamName);
	const int32 ProjectorParamIndex = CustomInstance->GetPrivate()->FindProjectorParameterNameIndex(ParamName);

	if ((ParameterIndexInObject >= 0) && (ProjectorParamIndex >= 0))
	{
		const TArray<FCustomizableObjectProjectorParameterValue>& ProjectorParameters = CustomInstance->GetPrivate()->GetDescriptor().ProjectorParameters;
		FCustomizableObjectProjector Value;

		if (RangeIndex == -1)
		{
			Value = ProjectorParameters[ProjectorParamIndex].Value;
		}
		else if (ProjectorParameters[ProjectorParamIndex].RangeValues.IsValidIndex(RangeIndex))
		{
			Value = ProjectorParameters[ProjectorParamIndex].RangeValues[RangeIndex];
		}
		else
		{
			check(false);
		}

		const UScriptStruct* Struct = Value.StaticStruct();
		FString Output = TEXT("");
		Struct->ExportText(Output, &Value, nullptr, nullptr, (PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited | PPF_IncludeTransient), nullptr);

		FPlatformApplicationMisc::ClipboardCopy(*Output);
	}

	return FReply::Handled();
}


FReply FCustomizableInstanceDetails::OnProjectorPasteTransform(const FString ParamName, const int32 RangeIndex)
{
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	FCustomizableObjectProjector DefaultValue;
	UScriptStruct* Struct = DefaultValue.StaticStruct();
	Struct->ImportText(*ClipboardText, &DefaultValue, nullptr, 0, GLog, GetPathNameSafe(Struct));

	{
		FScopedTransaction LocalTransaction(LOCTEXT("PasteTransform", "Paste Projector Transform"));
		CustomInstance->Modify();
		CustomInstance->SetProjectorParameterSelectedOption(ParamName, DefaultValue, RangeIndex);
	}

	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	Editor->ShowGizmoProjectorParameter(ParamName, RangeIndex);
	UpdateInstance();

	return FReply::Handled();
}


FReply FCustomizableInstanceDetails::OnProjectorResetTransform(const FString ParamName, const int32 RangeIndex)
{
	const FCustomizableObjectProjector DefaultValue = CustomInstance->GetCustomizableObject()->GetProjectorParameterDefaultValue(ParamName);

	{
		FScopedTransaction LocalTransaction(LOCTEXT("ResetTransform", "Reset Projector Transform"));
		CustomInstance->Modify();
		CustomInstance->SetProjectorParameterSelectedOption(ParamName, DefaultValue, RangeIndex);
	}

	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	Editor->ShowGizmoProjectorParameter(ParamName, RangeIndex);
	UpdateInstance();

	return FReply::Handled();
}


TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateMultidimensionalProjector(const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	const TArray<FCustomizableObjectProjectorParameterValue>& ProjectorParameters = CustomInstance->GetPrivate()->GetDescriptor().ProjectorParameters;
	const int32 ProjectorParamIndex = CustomInstance->GetPrivate()->FindProjectorParameterNameIndex(ParamName);

	check(ProjectorParamIndex < ProjectorParameters.Num());

	// Selected Pose UI
	const FString PoseSwitchEnumParamName = ParamName + POSE_PARAMETER_POSTFIX;
	const int32 PoseSwitchEnumParamIndexInObject = CustomizableObject.GetPrivate()->FindParameter(PoseSwitchEnumParamName);
	TSharedPtr<SVerticalBox> ProjectorBox = SNew(SVerticalBox);

	if (PoseSwitchEnumParamIndexInObject != INDEX_NONE)
	{
		const int32 NumPoseValues = CustomizableObject.GetPrivate()->GetEnumParameterNumValues(PoseSwitchEnumParamIndexInObject);

		TSharedPtr<TArray<TSharedPtr<FString>>>* FoundOptions = ProjectorParameterPoseOptions.Find(PoseSwitchEnumParamIndexInObject);
		TArray<TSharedPtr<FString>>& PoseOptionNamesAttribute = FoundOptions && FoundOptions->IsValid() ?
																*FoundOptions->Get()
																: *(ProjectorParameterPoseOptions.Add(PoseSwitchEnumParamIndexInObject, MakeShared<TArray<TSharedPtr<FString>>>()).Get());

		PoseOptionNamesAttribute.Empty();

		FString PoseValue = CustomInstance->GetIntParameterSelectedOption(PoseSwitchEnumParamName, -1);
		int32 PoseValueIndex = 0;

		for (int32 j = 0; j < NumPoseValues; ++j)
		{
			const FString PossibleValue = CustomizableObject.GetPrivate()->GetIntParameterAvailableOption(PoseSwitchEnumParamIndexInObject, j);
			if (PossibleValue == PoseValue)
			{
				PoseValueIndex = j;
			}

			PoseOptionNamesAttribute.Add(MakeShared<FString>(PossibleValue));
		}

		ProjectorBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		.FillHeight(10.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.FillWidth(0.45f)
			[
				SNew(SSearchableComboBox)
				.ToolTipText(LOCTEXT("Pose selector tooltip", "Select the skeletal mesh pose used for projection. This does not control the actual visual mesh pose in the viewport (or during gameplay for that matter). It has to be manually set. You can drag&drop a pose onto the preview viewport."))
				.OptionsSource(&PoseOptionNamesAttribute)
				.InitiallySelectedItem(PoseOptionNamesAttribute[PoseValueIndex])
				.Method(EPopupMethod::UseCurrentWindow)
				.OnSelectionChanged(this, &FCustomizableInstanceDetails::OnProjectorTextureParameterComboBoxChanged, PoseSwitchEnumParamName, -1)
				.OnGenerateWidget(this, &FCustomizableInstanceDetails::OnGenerateWidgetProjectorParameter)
				.Content()
				[
					SNew(STextBlock)
						.Text(FText::FromString(*PoseOptionNamesAttribute[PoseValueIndex]))
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.FillWidth(0.3f)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("Add Layer", "Add Layer"))
				.Text(LOCTEXT("Add Layer", "Add Layer"))
				.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorLayerAdded, ParamName)
				.HAlign(HAlign_Fill)
			]
		];
	}
	else
	{
		ProjectorBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		//.FillHeight(10.f)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("Add Layer", "Add Layer"))
			.Text(LOCTEXT("Add Layer", "Add Layer"))
			.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorLayerAdded, ParamName)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
		];
	}

	const FString TextureSwitchEnumParamName = ParamName + IMAGE_PARAMETER_POSTFIX;
	const FString OpacitySliderParamName = ParamName + OPACITY_PARAMETER_POSTFIX;

	IDetailGroup* ProjectorGroup = *ParentsGroups.Find(ParamName);

	for (int32 RangeIndex = 0; RangeIndex < ProjectorParameters[ProjectorParamIndex].RangeValues.Num(); ++RangeIndex)
	{
		const int32 TextureSwitchEnumParamIndexInObject = CustomizableObject.GetPrivate()->FindParameter(TextureSwitchEnumParamName);
		check(TextureSwitchEnumParamIndexInObject >= 0); TSharedPtr<FString> CurrentStateName = nullptr;

		const UProjectorParameter* ProjectorParameter = Editor->GetProjectorParameter();
		const bool bSelectedProjector = ProjectorParameter->IsProjectorSelected(ParamName, RangeIndex);

		//Vertical box that owns all the layer properties
		TSharedPtr<SVerticalBox> LayerProperties = SNew(SVerticalBox);
		//Horizontal box that owns all the projector properties
		TSharedPtr<SHorizontalBox> ProjectorProperties;
		// Widget to set the opacity and remove a layer
		TSharedPtr<SHorizontalBox> OpacityRemoveWidget = SNew(SHorizontalBox);
		// Button Ptr needed to edit its style
		TSharedPtr<SButton> Button;

		SAssignNew(ProjectorProperties, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(1, 0)
		[
			SNew(SBox)
			.MinDesiredWidth(115.f)
			.MaxDesiredWidth(115.f)
			[
				SAssignNew(Button, SButton)
				.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorSelectChanged, ParamName, RangeIndex)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Content()
				[
					SNew(STextBlock)
					.Text(bSelectedProjector ? LOCTEXT("Unselect Projector", "Unselect Projector") : LOCTEXT("Select Projector", "Select Projector"))
					.ToolTipText(bSelectedProjector ? LOCTEXT("Unselect Projector", "Unselect Projector") : LOCTEXT("Select Projector", "Select Projector"))
					.Justification(ETextJustify::Center)
					.Font(LayoutBuilder.Pin()->GetDetailFont())
				]
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(1, 0)
		[
			SNew(SButton)
				.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorCopyTransform, ParamName, RangeIndex)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Content()
				[
					SNew(STextBlock)
						.ToolTipText(LOCTEXT("Copy Transform", "Copy Transform"))
						.Text(LOCTEXT("Copy Transform", "Copy Transform"))
						.AutoWrapText(true)
						.Justification(ETextJustify::Center)
						.Font(LayoutBuilder.Pin()->GetDetailFont())
				]
		]

		+ SHorizontalBox::Slot()
		.Padding(1, 0)
		[
			SNew(SButton)
				.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorPasteTransform, ParamName, RangeIndex)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Content()
				[
					SNew(STextBlock)
						.ToolTipText(LOCTEXT("Paste Transform", "Paste Transform"))
						.Text(LOCTEXT("Paste Transform", "Paste Transform"))
						.Justification(ETextJustify::Center)
						.AutoWrapText(true)
						.Font(LayoutBuilder.Pin()->GetDetailFont())
				]
		]

		+ SHorizontalBox::Slot()
		.Padding(1, 0)
		[
			SNew(SButton)
				.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorResetTransform, ParamName, RangeIndex)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Content()
				[
					SNew(STextBlock)
						.ToolTipText(LOCTEXT("Reset Transform", "Reset Transform"))
						.Text(LOCTEXT("Reset Transform", "Reset Transform"))
						.Justification(ETextJustify::Center)
						.AutoWrapText(true)
						.Font(LayoutBuilder.Pin()->GetDetailFont())
				]
		];

		Button->SetBorderBackgroundColor(bSelectedProjector ? FLinearColor::Green : FLinearColor::White);

		// If number of options is equal to 1, Mutable does not consider it multidimensional parameters
		int32 NumValues = CustomizableObject.GetPrivate()->GetEnumParameterNumValues(TextureSwitchEnumParamIndexInObject);
		FString Value = CustomizableObject.IsParameterMultidimensional(TextureSwitchEnumParamName) ? 
			CustomInstance->GetIntParameterSelectedOption(TextureSwitchEnumParamName, RangeIndex) : CustomInstance->GetIntParameterSelectedOption(TextureSwitchEnumParamName);

		TArray<TSharedPtr<FString>> OptionNamesAttribute;
		int32 ValueIndex = 0;

		for (int32 CandidateIndex = 0; CandidateIndex < NumValues; ++CandidateIndex)
		{
			FString PossibleValue = CustomizableObject.GetPrivate()->GetIntParameterAvailableOption(TextureSwitchEnumParamIndexInObject, CandidateIndex);
			if (PossibleValue == Value)
			{
				ValueIndex = CandidateIndex;
			}

			OptionNamesAttribute.Add(MakeShared<FString>(CustomizableObject.GetPrivate()->GetIntParameterAvailableOption(TextureSwitchEnumParamIndexInObject, CandidateIndex)));
		}
		
		// Avoid filling this arraw with repeated array  options
		if (RangeIndex == 0)
		{
			ProjectorTextureOptions.Add(MakeShared<TArray<TSharedPtr<FString>>>(OptionNamesAttribute));
		}

		if (NumValues > 1)
		{
			OpacityRemoveWidget->AddSlot()
			.Padding(1, 0)
			.FillWidth(0.3f)
			[
				SNew(SBox)
				[
					SNew(SSearchableComboBox)
					.OptionsSource(ProjectorTextureOptions.Last().Get())
					.InitiallySelectedItem((*ProjectorTextureOptions.Last())[ValueIndex])
					.OnGenerateWidget(this, &FCustomizableInstanceDetails::MakeTextureComboEntryWidget)
					.OnSelectionChanged(this, &FCustomizableInstanceDetails::OnProjectorTextureParameterComboBoxChanged, TextureSwitchEnumParamName, RangeIndex)
					.Content()
					[
						SNew(STextBlock)
						.Text(FText::FromString(*(*ProjectorTextureOptions.Last())[ValueIndex]))
					]
				]
			];
		}
		
		OpacityRemoveWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.Padding(1, 0)
		.FillWidth(0.7f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(1, 0)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SNew(SSpinBox<float>)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Value(this, &FCustomizableInstanceDetails::GetFloatParameterValue, OpacitySliderParamName, RangeIndex)
				.OnValueChanged(this, &FCustomizableInstanceDetails::OnFloatParameterChanged, OpacitySliderParamName, RangeIndex)
				.OnValueCommitted(this, &FCustomizableInstanceDetails::OnFloatParameterCommited, OpacitySliderParamName, RangeIndex)
				.OnBeginSliderMovement(this, &FCustomizableInstanceDetails::OnFloatParameterSliderBegin, OpacitySliderParamName, RangeIndex)
				.Font(LayoutBuilder.Pin()->GetDetailFont())
			]

			+ SHorizontalBox::Slot()
			.Padding(1, 0)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &FCustomizableInstanceDetails::OnProjectorLayerRemoved, ParamName, RangeIndex)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Content()
				[
					SNew(STextBlock)
						.ToolTipText(LOCTEXT("LayerProjectorRemoveLayer_ToolTip", "Remove Layer"))
						.Text(LOCTEXT("LayerProjectorRemoveLayer_Text", "X"))
						.Justification(ETextJustify::Center)
						.AutoWrapText(true)
						.Font(LayoutBuilder.Pin()->GetDetailFont())
				]
			]
		];

		LayerProperties->AddSlot()
		.AutoHeight()
		[
			OpacityRemoveWidget.ToSharedRef()
		];
		LayerProperties->AddSlot()
		.AutoHeight()
		.Padding(0.0f,5.0f,0.0f,0.0f)
		[
			ProjectorProperties.ToSharedRef()
		];

		// Final composed widget
		ProjectorGroup->AddWidgetRow()
		.NameContent()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(STextBlock)
			.Text(FText::FromString("Layer " + FString::FromInt(RangeIndex)))
		]
		.ValueContent()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(0.0f, 5.0f, 0.0f, 5.0f)
			[
				LayerProperties.ToSharedRef()
			]
		];
	}
	
	return ProjectorBox.ToSharedRef();
}


TSharedRef<SWidget> FCustomizableInstanceDetails::OnGenerateWidgetProjectorParameter(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(FText::FromString(*InItem.Get()));
}


void FCustomizableInstanceDetails::OnExternalTypeParameterChanged(const FPropertyChangedEvent& PropertyChangedEvent, TSharedPtr<FStructOnScope> StructOnScope, FString ParamName, const int32 RangeIndex)
{
	if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(StructOnScope->GetStructPtr()))
	{
		CustomInstance->Modify();

		FInstancedStruct InstancedStruct;
		InstancedStruct.InitializeAs(ScriptStruct);
		ScriptStruct->CopyScriptStruct(InstancedStruct.GetMutableMemory(), StructOnScope->GetStructMemory());

		CustomInstance->SetExternalTypeParameterSelectedOption(ParamName, InstancedStruct, RangeIndex);

		UpdateInstance();
	}
}


FReply FCustomizableInstanceDetails::OnProjectorLayerAdded(FString ParamName)
{
	{
		FScopedTransaction LocalTransaction(LOCTEXT("AddProjectorLayer", "Add Projector Layer"));
		CustomInstance->Modify();

		const int32 NumLayers = CustomInstance->MultilayerProjectorNumLayers(*ParamName);
		CustomInstance->MultilayerProjectorCreateLayer(*ParamName, NumLayers);
	}

	UpdateInstance();

	FCoreUObjectDelegates::BroadcastOnObjectModified(CustomInstance.Get());

	return FReply::Handled();
}


FReply FCustomizableInstanceDetails::OnProjectorLayerRemoved(const FString ParamName, const int32 RangeIndex)
{
	{
		FScopedTransaction LocalTransaction(LOCTEXT("RemoveProjectorLayer", "Remove Projector Layer"));
		CustomInstance->Modify();

		// Unselect projector if it's the deleted one
		if (GetEditorChecked()->GetProjectorParameter()->IsProjectorSelected(ParamName, RangeIndex))
		{
			GetEditorChecked()->HideGizmo();
		}

		CustomInstance->MultilayerProjectorRemoveLayerAt(*ParamName, RangeIndex);
	}

	UpdateInstance();

	FCoreUObjectDelegates::BroadcastOnObjectModified(CustomInstance.Get());

	return FReply::Handled();
}


void FCustomizableInstanceDetails::OnProjectorTextureParameterComboBoxChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, FString ParamName, int32 RangeIndex)
{
	if (Selection.IsValid())
	{
		{
			FScopedTransaction LocalTransaction(LOCTEXT("SelectProjectorImage", "Select Projector Image"));
			CustomInstance->Modify();
			CustomInstance->SetIntParameterSelectedOption(ParamName, *Selection, RangeIndex);
		}

		UpdateInstance();
	}
}


TSharedPtr<ICustomizableObjectInstanceEditor> FCustomizableInstanceDetails::GetEditorChecked() const
{
	TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
	check(Editor);

	return Editor;
}


TSharedRef<SWidget> FCustomizableInstanceDetails::MakeTextureComboEntryWidget(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(FText::FromString(*InItem.Get()));
}


// BOOL PARAMETERS -----------------------------------------------------------------------------------------------------------------

TSharedRef<SWidget> FCustomizableInstanceDetails::GenerateBoolWidget(const FString& ParamName)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			SNew(SCheckBox)
			.HAlign(HAlign_Left)
			.IsChecked(this, &FCustomizableInstanceDetails::GetBoolParameterValue, ParamName)
			.OnCheckStateChanged(this, &FCustomizableInstanceDetails::OnBoolParameterChanged, ParamName)
		];
}


ECheckBoxState FCustomizableInstanceDetails::GetBoolParameterValue(FString ParamName) const
{
	const bool bResult = CustomInstance->GetBoolParameterSelectedOption(ParamName);
	return bResult ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


void FCustomizableInstanceDetails::OnBoolParameterChanged(ECheckBoxState InCheckboxState, FString ParamName)
{
	{
		FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("SetParameterBool", "Set Bool Parameter: {0}"), FText::FromString(ParamName)));
		CustomInstance->Modify();
		CustomInstance->SetBoolParameterSelectedOption(ParamName, InCheckboxState == ECheckBoxState::Checked);
	}

	UpdateInstance();
}


// PARAMETERS -----------------------------------------------------------------------------------------------------------------

FReply FCustomizableInstanceDetails::OnCopyAllParameters()
{
	const FString ExportedText = CustomInstance->GetPrivate()->GetDescriptor().ToString();
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	return FReply::Handled();
}


FReply FCustomizableInstanceDetails::OnPasteAllParameters()
{
	FString ClipText;
	FPlatformApplicationMisc::ClipboardPaste(ClipText);

	FCustomizableObjectInstanceDescriptor& Descriptor = CustomInstance->GetPrivate()->GetDescriptor();
	const UScriptStruct* Struct = Descriptor.StaticStruct();

	bool bImported = false;
	{
		FScopedTransaction LocalTransaction(LOCTEXT("OnPasteAllParameters", "Paste All Parameters"));
		CustomInstance->Modify();

		if (Struct->ImportText(*ClipText, &Descriptor, nullptr, 0, GLog, GetPathNameSafe(Struct)))
		{
			Descriptor.SetCustomizableObject(Descriptor.GetCustomizableObject());
			bImported = true;
		}
	}

	if (bImported)
	{
		UpdateInstance();
	}

	return FReply::Handled();
}


FReply FCustomizableInstanceDetails::OnResetAllParameters()
{
	{
		FScopedTransaction LocalTransaction(LOCTEXT("OnResetAllParameters", "Reset All Parameters"));
		CustomInstance->Modify();

		CustomInstance->SetDefaultValues();
	}

	UpdateInstance();

	return FReply::Handled();
}


void FCustomizableInstanceDetails::OnResetParameterButtonClicked(int32 ParameterIndex)
{
	FString ParameterName = CustomInstance->GetCustomizableObject()->GetParameterName(ParameterIndex);

	{
		FScopedTransaction LocalTransaction(FText::Format(LOCTEXT("OnResetParameter", "Reset Parameter: {0}"), FText::FromString(ParameterName)));
		CustomInstance->Modify();
		CustomInstance->SetDefaultValue(ParameterName);
	}

	UpdateInstance();
}

// TRANSACTION SYSTEM -----------------------------------------------------------------------------------------------------------------

void FCustomizableInstanceDetails::OnInstanceTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		// We want to make sure that the gizmo is hidden when we do an undo/redo transaction
		GetEditorChecked()->HideGizmo();
	}
}


#undef LOCTEXT_NAMESPACE
