// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "GameplayTagContainer.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class FReply;
class FTransactionObjectEvent;
class ICustomizableObjectInstanceEditor;
class IDetailCategoryBuilder;
class IDetailGroup;
class IDetailLayoutBuilder; 
class SWidget;
class UCustomizableObject;
class UCustomizableObjectInstance;
class FStructOnScope;

enum class ECheckBoxState : uint8;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

struct FGeometry;
struct FPointerEvent;
struct FUpdateContext;
struct FParameterInfo;

class FCustomizableInstanceDetails : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it.
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	// ILayoutDetails interface
	/** Do not use. Add details customization in the other CustomizeDetails signature. */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override {};

	/** Customize details here. */
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	
	// Own interface	
	/** Refresh the custom details. */
	void Refresh() const;

private:

	// Updat the Instance if a parameter has been modified. Also creates a Delegate to refresh the UI if the 
	// Instance has been updated successfully.
	void UpdateInstance();

	// Callback to regenerate the details when the instance has finished an update
	void InstanceUpdated(UCustomizableObjectInstance* Instance) const;

	void ObjectCompiled();

	// State Selector 
	// Generates The StateSelector Widget
	TSharedRef<SWidget> GenerateStateSelector();
	void OnStateComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// Function to change the parameters view
	void OnShowOnlyRuntimeSelectionChanged(ECheckBoxState InCheckboxState);
	void OnShowOnlyRelevantSelectionChanged(ECheckBoxState InCheckboxState);
	void OnUseUISectionsSelectionChanged(ECheckBoxState InCheckboxState);
	void OnUseUIThumbnailsSelectionChanged(ECheckBoxState InCheckboxState);

	// Main parameter generation functions
	// Returns true if parameters have been hidden due to runtime type
	bool GenerateParametersView(IDetailCategoryBuilder& MainCategory);

	/**
	 * Sort the contents of a Parameters Info array taking in consideration that the array should get sorted first by UI Section and then by
	 * parameter UI order or Param name.
	 * @note The result will be an array where the sections are in sequential order following an alphabetical ordering (A-Z)
	 * where each section contains its parameters in an incremental numerical order (ParamUIOrder int) or incremental lexical order (ParamName)
	 * @param CustomizableObject The customizable object the parameters are linked to.
	 * @param ParametersToSort The array of parameters we want to get sorted out.
	 */
	void SortParametersAndUISections(const UCustomizableObject& CustomizableObject, TArray<FParameterInfo>& ParametersToSort) const;
	
	void RecursivelyAddParamAndChildren(const UCustomizableObject& CustomizableObject, const int32 ParamIndexInObject, const FString ParentName, IDetailCategoryBuilder& DetailsCategory);
	void FillChildrenMap(int32 ParamIndexInObject);

	// Function to determine if a parameter widget should be generated
	bool IsVisible(int32 ParamIndexInObject);
	bool IsMultidimensionalProjector(int32 ParamIndexInObject);

	// @return true if the parameter doesn't match the current filter.
	bool IsIntParameterFilteredOut(const UCustomizableObject& CustomizableObject, const FString& ParamName, const FString& ParamOption) const;

	// Main widget generation functions
	IDetailGroup* GenerateParameterSection(IDetailCategoryBuilder& DetailsCategory, const UCustomizableObject& CustomizableObject, const FString& ParamName);
	void GenerateWidgetRow(FDetailWidgetRow& WidgetRow, const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject);
	TSharedRef<SWidget> GenerateParameterWidget(const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject);

	// Int FParameters Functions
	TSharedRef<SWidget> GenerateIntWidget(const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject);
	void OnIntParameterComboBoxChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, const FString ParamName);
	TSharedRef<SWidget> OnGenerateWidgetIntParameter(TSharedPtr<FString> OptionName, const FString ParameterName);
	
	// Float FParameters Functions
	TSharedRef<SWidget> GenerateFloatWidget(const UCustomizableObject& CustomizableObject, const FString& ParamName);
	float GetFloatParameterValue(const FString ParamName, int32 RangeIndex) const;
	void OnFloatParameterChanged(float Value, const FString ParamName, int32 RangeIndex);
	void OnFloatParameterSliderBegin(const FString ParamName, int32 RangeIndex);
	void OnFloatParameterSliderEnd(const FString ParamName, int32 RangeIndex);
	void OnFloatParameterCommited(float Value, ETextCommit::Type Type, const FString ParamName, int32 RangeIndex);	// Needed to have undo/redo

	// Texture FParameters Functions
	void OnTextureParameterSelected(const FAssetData& AssetData, const FString ParamName);
	
	// Skeletal Mesh FParameters Functions
	void OnSkeletalMeshParameterSelected(const FAssetData& AssetData, const FString ParamName);

	// Material FParameters Functions
	void OnMaterialParameterSelected(const FAssetData& AssetData, const FString ParamName);
	
	// Color FParameters Functions
	TSharedRef<SWidget> GenerateColorWidget(const FString& ParamName);
	FLinearColor GetColorParameterValue(const FString ParamName) const;
	FReply OnColorBlockMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const FString ParamName);
	void OnSetColorFromColorPicker(FLinearColor NewColor, const FString PickerParamName);

	// Transform FParameters Functions
	TSharedRef<SWidget> GenerateTransformWidget(const FString& ParamName);
	FTransform GetTransformParameterValue(const FString ParamName) const;
	void OnTransformParameterChanged(FTransform NewTransform, const FString ParamName);
	void OnTransformParameterCommitted(FTransform NewTransform, ETextCommit::Type Type, const FString ParamName);

	// Bool FParameters Functions
	TSharedRef<SWidget> GenerateBoolWidget(const FString& ParamName);
	ECheckBoxState GetBoolParameterValue(const FString ParamName) const;
	void OnBoolParameterChanged(ECheckBoxState InCheckboxState, const FString ParamName);

	// Projector FParameters Functions
	TSharedRef<SWidget> GenerateSimpleProjector(const FString& ParamName);
	TSharedRef<SWidget> GenerateMultidimensionalProjector(const UCustomizableObject& CustomizableObject, const FString& ParamName, const int32 ParamIndexInObject);
	TSharedPtr<ICustomizableObjectInstanceEditor> GetEditorChecked() const;
	FReply OnProjectorSelectChanged(const FString ParamName, const int32 RangeIndex) const;
	FReply OnProjectorCopyTransform(const FString ParamName, const int32 RangeIndex) const;
	FReply OnProjectorPasteTransform(const FString ParamName, const int32 RangeIndex);
	FReply OnProjectorResetTransform(const FString ParamName, const int32 RangeIndex);
	FReply OnProjectorLayerAdded(const FString ParamName);
	FReply OnProjectorLayerRemoved(const FString ParamName, const int32 RangeIndex);
	void OnProjectorTextureParameterComboBoxChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, const FString ParamName, int32 RangeIndex);
	TSharedRef<SWidget> MakeTextureComboEntryWidget(TSharedPtr<FString> InItem) const;
	TSharedRef<SWidget> OnGenerateWidgetProjectorParameter(TSharedPtr<FString> InItem) const;

	// External Type FParmeter Functions
	void OnExternalTypeParameterChanged(const FPropertyChangedEvent& PropertyChangedEvent, TSharedPtr<FStructOnScope> StructOnScope, FString ParamName, const int32 RangeIndex);
	
	// Parameter Functions
	FReply OnCopyAllParameters();
	FReply OnPasteAllParameters();
	FReply OnResetAllParameters();
	void OnResetParameterButtonClicked(int32 ParameterIndex);

	// Transaction System
	void OnInstanceTransacted(const FTransactionObjectEvent& TransactionEvent);

private:

	/** Pointer to the Customizable Object Instance */
	TWeakObjectPtr<UCustomizableObjectInstance> CustomInstance;

	/** Details builder pointer */
	TWeakPtr<IDetailLayoutBuilder> LayoutBuilder;

	/** Map to keep track of the generated parameter sections */
	TMap<FString, IDetailGroup*> GeneratedSections;

	/** Used to insert child params to a parent's expandable area */
	TMap<FString, IDetailGroup*> ParentsGroups;

	/** Array with all the possible states for the states combo box */
	TArray< TSharedPtr<FString> > StateNames;
	
	/** Weak pointer of the open editor */
	TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor;

	/** Maps param name to children param indices, used to walk the params in order respecting parent/children relationships */
	TMultiMap<FString, int32> ParamChildren;

	/** Maps param index to bool telling if it has parent, same use as previous line */
	TMap<int32, bool> ParamHasParent;

	/** Array with all the possible multilayer projector texture options */
	TArray<TSharedPtr<TArray<TSharedPtr<FString>>>> ProjectorTextureOptions;

	/** Map from ParamIndexInObject to the param's int selector options */
	TMap<int32, TSharedPtr<TArray<TSharedPtr<FString>>>> IntParameterOptions;

	/** Map from ParamIndexInObject to the projector param pose options  */
	TMap<int32, TSharedPtr<TArray<TSharedPtr<FString>>>> ProjectorParameterPoseOptions;

	/** True when a slider is being edited */
	bool bUpdatingSlider = false;

	/** Float parameter value at the moment a slider drag started. Used to record the pre-drag state for undo. */
	float SliderStartValue = 0.0f;

	/** Array to store dynamic brushes. Needed because an image widget only stores a pointer to a Brush. */
	TArray< TSharedPtr<class FDeferredCleanupSlateBrush> > DynamicBrushes;

	//** Editor gameplay tags filter and filter type. Used to filter int parameter options. */
	FGameplayTagContainer Filter;
	EGameplayContainerMatchType FilterType = EGameplayContainerMatchType::Any;

	/** Section name used for parameters that do have an empty string for its section name. */
	const FString DefaultSectionName = FString(TEXT("Miscellaneous"));
};

