// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectBenchmarkingUtils.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectSystem.h"


bool CustomizableObjectBenchmarkingUtils::GenerateDeterministicSetOfInstances(UCustomizableObject& TargetCustomizableObject, const uint16 InstancesPerState,TSpscQueue<TStrongObjectPtr<UCustomizableObjectInstance>>& OutGeneratedInstances,  uint32& OutSuccesfullyGeneratedInstanceCount)
{
	if (!TargetCustomizableObject.IsCompiled())
	{
		return false;
	}

	const UModelResources* ModelResources = TargetCustomizableObject.GetPrivate()->GetModelResources();
	if (!ModelResources)
	{
		return false;
	}
	
	// Test this parameter configuration in all the states of the CO
	const uint32 StateCount = TargetCustomizableObject.GetStateCount();
	check(StateCount >= 1);

	UE_LOGF(LogMutable, Display, "Requested Instances Count : %i", InstancesPerState);
	UE_LOGF(LogMutable, Display, "State Count = %i", StateCount);
	
	// Compute actual total amount of instances to generate
	const uint32 TotalInstancesToTestCount = InstancesPerState * StateCount;
	UE_LOGF(LogMutable,Display, "Generating %i instances (states * requested instances)...", TotalInstancesToTestCount);
	
	// Create randomization stream for the parameters of the instance
	FRandomStream RandomizationStream = FRandomStream(0);
		
	// Generate a series of instances to later update
	for (uint32 CustomizableObjectInstanceIndex = 0; CustomizableObjectInstanceIndex < InstancesPerState; CustomizableObjectInstanceIndex++)
	{
		if (UCustomizableObjectInstance* GeneratedInstance = TargetCustomizableObject.CreateInstance())
		{				
			// Randomize instance values
			GeneratedInstance->SetRandomValuesFromStream(RandomizationStream);
				
			for (uint32 State = 0; State < StateCount; State++)
			{
				// Set the state for the instance and store it for later update.
				GeneratedInstance->GetPrivate()->SetState(State);
				OutGeneratedInstances.Enqueue(GeneratedInstance->Clone());
				OutSuccesfullyGeneratedInstanceCount++;
			}
		}
		else
		{
			UE_LOGF(LogMutable,Error, "Failed to generate COI for the %ls CO.",*TargetCustomizableObject.GetName());
			return false;
		}
	}

	return true;
}


int32 CustomizableObjectBenchmarkingUtils::GetOptimizationLevelForBenchmarking()
{
	return UE_MUTABLE_MAX_OPTIMIZATION;
}

