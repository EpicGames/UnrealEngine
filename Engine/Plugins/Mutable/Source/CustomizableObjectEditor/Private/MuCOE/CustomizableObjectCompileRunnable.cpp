// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectCompileRunnable.h"

#include "CustomizableObjectCompiler.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "HAL/FileManager.h"
#include "MuCO/UnrealMutableModelDiskStreamer.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCOE/CompileRequest.h"
#include "MuR/Model.h"
#include "MuT/Compiler.h"
#include "MuT/ErrorLog.h"
#include "MuT/UnrealPixelFormatOverride.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Trace/Trace.inl"
#include "UObject/StrongObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/ICookInfo.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimInstance.h"

#include "DerivedDataCacheInterface.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"

#include "Interfaces/ITargetPlatform.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

#define UE_MUTABLE_CORE_REGION	TEXT("Mutable Core")

TAutoConsoleVariable<bool> CVarMutableCompilerFastCompression(
	TEXT("mutable.ForceFastTextureCompression"),
	false,
	TEXT("Force the use of lower quality but faster compression during cook."),
	ECVF_Default);


FCustomizableObjectCompileRunnable::FCustomizableObjectCompileRunnable(UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> Root, const TSharedRef<FCustomizableObjectCompiler>& InCompiler)
	: MutableRoot(Root)
	, WeakCompiler(InCompiler)
	, bThreadCompleted(false)
{
	PrepareUnrealCompression();
}


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> FCustomizableObjectCompileRunnable::LoadImageResourceReferenced(int32 ID)
{
	MUTABLE_CPUPROFILER_SCOPE(LoadResourceReferenced);

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Image;
	if (!ReferencedTextures.IsValidIndex(ID))
	{
		// The id is not valid for this CO
		check(false);
		return Image;
	}

	// Find the texture id
	FMutableSourceTextureData& TextureData = ReferencedTextures[ID];

	// In the editor the src data can be directly accessed
	Image = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>();
	constexpr int32 MipmapsToSkip = 0;
	constexpr bool bLoadMipTail = false;
	EUnrealToMutableConversionError Error = ConvertTextureUnrealSourceToMutable(Image.Get(), TextureData, MipmapsToSkip, bLoadMipTail);

	if (Error != EUnrealToMutableConversionError::Success)
	{
		// This could happen in the editor, because some source textures may have changed while there was a background compilation.
		// We just show a warning and move on. This cannot happen during cooks, so it is fine.
		UE_LOGF(LogMutable, Warning, "Failed to load some source texture data for texture ID [%d]. Some textures may be corrupted.", ID);
	}

	return Image;
}


uint32 FCustomizableObjectCompileRunnable::Run()
{
	TRACE_BEGIN_REGION(UE_MUTABLE_CORE_REGION);

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] FCustomizableObjectCompileRunnable::Run start.", FPlatformTime::Seconds());

	uint32 Result = 1;
	ErrorMsg = FString();

	// Translate CO compile options into UE::Mutable::Private::CompilerOptions
	UE::Mutable::Private::Ptr<UE::Mutable::Private::CompilerOptions> CompilerOptions = new UE::Mutable::Private::CompilerOptions();

	if (Options.OptimizationLevel > UE_MUTABLE_MAX_OPTIMIZATION)
	{
		UE_LOGF(LogMutable, Log, "Mutable compile optimization level out of range. Clamping to maximum.");
		Options.OptimizationLevel = UE_MUTABLE_MAX_OPTIMIZATION;
	}

	switch (Options.OptimizationLevel)
	{
	case 0:
		CompilerOptions->SetOptimisationEnabled(false);
		CompilerOptions->SetConstReductionEnabled(false);
		CompilerOptions->SetOptimisationMaxIteration(1);
		break;

	case 1:
		CompilerOptions->SetOptimisationEnabled(false);
		CompilerOptions->SetConstReductionEnabled(true);
		CompilerOptions->SetOptimisationMaxIteration(1);
		break;

	case 2:
		CompilerOptions->SetOptimisationEnabled(true);
		CompilerOptions->SetConstReductionEnabled(true);
		CompilerOptions->SetOptimisationMaxIteration(0);
		break;

	default:
		CompilerOptions->SetOptimisationEnabled(true);
		CompilerOptions->SetConstReductionEnabled(true);
		CompilerOptions->SetOptimisationMaxIteration(0);
		break;
	}

	// Texture compression override, if necessary
	bool bUseHighQualityCompression = (Options.TextureCompression == ECustomizableObjectTextureCompression::HighQuality);
	if (CVarMutableCompilerFastCompression->GetBool())
	{
		bUseHighQualityCompression = false;
	}

	if (bUseHighQualityCompression)
	{
		CompilerOptions->SetImagePixelFormatOverride( UnrealPixelFormatFunc );
	}

	CompilerOptions->SetReferencedResourceCallback(
		[this](int32 ID, TSharedPtr<UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage>> ResolvedImage, bool bRunImmediatlyIfPossible)
		{
			UE::Tasks::FTask LaunchTask = UE::Tasks::Launch(TEXT("LoadImageReferenceTasks"),
				[ID,ResolvedImage,this]()
				{
					UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Result = LoadImageResourceReferenced(ID);
					*ResolvedImage = Result;
				},
				LowLevelTasks::ETaskPriority::BackgroundLow
			);

			return LaunchTask;
		},

		[this](int32 ID, const FString& MorphNameRef, TSharedPtr<UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh>> ResolvedMesh, bool bRunImmediatlyIfPossible) -> UE::Tasks::FTask
		{
			MUTABLE_CPUPROFILER_SCOPE(LoadMeshReferenceTasks);

			ResolvedMesh->Reset();

			if (!ReferencedMeshes.IsValidIndex(ID))
			{
				// The id is not valid for this CO
				check(false);
				return UE::Tasks::MakeCompletedTask<void>();
			}

			UE::Tasks::FTaskEvent CompletionEvent(UE_SOURCE_LOCATION);

			// Find the mesh conversion data
			FMutableSourceMeshData& MeshData = ReferencedMeshes[ID];
			FString MorphName = MorphNameRef;

			// It would be great to be able to do this conversion in a worker thread, but the engine doesn't support it yet.
			auto ConvertMeshFunc = [MeshData, MorphName, ResolvedMesh, CompletionEvent, WeakCompiler = WeakCompiler, bRunImmediatlyIfPossible]() mutable
				{
					MUTABLE_CPUPROFILER_SCOPE(LoadMeshFunc);
					check(IsInGameThread());

					// If we are shutting down, we are not allowed to try to load anything.
					if (IsEngineExitRequested())
					{
						*ResolvedMesh = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();
						CompletionEvent.Trigger();
						return;
					}

					TSharedPtr<FCustomizableObjectCompiler> Compiler = WeakCompiler.Pin();
					if (!Compiler)
					{
						*ResolvedMesh = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();
						CompletionEvent.Trigger();
						return;
					}

					TSharedRef<FMeshConversionContext> MeshConversionContext = MakeShared<FMeshConversionContext>();
					MeshConversionContext->Source = MeshData;
					MeshConversionContext->MorphName = MorphName;
					MeshConversionContext->CompilationContext = Compiler->CompilationContext;

					// Convert Synchronously
					if (bRunImmediatlyIfPossible)
					{
						MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(UE::Mutable::Private::LoadObject(MeshData.Mesh)));
						MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(UE::Mutable::Private::LoadObject(MeshData.TableReferenceSkeletalMesh)));
						MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(UE::Mutable::Private::LoadClass(MeshData.AnimInstance)));

						UE::Tasks::FTask ConvertTask = ConvertSkeletalMeshToMutable(MeshConversionContext);
						ConvertTask.Wait();
						
						*ResolvedMesh = MeshConversionContext->Result;
						CompletionEvent.Trigger();
						return;
					}

					auto ConvertSkeletalMesh = [ResolvedMesh, MeshConversionContext, CompletionEvent]()
						{
							UE::Tasks::FTask ConvertTask = ConvertSkeletalMeshToMutable(MeshConversionContext);

							UE::Tasks::FTask LaunchTask = UE::Tasks::Launch(TEXT("ConvertSkeletalMeshToMutable Callback"),
								[ResolvedMesh, MeshConversionContext, CompletionEvent]() mutable
								{
									*ResolvedMesh = MeshConversionContext->Result;
									CompletionEvent.Trigger();
								},
								ConvertTask,
								UE::Tasks::ETaskPriority::Inherit);
						};

					// Find assets to stream
					TArray<FSoftObjectPath> AssetsToStream;
					auto AddReferencedStreamableRenderAsset = [MeshConversionContext, &AssetsToStream](TSoftObjectPtr<UStreamableRenderAsset>& SoftObject)
						{
							if (UObject* Object = SoftObject.Get())
							{
								MeshConversionContext->ReferencedObjects.Add(TStrongObjectPtr<UObject>(Object));
							}
							else if (!SoftObject.IsNull())
							{
								AssetsToStream.Add(SoftObject.ToSoftObjectPath());
							}
						};

					AddReferencedStreamableRenderAsset(MeshData.Mesh);
					AddReferencedStreamableRenderAsset(MeshData.TableReferenceSkeletalMesh);

					if (UObject* Object = MeshData.AnimInstance.Get())
					{
						MeshConversionContext->ReferencedObjects.Add(TStrongObjectPtr<UObject>(Object));
					}
					else if (!MeshData.AnimInstance.IsNull())
					{
						AssetsToStream.Add(MeshData.AnimInstance.ToSoftObjectPath());
					}

					if (AssetsToStream.IsEmpty())
					{
						ConvertSkeletalMesh();
					}
					else
					{
						UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();
						const TSharedRef<FMutableStreamableManager>& Streamable = System->GetPrivate()->StreamableManager;

						auto OnAssetLoadCallback = [MeshConversionContext, ConvertSkeletalMesh]()
							{
								MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(MeshConversionContext->Source.Mesh.Get()));
								MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(MeshConversionContext->Source.TableReferenceSkeletalMesh.Get()));
								MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(MeshConversionContext->Source.AnimInstance.Get()));

								ConvertSkeletalMesh();
							};

						if (IsRunningCookCommandlet())
						{
							Streamable->RequestSyncLoad(AssetsToStream);
							OnAssetLoadCallback();
						}
						else
						{
							Streamable->RequestAsyncLoad(AssetsToStream, OnAssetLoadCallback, FStreamableManager::AsyncLoadHighPriority - 1);
						}
					}
				};

			if (IsInGameThread())
			{
				ConvertMeshFunc();
			}
			else
			{
				check(!bRunImmediatlyIfPossible);
				if (TSharedPtr<FCustomizableObjectCompiler> Compiler = WeakCompiler.Pin())
				{
					Compiler->AddGameThreadCompileTask(MoveTemp(ConvertMeshFunc));
				}
				else
				{
					*ResolvedMesh = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();
					CompletionEvent.Trigger();
				}
			}

			return CompletionEvent;
		}

	);

	const int32 MinResidentMips = UTexture::GetStaticMinTextureResidentMipCount();
	CompilerOptions->SetDataPackingStrategy( MinResidentMips, Options.EmbeddedDataBytesLimit );

	// We always compile for progressive image generation.
	CompilerOptions->SetEnableProgressiveImages(true);

	// On server builds we don't want materials and images to be generted.
	if (Options.TargetPlatform && Options.TargetPlatform->IsServerOnly())
	{
		CompilerOptions->SetDisableMaterialGeneration(true);
	}

	TFunction<void()> WaitCallback = [WeakCompiler = WeakCompiler]()
	{
		TSharedPtr<FCustomizableObjectCompiler> Compiler = WeakCompiler.Pin();
		if (Compiler)
		{
			Compiler->Tick(true);
		}
			
		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
		if (System)
		{
			System->GetPrivate()->StreamableManager->Tick(true);
		}
	};

	UE::Mutable::Private::Ptr<UE::Mutable::Private::Compiler> Compiler = new UE::Mutable::Private::Compiler(CompilerOptions, WaitCallback);

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] FCustomizableObjectCompileRunnable Compile start.", FPlatformTime::Seconds());
	Model = Compiler->Compile(MutableRoot, ExternalOperations);

	// Dump all the log messages from the compiler
	TSharedPtr<const UE::Mutable::Private::FErrorLog> pLog = Compiler->GetLog();
	for (int32 i = 0; i < pLog->GetMessageCount(); ++i)
	{
		const FString& Message = pLog->GetMessageText(i);
		const UE::Mutable::Private::ErrorLogMessageType MessageType = pLog->GetMessageType(i);
		const UE::Mutable::Private::ErrorLogMessageAttachedDataView MessageAttachedData = pLog->GetMessageAttachedData(i);

		if (MessageType == UE::Mutable::Private::ELMT_WARNING || MessageType == UE::Mutable::Private::ELMT_ERROR)
		{
			const EMessageSeverity::Type Severity = MessageType == UE::Mutable::Private::ELMT_WARNING ? EMessageSeverity::Warning : EMessageSeverity::Error;
			const ELoggerSpamBin SpamBin = [&] {
				switch (pLog->GetMessageSpamBin(i)) {
				case UE::Mutable::Private::ErrorLogMessageSpamBin::ELMSB_UNKNOWN_TAG:
					return ELoggerSpamBin::TagsNotFound;
				case UE::Mutable::Private::ErrorLogMessageSpamBin::ELMSB_ALL:
				default:
					return ELoggerSpamBin::ShowAll;
			}
			}();

			const TArray<const void*, TInlineAllocator<4>>* MessageContexts = pLog->GetMessageContexts(i);
			if (ensure(MessageContexts))
			{
				if (MessageAttachedData.UnassignedUVs && MessageAttachedData.UnassignedUVsSize > 0) 
				{			
					TSharedPtr<FErrorAttachedData> ErrorAttachedData = MakeShared<FErrorAttachedData>();
					ErrorAttachedData->UnassignedUVs.Reset();
					ErrorAttachedData->UnassignedUVs.Append(MessageAttachedData.UnassignedUVs, MessageAttachedData.UnassignedUVsSize);
				
					const void* Context = MessageContexts->IsValidIndex(0) ? (*MessageContexts)[0] : nullptr;
					ArrayErrors.Add(FError(Severity, FText::AsCultureInvariant(Message), Context, ErrorAttachedData, SpamBin));
				}
				else
				{
					ArrayErrors.Add(FError(Severity, FText::AsCultureInvariant(Message), *MessageContexts, SpamBin));
				}
			}
		}
	}

	Compiler = nullptr;

	bThreadCompleted = true;

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] FCustomizableObjectCompileRunnable::Run end.", FPlatformTime::Seconds());

	TRACE_END_REGION(UE_MUTABLE_CORE_REGION);

	return Result;
}


bool FCustomizableObjectCompileRunnable::IsCompleted() const
{
	return bThreadCompleted;
}


const TArray<FCustomizableObjectCompileRunnable::FError>& FCustomizableObjectCompileRunnable::GetArrayErrors() const
{
	return ArrayErrors;
}


FCustomizableObjectSaveDDRunnable::FCustomizableObjectSaveDDRunnable(const TSharedPtr<FCompilationRequest>& InRequest,
	TSharedPtr<UE::Mutable::Private::FMutableCachedPlatformData>& InPlatformData)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectSaveDDRunnable::FCustomizableObjectSaveDDRunnable);

	PlatformData = InPlatformData;
	
	Options = InRequest->Options;

	DDCKey = InRequest->GetDerivedDataCacheKey();
	DefaultDDCPolicy = InRequest->GetDerivedDataCachePolicy();

	UCustomizableObject* CustomizableObject = InRequest->GetCustomizableObject();
	CustomizableObjectName = GetNameSafe(CustomizableObject);
	CustomizableObjectId = GenerateDataDistributionIdentifier(*CustomizableObject);

	CustomizableObjectHeader.InternalVersion = GetECustomizableObjectVersionEnumHash();
	CustomizableObjectHeader.VersionId = CustomizableObject->GetPrivate()->GetVersionId();

	// Cache ModelResources
	{
		FMemoryWriter64 MemoryWriter(ModelResourcesData);
		FObjectAndNameAsStringProxyArchive ObjectWriter(MemoryWriter, true);
		PlatformData->ModelResources->Serialize(ObjectWriter);
	}

	if (!Options.bIsCooking)
	{
		// We will be saving all compilation data in two separate files, write CO Data
		FullFileName = CustomizableObject->GetPrivate()->GetCompiledDataFileName(Options.TargetPlatform);
		PlatformData->ModelStreamableBulkData->FullFilePath = FullFileName;
	}
}

uint32 FCustomizableObjectSaveDDRunnable::Run()
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectSaveDDRunnable::Run)

	if (PlatformData->Model)
	{
		CachePlatformData();

		bool bStoredSuccessfully = false;

		// TODO UE-222775: Allow using DDC in editor builds, not just for cooking.
		if (Options.bStoreCompiledDataInDDC && !DDCKey.Hash.IsZero())
		{
			StoreCachedPlatformDataInDDC(bStoredSuccessfully);
		}

		if (!Options.bIsCooking && !bStoredSuccessfully)
		{
			StoreCachedPlatformDataToDisk();
		}
	}

	bThreadCompleted = true;

	return 1;
}


bool FCustomizableObjectSaveDDRunnable::IsCompleted() const
{
	return bThreadCompleted;
}


const ITargetPlatform* FCustomizableObjectSaveDDRunnable::GetTargetPlatform() const
{
	return Options.TargetPlatform;
}


void FCustomizableObjectSaveDDRunnable::CachePlatformData()
{
	MUTABLE_CPUPROFILER_SCOPE(CachePlatformData);

	if (!PlatformData->Model || !PlatformData->ModelStreamableBulkData)
	{
		check(false);
		return;
	}

	// Cache ModelStreamables
	{
		// Generate list of files and update streamable blocks ids and offsets
		if (Options.bIsCooking)
		{
			UE::Mutable::Private::GenerateBulkDataFilesListWithFileLimit(CustomizableObjectId, PlatformData->Model, *PlatformData->ModelStreamableBulkData.Get(), MAX_uint8, 
				Options.bIsCooking, false, *Options.TargetPlatform, PlatformData->BulkDataFiles);
		}
		else
		{
			UE::Mutable::Private::GenerateBulkDataFilesListWithSizeLimit(PlatformData->Model, *PlatformData->ModelStreamableBulkData.Get(), Options.TargetPlatform, MAX_uint64, PlatformData->BulkDataFiles);
		}
	}

	// Cache Model and Model Roms
	{
		FMemoryWriter64 ModelMemoryWriter(ModelData);
		FUnrealMutableModelBulkWriterCook Streamer(&ModelMemoryWriter, &PlatformData->ModelStreamableData);

		// Serialize UE::Mutable::Private::FModel and streamable resources 
		constexpr bool bDropData = true;
		UE::Mutable::Private::FModel::Serialise(PlatformData->Model.Get(), Streamer, bDropData);
	}
}


void FCustomizableObjectSaveDDRunnable::StoreCachedPlatformDataInDDC(bool& bStoredSuccessfully)
{
	MUTABLE_CPUPROFILER_SCOPE(StoreCachedPlatformDataInDDC);

	using namespace UE::DerivedData;

	check(PlatformData->Model.Get() != nullptr);
	check(DDCKey.Hash.IsZero() == false);

	UE::FSharedString SharedName = UE::FSharedString(CustomizableObjectName);

	bStoredSuccessfully = true;

	// DDC record
	FCacheRecordBuilder RecordBuilder(DDCKey);

	// ModelStreamable  will be modified for the DDC record. Modify a copy
	FModelStreamableBulkData ModelStreamablesDDC = *PlatformData->ModelStreamableBulkData.Get();
	
	// Store streamable resources as FValues.
	{
		MUTABLE_CPUPROFILER_SCOPE(SerializeBulkDataForDDC);

		// Generate list of files and update streamable blocks ids and offsets
		const int32 MaxDDCFiles = Options.bIsCooking ? MAX_uint8 : 1 << 13;
		UE::Mutable::Private::GenerateBulkDataFilesListWithFileLimit(CustomizableObjectId, PlatformData->Model, ModelStreamablesDDC, MaxDDCFiles, 
			Options.bIsCooking, !Options.bIsCooking, *Options.TargetPlatform, BulkDataFilesDDC);

		if (!Options.bIsCooking)
		{
			TArray<FCachePutValueRequest> PutValueRequests;
			PutValueRequests.Reserve(BulkDataFilesDDC.Num());

			const auto WriteBulkDataDDC = [&PutValueRequests, &ModelStreamablesDDC, SharedName, DefaultDDCPolicy = DefaultDDCPolicy, DDCKey = DDCKey]
			(UE::Mutable::Private::FFile& File, TArray64<uint8>& FileBulkData, uint32 FileIndex)
				{
					FCachePutValueRequest& ValueRequest = PutValueRequests.AddDefaulted_GetRef();
					ValueRequest.Name = SharedName;
					ValueRequest.Policy = DefaultDDCPolicy;
					ValueRequest.Value = FValue::Compress(FSharedBuffer::MakeView(FileBulkData.GetData(), FileBulkData.Num()));
					ValueRequest.Key = DDCKey;
					ValueRequest.Key.Hash = ValueRequest.Value.GetRawHash();
					ModelStreamablesDDC.DDCValues.Add(ValueRequest.Value.GetRawHash());
				};

			constexpr bool bDropData = false;
			UE::Mutable::Private::SerializeBulkDataFiles(*PlatformData.Get(), BulkDataFilesDDC, WriteBulkDataDDC, bDropData);

			FRequestOwner RequestOwner(UE::DerivedData::EPriority::Blocking);
			GetCache().PutValue(PutValueRequests, RequestOwner,
				[&bStoredSuccessfully](FCachePutValueResponse&& Response)
				{
					if (Response.Status != EStatus::Ok)
					{
						bStoredSuccessfully = false;
					}
				});
			RequestOwner.Wait();
		}
		else
		{
			const auto WriteBulkDataDDC = [&RecordBuilder](UE::Mutable::Private::FFile& File, TArray64<uint8>& FileBulkData, uint32 FileIndex)
				{
					const FValueId ValueId = GetDerivedDataValueIdForResource(File.DataType, FileIndex, File.ResourceType, File.Flags);
					const FValue Value = FValue::Compress(FSharedBuffer::MakeView(FileBulkData.GetData(), FileBulkData.Num()));

					RecordBuilder.AddValue(ValueId, Value);
				};

			constexpr bool bDropData = false;
			UE::Mutable::Private::SerializeBulkDataFiles(*PlatformData.Get(), BulkDataFilesDDC, WriteBulkDataDDC, bDropData);
		}
	}

	// Store streamable resources info as FValues
	{
		MUTABLE_CPUPROFILER_SCOPE(SerializeModelStreamables);

		TArray64<uint8> ModelStreamablesBytesDDC;
		FMemoryWriter64 MemoryWriterDDC(ModelStreamablesBytesDDC);
		MemoryWriterDDC << ModelStreamablesDDC;

		const FValue ModelStreamablesValue = FValue::Compress(FSharedBuffer::MakeView(ModelStreamablesBytesDDC.GetData(), ModelStreamablesBytesDDC.Num()));
		RecordBuilder.AddValue(UE::Mutable::Private::GetDerivedDataModelStreamableBulkDataId(), ModelStreamablesValue);
	}


	// Store BulkData Files as a FValue to reconstruct the data later on
	{
		MUTABLE_CPUPROFILER_SCOPE(SerializeBulkDataFilesForDDC);

		TArray<uint8> BulkDataFilesBytes;
		FMemoryWriter MemoryWriter(BulkDataFilesBytes);

		MemoryWriter << BulkDataFilesDDC;

		const FValue BulkDataFilesValue = FValue::Compress(FSharedBuffer::MakeView(BulkDataFilesBytes.GetData(), BulkDataFilesBytes.Num()));
		RecordBuilder.AddValue(UE::Mutable::Private::GetDerivedDataBulkDataFilesId(), BulkDataFilesValue);
	}

	// Store ModelResources bytes as a FValue
	{
		MUTABLE_CPUPROFILER_SCOPE(SerializeModelResourcesForDDC);

		const FValue ModelResourcesValue = FValue::Compress(FSharedBuffer::MakeView(ModelResourcesData.GetData(), ModelResourcesData.Num()));
		RecordBuilder.AddValue(UE::Mutable::Private::GetDerivedDataModelResourcesId(), ModelResourcesValue);
	}

	// Store Model bytes as a FValue
	{
		MUTABLE_CPUPROFILER_SCOPE(SerializeModelForDDC);

		const FValue ModelValue = FValue::Compress(FSharedBuffer::MakeView(ModelData.GetData(), ModelData.Num()));
		RecordBuilder.AddValue(UE::Mutable::Private::GetDerivedDataModelId(), ModelValue);
	}

	// Push record to the DDC
	{
		MUTABLE_CPUPROFILER_SCOPE(PushRecordToDDC);
		if (bStoredSuccessfully)
		{
			FRequestOwner RequestOwner(UE::DerivedData::EPriority::Blocking);
			const FCachePutRequest PutRequest = { UE::FSharedString(CustomizableObjectName), RecordBuilder.Build(), DefaultDDCPolicy };
			GetCache().Put(MakeArrayView(&PutRequest, 1), RequestOwner,
				[&bStoredSuccessfully](FCachePutResponse&& Response)
				{
					if (Response.Status != EStatus::Ok)
					{
						bStoredSuccessfully = false;
					}
				});

			RequestOwner.Wait();
		}

		if (bStoredSuccessfully)
		{
			if (!Options.bIsCooking)
			{
				*PlatformData->ModelStreamableBulkData.Get() = ModelStreamablesDDC;
			}

			PlatformData->ModelStreamableBulkData->bIsStoredInDDC = true;
			PlatformData->ModelStreamableBulkData->DDCKey = DDCKey;
			PlatformData->ModelStreamableBulkData->DDCDefaultPolicy = DefaultDDCPolicy;
		}
	}
}


void FCustomizableObjectSaveDDRunnable::StoreCachedPlatformDataToDisk()
{
	MUTABLE_CPUPROFILER_SCOPE(StoreCachedPlatformDataToDisk);

	check(PlatformData->Model.Get() != nullptr);
	check(!Options.bIsCooking);

	// Create folder...
	IFileManager& FileManager = IFileManager::Get();
	FileManager.MakeDirectory(*GetCompiledDataFolderPath(), true);

	// Delete files and create new memory writers for each streamable data type.
	bool bSuccess = true;
	TArray<TUniquePtr<FArchive>> MemoryWriters;

	const int32 NumDataTypes = static_cast<int32>(UE::Mutable::Private::EStreamableDataType::DataTypeCount);
	for (int32 DataType = 0; DataType < NumDataTypes; ++DataType)
	{
		const FString FilePath = FullFileName + GetDataTypeExtension(static_cast<UE::Mutable::Private::EStreamableDataType>(DataType));
		if (FileManager.FileExists(*FilePath)
			&& !FileManager.Delete(*FilePath, true, false, true))
		{
			UE_LOGF(LogMutable, Error, "Failed to delete file for data type [%d].", DataType);
			bSuccess = false;
			break;
		}

		if (FArchive* MemoryWriter = FileManager.CreateFileWriter(*FilePath))
		{
			*MemoryWriter << CustomizableObjectHeader;

			MemoryWriters.Emplace(MemoryWriter);
		}
	}

	if (bSuccess)
	{
		// Serialize Streamable resources
		const auto WriteBulkDataToDisk = [&MemoryWriters](UE::Mutable::Private::FFile& File, TArray64<uint8>& FileBulkData, uint32 FileIndex)
			{
				const int32 DataTypeIndex = static_cast<int32>(File.DataType);
				if (ensure(MemoryWriters.IsValidIndex(DataTypeIndex)))
				{
					MemoryWriters[DataTypeIndex]->Serialize(FileBulkData.GetData(), FileBulkData.Num() * sizeof(uint8));
				}
			};

		// Serialize streamable resources into a single file and fix offsets
		constexpr bool bDropData = true;
		UE::Mutable::Private::SerializeBulkDataFiles(*PlatformData.Get(), PlatformData->BulkDataFiles, WriteBulkDataToDisk, bDropData);


		// Serialize Model and ModelResources. Store after SerializeBulkDataFiles fixes the HashToStreamableFiles offsets.
		if (ensure(MemoryWriters.IsValidIndex(0)))
		{
			MemoryWriters[0]->Serialize(ModelResourcesData.GetData(), ModelResourcesData.Num() * sizeof(uint8));

			// ModelMemoryWriter (Writer to disk) doesn't handle FNames properly. Serialize them ModelStreamables in two steps.
			TArray64<uint8> ModelStreamablesBytes;
			FMemoryWriter64 ModelStreamablesMemoryWriter(ModelStreamablesBytes);
			ModelStreamablesMemoryWriter << *PlatformData->ModelStreamableBulkData.Get();

			MemoryWriters[0]->Serialize(ModelStreamablesBytes.GetData(), ModelStreamablesBytes.Num() * sizeof(uint8));
			MemoryWriters[0]->Serialize(ModelData.GetData(), ModelData.Num() * sizeof(uint8));
		}

		// Write to disk and close the files
		for (TUniquePtr<FArchive>& MemoryWriter : MemoryWriters)
		{
			if (MemoryWriter)
			{
				MemoryWriter->Flush();

				if (!MemoryWriter->Close())
				{
					UE_LOGF(LogMutable, Error, "Failed to write file to disk. File [%ls].", *MemoryWriter->GetArchiveName());
					bSuccess = false;
					break;
				}
			}
		}
	}
	
	if (!bSuccess)
	{
		// Delete model to invalidate compilation.
		PlatformData->Model.Reset();
	}
}


#undef LOCTEXT_NAMESPACE

