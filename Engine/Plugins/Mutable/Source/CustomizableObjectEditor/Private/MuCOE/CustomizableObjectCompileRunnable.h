// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"

#include "DerivedDataCacheKey.h"
#include "DerivedDataCachePolicy.h"

#include <atomic>

struct FCompilationRequest;
class ITargetPlatform;

class FCustomizableObjectCompileRunnable : public FRunnable
{
public:

	struct FErrorAttachedData
	{
		TArray<float> UnassignedUVs;
	};

	struct FError
	{
		EMessageSeverity::Type Severity = EMessageSeverity::Error;
		ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll;
		FText Message;
		TSharedPtr<FErrorAttachedData> AttachedData;
		
		TArray<const void*, TInlineAllocator<4>> Contexts;

		FError(const EMessageSeverity::Type InSeverity, const FText& InMessage, const TArray<const void*, TInlineAllocator<4>>& InContexts, const ELoggerSpamBin InSpamBin = ELoggerSpamBin::ShowAll )
			: Severity(InSeverity), SpamBin(InSpamBin), Message(InMessage), Contexts(InContexts) {}
		FError(const EMessageSeverity::Type InSeverity, const FText& InMessage, const void* InContext, const TSharedPtr<FErrorAttachedData>& InAttachedData, const ELoggerSpamBin InSpamBin = ELoggerSpamBin::ShowAll)
			: Severity(InSeverity), SpamBin(InSpamBin), Message(InMessage), AttachedData(InAttachedData), Contexts{InContext} {}
	};

private:

	UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> MutableRoot;
	TArray<FError> ArrayErrors;

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> LoadImageResourceReferenced(int32 ID);

public:

	FCustomizableObjectCompileRunnable(UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> Root, const TSharedRef<FCustomizableObjectCompiler>& InCompiler);

	// FRunnable interface
	uint32 Run() override;

	// Own interface

	//
	bool IsCompleted() const;

	//
	const TArray<FError>& GetArrayErrors() const;

	TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model;

	TMap<UE::Mutable::Private::FOperation::ADDRESS, FInstancedStruct> ExternalOperations;
	
	FCompilationOptions Options;

	TWeakPtr<FCustomizableObjectCompiler> WeakCompiler;

	TArray<FMutableSourceTextureData> ReferencedTextures;
	TArray<FMutableSourceMeshData> ReferencedMeshes;

	FString ErrorMsg;

	// Whether the thread has finished running
	std::atomic<bool> bThreadCompleted;
};


class FCustomizableObjectSaveDDRunnable : public FRunnable
{
public:

	FCustomizableObjectSaveDDRunnable(const TSharedPtr<FCompilationRequest>& InRequest, TSharedPtr<UE::Mutable::Private::FMutableCachedPlatformData>& InPlatformData);

	// FRunnable interface
	uint32 Run() override;

	//
	bool IsCompleted() const;


	const ITargetPlatform* GetTargetPlatform() const;

private:

	void CachePlatformData();

	void StoreCachedPlatformDataInDDC(bool& bStoredSuccessfully);

	void StoreCachedPlatformDataToDisk();

	FCompilationOptions Options;

	FString CustomizableObjectName;
	FGuid CustomizableObjectId;

	MutableCompiledDataStreamHeader CustomizableObjectHeader;

	// Used to save files to disk
	FString FullFileName;

	UE::DerivedData::FCacheKey DDCKey;
	UE::DerivedData::ECachePolicy DefaultDDCPolicy;

	// Whether the thread has finished running
	std::atomic<bool> bThreadCompleted = false;

public:

	TArray64<uint8> ModelData;
	TArray64<uint8> ModelResourcesData;

	// Cached platform data
	TSharedPtr<UE::Mutable::Private::FMutableCachedPlatformData> PlatformData;

	// DDC Helpers
	TArray<UE::Mutable::Private::FFile> BulkDataFilesDDC;
};
