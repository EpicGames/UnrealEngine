// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectCompiler.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ClothConfig.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MessageLogModule.h"
#include "GenerateMutableSource/GenerateMutableSourceComponent.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCOE/CustomizableObjectCompileRunnable.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/ICustomizableObjectPopulationModule.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/CustomizableObjectEditorModule.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/App.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCOE/CompileRequest.h"
#include "MuR/Model.h"
#include "MuCOE/CustomizableObjectVersionBridge.h"

#include "DerivedDataCache.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataRequestOwner.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuR/External/Operation.h"
#include "MuR/MutableEditorLogger.h"

class UTexture2D;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

#define UE_MUTABLE_COMPILE_REGION		TEXT("Mutable Compile")
#define UE_MUTABLE_PRELOAD_REGION		TEXT("Mutable Preload")
#define UE_MUTABLE_SAVEDD_REGION		TEXT("Mutable SaveDD")


void FCustomizableObjectCompiler::Tick(bool bBlocking)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectCompiler::Tick);

	if (bBlocking)
	{
		// Compilations require the asset registry to have finished.
		// If blocking, the asset registry will not finish by itself and we will deadlock. Force it to finish synchronously.
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		if (AssetRegistry.IsGathering())
		{
			AssetRegistry.SearchAllAssets(true);
		}
	}
	
	ProcessCompileTasks();
	
	bool bFinished = true;

	if (TryPopCompileRequest())
	{
		bFinished = false;
	}
	FName CurrentPackageName = CurrentObject ? CurrentObject->GetPackage()->GetFName() : NAME_None;
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(CurrentPackageName, PackageAccessTrackingOps::NAME_PostLoad);

	if (LoadModelDataFromDDCEvent.IsValid())
	{
		bFinished = false;

		if (LoadModelDataFromDDCEvent->IsCompleted())
		{
			FinishLoadingModelDataFromDDC();
		}
	}

	if (LoadStreamableDataFromDDCEvent.IsValid())
	{
		bFinished = false;

		if (LoadStreamableDataFromDDCEvent->IsCompleted())
		{
			FinishLoadingStreamableDataFromDDC();
		}
	}

	if (AsynchronousStreamableHandlePtr && AsynchronousStreamableHandlePtr->IsActive())
	{
		bFinished = false;
	}

	if (CompileTask)
	{
		bFinished = false;
		if (CompileTask->IsCompleted())
		{
			FinishCompilationTask();

			if (SaveDDTask.IsValid())
			{
				SaveCODerivedData();
			}
		}
	}

	if (SaveDDTask)
	{
		bFinished = false;

		if (SaveDDTask->IsCompleted())
		{
			FinishSavingDerivedDataTask();
		}
	}

	if (bFinished && CurrentRequest.IsValid())
	{
		CompleteRequest(ECompilationStatePrivate::Completed, GetCompilationResult());
	}

	if (CompileNotificationHandle.IsValid())
	{
		const int32 NumCompletedRequests = NumCompilationRequests - GetRemainingWork();
		FSlateNotificationManager::Get().UpdateProgressNotification(CompileNotificationHandle, NumCompletedRequests, NumCompilationRequests);
	}
}


int32 FCustomizableObjectCompiler::GetRemainingWork() const
{
	return GetNumCompileRequests();
}


int32 FCustomizableObjectCompiler::GetNumCompileRequests() const
{
	return static_cast<int32>(CurrentRequest.IsValid()) + CompileRequests.Num();
}


void FCustomizableObjectCompiler::PreloadReferencerAssets()
{
	TRACE_BEGIN_REGION(UE_MUTABLE_PRELOAD_REGION);
	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Preload asynchronously assets start.", FPlatformTime::Seconds());

	TArray<FAssetData> ReferencingAssets;
	GetReferencingPackages(*CurrentObject, ReferencingAssets);

	TArray<FSoftObjectPath> ArrayAssetToStream;
	for (FAssetData& Element : ReferencingAssets)
	{
		ArrayAssetToStream.Add(Element.GetSoftObjectPath());
	}

	bool bAssetsLoaded = true;

	const bool bAsync = CurrentRequest->bAsync;
	if (ArrayAssetToStream.Num() > 0)
	{
		// Customizations are marked as editoronly on load and are not packaged into the runtime game by default.
		// The ones that need to be kept will be copied into SoftObjectPath on the object during save.

		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();
		const TSharedRef<FMutableStreamableManager>& Streamable = System->GetPrivate()->StreamableManager;

		if (bAsync && !CurrentOptions.bIsCooking)
		{
			AddCompileNotification(LOCTEXT("LoadingReferencerAssets", "Loading assets"));

			AsynchronousStreamableHandlePtr = Streamable->RequestAsyncLoad(ArrayAssetToStream, FStreamableDelegate::CreateRaw(this, &FCustomizableObjectCompiler::PreloadingReferencerAssetsCallback, bAsync));
			bAssetsLoaded = false;
		}
		else
		{
			Streamable->RequestSyncLoad(ArrayAssetToStream);
		}
	}

	if (bAssetsLoaded)
	{
		PreloadingReferencerAssetsCallback(bAsync);
	}
}


void FCustomizableObjectCompiler::PreloadingReferencerAssetsCallback(bool bAsync)
{
	check(IsInGameThread());

	check(ArrayGCProtect.IsEmpty());
	
	if (AsynchronousStreamableHandlePtr)
	{
		TArray<FSoftObjectPath> AssetsToStream;
		AsynchronousStreamableHandlePtr->GetRequestedAssets(AssetsToStream);

		for (FSoftObjectPath AssetToStream : AssetsToStream)
		{
			ArrayGCProtect.Add(UE::Mutable::Private::LoadObject(AssetToStream)); // Already loaded.
		}

		AsynchronousStreamableHandlePtr = nullptr;
	}

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Preload asynchronously assets end.", FPlatformTime::Seconds());
	TRACE_END_REGION(UE_MUTABLE_PRELOAD_REGION);

	CompileInternal(bAsync);
}


void FCustomizableObjectCompiler::Compile(const TSharedRef<FCompilationRequest>& InCompileRequest)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectCompiler::Compile)
	
	check(IsInGameThread());

	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		check (AssetRegistry.IsSearchAllAssets());	// At this point the search of all assets should have already been triggered
		check (!AssetRegistry.IsGathering());		// And it should have also been completed as we wait for it to be done before asking for the compilation
	}
	
	UCustomizableObject* Object = InCompileRequest->GetCustomizableObject();
	if (!Object)
	{
		UE_LOGF(LogMutable, Warning, "Failed to compile Customizable Object. Object is missing.");
		
		FCompileCallbackParams Params;
		Params.bRequestFailed = true;
    	
		InCompileRequest->Callback.ExecuteIfBound(Params);
		InCompileRequest->CallbackNative.ExecuteIfBound(Params);
		return;
	}
	
	if (InCompileRequest->bSkipIfCompiled &&
		Object->IsCompiled())
	{
		FCompileCallbackParams Params;
		Params.bSkipped = true;
    	
		InCompileRequest->Callback.ExecuteIfBound(Params);
		InCompileRequest->CallbackNative.ExecuteIfBound(Params);
		return;
	}

	if (InCompileRequest->bSkipIfNotOutOfDate)
	{
		TArray<FName> OutOfDatePackages;
		TArray<FName> AddedPackages;
		TArray<FName> RemovedPackages;
		bool bReleaseVersion;
		if (!Object->GetPrivate()->IsCompilationOutOfDate(false, OutOfDatePackages, AddedPackages, RemovedPackages, bReleaseVersion))
		{
			FCompileCallbackParams Params;
			Params.bSkipped = true;
			Params.bCompiled = Object->IsCompiled();

			InCompileRequest->Callback.ExecuteIfBound(Params);
			InCompileRequest->CallbackNative.ExecuteIfBound(Params);
			return;
		}
	}

	TRACE_BEGIN_REGION(UE_MUTABLE_COMPILE_REGION);
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(Object, PackageAccessTrackingOps::NAME_PostLoad);

	check(!CurrentRequest);

	CurrentRequest = InCompileRequest.ToSharedPtr();
	CurrentObject = Object;
	CurrentOptions = InCompileRequest->Options;

	// Add visual separation in States tab between Compilation Messages and Update Messages
	if(CurrentObject->GetPrivate()->CompilationLogger)
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info, FText::FromString("=================== Object Compilation ==================="));
		CurrentObject->GetPrivate()->CompilationLogger->LogMessage(Message);
	}
	
	if (!UCustomizableObjectSystem::IsActive())
	{
		CompilerLog( LOCTEXT("CompilationFailure_MutableIsDisabled","Failed to compile Customizable Object. Mutable is disabled. To enable it set the CVar Mutable.Enabled to true."), CurrentObject, EMessageSeverity::Warning);
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
		return;
	}

	TArray<const UObject*> VisitedCustomizableObjects;
	const UCustomizableObject* RootObject = GraphTraversal::GetRootObject(CurrentObject, &VisitedCustomizableObjects); 
	
	// No root object means a cycle was found when looking for it
	if (!RootObject)
	{
		CompilerLog(LOCTEXT("CompilationFailure_COCircularReferenceFound", "Failed to compile Customizable Object. Customizable Object circular reference found."), CurrentObject, EMessageSeverity::Warning);

		// Report objects part of the cycle
		check(VisitedCustomizableObjects.Num() >= 2);
		if (VisitedCustomizableObjects.Num() >= 2)
		{
			const FString InvalidObjectName = "Unknown";
				
			const FText CycleStartObjectName =  FText::FromString( ensure(VisitedCustomizableObjects[0]) ? VisitedCustomizableObjects[0]->GetName() : InvalidObjectName );
			const FText CycleClosureObjectName = FText::FromString( ensure(VisitedCustomizableObjects[VisitedCustomizableObjects.Num() - 1]) ? VisitedCustomizableObjects[VisitedCustomizableObjects.Num() - 1]->GetName() : InvalidObjectName );
			const FText WarningText = FText::Format(LOCTEXT("CompilationFailure_COCircularReferenceFoundDetails", "Circular reference found starting at '{0}' CO and finishing at '{1}' CO. COs part of the cycle :"),  CycleStartObjectName,  CycleClosureObjectName);
		
			CompilerLog( WarningText , VisitedCustomizableObjects, EMessageSeverity::Warning);
		}

		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
		return;
	}

	if (RootObject->VersionBridge && !RootObject->VersionBridge->GetClass()->ImplementsInterface(UCustomizableObjectVersionBridgeInterface::StaticClass()))
	{
		const FText WarningText = FText::Format(LOCTEXT("CompilationFailure_RootNotImplementingVersionBridgeInterface", "Failed to compile Customizable Object. The VersionBridge asset {0} does not implement the required UCustomizableObjectVersionBridgeInterface"),  FText::FromString(RootObject->VersionBridge.GetName()));
		CompilerLog( WarningText, {RootObject, RootObject->VersionBridge}, EMessageSeverity::Warning);
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
		return;
	}

	if (!CurrentOptions.bIsCooking && IsRunningCookCommandlet())
	{
		CompilerLog( LOCTEXT("CompilationSuspended_CookInProgress", "Editor compilation suspended. Can not compile COs when the cook commandlet is running."), CurrentObject, EMessageSeverity::Info);
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
		return;
	}

	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();

	if (!CurrentRequest->bAsync)
	{
		// Sync compilation. Force finish all pending updates and async compilations
		System->GetPrivate()->BlockTillAllRequestsFinished();
	}

	check(!CurrentObject->GetPrivate()->IsLocked());

	// Lock object during asynchronous asset loading to avoid instance/mip updates and reentrant compilations
	if (!System->LockObject(CurrentObject))
	{
		FString Message = FString::Printf(TEXT("Customizable Object %s is already being compiled or updated. Please wait a few seconds and try again."), *CurrentObject->GetName());
		UE_LOGF(LogMutable, Warning, "%ls", *Message);
		
		FNotificationInfo Info(LOCTEXT("CustomizableObjectBeingCompilerOrUpdated", "Customizable Object compile and/or update still in process. Please wait a few seconds and try again."));
		Info.bFireAndForget = true;
		Info.bUseThrobber = true;
		Info.FadeOutDuration = 1.0f;
		Info.ExpireDuration = 1.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		CurrentObject = nullptr; // Someone else is compiling the CO. Invalidate the CurrentObject pointer to avoid changing the state of the ongoing compilation.
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
		return;
	}

	SetCompilationState(ECompilationStatePrivate::InProgress, ECompilationResultPrivate::Unknown);

	CompilationStartTime = FPlatformTime::Seconds();

	// Platform data to cache all compiled resources
	PlatformData = MakeShared<UE::Mutable::Private::FMutableCachedPlatformData>();

	// Now that we know for sure that the CO is locked and there are no pending updates of instances using the CO,
	// destroy any live update instances, as they become invalid when recompiling the CO
	for (TObjectIterator<UCustomizableObjectInstance> It; It; ++It)
	{
		UCustomizableObjectInstance* Instance = *It;
		if (IsValid(Instance) &&
			Instance->GetCustomizableObject() == CurrentObject)
		{
			Instance->DestroyLiveUpdateInstance();
		}
	}

	{
		const UEnum* OptimizationLevelEnum = StaticEnum<ECustomizableObjectOptimizationLevel>();
		check(OptimizationLevelEnum);

		const FString CurrentOptimizationLevelName = OptimizationLevelEnum->GetNameStringByIndex(CurrentOptions.OptimizationLevel);
		const FString MutableCompilationStartMessage = FString::Printf(TEXT("Compiling Customizable Object %s for platform %s and optimization level \"%s\"."), *CurrentObject->GetName(), *CurrentOptions.TargetPlatform->PlatformName(), *CurrentOptimizationLevelName);

		if (IsRunningCommandlet())
		{
			UE_LOGF(LogMutable, Display, "%ls", *MutableCompilationStartMessage);
		}
		else
		{
			const int32 MaxOptimizationLevelValue = ConvertOptimizationLevel(ECustomizableObjectOptimizationLevel::Maximum);
			if (CurrentOptions.OptimizationLevel == MaxOptimizationLevelValue)
			{
				UE_LOGF(LogMutable, Display, "%ls The Compilation will take more time to run due to the chosen optimization level.", *MutableCompilationStartMessage);
			}
			else
			{
				UE_LOGF(LogMutable, Display, "%ls", *MutableCompilationStartMessage);
			}
		}
		
	}

	if (CurrentOptions.bForceLargeLODBias)
	{
		UE_LOGF(LogMutable, Display, "Compiling Customizable Object with %d LODBias.", CurrentOptions.DebugBias);
	}

	// Create and update compilation progress notification
	const FText UpdateMsg = FText::FromString(FString::Printf(TEXT("Compiling Customizable Objects:\n%s"), *CurrentObject->GetName()));
	if (!CompileNotificationHandle.IsValid())
	{
		CompileNotificationHandle = FSlateNotificationManager::Get().StartProgressNotification(UpdateMsg, NumCompilationRequests);
	}
	else
	{
		const int32 NumCompletedRequests = NumCompilationRequests - GetRemainingWork();
		FSlateNotificationManager::Get().UpdateProgressNotification(CompileNotificationHandle, NumCompletedRequests, NumCompilationRequests, UpdateMsg);
	}

	// DDC check
	if (!TryLoadCompiledDataFromDDC(*CurrentObject))
	{
		// DDC is disabled, proceed with compilation
		PreloadReferencerAssets();
	}
}


void FCustomizableObjectCompiler::EnqueueCompileRequest(const TSharedRef<FCompilationRequest>& CompileRequest, bool bForceRequests)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectCompiler::EnqueueCompileRequest)

	check(IsInGameThread());

	const UCustomizableObject* CustomizableObject = CompileRequest->GetCustomizableObject();
	if (!CustomizableObject)
	{
		FCompileCallbackParams Params;
		Params.bRequestFailed = true;

		CompileRequest->Callback.ExecuteIfBound(Params);
		CompileRequest->CallbackNative.ExecuteIfBound(Params);
		return;
	}
	
	// Clear all previous log messages
	if (TSharedPtr<class IMutableEditorLogger> Logger = CustomizableObject->GetPrivate()->CompilationLogger)
	{
		Logger->ClearLogMessageList();
	}
	
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		if (AssetRegistry.IsSearchAllAssets())
		{
			// The full search of the AR has already been requested, but we do not know if it has ended by this point
			
			// Wait for the search to be completed if this new compilation is SYNC
			if (AssetRegistry.IsGathering() && !CompileRequest->bAsync)
			{
				UE_LOGF(LogMutable, Log, "Waiting for the AR scan to be completed before enqueueing the compilation of the %ls CO.", *CustomizableObject->GetName());
				AssetRegistry.WaitForCompletion();
			}
		}
		else
		{
			// The engine and AR system might not get ticked when running a non-cook commandlet.
			// In that case, force the search to be synchronous to ensure the compilation can proceed.
			const bool bShouldSearchAssetsSync = (IsRunningCommandlet() && !IsRunningCookCommandlet()) ? true : !CompileRequest->bAsync;
			
			UE_LOGF(LogMutable, Display, "Performing full %s Asset Registry search as required by the CO compilation process.", "ASYNC");
			
			// Scan for all the assets async/sync based on the compilation options as the AR has not yet been scanned
			AssetRegistry.SearchAllAssets(bShouldSearchAssetsSync);
		}
	}

	
	if (!CompileRequest->bAsync)
	{
		MUTABLE_CPUPROFILER_SCOPE(SyncCompile)
		
		TSharedRef<FCustomizableObjectCompiler> SyncCompiler = MakeShared<FCustomizableObjectCompiler>();
		SyncCompiler->Compile(CompileRequest);
	}
	else
	{
		if (!bForceRequests && (CustomizableObject->GetPrivate()->IsLocked() || IsRequestQueued(CompileRequest)))
		{
			FCompileCallbackParams Params;
			Params.bRequestFailed = true;
			Params.bCompiled = CompileRequest->GetCustomizableObject()->IsCompiled();

			CompileRequest->Callback.ExecuteIfBound(Params);
			CompileRequest->CallbackNative.ExecuteIfBound(Params);
			return;
		}

		++NumCompilationRequests;
		CompileRequests.Add(CompileRequest);
	}
}


bool FCustomizableObjectCompiler::IsRequestQueued(const TSharedRef<FCompilationRequest>& InCompileRequest) const
{
	return CurrentRequest == InCompileRequest || 
		CompileRequests.ContainsByPredicate([&InCompileRequest](const TSharedRef<FCompilationRequest>& Other)
		{
			return InCompileRequest.Get() == Other.Get(); // Compare the content of the request not the ref
		});
}


bool FCustomizableObjectCompiler::IsRequestQueued(const UCustomizableObject& Object) const
{
	return (CurrentRequest && CurrentRequest->GetCustomizableObject() == &Object) || 
		CompileRequests.ContainsByPredicate([ObjectPtr = &Object](const TSharedRef<FCompilationRequest>& Other)
		{
			return ObjectPtr == Other->GetCustomizableObject();
		});
}


void FCustomizableObjectCompiler::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ArrayGCProtect);
	Collector.AddReferencedObject(CurrentObject);
}


void FCustomizableObjectCompiler::AddGameThreadCompileTask(TFunction<void()>&& Task)
{
	PendingGameThreadCompileTasks.Enqueue(Task);
}


void ProcessChildObjectsRecursively(const UCustomizableObject* ParentObject, FMutableGraphGenerationContext& GenerationContext)
{
	TArray<FName> ReferencedObjectNames;
	
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().GetReferencers(*ParentObject->GetOuter()->GetPathName(), ReferencedObjectNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

	if (ReferencedObjectNames.IsEmpty())
	{
		return;
	}
	
	TArray<FAssetData> AssetDataArray;

	FARFilter Filter;
	Filter.PackageNames = MoveTemp(ReferencedObjectNames);
	Filter.ClassPaths = { UCustomizableObject::StaticClass()->GetClassPathName() };
	Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only enumerate on-disk assets- avoids indeterministic results

	AssetRegistryModule.Get().GetAssets(Filter, AssetDataArray);

	// Required to be deterministic.
	AssetDataArray.Sort([](const FAssetData& A, const FAssetData& B)
	{
		return A.PackageName.LexicalLess(B.PackageName);
	});
	
	for (FAssetData AssetData : AssetDataArray)
	{
		FSoftObjectPath SoftObjectPath = AssetData.GetSoftObjectPath();
		
		UCustomizableObject* ChildObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(SoftObjectPath));
		if (!ChildObject || ChildObject->HasAnyFlags(RF_Transient))
		{
			continue;
		}

		UCustomizableObjectNodeObject* Root = GetRootNode(ChildObject);
		if (!Root)
		{
			continue;
		}
		
		if (Root->ParentObject != ParentObject)
		{
			continue;
		}

		if (ChildObject->VersionStruct.IsValid())
		{
			if (!GenerationContext.RootVersionBridge)
			{
				UE_LOGF(LogMutable, Warning, "The child Customizable Object [%ls] defines its VersionStruct Property but its root CustomizableObject doesn't define the VersionBridge property. There's no way to verify the VersionStruct has to be included in this compilation, so the child CustomizableObject will be omitted.", 
					*ChildObject->GetName());
				continue;
			}

			ICustomizableObjectVersionBridgeInterface* CustomizableObjectVersionBridgeInterface = Cast<ICustomizableObjectVersionBridgeInterface>(GenerationContext.RootVersionBridge);
			if (ensure(CustomizableObjectVersionBridgeInterface))
			{
				if (!CustomizableObjectVersionBridgeInterface->IsVersionStructIncludedInCurrentRelease(ChildObject->VersionStruct))
				{
					continue;
				}
			}
		}
		
		if (const FGroupNodeIdsTempData* GroupGuid = GenerationContext.DuplicatedGroupNodeIds.FindPair(ParentObject, FGroupNodeIdsTempData(Root->ParentObjectGroupId)))
		{
			Root->ParentObjectGroupId = GroupGuid->NewGroupNodeId;
		}

		GenerationContext.GroupIdToExternalNodeMap.Add(Root->ParentObjectGroupId, Root);

		TArray<UCustomizableObjectNodeObjectGroup*> GroupNodes; 
		ChildObject->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeObjectGroup>(GroupNodes);

		if (GroupNodes.Num() > 0) // Only grafs with group nodes should have child grafs
		{
			for (int32 i = 0; i < GroupNodes.Num(); ++i)
			{
				const FGuid NodeId = GenerationContext.GetNodeIdUnique(GroupNodes[i]);
				if (NodeId != GroupNodes[i]->NodeGuid)
				{
					GenerationContext.DuplicatedGroupNodeIds.Add(ChildObject, FGroupNodeIdsTempData(GroupNodes[i]->NodeGuid, NodeId));
					GroupNodes[i]->NodeGuid = NodeId;
				}
			}

			ProcessChildObjectsRecursively(ChildObject, GenerationContext);
		}
	}
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> GenerateMutableRoot( 
	const UCustomizableObject* Object, 
	FMutableGraphGenerationContext& GenerationContext)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateMutableRoot)

	check(Object);
	
	if (!Object->GetPrivate()->GetSource())
	{
		GenerationContext.Log(LOCTEXT("NoSource", "Object with no valid graph found. Object not build."));

		if (IsRunningCookCommandlet() || IsRunningCookOnTheFly())
		{
			UE_LOGF(LogMutable, Warning, "Compilation failed! Missing EDITORONLY data for Customizable Object [%ls]. The object might have been loaded outside the Cooking context.", *Object->GetName());
		}

		return nullptr;
	}

	UCustomizableObjectNodeObject* LocalRootNodeObject = GetRootNode(Object);
	if (!LocalRootNodeObject)
	{
		GenerationContext.Log(LOCTEXT("NoRootBase","No base object node found. Object not built."));
		return nullptr;
	}
	
	const UCustomizableObject* RootObject = GenerationContext.CompilationContext->RootObject.Get();
	
	GenerationContext.RootVersionBridge = RootObject->VersionBridge;

	UCustomizableObjectNodeObject* RootNodeObject = GetRootNode(RootObject);
	GenerationContext.Root = RootNodeObject;

	if (!RootNodeObject)
	{
		GenerationContext.Log(LOCTEXT("NoActualRootBase", "No base object node found in root Customizable Object. Object not built."));
		return nullptr;
	}
	
	if (LocalRootNodeObject->GetObjectName(&GenerationContext.MacroNodesStack).IsEmpty())
	{
		GenerationContext.NoNameNodeObjectArray.AddUnique(LocalRootNodeObject);
	}

	if ((Object->GetPrivate()->GetMeshCompileType() == EMutableCompileMeshType::Full) || GenerationContext.CompilationContext->Options.bIsCooking)
	{
		if (LocalRootNodeObject->ParentObject!=nullptr && GenerationContext.CompilationContext->Options.bIsCooking)
		{
			// This happens while packaging.
			return nullptr;
		}

		// We cannot load while saving. This should only happen in cooking and all assets should have been preloaded.
		if (!UE::IsSavingPackage())
		{
			UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Begin search for children.", FPlatformTime::Seconds());

			// The object doesn't reference a root object but is a root object, look for all the objects that reference it and get their root nodes
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
			ProcessChildObjectsRecursively(RootObject, GenerationContext);
			UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] End search for children.", FPlatformTime::Seconds());
		}
	}
	else
	{
		// Local, local with children and working set modes: add parents until whole CO graph root
		TArray<UCustomizableObjectNodeObject*> ArrayNodeObject;
		TArray<const UCustomizableObject*> ArrayCustomizableObject;
		
		if (!GetParentsUntilRoot(Object, ArrayNodeObject, ArrayCustomizableObject))
		{
			GenerationContext.Log(LOCTEXT("SkeletalMeshCycleFound", "Error! Cycle detected in the Customizable Object hierarchy."), LocalRootNodeObject);
			return nullptr;
		}

		if ((Object->GetPrivate()->GetMeshCompileType() == EMutableCompileMeshType::AddWorkingSetNoChildren) ||
			(Object->GetPrivate()->GetMeshCompileType() == EMutableCompileMeshType::AddWorkingSetAndChildren))
		{
			const int32 MaxIndex = Object->GetPrivate()->GetWorkingSet().Num();
			for (int32 i = 0; i < MaxIndex; ++i)
			{
				if (UCustomizableObject* WorkingSetObject = GenerationContext.LoadObject(Object->GetPrivate()->GetWorkingSet()[i]))
				{
					ArrayCustomizableObject.Reset();

					if (!GetParentsUntilRoot(WorkingSetObject, ArrayNodeObject, ArrayCustomizableObject))
					{
						GenerationContext.Log(LOCTEXT("NoReferenceMesh", "Error! Cycle detected in the Customizable Object hierarchy."), LocalRootNodeObject);
						return nullptr;
					}
				}
			}
		}

		if ((Object->GetPrivate()->GetMeshCompileType() == EMutableCompileMeshType::LocalAndChildren) ||
			(Object->GetPrivate()->GetMeshCompileType() == EMutableCompileMeshType::AddWorkingSetAndChildren))
		{
			TArray<UCustomizableObjectNodeObjectGroup*> GroupNodes;
			Object->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeObjectGroup>(GroupNodes);

			if (GroupNodes.Num() > 0) // Only graphs with group nodes should have child graphs
			{
				ProcessChildObjectsRecursively(Object, GenerationContext);
			}
		}

		for (int32 i = 0; i < ArrayNodeObject.Num(); ++i)
		{
			if (GenerationContext.GroupIdToExternalNodeMap.FindKey(ArrayNodeObject[i]) == nullptr)
			{
				GenerationContext.GroupIdToExternalNodeMap.Add(ArrayNodeObject[i]->ParentObjectGroupId, ArrayNodeObject[i]);
			}
		}
	}

	
	// First pass. Only used to recollect info required for the primary pass.
	// Notice that the traversal is different form the primary pass. Here we follow all pins indiscriminately,
	// while the primary pass follows the Mutable Source structure (which may cut branches).
	GraphTraversal::VisitNodes(*RootNodeObject, [&GenerationContext](UCustomizableObjectNode& Node)
	{
		if (UCONodeComponentSkeletalMesh* NodeComponentMesh = Cast<UCONodeComponentSkeletalMesh>(&Node))
		{
			FirstPass(*NodeComponentMesh, GenerationContext);
		}
	}, &GenerationContext.GroupIdToExternalNodeMap, &GenerationContext.MacroNodesStack);
	
    GenerationContext.CompilationContext->RealTimeMorphTargetsOverrides = RootNodeObject->RealTimeMorphSelectionOverrides;

	if (!GenerationContext.CompilationContext->Options.ParamNamesToSelectedOptions.IsEmpty())
	{
		if (const UModelResources* ModelResources = Object->GetPrivate()->GetModelResources())
		{
			GenerationContext.TableToParamNames = ModelResources->TableToParamNames;
		}
	}

	GenerationContext.bPartialCompilation = LocalRootNodeObject->ParentObject != nullptr;

	// Generate the object expression
	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] GenerateMutableSource start.", FPlatformTime::Seconds());
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> MutableRoot = GenerateMutableSource(RootNodeObject->OutputPin(), GenerationContext);
	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] GenerateMutableSource end.", FPlatformTime::Seconds());

	// Generate ReferenceSkeletalMeshes data;
	PopulateReferenceSkeletalMeshesData(GenerationContext);

	// Display warnings for unnamed node objects
	FText Message = LOCTEXT("Unnamed Node Object", "Unnamed Node Object");
	for (const UCustomizableObjectNode* It : GenerationContext.NoNameNodeObjectArray)
	{
		GenerationContext.Log(Message, It, EMessageSeverity::Warning, true);
	}

	// If duplicated node ids are found, usually due to duplicating CustomizableObjects Assets, a warning
	// for the nodes with repeated ids will be generated
	for (const TPair<FGuid, TArray<const UObject*>>& It : GenerationContext.NodeIdsMap)
	{
		if (It.Value.Num() > 1)
		{
			FText MessageWarning = LOCTEXT("NodeWithRepeatedIds", "Several nodes have repeated NodeIds, reconstruct the nodes.");
			GenerationContext.Log(MessageWarning, It.Value, EMessageSeverity::Warning, true);
		}
	}

	// Display a warning for each node contains an orphan pin.
	for (const TPair<FGeneratedKey, FGeneratedData>& It : GenerationContext.Generated)
	{
		if (const UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(It.Value.Source))
		{
			if (Node->GetAllOrphanPins().Num() > 0)
			{
				GenerationContext.Log(LOCTEXT("OrphanPinsWarningCompiler", "Node contains deprecated pins"), Node, EMessageSeverity::Warning, false);
			}
		}
	}

	if (GenerationContext.CustomizableObjectWithCycle)
	{
		GenerationContext.Log(FText::Format(LOCTEXT("CycleDetected","Cycle detected in graph of CustomizableObject {0}. Object not built."),
			FText::FromString(GenerationContext.CustomizableObjectWithCycle->GetPathName())));

		return nullptr;
	}

	return MutableRoot;
}


void FCustomizableObjectCompiler::SaveCODerivedData()
{
	if (!SaveDDTask.IsValid())
	{
		return;
	}

	AddCompileNotification(LOCTEXT("SavingCustomizableObjectDerivedData", "Saving Data"));

	// Even for async saving derived data.
	static int SDDThreadCount = 0;
	FString ThreadName = FString::Printf(TEXT("MutableSDD-%03d"), ++SDDThreadCount);
	SaveDDThread = MakeShareable(FRunnableThread::Create(SaveDDTask.Get(), *ThreadName));
}


ECompilationResultPrivate FCustomizableObjectCompiler::GetCompilationResult() const
{
	if (CompilationLogsContainer.GetErrorCount())
	{
		return ECompilationResultPrivate::Errors;
	}
	else if (CompilationLogsContainer.GetWarningCount(true))
	{
		return ECompilationResultPrivate::Warnings;
	}
	else
	{
		return ECompilationResultPrivate::Success;
	}
}


void FCustomizableObjectCompiler::ProcessCompileTasks()
{
	MUTABLE_CPUPROFILER_SCOPE(CompileTasks);

	check(IsInGameThread());

	// See if there are compile-time tasks to run
	constexpr double MaxSecondsPerFrame = 0.1;
	double MaxTime = FPlatformTime::Seconds() + MaxSecondsPerFrame;

	TFunction<void()> Task;
	while (PendingGameThreadCompileTasks.Dequeue(Task))
	{
		Task();

		// Simple time limit enforcement to avoid blocking the game thread if there are many requests.
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime >= MaxTime)
		{
			break;
		}
	}
}


void FCustomizableObjectCompiler::SetCompilationState(ECompilationStatePrivate State, ECompilationResultPrivate Result) const
{
	check(CurrentRequest);
	CurrentRequest->SetCompilationState(State, Result);

	if (CurrentObject)
	{
		CurrentObject->GetPrivate()->CompilationResult = Result;
	}
}


void FCustomizableObjectCompiler::CompileInternal(bool bAsync)
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectCompiler::CompileInternal)
	
	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] FCustomizableObjectCompiler::Compile start.", FPlatformTime::Seconds());
	
	// This is redundant but necessary to keep static analysis happy.
	if (!CurrentObject)
	{
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
		return;
	}

	const UCustomizableObject* RootObject = GraphTraversal::GetRootObject(CurrentObject);
	check(RootObject);
	
	CompilationContext = MakeShared<FMutableCompilationContext>(RootObject, SharedThis(this), CurrentOptions);
	FMutableGraphGenerationContext GenerationContext(CompilationContext);

	// Perform a first participating objects pass
	TMap<FName, FGuid> ParticipatingObjects = ICustomizableObjectEditorModule::GetChecked().GetParticipatingObjects(CurrentObject, &CurrentOptions);

	// Clear Messages from previous Compilations
	CompilationLogsContainer.ClearMessageCounters();
	CompilationLogsContainer.ClearMessagesArray();

	AddCompileNotification(LOCTEXT("GraphCompilationMessage", "Compiling Graph"));

	// Generate the mutable node expression
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> MutableRoot = GenerateMutableRoot(CurrentObject, GenerationContext);
	if (!MutableRoot)
	{
		CompilerLog(FText(LOCTEXT("FailedToGenerateRoot", "Failed to generate the mutable node graph. Object not built.")));
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
	}
	else
	{
		TArray<FMutableSourceTextureData> NewCompileTimeReferencedTextures;
		{
			for (const TPair<TSoftObjectPtr<UTexture2D>, FMutableGraphGenerationContext::FGeneratedReferencedTexture>& Pair : GenerationContext.CompileTimeTextureMap)
			{
				check(Pair.Value.ID == NewCompileTimeReferencedTextures.Num());

				FMutableSourceTextureData Tex(*UE::Mutable::Private::LoadObject(Pair.Key));
				NewCompileTimeReferencedTextures.Add(Tex);
			}
		}

		TArray<FMutableSourceMeshData> NewCompileTimeReferencedMeshes;
		for (const TPair<FMutableSourceMeshData, FMutableGraphGenerationContext::FGeneratedReferencedMesh>& Pair : GenerationContext.CompileTimeMeshMap)
		{
			check(Pair.Value.ID == NewCompileTimeReferencedMeshes.Num());
			NewCompileTimeReferencedMeshes.Add(Pair.Key);
		}

		// The first part of model resources has to be filled from the GenerationContext.
		// The second part will be filled when the core compilation finishes so that the CompilationContext is complete.
		TObjectPtr<UModelResources> ModelResources = NewObject<UModelResources>(GetTransientPackage(), NAME_None, RF_Public);
		PlatformData->ModelResources = TStrongObjectPtr<UModelResources>(ModelResources);
		
		PlatformData->ModelStreamableBulkData = MakeShared<FModelStreamableBulkData>();

		ModelResources->CodeVersion = GetECustomizableObjectVersionEnumHash();

		ModelResources->ReferenceSkeletalMeshesData = MoveTemp(GenerationContext.ReferenceSkeletalMeshesData);
		
		for (const TPair<TSoftObjectPtr<UTexture2D>, FMutableGraphGenerationContext::FGeneratedReferencedTexture>& Pair : GenerationContext.RuntimeReferencedTextureMap)
		{
			check(Pair.Value.ID == ModelResources->RuntimeReferencedTextures.Num());
			ModelResources->RuntimeReferencedTextures.Add(Pair.Key);
		}

		ModelResources->TextureParameterDefaultValues = MoveTemp(GenerationContext.TextureParameterDefaultValues);
		ModelResources->SkeletalMeshParameterDefaultValues = MoveTemp(GenerationContext.SkeletalMeshParameterDefaultValues);
		ModelResources->MaterialParameterDefaultValues = MoveTemp(GenerationContext.MaterialParameterDefaultValues);
		ModelResources->ExternalTypeParameterDefaultValues = MoveTemp(GenerationContext.ExternalTypeParameterDefaultValues);

		ModelResources->ExternalTypeParameterTypes = MoveTemp(GenerationContext.ExternalTypeParameterTypes);

		ModelResources->AnimBPs = MoveTemp(GenerationContext.AnimBPAssets);

		TArray<FGeneratedImageProperties> ImageProperties;
		GenerationContext.ImageProperties.GenerateValueArray(ImageProperties);
		
		// Must sort image properties by ImagePropertiesIndex so that ImageNames point to the right properties.
		ImageProperties.Sort([](const FGeneratedImageProperties& PropsA, const FGeneratedImageProperties& PropsB)
			{ return PropsA.ImagePropertiesIndex < PropsB.ImagePropertiesIndex;	});

		ModelResources->ImageProperties.Empty(ImageProperties.Num());

		for (const FGeneratedImageProperties& ImageProp : ImageProperties)
		{
			ModelResources->ImageProperties.Add({ ImageProp.TextureParameterName,
										ImageProp.Filter,
										ImageProp.SRGB,
										ImageProp.bFlipGreenChannel,
										ImageProp.bIsPassThrough,
										ImageProp.LODBias,
										ImageProp.MipGenSettings,
										ImageProp.LODGroup,
										ImageProp.AddressX, ImageProp.AddressY });
		}

		ModelResources->ParameterUIDataMap = MoveTemp(GenerationContext.ParameterUIDataMap);
		ModelResources->StateUIDataMap = MoveTemp(GenerationContext.StateUIDataMap);
		ModelResources->IntParameterOptionDataTable = MoveTemp(GenerationContext.IntParameterOptionDataTable);

		ModelResources->GroupNodeMap = GenerationContext.GroupNodeMap;

		// If the optimization level is "none" disable texture streaming, because textures are all referenced
		// unreal assets and progressive generation is not supported.
		ModelResources->bIsTextureStreamingDisabled = CurrentOptions.OptimizationLevel == 0;

		ModelResources->bIsCompiledWithOptimization = CurrentOptions.OptimizationLevel == UE_MUTABLE_MAX_OPTIMIZATION;

		ModelResources->bCompiledWithHDTextureCompression = CurrentOptions.TextureCompression == ECustomizableObjectTextureCompression::HighQuality;

#if WITH_EDITORONLY_DATA
		// Cache the tables that are used by more than one param so that CompileOnlySelected can work properly
		ModelResources->TableToParamNames = MoveTemp(GenerationContext.TableToParamNames);
		ModelResources->CustomizableObjectPathMap = MoveTemp(GenerationContext.CustomizableObjectPathMap);
#endif

		ModelResources->ComponentNamesPerObjectComponent = MoveTemp(GenerationContext.ComponentNames);

		if (ICustomizableObjectVersionBridgeInterface* VersionBridge = Cast<ICustomizableObjectVersionBridgeInterface>(GraphTraversal::GetRootObject(CurrentObject)->VersionBridge))
		{
			ModelResources->ReleaseVersion = VersionBridge->GetCurrentVersionAsString();
		}
		
		ModelResources->ParticipatingObjects = MoveTemp(ParticipatingObjects);
		
		if (CurrentOptions.bGatherReferences)
		{
			CurrentObject->GetPrivate()->ReferencedObjects = Cast<UModelResources>(StaticDuplicateObject(ModelResources, CurrentObject));
			CurrentObject->GetPrivate()->ReferencedObjects->RuntimeReferencedTextures.Empty(); // Empty in case the of none optimization. In maximum optimization, they are Mutable textures. 
			CurrentObject->Modify();
		}

		CompileTask = MakeShareable(new FCustomizableObjectCompileRunnable(MutableRoot, SharedThis(this)));
		CompileTask->Options = CurrentOptions;
		CompileTask->ReferencedTextures = NewCompileTimeReferencedTextures;
		CompileTask->ReferencedMeshes = NewCompileTimeReferencedMeshes;

		if (!bAsync)
		{
			CompileTask->Init();
			CompileTask->Run();
			FinishCompilationTask();

			if (SaveDDTask.IsValid())
			{
				SaveDDTask->Init();
				SaveDDTask->Run();
				FinishSavingDerivedDataTask();
			}

			CompleteRequest(ECompilationStatePrivate::Completed, GetCompilationResult());
		}
		else
		{
			AddCompileNotification(LOCTEXT("CustomizableObjectCompileInProgress", "Compiling Object"));

			// Even for async build, we spawn a thread, so that we can set a large stack. 
			// Thread names need to be unique, apparently.
			static int32 ThreadCount = 0;
			FString ThreadName = FString::Printf(TEXT("MutableCompile-%03d"), ++ThreadCount);
			CompileThread = MakeShareable(FRunnableThread::Create(CompileTask.Get(), *ThreadName, 16 * 1024 * 1024, TPri_Normal));
		}
	}

	for (UCustomizableObjectNode* Node : GenerationContext.GeneratedNodes)
	{
		Node->ResetAttachedErrorData();
	}

	// Population Recompilation
	if (MutableRoot)
	{
		//Checking if there is the population plugin
		if (FModuleManager::Get().IsModuleLoaded("CustomizableObjectPopulation"))
		{
			ICustomizableObjectPopulationModule::Get().RecompilePopulations(CurrentObject);
		}
	}
}


void FCustomizableObjectCompiler::CompleteRequest(ECompilationStatePrivate State, ECompilationResultPrivate Result)
{
	check(IsInGameThread());
	check(CurrentRequest);

	ECompilationStatePrivate CurrentState = CurrentRequest->GetCompilationState();
	SetCompilationState(State, Result);

	if (CurrentState == ECompilationStatePrivate::InProgress && CurrentObject)
	{
		// Unlock the object so that instances can be updated
		UCustomizableObjectSystem* System = UCustomizableObjectSystem::IsCreated() ? UCustomizableObjectSystem::GetInstance() : nullptr;
		if (System && !System->HasAnyFlags(EObjectFlags::RF_BeginDestroyed))
		{	
			System->UnlockObject(CurrentObject);
		}

		if (PlatformData->Model)
		{
			PlatformData->Model->RomManager.UnloadRoms();
		}

		if (!CurrentOptions.bIsCooking)
		{
			if (Result == ECompilationResultPrivate::Success || Result == ECompilationResultPrivate::Warnings)
			{
				CurrentObject->GetPrivate()->SetModelResources(PlatformData->ModelResources.Get(), CurrentOptions.bIsCooking);
				CurrentObject->GetPrivate()->SetModelStreamableBulkData(PlatformData->ModelStreamableBulkData, CurrentOptions.bIsCooking);
				CurrentObject->GetPrivate()->SetModel(PlatformData->Model, GenerateIdentifier(*CurrentObject), CurrentOptions.bIsCooking);
			}
			else
			{
				CurrentObject->GetPrivate()->SetModelResources(nullptr, CurrentOptions.bIsCooking);
				CurrentObject->GetPrivate()->SetModelStreamableBulkData(nullptr, CurrentOptions.bIsCooking);
				CurrentObject->GetPrivate()->SetModel(nullptr, {}, CurrentOptions.bIsCooking);
			}

			CurrentObject->GetPrivate()->PostCompile();
		}

		UE_LOGF(LogMutable, Display, "Finished compiling Customizable Object %ls. Compilation took %5.3f seconds to complete.",
			*CurrentObject->GetName(), FPlatformTime::Seconds() - CompilationStartTime);
	}

	// Remove referenced objects
	ArrayGCProtect.Empty();

	// Notifications
	RemoveCompileNotification();
	NotifyCompilationErrors();

	if (CurrentObject && CurrentObject->GetPrivate()->CompilationLogger)
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info, FText::FromString("=========================================================="));
		CurrentObject->GetPrivate()->CompilationLogger->LogMessage(Message);
	}

	// Update compilation progress notification
	if (CompileNotificationHandle.IsValid())
	{
		const int32 NumCompletedRequests = NumCompilationRequests - CompileRequests.Num();
		FSlateNotificationManager::Get().UpdateProgressNotification(CompileNotificationHandle, NumCompletedRequests, NumCompilationRequests);

		if (NumCompletedRequests == NumCompilationRequests)
		{
			// Remove progress bar
			FSlateNotificationManager::Get().CancelProgressNotification(CompileNotificationHandle);
			CompileNotificationHandle.Reset();
			NumCompilationRequests = 0;
		}
	}

	// Copy warnings and errors to the request
	CompilationLogsContainer.GetMessages(CurrentRequest->Warnings, CurrentRequest->Errors);
	
	// Clear Messages
	CompilationLogsContainer.ClearMessageCounters();
	CompilationLogsContainer.ClearMessagesArray();

	if (GEngine)
	{
		GEngine->ForceGarbageCollection();
	}

	FCompileCallbackParams Params;
	Params.bErrors = Result == ECompilationResultPrivate::Errors;
	Params.bWarnings = Result == ECompilationResultPrivate::Warnings;
	
	if (CurrentObject)
	{
		Params.bCompiled = CurrentObject->IsCompiled();
		Params.bErrors |= !CurrentRequest->Errors.IsEmpty();
		Params.bWarnings |= !CurrentRequest->Warnings.IsEmpty();
	}
		
	CurrentRequest->Callback.ExecuteIfBound(Params);
	CurrentRequest->CallbackNative.ExecuteIfBound(Params);

	// Request completed, reset pointers and state
	CurrentObject = nullptr;
	CurrentRequest.Reset();
	PlatformData.Reset();
	

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Completed compile request.", FPlatformTime::Seconds());
	UE_LOGF(LogMutable, Verbose, "PROFILE: -----------------------------------------------------------");
}


bool FCustomizableObjectCompiler::TryPopCompileRequest()
{
	if (CurrentRequest.IsValid() || CompileRequests.IsEmpty())
	{
		return false;
	}

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	check(AssetRegistry.IsSearchAllAssets());	// in all cases this should return true as we are taking care of calling the searching of the AR before being able to add a request
	if (AssetRegistry.IsGathering())
	{
		// AR search is in progress, skip the compilation operation for now until it is ready.
		return false;
	}
	
	UCustomizableObjectSystemPrivate* SystemPrivate = UCustomizableObjectSystem::GetInstance()->GetPrivate();
	if (SystemPrivate->CurrentMutableOperation)
	{
		return false;
	}

	for (TArray<TSharedRef<FCompilationRequest>>::TIterator It = CompileRequests.CreateIterator(); It; ++It)
	{
		TSharedRef<FCompilationRequest>& CompileRequest = *It;

		UCustomizableObject* Object = CompileRequest->GetCustomizableObject();
		if (!Object)
		{
			TSharedRef<FCompilationRequest> SelectedCompileRequest = CompileRequest;
			It.RemoveCurrent();
			
			FCompileCallbackParams Params;
			Params.bRequestFailed = true;
    	
			SelectedCompileRequest->Callback.ExecuteIfBound(Params);
			SelectedCompileRequest->CallbackNative.ExecuteIfBound(Params);
			continue;
		}
		
		if (CompileRequest->Options.bIsCooking)
		{
			TSharedRef<FCompilationRequest> SelectedCompileRequest = CompileRequest;
			It.RemoveCurrent(); // Preserve order.

			Compile(SelectedCompileRequest);
			return true;
		}

		switch (Object->GetPrivate()->Status.Get())
		{
		case FCustomizableObjectStatusTypes::EState::Loading:
			// Wait.
			break;

		case FCustomizableObjectStatusTypes::EState::NoModel:
		case FCustomizableObjectStatusTypes::EState::ModelLoaded:
			{
				TSharedRef<FCompilationRequest> SelectedCompileRequest = CompileRequest;
				It.RemoveCurrent(); // Preserve order.

				Compile(SelectedCompileRequest);
				return true;
			}
			
		default:
			unimplemented();
		}
	}

	return false;
}


bool FCustomizableObjectCompiler::TryLoadCompiledDataFromDDC(UCustomizableObject& CustomizableObject)
{
	if (!CurrentRequest.IsValid())
	{
		return false;
	}

	using namespace UE::DerivedData;

	ECachePolicy DefaultPolicy = CurrentRequest->GetDerivedDataCachePolicy();
	if (!CurrentOptions.bQueryCompiledDatafromDDC)
	{
		// Compilation not allowed to query DDC requests. 
		return false;
	}

	MUTABLE_CPUPROFILER_SCOPE(TryLoadCompiledDataFromDDC);

	CurrentRequest->BuildDerivedDataCacheKey();

	FCacheKey CacheKey = CurrentRequest->GetDerivedDataCacheKey();
	check(CacheKey.Hash.IsZero() == false);

	using namespace UE::DerivedData;

	DDCHeapMemory = MakeShared<FDDCHeapMemory>();

	/* Overview.
	*	1. Create an initial pull request to look for the compiled data in the DDC. Skip streamable binary blobs.
	*	2. Try to load the compiled data.
	*	3. (Cooking) Create a second request to pull all streamable blobs and cache the compiled data.
	*/

	// Set the request policy to Default + SkipData to avoid pulling the streamable files until we know the compiled data can be used.
	FCacheRecordPolicyBuilder PolicyBuilder(DefaultPolicy | ECachePolicy::SkipData);

	// Overwrite the request policy for the resources we want to pull
	PolicyBuilder.AddValuePolicy(UE::Mutable::Private::GetDerivedDataModelResourcesId(), DefaultPolicy);
	PolicyBuilder.AddValuePolicy(UE::Mutable::Private::GetDerivedDataModelId(), DefaultPolicy);
	PolicyBuilder.AddValuePolicy(UE::Mutable::Private::GetDerivedDataModelStreamableBulkDataId(), DefaultPolicy);
	PolicyBuilder.AddValuePolicy(UE::Mutable::Private::GetDerivedDataBulkDataFilesId(), DefaultPolicy);

	FCacheGetRequest Request;
	Request.Name = GetPathNameSafe(CurrentObject);
	Request.Key = CacheKey;
	Request.Policy = PolicyBuilder.Build();

	LoadModelDataFromDDCEvent = MakeShared<UE::Tasks::FTaskEvent>(TEXT("TryGetBaseDataFromDDC"));

	// Sync request to retrieve the compiled data for validation. Streamable resources are excluded.
	FRequestOwner RequestOwner(!CurrentRequest->bAsync ? EPriority::Blocking : EPriority::Highest);
	GetCache().Get(MakeArrayView(&Request, 1), RequestOwner,
		[DDCHeapMemory = DDCHeapMemory, CompletionEvent = LoadModelDataFromDDCEvent](FCacheGetResponse&& Response) mutable
		{
			MUTABLE_CPUPROFILER_SCOPE(RetrieveModelDataFromDDC);
			
			if (Response.Status == EStatus::Ok)
			{
				const FCompressedBuffer& ModelCompressedBuffer = Response.Record.GetValue(UE::Mutable::Private::GetDerivedDataModelId()).GetData();
				DDCHeapMemory->ModelBytesDDC = ModelCompressedBuffer.Decompress();

				const FCompressedBuffer& ModelResourcesCompressedBuffer = Response.Record.GetValue(UE::Mutable::Private::GetDerivedDataModelResourcesId()).GetData();
				DDCHeapMemory->ModelResourcesBytesDDC = ModelResourcesCompressedBuffer.Decompress();

				const FCompressedBuffer& ModelStreamablesCompressedBuffer = Response.Record.GetValue(UE::Mutable::Private::GetDerivedDataModelStreamableBulkDataId()).GetData();
				DDCHeapMemory->ModelStreamablesBytesDDC = ModelStreamablesCompressedBuffer.Decompress();

				const FCompressedBuffer& BulkDataFilesCompressedBuffer = Response.Record.GetValue(UE::Mutable::Private::GetDerivedDataBulkDataFilesId()).GetData();
				DDCHeapMemory->BulkDataFilesBytesDDC = BulkDataFilesCompressedBuffer.Decompress();
			}
			
			CompletionEvent->Trigger();
		});

	if (!CurrentRequest->bAsync)
	{
		// Force sync
		RequestOwner.Wait();
		FinishLoadingModelDataFromDDC();
		FinishLoadingStreamableDataFromDDC();
	}
	else
	{
		RequestOwner.KeepAlive();
	}

	return true;
}


void FCustomizableObjectCompiler::FinishLoadingModelDataFromDDC()
{
	MUTABLE_CPUPROFILER_SCOPE(FinishLoadingModelDataFromDDC);

	LoadModelDataFromDDCEvent.Reset();

	using namespace UE::DerivedData;

	bool bHasValidData = false;

	// Check if it is possible to use the data stored in DDC.
	if (!DDCHeapMemory->ModelBytesDDC.IsNull() && !DDCHeapMemory->ModelResourcesBytesDDC.IsNull()
		&& !DDCHeapMemory->ModelStreamablesBytesDDC.IsNull() && !DDCHeapMemory->BulkDataFilesBytesDDC.IsNull())
	{
		MUTABLE_CPUPROFILER_SCOPE(LoadModelDataFromDDC);

		// Load the compiled data to validate it.
		FMemoryReaderView ModelResourcesReader(DDCHeapMemory->ModelResourcesBytesDDC.GetView());

		if (TObjectPtr<UModelResources> LocalModelResources = LoadModelResources_Internal(ModelResourcesReader, CurrentObject, CurrentOptions.TargetPlatform, CurrentOptions.bIsCooking))
		{
			PlatformData->ModelResources = TStrongObjectPtr<UModelResources>(LocalModelResources);

			FMemoryReaderView ModelStreamablesReader(DDCHeapMemory->ModelStreamablesBytesDDC.GetView());
			PlatformData->ModelStreamableBulkData = LoadModelStreamableBulk_Internal(ModelStreamablesReader);
			PlatformData->ModelStreamableBulkData->bIsStoredInDDC = true;
			PlatformData->ModelStreamableBulkData->DDCKey = CurrentRequest->GetDerivedDataCacheKey();
			PlatformData->ModelStreamableBulkData->DDCDefaultPolicy = ECachePolicy::Default;

			FMemoryReaderView ModelReader(DDCHeapMemory->ModelBytesDDC.GetView());
			PlatformData->Model = LoadModel_Internal(ModelReader);

			bHasValidData = true;
		}
	}

	LoadStreamableDataFromDDCEvent = MakeShared<UE::Tasks::FTaskEvent>(TEXT("LoadFromDDCCompletionEvent"));

	// Request loading all streamable data when cooking
	if (bHasValidData && CurrentOptions.bIsCooking)
	{
		// Create a new pull request to retrieve all compiled data. Streamable bulk data included
		FCacheGetRequest Request;
		Request.Name = GetPathNameSafe(CurrentObject);
		Request.Key = CurrentRequest->GetDerivedDataCacheKey();
		Request.Policy = ECachePolicy::Default;

		FRequestOwner RequestOwner(!CurrentRequest->bAsync ? EPriority::Blocking : EPriority::Highest);
		GetCache().Get(MakeArrayView(&Request, 1), RequestOwner,
			[DDCHeapMemory = DDCHeapMemory, PlatformData = PlatformData, CompletionEvent = LoadStreamableDataFromDDCEvent, CurrentRequest = CurrentRequest](FCacheGetResponse&& Response) mutable
			{
				MUTABLE_CPUPROFILER_SCOPE(GetStreamableDataFromDDC);

				if (Response.Status == EStatus::Ok)
				{
					UCustomizableObject* CustomizableObject = CurrentRequest->GetCustomizableObject();
					const FCompilationOptions& Options = CurrentRequest->Options;

					// Value Id to file mapping to reconstruct the cached data
					TMap<FValueId, UE::Mutable::Private::FFile> ValueIdToFile;

					{
						MUTABLE_CPUPROFILER_SCOPE(BuildValueIdToFile);
						TArray<UE::Mutable::Private::FFile> BulkDataFiles;
						FMemoryReaderView FilesReader(DDCHeapMemory->BulkDataFilesBytesDDC.GetView());
						FilesReader << BulkDataFiles;

						ValueIdToFile.Reserve(BulkDataFiles.Num());

						uint32 FileIndex = 0;
						for (UE::Mutable::Private::FFile& File : BulkDataFiles)
						{
							FValueId ValueId = GetDerivedDataValueIdForResource(File.DataType, FileIndex, File.ResourceType, File.Flags);
							ValueIdToFile.Add(ValueId, MoveTemp(File));

							++FileIndex;
						}
						BulkDataFiles.Empty();
					}

					// Get all values and convert them to FMutableCachedPlatformData's format
					TConstArrayView<FValueWithId> Values = Response.Record.GetValues();

					TArray64<uint8> TempData;
					for (const FValueWithId& Value : Values)
					{
						check(Value.IsValid());

						const UE::Mutable::Private::FFile* File = ValueIdToFile.Find(Value.GetId());
						if (!File) // Skip value. It is not a streamable binary blob.
						{
							continue;
						}

						const uint64 RawSize = Value.GetRawSize();
						TempData.SetNumUninitialized(RawSize, EAllowShrinking::No);

						// Decompress streamable binary blobs
						const bool bDecompressedSuccessfully = Value.GetData().TryDecompressTo(MakeMemoryView(TempData.GetData(), RawSize));
						check(bDecompressedSuccessfully);

						// Filter and cache the data by DataType
						switch (File->DataType)
						{
						case UE::Mutable::Private::EStreamableDataType::Model:
						{
							for (const UE::Mutable::Private::FBlock& Block : File->Blocks)
							{
								PlatformData->ModelStreamableData.Set(Block.Id, TempData.GetData() + Block.Offset, Block.Size);
							}
							break;
						}

						default:
							unimplemented();
							break;
						}
					}

					// Generate list of files and update streamable blocks ids and offsets
					if (Options.bIsCooking)
					{
						const uint32 NumBulkDataFilesPerBucket = MAX_uint8;
						UE::Mutable::Private::GenerateBulkDataFilesListWithFileLimit(GenerateDataDistributionIdentifier(*CustomizableObject), PlatformData->Model, *PlatformData->ModelStreamableBulkData.Get(),
							NumBulkDataFilesPerBucket, Options.bIsCooking, false, *Options.TargetPlatform, PlatformData->BulkDataFiles);
					}
					else
					{
						UE::Mutable::Private::GenerateBulkDataFilesListWithSizeLimit(PlatformData->Model, *PlatformData->ModelStreamableBulkData.Get(),
							Options.TargetPlatform, 0 /* unused */, PlatformData->BulkDataFiles);
					}
				}
				else
				{
					PlatformData->Model.Reset();
					PlatformData->ModelResources.Reset();
					PlatformData->ModelStreamableBulkData.Reset();
				}

				CompletionEvent->Trigger();
			});

		if (!CurrentRequest->bAsync)
		{
			// Force sync
			RequestOwner.Wait();
		}
		else
		{
			RequestOwner.KeepAlive();
		}
	}
	else
	{
		LoadStreamableDataFromDDCEvent->Trigger();
	}
}

void FCustomizableObjectCompiler::FinishLoadingStreamableDataFromDDC()
{
	DDCHeapMemory.Reset();
	LoadStreamableDataFromDDCEvent.Reset();

	if (PlatformData->Model && PlatformData->ModelResources && PlatformData->ModelStreamableBulkData)
	{
		if (CurrentOptions.bIsCooking)
		{
			UE::Mutable::Private::FMutableCachedPlatformData& CachedPlatformData = CurrentObject->GetPrivate()->CachedPlatformsData.Add(CurrentOptions.TargetPlatform->PlatformName(), {});
			CachedPlatformData = MoveTemp(*PlatformData.Get());
		}

		UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Finishing Compilation task for CO [%ls].", FPlatformTime::Seconds(), *CurrentObject->GetName());
		TRACE_END_REGION(UE_MUTABLE_COMPILE_REGION);

		UE_LOGF(LogMutable, Display, "Compiled data loaded from DDC");
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Success);
	}
	else
	{
		PreloadReferencerAssets();
	}
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> FCustomizableObjectCompiler::Export(UCustomizableObject* Object, const FCompilationOptions& InCompilerOptions, 
	TArray<TSoftObjectPtr<const UTexture>>& OutRuntimeReferencedTextures,
	TArray<FMutableSourceTextureData>& OutCompilerReferencedTextures, 
	TArray<TSoftObjectPtr<const UStreamableRenderAsset>>& OutRuntimeReferencedMeshes,
	TArray<FMutableSourceMeshData>& OutCompilerReferencedMeshes)
{
	UE_LOGF(LogMutable, Log, "Started Customizable Object Export %ls.", *Object->GetName());

	FCompilationOptions CompilerOptions = InCompilerOptions;
	CompilerOptions.bRealTimeMorphTargetsEnabled = Object->GetPrivate()->IsRealTimeMorphTargetsEnabled();
	CompilerOptions.b16BitBoneWeightsEnabled = Object->GetPrivate()->Is16BitBoneWeightsEnabled();
	CompilerOptions.bSkinWeightProfilesEnabled = Object->GetPrivate()->IsAltSkinWeightProfilesEnabled();
	CompilerOptions.bPhysicsAssetMergeEnabled = Object->GetPrivate()->IsPhysicsAssetMergeEnabled();
	CompilerOptions.bAnimBpPhysicsManipulationEnabled = Object->GetPrivate()->IsEnabledAnimBpPhysicsAssetsManipulation();
	CompilerOptions.bUseLegacyLayouts = Object->GetPrivate()->ShouldUseLegacyLayouts();
	
	CompilationContext = MakeShared<FMutableCompilationContext>(Object, SharedThis(this), CompilerOptions);
	FMutableGraphGenerationContext GenerationContext(CompilationContext);
	
	// Generate the mutable node expression
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> MutableRoot = GenerateMutableRoot(Object, GenerationContext);
	if (!MutableRoot)
	{
		CompilerLog(LOCTEXT("FailedToExport", "Failed to generate the mutable node graph. Object not built."), nullptr);
		return nullptr;
	}

	// Pass out the referenced textures
	OutRuntimeReferencedTextures.Empty();
	for (const TPair<TSoftObjectPtr<UTexture2D>, FMutableGraphGenerationContext::FGeneratedReferencedTexture>& Pair : GenerationContext.RuntimeReferencedTextureMap)
	{
		check(Pair.Value.ID == OutRuntimeReferencedTextures.Num());
		OutRuntimeReferencedTextures.Add(Pair.Key);
	}

	OutCompilerReferencedTextures.Empty();
	for (const TPair<TSoftObjectPtr<UTexture2D>, FMutableGraphGenerationContext::FGeneratedReferencedTexture>& Pair : GenerationContext.CompileTimeTextureMap)
	{
		check(Pair.Value.ID == OutCompilerReferencedTextures.Num());

		FMutableSourceTextureData Tex(*UE::Mutable::Private::LoadObject(Pair.Key));
		OutCompilerReferencedTextures.Add(Tex);
	}

	OutCompilerReferencedMeshes.Empty();
	for (const TPair<FMutableSourceMeshData, FMutableGraphGenerationContext::FGeneratedReferencedMesh>& Pair : GenerationContext.CompileTimeMeshMap)
	{
		check(Pair.Value.ID == OutCompilerReferencedMeshes.Num());

		FMutableSourceMeshData Data(Pair.Key);
		OutCompilerReferencedMeshes.Add(Data);
	}

	return MutableRoot;
}


void FCustomizableObjectCompiler::FinishCompilationTask()
{
	check(CompileTask.IsValid());

	UpdateCompilerLogData();
	TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> Model = CompileTask->Model;
	PlatformData->Model = Model;

	// Generate a map that using the resource id tells the offset and size of the resource inside the bulk data
	// At this point it is assumed that all data goes into a single file.
	if (Model)
	{
		const int32 NumStreamingFiles = Model->GetProgram().GetRomCount();

		TMap<uint32, FMutableStreamableBlock>& ModelStreamables = PlatformData->ModelStreamableBulkData->ModelStreamables;
		ModelStreamables.Empty(NumStreamingFiles);

		uint64 Offset = 0;
		for (int32 FileIndex = 0; FileIndex < NumStreamingFiles; ++FileIndex)
		{
			const uint32 ResourceSize = Model->GetProgram().GetRomSize(FileIndex);
			EMutableFileFlags Flags = Model->GetProgram().IsRomHighRes(FileIndex) ? EMutableFileFlags::HighRes : EMutableFileFlags::None;
			ModelStreamables.Add(FileIndex, FMutableStreamableBlock{ .FileId = 0, .Flags = uint16(Flags), .Offset = Offset });
			Offset += ResourceSize;
		}
	}
	else
	{
		CompilerLog(LOCTEXT("NoModel", "Unable to generate Model"), nullptr, EMessageSeverity::Error);
	}

	// Second part of model resources can be filled now.
	{
		TObjectPtr<UModelResources> ModelResources = PlatformData->ModelResources.Get();

		ModelResources->PassthroughObjects = MoveTemp(CompilationContext->PassthroughObjectFactory.GetAll());
		ModelResources->DuplicateObjects = MoveTemp(CompilationContext->PassthroughObjectFactory.GetDuplicate());

		ModelResources->ExternalOperations = MoveTemp(CompileTask->ExternalOperations);

		for (TPair<UE::Mutable::Private::FOperation::ADDRESS, FInstancedStruct> ExternalOperation : ModelResources->ExternalOperations)
		{
			const uint32 Version = ExternalOperation.Value.GetPtr<UE::Mutable::FExternalOperation>()->GetVersion();
			ModelResources->ExternalOperationVersions.Add(ExternalOperation.Value.GetScriptStruct(), Version);
		}
		
		ModelResources->AnimBpOverridePhysiscAssetsInfo = MoveTemp(CompilationContext->AnimBpOverridePhysicsAssetsInfo);
		
		TSharedPtr<FModelStreamableBulkData> ModelStreamableBulkData = PlatformData->ModelStreamableBulkData;
	}

	// Order matters
	CompileThread.Reset();
	CompileTask.Reset();

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Finishing Compilation task for CO [%ls].", FPlatformTime::Seconds(), *CurrentObject->GetName());
	TRACE_END_REGION(UE_MUTABLE_COMPILE_REGION);

	// Create SaveDD task
	TRACE_BEGIN_REGION(UE_MUTABLE_SAVEDD_REGION);

	SaveDDTask = MakeShareable(new FCustomizableObjectSaveDDRunnable(CurrentRequest, PlatformData));
}


void FCustomizableObjectCompiler::FinishSavingDerivedDataTask()
{
	MUTABLE_CPUPROFILER_SCOPE(FinishSavingDerivedDataTask)

	check(SaveDDTask.IsValid());

	if (CurrentOptions.bIsCooking)
	{
		MUTABLE_CPUPROFILER_SCOPE(CachePlatformData);
		const ITargetPlatform* TargetPlatform = CurrentOptions.TargetPlatform;

		FString PlatformName = TargetPlatform ? TargetPlatform->PlatformName() : FPlatformProperties::PlatformName();

		check(!CurrentObject->GetPrivate()->CachedPlatformsData.Find(PlatformName));

		UE::Mutable::Private::FMutableCachedPlatformData& Data = CurrentObject->GetPrivate()->CachedPlatformsData.Add(PlatformName);
		Data = MoveTemp(*PlatformData.Get());
	}

	// Order matters
	SaveDDThread.Reset();
	SaveDDTask.Reset();

	UE_LOGF(LogMutable, Verbose, "PROFILE: [ %16.8f ] Finished Saving Derived Data task for CO [%ls].", FPlatformTime::Seconds(), *CurrentObject->GetName());

	TRACE_END_REGION(UE_MUTABLE_SAVEDD_REGION);
}


void FCustomizableObjectCompiler::ForceFinishCompilation()
{
	if (AsynchronousStreamableHandlePtr)
	{
		AsynchronousStreamableHandlePtr->CancelHandle();
		AsynchronousStreamableHandlePtr = nullptr;
	}

	else if (CompileTask.IsValid())
	{
		// Compilation needs game thread tasks every now and then.
		// Wait for compilation to finish while giving execution time for these tasks.
		while (!CompileTask->IsCompleted())
		{
			ProcessCompileTasks();
		}

		// Order matters
		CompileThread.Reset();
		CompileTask.Reset();

		UE_LOGF(LogMutable, Verbose, "Force Finish Compilation task for Object.");
		TRACE_END_REGION(UE_MUTABLE_COMPILE_REGION);
	}

	else if (SaveDDTask.IsValid())
	{
		SaveDDThread->WaitForCompletion();

		// Order matters
		SaveDDThread.Reset();
		SaveDDTask.Reset();

		UE_LOGF(LogMutable, Verbose, "Forced Finish Saving Derived Data task.");
		TRACE_END_REGION(UE_MUTABLE_SAVEDD_REGION);
	}

	if (CurrentRequest)
	{
		CompleteRequest(ECompilationStatePrivate::Completed, ECompilationResultPrivate::Errors);
	}
}

void FCustomizableObjectCompiler::ClearCompileRequests()
{
	CompileRequests.Empty();
}


void FCustomizableObjectCompiler::AddCompileNotification(const FText& CompilationStep) const
{
	const FText Text = CurrentObject ? FText::FromString(FString::Printf(TEXT("Compiling %s"), *CurrentObject->GetName())) : LOCTEXT("CustomizableObjectCompileInProgressNotification", "Compiling Customizable Object");
	
	FCustomizableObjectEditorLogger::CreateLog(Text)
	.SubText(CompilationStep)
	.Category(ELoggerCategory::Compilation)
	.Notification(!CurrentRequest->bSilentCompilation)
	.CustomNotification()
	.FixNotification()
	.CustomizableObject(CurrentObject)
	.Log();
}


void FCustomizableObjectCompiler::RemoveCompileNotification()
{
	FCustomizableObjectEditorLogger::DismissNotification(ELoggerCategory::Compilation);
}


void FCustomizableObjectCompiler::NotifyCompilationErrors() const
{
	const uint32 NumWarnings = CompilationLogsContainer.GetWarningCount(false);
	const uint32 NumErrors = CompilationLogsContainer.GetErrorCount();
	const uint32 NumIgnoreds = CompilationLogsContainer.GetIgnoredCount();
	const bool NoWarningsOrErrors = !(NumWarnings || NumErrors);

	const EMessageSeverity::Type Severity = [&]
	{
		if (NumErrors)
		{
			return EMessageSeverity::Error;
		}
		else if (NumWarnings)
		{
			return EMessageSeverity::Warning;
		}
		else
		{
			return EMessageSeverity::Info;
		}
	}();

	const FText Prefix = FText::FromString(CurrentObject ? CurrentObject->GetName() : "Customizable Object");

	const FText Message = NoWarningsOrErrors ?
		FText::Format(LOCTEXT("CompilationFinishedSuccessfully", "{0} finished compiling."), Prefix) :
		NumIgnoreds > 0 ?
		FText::Format(LOCTEXT("CompilationFinished_WithIgnoreds", "{0} finished compiling with {1} {1}|plural(one=warning,other=warnings), {2} {2}|plural(one=error,other=errors) and {3} more similar warnings."), Prefix, NumWarnings, NumErrors, NumIgnoreds)
		:
		FText::Format(LOCTEXT("CompilationFinished_WithoutIgnoreds", "{0} finished compiling with {1} {1}|plural(one=warning,other=warnings) and {2} {2}|plural(one=error,other=errors)."), Prefix, NumWarnings, NumErrors);
	
	FCustomizableObjectEditorLogger::CreateLog(Message)
	.Category(ELoggerCategory::Compilation)
	.Severity(Severity)
	.Notification(!CurrentRequest->bSilentCompilation || !NoWarningsOrErrors)
	.CustomNotification()
	.CustomizableObject(CurrentObject)
	.Log();
}


void FCustomizableObjectCompiler::CompilerLog(const FText& Message, const TArray<const UObject*>& Context, const EMessageSeverity::Type MessageSeverity, const bool bAddBaseObjectInfo, const ELoggerSpamBin SpamBin)
{
	if (CompilationLogsContainer.AddMessage(Message, Context, MessageSeverity, SpamBin)) // Cache the message for later reference
	{
		FCustomizableObjectEditorLogger::CreateLog(Message)
		.Severity(MessageSeverity)
		.Context(Context)
		.BaseObject(bAddBaseObjectInfo)
		.SpamBin(SpamBin)
		.CustomizableObject(CurrentRequest ? CurrentRequest->GetCustomizableObject() : nullptr)
		.Log(); 
	}
}


void FCustomizableObjectCompiler::CompilerLog(const FText& Message, const UObject* Context, const EMessageSeverity::Type MessageSeverity, const bool bAddBaseObjectInfo, const ELoggerSpamBin SpamBin)
{
	TArray<const UObject*> ContextArray;
	if (Context)
	{
		ContextArray.Add(Context);
	}
	CompilerLog(Message, ContextArray, MessageSeverity, bAddBaseObjectInfo, SpamBin);
}


void FCustomizableObjectCompiler::UpdateCompilerLogData()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(FName("Mutable"), LOCTEXT("MutableLog", "Mutable"));

	const FText ObjectName = CurrentObject ? FText::FromString(CurrentObject->GetName()) : LOCTEXT("Unknown Object", "Unknown Object");

	const TArray<FCustomizableObjectCompileRunnable::FError>& ArrayCompileErrors = CompileTask->GetArrayErrors();
	for (const FCustomizableObjectCompileRunnable::FError& CompileError : ArrayCompileErrors)
	{
		TArray<const UObject*> ObjectArray;
		ObjectArray.Reserve(CompileError.Contexts.Num());
		for (const void* Context : CompileError.Contexts)
		{
			ObjectArray.Add(static_cast<const UObject*>(Context));
		}

		const UObject* CompileErrorContext = ObjectArray.IsValidIndex(0) ? ObjectArray[0] : nullptr; 
		
		if (CompileErrorContext && CompileError.AttachedData)
		{
			if (const UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(CompileErrorContext))
			{
				UCustomizableObjectNode::FAttachedErrorDataView ErrorDataView;
				ErrorDataView.UnassignedUVs = { CompileError.AttachedData->UnassignedUVs.GetData(),
												CompileError.AttachedData->UnassignedUVs.Num() };

				const_cast<UCustomizableObjectNode*>(Node)->AddAttachedErrorData(ErrorDataView);
			}			
		}

		FText FullMsg = FText::Format(LOCTEXT("MutableMessage", "{0} : {1}"), ObjectName, CompileError.Message);
		CompilerLog(FullMsg, ObjectArray, CompileError.Severity, true, CompileError.SpamBin);
	}
}

#undef LOCTEXT_NAMESPACE
