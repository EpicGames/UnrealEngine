// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCOE/CompilationMessageCache.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "UObject/GCObject.h"
#include "TickableEditorObject.h"
#include "MuR/Ptr.h"

#include "Framework/Notifications/NotificationManager.h"

struct FCompilationRequest;
class FCustomizableObjectCompileRunnable;
class FCustomizableObjectSaveDDRunnable;
class FReferenceCollector;
class FRunnableThread;
class FText;
class UCustomizableObject;
class UCustomizableObjectNode;
class UModelResources;
struct FMutableGraphGenerationContext;

namespace UE::Mutable::Private
{
	class NodeObject;
	class Node;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> GenerateMutableRoot(const UCustomizableObject* Object, FMutableGraphGenerationContext& GenerationContext);


class FCustomizableObjectCompiler : public TSharedFromThis<FCustomizableObjectCompiler>, public FGCObject
{
public:
	/** Check for pending compilation process. Returns true if an object has been updated. */
	void Tick(bool bBlocking);
	
	int32 GetRemainingWork() const;
	
	int32 GetNumCompileRequests() const;

	/** Generate the Mutable Graph from the Unreal Graph. */
	UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> Export(UCustomizableObject* Object, const FCompilationOptions& Options, 
		TArray<TSoftObjectPtr<const UTexture>>& OutRuntimeReferencedTextures, 
		TArray<FMutableSourceTextureData>& OutCompilerReferencedTextures,
		TArray<TSoftObjectPtr<const UStreamableRenderAsset>>& OutRuntimeReferencedMeshes,
		TArray<FMutableSourceMeshData>& OutCompilerReferencedMeshes);

	void CompilerLog(const FText& Message, const TArray<const UObject*>& UObject, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const bool bAddBaseObjectInfo = true, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll);
	void CompilerLog(const FText& Message, const UObject* Context = nullptr, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const bool bAddBaseObjectInfo = true, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll);
	void NotifyCompilationErrors() const;

	void FinishLoadingModelDataFromDDC();
	void FinishLoadingStreamableDataFromDDC();

	void FinishCompilationTask();
	void FinishSavingDerivedDataTask();

	void ForceFinishCompilation();
	void ClearCompileRequests();

	void AddCompileNotification(const FText& CompilationStep) const;
	static void RemoveCompileNotification();

	/** Enqueue a new compile request.
	 * @param bForceRequests enqueue even if the request is already in the queue. See FCompilationRequest::operator==(...). */
	void EnqueueCompileRequest(const TSharedRef<FCompilationRequest>& CompileRequest, bool bForceRequests);

	/** Queued or in progress. */
	bool IsRequestQueued(const TSharedRef<FCompilationRequest>& InCompileRequest) const;

	/** Queued or in progress. */
	bool IsRequestQueued(const UCustomizableObject& Object) const;
	
	/** FSerializableObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCustomizableObjectCompiler");
	}

	void AddGameThreadCompileTask(TFunction<void()>&& Task);

	/** */
	TSharedPtr<FMutableCompilationContext> CompilationContext;

private:

	// Object containing all error and warning logs raised during compilation.
	FCompilationMessageCache CompilationLogsContainer;

	void SetCompilationState(ECompilationStatePrivate State, ECompilationResultPrivate Result) const;
	
	/** Load required assets and compile.
	 *
	 * Loads assets which reference Object's package asynchronously before calling ProcessChildObjectsRecursively. */
	void Compile(const TSharedRef<FCompilationRequest>& InCompileRequest);

	void CompileInternal(bool bAsync = false);

	void CompleteRequest(ECompilationStatePrivate State, ECompilationResultPrivate Result);
	bool TryPopCompileRequest();

	/** Attempts to load the compiled data from DDC. */
	bool TryLoadCompiledDataFromDDC(UCustomizableObject& CustomizableObject);

	void PreloadReferencerAssets();
	void PreloadingReferencerAssetsCallback(bool bAsync);
	
	// Will output to Mutable Log the warning and error messages generated during the CO compilation
	// and update the values of NumWarnings and NumErrors
	void UpdateCompilerLogData();

	/** Launches the save derived data task in another thread after compiling a CO in the
	* editor
	* @param bShowNotification [in] whether to show the saving DD notification or not
	* @return nothing */
	void SaveCODerivedData();
	
	ECompilationResultPrivate GetCompilationResult() const;

	void ProcessCompileTasks();

	struct FDDCHeapMemory
	{
		FSharedBuffer ModelBytesDDC;
		FSharedBuffer ModelResourcesBytesDDC;
		FSharedBuffer ModelStreamablesBytesDDC;
		FSharedBuffer BulkDataFilesBytesDDC;
	};
	TSharedPtr<FDDCHeapMemory> DDCHeapMemory;

	/** Load compiled data from DDC completion events*/
	TSharedPtr<UE::Tasks::FTaskEvent> LoadModelDataFromDDCEvent;
	TSharedPtr<UE::Tasks::FTaskEvent> LoadStreamableDataFromDDCEvent;

	/** Pointer to the Asynchronous Preloading process call back */
	TSharedPtr<FStreamableHandle> AsynchronousStreamableHandlePtr;

	/** Compile task and thread. */
	TSharedPtr<FCustomizableObjectCompileRunnable> CompileTask;
	TSharedPtr<FRunnableThread> CompileThread;

	/** SaveDD task and thread. */
	TSharedPtr<FCustomizableObjectSaveDDRunnable> SaveDDTask;
	TSharedPtr<FRunnableThread> SaveDDThread;
	
	/** Array used to protect from garbage collection those COs loaded asynchronously */
	TArray<TObjectPtr<UObject>> ArrayGCProtect;

	/** Cached Platform Data. */
	TSharedPtr<UE::Mutable::Private::FMutableCachedPlatformData> PlatformData;

	// Protected from GC with FCustomizableObjectCompiler::AddReferencedObjects
	TObjectPtr<UCustomizableObject> CurrentObject = nullptr;

	FCompilationOptions CurrentOptions;

	/** Current Compilation request. */
	TSharedPtr<FCompilationRequest> CurrentRequest;

	/** Pending requests. */
	TArray<TSharedRef<FCompilationRequest>> CompileRequests;

	uint32 NumCompilationRequests = 0;

	/** Compilation progress bar handle */
	FProgressNotificationHandle CompileNotificationHandle;

	/** Compilation start time in seconds. */
	double CompilationStartTime = 0;

	TQueue< TFunction<void()>, EQueueMode::Mpsc > PendingGameThreadCompileTasks;
};

