// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableObjectConnectionDrawingPolicy.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"


void FCustomizableObjectConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FConnectionParams& Params)
{
	FConnectionDrawingPolicy::DetermineWiringStyle(OutputPin, InputPin, Params);

	Params.WireColor = UEdGraphSchema_CustomizableObject::GetPinTypeColor(OutputPin->PinType.PinCategory); // Color.

	if (HoveredPins.Num() > 0)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin,Params.WireThickness, Params.WireColor); // Emphasis highlight.
	}
}
