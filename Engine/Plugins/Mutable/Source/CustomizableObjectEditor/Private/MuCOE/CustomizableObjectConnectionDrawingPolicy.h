// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"


class FCustomizableObjectConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FCustomizableObjectConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
		: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements) {}
	
	// FConnectionDrawingPolicy interface
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FConnectionParams& Params) override;
};
