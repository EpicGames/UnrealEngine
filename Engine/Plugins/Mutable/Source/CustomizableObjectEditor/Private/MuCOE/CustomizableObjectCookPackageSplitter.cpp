// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectCookPackageSplitter.h"

#include "ClothConfigBase.h"
#include "ClothingAsset.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/LoadUtils.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"

REGISTER_COOKPACKAGE_SPLITTER(FCustomizableObjectCookPackageSplitter, UCustomizableObject);


UModelResources* FindModelResources(UCustomizableObject& Object)
{
	// All platforms should have the same resources
	const TMap<FString, UE::Mutable::Private::FMutableCachedPlatformData>& CachePlatforms = Object.GetPrivate()->CachedPlatformsData;
	for (const TPair<FString, UE::Mutable::Private::FMutableCachedPlatformData>& PlatformData : CachePlatforms)
	{
		if (PlatformData.Value.ModelResources)
		{
			return PlatformData.Value.ModelResources.Get();
		}
	}

	return nullptr;
}
	
	
const UModelResources* FindModelResources(const UCustomizableObject& Object)
{
	return FindModelResources(*const_cast<UCustomizableObject*>(&Object));
}


bool FCustomizableObjectCookPackageSplitter::ShouldSplit(UObject* SplitData)
{
	UCustomizableObject* Object = CastChecked<UCustomizableObject>(SplitData);
	
	if (Object->IsChildObject())
	{
		return false;
	}
	
	const UModelResources* ModelResources = FindModelResources(*Object);
	if (!ModelResources)
	{
		return false;
	}
	
	return !ModelResources->DuplicateObjects.IsEmpty();
}


FString FCustomizableObjectCookPackageSplitter::GetSplitterDebugName()
{
	return TEXT("FCustomizableObjectCookPackageSplitter");
}


bool FCustomizableObjectCookPackageSplitter::RequiresCachedCookedPlatformDataBeforeSplit()
{
	return true;
}


FString GenerateRelativePath(const UCustomizableObject& Object, UE::Mutable::Private::PASSTHROUGH_ID Id)
{
	return FString::Printf(TEXT("%s-%u"), *Object.GetName(), static_cast<uint32>(Id));
}


void FCustomizableObjectCookPackageSplitter::Initialize(UPackage* /*OwnerPackage*/, UObject* OwnerObject)
{
	const UCustomizableObject* Object = CastChecked<UCustomizableObject>(OwnerObject);

	// FindModelResources was called during ShouldSplit, and found a ModelResources. No GC is possible in between
	// ShouldSplit returning true and this call to Initialize, so the OwnerObject could not have been GC'd and it
	// should still have the ModelResources on it. We store a StrongPtr to OwnerObject to prevent it from being GC'd
	// until our splitter is destroyed, so we can rely on the continuing existence of ModelResources.
	const UModelResources* ModelResources = FindModelResources(*Object);
	check(ModelResources);
	StrongObject = TStrongObjectPtr(Object);
}

ICookPackageSplitter::FGenerationManifest FCustomizableObjectCookPackageSplitter::ReportGenerationManifest(const UPackage* OwnerPackage, const UObject* OwnerObject)
{
	// All platforms should have the same resources
	const UCustomizableObject* Object = CastChecked<UCustomizableObject>(OwnerObject);

	const UModelResources* ModelResources = FindModelResources(*Object);
	check(ModelResources);

	FGenerationManifest Result;

	// Generate a new package for each streamed Resource Data
	for (const UE::Mutable::Private::PASSTHROUGH_ID Id : ModelResources->DuplicateObjects)
	{
		FGeneratedPackage& GeneratedPackage = Result.GeneratedPackages.AddDefaulted_GetRef();
		// Because of the checks above, the container name must be unique within this Customizable
		// Object, so it's safe to use as a package path.
		GeneratedPackage.RelativePath = GenerateRelativePath(*Object, Id);
		GeneratedPackage.SetCreateAsMap(false);

		// To support iterative cooking, GenerationHash should only change when OwnerPackage
		// changes.
		//
		// The simplest and fastest way to do this is to set it to OwnerPackage's PackageSavedHash.
		{
			// Zero the hash, as we won't be writing all bytes of it below
			GeneratedPackage.GenerationHash.Reset();

			FIoHash OwnerSavedHash = OwnerPackage->GetSavedHash();
			static_assert(sizeof(GeneratedPackage.GenerationHash.GetBytes()) >= sizeof(OwnerSavedHash.GetBytes()));  // -V568
			static_assert(sizeof(GeneratedPackage.GenerationHash.GetBytes()) > 8); // It should be a byte array, not a pointer // -V568
			static_assert(sizeof(OwnerSavedHash.GetBytes()) > 8); // It should be a byte array, not a pointer // -V568
			FMemory::Memcpy(GeneratedPackage.GenerationHash.GetBytes(), OwnerSavedHash.GetBytes(), sizeof(OwnerSavedHash.GetBytes())); // -V568
		}
	}

	return Result;
}


bool FCustomizableObjectCookPackageSplitter::PopulateGeneratorPackage(FPopulateContext& PopulateContext)
{
	UCustomizableObject* Object = CastChecked<UCustomizableObject>(PopulateContext.GetOwnerObject());
	UModelResources* ModelResources = FindModelResources(*Object);
	check(ModelResources);

	for (const UE::Mutable::Private::PASSTHROUGH_ID Id : ModelResources->DuplicateObjects)
	{
		TSoftObjectPtr<UObject>& SoftPassthroughObject = ModelResources->PassthroughObjects[Id];
		
		UObject* PassthroughObject = UE::Mutable::Private::LoadObject(SoftPassthroughObject);
		if (PassthroughObject)
		{
			const FGeneratedPackageForPopulate* GeneratedPackage = PopulateContext.GetGeneratedPackages().FindByPredicate(
			[&](const FGeneratedPackageForPopulate& Element)
			{
				return Element.RelativePath == GenerateRelativePath(*Object, Id);
			});
			
			check(GeneratedPackage);
			
			UObject* DuplicatedObject = DuplicateObject(PassthroughObject, GeneratedPackage->Package, *PassthroughObject->GetName());
			DuplicatedObject->SetFlags(RF_Public);
			PopulateContext.ReportObjectToMove(DuplicatedObject);
			
			SoftPassthroughObject = DuplicatedObject;

			if (UClothingAssetCommon* ClothingAsset = Cast<UClothingAssetCommon>(DuplicatedObject))
			{
				for (TMap<FName, TObjectPtr<UClothConfigBase>>::TIterator ConfigIt = ClothingAsset->ClothConfigs.CreateIterator(); ConfigIt; ++ConfigIt)
				{
					TObjectPtr<UClothConfigBase>& ClothConfig = ConfigIt->Value;
					
					UClothConfigBase* DuplicatedClothConfig = DuplicateObject<UClothConfigBase>(ClothConfig, DuplicatedObject);
					
					ClothConfig = DuplicatedClothConfig;
				}
			}
		}
	}
	
	return true;
}


ICookPackageSplitter::EGeneratedRequiresGenerator FCustomizableObjectCookPackageSplitter::DoesGeneratedRequireGenerator()
{
	return EGeneratedRequiresGenerator::Save;
}


bool FCustomizableObjectCookPackageSplitter::UseInternalReferenceToAvoidGarbageCollect()
{
	return true;
}

bool FCustomizableObjectCookPackageSplitter::RequiresEmptyPackageBeforePopulate(const UPackage* Package)
{
	return false;
}
