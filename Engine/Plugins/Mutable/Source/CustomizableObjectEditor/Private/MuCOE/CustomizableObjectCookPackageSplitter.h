// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookPackageSplitter.h"
#include "MuCO/CustomizableObject.h"

/** Handles splitting the streamable Data constants into their own packages */
class FCustomizableObjectCookPackageSplitter : public ICookPackageSplitter
{
public:
	// REGISTER_COOKPACKAGE_SPLITTER interface
	static bool ShouldSplit(UObject* SplitData);
	static FString GetSplitterDebugName();
	static bool RequiresCachedCookedPlatformDataBeforeSplit();

	// ICookPackageSplitter interface
	virtual void Initialize(UPackage* OwnerPackage, UObject* OwnerObject) override;
	virtual FGenerationManifest ReportGenerationManifest(const UPackage* OwnerPackage, const UObject* OwnerObject) override;
	virtual bool PopulateGeneratorPackage(FPopulateContext& PopulateContext) override;
	virtual EGeneratedRequiresGenerator DoesGeneratedRequireGenerator() override;
	virtual bool UseInternalReferenceToAvoidGarbageCollect() override;
	virtual bool RequiresEmptyPackageBeforePopulate(const UPackage* Package) override;
	
	TStrongObjectPtr<const UCustomizableObject> StrongObject;
};
