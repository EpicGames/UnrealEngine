// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectCustomSettings.h"

#include "CustomizableObjectEditor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SpotLightComponent.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Misc/Paths.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "MuCOE/CustomizableObjectEditorViewportLights.h"
#include "MuCOE/SCustomizableObjectEditorViewport.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "ObjectTools.h"
#include "PropertyCustomizationHelpers.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/ICustomizableObjectInstanceEditor.h"
#include "Subsystems/ImportSubsystem.h"
#include "Toolkits/ToolkitManager.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class SWidget;
class UClass;
class UFactory;
struct FGeometry;

const FString SCustomizableObjectCustomSettings::LightPackagePath = "/Game/EditorAssets/Mutable/";

void SCustomizableObjectCustomSettings::Construct(const FArguments& InArgs)
{	
	WeakEditor = InArgs._PreviewSettings->GetEditor();

	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	
	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewpot = Editor->GetViewport();
	WeakViewport = Editor->GetViewport();

	TSharedPtr<FString> SelectedLight;
	for (ULightComponent* Light : Viewpot->GetViewportClient()->GetLightComponents())
	{
		TSharedPtr<FString> LightName = MakeShareable(new FString(Light->GetName()));
		LightNames.Add(LightName);

		if (Light == Editor->GetCustomSettings()->GetSelectedLight())
		{
			SelectedLight = LightName;
		}
	}		

	TSharedPtr<STextBlock> LightAssetText;

	// Viewport Lighting 
	AddSlot()
		.AutoHeight()
		.Padding(FMargin(0.f, 4.f))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
				[
					SNew(STextBlock)
					.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 10, "Bold"))
					.Text(FText::FromString("Lighting"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 2.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(.2)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString("Name: "))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.5)
				[
					SAssignNew(LightsAssetNameInputText, SEditableTextBox)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(.15)
				[
					SNew(SButton)
					.Text(FText::FromString("Save"))
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.OnClicked(this, &SCustomizableObjectCustomSettings::OnSaveViewportLightsAsset)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(.15)
				[
					SNew(SButton)
					.Text(FText::FromString("New"))
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.OnClicked(this, &SCustomizableObjectCustomSettings::OnNewViewportLightsAsset)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 2.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString("Viewport Lights"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.7f)
				.Padding(0.f, 0.3f, 0.6f, 0.3f)
				[
					SAssignNew(LightComboButton, SComboButton)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(UE_MUTABLE_GET_COLOR("PropertyEditor.AssetName.ColorAndOpacity"))
					.OnGetMenuContent(this, &SCustomizableObjectCustomSettings::GetLightComboButtonContent)
					.ContentPadding(2.0f)
					.ButtonContent()
					[
						SAssignNew(LightAssetText, STextBlock)
						.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 2.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(1.f, 4.f)
				.FillWidth(0.2f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString("Create:"))
				]
				+ SHorizontalBox::Slot()
				.Padding(1.f, 4.f)
				.FillWidth(0.4f)
				[
					SNew(SButton)
					.Text(FText::FromString("Point Light"))
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.OnClicked(this, &SCustomizableObjectCustomSettings::OnPointLightAdded)
				]
				+ SHorizontalBox::Slot()
				.Padding(1.f, 4.f)
				.FillWidth(0.4f)
				[
					SNew(SButton)
					.Text(FText::FromString("Spot Light"))
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.OnClicked(this, &SCustomizableObjectCustomSettings::OnSpotLightAdded)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 2.f)
			[
				SNew(SBorder)
				.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
				.Padding(6.f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f)
					[
						SAssignNew(LightsListView, SListView<TSharedPtr<FString>>)
						.ListItemsSource(&LightNames)
						.OnGenerateRow(this, &SCustomizableObjectCustomSettings::OnGenerateWidgetForList)
						.OnSelectionChanged(this, &SCustomizableObjectCustomSettings::OnListSelectionChanged)
						.SelectionMode(ESelectionMode::Single)
					]
					+ SVerticalBox::Slot()
					.HAlign(EHorizontalAlignment::HAlign_Right)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(0.5f)
						[
							SNew(SButton)
							.Text(FText::FromString("Unselect"))
							.OnClicked(this, &SCustomizableObjectCustomSettings::OnLightUnselected)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SButton)
							.Text(FText::FromString("Remove Selected"))
							.OnClicked(this, &SCustomizableObjectCustomSettings::OnLightRemoved)
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 2.f))
			[
				SNew(SExpandableArea)
				.AreaTitle(FText::FromString("Light Properties"))
				.BodyContent()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(.65f)
						.Padding(6.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Brightness"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(.35f)
						.Padding(0.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Value(this, &SCustomizableObjectCustomSettings::GetIntensityValue)
							.MinValue(0.f)
							.OnValueCommitted(this, &SCustomizableObjectCustomSettings::OnIntensityValueCommited)
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(.65f)
						.Padding(6.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Color"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(.35)
						.Padding(0.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SColorBlock)
							//.IgnoreAlpha(true)
							.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
							.Color(this, &SCustomizableObjectCustomSettings::GetLightColorValue)
							.OnMouseButtonDown(this, &SCustomizableObjectCustomSettings::OnLightColorBlockMouseButtonDown)
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(.65f)
						.Padding(6.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Attenuation Radius"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(.35f)
						.Padding(0.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Value(this, &SCustomizableObjectCustomSettings::GetAttenuationRadius)
							.MinValue(0.f)
							.OnValueCommitted(this, &SCustomizableObjectCustomSettings::OnAttenuationRadiusValueCommited)
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 2.f)
			[
				SAssignNew(PointLightProperties, SExpandableArea)
				.AreaTitle(FText::FromString("Point Light Properties"))
				.Visibility(EVisibility::HitTestInvisible)
				.InitiallyCollapsed(true)
				.BodyContent()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(.65f)
						.Padding(6.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Source Radius"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(.35f)
						.Padding(0.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Value(this, &SCustomizableObjectCustomSettings::GetLightSourceRadius)
							.MinValue(0.f)
							.OnValueCommitted(this, &SCustomizableObjectCustomSettings::OnLightSourceRadiusValueCommited)
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 3.0f, 6.0f, 3.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(.65f)
						.Padding(6.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString("Source Length"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(.35f)
						.Padding(0.0f, 2.0f, 6.0f, 2.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SNumericEntryBox<float>)
							.Value(this, &SCustomizableObjectCustomSettings::GetLightSourceLength)
							.MinValue(0.f)
							.OnValueCommitted(this, &SCustomizableObjectCustomSettings::OnLightSourceLengthValueCommited)
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 2.f)
			[
				SAssignNew(SpotLightProperties, SExpandableArea)
				.AreaTitle(FText::FromString("Spot Light Properties"))
			.Visibility(EVisibility::HitTestInvisible)
			.InitiallyCollapsed(true)
			.BodyContent()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 3.0f, 6.0f, 3.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(.65f)
					.Padding(6.0f, 2.0f, 6.0f, 2.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString("Inner Cone Angle"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(.35f)
					.Padding(0.0f, 2.0f, 6.0f, 2.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SNumericEntryBox<float>)
						.Value(this, &SCustomizableObjectCustomSettings::GetLightInnerConeAngle)
						.MinValue(0.f)
						.OnValueCommitted(this, &SCustomizableObjectCustomSettings::OnLightInnerConeAngleValueCommited)
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 3.0f, 6.0f, 3.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(.65f)
					.Padding(6.0f, 2.0f, 6.0f, 2.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString("Outer Cone Angle"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(.35f)
					.Padding(0.0f, 2.0f, 6.0f, 2.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SNumericEntryBox<float>)
						.Value(this, &SCustomizableObjectCustomSettings::GetLightOuterConeAngle)
						.MinValue(0.f)
						.OnValueCommitted(this, &SCustomizableObjectCustomSettings::OnLightOuterConeAngleValueCommited)
					]
				]
			]
		]
	];

	if (const UCustomizableObjectEditorViewportLights* LightPreset = Editor->GetCustomSettings()->GetLightsPreset())
	{
		const FText Name = FText::FromString(LightPreset->GetName());
		LightAssetText->SetText(Name);		
		LightsAssetNameInputText->SetText(Name);		
	}
	else
	{
		LightAssetText->SetText(FText::FromString("No Asset Selected"));		
		LightsAssetNameInputText->SetText(FText::FromString("Default Viewport Lights"));			
	}

	if (SelectedLight)
	{
		LightsListView->SetSelection(SelectedLight);		
	}
}


void SCustomizableObjectCustomSettings::SetViewportLightsByAsset(const FAssetData& InAsset)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	
	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return;	
	}
	
	UCustomizableObjectEditorViewportLights* InLights = Cast<UCustomizableObjectEditorViewportLights>(UE::Mutable::Private::LoadObject(InAsset));	
	if (!InLights)
	{
		return;
	}
	
	Editor->GetCustomSettings()->SetLightsPreset(*InLights);

	OnNewViewportLightsAsset();

	for (FViewportLightData& LightData : InLights->LightsData)
	{
		UPointLightComponent* NewLight = nullptr;

		if (LightData.bIsSpotLight)
		{
			USpotLightComponent* SpotLight = NewObject<USpotLightComponent>();
			SpotLight->SetInnerConeAngle(LightData.InnerConeAngle);
			SpotLight->SetOuterConeAngle(LightData.OuterConeAngle);
			NewLight = SpotLight;
		}
		else
		{
			NewLight = NewObject<UPointLightComponent>();
		}

		NewLight->SetWorldTransform(LightData.Transform);
		NewLight->SetIntensity(LightData.Intensity);
		NewLight->SetLightColor(LightData.Color);
		NewLight->SetAttenuationRadius(LightData.AttenuationRadius);
		NewLight->SetSourceLength(LightData.SourceLength);
		NewLight->SetSourceRadius(LightData.SourceRadius);

		Viewport->GetViewportClient()->AddLightToScene(NewLight);			
		
		LightNames.Add(MakeShareable(new FString(NewLight->GetName())));
	}

	Editor->HideGizmoLight();
}


TSharedRef<SWidget> SCustomizableObjectCustomSettings::GetLightComboButtonContent()
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	
	TArray<const UClass*> filterClasses;
	filterClasses.Add(UCustomizableObjectEditorViewportLights::StaticClass());
	return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
		FAssetData(Editor->GetCustomSettings()->GetLightsPreset()),
		true,
		filterClasses,
		TArray<UFactory*>(),
		FOnShouldFilterAsset(),
		FOnAssetSelected::CreateSP(this, &SCustomizableObjectCustomSettings::SetViewportLightsByAsset),
		FSimpleDelegate::CreateSP(this, &SCustomizableObjectCustomSettings::CloseLightComboButtonContent));
}

void SCustomizableObjectCustomSettings::CloseLightComboButtonContent()
{
	if (!LightComboButton.IsValid())
	{
		return;
	}

	LightComboButton->SetIsOpen(false);
}


FReply SCustomizableObjectCustomSettings::OnSaveViewportLightsAsset() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return FReply::Handled();	
	}
	
	FString ObjName = LightsAssetNameInputText->GetText().ToString();

	if (ObjName.IsEmpty() || ObjName == "None")
	{
		return FReply::Unhandled();
	}

	ObjName = ObjectTools::SanitizeObjectName(ObjName);
	FString PackageName = LightPackagePath + ObjName;


	UCustomizableObjectEditorViewportLights* ViewportLights = nullptr;

	bool bExistingAsset = false;
	if (UPackage* ExistingPackage = FindPackage(NULL, *PackageName))
	{
		UObject* ExistingObject = ExistingPackage ? StaticFindObject(UCustomizableObjectEditorViewportLights::StaticClass(), ExistingPackage, *ObjName) : nullptr;
		ViewportLights = ExistingObject ? Cast<UCustomizableObjectEditorViewportLights>(ExistingObject) : nullptr;

		if (ViewportLights)
		{
			ViewportLights->LightsData.Empty();
			bExistingAsset = true;
		}
	}

	if (!ViewportLights)
	{
		ViewportLights = NewObject<UCustomizableObjectEditorViewportLights>(CreatePackage(*PackageName), *ObjName, RF_Standalone | RF_Public);
	}

	if (ViewportLights)
	{
		for (ULightComponent* Light : Viewport->GetViewportClient()->GetLightComponents())
		{
			FViewportLightData& NewData = ViewportLights->LightsData.AddDefaulted_GetRef();

			if (const UPointLightComponent* PointLight = Cast<UPointLightComponent>(Light))
			{
				NewData.Intensity = PointLight->Intensity;
				NewData.Color = PointLight->LightColor;
				NewData.AttenuationRadius = PointLight->AttenuationRadius;
				NewData.Transform = PointLight->GetComponentTransform();
				NewData.SourceLength = PointLight->SourceLength;
				NewData.SourceRadius = PointLight->SourceRadius;
			}
			
			if (const USpotLightComponent* SpotLight = Cast<USpotLightComponent>(Light))
			{
				NewData.bIsSpotLight = true;
				NewData.InnerConeAngle = SpotLight->InnerConeAngle;
				NewData.OuterConeAngle = SpotLight->OuterConeAngle;
			}
		}

		ViewportLights->MarkPackageDirty();

		if (!bExistingAsset)
		{
			FAssetRegistryModule::AssetCreated(ViewportLights);
		}
	}

	return FReply::Handled();
}

FReply SCustomizableObjectCustomSettings::OnNewViewportLightsAsset() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	
	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return FReply::Handled();	
	}

	Viewport->GetViewportClient()->RemoveAllLightsFromScene();
	
	Editor->HideGizmoLight();
	
	return FReply::Handled();
}

FReply SCustomizableObjectCustomSettings::OnPointLightAdded() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return FReply::Handled();	
	}

	UPointLightComponent* Component = NewObject<UPointLightComponent>(GetTransientPackage(), NAME_None, RF_Transactional);
	Viewport->GetViewportClient()->AddLightToScene(Component);
	Editor->ShowGizmoLight(*Component);
	
	return FReply::Handled();
}


FReply SCustomizableObjectCustomSettings::OnSpotLightAdded() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();
	
	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return FReply::Handled();	
	}
	
	USpotLightComponent* Component = NewObject<USpotLightComponent>();
		
	Viewport->GetViewportClient()->AddLightToScene(Component);
	Editor->ShowGizmoLight(*Component);			

	return FReply::Handled();
}


FReply SCustomizableObjectCustomSettings::OnLightRemoved() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return FReply::Handled();	
	}
	
	const TArray<TSharedPtr<FString>> SelectedItems = LightsListView->GetSelectedItems();
	if (!SelectedItems.Num())
	{
		return FReply::Handled();
	}

	const int32 LightIndex = LightNames.Find(SelectedItems[0]);

	// Delete Light from ViewportClient  
	if (ULightComponent* LightToRemove = Viewport->GetViewportClient()->GetLightComponents()[LightIndex])
	{
		Viewport->GetViewportClient()->RemoveLightFromScene(LightToRemove);
		LightToRemove->DestroyComponent();
	}

	Editor->HideGizmoLight();

	return FReply::Handled();
}

FReply SCustomizableObjectCustomSettings::OnLightUnselected()
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	const TArray<TSharedPtr<FString>> SelectedItems = LightsListView->GetSelectedItems();
	if (!SelectedItems.Num())
	{
		return FReply::Handled();
	}
	
	Editor->HideGizmoLight();

	return FReply::Handled();
}


TSharedRef<ITableRow> SCustomizableObjectCustomSettings::OnGenerateWidgetForList(TSharedPtr<FString> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		.Padding(FMargin(0.2f, 0.1f, 0.2f, 0.1f))
		[
			SNew(STextBlock)
			.Text(FText::FromString(*Item.Get()))
		];
}


void SCustomizableObjectCustomSettings::OnListSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	SpotLightProperties->SetVisibility(EVisibility::HitTestInvisible);
	SpotLightProperties->SetExpanded(false);
	PointLightProperties->SetVisibility(EVisibility::HitTestInvisible);
	PointLightProperties->SetExpanded(false);
	
	TArray<TSharedPtr<FString>> SelectedItems = LightsListView->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return;
	}

	const TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport = WeakViewport.Pin();
	if (!Viewport)
	{
		return;	
	}
	
	const int32 LightIndex = LightNames.Find(SelectedItems[0]);
	
	ULightComponent* SelectedLight = Viewport->GetViewportClient()->GetLightComponents()[LightIndex];
	Editor->ShowGizmoLight(*SelectedLight);			

	if (Cast<USpotLightComponent>(SelectedLight))
	{
		SpotLightProperties->SetVisibility(EVisibility::Visible);
		SpotLightProperties->SetExpanded(true);
	}
	else
	{
		SpotLightProperties->SetExpanded(false);
		SpotLightProperties->SetVisibility(EVisibility::HitTestInvisible);
	}

	PointLightProperties->SetVisibility(EVisibility::Visible);
	PointLightProperties->SetExpanded(true);	
}


TOptional<float> SCustomizableObjectCustomSettings::GetIntensityValue() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	return (SelectedLight) ? SelectedLight->Intensity : 1.0f;
}


void SCustomizableObjectCustomSettings::OnIntensityValueCommited(float Value, ETextCommit::Type CommitType)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (SelectedLight && CommitType != ETextCommit::OnCleared)
	{
		SelectedLight->SetIntensity(Value);
	}
}


FLinearColor SCustomizableObjectCustomSettings::GetLightColorValue() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	const ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	return (SelectedLight) ? SelectedLight->LightColor : FLinearColor::White;
}


FReply SCustomizableObjectCustomSettings::OnLightColorBlockMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	FColorPickerArgs args;
	args.bIsModal = true;
	args.bUseAlpha = false;
	args.bOnlyRefreshOnMouseUp = false;
	args.InitialColor = GetLightColorValue();
	args.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SCustomizableObjectCustomSettings::OnSetLightColorFromColorPicker);

	OpenColorPicker(args);

	return FReply::Handled();
}


void SCustomizableObjectCustomSettings::OnSetLightColorFromColorPicker(FLinearColor InColor)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	if (ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight())
	{
		SelectedLight->SetLightColor(InColor);
	}
}


TOptional<float> SCustomizableObjectCustomSettings::GetAttenuationRadius() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (const UPointLightComponent* PointLight = Cast<const UPointLightComponent>(SelectedLight))
	{
		return PointLight->AttenuationRadius;
	}

	return 0.f;
}

void SCustomizableObjectCustomSettings::OnAttenuationRadiusValueCommited(float Value, ETextCommit::Type CommitType)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(SelectedLight))
	{
		PointLight->SetAttenuationRadius(Value);
	}
}

TOptional<float> SCustomizableObjectCustomSettings::GetLightSourceRadius() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (const UPointLightComponent* PointLight = Cast<const UPointLightComponent>(SelectedLight))
	{
		return PointLight->SourceRadius;
	}

	return 0.f;
}

void SCustomizableObjectCustomSettings::OnLightSourceRadiusValueCommited(float Value, ETextCommit::Type CommitType)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(SelectedLight))
	{
		PointLight->SetSourceRadius(Value);
	}
}

TOptional<float> SCustomizableObjectCustomSettings::GetLightSourceLength() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (const UPointLightComponent* PointLight = Cast<const UPointLightComponent>(SelectedLight))
	{
		return PointLight->SourceLength;
	}

	return 0.f;
}

void SCustomizableObjectCustomSettings::OnLightSourceLengthValueCommited(float Value, ETextCommit::Type CommitType)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(SelectedLight))
	{
		PointLight->SetSourceLength(Value);
	}
}

TOptional<float> SCustomizableObjectCustomSettings::GetLightInnerConeAngle() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (const USpotLightComponent* SpotLight = Cast<const USpotLightComponent>(SelectedLight))
	{
		return SpotLight->InnerConeAngle;
	}

	return 0.f;
}

void SCustomizableObjectCustomSettings::OnLightInnerConeAngleValueCommited(float Value, ETextCommit::Type CommitType)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (USpotLightComponent* SpotLight = Cast<USpotLightComponent>(SelectedLight))
	{
		SpotLight->SetInnerConeAngle(Value);
	}
}

TOptional<float> SCustomizableObjectCustomSettings::GetLightOuterConeAngle() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (const USpotLightComponent* SpotLight = Cast<const USpotLightComponent>(SelectedLight))
	{
		return SpotLight->OuterConeAngle;
	}

	return 0.f;
}

void SCustomizableObjectCustomSettings::OnLightOuterConeAngleValueCommited(float Value, ETextCommit::Type CommitType)
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = GetEditorChecked();

	ULightComponent* SelectedLight = Editor->GetCustomSettings()->GetSelectedLight();
	
	if (USpotLightComponent* SpotLight = Cast<USpotLightComponent>(SelectedLight))
	{
		SpotLight->SetOuterConeAngle(Value);
	}
}


TSharedPtr<ICustomizableObjectInstanceEditor> SCustomizableObjectCustomSettings::GetEditorChecked() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
	check(Editor);
	
	return Editor;
}
