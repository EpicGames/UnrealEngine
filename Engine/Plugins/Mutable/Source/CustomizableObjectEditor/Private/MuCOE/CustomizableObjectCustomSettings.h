// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CustomizableObjectInstanceEditor.h"
#include "Widgets/SBoxPanel.h"

class UPointLightComponent;
class SCustomizableObjectEditorViewportTabBody;
class ICustomizableObjectInstanceEditor;

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename ItemType> class SListView;

class SWidget;
class SEditableTextBox;
class SComboButton;
class UCustomizableObject;
struct FAssetData;
struct FGeometry;
struct FPointerEvent;


/**
 * Custom slate that grants the user the ability of adding, moving and removing lights from the preview scene.
 */
class SCustomizableObjectCustomSettings : public SVerticalBox
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectCustomSettings) {}
	SLATE_ARGUMENT(UCustomSettings*, PreviewSettings)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	void SetViewportLightsByAsset(const FAssetData& InAsset);

private:
	TSharedRef<SWidget> GetLightComboButtonContent();
	void CloseLightComboButtonContent();
	FReply OnSaveViewportLightsAsset() const;
	FReply OnNewViewportLightsAsset() const;

	FReply OnPointLightAdded() const;
	FReply OnSpotLightAdded() const;
	FReply OnLightRemoved() const;
	FReply OnLightUnselected();
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FString> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnListSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// LightProperties
	TOptional<float> GetIntensityValue() const;
	void OnIntensityValueCommited(float Value, ETextCommit::Type CommitType);

	FLinearColor GetLightColorValue() const;
	FReply OnLightColorBlockMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void OnSetLightColorFromColorPicker(FLinearColor InColor);

	TOptional<float> GetAttenuationRadius() const;
	void OnAttenuationRadiusValueCommited(float Value, ETextCommit::Type CommitType);

	TOptional<float> GetLightSourceRadius() const;
	void OnLightSourceRadiusValueCommited(float Value, ETextCommit::Type CommitType);

	TOptional<float> GetLightSourceLength() const;
	void OnLightSourceLengthValueCommited(float Value, ETextCommit::Type CommitType);

	TOptional<float> GetLightInnerConeAngle() const;
	void OnLightInnerConeAngleValueCommited(float Value, ETextCommit::Type CommitType);

	TOptional<float> GetLightOuterConeAngle() const;
	void OnLightOuterConeAngleValueCommited(float Value, ETextCommit::Type CommitType);

	TSharedPtr<ICustomizableObjectInstanceEditor> GetEditorChecked() const;
	
	TArray<TSharedPtr<FString>> LightNames;

	static const FString LightPackagePath;
	TSharedPtr<SEditableTextBox> LightsAssetNameInputText;

	TSharedPtr<SComboButton> LightComboButton;
	TSharedPtr<SListView<TSharedPtr<FString>>> LightsListView;
	TSharedPtr<SExpandableArea> PointLightProperties;
	TSharedPtr<SExpandableArea> SpotLightProperties;

	TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor;
	TWeakPtr<SCustomizableObjectEditorViewportTabBody> WeakViewport;
};
