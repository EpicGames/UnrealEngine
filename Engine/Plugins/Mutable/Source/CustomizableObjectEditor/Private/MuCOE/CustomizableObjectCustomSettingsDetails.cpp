// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectCustomSettingsDetails.h"

#include "Animation/Skeleton.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "PropertyCustomizationHelpers.h"
#include "SCustomizableObjectEditorViewport.h"
#include "SMutableScrubPanel.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "MuCO/CustomizableObject.h"
#include "MuCOE/CustomizableObjectCustomSettings.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "Toolkits/ToolkitManager.h"


TSharedRef<IDetailCustomization> FCustomizableObjectCustomSettingsDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectCustomSettingsDetails);
}


void FCustomizableObjectCustomSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num() > 0 && DetailsView->GetSelectedObjects()[0].IsValid())
	{
		UCustomSettings* CustomSettings = Cast<UCustomSettings>(DetailsView->GetSelectedObjects()[0]);

		TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor = CustomSettings->GetEditor();
		TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();

		if (Editor->ShowLightingSettings())
		{
			IDetailCategoryBuilder& MainCategory = DetailBuilder.EditCategory("Custom Settings");
			MainCategory.AddCustomRow(FText::FromString("Custom Settings"))
			[
				SNew(SCustomizableObjectCustomSettings)
					.PreviewSettings(CustomSettings)
			];
		}
		
		TSharedPtr<FCustomizableObjectEditorViewportClient> ViewportClient = Editor->GetViewport()->GetViewportClient();
		
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_STRING_CHECKED(UCustomSettings, Animation));
		IDetailPropertyRow* DetailPropertyRow = DetailBuilder.EditDefaultProperty(PropertyHandle);
		
		DetailPropertyRow->CustomWidget()
		.NameContent()
		[
			DetailPropertyRow->GetPropertyHandle()->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(250.0f)
		.MinDesiredWidth(250.0f)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UAnimationAsset::StaticClass())
			.PropertyHandle(DetailPropertyRow->GetPropertyHandle())
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateLambda([WeakEditor](const FAssetData& InAssetData)
			{
				TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
				if (!Editor)
				{
					return true;
				}

				for (TPair<FName,TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : Editor->GetViewport()->GetViewportClient()->GetPreviewMeshComponents())
				{
					UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
					if (!PreviewMeshComponent)
					{
						continue;
					}

					USkeletalMesh* SkeletalMeshAsset = PreviewMeshComponent->GetSkeletalMeshAsset();
					if (!SkeletalMeshAsset)
					{
						continue;
					}

					if (USkeleton* Skeleton = SkeletalMeshAsset->GetSkeleton())
					{
						return !Skeleton->IsCompatibleForEditor(InAssetData);
					}
				}

				return true;
			}))
			.ThumbnailPool(DetailBuilder.GetThumbnailPool())
		];
	}
}

