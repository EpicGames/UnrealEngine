// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"

class ITableRow;


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"

TSharedRef<IDetailCustomization> FCustomizableObjectDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectDetails );
}


void FCustomizableObjectDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		CustomizableObject = Cast<UCustomizableObject>(DetailsView->GetSelectedObjects()[0].Get());
	}
	
	DetailBuilder.HideProperty("CustomizableObjectClassTags");
	DetailBuilder.HideProperty("PopulationClassTags");

	if (CustomizableObject && CustomizableObject->IsChildObject())
	{
		DetailBuilder.HideProperty("bEnableRealTimeMorphTargets");
		DetailBuilder.HideProperty("bEnable16BitBoneWeights");
		DetailBuilder.HideProperty("bEnableAltSkinWeightProfiles");
		DetailBuilder.HideProperty("bEnablePhysicsAssetMerge");
		DetailBuilder.HideProperty("bEnableAnimBpPhysicsAssetsManipulation");
		DetailBuilder.HideProperty("bDisableTableMaterialsParentCheck");
		DetailBuilder.HideProperty("bEnableMeshCache");
		DetailBuilder.HideProperty("bEnableUseRefSkeletalMeshAsPlaceholder");
		DetailBuilder.HideProperty("bUseLegacyLayouts");
	}

	TSharedRef<IPropertyHandle> VersionBridgeProperty = DetailBuilder.GetProperty("VersionBridge");

	if (VersionBridgeProperty->IsValidHandle() && CustomizableObject)
	{
		if (CustomizableObject->IsChildObject())
		{
			VersionBridgeProperty->MarkHiddenByCustomization();
		}
		else
		{
			VersionBridgeProperty->MarkResetToDefaultCustomized();
		}
	}

	TSharedRef<IPropertyHandle> VersionStructProperty = DetailBuilder.GetProperty("VersionStruct");

	if (VersionStructProperty->IsValidHandle() && CustomizableObject)
	{
		if (CustomizableObject->IsChildObject())
		{
			VersionStructProperty->MarkResetToDefaultCustomized();
		}
		else
		{
			VersionStructProperty->MarkHiddenByCustomization();
		}
	}
}


#undef LOCTEXT_NAMESPACE
