// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Widgets/Views/STreeView.h"

class IDetailLayoutBuilder;
class ITableRow;


class FCustomizableObjectDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	class UCustomizableObject* CustomizableObject = nullptr;
};
