// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditor.h"

#include "CustomizableObjectCompiler.h"
#include "CustomizableObjectCompileRunnable.h"
#include "CustomizableObjectEditorModule.h"
#include "CustomizableObjectEditorPerformanceAnalyzer.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DetailsViewArgs.h"
#include "EdGraphUtilities.h"
#include "EditorModeManager.h"
#include "FileHelpers.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GraphEditorActions.h"
#include "IDetailsView.h"
#include "IMessageLogListing.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCOE/CustomizableObjectCustomSettings.h"
#include "MuCOE/CustomizableObjectEditorActions.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOE/CustomizableObjectEditorNodeContextCommands.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectPreviewScene.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierTransformInMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CONodeClipMeshWithMesh.h"
#include "MuCOE/SCustomizableObjectEditorAdvancedPreviewSettings.h"
#include "MuCOE/SCustomizableObjectEditorTagExplorer.h"
#include "MuCOE/SCustomizableObjectEditorTextureAnalyzer.h"
#include "MuCOE/SCustomizableObjectEditorViewport.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "SMutableCodeViewer.h"
#include "SMutableGraphViewer.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/UnrealMutableImageProvider.h"
#include "MuCOE/CompileRequest.h"
#include "MessageLogModule.h"
#include "ActorFactories/ActorFactoryBasicShape.h"
#include "MuCOE/CompilationOptions.h"
#include "Nodes/CONodeClipSkeletalMeshWithSkeletalMesh.h"
#include "Nodes/CONodeTransformInMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectEditor)

class FAdvancedPreviewScene;
class FWorkspaceItem;
class IToolkitHost;
class SWidget;
enum class EColorArithmeticOperation : uint8;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

DEFINE_LOG_CATEGORY_STATIC(LogCustomizableObjectEditor, Log, All);

const FName FCustomizableObjectEditor::ViewportTabId( TEXT( "CustomizableObjectEditor_Viewport" ) );
const FName FCustomizableObjectEditor::DetailsTabId( TEXT( "CustomizableObjectEditor_ObjectProperties" ) );
const FName FCustomizableObjectEditor::InstancePropertiesTabId( TEXT( "CustomizableObjectEditor_InstanceProperties" ) );
const FName FCustomizableObjectEditor::GraphTabId( TEXT( "CustomizableObjectEditor_Graph" ) );
const FName FCustomizableObjectEditor::AdvancedPreviewSettingsTabId(TEXT("CustomizableObjectEditor_AdvancedPreviewSettings"));
const FName FCustomizableObjectEditor::TextureAnalyzerTabId(TEXT("CustomizableObjectEditor_TextureAnalyzer"));
const FName FCustomizableObjectEditor::PerformanceAnalyzerTabId(TEXT("CustomizableObjectEditor_MewPerformanceReport"));
const FName FCustomizableObjectEditor::TagExplorerTabId(TEXT("CustomizableObjectEditor_TagExplorer"));
const FName FCustomizableObjectEditor::ObjectDebuggerTabId(TEXT("CustomizableObjectEditor_ObjectDebugger"));
const FName FCustomizableObjectEditor::PopulationClassTagManagerTabId(TEXT("CustomizableObjectEditor_PopulationClassTabManager"));
const FName FCustomizableObjectEditor::StatsTabId(TEXT("CustomizableObjectEditor_StatsTabManager"));


void UUpdateClassWrapper::DelegatedCallback(UCustomizableObjectInstance* Instance)
{
	Delegate.ExecuteIfBound();
}


void FCustomizableObjectEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CustomizableObjectEditor", "Customizable Object Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();
	
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(ViewportTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef);
	
	InTabManager->RegisterTabSpawner(DetailsTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef);
	
	InTabManager->RegisterTabSpawner(InstancePropertiesTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_InstanceProperties))
		.SetDisplayName(LOCTEXT("InstancePropertiesTab", "Instance Properties"))
		.SetGroup(WorkspaceMenuCategoryRef);
	
	InTabManager->RegisterTabSpawner(GraphTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_Graph))
		.SetDisplayName(LOCTEXT("GraphTab", "Object Graph"))
		.SetGroup(WorkspaceMenuCategoryRef);
	
	InTabManager->RegisterTabSpawner(AdvancedPreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_AdvancedPreviewSettings))
		.SetDisplayName(LOCTEXT("AdvancedPreviewSettingsTab", "Advanced Preview Settings"))
		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(TextureAnalyzerTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_TextureAnalyzer))
		.SetDisplayName(LOCTEXT("TextureAnalyzer", "Texture Analyzer"))
		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(PerformanceAnalyzerTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_PerformanceAnalyzer))
		.SetDisplayName(LOCTEXT("PerformanceAnalyzer", "Performance Analyzer"))
		.SetGroup(WorkspaceMenuCategoryRef);
	
	InTabManager->RegisterTabSpawner(TagExplorerTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_TagExplorer))
		.SetDisplayName(LOCTEXT("TagExplorerTab", "Tag Explorer"))
		.SetGroup(WorkspaceMenuCategoryRef);
	
	InTabManager->RegisterTabSpawner(StatsTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectEditor::SpawnTab_Stats))
		.SetDisplayName(LOCTEXT("StatsTab", "Mutable Log"))
		.SetGroup(WorkspaceMenuCategoryRef);
}


void FCustomizableObjectEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner( ViewportTabId );
	InTabManager->UnregisterTabSpawner( DetailsTabId );
	InTabManager->UnregisterTabSpawner( InstancePropertiesTabId );
	InTabManager->UnregisterTabSpawner( GraphTabId );
	InTabManager->UnregisterTabSpawner( AdvancedPreviewSettingsTabId );
	InTabManager->UnregisterTabSpawner( TextureAnalyzerTabId );
	InTabManager->UnregisterTabSpawner( PerformanceAnalyzerTabId );
	InTabManager->UnregisterTabSpawner( StatsTabId );
}	



FCustomizableObjectEditor::~FCustomizableObjectEditor()
{
	if (PreviewInstance)
	{
		// Remove the message log list when closing the editor
		if (PreviewInstance->GetPrivate()->UpdateLogger)
		{
			PreviewInstance->GetPrivate()->UpdateLogger.Reset();
		}
	}
	
	ObjectDetailsView.Reset();

	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);

	CustomizableObject->GetPrivate()->Status.GetOnStateChangedDelegate().RemoveAll(this);

	// Remove the message log list when closing the editor
	if (CustomizableObject->GetPrivate()->CompilationLogger)
	{
		CustomizableObject->GetPrivate()->CompilationLogger.Reset();
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnFilesLoaded().RemoveAll(this);

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);

	GEngine->ForceGarbageCollection(true);
}


void FCustomizableObjectEditor::InitCustomizableObjectEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost)
{
	ProjectorParameter = NewObject<UProjectorParameter>();

	CustomSettings = NewObject<UCustomSettings>();
	CustomSettings->SetEditor(SharedThis(this));
	
	EditorProperties = NewObject<UCustomizableObjectEditorProperties>();

	// Register our commands. This will only register them if not previously registered
	FGraphEditorCommands::Register();
	FCustomizableObjectEditorCommands::Register();
	FCustomizableObjectEditorViewportCommands::Register();
	FCustomizableObjectEditorNodeContextCommands::Register();

	BindCommands();
	BindGraphCommands();

	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bShowScrollBar = false;

	// Detail Panels
	ObjectDetailsView = PropPlugin.CreateDetailView( DetailsViewArgs );
	CustomizableInstanceDetailsView = PropPlugin.CreateDetailView( DetailsViewArgs );

	// Create EditorModeManager before viewport construction so viewport clients
	// receive the editor's ModeTools instead of creating their own.
	CreateEditorModeManager();

	// Viewport Panel
	Viewport = SNew(SCustomizableObjectEditorViewportTabBody).CustomizableObjectEditor(SharedThis(this));
	Viewport->SetCustomizableObject(CustomizableObject);
	ViewportClient = Viewport->GetViewportClient();

	if (ViewportClient)
	{
		if (FEditorModeTools* const ModeTools = ViewportClient->GetModeTools())
		{
			// Opt out of new ITF gizmos and viewport inputs for now
			ModeTools->SetSupportsViewportITF(false);
		}
	}

	// \TODO: Create only when needed?
	TextureAnalyzer = SNew(SCustomizableObjecEditorTextureAnalyzer).CustomizableObjectEditor(this).CustomizableObjectInstanceEditor(nullptr);

	// \TODO: Create only when needed?
	TagExplorer = SNew(SCustomizableObjectEditorTagExplorer).CustomizableObjectEditor(this);
	
	TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene = StaticCastSharedPtr<FAdvancedPreviewScene>(Viewport->GetPreviewScene());

	CustomizableObjectEditorAdvancedPreviewSettings =
		SNew(SCustomizableObjectEditorAdvancedPreviewSettings, AdvancedPreviewScene.ToSharedRef())
		.CustomSettings(CustomSettings)
		.CustomizableObjectEditor(SharedThis(this).ToWeakPtr());
	CustomizableObjectEditorAdvancedPreviewSettings->LoadProfileEnvironment();
	AdvancedPreviewSettingsWidget = CustomizableObjectEditorAdvancedPreviewSettings;

	// Initialize the Log List for the current Customizable Object
	InitLogList();

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_CustomizableObjectEditor_Layout_v1.5" )
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
		->Split
		(
			FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
			->SetSizeCoefficient(0.6f)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.8f)
				->AddTab(GraphTabId, ETabState::OpenedTab)
				->SetForegroundTab(GraphTabId)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->AddTab(StatsTabId, ETabState::OpenedTab)
				->SetForegroundTab(StatsTabId)
			)
		)
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal) 
			->SetSizeCoefficient(0.4f)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical) 
				->SetSizeCoefficient(0.5f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(DetailsTabId, ETabState::OpenedTab)
					->AddTab(TagExplorerTabId, ETabState::OpenedTab)
					->SetForegroundTab(DetailsTabId)
				)
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical) 
				->SetSizeCoefficient(0.5f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(InstancePropertiesTabId, ETabState::OpenedTab)
					->AddTab(AdvancedPreviewSettingsTabId, ETabState::OpenedTab)
					->SetForegroundTab(InstancePropertiesTabId)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(ViewportTabId, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			)
		)	
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, CustomizableObjectEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, CustomizableObject);
	
	ObjectDetailsView->SetObject(CustomizableObject); // Can only be called after initializing the Asset Editor

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// Clears selection highlight.
	OnObjectPropertySelectionChanged(NULL);
	OnInstancePropertySelectionChanged(NULL);
	FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FCustomizableObjectEditor::OnObjectModified);
	
	UCustomizableObjectPrivate* CustomizableObjectPrivate = CustomizableObject->GetPrivate();
	
	PreviewInstance = CustomizableObject->CreateInstance();	
	PreviewInstance->UpdatedNativeDelegate.AddSP(SharedThis(this), &FCustomizableObjectEditor::OnUpdatePreviewInstance);
	PreviewInstance->SetBuildParameterRelevancy(true);
	// Set the message log listing to get update messages
	PreviewInstance->GetPrivate()->UpdateLogger = CustomizableObjectPrivate->CompilationLogger;

	CustomizableInstanceDetailsView->SetObject(PreviewInstance, true);
	
	CustomizableObjectPrivate->Status.GetOnStateChangedDelegate().AddRaw(this, &FCustomizableObjectEditor::OnCustomizableObjectStatusChanged);
	OnCustomizableObjectStatusChanged(FCustomizableObjectStatusTypes::EState::Loading, CustomizableObjectPrivate->Status.Get());  // Fake we are still in the loading phase.
	
	CustomizableObject->GetPostCompileDelegate().AddSP(this, &FCustomizableObjectEditor::OnPostCompile); // Must be attached after creating the Instance since the Instance also does some work in this delegate.
}


FName FCustomizableObjectEditor::GetToolkitFName() const
{
	return FName("CustomizableObjectEditor");
}


FText FCustomizableObjectEditor::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Customizable Object Editor");
}


void FCustomizableObjectEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( CustomizableObject );
	Collector.AddReferencedObject( PreviewInstance );
	Collector.AddReferencedObject( ProjectorParameter );
	Collector.AddReferencedObject( CustomSettings );
	Collector.AddReferencedObject( EditorProperties );
}


FCustomizableObjectEditor::FCustomizableObjectEditor(UCustomizableObject& ObjectToEdit) :
	CustomizableObject(&ObjectToEdit) {}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_Viewport( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == ViewportTabId );

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.Label( FText::FromString( GetTabPrefix() + LOCTEXT("CustomizableObjectViewport_TabTitle", "Viewport").ToString() ) )
		[
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.Padding( 2.0f )
			.FillHeight(1.0f)
			[
				Viewport.ToSharedRef()
			]
		];

	DockTab->SetTabIcon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.Preview"));

	return DockTab;
}

TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_Details( const FSpawnTabArgs& Args )
{
	check(Args.GetTabId() == DetailsTabId);

	TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox) 
	+SScrollBox::Slot()
	[
		ObjectDetailsView.ToSharedRef()
	];

	ScrollBox->SetScrollBarRightClickDragAllowed(true);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab).Label(FText::FromString(GetTabPrefix() + LOCTEXT("Details_TabTitle", "Details").ToString()))
	[
		ScrollBox
	];

	DockTab->SetTabIcon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.CustomizableObjectProperties"));

	return DockTab;
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_InstanceProperties( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == InstancePropertiesTabId );

	TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			CustomizableInstanceDetailsView.ToSharedRef()
		];

	ScrollBox->SetScrollBarRightClickDragAllowed(true);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.Label( FText::FromString( GetTabPrefix() + LOCTEXT( "CustomizableInstanceProperties_TabTitle", "Preview Instance" ).ToString() ) )
		[
			ScrollBox
		];

	DockTab->SetTabIcon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.CustomizableInstanceProperties"));

	return DockTab;
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_Graph( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == GraphTabId );

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FCustomizableObjectEditor::OnSelectedGraphNodesChanged);

	CreateGraphEditorWidget(CustomizableObject->GetPrivate()->GetSource(), InEvents);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.Label( FText::FromString( GetTabPrefix() + LOCTEXT( "SourceGraph", "Source Graph" ).ToString() ) )
		.TabColorScale( GetTabColorScale() )
		[
			GraphEditor.ToSharedRef()
		];

	DockTab->SetTabIcon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.NodeGraph"));

	return DockTab;
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_AdvancedPreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == AdvancedPreviewSettingsTabId);
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.Label(LOCTEXT("StaticMeshPreviewScene_TabTitle", "Preview Scene Settings"))
		[
			AdvancedPreviewSettingsWidget.ToSharedRef()
		];

	DockTab->SetTabIcon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.PreviewSettings"));

	return DockTab;
}


UCustomizableObjectInstance* FCustomizableObjectEditor::GetPreviewInstance()
{
	return PreviewInstance;
}


void FCustomizableObjectEditor::BindCommands()
{
	const FCustomizableObjectEditorCommands& Commands = FCustomizableObjectEditorCommands::Get();

	// Toolbar
	// Compile and options
	ToolkitCommands->MapAction(
		Commands.Compile,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::CompileObject, false, false),
		FCanExecuteAction::CreateStatic(&UCustomizableObjectSystem::IsActive),
		FIsActionChecked());

	ToolkitCommands->MapAction(
		Commands.CompileOnlySelected,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::CompileObject, true, false),
		FCanExecuteAction::CreateStatic(&UCustomizableObjectSystem::IsActive),
		FIsActionChecked());

	// Compile and options
	ToolkitCommands->MapAction(
		Commands.ResetCompileOptions,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::ResetCompileOptions),
		FCanExecuteAction(),
		FIsActionChecked());

	// Debug and options
	ToolkitCommands->MapAction(
		Commands.GraphViewer,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::GraphViewer),
		FCanExecuteAction(),
		FIsActionChecked());

	ToolkitCommands->MapAction(
		Commands.CodeViewer,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::CodeViewer),
		FCanExecuteAction(),
		FIsActionChecked());

	ToolkitCommands->MapAction(
		Commands.UpdateCookDataDistributionId,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::UpdateCookDataDistributionId),
		FCanExecuteAction(),
		FIsActionChecked());
	
	// References
	ToolkitCommands->MapAction(
		Commands.CompileGatherReferences,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::CompileObject, false, true),
		FCanExecuteAction(),
		FIsActionChecked());

	ToolkitCommands->MapAction(
		Commands.ClearGatheredReferences,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::ClearGatheredReferences),
		FCanExecuteAction(),
		FIsActionChecked());
	
	// Texture Analyzer
	ToolkitCommands->MapAction(
		Commands.TextureAnalyzer,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::OpenTextureAnalyzerTab),
		FCanExecuteAction(),
		FIsActionChecked());
	
	// Performance Analyzer
	ToolkitCommands->MapAction(
		Commands.PerformanceAnalyzer,
		FExecuteAction::CreateSP(this, &FCustomizableObjectEditor::OpenPerformanceAnalyzerTab),
		FCanExecuteAction(),
		FIsActionChecked());
}


bool FCustomizableObjectEditor::GroupNodeIsLinkedToParentByName(UCustomizableObjectNodeObject* Node, UCustomizableObject* Test, const FString& ParentGroupName)
{
	TArray<UCustomizableObjectNodeObjectGroup*> GroupNodes;
	Test->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeObjectGroup>(GroupNodes);

	for (UCustomizableObjectNodeObjectGroup* GroupNode : GroupNodes)
	{
		if ((Node->ParentObjectGroupId == GroupNode->NodeGuid) && (ParentGroupName == GroupNode->GetGroupName()))
		{
			return true;
		}
	}

	return false;
}


// TODO FutureGMT, use graph traversal abstraction instead of a hardcoded implementation.
void FCustomizableObjectEditor::ReconstructAllChildNodes(UCustomizableObjectNode& StartNode, const UClass& NodeType)
{
	UCustomizableObject* Object = CastChecked<UCustomizableObject>(StartNode.GetCustomizableObjectGraph()->GetOuter());
	const TMultiMap<FGuid, UCustomizableObjectNodeObject*> Mapping = GetNodeGroupObjectNodeMapping(Object);
	
	TArray<UCustomizableObjectNode*> NodesToVisit;
	NodesToVisit.Add(&StartNode);
	
	while (!NodesToVisit.IsEmpty())
	{
		UCustomizableObjectNode* Node = NodesToVisit.Pop();

		if (&NodeType == Node->GetClass())
		{
			Node->UCustomizableObjectNode::ReconstructNode();							
		}

		if (const UCustomizableObjectNodeObjectGroup* GroupNode = Cast<UCustomizableObjectNodeObjectGroup>(Node))
		{
			TArray<UCustomizableObjectNodeObject*> ObjectNodes;
			Mapping.MultiFind(GroupNode->NodeGuid, ObjectNodes);
			
			for (UCustomizableObjectNodeObject* ObjectNode : ObjectNodes)
			{
				NodesToVisit.Add(ObjectNode);	
			}
		}
		
		for (const UEdGraphPin* Pin : Node->GetAllPins()) // Not using GetAllNonOrphanPins on purpose since we want want to be able to reconstruct nodes that have non-orphan pins.
		{
			if (Pin->Direction != EGPD_Input)
			{
				continue;
			}

			for (const UEdGraphPin* ConnectedPin : FollowInputPinArray(*Pin))
			{
				if (UCustomizableObjectNode* TypedNode = Cast<UCustomizableObjectNode>(ConnectedPin->GetOwningNode()))
				{
					NodesToVisit.Add(TypedNode);
				}
			}
		}
	}
}


UProjectorParameter* FCustomizableObjectEditor::GetProjectorParameter()
{
	return ProjectorParameter;
}


UCustomSettings* FCustomizableObjectEditor::GetCustomSettings()
{
	return CustomSettings;
}


void FCustomizableObjectEditor::HideGizmo()
{
	HideGizmoProjectorNodeProjectorConstant();
	HideGizmoProjectorNodeProjectorParameter();
	HideGizmoProjectorParameter();
	HideGizmoClipMorph();
	HideGizmoClipMesh();
	HideGizmoLight();
}


void FCustomizableObjectEditor::ShowGizmoProjectorNodeProjectorConstant(UCustomizableObjectNodeProjectorConstant& Node)
{
	if (GizmoType != EGizmoType::NodeProjectorConstant)
	{
		HideGizmo();
	}

	GizmoType = EGizmoType::NodeProjectorConstant;

	SelectSingleNode(Node);
	
	FProjectorTypeDelegate ProjectorTypeDelegate;
	ProjectorTypeDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::GetProjectorType);		

	FWidgetColorDelegate WidgetColorDelegate;
	WidgetColorDelegate.BindLambda([]() { return FColor::Red; });

	FWidgetLocationDelegate WidgetLocationDelegate;
	WidgetLocationDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::GetProjectorPosition);

	FOnWidgetLocationChangedDelegate OnWidgetLocationChangedDelegate;
	OnWidgetLocationChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::SetProjectorPosition);

	FWidgetDirectionDelegate WidgetDirectionDelegate;
	WidgetDirectionDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::GetProjectorDirection);

	FOnWidgetDirectionChangedDelegate OnWidgetDirectionChangedDelegate;
	OnWidgetDirectionChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::SetProjectorDirection);

	FWidgetUpDelegate WidgetUpDelegate;
	WidgetUpDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::GetProjectorUp);

	FOnWidgetUpChangedDelegate OnWidgetUpChangedDelegate;
	OnWidgetUpChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::SetProjectorUp);

	FWidgetScaleDelegate WidgetScaleDelegate;
	WidgetScaleDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::GetProjectorScale);

	FOnWidgetScaleChangedDelegate OnWidgetScaleChangedDelegate;
	OnWidgetScaleChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::SetProjectorScale);

	FWidgetAngleDelegate WidgetAngleDelegate;
	WidgetAngleDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorConstant::GetProjectorAngle);

	FWidgetTrackingStartedDelegate WidgetTrackingStartedDelegate;
	WidgetTrackingStartedDelegate.BindLambda([WeakNode = MakeWeakObjectPtr(&Node)]()
	{
		if (UCustomizableObjectNodeProjectorConstant* Node = WeakNode.Get())
		{
			Node->Modify();
		}
	});
	
	Viewport->ShowGizmoProjector(WidgetLocationDelegate, OnWidgetLocationChangedDelegate,
		WidgetDirectionDelegate, OnWidgetDirectionChangedDelegate,
		WidgetUpDelegate, OnWidgetUpChangedDelegate,
		WidgetScaleDelegate, OnWidgetScaleChangedDelegate,
		WidgetAngleDelegate,
		ProjectorTypeDelegate,
		WidgetColorDelegate,
		WidgetTrackingStartedDelegate);
}


void FCustomizableObjectEditor::HideGizmoProjectorNodeProjectorConstant()
{
	if (GizmoType != EGizmoType::NodeProjectorConstant)
	{
		return;
	}

	GizmoType = EGizmoType::Hidden;
	
	Viewport->HideGizmoProjector();

	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt( SelectedNodes ); NodeIt; ++NodeIt)
	{
		const UObject* Node = *NodeIt;
		if (Node->IsA<UCustomizableObjectNodeProjectorConstant>())
		{
			GraphEditor->ClearSelectionSet();
			break;
		}
	}			
}


void FCustomizableObjectEditor::ShowGizmoProjectorNodeProjectorParameter(UCustomizableObjectNodeProjectorParameter& Node)
{
	if (GizmoType != EGizmoType::NodeProjectorParameter)
	{
		HideGizmo();
		GizmoType = EGizmoType::NodeProjectorParameter;
	}

	SelectSingleNode(Node);
	
	FProjectorTypeDelegate ProjectorTypeDelegate;
	ProjectorTypeDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::GetProjectorType);		

	FWidgetColorDelegate WidgetColorDelegate;
	WidgetColorDelegate.BindLambda([]() { return FColor::Red; });
	
	FWidgetLocationDelegate WidgetLocationDelegate;
	WidgetLocationDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultPosition);

	FOnWidgetLocationChangedDelegate OnWidgetLocationChangedDelegate;
	OnWidgetLocationChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultPosition);

	FWidgetDirectionDelegate WidgetDirectionDelegate;
	WidgetDirectionDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultDirection);

	FOnWidgetDirectionChangedDelegate OnWidgetDirectionChangedDelegate;
	OnWidgetDirectionChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultDirection);

	FWidgetUpDelegate WidgetUpDelegate;
	WidgetUpDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultUp);

	FOnWidgetUpChangedDelegate OnWidgetUpChangedDelegate;
	OnWidgetUpChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultUp);

	FWidgetScaleDelegate WidgetScaleDelegate;
	WidgetScaleDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultScale);

	FOnWidgetScaleChangedDelegate OnWidgetScaleChangedDelegate;
	OnWidgetScaleChangedDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultScale);

	FWidgetAngleDelegate WidgetAngleDelegate;
	WidgetAngleDelegate.BindUObject(&Node, &UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultAngle);
	
	FWidgetTrackingStartedDelegate WidgetTrackingStartedDelegate;
	WidgetTrackingStartedDelegate.BindLambda([WeakNode = MakeWeakObjectPtr(&Node)]()
	{
		if (UCustomizableObjectNodeProjectorParameter* Node = WeakNode.Get())
		{
			Node->Modify();
		}
	});
	
	Viewport->ShowGizmoProjector(WidgetLocationDelegate, OnWidgetLocationChangedDelegate,
		WidgetDirectionDelegate, OnWidgetDirectionChangedDelegate,
		WidgetUpDelegate, OnWidgetUpChangedDelegate,
		WidgetScaleDelegate, OnWidgetScaleChangedDelegate,
		WidgetAngleDelegate,
		ProjectorTypeDelegate,
		WidgetColorDelegate,
		WidgetTrackingStartedDelegate);
}


void FCustomizableObjectEditor::HideGizmoProjectorNodeProjectorParameter()
{
	if (GizmoType != EGizmoType::NodeProjectorParameter)
	{
		return;
	}

	GizmoType = EGizmoType::Hidden;

	Viewport->HideGizmoProjector();

	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt( SelectedNodes ); NodeIt; ++NodeIt)
	{
		const UObject* Node = *NodeIt;
		if (Node->IsA<UCustomizableObjectNodeProjectorParameter>())
		{
			GraphEditor->ClearSelectionSet();
			break;
		}
	}	
}


void FCustomizableObjectEditor::ShowGizmoProjectorParameter(const FString& ParamName, int32 RangeIndex)
{
	if (GizmoType != EGizmoType::ProjectorParameter)
	{
		HideGizmo();		
		GizmoType = EGizmoType::ProjectorParameter;
	}
	
	FCustomizableObjectInstanceEditor::ShowGizmoProjectorParameter(ParamName, RangeIndex, SharedThis(this), Viewport, CustomizableInstanceDetailsView, ProjectorParameter, PreviewInstance);
}


void FCustomizableObjectEditor::HideGizmoProjectorParameter()
{
	if (GizmoType != EGizmoType::ProjectorParameter)
	{
		return;	
	}

	GizmoType = EGizmoType::Hidden;

	FCustomizableObjectInstanceEditor::HideGizmoProjectorParameter(SharedThis(this), Viewport, CustomizableInstanceDetailsView);
}


void FCustomizableObjectEditor::ShowGizmoClipMorph(UCustomizableObjectNodeModifierClipMorph& Node)
{
	if (Node.BoneName == FName())
	{	
		return;
	}

	if (GizmoType != EGizmoType::ClipMorph)
	{
		HideGizmo();		
		GizmoType = EGizmoType::ClipMorph;
	}
	
	SelectSingleNode(Node);

	Viewport->ShowGizmoClipMorph(Node);
}


void FCustomizableObjectEditor::HideGizmoClipMorph(bool bClearGraphSelectionSet /**= true*/)
{
	if (GizmoType != EGizmoType::ClipMorph)
	{
		return;	
	}
	
	GizmoType = EGizmoType::Hidden;

	Viewport->HideGizmoClipMorph();

	if (bClearGraphSelectionSet)
	{
		const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
		for (FGraphPanelSelectionSet::TConstIterator NodeIt( SelectedNodes ); NodeIt; ++NodeIt)
		{
			const UObject* Node = *NodeIt;
			if (Node->IsA<UCustomizableObjectNodeModifierClipMorph>())
			{
				GraphEditor->ClearSelectionSet();
				break;
			}
		}	
	}
}


void FCustomizableObjectEditor::ShowGizmoClipMesh(UCustomizableObjectNode& Node, FTransform* Transform, const UEdGraphPin& MeshPin)
{
	UObject* ClipMesh = nullptr;
	int32 LODIndex = 0;
	int32 SectionIndex = 0;
	int32 MaterialSlotIndex = 0;
	
	// Try to get the CLipMesh from the preview mesh part of the UCONodeClipSkeletalMeshWithSkeletalMesh node. This is the only node with this functionality for now.
	if (const UCONodeClipSkeletalMeshWithSkeletalMesh* ClipSkeletalMeshWithSkeletalMeshNode = Cast<UCONodeClipSkeletalMeshWithSkeletalMesh>(&Node))
	{
		ClipMesh = ClipSkeletalMeshWithSkeletalMeshNode->PreviewMesh;
		LODIndex = ClipSkeletalMeshWithSkeletalMeshNode->PreviewMeshLOD;
		SectionIndex = ClipSkeletalMeshWithSkeletalMeshNode->PreviewMeshSection;			// -1 Will show all sections
		MaterialSlotIndex = ClipSkeletalMeshWithSkeletalMeshNode->PreviewMeshSection;		// -1 Will show all sections
	}
	
	// No ClipMesh could be found. try getting it from the connected node
	if (!ClipMesh)
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(MeshPin))
		{
			if (const UEdGraphNode* ConnectedNode = ConnectedPin->GetOwningNode())
			{
				if (const UCustomizableObjectNodeStaticMesh* StaticMeshNode = Cast<UCustomizableObjectNodeStaticMesh>(ConnectedNode))
				{
					ClipMesh = UE::Mutable::Private::LoadObject(StaticMeshNode->GetMesh());
					StaticMeshNode->GetPinSection(*ConnectedPin, LODIndex, SectionIndex);
					MaterialSlotIndex = SectionIndex;
				}
				else if (const UCustomizableObjectNodeSkeletalMesh* SkeletalMeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(ConnectedNode))
				{
					ClipMesh = UE::Mutable::Private::LoadObject(SkeletalMeshNode->GetMesh());
					SkeletalMeshNode->GetPinSection(*ConnectedPin, LODIndex, SectionIndex);
					MaterialSlotIndex = SkeletalMeshNode->GetSkeletalMaterialIndexFor(*ConnectedPin);
				}
				else if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(ConnectedNode))
				{
					ClipMesh = TableNode->GetColumnDefaultAssetByType<UObject>(ConnectedPin);

					TableNode->GetPinLODAndSection(ConnectedPin, LODIndex, SectionIndex);
					MaterialSlotIndex = SectionIndex;

					if (TableNode->GetPinMeshType(ConnectedPin) == ETableMeshPinType::SKELETAL_MESH)
					{
						MaterialSlotIndex = TableNode->GetDefaultSkeletalMaterialIndexFor(*ConnectedPin);
					}
				}
			}
		}
	}
	
	if (ClipMesh && LODIndex >= 0)
	{
		if (GizmoType != EGizmoType::ClipMesh)
		{
			HideGizmo();
			GizmoType = EGizmoType::ClipMesh;
		}

		SelectSingleNode(Node);

		Viewport->ShowGizmoClipMesh(Node, Transform, *ClipMesh, LODIndex, SectionIndex, MaterialSlotIndex);
	}
	else
	{
		// If the mesh to set is not valid ensure the viewport shows no mesh.
		Viewport->HideGizmoClipMesh();
	}
}


void FCustomizableObjectEditor::HideGizmoClipMesh()
{
	if (GizmoType != EGizmoType::ClipMesh)
	{
		return;	
	}

	GizmoType = EGizmoType::Hidden;

	Viewport->HideGizmoClipMesh();

	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt( SelectedNodes ); NodeIt; ++NodeIt)
	{
		const UObject* Node = *NodeIt;
		if (Node->IsA<UCustomizableObjectNodeModifierClipWithMesh>() ||
			Node->IsA<UCustomizableObjectNodeModifierTransformInMesh>() ||
			Node->IsA<UCONodeTransformInMesh>() ||
			Node->IsA<UCONodeClipMeshWithMesh>() ||
			Node->IsA<UCONodeClipSkeletalMeshWithSkeletalMesh>())
		{
			GraphEditor->ClearSelectionSet();
			break;
		}
	}	
}


void FCustomizableObjectEditor::ShowGizmoLight(ULightComponent& InSelectedLight)
{
	if (GizmoType != EGizmoType::Light)
	{
		HideGizmo();
		GizmoType = EGizmoType::Light;
	}
	
	CustomSettings->SetSelectedLight(&InSelectedLight);

	Viewport->ShowGizmoLight(InSelectedLight);
	
	CustomizableObjectEditorAdvancedPreviewSettings->Refresh();
}


void FCustomizableObjectEditor::HideGizmoLight()
{
	if (GizmoType != EGizmoType::Light)
	{
		return;	
	}
	
	GizmoType = EGizmoType::Hidden;

	CustomSettings->SetSelectedLight(nullptr);

	Viewport->HideGizmoLight();

	CustomizableObjectEditorAdvancedPreviewSettings->Refresh();
}


UCustomizableObjectEditorProperties* FCustomizableObjectEditor::GetEditorProperties()
{
	return EditorProperties;
}


void FCustomizableObjectEditor::PostUndo(bool bSuccess)
{
	FCustomizableObjectGraphEditorToolkit::PostUndo(bSuccess);

	if (bSuccess)
	{
		if (ObjectDetailsView.IsValid())
		{
			ObjectDetailsView->RemoveInvalidObjects();
		}

		if (CustomizableInstanceDetailsView.IsValid())
		{
			CustomizableInstanceDetailsView->RemoveInvalidObjects();
		}

		CustomizableObject->MarkPackageDirty();

		FSlateApplication::Get().DismissAllMenus();
	}
}


FString FCustomizableObjectEditor::GetDocumentationLink() const
{
	return DocumentationURL;
}


void FCustomizableObjectEditor::ExtendToolbar()
{
	TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FCustomizableObjectEditor* Editor, TSharedPtr<FUICommandList> CommandList)
		{
			ToolbarBuilder.BeginSection("Compilation");
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorCommands::Get().Compile);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorCommands::Get().CompileOnlySelected);
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(Editor, &FCustomizableObjectEditor::GenerateCompileOptionsMenuContent, CommandList.ToSharedRef()),
				LOCTEXT("Compile_Options_Label", "Compile Options"),
				LOCTEXT("Compile_Options_Tooltip", "Change Compile Options"),
				TAttribute<FSlateIcon>(),
				true);
			ToolbarBuilder.EndSection();
			
			ToolbarBuilder.BeginSection("Information");
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorCommands::Get().TextureAnalyzer);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorCommands::Get().PerformanceAnalyzer);
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this, CommandList));

	AddToolbarExtender(ToolbarExtender);

	ICustomizableObjectEditorModule* CustomizableObjectEditorModule = &FModuleManager::LoadModuleChecked<ICustomizableObjectEditorModule>("CustomizableObjectEditor");
	AddToolbarExtender(CustomizableObjectEditorModule->GetCustomizableObjectEditorToolBarExtensibilityManager()->GetAllExtenders());
}


TSharedRef<SWidget> FCustomizableObjectEditor::GenerateCompileOptionsMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	// settings
	MenuBuilder.BeginSection("ResetCompileOptions");
	{
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorCommands::Get().ResetCompileOptions);
	}
	MenuBuilder.EndSection();

	if (!CustomizableObject)
	{
		return MenuBuilder.MakeWidget();
	}

	MenuBuilder.BeginSection("Optimization", LOCTEXT("MutableCompileOptimizationHeading", "Optimization"));
	{
		// Level
		CompileOptimizationStrings.Empty();
		CompileOptimizationStrings.Add(MakeShareable(new FString(LOCTEXT("OptimizationNone", "None (Disable texture streaming)").ToString())));
		CompileOptimizationStrings.Add(MakeShareable(new FString(LOCTEXT("OptimizationMin", "Minimal").ToString())));
		CompileOptimizationStrings.Add(MakeShareable(new FString(LOCTEXT("OptimizationMax", "Maximum").ToString())));
		check(CompileOptimizationStrings.Num() == UE_MUTABLE_MAX_OPTIMIZATION + 1);

		if (CustomizableObject)
		{
			int32 SelectedOptimization = FMath::Clamp(CustomizableObject->GetPrivate()->OptimizationLevel, 0, CompileOptimizationStrings.Num() - 1);
			CompileOptimizationCombo =
				SNew(STextComboBox)
				.OptionsSource(&CompileOptimizationStrings)
				.InitiallySelectedItem(CompileOptimizationStrings[SelectedOptimization])
				.OnSelectionChanged(this, &FCustomizableObjectEditor::OnChangeCompileOptimizationLevel)
				;

			MenuBuilder.AddWidget(CompileOptimizationCombo.ToSharedRef(), LOCTEXT("MutableCompileOptimizationLevel", "Optimization Level"));
		}

		{
			CompileTextureCompressionStrings.Empty();
			CompileTextureCompressionStrings.Add(MakeShareable(new FString(LOCTEXT("MutableTextureCompressionNone", "None").ToString())));
			CompileTextureCompressionStrings.Add(MakeShareable(new FString(LOCTEXT("MutableTextureCompressionFast", "Fast").ToString())));
			CompileTextureCompressionStrings.Add(MakeShareable(new FString(LOCTEXT("MutableTextureCompressionHighQuality", "High Quality").ToString())));

			int32 SelectedCompression = FMath::Clamp(int32(CustomizableObject->GetPrivate()->TextureCompression), 0, CompileTextureCompressionStrings.Num() - 1);
			CompileTextureCompressionCombo =
				SNew(STextComboBox)
				.OptionsSource(&CompileTextureCompressionStrings)
				.InitiallySelectedItem(CompileTextureCompressionStrings[SelectedCompression])
				.OnSelectionChanged(this, &FCustomizableObjectEditor::OnChangeCompileTextureCompressionType)
				;

			MenuBuilder.AddWidget(CompileTextureCompressionCombo.ToSharedRef(), LOCTEXT("MutableCompileTextureCompressionType", "Texture Compression"));
		}

	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Packaging", LOCTEXT("MutableCompilePackagingHeading", "Packaging"));
	{
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorCommands::Get().UpdateCookDataDistributionId);

		// Unfortunately SNumericDropDown doesn't work with integers at the time of writing.
		TArray<SNumericDropDown<float>::FNamedValue> EmbeddedOptions;
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(0, FText::FromString(TEXT("0")), FText::FromString(TEXT("Disabled"))));
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(16, FText::FromString(TEXT("16")), FText::FromString(TEXT("16"))));
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(64, FText::FromString(TEXT("64")), FText::FromString(TEXT("64"))));
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(256, FText::FromString(TEXT("256")), FText::FromString(TEXT("256"))));
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(512, FText::FromString(TEXT("512")), FText::FromString(TEXT("512"))));
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(1024, FText::FromString(TEXT("1024")), FText::FromString(TEXT("1024"))));
		EmbeddedOptions.Add(SNumericDropDown<float>::FNamedValue(4096, FText::FromString(TEXT("4096")), FText::FromString(TEXT("4096"))));

		EmbeddedDataLimitCombo = SNew(SNumericDropDown<float>)
			.DropDownValues(EmbeddedOptions)
			.Value_Lambda([this]()
				{
					return CustomizableObject ? float(CustomizableObject->GetPrivate()->EmbeddedDataBytesLimit) : 0.0f;
				})
			.OnValueChanged_Lambda([this](float Value)
				{
					if (CustomizableObject)
					{
						CustomizableObject->GetPrivate()->EmbeddedDataBytesLimit = uint64(Value);
						CustomizableObject->Modify();
					}
				});
			MenuBuilder.AddWidget(EmbeddedDataLimitCombo.ToSharedRef(), LOCTEXT("MutableCompileEmbeddedLimit", "Embedded Data Limit (Bytes)"));
	}
	MenuBuilder.EndSection();

	// Debugging options
	MenuBuilder.BeginSection("Debugger", LOCTEXT("MutableDebugger", "Debugger"));
	{
		// Platform
		TargetPlatformNames.Empty();
		
		if (ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager())
		{
			const ITargetPlatform* CurrentPlatform = TargetPlatformManager->GetRunningTargetPlatform();
			for (const ITargetPlatform* Platform : TargetPlatformManager->GetTargetPlatforms())
			{
				const TSharedPtr<FString> ThisPlatform = MakeShareable(new FString(Platform->PlatformName()));
				TargetPlatformNames.Add(ThisPlatform);

				if (Platform == CurrentPlatform)
				{
					DebugTargetPlatformName = ThisPlatform;
				}
			}
		}
		
		TSharedRef<STextComboBox> DebugPlatformCombo = SNew(STextComboBox)
			.OptionsSource(&TargetPlatformNames)
			.InitiallySelectedItem(DebugTargetPlatformName)
			.OnSelectionChanged(this, &FCustomizableObjectEditor::OnChangeDebugPlatform);

		MenuBuilder.AddWidget(DebugPlatformCombo, LOCTEXT("MutableDebugPlatform", "Target Platform")); 

		MenuBuilder.AddMenuEntry(
				LOCTEXT("ForceLargeLODBias", "Force a large texture LODBias."),
				LOCTEXT("ForceLargeLODBiasTooltip", "This is useful to test compilation of special cook modes."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this]() { bDebugForceLargeLODBias = !bDebugForceLargeLODBias; }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this]() { return bDebugForceLargeLODBias; })),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorCommands::Get().GraphViewer);
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorCommands::Get().CodeViewer);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("References", LOCTEXT("References", "References"));
	{
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorCommands::Get().CompileGatherReferences);
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorCommands::Get().ClearGatheredReferences);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


FText FCustomizableObjectEditor::GetToolkitName() const
{
	return FText::FromString(GetEditingObject()->GetName());
}


FString FCustomizableObjectEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT( "WorldCentricTabPrefix", "CustomizableObject " ).ToString();
}


FLinearColor FCustomizableObjectEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.3f, 0.2f, 0.5f, 0.5f );
}


UCustomizableObject* FCustomizableObjectEditor::GetCustomizableObject()
{
	return CustomizableObject;
}


void FCustomizableObjectEditor::RefreshTool()
{
	if (ViewportClient)
	{
		ViewportClient->Invalidate();
	}
}


TSharedPtr<SCustomizableObjectEditorViewportTabBody> FCustomizableObjectEditor::GetViewport()
{
	return Viewport;
}


void FCustomizableObjectEditor::OnObjectPropertySelectionChanged(FProperty* InProperty)
{
	CustomizableObject->PostEditChange();

	if (ViewportClient)
	{
		ViewportClient->Invalidate();
	}
}


void FCustomizableObjectEditor::OnInstancePropertySelectionChanged(FProperty* InProperty)
{
	if (ViewportClient)
	{
		ViewportClient->Invalidate();
	}
}


void FCustomizableObjectEditor::OnObjectModified(UObject* Object)
{
	if (const UCustomizableObjectInstance* Instance = Cast<UCustomizableObjectInstance>(Object); !Instance)
	{
		// Sometimes when another CO is open in another editor window/tab, it triggers this callback, so prevent the modification of this object by a callback triggered by another one
		if (UCustomizableObject* AuxCustomizableObject = Cast<UCustomizableObject>(Object))
		{
			AuxCustomizableObject->GetPrivate()->UpdateVersionId();
		}
		else if (UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(Object))
		{
			if (UCustomizableObjectGraph* Graph = Cast<UCustomizableObjectGraph>(Node->GetOuter()))
			{
				if (UCustomizableObject* AuxOuterCustomizableObject = Cast<UCustomizableObject>(Graph->GetOuter()))
				{
					AuxOuterCustomizableObject->GetPrivate()->UpdateVersionId();
				}
			}
		}
		else if (UCustomizableObjectGraph* Graph = Cast<UCustomizableObjectGraph>(Object))
		{
			if (UCustomizableObject* AuxOuterCustomizableObject = Cast<UCustomizableObject>(Graph->GetOuter()))
			{
				AuxOuterCustomizableObject->GetPrivate()->UpdateVersionId();
			}
		}
	}
}


void FCustomizableObjectEditor::CompileObject(bool bOnlySelectedParameters, bool bGatherReferences)
{
	// Resetting viewport parameters
	Viewport->SetDrawDefaultUVMaterial();

	if (CustomizableObject->GetPrivate()->Status.Get() == FCustomizableObjectStatus::EState::Loading)
	{
		FNotificationInfo Info(LOCTEXT("CustomizableObjectCompileTryLater", "Please wait until Customizable Object is loaded"));
		Info.bFireAndForget = true;
		Info.bUseThrobber = true;
		Info.FadeOutDuration = 1.0f;
		Info.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}

	if (!CustomizableObject->GetPrivate()->GetSource())
	{
		return;
	}

	TSharedRef<FCompilationRequest> CompileRequest = MakeShared<FCompilationRequest>(*CustomizableObject);
	CompileRequest->bSilentCompilation = false;
	CompileRequest->Options.bGatherReferences = bGatherReferences;

	if (bOnlySelectedParameters)
	{
		CompileRequest->Options.ParamNamesToSelectedOptions = GetCompileOnlySelectedParameters(*GetPreviewInstance());
	}

	//Ensure the Stats tab is open and empty
	GetTabManager()->TryInvokeTab(FCustomizableObjectEditor::StatsTabId);

	ICustomizableObjectEditorModulePrivate::GetChecked().EnqueueCompileRequest(CompileRequest);
}


const ITargetPlatform* GetTargetPlatform(const TSharedPtr<FString>& TargetPlatformName)
{
	ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager();
	if (!TargetPlatformManager)
	{
		return nullptr;
	}

	for (const ITargetPlatform* Platform : TargetPlatformManager->GetTargetPlatforms())
	{
		if (*TargetPlatformName.Get() == Platform->PlatformName())
		{
			return Platform;
		}
	}

	return nullptr;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> FCustomizableObjectEditor::DebuggerCompile(TSharedPtr<UE::Mutable::Private::FModel>& Model,
	TArray<TSoftObjectPtr<const UTexture>>& RuntimeTextures,
	TArray<FMutableSourceTextureData>& CompilerTextures,
	TArray<TSoftObjectPtr<const UStreamableRenderAsset>>& RuntimeMeshes,
	TArray<FMutableSourceMeshData>& CompilerMeshes) const
{
	FCompilationOptions CompileOptions = GetCompilationOptions(*CustomizableObject);
	CompileOptions.TargetPlatform = GetTargetPlatform(DebugTargetPlatformName);
	CompileOptions.bForceLargeLODBias = bDebugForceLargeLODBias;
	
	if (CompileOptions.bForceLargeLODBias)
	{
		// Debug compile with many different biases
		constexpr int32 MaxBias = 15;
		for (int32 Bias = 0; Bias < MaxBias; ++Bias)
		{
			CompileOptions.DebugBias = Bias;

			TSharedRef<FCustomizableObjectCompiler> Compiler = MakeShared<FCustomizableObjectCompiler>();
			RuntimeTextures.Empty();
			CompilerTextures.Empty();
			RuntimeMeshes.Empty();
			CompilerMeshes.Empty();
			UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> RootNode = Compiler->Export(CustomizableObject, CompileOptions, RuntimeTextures, CompilerTextures, RuntimeMeshes, CompilerMeshes);
			if (!RootNode)
			{
				// TODO: Show errors
				ensure(false);
				return nullptr;
			}

			// Do the compilation to Mutable Code synchronously.
			TSharedPtr<FCustomizableObjectCompileRunnable> CompileTask = MakeShareable(new FCustomizableObjectCompileRunnable(RootNode, Compiler));
			CompileTask->Options = CompileOptions;
			CompileTask->ReferencedTextures = CompilerTextures;
			CompileTask->ReferencedMeshes = CompilerMeshes;
			CompileTask->Init();
			CompileTask->Run();
		}
	}

	// Convert from Unreal graph to Mutable graph.
	TSharedRef<FCustomizableObjectCompiler> Compiler = MakeShared<FCustomizableObjectCompiler>();
	RuntimeTextures.Empty();
	CompilerTextures.Empty();
	RuntimeMeshes.Empty();
	CompilerMeshes.Empty();
	UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> RootNode = Compiler->Export(CustomizableObject, CompileOptions, RuntimeTextures, CompilerTextures, RuntimeMeshes, CompilerMeshes);
	if (!RootNode)
	{
		// TODO: Show errors
		ensure(false);
		return nullptr;
	}

	// Do the compilation to Mutable Code synchronously.
	TSharedPtr<FCustomizableObjectCompileRunnable> CompileTask = MakeShareable(new FCustomizableObjectCompileRunnable(RootNode, Compiler));
	CompileTask->Options = CompileOptions;
	CompileTask->ReferencedTextures = CompilerTextures;
	CompileTask->ReferencedMeshes = CompilerMeshes;
	CompileTask->Init();
	CompileTask->Run();

	Model = CompileTask->Model;
	
	return RootNode;
}


void FCustomizableObjectEditor::GraphViewer() const
{
	TSharedPtr<UE::Mutable::Private::FModel> Model;
	TArray<TSoftObjectPtr<const UTexture>> RuntimeTextures;
	TArray<FMutableSourceTextureData> CompilerTextures;
	TArray<TSoftObjectPtr<const UStreamableRenderAsset>> RuntimeMeshes;
	TArray<FMutableSourceMeshData> CompilerMeshes;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> RootNode = DebuggerCompile(Model, RuntimeTextures, CompilerTextures, RuntimeMeshes, CompilerMeshes);
	
	if (!RootNode)
	{
		FNotificationInfo Info(LOCTEXT("GraphViewerError", "Unable to open Graph Viewer: No root node"));
		FSlateNotificationManager::Get().AddNotification(Info);
		
		return;
	}
	
	const TSharedPtr<SDockTab> NewMutableObjectTab = SNew(SDockTab)
	.Label(LOCTEXT("GraphViewer", "Graph Viewer"))
	[
		SNew(SMutableGraphViewer, RootNode)
		.ReferencedRuntimeTextures(RuntimeTextures)
		.ReferencedCompileTextures(CompilerTextures)
		.ReferencedRuntimeMeshes(RuntimeMeshes)
		.ReferencedCompileMeshes(CompilerMeshes)
	];
	
	// Spawn the debugger tab alongside the Graph Tab 
	TabManager->InsertNewDocumentTab(GraphTabId, FTabManager::ESearchPreference::PreferLiveTab, NewMutableObjectTab.ToSharedRef());
}


void FCustomizableObjectEditor::CodeViewer() const
{
	TSharedPtr<UE::Mutable::Private::FModel> Model;
	TArray<TSoftObjectPtr<const UTexture>> RuntimeTextures;
	TArray<FMutableSourceTextureData> CompilerTextures;
	TArray<TSoftObjectPtr<const UStreamableRenderAsset>> RuntimeMeshes;
	TArray<FMutableSourceMeshData> CompilerMeshes;

	DebuggerCompile(Model, RuntimeTextures, CompilerTextures, RuntimeMeshes, CompilerMeshes);
	
	if (!Model)
	{
		FNotificationInfo Info(LOCTEXT("CodeViewerError", "Unable to open Code Viewer: No model"));
		FSlateNotificationManager::Get().AddNotification(Info);

		return;
	}
	
	TSharedPtr<FUnrealMutableResourceProvider> ExternalResourceProvider = MakeShared<FUnrealMutableResourceProvider>();
	
	const TSharedPtr<SDockTab> NewMutableObjectTab = SNew(SDockTab)
	.Label(LOCTEXT("CodeViewer", "Code Viewer"))
	[
		SNew(SMutableCodeViewer, *CustomizableObject, *PreviewInstance, Model)
		.ExternalResourceProvider(ExternalResourceProvider)
	];
	
	// Spawn the debugger tab alongside the Graph Tab 
	TabManager->InsertNewDocumentTab(GraphTabId, FTabManager::ESearchPreference::PreferLiveTab, NewMutableObjectTab.ToSharedRef());
}


void FCustomizableObjectEditor::UpdateCookDataDistributionId()
{
	CustomizableObject->GetPrivate()->UpdateDataDistributionId();
	CustomizableObject->Modify();
}


void FCustomizableObjectEditor::ClearGatheredReferences()
{
	CustomizableObject->GetPrivate()->ReferencedObjects = {};
	CustomizableObject->Modify();
}


void FCustomizableObjectEditor::ResetCompileOptions()
{
	const FScopedTransaction Transaction(LOCTEXT("ResetCompilationOptionsTransaction", "Reset Compilation Options"));
	CustomizableObject->Modify();
	
	UCustomizableObjectPrivate* DefaultObject = Cast<UCustomizableObjectPrivate>(CustomizableObject->GetPrivate()->StaticClass()->GetDefaultObject());
	CustomizableObject->GetPrivate()->OptimizationLevel = DefaultObject->OptimizationLevel;
	CustomizableObject->GetPrivate()->TextureCompression = DefaultObject->TextureCompression;
	CustomizableObject->GetPrivate()->EmbeddedDataBytesLimit = DefaultObject->EmbeddedDataBytesLimit;
}

void FCustomizableObjectEditor::OnChangeCompileOptimizationLevel(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	const FScopedTransaction Transaction(LOCTEXT("ChangedOptimizationLevelTransaction", "Changed Optimization Level"));
	CustomizableObject->Modify();
	CustomizableObject->GetPrivate()->OptimizationLevel = CompileOptimizationStrings.Find(NewSelection);
}


void FCustomizableObjectEditor::OnChangeCompileTextureCompressionType(TSharedPtr<FString> NewSelection, ESelectInfo::Type)
{
	const FScopedTransaction Transaction(LOCTEXT("ChangedTextureCompressionTransaction", "Changed Texture Compression Type"));
	CustomizableObject->Modify();
	CustomizableObject->GetPrivate()->TextureCompression = ECustomizableObjectTextureCompression(CompileTextureCompressionStrings.Find(NewSelection));
}


void FCustomizableObjectEditor::SaveAsset_Execute()
{
	UPackage* Package = CustomizableObject->GetOutermost();

	if (Package)
	{
		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(Package);

		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
	}
}


void FCustomizableObjectEditor::OnSelectedGraphNodesChanged(const FGraphPanelSelectionSet& NewSelection)
{
	TArray<UObject*> Objects;
	for (FGraphPanelSelectionSet::TConstIterator It(NewSelection); It; ++It)
	{
		Objects.Add(*It);
	}

	// If the previously active node was one with a preview mesh (Clip Skeletal mesh with Skeletal Mesh) then clear its delegate
	if (PreviouslySelectedNodeWithPreview)
	{
		check(PreviewMeshChangedDelegateHandle.IsValid());
		PreviouslySelectedNodeWithPreview->PreviewMeshChangedDelegate.Remove(PreviewMeshChangedDelegateHandle);
		
		PreviouslySelectedNodeWithPreview = nullptr;
	}
	
	// Standard details
	if (ObjectDetailsView.IsValid())
	{
		if (Objects.Num())
		{
			ObjectDetailsView->SetObjects(Objects);
		}
		else
		{
			ObjectDetailsView->SetObject(CustomizableObject);
		}
	}		
	
	if (!bRecursionGuard) // Calling the following functions will unselect some nodes causing OnSelectedGraphNodesChanged to be called again
	{
		TGuardValue<bool> RecursionGuard(bRecursionGuard, true);

		if (Objects.Num() != 1)
		{
			HideGizmoClipMorph();
			HideGizmoClipMesh();
			HideGizmoProjectorNodeProjectorConstant();
			HideGizmoProjectorNodeProjectorParameter();

			for (UObject* Object : Objects) // Reselect the multiple selection. Clearly showing gizmos when selecting a node is a really bad idea. Remove on MTBL-1684
			{
				GraphEditor->SetNodeSelection(Cast<UEdGraphNode>(Object), true);
			}
			
			return;
		}

		if (UCustomizableObjectNodeModifierClipMorph* NodeModifierClipMorph = Cast<UCustomizableObjectNodeModifierClipMorph>(Objects[0]))
		{		
			ShowGizmoClipMorph(*NodeModifierClipMorph);
		}
		else if (UCustomizableObjectNodeModifierClipWithMesh* NodeModifierClipWithMesh = Cast<UCustomizableObjectNodeModifierClipWithMesh>(Objects[0]))
		{
			if (const UEdGraphPin* Pin = NodeModifierClipWithMesh->GetClipMeshPin())
			{
				ShowGizmoClipMesh(*NodeModifierClipWithMesh, &NodeModifierClipWithMesh->Transform, *Pin);
			}
		}
		else if (UCONodeClipMeshWithMesh* NodeClipMeshWithMesh = Cast<UCONodeClipMeshWithMesh>(Objects[0]))
		{
			if (const UEdGraphPin* CLipMeshPin = NodeClipMeshWithMesh->ClipMeshPin.Get())
			{
				ShowGizmoClipMesh(*NodeClipMeshWithMesh, &NodeClipMeshWithMesh->Transform, *CLipMeshPin);
			}
		}
		else if (UCONodeClipSkeletalMeshWithSkeletalMesh* NodeClipSkeletalMeshWithSkeletalMesh = Cast<UCONodeClipSkeletalMeshWithSkeletalMesh>(Objects[0]))
		{
			if (const UEdGraphPin* CLipSkeletalMeshPin = NodeClipSkeletalMeshWithSkeletalMesh->ClipSkeletalMeshPin.Get())
			{
				ShowGizmoClipMesh(*NodeClipSkeletalMeshWithSkeletalMesh, &NodeClipSkeletalMeshWithSkeletalMesh->Transform, *CLipSkeletalMeshPin);
				
				// Handle changes in the preview mesh or the preview mesh settings
				PreviewMeshChangedDelegateHandle = NodeClipSkeletalMeshWithSkeletalMesh->PreviewMeshChangedDelegate.AddSP(this,
					&FCustomizableObjectEditor::ShowGizmoClipMesh);
				
				// Keep a strong reference to this object so we ensure it is valid at least until we select another node or we close the editor.
				// If we do not do this it may happen that the node gets removed 
				PreviouslySelectedNodeWithPreview = TStrongObjectPtr<UCONodeClipSkeletalMeshWithSkeletalMesh>(NodeClipSkeletalMeshWithSkeletalMesh);
			}
		}
		else if (UCustomizableObjectNodeModifierTransformInMesh* NodeModifierTransformInMesh = Cast<UCustomizableObjectNodeModifierTransformInMesh>(Objects[0]))
		{
			if (const UEdGraphPin* Pin = NodeModifierTransformInMesh->BoundingMeshPin.Get())
			{
				ShowGizmoClipMesh(*NodeModifierTransformInMesh, &NodeModifierTransformInMesh->BoundingMeshTransform, *Pin);
			}
		}
		else if (UCONodeTransformInMesh* NodeTransformInMesh = Cast<UCONodeTransformInMesh>(Objects[0]))
		{
			if (const UEdGraphPin* Pin = NodeTransformInMesh->BoundingMeshPin.Get())
			{
				ShowGizmoClipMesh(*NodeTransformInMesh, &NodeTransformInMesh->BoundingMeshTransform, *Pin);
			}
		}
		else if (UCustomizableObjectNodeProjectorConstant* NodeProjectorConstant = Cast<UCustomizableObjectNodeProjectorConstant>(Objects[0]))
		{
			ShowGizmoProjectorNodeProjectorConstant(*NodeProjectorConstant);
		}
		else if (UCustomizableObjectNodeProjectorParameter* NodeProjectorParameter = Cast<UCustomizableObjectNodeProjectorParameter>(Objects[0]))
		{
			ShowGizmoProjectorNodeProjectorParameter(*NodeProjectorParameter);		
		}
		else
		{
			HideGizmoClipMorph();	
			HideGizmoClipMesh();
			HideGizmoProjectorNodeProjectorParameter();
			HideGizmoProjectorNodeProjectorConstant();
		}
	}
}


void FCustomizableObjectEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged  )
{
	// Is it a source graph node?
	const UObject* OuterObject = PropertyThatChanged->GetOwner<UObject>();
	const UClass* OuterClass = Cast<UClass>(OuterObject);
	if (OuterClass && OuterClass->IsChildOf(UCustomizableObjectNode::StaticClass()))
	{
		FPropertyChangedEvent Event(PropertyThatChanged);
		CustomizableObject->GetPrivate()->GetSource()->PostEditChangeProperty(Event);
		CustomizableObject->PostEditChangeProperty(Event);

		if (GraphEditor.IsValid())
		{
			GraphEditor->NotifyGraphChanged();
		}
	}
}


TSharedPtr<SCustomizableObjectEditorAdvancedPreviewSettings> FCustomizableObjectEditor::GetAdvancedPreviewSettings()
{
	return CustomizableObjectEditorAdvancedPreviewSettings;
}


bool FCustomizableObjectEditor::ShowLightingSettings()
{
	return false;
}


bool FCustomizableObjectEditor::ShowProfileManagementOptions()
{
	return true;
}


const UObject* FCustomizableObjectEditor::GetObjectBeingEdited()
{
	check(GetObjectsCurrentlyBeingEdited()->Num());

	return (*GetObjectsCurrentlyBeingEdited())[0];
}

FEditorModeTools* FCustomizableObjectEditor::GetEditorModeTools()
{
	// EditorModeManager may not be created yet during early widget construction
	// (viewport is built in InitCustomizableObjectEditor before InitAssetEditor calls CreateEditorModeManager)
	return EditorModeManager.IsValid() ? EditorModeManager.Get() : nullptr;
}

void FCustomizableObjectEditor::OnPostCompile()
{
	Viewport->CreatePreviewActor(PreviewInstance);
	PreviewInstance->UpdateSkeletalMeshAsync(true, true);
}


void FCustomizableObjectEditor::OnUpdatePreviewInstance(UCustomizableObjectInstance* Instance)
{
	if (TextureAnalyzer.IsValid())
	{
		TextureAnalyzer->RefreshTextureAnalyzerTable(PreviewInstance);
	}
}


void FCustomizableObjectEditor::OpenTextureAnalyzerTab()
{
	TabManager->TryInvokeTab(TextureAnalyzerTabId);
}


void FCustomizableObjectEditor::OpenPerformanceAnalyzerTab()
{
	TabManager->TryInvokeTab(PerformanceAnalyzerTabId);
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_TextureAnalyzer(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TextureAnalyzerTabId);
	
	return SNew(SDockTab)
	.Label(LOCTEXT("Texture Analyzer", "Texture Analyzer"))
	[
		TextureAnalyzer.ToSharedRef()
	];
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_PerformanceAnalyzer(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PerformanceAnalyzerTabId);
	check(CustomizableObject);

	if (!PerformanceAnalyzer.IsValid())
	{
		PerformanceAnalyzer = SNew(SCustomizableObjectEditorPerformanceAnalyzer).CustomizableObject(CustomizableObject);
	}

	return SNew(SDockTab)
	.Label(LOCTEXT("Performance Analyzer", "Performance Analyzer"))
	[
		PerformanceAnalyzer.ToSharedRef()
	];
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_TagExplorer(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TagExplorerTabId);

	return SNew(SDockTab)
	.Label(LOCTEXT("Tag_Explorer", "Tag Explorer"))
	[
		TagExplorer.ToSharedRef()
	];
}


TSharedRef<SDockTab> FCustomizableObjectEditor::SpawnTab_Stats(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == StatsTabId);

	return SNew(SDockTab)
	.Label(LOCTEXT("StatsTabLabel", "Stats"))
	[
		StatsWidget.ToSharedRef()
	];
}


UCustomizableObject* FCustomizableObjectEditor::GetAbsoluteCOParent(const UCustomizableObjectNodeObject* const Root)
{
	if (Root->ParentObject != nullptr)
	{
		//Get all the NodeObjects
		TArray<UCustomizableObjectNodeObject*> ObjectNodes;
		Root->ParentObject->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeObject>(ObjectNodes);
		if (!ObjectNodes.IsEmpty())
		{
			//Getting the parent of the root
			UCustomizableObjectNodeObject* FirstObjectNode = ObjectNodes[0];
			if (FirstObjectNode->ParentObject == nullptr)
			{
				return Root->ParentObject;
			}

			return GetAbsoluteCOParent(FirstObjectNode);
		}
	}

	return nullptr;
}


void FCustomizableObjectEditor::OnCustomizableObjectStatusChanged(FCustomizableObjectStatus::EState PreviousState, const FCustomizableObjectStatus::EState CurrentState)
{
	if (PreviousState == FCustomizableObjectStatusTypes::EState::Loading)
	{
		if (CurrentState == FCustomizableObjectStatusTypes::EState::ModelLoaded)
		{
			Viewport->CreatePreviewActor(PreviewInstance);
			PreviewInstance->UpdateSkeletalMeshAsync(true, true);
		}
		else if (CurrentState == FCustomizableObjectStatusTypes::EState::NoModel)
		{
			TSharedRef<FCompilationRequest> CompileRequest = MakeShared<FCompilationRequest>(*CustomizableObject);
			CompileRequest->bSkipIfNotOutOfDate = true;
			CompileRequest->bSilentCompilation = false;
			CompileRequest->SetDerivedDataCachePolicy(GetDerivedDataCachePolicyForEditor());

			ICustomizableObjectEditorModulePrivate::GetChecked().EnqueueCompileRequest(CompileRequest);
		}
	}
}


void FCustomizableObjectEditor::OnChangeDebugPlatform(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	DebugTargetPlatformName = NewSelection;
}


void FCustomizableObjectEditor::InitLogList()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	TSharedPtr<FEditorLogger> EditorLogger;

	if (!CustomizableObject)
	{
		return;
	}

	if (!CustomizableObject->GetPrivate()->CompilationLogger)
	{
		FMessageLogInitializationOptions LogOptions;
		LogOptions.bShowPages = false;
		LogOptions.bShowFilters = false;
		LogOptions.bAllowClear = false;
		LogOptions.MaxPageCount = 1;

		EditorLogger = MakeShared<FEditorLogger>();
		EditorLogger->MessageLogListing = MessageLogModule.CreateLogListing("MutableEditor", LogOptions);

		CustomizableObject->GetPrivate()->CompilationLogger = EditorLogger;
	}
	else
	{
		EditorLogger = StaticCastSharedPtr<FEditorLogger>(CustomizableObject->GetPrivate()->CompilationLogger);
	}

	// Set the CO's LogListing as the point to get the messages of the States Tab
	StatsWidget = MessageLogModule.CreateLogListingWidget(EditorLogger->MessageLogListing.ToSharedRef());
}


// ---- FEditorLogger class ---

bool FEditorLogger::LogMessage(TSharedRef<FTokenizedMessage> Message)
{
	if (MessageLogListing)
	{
		MessageLogListing->AddMessage(Message);
		
		return true;
	}

	return false;
}


void FEditorLogger::ClearLogMessageList()
{
	if (MessageLogListing)
	{
		MessageLogListing->ClearMessages();
	}
}

#undef LOCTEXT_NAMESPACE
