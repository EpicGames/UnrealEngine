// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectInstanceEditor.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "GraphEditor.h"
#include "Misc/NotifyHook.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuR/MutableEditorLogger.h"
#include "MuT/Node.h"
#include "Widgets/Input/SNumericDropDown.h"

#include "CustomizableObjectEditor.generated.h"

class UCONodeClipSkeletalMeshWithSkeletalMesh;
struct FMutableSourceTextureData;
struct FMutableSourceMeshData;
enum class ECustomizableObjectProjectorType : uint8;
namespace ESelectInfo { enum Type : int; }

class SCustomizableObjectEditorPerformanceAnalyzer;
class FCustomizableObjectEditorViewportClient;
class FProperty;
class FSpawnTabArgs;
class FUICommandList;
class IMessageLogListing;
class IToolkitHost;
class SCustomizableObjectEditorViewportTabBody;
class SDockTab;
class STextComboBox;
class SWidget;
class UCustomizableObject;
class UCustomizableObjectInstance;
class UCustomizableObjectNode;
class UCustomizableObjectNodeModifierClipMorph;
class UCustomizableObjectNodeModifierClipWithMesh;
class UCustomizableObjectNodeObject;
class UCustomizableObjectNodeProjectorConstant;
class UCustomizableObjectNodeProjectorParameter;
class UCustomizableSkeletalComponent;
class UDebugSkelMeshComponent;
class UEdGraph;
class UPoseAsset;
class UStaticMeshComponent;
struct FAssetData;
struct FFrame;
struct FPropertyChangedEvent;
template <typename FuncType> class TFunction;

DECLARE_DELEGATE(FCreatePreviewInstanceFlagDelegate);


class FEditorLogger : public IMutableEditorLogger
{
public:

	/** Logs message into the Message Log Listing */
	bool LogMessage(TSharedRef<FTokenizedMessage> Message) override;

	/** Claer all the messages of the Message Log Listing */
	void ClearLogMessageList() override;

public:

	TSharedPtr<IMessageLogListing> MessageLogListing;
};


enum class EGizmoType : uint8
{
	Hidden,
	ProjectorParameter,
	NodeProjectorConstant,
	NodeProjectorParameter,
	ClipMorph,
	ClipMesh,
	Light,
};


/**
* Wrapper UObject class for the UCustomizableObjectInstance::FObjectInstanceUpdatedDelegate dynamic multicast delegate
*/
UCLASS()
class UUpdateClassWrapper : public UObject
{
public:
	GENERATED_BODY()

	/** Method to assign for the callback */
	UFUNCTION()
	void DelegatedCallback(UCustomizableObjectInstance* Instance);

	FCreatePreviewInstanceFlagDelegate Delegate;
};


/**
 * CustomizableObject Editor class
 */
class FCustomizableObjectEditor :
	public FCustomizableObjectGraphEditorToolkit,
	public ICustomizableObjectInstanceEditor,
	public FGCObject
{
public:
	explicit FCustomizableObjectEditor(UCustomizableObject& ObjectToEdit);

	virtual ~FCustomizableObjectEditor() override;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Initialize a new Customizable Object editor. Called immediately after construction.
	 * Required due to being unable to use SharedThis in the constructor.
	 *
	 * See static Create(...) function.
	 */
	void InitCustomizableObjectEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost);

	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override; 
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCustomizableObjectEditor");
	}

	// IToolkit interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// ICustomizableObjectInstanceEditor interface
	virtual void RefreshTool() override;
	virtual TSharedPtr<SCustomizableObjectEditorViewportTabBody> GetViewport() override;
	virtual UCustomizableObjectInstance* GetPreviewInstance() override;
	virtual UProjectorParameter* GetProjectorParameter() override;
	virtual UCustomSettings* GetCustomSettings() override;
	virtual void HideGizmo() override;
	virtual void ShowGizmoProjectorNodeProjectorConstant(UCustomizableObjectNodeProjectorConstant& Node) override;
	virtual void HideGizmoProjectorNodeProjectorConstant() override;
	virtual void ShowGizmoProjectorNodeProjectorParameter(UCustomizableObjectNodeProjectorParameter& Node) override;
	virtual void HideGizmoProjectorNodeProjectorParameter() override;
	virtual void ShowGizmoProjectorParameter(const FString& ParamName, int32 RangeIndex = -1) override;
	virtual void HideGizmoProjectorParameter() override;
	virtual void ShowGizmoClipMorph(UCustomizableObjectNodeModifierClipMorph& Node) override;
	virtual void HideGizmoClipMorph(bool bClearGraphSelectionSet = true) override;
	virtual void ShowGizmoClipMesh(UCustomizableObjectNode& Node, FTransform* Transform, const UEdGraphPin& MeshPin) override;
	virtual void HideGizmoClipMesh() override;
	virtual void ShowGizmoLight(ULightComponent& SelectedLight) override;
	virtual void HideGizmoLight() override;
	virtual UCustomizableObjectEditorProperties* GetEditorProperties() override;
	virtual TSharedPtr< SCustomizableObjectEditorAdvancedPreviewSettings> GetAdvancedPreviewSettings() override;
	virtual bool ShowLightingSettings() override;
	virtual bool ShowProfileManagementOptions() override;
	virtual const UObject* GetObjectBeingEdited() override;
	virtual FEditorModeTools* GetEditorModeTools() override;

	/** Returns the CO being edited in this editor */
	UCustomizableObject* GetCustomizableObject();

	/** Utility method: Test whether the CO Node Object given as parameter is linked to any of the CO Node Object Group nodes
	* in the Test CO given as parameter */
	static bool GroupNodeIsLinkedToParentByName(UCustomizableObjectNodeObject* Node, UCustomizableObject* Test, const FString& ParentGroupName);

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;

	// FNotifyHook interface
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;

	// Delegates
	void DeleteSelectedNodes();
	bool CanDeleteNodes() const;
	void DuplicateSelectedNodes();
	bool CanDuplicateSelectedNodes() const;

	// FCustomizableObjectGraphEditorToolkit interface
	virtual void OnSelectedGraphNodesChanged(const FGraphPanelSelectionSet& NewSelection) override;
	virtual void ReconstructAllChildNodes(UCustomizableObjectNode& StartNode, const UClass& NodeType) override;

	/** Callback to notify the editor when the PreviewInstance has been updated */
	void OnUpdatePreviewInstance(UCustomizableObjectInstance* Instance);

	UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> DebuggerCompile(TSharedPtr<UE::Mutable::Private::FModel>& Model,
		TArray<TSoftObjectPtr<const UTexture>>& RuntimeTextures,
		TArray<FMutableSourceTextureData>& CompilerTextures,
		TArray<TSoftObjectPtr<const UStreamableRenderAsset>>& RuntimeMeshes,
		TArray<FMutableSourceMeshData>& CompilerMeshes) const;
	
	void GraphViewer() const;
	void CodeViewer() const;

	/** Regenerate CookDataDistributionId to generate a new entry of cooked data distribution in the DDC next time the object is cooked. */
	void UpdateCookDataDistributionId();

	/** Clear game asset references saved in the Customizable Object. */
	void ClearGatheredReferences();

	void InitLogList();

private:
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_InstanceProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Graph(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AdvancedPreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_TextureAnalyzer(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PerformanceAnalyzer(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_TagExplorer(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Stats(const FSpawnTabArgs& Args);
	
	/** Binds commands associated with the Static Mesh Editor. */
	void BindCommands();
	
	/** Command list for the graph editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	/** Compile the Customizable Object.
	 * 
	 * @param bOnlySelectedParameters If true, compile only selected int parameters.
	 * @param bGatherReferences If true, also gather asset references and save them in the Customizable Object. Marks the objects as modified. */
	void CompileObject(bool bOnlySelectedParameters, bool bGatherReferences);
	
	// Compile options menu callbacks
	TSharedRef<SWidget> GenerateCompileOptionsMenuContent(TSharedRef<FUICommandList> InCommandList);
	void ResetCompileOptions();
	TSharedPtr<STextComboBox> CompileOptimizationCombo;
	TArray< TSharedPtr<FString> > CompileOptimizationStrings;
	TSharedPtr<STextComboBox> CompileTextureCompressionCombo;
	TArray< TSharedPtr<FString> > CompileTextureCompressionStrings;
	TSharedPtr<SNumericDropDown<float>> EmbeddedDataLimitCombo;

	void OnChangeCompileOptimizationLevel(TSharedPtr<FString> NewSelection, ESelectInfo::Type);
	void OnChangeCompileTextureCompressionType(TSharedPtr<FString> NewSelection, ESelectInfo::Type);

	/** Save Customizable Object open in editor */
	void SaveAsset_Execute() override;

	/** Callback when selection changes in the Property Tree. */
	void OnObjectPropertySelectionChanged(FProperty* InProperty);

	/** Callback when selection changes in the Property Tree. */
	void OnInstancePropertySelectionChanged(FProperty* InProperty);

	/** Callback for the object modified event */
	void OnObjectModified(UObject* Object);

	/** Open the Texture Analyzer tab */
	void OpenTextureAnalyzerTab();

	/** Open the Performance Analyzer tab */
	void OpenPerformanceAnalyzerTab();

	void OnPostCompile();

	void OnChangeDebugPlatform(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	
public:
	void OnCustomizableObjectStatusChanged(FCustomizableObjectStatus::EState PreviousState, FCustomizableObjectStatus::EState CurrentState);

	// Helpers to get the absolute parent of a Customizable Object
	static UCustomizableObject* GetAbsoluteCOParent(const UCustomizableObjectNodeObject* const Root);

	/**	The tab ids for all the tabs used */
	static const FName ViewportTabId;
	static const FName DetailsTabId;
	static const FName InstancePropertiesTabId;
	static const FName GraphTabId;
	static const FName SystemPropertiesTabId;
	static const FName AdvancedPreviewSettingsTabId;
	static const FName TextureAnalyzerTabId;
	static const FName PerformanceAnalyzerTabId;
	static const FName TagExplorerTabId;
	static const FName ObjectDebuggerTabId;
	static const FName PopulationClassTagManagerTabId;
	static const FName StatsTabId;

private:

	/** The currently viewed object. */
	TObjectPtr<UCustomizableObject> CustomizableObject = nullptr;
	TObjectPtr<UCustomizableObjectInstance> PreviewInstance = nullptr;

	/** List of open tool panels; used to ensure only one exists at any one time */
	TMap< FName, TWeakPtr<class SDockableTab> > SpawnedToolPanels;

	/** Preview Viewport widget */
	TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport;
	TSharedPtr<FCustomizableObjectEditorViewportClient> ViewportClient;

	TSharedPtr<IDetailsView> CustomizableInstanceDetailsView;


	/** Property View for Customizable Object and Nodes. */
	TSharedPtr<class IDetailsView> ObjectDetailsView;

	/** Widget to select which node pins are visible. */
	TSharedPtr<class SCustomizableObjectNodePinViewer> NodePinViewer;
	
	/** UObject class to be able to use the update callback */
	TObjectPtr<UUpdateClassWrapper> HelperCallback;
	
	/** Scene preview settings widget, upcast of CustomizableObjectEditorAdvancedPreviewSettings */
	TSharedPtr<SWidget> AdvancedPreviewSettingsWidget;

	/** Scene preview settings widget */
	TSharedPtr<class SCustomizableObjectEditorAdvancedPreviewSettings> CustomizableObjectEditorAdvancedPreviewSettings;
	
	/** Texture Analyzer table widget which shows the information of the transient textures used in the customizable object instance */
	TSharedPtr<class SCustomizableObjecEditorTextureAnalyzer> TextureAnalyzer;

	/** New performance analyzer widget */
	TSharedPtr<SCustomizableObjectEditorPerformanceAnalyzer> PerformanceAnalyzer;
	
	/** Widget to explore all the tags related with the Customizable Object open in the editor */
	TSharedPtr<class SCustomizableObjectEditorTagExplorer> TagExplorer;

	/** Adds the customizable Object Editor commands to the default toolbar */
	void ExtendToolbar();

	/** URL to open when pressing the documentation button generated by UE */
	const FString DocumentationURL{ TEXT("https://github.com/anticto/Mutable-Documentation/wiki") };
	
	TObjectPtr<UProjectorParameter> ProjectorParameter = nullptr;

	TObjectPtr<UCustomSettings> CustomSettings = nullptr;

	TObjectPtr<UCustomizableObjectEditorProperties> EditorProperties = nullptr;
	
	bool bRecursionGuard = false;
	
	EGizmoType GizmoType = EGizmoType::Hidden;

	TArray<TSharedPtr<FString>> TargetPlatformNames;
	TSharedPtr<FString> DebugTargetPlatformName;
	
	bool bDebugForceLargeLODBias = false;
	
	TSharedPtr<SWidget> StatsWidget;

	/** Pointer to the CLip Skeletal Mesh with Skeletal Mesh node that was being shown in the CO Editor*/
	TStrongObjectPtr<UCONodeClipSkeletalMeshWithSkeletalMesh> PreviouslySelectedNodeWithPreview = nullptr;
	
	/** Handle for the callback performed when the PreviewMesh of the PreviouslySelectedNodeWithPreview is changed (or any of its related properties) */
	FDelegateHandle PreviewMeshChangedDelegateHandle;
	
protected:
	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override;
};
