// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorActions.h"

#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/SCustomizableObjectEditorViewport.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FCustomizableObjectEditorCommands::FCustomizableObjectEditorCommands() 
	: TCommands<FCustomizableObjectEditorCommands>
(
	"CustomizableObjectEditor", // Context name for fast lookup
	NSLOCTEXT("Contexts", "CustomizableObjectEditor", "CustomizableObject Editor"), // Localized context name for displaying
	NAME_None, // Parent
	FCustomizableObjectEditorStyle::GetStyleSetName()
	)
{
}


void FCustomizableObjectEditorCommands::RegisterCommands()
{
	UI_COMMAND(Compile, "Compile", "Compile the source graph of the customizable object and update the previews. \nActive if the CVar Mutable.Enabled is set to true.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CompileOnlySelected, "Compile Only Selected", "Compile the source graph of the customizable object and update the previews, only for the selected options in the preview. The rest of options are discarded. If they are selected, press again this button to see their effect in the preview. \nActive if the CVar Mutable.Enabled is set to true.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetCompileOptions, "Reset Compilation Options", "Set reasonable defaults for the compilation options.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GraphViewer, "Graph Viewer", "Open the Customizable Object Graph Viewer.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CodeViewer, "Code Viewer", "Open the Customizable Object Code Viewer.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UpdateCookDataDistributionId, "Regenerate Cooked Data Distribution", "Optimize cooked data distribution based on current graph and generated data. Data distribution is stored in the DDC. \n\nWarning! Changing the data distribution may result in big patch sizes.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(PerformanceAnalyzer, "Performance Analyzer", "Open the Performance Analyzer window to set up and perform all tests relevant to Customizable Objects.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TextureAnalyzer, "Texture Memory Analyzer", "Open the Texture Analyzer window to check all the information of the textures created by Mutable.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CompileGatherReferences, "Compile and Gather References", "Compile and gather all asset references used in this Customizable Object. Marks the object as modified.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearGatheredReferences, "Clear References", "Clear gathered asset references. Marks the object as modified.", EUserInterfaceActionType::Button, FInputChord());
}


FCustomizableObjectEditorViewportCommands::FCustomizableObjectEditorViewportCommands() 
	: TCommands<FCustomizableObjectEditorViewportCommands>
(
	"CustomizableObjectEditorViewport", // Context name for fast lookup
	NSLOCTEXT("Contexts", "CustomizableObjectEditorViewport", "CustomizableObject Editor Viewport"), // Localized context name for displaying
	NAME_None, // Parent
	FCustomizableObjectEditorStyle::GetStyleSetName()
	)
{
	PlaybackSpeedCommands.AddZeroed(EMutableAnimationPlaybackSpeeds::NumPlaybackSpeeds);
}


void FCustomizableObjectEditorViewportCommands::RegisterCommands()
{
	UI_COMMAND( SetDrawUVs, "UV", "Toggles display of the static mesh's UVs for the specified channel.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( SetShowSky, "Sky", "Displays the viewport sky.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowBounds, "Bounds", "Toggles display of the bounds of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowCollision, "Collision", "Toggles display of the simplified collision mesh of the static mesh, if one has been assigned.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetCameraLock, "Camera Lock", "Toggles viewport navigation between orbit and freely moving about.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SaveThumbnail, "Save Thumbnail", "Saves the viewpoint position and orientation in the Preview Pane for use as the thumbnail preview in the Content Browser.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND(BakeInstance, "Bake Instance", "Create baked unreal resources for the current preview instance.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StateChangeShowData, "Show or hide test results", "Show or hide test results", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(StateChangeShowGeometryData, "Show instance geometry data", "Show instance geometry data", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::OneTenth],	"x0.1", "Set the animation playback speed to a tenth of normal", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::Quarter],		"x0.25", "Set the animation playback speed to a quarter of normal", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::Half],		"x0.5", "Set the animation playback speed to a half of normal", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::ThreeQuarters],		"x0.75", "Set the animation playback speed to three quarters of normal", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::Normal],		"x1.0", "Set the animation playback speed to normal", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::Double],		"x2.0", "Set the animation playback speed to double the speed of normal", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::FiveTimes],	"x5.0", "Set the animation playback speed to five times the normal speed", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::TenTimes],	"x10.0", "Set the animation playback speed to ten times the normal speed", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( PlaybackSpeedCommands[EMutableAnimationPlaybackSpeeds::Custom],	"xCustom", "Set the animation playback speed to assigned custom speed", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND(ShowDisplayInfo, "Mesh Info", "Display mesh info in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EnableClothSimulation, "Enable Cloth Simulation", "Show simulated cloth mesh", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DebugDrawPhysMeshWired, "Physical Mesh (Wireframe)", "Draws the current physical mesh result in wireframe", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowNormals, "Normals", "Toggles display of vertex normals in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowTangents, "Tangents", "Toggles display of vertex tangents in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowBinormals, "Binormals", "Toggles display of vertex binormals (orthogonal vector to normal and tangent) in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
}


#undef LOCTEXT_NAMESPACE

