// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FUICommandInfo;

/**
 * 
 */
class FCustomizableObjectEditorCommands : public TCommands<FCustomizableObjectEditorCommands>
{
public:
	FCustomizableObjectEditorCommands();
	
	TSharedPtr< FUICommandInfo > Compile;
	TSharedPtr< FUICommandInfo > CompileOnlySelected;
	TSharedPtr< FUICommandInfo > ResetCompileOptions;
	TSharedPtr< FUICommandInfo > GraphViewer;
	TSharedPtr< FUICommandInfo > CodeViewer;
	TSharedPtr< FUICommandInfo > UpdateCookDataDistributionId;

	TSharedPtr< FUICommandInfo > PerformanceAnalyzer;
	TSharedPtr< FUICommandInfo > ResetPerformanceReportOptions;
	TSharedPtr< FUICommandInfo > TextureAnalyzer;

	TSharedPtr< FUICommandInfo > CompileGatherReferences;
	TSharedPtr< FUICommandInfo > ClearGatheredReferences;
	
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};


/**
 * 
 */
class FCustomizableObjectEditorViewportCommands : public TCommands<FCustomizableObjectEditorViewportCommands>
{

public:
	FCustomizableObjectEditorViewportCommands();
	
	TSharedPtr<FUICommandInfo> SetDrawUVs;
	TSharedPtr<FUICommandInfo> SetShowGrid;
	TSharedPtr<FUICommandInfo> SetShowSky;
	TSharedPtr<FUICommandInfo> SetShowBounds;
	TSharedPtr<FUICommandInfo> SetShowCollision;
	TSharedPtr<FUICommandInfo> SetCameraLock;
	TSharedPtr<FUICommandInfo> SaveThumbnail;
	TSharedPtr<FUICommandInfo> BakeInstance;
	TSharedPtr<FUICommandInfo> StateChangeShowData;
	TSharedPtr<FUICommandInfo> StateChangeShowGeometryData;
	TSharedPtr<FUICommandInfo> ShowDisplayInfo;
	TSharedPtr<FUICommandInfo> EnableClothSimulation;
	TSharedPtr<FUICommandInfo> DebugDrawPhysMeshWired;
	TSharedPtr<FUICommandInfo> SetShowNormals;
	TSharedPtr<FUICommandInfo> SetShowTangents;
	TSharedPtr<FUICommandInfo> SetShowBinormals;
	
	/** Command list for playback speed, indexed by EPlaybackSpeeds*/
	TArray<TSharedPtr< FUICommandInfo >> PlaybackSpeedCommands;
	
	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
