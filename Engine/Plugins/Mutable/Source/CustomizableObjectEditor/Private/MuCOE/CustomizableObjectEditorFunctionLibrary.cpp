// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorFunctionLibrary.h"

#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "CustomizableObjectEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOE/CompileRequest.h"
#include "MuCOE/CustomizableObjectFactory.h"
#include "UObject/SavePackage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectEditorFunctionLibrary)


ECustomizableObjectCompilationState UCustomizableObjectEditorFunctionLibrary::CompileCustomizableObjectSynchronously(
	UCustomizableObject* CustomizableObject,
	ECustomizableObjectOptimizationLevel InOptimizationLevel,
	ECustomizableObjectTextureCompression InTextureCompression,
	bool bGatherReferences)
{
	if (!CustomizableObject)
	{
		UE_LOGF(LogMutable, Warning, "Compilation Failed: Trying to compile a null CO!");
		return ECustomizableObjectCompilationState::Failed;
	}

	const double StartTime = FPlatformTime::Seconds();

	FCompileParams Params;
	Params.bAsync = false;
	Params.OptimizationLevel = InOptimizationLevel;
	Params.TextureCompression = InTextureCompression;
	Params.bGatherReferences = bGatherReferences;

	bool bCompilationSuccess = false;
	Params.CallbackNative.BindLambda([&bCompilationSuccess](const FCompileCallbackParams& Params)
		{
			bCompilationSuccess = Params.bCompiled;
		});

	CustomizableObject->Compile(Params);
	
	const double CurrentTime = FPlatformTime::Seconds();
	UE_LOGF( LogMutable, Display,
		"Synchronously Compiled %ls %ls in %f seconds",
		*GetPathNameSafe(CustomizableObject), 
		bCompilationSuccess ? TEXT("successfully") : TEXT("unsuccessfully"),
		CurrentTime - StartTime
	);

	if (!CustomizableObject->IsCompiled())
	{
		UE_LOGF(LogMutable, Warning, "CO not marked as compiled");
	}

	return bCompilationSuccess ? ECustomizableObjectCompilationState::Completed : ECustomizableObjectCompilationState::Failed;
}


UCustomizableObject* UCustomizableObjectEditorFunctionLibrary::NewCustomizableObject(const FNewCustomizableObjectParameters& Parameters)
{
	const FString PackageName = Parameters.PackagePath + "/" + Parameters.AssetName;
	if (FindPackage(nullptr, *PackageName))
	{
		UE_LOGF(LogMutable, Error, "Package [%ls] already exists.", *PackageName);
		return nullptr;
	}

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	
	FString OutAssetName;
	FString OutPackageName;
	AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), OutPackageName, OutAssetName);

	if (OutPackageName != PackageName)
	{
		UE_LOGF(LogMutable, Error, "Invalid package name [%ls]. Possible valid package name: [%ls]", *PackageName, *OutPackageName);
		return nullptr;
	}
	
	if (OutAssetName != Parameters.AssetName)
	{
		UE_LOGF(LogMutable, Error, "Invalid asset name [%ls]. Possible valid asset name: [%ls]", *Parameters.AssetName, *OutAssetName);
		return nullptr;
	}
		
	UPackage* Package = CreatePackage(*PackageName);
	if (!Package)
	{
		UE_LOGF(LogMutable, Error, "Could not create package [%ls].", *PackageName);
		return nullptr;
	}
	
	UCustomizableObjectFactory* Factory = NewObject<UCustomizableObjectFactory>();

	const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

	UObject* Object = AssetToolsModule.Get().CreateAsset(Parameters.AssetName, PackagePath, UCustomizableObject::StaticClass(), Factory);
	if (!Object)
	{
		UE_LOGF(LogMutable, Error, "Could not create Asset [%ls].", *Parameters.AssetName);
		return nullptr;
	}

	FSavePackageArgs SavePackageArgs;
	SavePackageArgs.TopLevelFlags = RF_Public | RF_Standalone;
	UPackage::Save(Package, Object, *FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension()), SavePackageArgs);

	FAssetRegistryModule::AssetCreated(Object);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	TArray<UObject*> Objects;
	Objects.Add(Object);
	ContentBrowserModule.Get().SyncBrowserToAssets(Objects);
	
	return CastChecked<UCustomizableObject>(Object);
}
