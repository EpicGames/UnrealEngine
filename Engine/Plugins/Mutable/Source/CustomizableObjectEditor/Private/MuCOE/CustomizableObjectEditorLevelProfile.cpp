// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorLevelProfile.h"

#include "AssetViewerSettings.h"
#include "Components/LightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/SkyLight.h"
#include "Engine/TextureCube.h"
#include "EngineUtils.h"
#include "Misc/MessageDialog.h"
#include "MuCO/LoadUtils.h"

class UClass;


#define LOCTEXT_NAMESPACE "LevelSettingsPreview"


bool LevelProfileManage::ProfileNameExist(const TArray<FPreviewSceneProfile>& ArrayProfile, const FString& Name)
{
	for (int32 i = 0; i < ArrayProfile.Num(); ++i)
	{
		if (Name == ArrayProfile[i].ProfileName)
		{
			return true;
		}
	}

	return false;
}


bool LevelProfileManage::LoadProfileUObjectNames(const FString& AssetPath,
											     TArray<FString>& ArrayDirectionalLight,
											     TArray<FString>& ArrayPostProcess,
											     TArray<FString>& ArraySkyLight)
{
	UWorld* World = UE::Mutable::Private::LoadObject<UWorld>(nullptr, *AssetPath);

	if (World == nullptr)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoValidLevelAsset", "Level Aset not found"));
		return false;
	}

	ArrayDirectionalLight.Reset();
	ArrayPostProcess.Reset();
	ArraySkyLight.Reset();
	
	for (TActorIterator<AActor> AActorItr(World); AActorItr; ++AActorItr)
	{
		UClass* ClassType = AActorItr->GetClass();

		if (ClassType == ADirectionalLight::StaticClass())
		{
			ArrayDirectionalLight.Add(AActorItr->GetName());
		}

		if (ClassType == APostProcessVolume::StaticClass())
		{
			ArrayPostProcess.Add(AActorItr->GetName());
		}

		if (ClassType == ASkyLight::StaticClass())
		{
			ArraySkyLight.Add(AActorItr->GetName());
		}
	}

	if (ArrayDirectionalLight.Num() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoEnoughDirectionaLight", "Level has zero Directional Light Actors"));
		return false;
	}

	if (ArrayPostProcess.Num() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoEnoughPostProcessVolumeLight", "Level has zero Post Process Volume Actors"));
		return false;
	}

	if (ArraySkyLight.Num() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoEnoughSkyLight", "Level has zero Sky Light Actors"));
		return false;
	}

	return true;
}


bool LevelProfileManage::LoadProfileUObjects(const FString& AssetPath,
										     const FString DirectionalLightName,
										     const FString PostProcessVolumeName,
										     const FString SkyLightName,
								             ADirectionalLight*& DirectionalLight,
								             APostProcessVolume*& PostProcessVolume,
								             ASkyLight*& SkyLight)
{
	UWorld* World = UE::Mutable::Private::LoadObject<UWorld>(nullptr, *AssetPath);

	DirectionalLight = nullptr;
	PostProcessVolume = nullptr;
	SkyLight = nullptr;

	if (World == nullptr)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NoValidLevelAsset", "Level Aset not found"));
		return false;
	}

	for (TActorIterator<AActor> AActorItr(World); AActorItr; ++AActorItr)
	{
		UClass* ClassType = AActorItr->GetClass();

		if ((ClassType == ADirectionalLight::StaticClass()) && (AActorItr->GetName() == DirectionalLightName))
		{
			DirectionalLight = Cast<ADirectionalLight>(*AActorItr);
		}

		if ((ClassType == APostProcessVolume::StaticClass()) && (AActorItr->GetName() == PostProcessVolumeName))
		{
			PostProcessVolume = Cast<APostProcessVolume>(*AActorItr);
		}

		if ((ClassType == ASkyLight::StaticClass()) && (AActorItr->GetName() == SkyLightName))
		{
			SkyLight = Cast<ASkyLight>(*AActorItr);
		}
	}

	return ((DirectionalLight != nullptr) && (PostProcessVolume != nullptr) && (SkyLight != nullptr));
}


bool LevelProfileManage::FillPreviewSceneProfile(const FString& ProfileName,
												 const FString& ProfilePath,
												 const FString& DirectionalLightName,
												 const FString& PostProcessVolumeName,
												 const FString& SkyLightName,
												 FPreviewSceneProfile& PreviewSceneProfile)
{
	ADirectionalLight* DirectionalLight = nullptr;
	APostProcessVolume* PostProcessVolume = nullptr;
	ASkyLight* SkyLight = nullptr;

	bool Result = LevelProfileManage::LoadProfileUObjects(ProfilePath,
														DirectionalLightName,
														PostProcessVolumeName,
														SkyLightName,
														DirectionalLight,
														PostProcessVolume,
														SkyLight);

	PreviewSceneProfile.bSharedProfile = false;
	PreviewSceneProfile.bShowFloor = true;
	PreviewSceneProfile.bShowEnvironment = true;
	PreviewSceneProfile.bRotateLightingRig = false;
	PreviewSceneProfile.DirectionalLightIntensity = DirectionalLight->GetBrightness();
	PreviewSceneProfile.DirectionalLightColor = DirectionalLight->GetLightColor();
	PreviewSceneProfile.SkyLightIntensity = SkyLight ? SkyLight->GetLightComponent()->Intensity : 0.0f;
	PreviewSceneProfile.LightingRigRotation = 0.0f;
	PreviewSceneProfile.RotationSpeed = 0.0f;
	PreviewSceneProfile.EnvironmentCubeMap = SkyLight ? SkyLight->GetLightComponent()->Cubemap : nullptr;
	PreviewSceneProfile.EnvironmentCubeMapPath = SkyLight ? SkyLight->GetLightComponent()->Cubemap->GetPathName() : "";
	PreviewSceneProfile.bPostProcessingEnabled = true;
	PreviewSceneProfile.DirectionalLightRotation = DirectionalLight->GetLightComponent()->GetDirection().Rotation(); // FRotator(-40.f, -67.5f, 0.f);
	PreviewSceneProfile.PostProcessingSettings = PostProcessVolume->Settings;
	PreviewSceneProfile.ProfileName = ProfileName;

	return Result;
}


#undef LOCTEXT_NAMESPACE
