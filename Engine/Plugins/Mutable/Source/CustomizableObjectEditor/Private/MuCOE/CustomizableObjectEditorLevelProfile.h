// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"

struct FPreviewSceneProfile;

class LevelProfileManage
{
public:
	/** Verify if given profile name alreday exists */
	static bool ProfileNameExist(const TArray<FPreviewSceneProfile>& ArrayProfile, const FString& Name);

	/** Once selected a ULevel asset, load all possible elements from this level so the user selects which ones to use for the profile */
	static bool LoadProfileUObjectNames(const FString& AssetPath,
									  TArray<FString>& ArrayDirectionalLight,
									  TArray<FString>& ArrayPostProcess,
									  TArray<FString>& ArraySkyLight);

	/** Once selected a ULevel asset, load all elements needed (directional light, post process volume and sky light AActors */
	static bool LoadProfileUObjects(const FString& AssetPath,
								   const FString DirectionalLightName,
								   const FString PostProcessVolumeName,
								   const FString SkyLightName,
								   class ADirectionalLight*& DirectionalLight,
								   class APostProcessVolume*& PostProcessVolume,
								   class ASkyLight*& SkyLight);

	/** Fill the struct parameter PreviewSceneProfile with the information from the profile with name given */
	static bool FillPreviewSceneProfile(const FString& ProfileName,
										const FString& ProfilePath,
										const FString& DirectionalLightName,
										const FString& PostProcessVolumeName,
										const FString& SkyLightName,
										FPreviewSceneProfile& PreviewSceneProfile);
};
