// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorLogger.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibraryEditor.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuR/MutableEditorLogger.h"
#include "Toolkits/ToolkitManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "CustomizableObject"


const FName FCustomizableObjectEditorLogger::LOG_NAME = "Mutable";

const float FCustomizableObjectEditorLogger::NOTIFICATION_DURATION = 14.0f;

/** A Message Log token that links to an element in an graph. */
class FCustomizableObjectToken : public IMessageToken
{
public:
	/** Factory method, tokens can only be constructed as shared refs */
	CUSTOMIZABLEOBJECTEDITOR_API static TSharedRef<IMessageToken> Create(const UEdGraphNode* Node)
	{
		return MakeShareable(new FCustomizableObjectToken(Node));
	}
	
	// IMessageToken
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Text;
	}

	// Own interface
	CUSTOMIZABLEOBJECTEDITOR_API const UEdGraphNode* GetNode() const
	{
		return Cast<UEdGraphNode>(NodeBeingReferenced.Get());
	}

private:
	/** Private constructor */
	explicit FCustomizableObjectToken(const UEdGraphNode* Node) : NodeBeingReferenced(Node)
	{
		CachedText = Node ?
			FText::FromString(Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString().Replace(TEXT("\n"), TEXT(" "))) :
			LOCTEXT("TokenNoNode", "<None>");
	}

	/** An object being referenced by this token, if any */
	FWeakObjectPtr NodeBeingReferenced;
};


void OnMessageLogLinkActivated(const TSharedRef<IMessageToken>& Token)
{
	// Just an object link
	if (Token->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
		
		if (UObjectToken->GetObject().IsValid())
		{
			if (UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(UObjectToken->GetObject().Get()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CustomizableObject);
			}
			else if (UCustomizableObjectMacro* CustomizableObjectMacro = Cast<UCustomizableObjectMacro>(UObjectToken->GetObject().Get()))
			{
				if (UCustomizableObjectMacroLibrary* ParentMacroLibrary = Cast<UCustomizableObjectMacroLibrary>(CustomizableObjectMacro->GetOuter()))
				{
					// Open the editor for the Macro library
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentMacroLibrary);

					// Find the editor we just opened
					IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ParentMacroLibrary, false);

					// Set the Parent Macro to edit
					if (FCustomizableObjectMacroLibraryEditor* Editor = static_cast<FCustomizableObjectMacroLibraryEditor*>(AssetEditor))
					{
						Editor->SetSelectedMacro(CustomizableObjectMacro, true);
					}
				}
			}
		}
	}

	// Fake type, but no possibility for custom types
	else if (Token->GetType() == EMessageToken::Text)
	{
		const TSharedRef<FCustomizableObjectToken> UObjectToken = StaticCastSharedRef<FCustomizableObjectToken>(Token);

		if (UObjectToken->GetNode())
		{
			const UEdGraphNode* Node = UObjectToken->GetNode();
			UObject* Object = Node->GetOuter()->GetOuter();

			// Find it
			if (const UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(Object))
			{
				// Make sure the editor exists for this asset
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CustomizableObject);

				if (const TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(CustomizableObject))
				{
					StaticCastSharedPtr<FCustomizableObjectGraphEditorToolkit>(FoundAssetEditor)->SelectNode(Node);
				}
			}

			else if (UCustomizableObjectMacro* CustomizableObjectMacro = Cast<UCustomizableObjectMacro>(Object))
			{
				if (UCustomizableObjectMacroLibrary* ParentMacroLibrary = Cast<UCustomizableObjectMacroLibrary>(CustomizableObjectMacro->GetOuter()))
				{
					// Open the editor for the Macro library
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentMacroLibrary);

					// Find the editor we just opened
					IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ParentMacroLibrary, false);

					// Set the Parent Macro to edit
					if (FCustomizableObjectMacroLibraryEditor* Editor = static_cast<FCustomizableObjectMacroLibraryEditor*>(AssetEditor))
					{
						Editor->SetSelectedMacro(CustomizableObjectMacro, true);
						Editor->SelectNode(Node);
					}
				}
			}
		}
	}
}


FLogParameters::FLogParameters(FCustomizableObjectEditorLogger& Logger, const FText& Text) :
	Logger(Logger),
	ParamText(Text)
{
}


FLogParameters::FLogParameters(FCustomizableObjectEditorLogger& Logger, FText&& Text) :
	Logger(Logger),
	ParamText(MoveTemp(Text))
{
}


FLogParameters& FLogParameters::SubText(const FText& SubText)
{
	ParamSubText = SubText;
	return *this;
}


FLogParameters& FLogParameters::SubText(FText&& SubText)
{
	ParamSubText = MoveTemp(SubText);
	return *this;
}


FLogParameters& FLogParameters::Category(const ELoggerCategory Category)
{
	ParamCategory = Category;
	return *this;
}


FLogParameters& FLogParameters::Severity(const EMessageSeverity::Type Severity)
{
	ParamSeverity = Severity;
	return *this;
}


FLogParameters& FLogParameters::Context(const TArray<const UObject*>& Context)
{
	ParamContext = Context;
	return *this;
}


FLogParameters& FLogParameters::Context(const UObject& Context)
{
	ParamContext.Add(&Context);
	return *this;
}


FLogParameters& FLogParameters::BaseObject(const bool BaseObject)
{
	bParamBaseObject = BaseObject;
	return *this;
}


FLogParameters& FLogParameters::CustomNotification(const bool CustomNotification)
{
	bParamCustomNotification = CustomNotification;
	return *this;
}


FLogParameters& FLogParameters::Notification(const bool Notification)
{
	bParamNotification = Notification;
	return *this;
}


FLogParameters& FLogParameters::FixNotification(const bool FixNotification)
{
	bParamFixNotification = FixNotification;
	return *this;
}


FLogParameters& FLogParameters::SpamBin(const ELoggerSpamBin SpamBin)
{
	ParamSpamBin = SpamBin;
	return *this;
}


FLogParameters& FLogParameters::CustomizableObject(UCustomizableObject* Object)
{
	ParamCustomizableObject = Object;
	return *this;
}


void FLogParameters::Log()
{
	Logger.Log(*this);
}


FLogParameters FCustomizableObjectEditorLogger::CreateLog(const FText& Text)
{
	FCustomizableObjectEditorLogger& Logger = ICustomizableObjectEditorModule::GetChecked().GetLogger();
	return FLogParameters(Logger, Text);
}

FLogParameters FCustomizableObjectEditorLogger::CreateLog(FText&& Text)
{
	FCustomizableObjectEditorLogger& Logger = ICustomizableObjectEditorModule::GetChecked().GetLogger();
	return FLogParameters(Logger, MoveTemp(Text));
}


void FCustomizableObjectEditorLogger::Log(FLogParameters& LogParameters)
{
	FMessageLog MessageLog(LOG_NAME);
	const TSharedRef<FTokenizedMessage> Message = MessageLog.Message(LogParameters.ParamSeverity);

	const FText MessageText = LogParameters.ParamSubText.IsSet() ?
		FText::Format(LOCTEXT("EditorLoggerSubText", "{0}. {1}"), LogParameters.ParamText, LogParameters.ParamSubText.Get()) :
		LogParameters.ParamText;
	
	Message->AddToken(FTextToken::Create(MessageText));

	for (const UObject* Context : LogParameters.ParamContext)
	{
		if (const UEdGraphNode* Node = Cast<const UEdGraphNode>(Context))
		{
			if (LogParameters.bParamBaseObject)
			{
				Message->AddToken(FTextToken::Create(FText::FromString(TEXT(" "))));
				const UObject* Asset = Node->GetOuter()->GetOuter();
				Message->AddToken(FUObjectToken::Create(Asset)->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&OnMessageLogLinkActivated)));
			}

			Message->AddToken(FTextToken::Create(FText::FromString(TEXT(" (Node "))));
			Message->AddToken(FCustomizableObjectToken::Create(Node)->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&OnMessageLogLinkActivated)));
			Message->AddToken(FTextToken::Create(FText::FromString(TEXT(")"))));			
		}
		else
		{
			Message->AddToken(FTextToken::Create(FText::FromString(TEXT(" "))));
			Message->AddToken(FUObjectToken::Create(Context)->OnMessageTokenActivated(FOnMessageTokenActivated::CreateStatic(&OnMessageLogLinkActivated)));
		}
	}

	Message->SetIndentationLevel(LogParameters.ParamSeverity == EMessageSeverity::Info ? 0 : 1);

	if (UCustomizableObject* CustomizableObject = LogParameters.ParamCustomizableObject.Get())
	{
		if (CustomizableObject->GetPrivate()->CompilationLogger && CustomizableObject->GetPrivate()->CompilationLogger->LogMessage(Message))
		{
			// Since we are compiling from an editor, we do not need to send any popup notification 
			return;
		}
	}

	if (LogParameters.bParamNotification)
	{
		FCategoryData& CategoryData = CategoriesData.FindOrAdd(LogParameters.ParamCategory);

		// Update notification count.
		if (CategoryData.Notification.IsValid())
		{
			CategoryData.NumMessages++;
		}
		else
		{
			CategoryData.NumMessages = 1; // Reset counter since the last notification has expired.
		}
	
		const FText NotificationText = LogParameters.bParamCustomNotification ?
			LogParameters.ParamText :
			FText::Format(LOCTEXT("ThereAreMessages", "There {0}|plural(one=is,other=are) {0} new {0}|plural(one=message,other=messages)"), FText::AsNumber(CategoryData.NumMessages));

		const FSimpleDelegate HyperLinkDelegate = FSimpleDelegate::CreateRaw(this, &FCustomizableObjectEditorLogger::OpenMessageLog);
		
		// Update or throw a new notification.
		if (const TSharedPtr<SNotificationItem> LastNotificationItem = CategoryData.Notification.Pin())
		{
			if (!LogParameters.bParamFixNotification)
			{
				LastNotificationItem->ExpireAndFadeout();
			}
			
			LastNotificationItem->SetText(NotificationText);
			LastNotificationItem->SetSubText(LogParameters.ParamSubText);
		}
		else
		{
			FNotificationInfo NotificationInfo(NotificationText);
			NotificationInfo.bFireAndForget = !LogParameters.bParamFixNotification;
			NotificationInfo.ExpireDuration = NOTIFICATION_DURATION;
			NotificationInfo.bUseThrobber = true;
			NotificationInfo.Hyperlink = HyperLinkDelegate;
			NotificationInfo.HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log"); // Can not be updated once the notification is shown.
			NotificationInfo.SubText = LogParameters.ParamSubText;
		
			CategoryData.Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}


void FCustomizableObjectEditorLogger::DismissNotification(ELoggerCategory Category)
{
	FCustomizableObjectEditorLogger& Logger = ICustomizableObjectEditorModule::GetChecked().GetLogger();
	if (const FCategoryData* Result = Logger.CategoriesData.Find(Category))
	{
		if (const TSharedPtr<SNotificationItem> LastNotification = Result->Notification.Pin())
		{
    		LastNotification->ExpireAndFadeout();		
		}
	}
}


void FCustomizableObjectEditorLogger::OpenMessageLog() const
{
	FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.OpenMessageLog(LOG_NAME);
}


#undef LOCTEXT_NAMESPACE
