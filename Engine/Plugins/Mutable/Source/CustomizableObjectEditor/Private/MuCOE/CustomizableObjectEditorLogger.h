// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/TokenizedMessage.h"
#include "MuCOE/CustomizableObjectEditor.h"

class UCustomizableObjectNode;
class UCustomizableObject;

class FCustomizableObjectEditorLogger;
class FName;
class SNotificationItem;

/** Notification categories. */
enum class ELoggerCategory
{
	General,
	GraphSearch,
	Compilation,
	COInstanceBaking
};


/** Notification spam prevention bins. Each bin will only show a limited amount of messages of its type, after which it will simply state "and X more BINNAME warnings/errors" */
enum class ELoggerSpamBin : uint8
{
	TagsNotFound = 0,
	ShowAll	// Notifications added to this bin will always be shown.
};


/** Log parameters. Holds all parameters required to create a new CustomizableObjectEditorLogger log. */
class FLogParameters
{
	friend class FCustomizableObjectEditorLogger;

public:
	// Optional parameters setters
	FLogParameters& SubText(const FText& SubText);
	FLogParameters& SubText(FText&& SubText);
	
	FLogParameters& Category(ELoggerCategory Category);

	FLogParameters& Severity(EMessageSeverity::Type MessageSeverity);

	FLogParameters& Context(const TArray<const UObject*>& Context);

	FLogParameters& Context(const UObject& Context);

	FLogParameters& BaseObject(bool BaseObject = true);

	FLogParameters& CustomNotification(bool CustomNotification = true);
	
	FLogParameters& Notification(bool Notification = true);

	FLogParameters& FixNotification(bool FixNotification = true);

	FLogParameters& SpamBin(ELoggerSpamBin SpamBin);

	FLogParameters& CustomizableObject(UCustomizableObject* Object = nullptr);
	
	/** Actually display the log. */
	void Log();
	
private:
	/** Constructor. Contains all required parameters. */
	FLogParameters(FCustomizableObjectEditorLogger& Logger, const FText& Text);
	FLogParameters(FCustomizableObjectEditorLogger& Logger, FText&& Text);
	
	FCustomizableObjectEditorLogger& Logger;

	// Required parameters
	const FText ParamText;

	// Optional parameters
	TAttribute<FText> ParamSubText;
	
	ELoggerCategory ParamCategory = ELoggerCategory::General;

	ELoggerSpamBin ParamSpamBin = ELoggerSpamBin::ShowAll;
	
	EMessageSeverity::Type ParamSeverity = EMessageSeverity::Info;
	
	TArray<const UObject*> ParamContext;

	bool bParamBaseObject = false;

	bool bParamCustomNotification = false;
	
	bool bParamNotification = true;

	bool bParamFixNotification = false;

	TWeakObjectPtr<UCustomizableObject> ParamCustomizableObject;
};


/** Customizable Object Editor Module Logger. Common log utilities.
 *
 * Avoids bombarding the user with many notifications. The Logger will only shows a notification per category.
 * Any new notification will override the previous one. */
class FCustomizableObjectEditorLogger
{
public:
	/** Create a new log. */
	static FLogParameters CreateLog(const FText& Text);
	static FLogParameters CreateLog(FText&& Text);

	/** Display the log. Equivalent to FLogParameters::Log(). */
	void Log(FLogParameters& LogParameters);

	/** Dismiss a fixed notification. */
	static void DismissNotification(ELoggerCategory Category);
	
private:
	/** Category data. */
	struct FCategoryData
	{
		/** Pointer to the last notification widget. */
		TWeakPtr<SNotificationItem> Notification;

		/** Number of new messages throw to this category since the notification was shown. */
		uint32 NumMessages = 0;
	};

	/** Data for each category. */
	TMap<ELoggerCategory, FCategoryData> CategoriesData;

	const static FName LOG_NAME;

	const static float NOTIFICATION_DURATION;

	/** Callback. Open the message log window. */
	void OpenMessageLog() const;
};
