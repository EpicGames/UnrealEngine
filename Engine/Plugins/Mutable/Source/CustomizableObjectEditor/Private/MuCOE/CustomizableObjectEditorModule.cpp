// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorModule.h"

#include "Animation/AnimBlueprint.h"
#include "AssetToolsModule.h"
#include "CustomizableObjectProjectSettings.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "MessageLogModule.h"
#include "Misc/MessageDialog.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"
#include "HAL/FileManager.h"
#include "MuCO/CustomizableObjectSystem.h"		// For defines related to memory function replacements.
#include "MuCO/CustomizableObjectSkeletalMesh.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/ICustomizableObjectModule.h"		// For instance editor command utility function
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuCOE/CustomizableInstanceDetails.h"
#include "MuCOE/CustomizableObjectCustomSettingsDetails.h"
#include "MuCOE/CustomizableObjectDetails.h"
#include "MuCOE/CustomizableObjectEditorSettings.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/CustomizableObjectIdentifierCustomization.h"
#include "MuCOE/CustomizableObjectInstanceEditor.h"
#include "MuCOE/CustomizableObjectInstanceFactory.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroDetails.h"
#include "MuCOE/CustomizableObjectVersionBridge.h"
#include "MuCOE/CustomizableObjectNodeObjectGroupDetails.h"
#include "MuCOE/CustomizableObjectSkeletalMeshThumbnailRenderer.h"
#include "MuCOE/Nodes/CONodeMaterialBreak.h"
#include "MuCOE/Nodes/CONodeMaterialModify.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPinDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstanceDetails.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMaterialParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshSectionDetails.h"
#include "MuCOE/Nodes/CONodeModifierEditSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditMeshSectionDetails.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierExtendMeshSectionDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSectionDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocks.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocksDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorphDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithUVMask.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipDeform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeCommon.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeParameterDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorParameterDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameterDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTableDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTexture.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureProject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureTransform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"
#include "MuCOE/Nodes/CustomizableObjectNodeCurve.h"
#include "MuCOE/Widgets/CustomizableObjectVariationCustomization.h"
#include "MuCOE/Widgets/CustomizableObjectNodeTableCompilationFilterEditor.h"
#include "PropertyEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"
#include "Subsystems/PlacementSubsystem.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/CustomizableObjectInstanceBaker.h"
#include "Editor.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCOE/CompileRequest.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"
#include "MuR/External/Operation.h"
#include "Nodes/CONodeExternalOperation.h"
#include "Nodes/CONodeExternalOperationDetails.h"
#include "Nodes/CONodeModifierTransformWithBone.h"
#include "Nodes/CustomizableObjectNodeModifierClipWithMesh.h"
#include "Nodes/CustomizableObjectNodeModifierTransformInMesh.h"
#include "Nodes/CustomizableObjectNodeTextureParameter.h"
#include "Nodes/ExternalTypeCustomization.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Components/SkeletalMeshComponent.h"
#include "Nodes/CONodeSkeletalMeshBreak_V2.h"
#include "Nodes/CONodeSkeletalMeshMake_V2.h"
#include "Nodes/CONodeSkeletalMeshModify.h"
#include "Engine/CompositeDataTable.h"
#include "MuCOE/CompilationOptions.h"
#include "Nodes/CustomizableObjectNodeComponentMesh.h"
#include "Nodes/CONodeExternalTypeParameter.h"
#include "Nodes/CustomizableObjectNodeColorParameter.h"
#include "Nodes/CustomizableObjectNodeEnumParameter.h"
#include "Nodes/CustomizableObjectNodeFloatParameter.h"
#include "Nodes/CustomizableObjectNodeTransformParameter.h"

class AActor;
class FString;
class ICustomizableObjectDebugger;
class ICustomizableObjectEditor;
class ICustomizableObjectInstanceEditor;
class IToolkitHost;
class UObject;

#define LOCTEXT_NAMESPACE "MutableSettings"


const FName CustomizableObjectEditorAppIdentifier = FName(TEXT("CustomizableObjectEditorApp"));
const FName CustomizableObjectInstanceEditorAppIdentifier = FName(TEXT("CustomizableObjectInstanceEditorApp"));
const FName CustomizableObjectDebuggerAppIdentifier = FName(TEXT("CustomizableObjectDebuggerApp"));
const FName CustomizableObjectMacroLibraryEditorAppIdentifier = FName(TEXT("CustomizableObjectMacroLibraryApp"));


/** Max timespan in days before a Saved/MutableStreamedDataEditor file is deleted. */
constexpr int32 MaxAccessTimespan = 30;

TAutoConsoleVariable<bool> CVarMutableDerivedDataCacheUsage(
	TEXT("mutable.DerivedDataCacheUsage"),
	true,
	TEXT("Derived data cache access for compiled data.")
	TEXT("true - Allows DDC access according to the DDC policies specified in the plugin settings."),
	ECVF_Default);

IMPLEMENT_MODULE( FCustomizableObjectEditorModule, CustomizableObjectEditor );

// TODO UE-226453: UPackage and FAssetPackageData originally had an FGuid PackageGuid, but this was changed to
// FIoHash PackageSavedHash for SavePackage determinism. We wrote our ParticipatingObject change detection in terms of
// PackageGuid, and we need to instead save and compare FIoHashes. Change our TMap<FName,FGuid> into
// TMap<FName,FIoHash>. In the meantime, we change the format back to an FGuid by truncating the FIoHash.
static FGuid TruncatePackageSavedHash(const FIoHash& PackageSavedHash)
{
	FGuid Result;
	static_assert(sizeof(Result) < sizeof(PackageSavedHash.GetBytes()), "We copy sizeof(FGuid) bytes from FIoHash.GetBytes");
	FMemory::Memcpy(&Result, PackageSavedHash.GetBytes(), sizeof(Result));
	return Result;
}

void DeleteUnusedMutableStreamedDataEditorFiles()
{
	const FDateTime CurrentTime = FDateTime::Now();

	const FString CompiledDataFolder = GetCompiledDataFolderPath();
	const FString FileExtension = TEXT(".mut");

	TArray<FString> Files;
	IFileManager& FileManager = IFileManager::Get();
	FileManager.FindFiles(Files, *CompiledDataFolder, *FileExtension);
	
	for (const FString& File : Files)
	{
		const FString FullFilePath = CompiledDataFolder + File;
		const FDateTime AccessTimeStamp = FileManager.GetAccessTimeStamp(*(FullFilePath));
		if (AccessTimeStamp == FDateTime::MinValue())
		{
			continue;
		}

		// Delete files that remain unused for more than MaxAccessTimespan
		const FTimespan TimeSpan = CurrentTime - AccessTimeStamp;
		if (TimeSpan.GetDays() > MaxAccessTimespan)
		{
			FileManager.Delete(*FullFilePath);
		}
	}
}

int32 ConvertOptimizationLevel(ECustomizableObjectOptimizationLevel OptimizationLevel)
{
	switch (OptimizationLevel)
	{
	case ECustomizableObjectOptimizationLevel::None:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
	case ECustomizableObjectOptimizationLevel::Minimal:
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return 0;

	case ECustomizableObjectOptimizationLevel::Maximum:
		return UE_MUTABLE_MAX_OPTIMIZATION;
		
	default:
		unimplemented();
		return 0;
	}
}


TMap<FString, FString> GetCompileOnlySelectedParameters(const UCustomizableObjectInstance& Instance)
{
	TMap<FString, FString> Parameters;
	
	const TArray<FCustomizableObjectIntParameterValue>& IntParameters = Instance.GetPrivate()->GetDescriptor().IntParameters;
	Parameters.Reserve(IntParameters.Num());

	for (const FCustomizableObjectIntParameterValue& IntParam : IntParameters)
	{
		Parameters.Add(IntParam.ParameterName, IntParam.ParameterValueName);
	}

	return Parameters;
}


UE::DerivedData::ECachePolicy ConvertDerivedDataCachePolicy(ECustomizableObjectDDCPolicy InPolicy)
{
	switch (InPolicy)
	{
	case ECustomizableObjectDDCPolicy::Default: return UE::DerivedData::ECachePolicy::Default;
	case ECustomizableObjectDDCPolicy::Local: return UE::DerivedData::ECachePolicy::Local;
	default: return UE::DerivedData::ECachePolicy::None;
	}
}


UE::DerivedData::ECachePolicy GetDerivedDataCachePolicyForEditor()
{
	if (UCustomizableObjectSystem::IsCreated() && CVarMutableDerivedDataCacheUsage.GetValueOnAnyThread())
	{
		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
		return ConvertDerivedDataCachePolicy(System->GetPrivate()->EditorSettings.EditorDerivedDataCachePolicy);
	}

	return UE::DerivedData::ECachePolicy::None;
}


void FCustomizableObjectEditorModule::StartupModule()
{
	// Delete unused local compiled data
	DeleteUnusedMutableStreamedDataEditorFiles();
	
	// Property views
	// Nodes
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	RegisterCustomDetails(PropertyModule, UCONodeModifierEditSkeletalMeshSection::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierEditMeshSectionDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeModifierExtendSkeletalMeshSection::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierExtendMeshSectionDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierRemoveMesh::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierRemoveMeshDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierRemoveMeshBlocks::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierMorphMeshSection::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierMorphMeshSectionDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierClipMorph::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierClipMorphDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierClipWithMesh::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierBaseDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierClipWithUVMask::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierBaseDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierClipDeform::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierBaseDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeModifierTransformInMesh::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierBaseDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeModifierTransformWithBone::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeModifierBaseDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeObject::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeObjectDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeObjectGroup::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeObjectGroupDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeProjectorConstant::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeProjectorParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeExternalPin::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeExternalPinDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeSkeletalMeshSection::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeMeshSectionDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeSkeletalMesh::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeSkeletalMeshDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeStaticMesh::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeTable::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeTableDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeSkeletalMeshMake_V2::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeMacroInstance::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeMacroInstanceDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeMaterialBreak::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeExternalOperation::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCONodeExternalOperationDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeSkeletalMeshBreak_V2::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeSkeletalMeshModify::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeComponentSkeletalMesh::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCONodeExternalTypeParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeColorParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeEnumParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeFloatParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeMaterialParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeProjectorParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeProjectorParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeSkeletalMeshParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeSkeletalMeshParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeTextureParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectNodeTransformParameter::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectNodeParameterDetails::MakeInstance));
	
	// Other Objects
	RegisterCustomDetails(PropertyModule, UCustomizableObject::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectInstance::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableInstanceDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomSettings::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectCustomSettingsDetails::MakeInstance));
	RegisterCustomDetails(PropertyModule, UCustomizableObjectMacro::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectMacroDetails::MakeInstance));
	
	// Custom properties
	PropertyModule.RegisterCustomPropertyTypeLayout("CustomizableObjectIdentifier", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCustomizableObjectIdentifierCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(NAME_StrProperty, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCustomizableObjectStateParameterSelector::MakeInstance), MakeShared<FStatePropertyTypeIdentifier>());
	PropertyModule.RegisterCustomPropertyTypeLayout(FCustomizableObjectVariation::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCustomizableObjectVariationCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FCustomizableObjectTextureVariation::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCustomizableObjectVariationCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FTableNodeCompilationFilter::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCustomizableObjectNodeTableCompilationFilterEditor::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("InstancedStruct", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FExternalTypeCustomization::MakeInstance), MakeShared<FExternalTypeTypeIdentifier>());

	PropertyModule.NotifyCustomizationModuleChanged();

	UThumbnailManager::Get().RegisterCustomRenderer(UCustomizableObjectSkeletalMesh::StaticClass(), UCustomizableObjectSkeletalMeshThumbnailRenderer::StaticClass());

	// Register factory
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this,&FCustomizableObjectEditorModule::RegisterFactory);

	// Additional UI style
	FCustomizableObjectEditorStyle::Initialize();

	RegisterSettings();

	// Create the message log category
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(FName("Mutable"), LOCTEXT("MutableLog", "Mutable"));

	CustomizableObjectEditor_ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
	CustomizableObjectEditor_MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);

	LaunchCOIECommand = IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("mutable.OpenCOIE"),
		TEXT("Looks for a Customizable Object Instance within the player pawn and opens its Customizable Object Instance Editor. Specify slot ID to control which component is edited."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&FCustomizableObjectEditorModule::OpenCOIE));
}


void FCustomizableObjectEditorModule::ShutdownModule()
{
	FEditorDelegates::PreBeginPIE.RemoveAll(this);
	
	if( FModuleManager::Get().IsModuleLoaded( "PropertyEditor" ) )
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		// Unregister Property views
		for (const auto& ClassName : RegisteredCustomDetails)
		{
			PropertyModule.UnregisterCustomClassLayout(ClassName);
		}

		// Unregister Custom properties
		PropertyModule.UnregisterCustomPropertyTypeLayout("CustomizableObjectIdentifier");
		PropertyModule.UnregisterCustomPropertyTypeLayout(NAME_StrProperty);

		PropertyModule.NotifyCustomizationModuleChanged();
	}

	CustomizableObjectEditor_ToolBarExtensibilityManager.Reset();
	CustomizableObjectEditor_MenuExtensibilityManager.Reset();

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FCustomizableObjectEditorStyle::Shutdown();
}


FCustomizableObjectEditorLogger& FCustomizableObjectEditorModule::GetLogger()
{
	return Logger;
}


bool FCustomizableObjectEditorModule::HandleSettingsSaved()
{
	UCustomizableObjectEditorSettings* CustomizableObjectSettings = GetMutableDefault<UCustomizableObjectEditorSettings>();

	if (CustomizableObjectSettings != nullptr)
	{
		CustomizableObjectSettings->SaveConfig();
		
		FEditorCompileSettings CompileSettings;
		CompileSettings.bIsMutableEnabled = !CustomizableObjectSettings->bDisableMutableCompileInEditor;
		CompileSettings.bEnableAutomaticCompilation = CustomizableObjectSettings->bEnableAutomaticCompilation;
		CompileSettings.bCompileObjectsSynchronously = CustomizableObjectSettings->bCompileObjectsSynchronously;
		CompileSettings.EditorDerivedDataCachePolicy = CustomizableObjectSettings->EditorDerivedDataCachePolicy;
		CompileSettings.CookDerivedDataCachePolicy = CustomizableObjectSettings->CookDerivedDataCachePolicy;

		UCustomizableObjectSystem::GetInstance()->EditorSettingsChanged(CompileSettings);
	}

    return true;
}


void FCustomizableObjectEditorModule::RegisterSettings()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
    {
		UCustomizableObjectEditorSettings* EditorSettings = GetMutableDefault<UCustomizableObjectEditorSettings>();

        ISettingsSectionPtr SettingsSectionPtr = SettingsModule->RegisterSettings("Editor", "Plugins", "CustomizableObjectSettings",
            LOCTEXT("MutableEditorSettings_Setting", "Mutable"),
            LOCTEXT("MutableEditorSettings_Setting_Desc", "Mutable Settings"),
            EditorSettings);

		if (SettingsSectionPtr.IsValid())
		{
			SettingsSectionPtr->OnModified().BindRaw(this, &FCustomizableObjectEditorModule::HandleSettingsSaved);
		}
		
		if (UCustomizableObjectSystem::GetInstance() != nullptr)
		{
			if (EditorSettings != nullptr)
			{
				FEditorCompileSettings CompileSettings;
				CompileSettings.bIsMutableEnabled = !EditorSettings->bDisableMutableCompileInEditor;
				CompileSettings.bEnableAutomaticCompilation = EditorSettings->bEnableAutomaticCompilation;
				CompileSettings.bCompileObjectsSynchronously = EditorSettings->bCompileObjectsSynchronously;
				CompileSettings.EditorDerivedDataCachePolicy = EditorSettings->EditorDerivedDataCachePolicy;
				CompileSettings.CookDerivedDataCachePolicy = EditorSettings->CookDerivedDataCachePolicy;

				UCustomizableObjectSystem::GetInstance()->EditorSettingsChanged(CompileSettings);
			}
		}
		
		SettingsModule->RegisterSettings("Project", "Plugins", "CustomizableObjectSettings",
			LOCTEXT("MutableProjectSettings_Setting", "Mutable"),
			LOCTEXT("MutableProjectSettings_Setting_Desc", "Mutable Settings"),
			GetMutableDefault<UCustomizableObjectProjectSettings>());
    }
}


void FCustomizableObjectEditorModule::RegisterCustomDetails(FPropertyEditorModule& PropertyModule, const UClass* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	const FName ClassName = FName(Class->GetName());
	PropertyModule.RegisterCustomClassLayout(ClassName, DetailLayoutDelegate);

	RegisteredCustomDetails.Add(ClassName);
}


void FCustomizableObjectEditorModule::OpenCOIE(const TArray<FString>& Arguments)
{
	int32 SlotID = INDEX_NONE;
	if (Arguments.Num() >= 1)
	{
		SlotID = FCString::Atoi(*Arguments[0]);
	}

	const UWorld* CurrentWorld = []() -> const UWorld*
	{
		UWorld* WorldForCurrentCOI = nullptr;
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if ((Context.WorldType == EWorldType::Game) && (Context.World() != NULL))
			{
				WorldForCurrentCOI = Context.World();
			}
		}
		// Fall back to GWorld if we don't actually have a world.
		if (WorldForCurrentCOI == nullptr)
		{
			WorldForCurrentCOI = GWorld;
		}
		return WorldForCurrentCOI;
	}();
	const int32 PlayerIndex = 0;

	// Open the Customizable Object Instance Editor
	if (UCustomizableObjectInstanceUsage* SelectedCustomizableObjectInstanceUsage = GetPlayerCustomizableObjectInstanceUsage(SlotID, CurrentWorld, PlayerIndex))
	{
		if (UCustomizableObjectInstance* COInstance = SelectedCustomizableObjectInstanceUsage->GetCustomizableObjectInstance())
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
			TWeakPtr<IAssetTypeActions> WeakAssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(UCustomizableObjectInstance::StaticClass());

			if (TSharedPtr<IAssetTypeActions> AssetTypeActions = WeakAssetTypeActions.Pin())
			{
				TArray<UObject*> AssetsToEdit;
				AssetsToEdit.Add(COInstance);
				AssetTypeActions->OpenAssetEditor(AssetsToEdit);
			}
		}
	}
}


void FCustomizableObjectEditorModule::RegisterFactory()
{
	if (GEditor)
	{
		GEditor->ActorFactories.Add(NewObject<UCustomizableObjectInstanceFactory>());
		if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
		{
			PlacementSubsystem->RegisterAssetFactory(NewObject<UCustomizableObjectInstanceFactory>());
		}
	}
}


/** Recursively get all Customizable Objects that reference the given Customizable Object. */
void GetReferencingCustomizableObjects(FName CustomizableObjectName, TArray<FName>& VisitedObjectNames, TArray<FAssetData>& ReferencingAssets)
{
	if (VisitedObjectNames.Contains(CustomizableObjectName))
	{
		return;
	}

	VisitedObjectNames.Add(CustomizableObjectName);

	TArray<FName> ReferencedObjectNames;
	
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().GetReferencers(CustomizableObjectName, ReferencedObjectNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

	// Required to be deterministic.
	ReferencedObjectNames.Sort([](const FName& A, const FName& B)
	{
		return A.LexicalLess(B);
	});
	
	TArray<FAssetData> AssetDataArray;

	FARFilter Filter;
	Filter.PackageNames = MoveTemp(ReferencedObjectNames);
	Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only search on-disk packages (ensures deterministic results)

	AssetRegistryModule.Get().GetAssets(Filter, AssetDataArray);

	// Required to be deterministic.
	AssetDataArray.Sort([](const FAssetData& A, const FAssetData& B)
		{
			return A.PackageName.LexicalLess(B.PackageName);
		});

	for (FAssetData AssetData : AssetDataArray)
	{
		if (AssetData.GetClass() == UCustomizableObject::StaticClass())
		{
			FName ReferencedObjectName = AssetData.GetPackage()->GetFName();
	
			ReferencingAssets.Add(AssetData);

			GetReferencingCustomizableObjects(ReferencedObjectName, VisitedObjectNames, ReferencingAssets);
		}
	}
}


void GetReferencingPackages(const UCustomizableObject& Object, TArray<FAssetData>& ReferencingAssets)
{
	// Gather all child CustomizableObjects
	TArray<FName> VisitedObjectNames;
	GetReferencingCustomizableObjects(Object.GetPackage()->GetFName(), VisitedObjectNames, ReferencingAssets);

	// Gather all tables which will composite the final tables
	TArray<FAssetData> ReferencingCustomizableObjects = ReferencingAssets;
	for (const FAssetData& ReferencingCustomizableObject : ReferencingCustomizableObjects)
	{
		const TSoftObjectPtr SoftObjectPtr(ReferencingCustomizableObject.ToSoftObjectPath());

		const UCustomizableObject* ChildCustomizableObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(SoftObjectPtr));
		if (!ChildCustomizableObject)
		{
			continue;
		}

		TArray<UCustomizableObjectNodeTable*> TableNodes;
		ChildCustomizableObject->GetPrivate()->GetSource()->GetNodesOfClass(TableNodes);

		for (const UCustomizableObjectNodeTable* TableNode : TableNodes)
		{
			TArray<FAssetData> DataTableAssets = TableNode->GetParentTables();

			for (const FAssetData& DataTableAsset : DataTableAssets)
			{
				if (DataTableAsset.IsValid())
				{
					ReferencingAssets.AddUnique(DataTableAsset);
				}
			}
		}		
	}
}


bool FCustomizableObjectEditorModule::IsCompilationOutOfDate(const UCustomizableObject& Object, bool bSkipIndirectReferences, TArray<FName>& OutOfDatePackages, TArray<FName>& OutAddedPackages, TArray<FName>& OutRemovedPackages, bool& bOutReleaseVersion) const
{
	bool Result;
	
	auto Callback = [&](bool bOutOfDate, bool bVersionDiff, TArray<FName> OfDatePackages, TArray<FName> AddedPackages, TArray<FName> RemovedPackages)
	{
		Result = bOutOfDate;
		OutOfDatePackages = OfDatePackages;
		OutAddedPackages = AddedPackages;
		OutRemovedPackages = RemovedPackages;
		bOutReleaseVersion = bVersionDiff;
	};

	IsCompilationOutOfDate(Object, bSkipIndirectReferences, TNumericLimits<float>::Max(), Callback);
	
	return Result;
}


struct FCompilationOutOfDateContext
{
	TArray<TPair<FName, FGuid>> ParticipatingObjects;

	int32 IndexParticipatingObject = 0;

	float MaxTime = 0;
	
	TArray<FName> OutOfDatePackages;
	TArray<FName> AddedPackages;
	TArray<FName> RemovedPackages;

	bool bExternalOperationVersionDiff = false;
	bool bVersionDiff = false;
	
	ICustomizableObjectEditorModule::IsCompilationOutOfDateCallback Callback;
};


/** Async because work is split in between ticks. */
void IsCompilationOutOfDate_Async(const TSharedRef<FCompilationOutOfDateContext>& Context)
{
	MUTABLE_EDITOR_CPUPROFILER_SCOPE(IsCompilationOutOfDate_Async)

	check(IsInGameThread());
	
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	
	const double StartTime = FPlatformTime::Seconds();
	
	while (FPlatformTime::Seconds() - StartTime < Context->MaxTime &&
		Context->IndexParticipatingObject < Context->ParticipatingObjects.Num())
	{
		const TPair<FName, FGuid>& ParticipatingObject = Context->ParticipatingObjects[Context->IndexParticipatingObject];
			
		TSoftObjectPtr SoftObjectPtr(FSoftObjectPath(ParticipatingObject.Key.ToString()));
		if (SoftObjectPtr) // If loaded
		{
			const FGuid PackageGuid = TruncatePackageSavedHash(SoftObjectPtr->GetPackage()->GetSavedHash());
			if (PackageGuid != ParticipatingObject.Value)
			{
				Context->OutOfDatePackages.Add(ParticipatingObject.Key);
			}
		}
		else // Not loaded
		{
			FAssetPackageData AssetPackageData;
			const UE::AssetRegistry::EExists Result = AssetRegistry.TryGetAssetPackageData(ParticipatingObject.Key, AssetPackageData);
				
			if (Result != UE::AssetRegistry::EExists::Exists)
			{
				Context->RemovedPackages.Add(ParticipatingObject.Key);
			}

			const FGuid PackageGuid = TruncatePackageSavedHash(AssetPackageData.GetPackageSavedHash());
			if (PackageGuid != ParticipatingObject.Value)
			{
				Context->OutOfDatePackages.Add(ParticipatingObject.Key);
			}
		}

		++Context->IndexParticipatingObject;
	}

	if (Context->IndexParticipatingObject == Context->ParticipatingObjects.Num())
	{
		const bool bOutOfDate = Context->bVersionDiff || !Context->OutOfDatePackages.IsEmpty() || !Context->AddedPackages.IsEmpty() || !Context->RemovedPackages.IsEmpty();
		Context->Callback(bOutOfDate, Context->bVersionDiff, Context->OutOfDatePackages, Context->AddedPackages, Context->RemovedPackages);
	}
	else if (GEditor)
	{
		GEditor->GetTimerManager()->SetTimerForNextTick([=]()
		{
			IsCompilationOutOfDate_Async(Context);
		});
	}
}


void FCustomizableObjectEditorModule::IsCompilationOutOfDate(const UCustomizableObject& Object, bool bSkipIndirectReferences, float MaxTime, const IsCompilationOutOfDateCallback& Callback) const
{
	MUTABLE_EDITOR_CPUPROFILER_SCOPE(FCustomizableObjectEditorModule::IsCompilationOutOfDate)

	check(IsInGameThread());
	
	// TODO CO Custom version
	// TODO List of plugins and their custom versions
	// Maybe use BuildDerivedDataKey? BuildDerivedDataKey should also consider bSkipIndirectReferences
	
	const UModelResources* ModelResources = Object.GetPrivate()->GetModelResources();
	if (!ModelResources)
	{
		Callback(true, false, {}, {}, {});
		return;
	}

	TSharedRef<FCompilationOutOfDateContext> Context = MakeShared<FCompilationOutOfDateContext>();
	Context->ParticipatingObjects = ModelResources->ParticipatingObjects.Array();
	Context->MaxTime = MaxTime;
	Context->Callback = Callback;
	
	Context->bVersionDiff = false;
	if (ICustomizableObjectVersionBridgeInterface* VersionBridge = Cast<ICustomizableObjectVersionBridgeInterface>(Object.VersionBridge))
	{
		Context->bVersionDiff = ModelResources->ReleaseVersion != VersionBridge->GetCurrentVersionAsString();
	}

	for (TPair<TObjectPtr<const UScriptStruct>, unsigned> Pair : Object.GetPrivate()->GetModelResources()->ExternalOperationVersions)
	{
		FInstancedStruct OperationInstancedStruct;
		OperationInstancedStruct.InitializeAs(Pair.Key);

		const UE::Mutable::FExternalOperation* Operation = OperationInstancedStruct.GetPtr<UE::Mutable::FExternalOperation>();
		if (!Operation)
		{
			Context->bExternalOperationVersionDiff = true;
			break;
		}
		
		if (Operation->GetVersion() != Pair.Value)
		{
			Context->bExternalOperationVersionDiff = true;
			break;
		}
	}

	check(MaxTime == TNumericLimits<float>::Max()  || bSkipIndirectReferences); // If async, bSkipIndirectReferences must be false since it is very expensive and can not be split in subtasks.

	// Check that we have the exact same set of participating object as before. This can change due to indirect references and versioning.
	if (!bSkipIndirectReferences)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		const TMap<FName, FGuid>& OldParticipatingObjects = ModelResources->ParticipatingObjects;

		const int32 Num = OldParticipatingObjects.Num();

		Context->AddedPackages.Reserve(Num);
		
		// Due to performance issues, we will skip loading all objects. We can do that since loading/not loading objects do not affect the number of indirect objects discovered
		// (e.g., we will traverse the same number of COs/Tables regardless if we do not load meshes/textures...). 
		TMap<FName, FGuid> ParticipatingObjects = GetParticipatingObjects(&Object);
		
		for (const TTuple<FName, FGuid>& ParticipatingObject : ParticipatingObjects)
		{
			// Since here we are checking if the smaller set (objects found now without loading all objects) is contained in the larger set (objects found in the compilation pass),
			// there is no need to check if the asset is a indirect reference (CO or Table).
			if (!OldParticipatingObjects.Contains(ParticipatingObject.Key))
			{
				Context->AddedPackages.AddUnique(ParticipatingObject.Key);
			}
		}

		for (const TTuple<FName, FGuid>& OldParticipatingObject : OldParticipatingObjects)
		{
			FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(OldParticipatingObject.Key.ToString()));
			if (AssetData.AssetClassPath == UCustomizableObject::StaticClass()->GetClassPathName() ||
				AssetData.AssetClassPath == UDataTable::StaticClass()->GetClassPathName())
			{
				if (!ParticipatingObjects.Contains(OldParticipatingObject.Key))
				{
					Context->RemovedPackages.AddUnique(OldParticipatingObject.Key);
				}
			}
		}
	}

	IsCompilationOutOfDate_Async(Context);
}


bool FCustomizableObjectEditorModule::IsRootObject(const UCustomizableObject& Object) const
{
	return GraphTraversal::IsRootObject(Object);
}

FString FCustomizableObjectEditorModule::GetCurrentReleaseVersionForObject(const UCustomizableObject& Object) const
{
	if (Object.VersionBridge && Object.VersionBridge->GetClass()->ImplementsInterface(UCustomizableObjectVersionBridgeInterface::StaticClass()))
	{
		ICustomizableObjectVersionBridgeInterface* CustomizableObjectVersionBridgeInterface = Cast<ICustomizableObjectVersionBridgeInterface>(Object.VersionBridge);

		if (CustomizableObjectVersionBridgeInterface)
		{
			return CustomizableObjectVersionBridgeInterface->GetCurrentVersionAsString();
		}
	}

	return FString();
}


UCustomizableObject* FCustomizableObjectEditorModule::GetRootObject(UCustomizableObject* ChildObject) const
{
	return GraphTraversal::GetRootObject(ChildObject);
}


const UCustomizableObject* FCustomizableObjectEditorModule::GetRootObject(const UCustomizableObject* ChildObject) const
{
	return GraphTraversal::GetRootObject(ChildObject);
}


void FCustomizableObjectEditorModule::GetAllObjectsInGraph(UCustomizableObject* Object, TSet<UCustomizableObject*>& OutRelated) const
{
	GraphTraversal::GetAllObjectsInGraph(Object, OutRelated);
}


void FCustomizableObjectEditorModule::OnUpstreamCOsLoaded(UCustomizableObject* Object) const
{
	check(Object)
	const int32 CustomVersion = Object->GetLinkerCustomVersion(FCustomizableObjectCustomVersion::GUID);
	for (int32 Version = CustomVersion + 1; Version <= FCustomizableObjectCustomVersion::LatestVersion; ++Version)
	{
		OnUpstreamCOsLoadedFixup(Object, Version);
	}
}


void FCustomizableObjectEditorModule::OnUpstreamCOsLoadedFixup(UCustomizableObject* Object, int32 CustomizableObjectCustomVersion) const
{
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MovedLODSettingsToMeshComponentNode)
	{
		check(Object)
		const UCustomizableObject* RootObject = GetRootObject(Object);
		check(RootObject);
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FMutableLODSettings RootObjectLODSettings = RootObject->LODSettings;
		Object->LODSettings = RootObjectLODSettings;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Override the LOD Settings found in the UCustomizableObjectNodeComponentMesh of this CO using the LODSettings of the parent CO
		{
			TArray<UCustomizableObjectNodeComponentMesh*> ComponentMeshNodes;
			Object->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeComponentMesh>(ComponentMeshNodes);
			for (UCustomizableObjectNodeComponentMesh* ComponentNode : ComponentMeshNodes)
			{
				ComponentNode->LODSettings_DEPRECATED = RootObjectLODSettings;
			}
		}
	}
}



void FCustomizableObjectEditorModule::BakeCustomizableObjectInstance(UCustomizableObjectInstance* InTargetInstance, const FBakingConfiguration& InBakingConfig)
{
	UCustomizableObjectInstanceBaker* InstanceBaker = NewObject<UCustomizableObjectInstanceBaker>();

	// Add the heap object to the root so we prevent it from being removed. It will get removed from there once it finishes it's work.
	InstanceBaker->AddToRoot();
	
	// On baker operation completed just remove it from the root so it gets eventually destroyed by the GC system
	const TSharedPtr<FOnBakerFinishedWork> OnBakerFinishedWorkCallback = MakeShared<FOnBakerFinishedWork>();
	OnBakerFinishedWorkCallback->BindLambda([InstanceBaker]
	{
		InstanceBaker->RemoveFromRoot();
	});
	
	// Ask for the baking of the instance
	InstanceBaker->BakeInstance(InTargetInstance, InBakingConfig, OnBakerFinishedWorkCallback);
}


void FCustomizableObjectEditorModule::EvictSkeletalMeshThumbnailScene(USkeletalMesh* SkeletalMesh) const
{
	if (!SkeletalMesh)
	{
		return;
	}

	FThumbnailRenderingInfo* RenderingInfo = UThumbnailManager::Get().GetRenderingInfo(SkeletalMesh);

	if (!RenderingInfo)
	{
		return;
	}

	if (UCustomizableObjectSkeletalMeshThumbnailRenderer* ThumbnailRenderer = 
			Cast<UCustomizableObjectSkeletalMeshThumbnailRenderer>(RenderingInfo->Renderer))
	{
		ThumbnailRenderer->EvictThumbnailScene(SkeletalMesh);
	}
}


USkeletalMesh* FCustomizableObjectEditorModule::GetReferenceSkeletalMesh(const UCustomizableObject& Object, const FName& ComponentName) const
{
	TSet<UCustomizableObject*> Objects;
	GetAllObjectsInGraph(const_cast<UCustomizableObject*>(&Object), Objects);
	
	for (const UCustomizableObject* CurrentObject : Objects)
	{
		TObjectPtr<UEdGraph> Source = CurrentObject->GetPrivate()->GetSource();
		check(Source)
		
		for (TObjectPtr<UEdGraphNode> Node : CurrentObject->GetPrivate()->GetSource()->Nodes)
		{
			if (const UCONodeComponentSkeletalMesh* NodeComponentMesh = Cast<UCONodeComponentSkeletalMesh>(Node))
			{
				if (NodeComponentMesh->GetComponentName() == ComponentName)
				{
					return NodeComponentMesh->ReferenceSkeletalMesh;
				}
			}
		}	
	}

	return {};
}


TMap<FName, FGuid> FCustomizableObjectEditorModule::GetParticipatingObjects(const UCustomizableObject* Object, const FCompilationOptions* InOptions) const
{
	MUTABLE_EDITOR_CPUPROFILER_SCOPE(FCustomizableObjectEditorModule::GetParticipatingObjects);

	FCompilationOptions Options = InOptions ? *InOptions : GetCompilationOptions(*Object);
	TSharedPtr<FMutableCompilationContext> CompilationContext = MakeShared<FMutableCompilationContext>(Object, nullptr, Options);
	FMutableGraphGenerationContext Context(CompilationContext);

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	
	// Store the list of participating assets here
	TMap<FName, FGuid> Result;
		
	auto VisitDependencies = [&AssetRegistry](const FSoftObjectPath& RootAsset, const TArray<UClass*>& FilterClasses, bool bRecursively, const TFunction<void(const FSoftObjectPath&)>& VisitFunc)
	{
		TArray<FSoftObjectPath> References;
		
		TArray<FName> PackageDependencies;
		AssetRegistry.GetDependencies(RootAsset.GetLongPackageFName(), PackageDependencies);

		FARFilter Filter;
		Filter.bRecursivePaths = bRecursively;
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only enumerate on-disk assets- avoids indeterministic results
		
		for (const FName& PackageName : PackageDependencies)
		{
			if (!PackageName.ToString().StartsWith(TEXT("/TempAutosave")))
			{
				Filter.PackageNames.Add(PackageName);
			}
		}

		TArray<FAssetData> AssetDataArray;
		AssetRegistry.GetAssets(Filter, AssetDataArray);

		// Required to be deterministic.
		AssetDataArray.Sort([](const FAssetData& A, const FAssetData& B)
			{
				return A.PackageName.LexicalLess(B.PackageName);
			});

		for (const FAssetData& Asset : AssetDataArray)
		{
			if (!FilterClasses.Contains(Asset.GetClass(EResolveClass::Yes)))
			{
				VisitFunc(Asset.GetSoftObjectPath());
			}
		}
	};
	
	// Use these 2 helpers to add assets to the list.
	auto AddObject = [&Result](UObject* Candidate)
	{
		if (Candidate)
		{
			// TODO UE-226453
			const FGuid PackageGuid = TruncatePackageSavedHash(Candidate->GetPackage()->GetSavedHash());
			FName PackageName = Candidate->GetPackage()->GetFName();
			Result.Emplace(PackageName, PackageGuid);
		}
	};

	auto AddObjectSoft = [&Result, &AssetRegistry](const FSoftObjectPath& SoftPath)
	{
		FAssetPackageData AssetPackageData;
		const UE::AssetRegistry::EExists Query = AssetRegistry.TryGetAssetPackageData(SoftPath.GetLongPackageFName(), AssetPackageData);
		if (Query != UE::AssetRegistry::EExists::Exists)
		{
			return;
		}

		const FGuid PackageGuid = TruncatePackageSavedHash(AssetPackageData.GetPackageSavedHash());
		Result.Emplace(SoftPath.GetLongPackageFName(), PackageGuid);
	};
	
	auto Visit = [&](UCustomizableObjectNode& Node)
	{
		if (UCustomizableObjectNodeObject* NodeObject = Cast<UCustomizableObjectNodeObject>(&Node))
		{
			AddObject(GraphTraversal::GetObject(*NodeObject));
		}
		
		if (UCustomizableObjectNodeSkeletalMesh* SkeletalNode = Cast<UCustomizableObjectNodeSkeletalMesh>(&Node))
		{
			{
				const FSoftObjectPath SoftObjectPath = SkeletalNode->AnimInstance.ToSoftObjectPath();
				AddObjectSoft(SoftObjectPath);
				VisitDependencies(SoftObjectPath, { UAnimBlueprint::StaticClass() }, false, AddObjectSoft);
			}

			{
				const FSoftObjectPath SoftObjectPath = SkeletalNode->SkeletalMesh.ToSoftObjectPath();
				AddObjectSoft(SoftObjectPath);
				VisitDependencies(SoftObjectPath, { UPhysicsAsset::StaticClass(), USkeleton::StaticClass() }, false, AddObjectSoft);
				VisitDependencies(SoftObjectPath, { UTexture::StaticClass() }, true, AddObjectSoft);
			}
		}

		else if (UCustomizableObjectNodeStaticMesh* StaticMeshNode = Cast<UCustomizableObjectNodeStaticMesh>(&Node))
		{
			const FSoftObjectPath SoftObjectPath = StaticMeshNode->StaticMesh.ToSoftObjectPath();

			AddObjectSoft(SoftObjectPath);
		}

		else if (UCustomizableObjectNodeAnimationPose* PoseNode = Cast<UCustomizableObjectNodeAnimationPose>(&Node))
		{
			AddObjectSoft(PoseNode->PoseAsset);
		}

		else if (UCustomizableObjectNodeCurve* CurveNode = Cast<UCustomizableObjectNodeCurve>(&Node))
		{
			AddObjectSoft(CurveNode->CurveAsset);
		}

		else if (UCONodeComponentSkeletalMesh* CompNode = Cast<UCONodeComponentSkeletalMesh>(&Node))
		{
			AddObjectSoft(CompNode->ReferenceSkeletalMesh);
		}

		else if (UCustomizableObjectNodeTexture* TexNode = Cast<UCustomizableObjectNodeTexture>(&Node))
		{
			AddObjectSoft(TexNode->Texture);
		}

		else if (UCustomizableObjectNodeTextureProject* ProjectNode = Cast<UCustomizableObjectNodeTextureProject>(&Node))
		{
			AddObjectSoft(ProjectNode->ReferenceTexture);
		}

		else if (UCustomizableObjectNodeTextureTransform* TransformNode = Cast<UCustomizableObjectNodeTextureTransform>(&Node))
		{
			AddObjectSoft(TransformNode->ReferenceTexture);
		}

		else if (UCustomizableObjectNodeSkeletalMeshParameter* MeshParamNode = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(&Node))
		{
			AddObjectSoft(MeshParamNode->DefaultValue.ToSoftObjectPath());
			AddObjectSoft(MeshParamNode->ReferenceValue.ToSoftObjectPath());
		}

		else if (UCustomizableObjectNodeTextureParameter* TextureParameterNode = Cast<UCustomizableObjectNodeTextureParameter>(&Node))
		{
			AddObjectSoft(TextureParameterNode->DefaultValue);
			AddObjectSoft(TextureParameterNode->ReferenceValue);
		}

		else if (UCustomizableObjectNodeMaterialParameter* MaterialParamNode = Cast<UCustomizableObjectNodeMaterialParameter>(&Node))
		{
			AddObjectSoft(MaterialParamNode->DefaultValue.ToSoftObjectPath());
			AddObjectSoft(MaterialParamNode->ReferenceValue.ToSoftObjectPath());
		}

		else if (UCONodeSkeletalMeshSection* MeshSectionNode = Cast<UCONodeSkeletalMeshSection>(&Node))
		{
			AddObjectSoft(MeshSectionNode->GetMaterial());
			int32 NumImages = MeshSectionNode->GetNumParameters(EMaterialParameterType::Texture);
			for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
			{
				AddObject(MeshSectionNode->GetImageReferenceTexture(ImageIndex));
			}
		}
			
		else if (UCONodeMaterialModify* MaterialModifyNode = Cast<UCONodeMaterialModify>(&Node))
		{
			TArray<FEdGraphPinReference> MaterialTextureParameterPins;
			MaterialModifyNode->GetMaterialTextureParameterPins(MaterialTextureParameterPins);
			
			for (FEdGraphPinReference MaterialTextureParameterPin : MaterialTextureParameterPins)
			{
				AddObjectSoft(MaterialModifyNode->GetTexturePinParameterReferenceTexture(*MaterialTextureParameterPin.Get()).ToSoftObjectPath());
			}
		}

		else if (UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(&Node))
		{
			UDataTable* DataTable = nullptr;
				
			if (TableNode->TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry)
			{
				DataTable = GenerateDataTableFromStruct(TableNode, Context);
			}
			else
			{
				DataTable = UE::Mutable::Private::LoadObject(TableNode->Table);
			}

			if (DataTable)
			{
				AddObject(DataTable);

				TArray<FName> RowNames = TableNode->GetEnabledRows(*DataTable);

				TArray<FName> ExpectedPropNames = DataTableUtils::GetStructPropertyNames(DataTable->RowStruct);
				for (const FName& ColumnName : ExpectedPropNames)
				{
					FProperty* ColumnProperty = DataTable->FindTableProperty(ColumnName);
					if (!ColumnProperty)
					{
						continue;
					}

					const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty);
					if (!SoftObjectProperty)
					{
						continue;
					}

					for (const FName& RowName : RowNames )
					{
						uint8* RowData = DataTable->FindRowUnchecked(RowName);
						if (RowData)
						{
							uint8* CellData = ColumnProperty->ContainerPtrToValuePtr<uint8>(RowData, 0);
							if (CellData)
							{
								const FSoftObjectPtr& Path = SoftObjectProperty->GetPropertyValue(CellData);
								AddObjectSoft(Path.ToSoftObjectPath());
							}
						}
					}
				}

				if (UCompositeDataTable* CompositeDataTable = Cast<UCompositeDataTable>(DataTable))
				{
					const TArray<UDataTable*>& ParentTables = CompositeDataTable->GetParentTables();
					for (UDataTable* ParentTable : ParentTables)
					{
						if (ParentTable)
						{
							AddObject(ParentTable);
						}
					}
				}
			}
		}
		
		else if (UCustomizableObjectNodeMacroInstance* MacroInstanceNode = Cast<UCustomizableObjectNodeMacroInstance>(&Node))
		{
			AddObject(MacroInstanceNode->ParentMacroLibrary);
		}
	};

	TArray<const UCustomizableObject*> VisitedObjects;
    UCustomizableObjectNodeObject* RootNode = GraphTraversal::GetFullGraphRootNode(Object, VisitedObjects);
	
	if (RootNode)
	{
		GraphTraversal::VisitNodes(*RootNode, Visit);
	}
	
	// Done
	return Result;
}


void FCustomizableObjectEditorModule::BackwardsCompatibleFixup(UEdGraph& Graph, int32 CustomizableObjectCustomVersion)
{
	if (UCustomizableObjectGraph* COGraph = Cast<UCustomizableObjectGraph>(&Graph))
	{
		COGraph->BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	}
}


void FCustomizableObjectEditorModule::PostBackwardsCompatibleFixup(UEdGraph& Graph)
{
	if (UCustomizableObjectGraph* COGraph = Cast<UCustomizableObjectGraph>(&Graph))
	{
		COGraph->PostBackwardsCompatibleFixup();
	}
}


void FCustomizableObjectEditorModule::AddPendingLoad(UCustomizableObject* Object)
{
	check(IsInGameThread());
	ObjectsPendingLoad.Add(TStrongObjectPtr(Object));
}


void FCustomizableObjectEditorModule::EnqueueCompileRequest(const TSharedRef<FCompilationRequest>& InCompilationRequest, bool bForceRequest)
{
	Compiler->EnqueueCompileRequest(InCompilationRequest, bForceRequest);
}


const TMap<FName, FLinearColor>& FCustomizableObjectEditorModule::GetExternalPinTypeColors() const
{
	return ExternalTypeTypePinColors;
}


void FCustomizableObjectEditorModule::RegisterExternalOperation(const UScriptStruct* Operation)
{
	if (!Operation)
	{
		return;
	}
	
	if (!Operation->IsChildOf(UE::Mutable::FExternalOperation::StaticStruct()))
	{
		return;
	}
	
	auto Register = [&](const UScriptStruct* ExternalPinType)
	{
		if (ExternalPinType->GetCppStructOps()->GetCapabilities().IsAbstract)
		{
			return;
		}

		FName PinType = UEdGraphSchema_CustomizableObject::GetPinCategoryName(ExternalPinType);
		FLinearColor Color;
			
		const FString Hex = ExternalPinType->GetMetaData("PinColor"); // e.g. "#FF00AAFF"
		if (!Hex.IsEmpty())
		{
			Color = FColor::FromHex(Hex);
		}
		else
		{
			const uint32 Hash = GetTypeHash(PinType.ToString());
			Color = FLinearColor::MakeRandomSeededColor(Hash);
		}

		TStrongObjectPtr<const UScriptStruct> StrongExternalPinType = TStrongObjectPtr(ExternalPinType);
	
		TStrongObjectPtr<const UScriptStruct>* Result = ExternalTypeCategories.Find(PinType);
		if (Result && *Result != StrongExternalPinType)
		{
			UE_LOGF(LogMutable, Error, "External Pin Type already registered: %ls", *PinType.ToString());
			return;
		}
			
		ExternalTypeTypePinColors.Add(PinType, Color);
		ExternalTypeFriendlyName.Add(PinType, ExternalPinType->GetDisplayNameText());
		ExternalTypeCategories.Add(PinType, StrongExternalPinType);
			
		++RegisteredExternalType.FindOrAdd(PinType, 0);
			
		check(RegisteredExternalType.Num() == ExternalTypeTypePinColors.Num());
		check(RegisteredExternalType.Num() == ExternalTypeFriendlyName.Num());
		check(RegisteredExternalType.Num() == ExternalTypeCategories.Num());
	};
	
	FInstancedStruct InstancedStruct;
	InstancedStruct.InitializeAs(Operation);
	
	const UE::Mutable::FExternalOperation& ExternalOperation = InstancedStruct.Get<UE::Mutable::FExternalOperation>();

	Register(ExternalOperation.GetOutput().Value);
	
	for (const TPair<FText, const UScriptStruct*>& Input : ExternalOperation.GetInputs())
	{
		Register(Input.Value);
	}
}


void FCustomizableObjectEditorModule::UnregisterExternalOperation(const UScriptStruct* Operation)
{
	if (IsEngineExitRequested())
	{
		return;
	}
	
	if (!Operation)
	{
		return;
	}
	
	if (!Operation->IsChildOf(UE::Mutable::FExternalOperation::StaticStruct()))
	{
		return;
	}

	auto Unregister = [&](const UScriptStruct* ExternalPinType)
	{
		if (ExternalPinType->GetCppStructOps()->GetCapabilities().IsAbstract)
		{
			return;
		}
			
		FName PinType = UEdGraphSchema_CustomizableObject::GetPinCategoryName(ExternalPinType);

		uint32* Count = RegisteredExternalType.Find(PinType);
		if (!Count)
		{
			return;
		}
			
		--*Count;
		if (*Count == 0)
		{
			ExternalTypeTypePinColors.Remove(PinType);
			ExternalTypeFriendlyName.Remove(PinType);
			ExternalTypeCategories.Remove(PinType);
			
			RegisteredExternalType.Remove(PinType);
			
			check(RegisteredExternalType.Num() == ExternalTypeTypePinColors.Num());
			check(RegisteredExternalType.Num() == ExternalTypeFriendlyName.Num());
			check(RegisteredExternalType.Num() == ExternalTypeCategories.Num());
		}
	};
	
	FInstancedStruct InstancedStruct;
	InstancedStruct.InitializeAs(Operation);
	
	const UE::Mutable::FExternalOperation& ExternalOperation = InstancedStruct.Get<UE::Mutable::FExternalOperation>();

	Unregister(ExternalOperation.GetOutput().Value);
	
	for (const TPair<FText, const UScriptStruct*>& Input : ExternalOperation.GetInputs())
	{
		Unregister(Input.Value);
	}
}


const TMap<FName, TStrongObjectPtr<const UScriptStruct>>& FCustomizableObjectEditorModule::GetExternalPinTypes() const
{
	return ExternalTypeCategories;
}


const TMap<FName, FText>& FCustomizableObjectEditorModule::GetExternalPinFriendlyName() const
{
	return ExternalTypeFriendlyName;
}


void FCustomizableObjectEditorModule::CompileCustomizableObject(UCustomizableObject& Object, const FCompileParams* Params, bool bSilent, bool bForce)
{
	UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
	if (!System)
	{
		if (Params)
		{
			FCompileCallbackParams CallbackParams;
			CallbackParams.bRequestFailed = true;
			CallbackParams.bCompiled = Object.IsCompiled();
		
			Params->Callback.ExecuteIfBound(CallbackParams);
			Params->CallbackNative.ExecuteIfBound(CallbackParams);
		}
		
		return;
	}

	TSharedRef<FCompilationRequest> CompileRequest = MakeShared<FCompilationRequest>(Object);

	if (Params)
	{
		CompileRequest->bAsync = Params->bAsync;
		CompileRequest->bSkipIfCompiled = Params->bSkipIfCompiled;
		CompileRequest->bSkipIfNotOutOfDate = Params->bSkipIfNotOutOfDate;
		CompileRequest->Options.TextureCompression = Params->TextureCompression;
		CompileRequest->Options.bGatherReferences = Params->bGatherReferences;
		CompileRequest->Callback = Params->Callback;
		CompileRequest->CallbackNative = Params->CallbackNative;

		// Override the optimization level provided in the request with the one set in the CO
		if (Params->OptimizationLevel == ECustomizableObjectOptimizationLevel::FromCustomizableObject)
		{
			CompileRequest->Options.OptimizationLevel = Object.GetPrivate()->OptimizationLevel;
		}
		else
		{
			CompileRequest->Options.OptimizationLevel = ConvertOptimizationLevel(Params->OptimizationLevel);
		}

		if (Params->CompileOnlySelectedInstance)
		{
			CompileRequest->Options.ParamNamesToSelectedOptions = GetCompileOnlySelectedParameters(*Params->CompileOnlySelectedInstance);
		}
	}

	
	CompileRequest->SetDerivedDataCachePolicy(GetDerivedDataCachePolicyForEditor());
	CompileRequest->bSilentCompilation = bSilent;
	
	EnqueueCompileRequest(CompileRequest, bForce);
}


void FCustomizableObjectEditorModule::Tick(bool bBlocking)
{
	if (TSet<TStrongObjectPtr<UCustomizableObject>>::TIterator It = ObjectsPendingLoad.CreateIterator())
	{
		UCustomizableObject* Object = It->Get();
		if (IsValid(Object)) // Someone may have called MarkAsGarbage.
		{
			TSet<UCustomizableObject*> RelatedObjects;
			const bool bPendingLoad = GraphTraversal::GetAllObjectsInGraphIncremental(Object, RelatedObjects);

			if (!bPendingLoad)
			{
				bool bHierarchyLoaded = true;
				for (const UCustomizableObject* RelatedObject : RelatedObjects)
				{
					if (RelatedObject && !RelatedObject->HasAnyFlags(EObjectFlags::RF_LoadCompleted))
					{
						bHierarchyLoaded = false;
						break;
					}
				}
			
				if (bHierarchyLoaded)
				{	
					// Traversing the graph upwards does not take into account Group Object Nodes. 
					// A Child Object may reference a parent but not a Group Object. This can cause the loaded CO to not appear in the Related objects. Added manually to fix this.
					RelatedObjects.Add(Object);
				
					for (UCustomizableObject* RelatedObject : RelatedObjects)
					{
						if (ObjectsPendingLoad.Remove(TStrongObjectPtr(RelatedObject)))
						{
							OnUpstreamCOsLoaded(RelatedObject);
							RelatedObject->GetPrivate()->LoadCompiledDataFromDisk();
						}
					}
				}
			}	
		}
		else
		{
			It.RemoveCurrent();
		}
	}
	
	Compiler->Tick(bBlocking);
}


int32 FCustomizableObjectEditorModule::GetRemainingWork() const
{
	return Compiler->GetRemainingWork() + static_cast<int32>(!ObjectsPendingLoad.IsEmpty());
}


void FCustomizableObjectEditorModule::BeginCacheForCookedPlatformData(UCustomizableObject& Object, const ITargetPlatform* TargetPlatform)
{
	if (!TargetPlatform)
	{
		return;
	}

	TArray<TSharedRef<FCompilationRequest>>& ObjectCompileRequests = CookCompileRequests.FindOrAdd(&Object);
	
	const bool Exists = ObjectCompileRequests.ContainsByPredicate([&TargetPlatform](const TSharedPtr<FCompilationRequest>& Request)
	{
		return Request->Options.TargetPlatform == TargetPlatform;
	});

	if (Exists)
	{
		return;
	}

	if (!IsRootObject(Object))
	{
		Object.GetPrivate()->SetIsChildObject(true);
		return;
	}

	const bool bAsync = CVarMutableAsyncCook.GetValueOnAnyThread();

	TSharedRef<FCompilationRequest> CompileRequest = MakeShared<FCompilationRequest>(Object);
	CompileRequest->bAsync = bAsync;
	CompileRequest->Options.OptimizationLevel = UE_MUTABLE_MAX_OPTIMIZATION; // Force max optimization when packaging.
	CompileRequest->Options.TextureCompression = ECustomizableObjectTextureCompression::HighQuality;
	CompileRequest->Options.bIsCooking = true;
	CompileRequest->Options.TargetPlatform = TargetPlatform;

	if (CVarMutableDerivedDataCacheUsage.GetValueOnAnyThread())
	{
		UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();
		CompileRequest->SetDerivedDataCachePolicy(ConvertDerivedDataCachePolicy(System->GetPrivate()->EditorSettings.CookDerivedDataCachePolicy));
	}

	ObjectCompileRequests.Add(CompileRequest);

	EnqueueCompileRequest(CompileRequest, true);
}


bool FCustomizableObjectEditorModule::IsCachedCookedPlatformDataLoaded(UCustomizableObject& Object, const ITargetPlatform* TargetPlatform)
{
	if (!TargetPlatform)
	{
		return true;
	}

	TArray<TSharedRef<FCompilationRequest>>* ObjectCompileRequests = CookCompileRequests.Find(&Object);
	if (!ObjectCompileRequests)
	{
		return true;
	}
	
	const TSharedRef<FCompilationRequest>* CompileRequest = ObjectCompileRequests->FindByPredicate([&TargetPlatform](const TSharedRef<FCompilationRequest>& Request)
	{
		return Request->Options.TargetPlatform == TargetPlatform;
	});
	
	if (CompileRequest)
	{
		return CompileRequest->Get().GetCompilationState() == ECompilationStatePrivate::Completed;
	}

	return true;
}


void FCustomizableObjectEditorModule::CancelCompileRequests()
{
	Compiler->ForceFinishCompilation();
	Compiler->ClearCompileRequests();
}


int32 FCustomizableObjectEditorModule::GetNumCompileRequests()
{
	return Compiler->GetNumCompileRequests();
}


bool FCustomizableObjectEditorModule::IsCompiling(const UCustomizableObject& Object) const
{
	return Object.GetPrivate()->IsLocked() || Compiler->IsRequestQueued(Object);
}


bool FCustomizableObjectEditorModule::IsTickable() const
{
	return UCustomizableObjectSystem::GetInstance()->GetPrivate()->GetRemainingWork() > 0;
}


void FCustomizableObjectEditorModule::Tick(float InDeltaTime)
{
	check(false);
}


TStatId FCustomizableObjectEditorModule::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCustomizableObjectEditorModule, STATGROUP_Tickables);
}


void FCustomizableObjectEditorModule::TickCook(float DeltaTime, bool bCookCompete)
{
	MUTABLE_EDITOR_CPUPROFILER_SCOPE(FCustomizableObjectEditorModule::TickCook);
	UCustomizableObjectSystem::GetInstance()->GetPrivate()->MainTick(false);
}


#undef LOCTEXT_NAMESPACE
