// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/ICustomizableObjectEditorModulePrivate.h"

#include "TickableEditorObject.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "Toolkits/AssetEditorToolkit.h"

struct FBakingConfiguration;
class USkeletalMeshComponent;
class FPropertyEditorModule;
class FExtensibilityManager;
class ICustomizableObjectEditor;
class ICustomizableObjectInstanceEditor;
class FBakeOperationCompletedDelegate;
class FName;

namespace UE::DerivedData
{
	enum class ECachePolicy : uint32;
}

/** Get a list of packages that are used by the compilation but are not directly referenced.
  * List includes:
  * - Child UCustomizableObjects: Have inverted references.
  * - UDataTable: Data Tables used by Composite Data Tables are indirectly referenced by the UStruct and filtered by path. */
void GetReferencingPackages(const UCustomizableObject& Object, TArray<FAssetData>& ObjectNames);

/** Convert the Optimization Level enum to the int32 internal format.  */
int32 ConvertOptimizationLevel(ECustomizableObjectOptimizationLevel OptimizationLevel);

TMap<FString, FString> GetCompileOnlySelectedParameters(const UCustomizableObjectInstance& Instance);

UE::DerivedData::ECachePolicy GetDerivedDataCachePolicyForEditor();

/**
 * StaticMesh editor module
 */
class FCustomizableObjectEditorModule : public ICustomizableObjectEditorModulePrivate, public FTickableCookObject
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// ICustomizableObjectEditorModule interface
	virtual FCustomizableObjectEditorLogger& GetLogger() override;
	virtual bool IsCompilationOutOfDate(const UCustomizableObject& Object, bool bSkipIndirectReferences, TArray<FName>& OutOfDatePackages, TArray<FName>& OutAddedPackages, TArray<FName>& OutRemovedPackages, bool& bOutVersionDiff) const override;
	virtual void IsCompilationOutOfDate(const UCustomizableObject& Object, bool bSkipIndirectReferences, float MaxTime, const IsCompilationOutOfDateCallback& Callback) const override;
	virtual bool IsRootObject(const UCustomizableObject& Object) const override;
	virtual FString GetCurrentReleaseVersionForObject(const UCustomizableObject& Object) const override;
	virtual UCustomizableObject* GetRootObject(UCustomizableObject* ChildObject) const override;
	virtual const UCustomizableObject* GetRootObject(const UCustomizableObject* ChildObject) const override;
	virtual void GetAllObjectsInGraph(UCustomizableObject*, TSet<UCustomizableObject*>& OutRelated) const override;
	virtual void OnUpstreamCOsLoadedFixup(UCustomizableObject* Object, int32 CustomizableObjectCustomVersion) const override;
	virtual void BakeCustomizableObjectInstance(UCustomizableObjectInstance* InTargetInstance, const FBakingConfiguration& InBakingConfig) override;
	virtual void EvictSkeletalMeshThumbnailScene(USkeletalMesh* SkeletalMesh) const override;
	virtual USkeletalMesh* GetReferenceSkeletalMesh(const UCustomizableObject& Object, const FName& ComponentName) const override;
	virtual TMap<FName, FGuid> GetParticipatingObjects(const UCustomizableObject* Object, const FCompilationOptions* Options = nullptr) const override;
	virtual void BackwardsCompatibleFixup(UEdGraph& Graph, int32 CustomizableObjectCustomVersion) override;
	virtual void PostBackwardsCompatibleFixup(UEdGraph& Graph) override;
	virtual void CancelCompileRequests() override;
	virtual int32 GetNumCompileRequests() override;
	virtual bool IsCompiling(const UCustomizableObject& Object) const override;
	virtual void CompileCustomizableObject(UCustomizableObject& Object, const FCompileParams* Params, bool bSilent, bool bForce) override;
	virtual void Tick(bool bBlocking) override;
	virtual int32 GetRemainingWork() const override;
	virtual void BeginCacheForCookedPlatformData(UCustomizableObject& Object, const ITargetPlatform* TargetPlatform) override;
	virtual bool IsCachedCookedPlatformDataLoaded(UCustomizableObject& Object, const ITargetPlatform* TargetPlatform) override;
	virtual TSharedPtr<FExtensibilityManager> GetCustomizableObjectEditorToolBarExtensibilityManager() override { return CustomizableObjectEditor_ToolBarExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetCustomizableObjectEditorMenuExtensibilityManager() override { return CustomizableObjectEditor_MenuExtensibilityManager; }
	virtual void AddPendingLoad(UCustomizableObject* Object) override;

	// ICustomizableObjectEditorModulePrivate interface
	virtual void EnqueueCompileRequest(const TSharedRef<FCompilationRequest>& InCompilationRequest, bool bForceRequest = false) override;
	virtual const TMap<FName, FLinearColor>& GetExternalPinTypeColors() const override;
	virtual void RegisterExternalOperation(const UScriptStruct* Operation) override;
	virtual void UnregisterExternalOperation(const UScriptStruct* Operation) override;
	virtual const TMap<FName, TStrongObjectPtr<const UScriptStruct>>& GetExternalPinTypes() const override;
	virtual const TMap<FName, FText>& GetExternalPinFriendlyName() const override;
	
	// FTickableGameObject interface
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; };
	virtual bool IsTickable() const override;
	virtual void Tick( float InDeltaTime ) override;
	virtual TStatId GetStatId() const override;

	// FTickableCookObject interface
	virtual void TickCook(float DeltaTime, bool bCookCompete) override;
	
private:	
	TSharedPtr<FExtensibilityManager> CustomizableObjectEditor_ToolBarExtensibilityManager;
	TSharedPtr<FExtensibilityManager> CustomizableObjectEditor_MenuExtensibilityManager;

	/** List of registered custom details to remove later. */
	TArray<FName> RegisteredCustomDetails;

	/** How many times a pin type has been registered. */
	TMap<FName, uint32> RegisteredExternalType;
	
	TMap<FName, FLinearColor> ExternalTypeTypePinColors;
	
	TMap<FName, FText> ExternalTypeFriendlyName;
	
	TMap<FName, TStrongObjectPtr<const UScriptStruct>> ExternalTypeCategories;

	virtual void OnUpstreamCOsLoaded(UCustomizableObject* Object) const;
	
	/** Register Custom details. Also adds them to RegisteredCustomDetails list. */
	void RegisterCustomDetails(FPropertyEditorModule& PropertyModule, const UClass* Class, FOnGetDetailCustomizationInstance DetailLayoutDelegate);

	FCustomizableObjectEditorLogger Logger;
	
	TSharedRef<FCustomizableObjectCompiler> Compiler = MakeShared<FCustomizableObjectCompiler>();

	// Command to look for Customizable Object Instance in the player pawn of the current world and open its Customizable Object Instance Editor
	IConsoleCommand* LaunchCOIECommand = nullptr;
	
	static void OpenCOIE(const TArray<FString>& Arguments);

	/** Register the COI factory */
	void RegisterFactory();

	bool HandleSettingsSaved();
	void RegisterSettings();
	
	/** Cook requests. */
	TMap<TWeakObjectPtr<UCustomizableObject>, TArray<TSharedRef<FCompilationRequest>>> CookCompileRequests;
	
	/** List of CustomizableObjects pending to complete loading. */
	TSet<TStrongObjectPtr<UCustomizableObject>> ObjectsPendingLoad;
};
