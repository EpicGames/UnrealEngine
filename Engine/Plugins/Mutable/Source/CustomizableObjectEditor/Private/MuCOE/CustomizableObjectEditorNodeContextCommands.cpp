// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorNodeContextCommands.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditorNodeContextCommands"

void FCustomizableObjectEditorNodeContextCommands::RegisterCommands()
{
	FKey CommentSpawnKey = FKey(TEXT("C"));
	UI_COMMAND(CreateComment, "Create a Comment node", "Create a Comment node", EUserInterfaceActionType::Button, FInputChord(CommentSpawnKey));
}
#undef LOCTEXT_NAMESPACE
