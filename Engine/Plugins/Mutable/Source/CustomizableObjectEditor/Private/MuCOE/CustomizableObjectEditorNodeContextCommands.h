// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FUICommandInfo;

/**
* Class containing commands for viewport menu actions
*/
class FCustomizableObjectEditorNodeContextCommands : public TCommands<FCustomizableObjectEditorNodeContextCommands>
{
public:
	FCustomizableObjectEditorNodeContextCommands()
		: TCommands<FCustomizableObjectEditorNodeContextCommands>
		(
			TEXT("CustomizableObjectEditorNodeContext"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "CustomizableObjectEditorNodeContext", "CustomizableObject Editor Node Context"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
			)
	{
	}

	/** Create comment*/
	TSharedPtr<FUICommandInfo> CreateComment;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};


