// Copyright Epic Games, Inc. All Rights Reserved.


#include "CustomizableObjectEditorPerformanceAnalyzer.h"

#include "SWarningOrErrorBox.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "Editor/PropertyEditor/Public/PropertyCustomizationHelpers.h"
#include "Editor/UnrealEd/Classes/ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/SCompoundWidget.h"
#include "Runtime/UMG/Public/Components/HorizontalBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Internationalization/Internationalization.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/LogBenchmarkUtil.h"
#include "MuCOE/CompilationOptions.h"
#include "MuCOE/CustomizableObjectBenchmarkingUtils.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditorPerformanceAnalyzer"

// Static vars
bool SCustomizableObjectEditorPerformanceAnalyzer::bIsPerformanceAnalyzerRunning = false;

namespace InstanceUpdatesMainDataColumns
{
	static const FName InstanceNameColumnID(TEXT("InstanceName"));
	static const FName InstanceDescriptorColumnID(TEXT("Descriptor"));
	static const FName InstanceUpdateResultColumnID(TEXT("UpdateResult"));

	static const FName InstanceTriangleCount(TEXT("TriangleCount"));

	// Initial Gen
	static const FName QueueTimeColumnID (TEXT("QueueTime"));
	static const FName UpdateTimeColumnID (TEXT("UpdateTime"));
	static const FName GetMeshTimeColumnID (TEXT("GetMeshTime"));
	static const FName GetImagesTimeColumnID (TEXT("GetImagesTime"));
	static const FName ConvertResourcesTimeColumnID (TEXT("ConvertResourcesTime"));
	static const FName CallbacksTimeColumnID (TEXT("CallbacksTime"));
	static const FName UpdatePeakMemoryColumnID (TEXT("UpdatePeakMem"));
	static const FName UpdateRealPeakMemoryColumnID (TEXT("UpdateRealPeakMem"));
	static const FName InstanceUpdateTypeID(TEXT("UpdateType"));
};


#pragma region SInstanceUpdateDataRow

void SInstanceUpdateDataRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FInstanceUpdateDataElement>& Element, const TSharedPtr<FInstanceUpdateStatistics>& Statistics)
{
	InstanceUpdateElement = Element;
	InstanceUpdateStatistics = Statistics;
	
	SMultiColumnTableRow<TSharedPtr<FInstanceUpdateDataElement>>::Construct(
		STableRow::FArguments()
		.ShowSelection(true)
		, InOwnerTableView
	);
}


void SInstanceUpdateDataRow::OnInstanceNameNavigation() const
{
	if (InstanceUpdateElement && InstanceUpdateElement->Instance)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InstanceUpdateElement->Instance.Get());
	}
}


FSlateColor TextColor(double Value, const FStatistic& Statistic)
{
	if (Value < Statistic.LowerBound)
	{
		if (Statistic.Min >= Statistic.LowerBound)
		{
			return FLinearColor::Green;
		}
		else
		{
			const float Normalized = (Value - Statistic.Min) / (Statistic.LowerBound - Statistic.Min);
			return FLinearColor::LerpUsingHSV(FLinearColor::Green, FLinearColor::Red, 1.0 - Normalized);
		}
	}
	else if (Value > Statistic.UpperBound)
	{
		if (Statistic.Max <= Statistic.UpperBound)
		{
			return FLinearColor::Green;
		}
		else
		{
			const float Normalized = (Value - Statistic.UpperBound) / (Statistic.Max - Statistic.UpperBound);
			return FLinearColor::LerpUsingHSV(FLinearColor::Green, FLinearColor::Red, Normalized);
		}
	}
	else
	{
		return FLinearColor::Green;
	}
}


TSharedRef<SWidget> SInstanceUpdateDataRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (!InstanceUpdateElement)
	{
		return SNullWidget::NullWidget;
	}
	
	if (InColumnName == InstanceUpdatesMainDataColumns::InstanceNameColumnID)
	{
		return SNew(SHyperlink)
			.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
			.Text(FText::FromString(InstanceUpdateElement->Instance->GetName()))
			.OnNavigate(this, &SInstanceUpdateDataRow::OnInstanceNameNavigation);
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::InstanceUpdateTypeID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(InstanceUpdateElement->UpdateStats.UpdateType));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::InstanceTriangleCount)
	{
		return SNew(STextBlock)
			.Text( FText::AsNumber(InstanceUpdateElement->UpdateStats.TriangleCount))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.TriangleCount, InstanceUpdateStatistics->TriangleCount));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::InstanceDescriptorColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(InstanceUpdateElement->UpdateStats.Descriptor));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::InstanceUpdateResultColumnID)
	{
		FSlateColor TextColor;
		switch (InstanceUpdateElement->UpdateStats.UpdateResult)
		{
		case EUpdateResult::Success:
			TextColor = FSlateColor(FLinearColor::Green);
			break;
		case EUpdateResult::Warning:
			TextColor = FSlateColor(FLinearColor::Yellow);
			break;
		case EUpdateResult::Error:
		case EUpdateResult::ErrorOptimized:
		case EUpdateResult::ErrorReplaced:
		case EUpdateResult::ErrorDiscarded:
		case EUpdateResult::Error16BitBoneIndex:
			TextColor = FSlateColor(FLinearColor::Red);
			break;
		}
		
		return SNew(STextBlock)
			  .Text(UEnum::GetDisplayValueAsText( InstanceUpdateElement->UpdateStats.UpdateResult))
			  .ColorAndOpacity(TextColor);
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::QueueTimeColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.QueueTime)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.QueueTime, InstanceUpdateStatistics->QueueTime));

	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::UpdateTimeColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.UpdateTime)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.UpdateTime, InstanceUpdateStatistics->UpdateTime));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::GetMeshTimeColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.TaskGetMeshesTime)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.TaskGetMeshesTime, InstanceUpdateStatistics->TaskGetMeshTime));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::GetImagesTimeColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.4f"),InstanceUpdateElement->UpdateStats.TaskGetImagesTime)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.TaskGetImagesTime, InstanceUpdateStatistics->TaskGetImagesTime));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::ConvertResourcesTimeColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.TaskConvertResourcesTime)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.TaskConvertResourcesTime, InstanceUpdateStatistics->TaskConvertResourcesTime));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::CallbacksTimeColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.TaskCallbacksTime)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.TaskCallbacksTime, InstanceUpdateStatistics->TaskCallbacksTime));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::UpdatePeakMemoryColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.UpdatePeakMemory)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.UpdatePeakMemory, InstanceUpdateStatistics->UpdatePeakMemory));
	}
	else if (InColumnName == InstanceUpdatesMainDataColumns::UpdateRealPeakMemoryColumnID)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%.2f"),InstanceUpdateElement->UpdateStats.UpdateRealPeakMemory)))
			.ColorAndOpacity(TextColor(InstanceUpdateElement->UpdateStats.UpdateRealPeakMemory, InstanceUpdateStatistics->UpdateRealPeakMemory));

	}

	// Invalid column name so no widget will be produced 
	return SNullWidget::NullWidget;
}
#pragma endregion 



// For now limit this slate to only show the generated instance data for the current Co and current compiled data
void SCustomizableObjectEditorPerformanceAnalyzer::Construct(const FArguments& InArgs)
{
	CustomizableObject = TStrongObjectPtr(InArgs._CustomizableObject);
	System = UCustomizableObjectSystem::GetInstance();
	
	// Bind ourselves to the compilation of the CO so we can extract up-to-date data from it
	FPostCompileDelegate& PostCompilationDelegate = CustomizableObject->GetPostCompileDelegate();
	PostCompilationDelegate.AddSP(this,&SCustomizableObjectEditorPerformanceAnalyzer::OnCustomizableObjectCompilationFinished);
	
	if (CustomizableObject->IsCompiled())
	{
		CacheCustomizableObjectModelData();
	}
	
	const FSlateColor StopUpdatesButtonColor = FSlateColor(FLinearColor(1,0,0));
	
	// Construct a structure that shows the compilation data for the provided Customizable Object after compiling it
	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SVerticalBox)
			
			// Show compiled model information
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(5,2)
				.Visibility(this,&SCustomizableObjectEditorPerformanceAnalyzer::GetVisibilityForBenchmarkingSettingsMessage)
				[
					SNew(SWarningOrErrorBox)
					.Message(LOCTEXT("SCustomizableObjectEditorPerformanceAnalyzerBenchmarkSettingsLabel","For a Benchmark run it is recomended to use as \"Optimization Level\" the value of \"MAXIMUM\""))
					.MessageStyle(EMessageStyle::Warning)
				]
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.Visibility(this,&SCustomizableObjectEditorPerformanceAnalyzer::GetVisibilityForBenchmarkingSettingsMessage)
			]

			// Instance generation controls 
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Bottom)
			.Padding(5,10)
			[
				SNew(SVerticalBox)
				
				// Set the amount of instances to generate per state
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot().HAlign(HAlign_Left)
					.FillWidth(1)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SCustomizableObjectEditorPerformanceAnalyzerAmountOfInstancesLabel", "Instances per State : "))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SNumericEntryBox<uint32>)
						.OnValueChanged(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnTargetAmountOfInstancesValueChange)
						.OnValueCommitted(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnTargetAmountOfInstancesValueCommited)
						.Value(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnTargetAmountOfInstancesValueRequested)
						.AllowSpin(false)
						.Delta(1)
					]
				]

				// Button to generate the instances
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
					.OnClicked(this,&SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateButtonClicked)
					.IsEnabled(this, &SCustomizableObjectEditorPerformanceAnalyzer::IsInstanceUpdateButtonEnabled)
					.Text(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetGenerateInstancesButtonText)
					.HAlign(HAlign_Center)
				]

				// Button to stop the generation of instances
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
					.OnClicked(this,&SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateAbortButtonClicked)
					.IsEnabled(this, &SCustomizableObjectEditorPerformanceAnalyzer::IsInstanceUpdateAbortButtonEnabled)
					.Visibility(this, &SCustomizableObjectEditorPerformanceAnalyzer::ShouldInstanceUpdateAbortButtonBeVisible)
					.Text(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetStopUpdatesButtonText)
					.ButtonColorAndOpacity(StopUpdatesButtonColor)
					.HAlign(HAlign_Center)
				]
			]
		]

		// Listview space
		+ SHorizontalBox::Slot()
		.FillWidth(6)
		.Padding(5,10)
		[
			SAssignNew(InstanceUpdatesListView,SListView<TSharedPtr<FInstanceUpdateDataElement>>)
			.ListItemsSource(&InstanceUpdateElements)
			.OnGenerateRow(this,&SCustomizableObjectEditorPerformanceAnalyzer::OnGenerateInstanceUpdateRow)
			.SelectionMode(ESelectionMode::Single)
			.IsFocusable(true)
			.Orientation(Orient_Vertical)
			.HeaderRow
			(
				SNew(SHeaderRow)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::InstanceNameColumnID)
				.DefaultLabel(FText(LOCTEXT("InstanceNameColumnLabel","Name")))
				.DefaultTooltip(FText(LOCTEXT("InstanceNameColumnToolTip","Customizable Object Instance name")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::InstanceNameColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.FillWidth(2)
				
				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::InstanceUpdateResultColumnID)
				.DefaultLabel(FText(LOCTEXT("InstanceUpdateResultColumnLabel", "Result")))
				.DefaultTooltip(FText(LOCTEXT("InstanceUpdateResultColumnToolTip", "Instance generation result")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::InstanceUpdateResultColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::InstanceTriangleCount)
				.DefaultLabel(FText(LOCTEXT("InstanceTriangleCountColumnLabel","Triangle Count")))
				.DefaultTooltip(FText(LOCTEXT("InstanceTriangleCountColumnToolTip","Customizable Object Instance amount of triangles generated for all Components and all Lods")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::InstanceTriangleCount)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.FillWidth(1)
				
				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::QueueTimeColumnID)
				.DefaultLabel(FText(LOCTEXT("QueueTimeColumnLabel", "Queue (ms)")))
				.DefaultTooltip(FText(LOCTEXT("QueueTimeColumnToolTip", "Time spent before starting the update")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::QueueTimeColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::UpdateTimeColumnID)
				.DefaultLabel(FText(LOCTEXT("UpdateTimeColumnLabel", "Initial Generation (ms)")))
				.DefaultTooltip(FText(LOCTEXT("UpdateTimeColumnToolTip", "Initial generation - Total time")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::UpdateTimeColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::GetMeshTimeColumnID)
				.DefaultLabel(FText(LOCTEXT("GetMeshTimeColumnLabel", "GetMesh (ms)")))
				.DefaultTooltip( FText(LOCTEXT("GetMeshTimeColumnToolTip", "Initial generation - Mesh generation time")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::GetMeshTimeColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::GetImagesTimeColumnID)
				.DefaultLabel(FText(LOCTEXT("GetImagesTimeColumnLabel", "GetImages (ms)")))
				.DefaultTooltip(FText(LOCTEXT("GetImagesTimeColumnToolTip", "Initial generation - Image generation time")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::GetImagesTimeColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::ConvertResourcesTimeColumnID)
				.DefaultLabel( FText(LOCTEXT("ConvertResourcesTimeColumnLabel", "ConvertResources (ms)")))
				.DefaultTooltip(FText(LOCTEXT("ConvertResourcesTimeColumnToolTip", "Initial generation - Time converting resources from Mutable format to target format")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::ConvertResourcesTimeColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::CallbacksTimeColumnID)
				.DefaultLabel(FText(LOCTEXT("CallbacksTimeColumnLabel", "Callbacks (ms)")))
				.DefaultTooltip(FText(LOCTEXT("CallbacksTimeColumnToolTip", "Initial generation - User attached callbacks")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::CallbacksTimeColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::UpdatePeakMemoryColumnID)
				.DefaultLabel(FText(LOCTEXT("UpdatePeakMemoryColumnLabel", "Update Peak (MB)")))
				.DefaultTooltip(FText(LOCTEXT("UpdatePeakMemoryColumnToolTip", "Maximum memory used by Mutable by the Update")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::UpdatePeakMemoryColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)

				+ SHeaderRow::Column(InstanceUpdatesMainDataColumns::UpdateRealPeakMemoryColumnID)
				.DefaultLabel(FText(LOCTEXT("UpdateRealPeakMemoryColumnLabel", "Update Real Peak (MB)")))
				.DefaultTooltip(FText(LOCTEXT("UpdateRealPeakMemoryColumnToolTip", "Maximum memory used by Mutable during the Update (Update Peak + Previously used Mutable memory)")))
				.OnSort(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort)
				.SortMode(this, &SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode, InstanceUpdatesMainDataColumns::UpdateRealPeakMemoryColumnID)
				.HAlignCell(EHorizontalAlignment::HAlign_Right)
				.FillWidth(1)
			)
		]
	];
}


SCustomizableObjectEditorPerformanceAnalyzer::~SCustomizableObjectEditorPerformanceAnalyzer()
{
	// If we are the instance that made the bIsPerformanceAnalyzerRunning true then reset it.
	if (AreUpdatesPending())
	{
		ClearPendingUpdatesQueue();
		bIsPerformanceAnalyzerRunning = false;
	}

	SetMutableBenchmarkingSystemState(false);
}


EVisibility SCustomizableObjectEditorPerformanceAnalyzer::GetVisibilityForBenchmarkingSettingsMessage() const
{
	if (LastCompilationOptions)
	{
		if (LastCompilationOptions->OptimizationLevel == CustomizableObjectBenchmarkingUtils::GetOptimizationLevelForBenchmarking())
		{
			return EVisibility::Collapsed;
		}
		else
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}


void SCustomizableObjectEditorPerformanceAnalyzer::OnTargetAmountOfInstancesValueChange(uint32 NewValue)
{
	if (NewValue > 0)
	{
		InstancesToGeneratePerState = NewValue;
	}
}


void SCustomizableObjectEditorPerformanceAnalyzer::OnTargetAmountOfInstancesValueCommited(uint32 NewValue, ETextCommit::Type CommitType)
{
	if (NewValue > 0)
	{
		InstancesToGeneratePerState = NewValue;
	}
	else
	{
		InstancesToGeneratePerState = 24;
	}
}


TOptional<uint32> SCustomizableObjectEditorPerformanceAnalyzer::OnTargetAmountOfInstancesValueRequested() const
{
	return InstancesToGeneratePerState;
}


bool SCustomizableObjectEditorPerformanceAnalyzer::IsInstanceUpdateButtonEnabled() const
{
	return !bIsPerformanceAnalyzerRunning && CustomizableObject && CustomizableObject->IsCompiled() && !AreUpdatesPending();
}


FReply SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateButtonClicked()
{
	check(CustomizableObject);

	// Ensure there is no work to do
	check(!AreUpdatesPending());
		
	InstanceUpdateElements.Reset();
	InstanceUpdatesListView->RequestListRefresh();
	
	UpdatedInstancesCount = 0;
	TotalScheduledUpdates = 0;

	// Set the CVArs as in CIS
	//SetForBenchmarkMutableCVarValues();
		
	if (!CustomizableObjectBenchmarkingUtils::GenerateDeterministicSetOfInstances(*CustomizableObject ,InstancesToGeneratePerState,InstancesToUpdate,TotalScheduledUpdates))
	{
		UE_LOGF(LogMutable,Error, "Mutable Customizable Object Instance generation failed. Aborting perf test.")
		return FReply::Handled();
	}
	check (!InstancesToUpdate.IsEmpty())

	// tell the system a test is already running so no new test can be invoked while that is being done
	bIsPerformanceAnalyzerRunning = true;
	ScheduleNextInstanceUpdate();
	
	return FReply::Handled();
}


FText SCustomizableObjectEditorPerformanceAnalyzer::GetGenerateInstancesButtonText() const
{
	if (CustomizableObject && !CustomizableObject->IsCompiled())
	{
		return FText::FromString(TEXT("CO is not compiled"));
	}
	
	if (bIsPerformanceAnalyzerRunning && !AreUpdatesPending())
	{
		return FText::FromString(TEXT("Another instance is running"));
	}

	if (!AreUpdatesPending())
	{
		return FText::FromString(TEXT("Generate Random Instances"));
	}
	else
	{
		return FText::FromString(FString::Printf(TEXT("Updated %u / %u instances"),UpdatedInstancesCount, TotalScheduledUpdates));;
	}
}


EVisibility SCustomizableObjectEditorPerformanceAnalyzer::ShouldInstanceUpdateAbortButtonBeVisible() const
{
	return  AreUpdatesPending() ? EVisibility::Visible :  EVisibility::Hidden;
}


FText SCustomizableObjectEditorPerformanceAnalyzer::GetStopUpdatesButtonText() const
{
	// Already cleared the pending updates and we are waiting for the last one
	if (InstancesToUpdate.IsEmpty() && CurrentInstance)
	{
		// Stopping
		return FText::FromString(TEXT("STOPPING..."));
	}
	else
	{
		return FText::FromString(TEXT("STOP"));
	}
}


FReply SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateAbortButtonClicked()
{
	ClearPendingUpdatesQueue();
	bIsPerformanceAnalyzerRunning = false;
	
	// The update finish of the last instance to be updated will take care of clearing the update data (like cvars)
	return FReply::Handled();
}


bool SCustomizableObjectEditorPerformanceAnalyzer::IsInstanceUpdateAbortButtonEnabled() const
{
	// Allow the abortion of the test if one slate is running the test and that slate is us (since we have Instances to yet update)
	return !InstancesToUpdate.IsEmpty();
}


void SCustomizableObjectEditorPerformanceAnalyzer::ClearPendingUpdatesQueue()
{
	while(InstancesToUpdate.Dequeue()) {}
}


void SCustomizableObjectEditorPerformanceAnalyzer::CacheCustomizableObjectModelData()
{
	if (!CustomizableObject)
	{
		return;
	}
	
	// Store the compilation options once the compilation has been performed. This way we should be able to compare the options
	// with what we know are the CIS ones
	LastCompilationOptions = MakeShared<FCompilationOptions>(GetCompilationOptions(*CustomizableObject));
}


void SCustomizableObjectEditorPerformanceAnalyzer::OnCustomizableObjectCompilationFinished()
{
	check(CustomizableObject);

	// If updates are pending, since only one instance of this slate can run, mark the execution status of this instance
	// as halted. Only do this for instances of the slate where the execution has been possible.
	if (AreUpdatesPending())
	{
		bIsPerformanceAnalyzerRunning = false;
	}
	
	// Clear any old data from previous compilation operation
	//		This will not always happen since we prevent the compilation of the CO if it has instances being updated
	//		if no CO is actually compiled then this method will not be called
	{
		ClearPendingUpdatesQueue();
		CurrentInstance = nullptr;
		if (!InstanceUpdateElements.IsEmpty())
		{
			// Clear the list of update data
			InstanceUpdateElements.Empty();
			InstanceUpdatesListView->RequestListRefresh();
		}
	}
	
	CacheCustomizableObjectModelData();
}


void SCustomizableObjectEditorPerformanceAnalyzer::SetMutableBenchmarkingSystemState(const bool bNewState)
{
	// Tell the benchmarking system that there is no need to continue benchmarking.
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(bNewState);

	// Setting this to false will allow for the system to retrieve the values set by CVars and other non code ways
	UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(bNewState);
}


void SCustomizableObjectEditorPerformanceAnalyzer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(System);
}


FString SCustomizableObjectEditorPerformanceAnalyzer::GetReferencerName() const
{
	return FString(TEXT("SCustomizableObjectEditorPerformanceAnalyzer"));
}


EColumnSortMode::Type SCustomizableObjectEditorPerformanceAnalyzer::GetColumnSortMode(FName ColumnName) const
{
	if (CurrentSortColumn != ColumnName)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}


TSharedRef<ITableRow> SCustomizableObjectEditorPerformanceAnalyzer::OnGenerateInstanceUpdateRow(TSharedPtr<FInstanceUpdateDataElement> InstanceUpdateDataElement, const TSharedRef<STableViewBase>& TableViewBase)
{
	TSharedRef<SInstanceUpdateDataRow> Row = SNew(SInstanceUpdateDataRow, TableViewBase, InstanceUpdateDataElement, InstanceUpdateStatistics);
	return Row;
}


void SCustomizableObjectEditorPerformanceAnalyzer::OnInstanceUpdateListViewSort(EColumnSortPriority::Type ColumnPriority, const FName& ColumnId, EColumnSortMode::Type NewSortMode)
{
	CurrentSortColumn = ColumnId;
	SortMode = NewSortMode;

	if (ColumnId == InstanceUpdatesMainDataColumns::InstanceNameColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				return NewSortMode == EColumnSortMode::Ascending ? ElementA->UpdateIndex < ElementB->UpdateIndex : ElementA->UpdateIndex > ElementB->UpdateIndex;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::InstanceUpdateResultColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const FString NameA = UEnum::GetValueAsString(ElementA->UpdateStats.UpdateResult);
				const FString NameB =  UEnum::GetValueAsString(ElementB->UpdateStats.UpdateResult);

				return NewSortMode == EColumnSortMode::Ascending ? NameA < NameB : NameA > NameB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::QueueTimeColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.QueueTime;
				const double ValB =  ElementB->UpdateStats.QueueTime;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::InstanceTriangleCount)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const uint32 ValA = ElementA->UpdateStats.TriangleCount;
				const uint32 ValB =  ElementB->UpdateStats.TriangleCount;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::UpdateTimeColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.UpdateTime;
				const double ValB =  ElementB->UpdateStats.UpdateTime;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::GetMeshTimeColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.TaskGetMeshesTime;
				const double ValB =  ElementB->UpdateStats.TaskGetMeshesTime;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::GetImagesTimeColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.TaskGetImagesTime;
				const double ValB =  ElementB->UpdateStats.TaskGetImagesTime;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::ConvertResourcesTimeColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.TaskConvertResourcesTime;
				const double ValB =  ElementB->UpdateStats.TaskConvertResourcesTime;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::CallbacksTimeColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.TaskCallbacksTime;
				const double ValB =  ElementB->UpdateStats.TaskCallbacksTime;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::UpdatePeakMemoryColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.UpdatePeakMemory;
				const double ValB =  ElementB->UpdateStats.UpdatePeakMemory;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::UpdateRealPeakMemoryColumnID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const double ValA = ElementA->UpdateStats.UpdateRealPeakMemory;
				const double ValB =  ElementB->UpdateStats.UpdateRealPeakMemory;

				return NewSortMode == EColumnSortMode::Ascending ? ValA < ValB : ValA > ValB;
			}

			return ElementA.IsValid();
		});
	}
	else if (ColumnId == InstanceUpdatesMainDataColumns::InstanceUpdateTypeID)
	{
		InstanceUpdateElements.StableSort([&](const TSharedPtr<FInstanceUpdateDataElement>& ElementA, const TSharedPtr<FInstanceUpdateDataElement>& ElementB)
		{
			if (ElementA.IsValid() && ElementB.IsValid())
			{
				const FString NameA = ElementA->UpdateStats.UpdateType;
				const FString NameB =  ElementB->UpdateStats.UpdateType;

				return NewSortMode == EColumnSortMode::Ascending ? NameA < NameB : NameA > NameB;
			}

			return ElementA.IsValid();
		});
	}
	else
	{
		check(false); // Unknown method.
	}

	InstanceUpdatesListView->RequestListRefresh();
}


void SCustomizableObjectEditorPerformanceAnalyzer::OnBenchmarkMeshUpdated(TSharedRef<FUpdateContextPrivate> UpdateContextPrivate, FInstanceUpdateStats UpdateStats)
{
	if (CurrentInstance.Get() != UpdateContextPrivate->Instance)
	{
		return;
	}
	
	// Create a new element for the mesh update data for this instance
	TSharedPtr<FInstanceUpdateDataElement> InstanceElement = MakeShared<FInstanceUpdateDataElement>();
	InstanceElement->Instance = CurrentInstance;
	InstanceElement->UpdateStats = UpdateStats;
	InstanceElement->UpdateIndex = UpdatedInstancesCount++;

	// 3) Store the element to keep the instance alive
	check(InstanceElement);
	InstanceUpdateElements.Push(InstanceElement);

	ComputeStatistics();
	
	// Reset control flags for the next instance
	CurrentInstance = nullptr;
	
	// And once the updates have completed update the list view
	check(InstanceUpdatesListView);
	InstanceUpdatesListView->RebuildList();
	
	// Stop listening for instance update reported data
	System->GetPrivate()->LogBenchmarkUtil.OnMeshUpdateReported.Remove(OnMeshUpdatePerfReportDelegateHandle);

	// Handle here the waiting for the update of the next instances
	if (!InstancesToUpdate.IsEmpty())
	{
		ScheduleNextInstanceUpdate();
	}
	else
	{
		SetMutableBenchmarkingSystemState(false);
		bIsPerformanceAnalyzerRunning = false;
	}
}


void SCustomizableObjectEditorPerformanceAnalyzer::ScheduleNextInstanceUpdate()
{
	// We always expect to find an instance here after the Dequeue operation
	InstancesToUpdate.Dequeue(CurrentInstance);
	check(CurrentInstance)
	
	// Enable the benchmarking support mutable systems
	SetMutableBenchmarkingSystemState(true);

	// Request the actual update
	FInstanceUpdateNativeDelegate InstanceUpdateNativeDelegate;
	CurrentInstance->UpdateSkeletalMeshAsyncResult(InstanceUpdateNativeDelegate, true, true);

	// Bind our listView update to the update of the benchmarking tool
	OnMeshUpdatePerfReportDelegateHandle = System->GetPrivate()->LogBenchmarkUtil.OnMeshUpdateReported.AddSP(this, &SCustomizableObjectEditorPerformanceAnalyzer::OnBenchmarkMeshUpdated);
}


bool SCustomizableObjectEditorPerformanceAnalyzer::AreUpdatesPending() const
{
	return CurrentInstance || !InstancesToUpdate.IsEmpty();
}


void SCustomizableObjectEditorPerformanceAnalyzer::ComputeStatistics()
{
	const int32 Num = InstanceUpdateElements.Num();

	auto Compute = [this, Num](FStatistic& Statistic, const TFunction<double(int32)>& GetValue)
	{
		Statistic = {};

		if (!Num)
		{
			return;
		}
			
		TArray<double> Values;
		Values.Reserve(Num);
		for (int32 Index = 0; Index < Num; ++Index)
		{
			Values.Add(GetValue(Index));
		}

		Values.Sort();

		Statistic.Min = Values[0];
		Statistic.Max = Values.Last();

		const double Q1 = Values[static_cast<int32>(Num * 0.25)];
		const double Q3 = Values[static_cast<int32>(Num * 0.75)];
		const double IQR = Q3 - Q1;
			
		Statistic.LowerBound = Q1 - 1.5 * IQR;
		Statistic.UpperBound = Q3 + 1.5 * IQR;
	};
	
	Compute(InstanceUpdateStatistics->TriangleCount, [&](int32 Index){	return InstanceUpdateElements[Index]->UpdateStats.TriangleCount; });
	Compute(InstanceUpdateStatistics->QueueTime, [&](int32 Index){	return InstanceUpdateElements[Index]->UpdateStats.QueueTime; });
	Compute(InstanceUpdateStatistics->UpdateTime, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.UpdateTime; });
	Compute(InstanceUpdateStatistics->TaskGetMeshTime, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskGetMeshesTime; });
	Compute(InstanceUpdateStatistics->TaskGetImagesTime, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskGetImagesTime; });
	Compute(InstanceUpdateStatistics->TaskConvertResourcesTime, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskConvertResourcesTime; });
	Compute(InstanceUpdateStatistics->TaskCallbacksTime, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskCallbacksTime; });
	Compute(InstanceUpdateStatistics->UpdatePeakMemory, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.UpdatePeakMemory; });
	Compute(InstanceUpdateStatistics->UpdateRealPeakMemory, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.UpdateRealPeakMemory; });
	Compute(InstanceUpdateStatistics->TaskUpdateImageTime, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskUpdateImageTime; });
	Compute(InstanceUpdateStatistics->TaskUpdateImagePeakMemory, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskUpdateImagePeakMemory; });
	Compute(InstanceUpdateStatistics->TaskUpdateImageRealPeakMemory, [&](int32 Index) { return InstanceUpdateElements[Index]->UpdateStats.TaskUpdateImageRealPeakMemory; });
}

#undef LOCTEXT_NAMESPACE
