// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/SpscQueue.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/LogBenchmarkUtil.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"



struct FCompilationRequest;
struct FCompilationOptions;
struct InstanceUpdateData;
class SObjectPropertyEntryBox;
class UCustomizableObject;
class UCustomizableObjectInstance;
class FCustomizableObjectEditor;
class FCustomizableObjectInstanceEditor;
class UCustomizableObjectSystem;
enum class ECustomizableObjectTextureCompression : uint8;

/**
 * Backend object that represents the update of one instance. Currently only represents the data for the initial generation of the instance update
 */
class FInstanceUpdateDataElement 
{
public:
	/** Instance whose data this object exposes */
	TStrongObjectPtr<UCustomizableObjectInstance> Instance = nullptr;

	/** Index to aid in the sorting of the FInstanceUpdateDataElement in the SListView that contains them */
	uint32 UpdateIndex = 0;
	
	/** Container with all the perf data in relation to the instance this object represents */
	FInstanceUpdateStats UpdateStats;
};


/**
 * Slate that exposes the perf data of one single instance update.
 */
class SInstanceUpdateDataRow : public SMultiColumnTableRow<TSharedPtr<FInstanceUpdateDataElement>>
{
public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FInstanceUpdateDataElement>& Element, const TSharedPtr<FInstanceUpdateStatistics>& Statistics);
	
private:
	/** Backend object whose data we are drawing */
	TSharedPtr<FInstanceUpdateDataElement> InstanceUpdateElement;
	TSharedPtr<FInstanceUpdateStatistics> InstanceUpdateStatistics;

	/** Method invoked when the hyperlink showing the name of the instance is clicked */
	void OnInstanceNameNavigation() const;
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
};


/**
 * Slate that exposes perf data of a deterministic set of instances generated and then updated in the context of the editor.
 * Will update the values of some CVARS that will, at the end, be restored.
 */
class SCustomizableObjectEditorPerformanceAnalyzer : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectEditorPerformanceAnalyzer){}
		SLATE_ARGUMENT(UCustomizableObject*,CustomizableObject)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SCustomizableObjectEditorPerformanceAnalyzer() override;

private:

	// FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	// FGCObject
	
	EVisibility GetVisibilityForBenchmarkingSettingsMessage() const;

	/** Instance amount to generate and update controls */
	void OnTargetAmountOfInstancesValueChange(uint32 NewValue);
	void OnTargetAmountOfInstancesValueCommited(uint32 NewValue, ETextCommit::Type CommitType);
	TOptional<uint32> OnTargetAmountOfInstancesValueRequested() const;

	/** Random Instances generation and Update button */
	FReply OnInstanceUpdateButtonClicked();
	bool IsInstanceUpdateButtonEnabled() const;
	FText GetGenerateInstancesButtonText() const;

	/** Methods for the button that allows the user to cancel the execution of the test */
	FReply OnInstanceUpdateAbortButtonClicked();
	EVisibility ShouldInstanceUpdateAbortButtonBeVisible() const;
	bool IsInstanceUpdateAbortButtonEnabled() const;
	FText GetStopUpdatesButtonText() const;

	/** Utility methods */
	void ClearPendingUpdatesQueue();

	/** Compilation handling */
	void OnCustomizableObjectCompilationFinished();
	void CacheCustomizableObjectModelData();

	/**
	 * Enable or disable the mutable benchmarking settings.
	 * @param bNewState Set it to true to enable the benchmarking configurations and false to disable them.
	 */
	void SetMutableBenchmarkingSystemState(const bool bNewState);
	
	/** ListView methods */
	TSharedRef<ITableRow> OnGenerateInstanceUpdateRow(TSharedPtr<FInstanceUpdateDataElement> InstanceUpdateDataElement, const TSharedRef<STableViewBase>& TableViewBase);
	void OnInstanceUpdateListViewSort(EColumnSortPriority::Type ColumnPriority, const FName& ColumnId, EColumnSortMode::Type NewSortMode);
	EColumnSortMode::Type GetColumnSortMode(FName ColumnName) const;

	/**
	 * Callback invoked each time the LogBenchmarkUtil notifies us that an initial generation update (mesh Update) has been produced
	 * @param UpdateContextPrivate The context of the instance that got updated
	 * @param UpdateStats The Perf data of the instance that was updated.
	 */
	void OnBenchmarkMeshUpdated(TSharedRef<FUpdateContextPrivate> UpdateContextPrivate, FInstanceUpdateStats UpdateStats);

	/** Grab the next available instance to be updated*/
	void ScheduleNextInstanceUpdate();

	/** Determines if there are updates to wait for. Checks not only the queue but also the current instnace being udpated*/
	bool AreUpdatesPending() const;
	
	void ComputeStatistics();
	
	/** Delegate pointing to the LogBenchmarkUtil on Mesh Updated event */
	FDelegateHandle OnMeshUpdatePerfReportDelegateHandle;

	/** Is this slate running? Used to determine if another instance of the slate is performing the test */
	static bool bIsPerformanceAnalyzerRunning;

	/** The amount of instances per state we want to generate and then update */
	uint32 InstancesToGeneratePerState = 24;

	/** Cached Customizable Object System object */
	TObjectPtr<UCustomizableObjectSystem> System;
	
	/** CO whose data we are we reporting. */
	TStrongObjectPtr<UCustomizableObject> CustomizableObject;
	
	/** List view showing all the instance update data elements */
	TSharedPtr<SListView<TSharedPtr<FInstanceUpdateDataElement>>> InstanceUpdatesListView;
	
	/** Instance update elements (size == RandomInstancesToGenerate) at the time of the update request */
	TArray<TSharedPtr<FInstanceUpdateDataElement>> InstanceUpdateElements;

	TSharedPtr<FInstanceUpdateStatistics> InstanceUpdateStatistics = MakeShared<FInstanceUpdateStatistics>();
	
	/** The name of the last column the user wanted to sort */
	FName CurrentSortColumn;
	/** The sort type applied to the last column that got sorted */
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::None;

	/** Updating process variables */
	TSpscQueue<TStrongObjectPtr<UCustomizableObjectInstance>> InstancesToUpdate;
	/** Current instance being updated */
	TStrongObjectPtr<UCustomizableObjectInstance> CurrentInstance = nullptr;
	
	/** Control counters (UI) */
	uint32 UpdatedInstancesCount = 0;
	uint32 TotalScheduledUpdates = 0;

	/** The compilation options the last time we compiled the CO */
	TSharedPtr<FCompilationOptions> LastCompilationOptions = nullptr;
};
