// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObjectSystemPrivate.h"

#include "CustomizableObjectEditorSettings.generated.h"

enum class ECustomizableObjectDDCPolicy : uint8;


/** COE Module, Editor Settings. */
UCLASS(config = EditorPerProjectUserSettings)
class UCustomizableObjectEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** If true, Mutable won't compile any COs in the Editor. */
	UPROPERTY(config, EditAnywhere, Category = Compilation)
	bool bDisableMutableCompileInEditor;

	/**	If true, Mutable will automatically compile, if needed, COs being used by Actors. */
	UPROPERTY(config, EditAnywhere, Category = AutomaticCompilation)
	bool bEnableAutomaticCompilation = true;

	/**	If true, AutomaticCompilation will happen synchronously. */
	UPROPERTY(config, EditAnywhere, Category = AutomaticCompilation)
	bool bCompileObjectsSynchronously = false;

	UPROPERTY(config, EditAnywhere, Category = Developer)
	bool bEnableDeveloperOptions = false;
	
	UPROPERTY(config, EditAnywhere, Category = DerivedDataCache, DisplayName = "DDC policy for editor compilations")
	ECustomizableObjectDDCPolicy EditorDerivedDataCachePolicy = ECustomizableObjectDDCPolicy::Default;
	
	UPROPERTY(config, EditAnywhere, Category = DerivedDataCache, DisplayName = "DDC policy for cook compilations")
	ECustomizableObjectDDCPolicy CookDerivedDataCachePolicy = ECustomizableObjectDDCPolicy::Default;
};
