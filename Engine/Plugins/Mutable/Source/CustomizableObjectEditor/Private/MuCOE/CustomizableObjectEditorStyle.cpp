// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"


TSharedPtr<FSlateStyleSet> FCustomizableObjectEditorStyle::CustomizableObjectEditorStyleInstance;


void FCustomizableObjectEditorStyle::Initialize()
{
	// Only register once
	if (!CustomizableObjectEditorStyleInstance.IsValid())
	{
		CustomizableObjectEditorStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*CustomizableObjectEditorStyleInstance);
	}
}


void FCustomizableObjectEditorStyle::Shutdown()
{
	if (CustomizableObjectEditorStyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*CustomizableObjectEditorStyleInstance.Get());
		ensure(CustomizableObjectEditorStyleInstance.IsUnique());
		CustomizableObjectEditorStyleInstance.Reset();
	}
}


const ISlateStyle& FCustomizableObjectEditorStyle::Get()
{
	return *CustomizableObjectEditorStyleInstance;
}


FName FCustomizableObjectEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("CustomizableObjectEditorStyle"));
	return StyleSetName;
}


FString FCustomizableObjectEditorStyle::RelativePathToPluginPath(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("Mutable"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}


#define IMAGE_PLUGIN_BRUSH(Style, RelativePath, ... ) FSlateImageBrush(Style->RootToContentDir( RelativePath, TEXT(".png")), __VA_ARGS__)

#define IMAGE_BRUSH(Style, RelativePath, ... ) FSlateImageBrush(Style->RootToCoreContentDir( RelativePath, TEXT(".png")), __VA_ARGS__)
#define IMAGE_BRUSH_SVG( Style, RelativePath, ... ) FSlateVectorImageBrush(Style->RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)


TSharedRef<FSlateStyleSet> FCustomizableObjectEditorStyle::Create()
{
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("CustomizableObjectEditorStyle"));
	Style->SetCoreContentRoot(FPaths::EngineContentDir());
	Style->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Mutable/Content"));
	
	Style->Set("CustomizableObjectEditor.Save", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/SaveCurrent", Icon40x40));
	Style->Set("CustomizableObjectEditor.Save.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/SaveCurrent", Icon20x20));
	Style->Set("CustomizableObjectEditor.Compile", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon40x40));
	Style->Set("CustomizableObjectEditor.Compile.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon20x20));
	Style->Set("CustomizableObjectEditor.CompileOnlySelected", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon40x40));
	Style->Set("CustomizableObjectEditor.CompileOnlySelected.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon20x20));
	Style->Set("CustomizableObjectEditor.ResetCompileOptions", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon40x40));
	Style->Set("CustomizableObjectEditor.ResetCompileOptions.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon20x20));
	Style->Set("CustomizableObjectEditor.GraphViewer", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/BlueprintDebugger", Icon40x40));
	Style->Set("CustomizableObjectEditor.GraphViewer.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/BlueprintDebugger", Icon20x20));
	Style->Set("CustomizableObjectEditor.CodeViewer", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Debug", Icon40x40));
	Style->Set("CustomizableObjectEditor.CodeViewer.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Debug", Icon20x20));
	Style->Set("CustomizableObjectEditor.TextureAnalyzer", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/IssueTracker", Icon40x40));
	Style->Set("CustomizableObjectEditor.TextureAnalyzer.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/IssueTracker", Icon20x20));
	Style->Set("CustomizableObjectEditor.PerformanceReport", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/simulate", Icon40x40));
	Style->Set("CustomizableObjectEditor.PerformanceReport.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/simulate", Icon20x20));
	Style->Set("CustomizableObjectEditor.PerformanceAnalyzer", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/simulate", Icon40x40));
	Style->Set("CustomizableObjectEditor.PerformanceAnalyzer.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/simulate", Icon20x20));

	Style->Set("CustomizableObjectEditor.CompileGatherReferences", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon40x40));
	Style->Set("CustomizableObjectEditor.CompileGatherReferences.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/compile", Icon20x20));
	Style->Set("CustomizableObjectEditor.ClearGatheredReferences", new IMAGE_BRUSH_SVG(Style, "Slate/Starship/Common/Delete", Icon40x40));
	Style->Set("CustomizableObjectEditor.ClearGatheredReferences.Small", new IMAGE_BRUSH_SVG(Style, "Slate/Starship/Common/Delete", Icon20x20));
	
	Style->Set("CustomizableObjectEditor.Tabs.Preview", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Persona/PersonaPreviewAnimation", Icon20x20));
	Style->Set("CustomizableObjectEditor.Tabs.PreviewSettings", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/AssetIcons/CameraActor_64", Icon20x20));
	Style->Set("CustomizableObjectEditor.Tabs.CustomizableObjectProperties", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Persona/SectionSelection", Icon20x20));
	Style->Set("CustomizableObjectEditor.Tabs.CustomizableInstanceProperties", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Details", Icon20x20));
	Style->Set("CustomizableObjectEditor.Tabs.NodeGraph", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/blueprint", Icon20x20));
	Style->Set("CustomizableObjectEditor.Tabs.NodeProperties", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Blueprints/icon_BlueprintEditor_EventGraph", Icon20x20));

	Style->Set("CustomizableObjectInstanceEditor.SetShowWireframe", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Wireframe_40x", Icon40x40));
	Style->Set("CustomizableObjectInstanceEditor.SetShowWireframe.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Wireframe_40x", Icon20x20));

	Style->Set("CustomizableObjectInstanceEditor.Save", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/SaveCurrent", Icon40x40));
	Style->Set("CustomizableObjectInstanceEditor.Save.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/SaveCurrent", Icon20x20));
	Style->Set("CustomizableObjectInstanceEditor.ShowParentCO", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/eject", Icon40x40));
	Style->Set("CustomizableObjectInstanceEditor.ShowParentCO.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/eject", Icon20x20));
	Style->Set("CustomizableObjectInstanceEditor.EditParentCO", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/EditorModes", Icon40x40));
	Style->Set("CustomizableObjectInstanceEditor.EditParentCO.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/EditorModes", Icon20x20));
	Style->Set("CustomizableObjectInstanceEditor.TextureAnalyzer", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/IssueTracker", Icon40x40));
	Style->Set("CustomizableObjectInstanceEditor.TextureAnalyzer.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/IssueTracker", Icon20x20));

	Style->Set("CustomizableObjectEditorViewport.SetShowWireframe", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Wireframe_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetShowWireframe.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Wireframe_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetRealtimePreview", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_MatEd_Realtime_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetRealtimePreview.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_MatEd_Realtime_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetShowBounds", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Bounds_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetShowBounds.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Bounds_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetShowNormals", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Normals_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetShowNormals.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Normals_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetShowTangents", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Tangents_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetShowTangents.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Tangents_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetShowBinormals", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Binormals_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetShowBinormals.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_Binormals_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SaveThumbnail", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_SaveThumbnail_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SaveThumbnail.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_SaveThumbnail_40x", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetShowPivot", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_ShowPivot_40x", Icon40x40));
	Style->Set("CustomizableObjectEditorViewport.SetShowPivot.Small", new IMAGE_BRUSH(Style, "Editor/Slate/Icons/icon_StaticMeshEd_ShowPivot_40x", Icon20x20));

	Style->Set("CustomizableObjectEditorViewport.SetShowGrid", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Grid", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetShowSky", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Reflections", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.SetDrawUVs", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/SetDrawUVs", Icon20x20));
	Style->Set("CustomizableObjectEditorViewport.BakeInstance", new IMAGE_PLUGIN_BRUSH(Style, "Icons/BakerIcon", Icon40x40));

	Style->Set("COEditorViewportLODCommands.TranslateMode", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/EditorViewport/translate", Icon20x20));
	Style->Set("COEditorViewportLODCommands.RotateMode", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/EditorViewport/rotate", Icon20x20));
	Style->Set("COEditorViewportLODCommands.ScaleMode", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/EditorViewport/scale", Icon20x20));
	Style->Set("COEditorViewportLODCommands.RotationGridSnap", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/EditorViewport/angle", Icon20x20));

	Style->Set("LayoutEditorCommands.AddBlock", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Layout", Icon40x40));
	Style->Set("LayoutEditorCommands.AddBlock.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/Layout", Icon20x20));
	Style->Set("LayoutEditorCommands.RemoveBlock", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/LayoutRemove", Icon40x40));
	Style->Set("LayoutEditorCommands.RemoveBlock.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/LayoutRemove", Icon20x20));
	Style->Set("LayoutEditorCommands.GenerateBlocks", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Persona/AnimationPreviewMesh", Icon40x40));
	Style->Set("LayoutEditorCommands.GenerateBlocks.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Persona/AnimationPreviewMesh", Icon20x20));

	Style->Set("LayoutBlockSelector.SelectAll", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/LayoutLoad", Icon40x40));
	Style->Set("LayoutBlockSelector.SelectAll.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/LayoutLoad", Icon20x20));
	Style->Set("LayoutBlockSelector.SelectNone", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/LayoutSave", Icon40x40));
	Style->Set("LayoutBlockSelector.SelectNone.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/LayoutSave", Icon20x20));

	Style->Set("Nodes.ArrowUp", new IMAGE_BRUSH(Style, "Editor/Slate/Old/ArrowUp", Icon16x16));
	Style->Set("Nodes.ArrowDown", new IMAGE_BRUSH(Style, "Editor/Slate/Old/ArrowDown", Icon16x16));

	Style->Set("CustomizableObjectDebugger.GenerateMutableGraph", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/blueprint", Icon40x40));
	Style->Set("CustomizableObjectDebugger.GenerateMutableGraph.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/Common/blueprint", Icon20x20));
	Style->Set("CustomizableObjectDebugger.CompileMutableCode", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/Compile", Icon40x40));
	Style->Set("CustomizableObjectDebugger.CompileMutableCode.Small", new IMAGE_BRUSH_SVG(Style, "Editor/Slate/Starship/MainToolbar/Compile", Icon20x20));
	
	FTableRowStyle PerformanceReportRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
	PerformanceReportRowStyle.SetEvenRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.AccentBrown"));
	PerformanceReportRowStyle.SetOddRowBackgroundBrush(*FAppStyle::Get().GetBrush("Brushes.AccentGray"));

	Style->Set("CustomizableObjectPerformanceReport.Row", PerformanceReportRowStyle);

	FTableRowStyle PerformanceReportAllEvenRowStyle = PerformanceReportRowStyle;
	PerformanceReportAllEvenRowStyle.SetOddRowBackgroundBrush(PerformanceReportRowStyle.EvenRowBackgroundBrush);

	Style->Set("CustomizableObjectPerformanceReportAllEven.Row", PerformanceReportAllEvenRowStyle);

	FTableRowStyle PerformanceReportAllOddRowStyle = PerformanceReportRowStyle;
	PerformanceReportAllOddRowStyle.SetEvenRowBackgroundBrush(PerformanceReportRowStyle.OddRowBackgroundBrush);

	Style->Set("CustomizableObjectPerformanceReportAllOdd.Row", PerformanceReportAllOddRowStyle);

	return Style;
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH
#undef IMAGE_BRUSH_SVG
