// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorUtilities.h"

#include "UObject/Object.h"
#include "UObject/LinkerLoad.h"


bool CompareNames(const TSharedPtr<FString>& sp1, const TSharedPtr<FString>& sp2)
{
	if (const FString* s1 = sp1.Get())
	{
		if (const FString* s2 = sp2.Get())
		{
			return (s1->Compare(*s2, ESearchCase::IgnoreCase) < 0);
		}
		else
		{
			return false;
		}
	}
	else
	{
		return true;
	}
}


void ConditionalPostLoadReference(UObject& Object)
{
	Object.ConditionalPreload();
	Object.ConditionalPostLoad();
}
