// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UObject;


bool CompareNames(const TSharedPtr<FString>& sp1, const TSharedPtr<FString>& sp2);

void ConditionalPostLoadReference(UObject& Object);
