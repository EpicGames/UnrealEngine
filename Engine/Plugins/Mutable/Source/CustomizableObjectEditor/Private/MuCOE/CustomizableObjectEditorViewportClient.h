// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectParameterTypeDefinitions.h"

namespace EAnimationMode { enum Type : int; }

class FCanvas;
class FMaterialRenderProxy;
class FPreviewScene;
class FPrimitiveDrawInterface;
class FReferenceCollector;
class FSceneView;
class FViewport;
class ICustomizableObjectInstanceEditor;
class UAnimationAsset;
class UCustomizableObject;
class UCustomizableObjectNode;
class UCustomizableObjectNodeModifierClipMorph;
class UCustomizableObjectNodeProjectorConstant;
class UCustomizableObjectNodeProjectorParameter;
class UDebugSkelMeshComponent;
class ULightComponent;
class UMaterial;
class UMaterialInterface;
class UObject;
class UStaticMesh;
class UStaticMeshComponent;
struct FInputKeyEventArgs;
struct FGizmoState;

DECLARE_DELEGATE_RetVal(FVector, FWidgetLocationDelegate);
DECLARE_DELEGATE_OneParam(FOnWidgetLocationChangedDelegate, const FVector&)

DECLARE_DELEGATE_RetVal(FVector, FWidgetDirectionDelegate);
DECLARE_DELEGATE_OneParam(FOnWidgetDirectionChangedDelegate, const FVector&)

DECLARE_DELEGATE_RetVal(FVector, FWidgetUpDelegate);
DECLARE_DELEGATE_OneParam(FOnWidgetUpChangedDelegate, const FVector&)

DECLARE_DELEGATE_RetVal(FVector, FWidgetScaleDelegate);
DECLARE_DELEGATE_OneParam(FOnWidgetScaleChangedDelegate, const FVector&)

DECLARE_DELEGATE_RetVal(float, FWidgetAngleDelegate);

DECLARE_DELEGATE_RetVal(ECustomizableObjectProjectorType, FProjectorTypeDelegate);

DECLARE_DELEGATE_RetVal(FColor, FWidgetColorDelegate);

DECLARE_DELEGATE(FWidgetTrackingStartedDelegate);




enum class EWidgetType : uint8
{
	Hidden,
	Projector,
	ClipMorph,
	ClipMesh,
	Light,
};


namespace EMutableAnimationPlaybackSpeeds
{
	enum Type
	{
		OneTenth = 0,
		Quarter,
		Half,
		ThreeQuarters,
		Normal,
		Double,
		FiveTimes,
		TenTimes,
		Custom,
		NumPlaybackSpeeds
	};

	extern float Values[NumPlaybackSpeeds];
}


/** Viewport which shows the scene with the generated Instance. */
class FCustomizableObjectEditorViewportClient : public FEditorViewportClient, public TSharedFromThis<FCustomizableObjectEditorViewportClient>
{
public:
	FCustomizableObjectEditorViewportClient(TWeakPtr<ICustomizableObjectInstanceEditor> InCustomizableObjectEditor, FEditorModeTools* InModeTools, FPreviewScene* InPreviewScene, const TSharedPtr<SEditorViewport>& EditorViewportWidget);
	~FCustomizableObjectEditorViewportClient();
	
	/** persona config options **/
	class UPersonaOptions* ConfigOption;

	// FEditorViewportClient interface
	virtual void Draw(const FSceneView* View,FPrimitiveDrawInterface* PDI) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;

	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual bool InputWidgetDelta(FViewport* Viewport, EAxisList::Type CurrentAxis, FVector& Drag, FRotator& Rot, FVector& Scale) override;
	virtual void TrackingStarted(const FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
	virtual void TrackingStopped() override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	virtual bool CanSetWidgetMode(UE::Widget::EWidgetMode NewMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual FMatrix GetWidgetCoordSystem() const override;
	virtual ECoordSystem GetWidgetCoordSystemSpace() const override;
	virtual void SetWidgetCoordSystemSpace(ECoordSystem NewCoordSystem) override;
	virtual void SetViewportType(ELevelViewportType InViewportType) override;
	// End of FEditorViewportClient
	
	void CreatePreviewActor(const TWeakObjectPtr<UCustomizableObjectInstance>& InInstance);

	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& GetPreviewMeshComponents();

	void SetPreviewAnimationAsset(UAnimationAsset* AnimAsset);

	/**
	 *	Draws the UV overlay for the current LOD.
	 *
	 *	@param	InViewport					The viewport to draw to
	 *	@param	InCanvas					The canvas to draw to
	 *	@param	InTextYPos					The Y-position to begin drawing the UV-Overlay at (due to text displayed above it).
	 */
	void DrawUVs(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos);

	/**
	 * Bake the instance currently present in the editor. Internally will schedule the compilation and later the update of the instance before baking
	 * its resources.
	 */
	void BakeInstance();
	
private:
	/**
	 * Callback executed after the Customizable Object instance for baking had it's customizable object compiled
	 * @param InCompileCallbackParams Data returned on compilation that includes the end state of the compilation
	 */
	void OnCOForBakingCompilationEnd(const FCompileCallbackParams& InCompileCallbackParams);
	
	/**
	 * Callback executed after the instance in the editor gets updated for baking its contents.
	 */
	void OnInstanceForBakingUpdate(const FUpdateContext& Result);
	
public:
	// Callback to show / hide instance geometry data
	void StateChangeShowGeometryData();

	/** Callback for toggling the UV overlay show flag. */
	void SetDrawUVOverlay();
	
	/** Callback for checking the UV overlay show flag. */
	bool IsSetDrawUVOverlayChecked() const;

	/** Specify which UV to draw. -1 values will not draw anything. */
	void SetDrawUV(const FName ComponentName, const int32 LODIndex, const int32 SectionIndex, const int32 UVIndex);

	/** Callback for toggling the grid show flag. */
	void UpdateShowGridFromButton();

	/** 
	 * Updates the visual state of the ShowGrid Button and CheckBox in the Preview Settings
	 * @param bKeepOldValue - If true, it will keep the visibility of the grid and floor, otherwise it will invert it
	*/
	void UpdateShowGrid(bool bKeepOldValue);
	
	/** Callback for checking the grid show flag. */
	bool IsShowGridChecked() const;

	/** Callback for toggling the sky show flag. */
	void UpdateShowSkyFromButton();

	/**
	 * Updates the visual state of the ShowSky Button and CheckBox in the Preview Settings
	 * @param bKeepOldValue - If true, it will keep the visibility of the sky end environment, otherwise it will invert it
	*/
	void UpdateShowSky(bool bKeepOldValue);

	/** Callback for checking the sky show flag. */
	bool IsShowSkyChecked() const;

	/** Callback for toggling the bounds show flag. */
	void SetShowBounds();

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoClipMorph(UCustomizableObjectNodeModifierClipMorph& ClipPlainNode);

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoClipMorph();
	
	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoClipMesh(UCustomizableObjectNode& ClipMeshNode, FTransform* ClipMeshTransform, UObject& InClipMesh, int32 LODIndex, int32 SectionIndex, int32 MaterialSlotIndex);

private:
	/** Do not call directly. Method invoked each time the transform value of the clipping mesh is modified from the node. */
	void UpdateGizmoClipMeshTransform(const FTransform& InTransform);

public:
	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoClipMesh();
	
	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoProjector(
		const FWidgetLocationDelegate& WidgetLocationDelegate, const FOnWidgetLocationChangedDelegate& OnWidgetLocationChangedDelegate,
		const FWidgetDirectionDelegate& WidgetDirectionDelegate, const FOnWidgetDirectionChangedDelegate& OnWidgetDirectionChangedDelegate,
		const FWidgetUpDelegate& WidgetUpDelegate, const FOnWidgetUpChangedDelegate& OnWidgetUpChangedDelegate,
		const FWidgetScaleDelegate& WidgetScaleDelegate, const FOnWidgetScaleChangedDelegate& OnWidgetScaleChangedDelegate,
		const FWidgetAngleDelegate& WidgetAngleDelegate,
		const FProjectorTypeDelegate& ProjectorTypeDelegate,
		const FWidgetColorDelegate& WidgetColorDelegate,
		const FWidgetTrackingStartedDelegate& WidgetTrackingStartedDelegate);

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoProjector();
	
	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoLight(ULightComponent& Light);

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoLight();

	/** Play the animation. */
	void SetAnimation(UAnimationAsset* Animation);

	/** Add Light Component to the scene. */
	void AddLightToScene(ULightComponent* AddedLight);

	/** Remove Light component from the scene. */
	void RemoveLightFromScene(ULightComponent* RemovedLight);

	/** Remove all Light components from the scene. */
	void RemoveAllLightsFromScene();

	/** Setter of CustomizableObject */
	void SetCustomizableObject(UCustomizableObject* CustomizableObjectParameter);

	/** Helper method to draw shadowed string on viewport */
	void DrawShadowedString(FCanvas* Canvas, float StartX, float StartY, const FLinearColor& Color, float TextScale, FString String);

	/** Show pero LOD geometric information of the instance */
	void ShowInstanceGeometryInformation(FCanvas* InCanvas);

	/** Sets up the ShowFlag according to the current preview scene profile */
	void SetAdvancedShowFlagsForScene(const bool bAdvancedShowFlags);

	/** Delegate for preview profile is changed (used for updating show flags) */
	void OnAssetViewerSettingsChanged(const FName& InPropertyName);

	/** Debug draw a partial cylinder, given by MaxAngle in [0, 2PI] */
	void DrawCylinderArc(FPrimitiveDrawInterface* PDI, const FMatrix& CylToWorld, const FVector& Base, const FVector& XAxis, const FVector& YAxis, const FVector& ZAxis, float Radius, float HalfHeight, uint32 Sides, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, FColor Color, float MaxAngle);

	/** Getter of viewport's floor visibility */
	bool GetFloorVisibility();

	/** Setter of viewport's floor visibility */
	void SetFloorVisibility(bool Value);

	/** Getter of viewport's grid visibility */
	bool GetGridVisibility();

	/** Getter of viewport's environment visibility */
	bool GetEnvironmentMeshVisibility();

	/** Setter of viewport's environment visibility */
	void SetEnvironmentMeshVisibility(uint32 Value);

	/** Returns camera mode */
	bool IsOrbitalCameraActive() const; 

	/** Sets camera mode */
	void SetCameraMode(bool Value);

	/** Sets the skeletal mesh bones visibility */
	void SetShowBones();

	/** Returns true if bones are visible in viewport */
	bool IsShowingBones() const;

	const TArray<ULightComponent*>& GetLightComponents() const;

	void OnShowDisplayInfo();

	bool IsShowingMeshInfo() const;
	
	void OnEnableClothSimulation();
	
	bool IsClothSimulationEnabled() const;

	void OnDebugDrawPhysMeshWired();

	bool IsDebugDrawPhysMeshWired() const;
	
	FText GetMeshInfoText() const;
	
	void ToggleShowNormals();

	bool IsSetShowNormalsChecked() const;
	
	void ToggleShowTangents();

	bool IsSetShowTangentsChecked() const;
	
	void ToggleShowBinormals();

	bool IsSetShowBinormalsChecked() const;
	
	void SetPlaybackSpeedMode(EMutableAnimationPlaybackSpeeds::Type InMode);

	void SetCustomAnimationSpeed(float Speed);
	
	float GetCustomAnimationSpeed() const;

	EMutableAnimationPlaybackSpeeds::Type GetPlaybackSpeedMode() const;

private:
	/** Draws Mesh Bones in foreground (From: FAnimationViewportClient) */
	void DrawMeshBones(const UDebugSkelMeshComponent* MeshComponent, FPrimitiveDrawInterface* PDI);

	void SetWidgetType(EWidgetType Type);

	bool HandleBeginTransform();
	bool HandleEndTransform();

	void OnPreSetSkeletalMesh(const FPreSetSkeletalMeshParams& Params);

	void OnInstanceUpdate(UCustomizableObjectInstance* Instance);

	/** Preview Actor. All preview components are attached to this actor. */
	TStrongObjectPtr<AActor> Actor;
	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>> SkeletalMeshComponents;

	/** True if the widget is being dragged. */
	bool bManipulating = false;

	/** Pointer back to the editor tool that owns us */
	TWeakPtr<ICustomizableObjectInstanceEditor> CustomizableObjectEditorPtr;

	/** Flags for various options in the editor. */
	bool bDrawUVs;
	bool bDrawSky;

	// UV to draw
	FName UVDrawComponentName = NAME_None;
	int32 UVDrawSectionIndex = 0;
	int32 UVDrawLODIndex = 0;
	int32 UVDrawUVIndex = 0;
	
	UCustomizableObjectNodeModifierClipMorph* ClipMorphNode;
	TStrongObjectPtr<UMaterial> ClipMorphMaterial;
	bool bClipMorphLocalStartOffset;
	FVector ClipMorphOrigin;
	FVector ClipMorphOffset;
	FVector ClipMorphLocalOffset;
	FVector ClipMorphNormal;
	FVector ClipMorphXAxis;
	FVector ClipMorphYAxis;
	float MorphLength;
	FBoxSphereBounds MorphBounds;
	TStrongObjectPtr<UObject> ClipMeshNode;
	TStrongObjectPtr<UObject> ClipMesh;
	FDelegateHandle TransformExternallyChangedDelegateHandle;
	FTransform* ClipMeshTransform = nullptr;
	TStrongObjectPtr<UMaterial> ClipMeshMaterial;
	UStaticMeshComponent* ClipMeshStaticMeshComp;
	USkeletalMeshComponent* ClipMeshSkeletalMeshComp;
	float Radius1;
	float Radius2;
	float RotationAngle;

	/** Light being edited */
	ULightComponent* SelectedLightComponent;

	/** Spawned light components. */
	TArray<ULightComponent*> LightComponents;
	
	/** true if the camera has already being setup. */
	bool bIsCameraSetup = false;

	/** true if if has been updated after changing the actor. */
	bool bUpdated = false;
	
	/** Customizable object being used */
	UCustomizableObject* CustomizableObject;

	/** Flag to control whether to show / hide the instance geometry information data */
	bool StateChangeShowGeometryDataFlag;
	
	/** Material for cylinder arc solid render */
	TStrongObjectPtr<UMaterialInterface> TransparentPlaneMaterialXY;

	/** bool to return the Camera mode to Orbital when changing the Camera view to Perspective */
	bool bSetOrbitalOnPerspectiveMode;

	/** Flag to control the bones visibility in the viewport */
	bool bShowBones;

	/** Draw wireframe physics mesh. */
	bool bShowDebugClothing = false;
	
	// Temp Instance used in the bake process if a new instance is needed because mutable texture streaming is enabled so the viewport 
	// instance does not have the high quality mips in the texture's platform data
	TStrongObjectPtr<UCustomizableObjectInstance> BakeTempInstance = nullptr;

	// Cache System configuration cached before performing the mandatory instance update for baking so we can restore it after the bake operation
	bool bIsProgressiveMipStreamingEnabled = false;
	bool bIsOnlyGenerateRequestedLODsEnabled = false;

	/** Show detailed mesh info text. */
	bool bShowDisplayInfo = false;

	/** See USkeletalMeshComponent::bDisableClothSimulation. */
	bool bDisableClothSimulation = false;

	/** See USkeletalMeshComponent::bDrawNormals. */
	bool bDrawNormals = false;

	/** See USkeletalMeshComponent::bDrawTangents. */
	bool bDrawTangents = false;

	/** See USkeletalMeshComponent::bDrawBinormals. */
	bool bDrawBinormals = false;
	
	// The following delegates currently are only used by the EWidgetType::Projector
	FWidgetLocationDelegate WidgetLocationDelegate;
	FOnWidgetLocationChangedDelegate OnWidgetLocationChangedDelegate;

	FWidgetLocationDelegate WidgetDirectionDelegate;
	FOnWidgetLocationChangedDelegate OnWidgetDirectionChangedDelegate;

	FWidgetLocationDelegate WidgetUpDelegate;
	FOnWidgetLocationChangedDelegate OnWidgetUpChangedDelegate;

	FWidgetLocationDelegate WidgetScaleDelegate;
	FOnWidgetLocationChangedDelegate OnWidgetScaleChangedDelegate;

	FWidgetAngleDelegate WidgetAngleDelegate; 

	FProjectorTypeDelegate ProjectorTypeDelegate;

	FWidgetColorDelegate WidgetColorDelegate;

	FWidgetTrackingStartedDelegate WidgetTrackingStartedDelegate;
	
	EWidgetType WidgetType = EWidgetType::Hidden;

	/** Selected playback speed mode, used for deciding scale */
	EMutableAnimationPlaybackSpeeds::Type AnimationPlaybackSpeedMode = EMutableAnimationPlaybackSpeeds::Normal;

	/** Custom Animation speed in the viewport. Transient setting. */
	float CustomAnimationSpeed = 1.0f;
};
