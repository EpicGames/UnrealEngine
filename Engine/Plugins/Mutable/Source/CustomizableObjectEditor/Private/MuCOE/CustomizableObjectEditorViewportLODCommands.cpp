// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorViewportLODCommands.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditorViewportLODCommands"

void FCustomizableObjectEditorViewportLODCommands::RegisterCommands()
{
	UI_COMMAND(LODAuto, "LOD Auto", "Automatically select LOD", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(LOD0, "LOD 0", "Force select LOD 0", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(TranslateMode, "Translate Mode", "Select and translate objects", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(RotateMode, "Rotate Mode", "Select and rotate objects", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ScaleMode, "Scale Mode", "Select and scale objects", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(RotationGridSnap, "Grid Snap", "Enables or disables snapping objects to a rotation grid", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(HighResScreenshot, "High Resolution Screenshot...", "Opens the control panel for high resolution screenshots", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OrbitalCamera, "Orbital Camera", "Sets the camera in orbital mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FreeCamera, "Free Camera", "Sets the camera in free mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowBones, "Show Bones", "Shows the bone hierarchy of the skeletal mesh", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
