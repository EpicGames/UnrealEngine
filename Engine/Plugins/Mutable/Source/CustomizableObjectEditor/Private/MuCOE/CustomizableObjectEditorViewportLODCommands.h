// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"

class FUICommandInfo;

/**
* Class containing commands for persona viewport LOD actions
*/
class FCustomizableObjectEditorViewportLODCommands : public TCommands<FCustomizableObjectEditorViewportLODCommands>
{
public:
	FCustomizableObjectEditorViewportLODCommands()
		: TCommands<FCustomizableObjectEditorViewportLODCommands>
		(
			TEXT("COEditorViewportLODCommands"), // Context name for fast lookup
			NSLOCTEXT("CustomizableObjectEditor", "COEditorViewportLODCommands", "Customizable Object Editor Commands"), // Localized context name for displaying
			NAME_None, // Parent context name. 
			FCustomizableObjectEditorStyle::GetStyleSetName() // Icon Style Set
			)
	{
	}

	/** LOD Auto */
	TSharedPtr< FUICommandInfo > LODAuto;

	/** LOD 0 */
	TSharedPtr< FUICommandInfo > LOD0;

	/** Translate mode */
	TSharedPtr< FUICommandInfo > TranslateMode;

	/** Rotate mode */
	TSharedPtr< FUICommandInfo > RotateMode;

	/** Scale mode */
	TSharedPtr< FUICommandInfo > ScaleMode;

	/** Enables or disables snapping objects to a rotation grid */
	TSharedPtr< FUICommandInfo > RotationGridSnap;

	/** Opens the control panel for high resolution screenshots */
	TSharedPtr< FUICommandInfo > HighResScreenshot;

	/** Sets the camera mode to orbital camera */
	TSharedPtr< FUICommandInfo > OrbitalCamera;

	/** Sets the camera mode to free camera */
	TSharedPtr< FUICommandInfo > FreeCamera;

	/** Shows/Hide the bones of the skeletal mesh */
	TSharedPtr< FUICommandInfo > ShowBones;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};

