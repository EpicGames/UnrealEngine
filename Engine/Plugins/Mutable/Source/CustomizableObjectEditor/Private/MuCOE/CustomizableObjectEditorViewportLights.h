// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CustomizableObjectEditorViewportLights.generated.h"

USTRUCT()
struct FViewportLightData
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIsSpotLight = false;

	UPROPERTY()
	FTransform Transform;

	UPROPERTY()
	float Intensity = 0.0f;

	UPROPERTY()
	FLinearColor Color = FLinearColor(EForceInit::ForceInit);

	UPROPERTY()
	float AttenuationRadius = 0.0f;

	UPROPERTY()
	float SourceRadius = 0.0f;
	
	UPROPERTY()
	float SourceLength = 0.0f;

	UPROPERTY()
	float InnerConeAngle = 0.0f;

	UPROPERTY()
	float OuterConeAngle = 0.0f;

};

UCLASS()
class UCustomizableObjectEditorViewportLights : public UObject
{
public:
	GENERATED_BODY()


	UPROPERTY()
	TArray<FViewportLightData> LightsData;
};
