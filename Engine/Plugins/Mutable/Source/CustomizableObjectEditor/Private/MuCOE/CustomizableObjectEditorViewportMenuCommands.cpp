// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectEditorViewportMenuCommands.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditorViewportMenuCommands"

void FCustomizableObjectEditorViewportMenuCommands::RegisterCommands()
{
	//UI_COMMAND(PreviewSceneSettings, "Preview Scene Settings...", "The Advanced Preview Settings tab will let you alter the preview scene's settings.", EUserInterfaceActionType::Button, FInputChord());

	//UI_COMMAND(CameraFollow, "Camera Follow", "Follow the bound of the mesh ", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(SetCPUSkinning, "CPU Skinning", "Toggles display of CPU skinning in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowNormals, "Normals", "Toggles display of vertex normals in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowTangents, "Tangents", "Toggles display of vertex tangents in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetShowBinormals, "Binormals", "Toggles display of vertex binormals (orthogonal vector to normal and tangent) in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(AnimSetDrawUVs, "UV", "Toggles display of the mesh's UVs for the specified channel.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
