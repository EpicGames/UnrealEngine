// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"

class FUICommandInfo;

/**
* Class containing commands for viewport menu actions
*/
class FCustomizableObjectEditorViewportMenuCommands : public TCommands<FCustomizableObjectEditorViewportMenuCommands>
{
public:
	FCustomizableObjectEditorViewportMenuCommands()
		: TCommands<FCustomizableObjectEditorViewportMenuCommands>
		(
			TEXT("CustomizableObjectEditorViewportMenu"), // Context name for fast lookup
			NSLOCTEXT("CustomizableObjectEditor", "CustomizableObjectEditorViewportMenu", "CustomizableObject Viewport Menu"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FCustomizableObjectEditorStyle::GetStyleSetName() // Icon Style Set
			)
	{
	}

	/** Open settings for the preview scene */
	//TSharedPtr< FUICommandInfo > PreviewSceneSettings;

	/** Select camera follow */
	//TSharedPtr< FUICommandInfo > CameraFollow;

	/** Show vertex normals */
	TSharedPtr< FUICommandInfo > SetCPUSkinning;

	/** Show vertex normals */
	TSharedPtr< FUICommandInfo > SetShowNormals;

	/** Show vertex tangents */
	TSharedPtr< FUICommandInfo > SetShowTangents;

	/** Show vertex binormals */
	TSharedPtr< FUICommandInfo > SetShowBinormals;

	/** Draw UV mapping to viewport */
	TSharedPtr< FUICommandInfo > AnimSetDrawUVs;
public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};


