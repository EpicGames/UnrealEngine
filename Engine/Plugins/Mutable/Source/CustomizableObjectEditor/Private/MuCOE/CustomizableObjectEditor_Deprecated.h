// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/ObjectPtr.h"
#include "EdGraph/EdGraphPin.h"

#include "CustomizableObjectEditor_Deprecated.generated.h"

class UTexture2D;

// Place to hide all deprecated data structures. They are still needed for deserialization backwards compatibility.

// UCONodeSkeletalMeshSection
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeMaterialImage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	int32 UVLayout = 0;

	UPROPERTY()
	TObjectPtr<UTexture2D> ReferenceTexture = nullptr;
	
	UPROPERTY()
	int32 LayerIndex = 0;

	UPROPERTY()
	FString PinName;
};

// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeMaterialVector
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	FString Name;
	
	UPROPERTY()
	int32 LayerIndex = 0;

	UPROPERTY()
	FString PinName;
};

// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeMaterialScalar
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Name;
	
	UPROPERTY()
	int32 LayerIndex = 0;

	UPROPERTY()
	FString PinName;
};

// Deprecated, do not use!
constexpr int32 UV_LAYOUT_DEFAULT = -2;

// UCONodeModifierEditSkeletalMeshSection
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeEditMaterialImage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Name;
};

// UCONodeModifierExtendSkeletalMeshSection
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeExtendMaterialImage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Name;
};

// UCustomizableObjectNodeSkeletalMesh
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeSkeletalMeshMaterial
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	TObjectPtr<UEdGraphPin_Deprecated> MeshPin_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<UEdGraphPin_Deprecated>> LayoutPins_DEPRECATED;

	UPROPERTY()
	TArray<TObjectPtr<UEdGraphPin_Deprecated>> ImagePins_DEPRECATED;

	UPROPERTY()
	FEdGraphPinReference MeshPinRef;

	UPROPERTY()
	TArray<FEdGraphPinReference> LayoutPinsRef;

	UPROPERTY()
	TArray<FEdGraphPinReference> ImagePinsRef;
};

// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectNodeSkeletalMeshLOD
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FCustomizableObjectNodeSkeletalMeshMaterial> Materials;
};

// UCustomizableObjectNodeColorVariation
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectColorVariation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString Tag;
};

// UCustomizableObjectFloatVariation
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectFloatVariation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString Tag;
};

// UCustomizableObjectMaterialVariation
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectMaterialVariation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString Tag;
};

// UCustomizableObjectMeshVariation
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectMeshVariation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString Tag;
};


// UCustomizableObjectNodeObject
// Deprecated, do not use!
USTRUCT()
struct FComponentSettings
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	FString ComponentName;
};


// UCustomizableObjectNodeTextureVariation
// Deprecated, do not use!
USTRUCT()
struct FCustomizableObjectTextureVariation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString Tag;
};
