// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectFactory.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/SkeletalMesh.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SWindow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectFactory)


#define LOCTEXT_NAMESPACE "CustomizableObjectFactory"


UCustomizableObjectFactory::UCustomizableObjectFactory()
	: Super()
{
	// Property initialization
	bCreateNew = true;
	SupportedClass = UCustomizableObject::StaticClass();
	bEditAfterNew = true;
}


bool UCustomizableObjectFactory::DoesSupportClass(UClass * Class)
{
	return ( Class == UCustomizableObject::StaticClass() );
}


UClass* UCustomizableObjectFactory::ResolveSupportedClass()
{
	return UCustomizableObject::StaticClass();
}


UObject* UCustomizableObjectFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCustomizableObject* NewObj = NewObject<UCustomizableObject>(InParent, Class, Name, Flags);
	if (!NewObj)
	{
		return NewObj;
	}
	
	UCustomizableObjectGraph* Source = NewObject<UCustomizableObjectGraph>(NewObj, NAME_None, RF_Transactional);
	UCustomizableObjectPrivate* ObjectPrivate = NewObj->GetPrivate(); // Needed to avoid a static analysis warning

	if (!Source || !ObjectPrivate)
	{
		return NewObj;
	}

	ObjectPrivate->GetSource() = Source;
	
	Source->AddEssentialGraphNodes();
	
	return NewObj;
}


#undef LOCTEXT_NAMESPACE
