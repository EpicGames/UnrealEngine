// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectGraph.h"

#include "CustomizableObjectSchemaActions.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectGraph)

class UObject;


UCustomizableObjectGraph::UCustomizableObjectGraph()
	: Super()
{
	Schema = UEdGraphSchema_CustomizableObject::StaticClass();
}


void UCustomizableObjectGraph::PostLoad()
{
	Super::PostLoad();

	// TODO UE-222779 If compatibility code has to be executed, loaded the full hierarchy and in sync the BackwardsCompatibleFixup 

	// Make sure all nodes have finished loading.
	for (UEdGraphNode* Node : Nodes)
	{
		if (UCustomizableObjectNode* CustomizableObjectNode = Cast<UCustomizableObjectNode>(Node))
		{
			CustomizableObjectNode->ConditionalPostLoad();
		}
	}
	
	// Remove any null links
	//
	// Links can become null if the node they're linked to can't be loaded
	for (UEdGraphNode* Node : Nodes)
	{
		for (UEdGraphPin* Pin : Node->Pins)
		{
			Pin->LinkedTo.RemoveAll([](UEdGraphPin* Other) { return Other == nullptr; });
		}
	}
}


void UCustomizableObjectGraph::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	TArray<TObjectPtr<UEdGraphNode>> NodesCopy = Nodes; // Copy to be able to remove nodes inside the BackwardsCompatibleFixup.
	for (UEdGraphNode* Node : NodesCopy)
	{
		if (UCustomizableObjectNode* CustomizableObjectNode = Cast<UCustomizableObjectNode>(Node))
		{
			TArray<UEdGraphPin*> PinsCopy = CustomizableObjectNode->GetAllPins(); // Copy to be able to remove pins inside the BackwardsCompatibleFixup.
			for (UEdGraphPin* Pin : PinsCopy)
			{
				if (!Pin)
				{
					continue;
				}
					
				UCustomizableObjectNodePinData* PinData = CustomizableObjectNode->GetPinData(*Pin);
				if (!PinData)
				{
					continue;
				}
					
				PinData->BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
			}
				
			CustomizableObjectNode->BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
		}
	}
}


void UCustomizableObjectGraph::PostBackwardsCompatibleFixup()
{
	// Do any additional work which require nodes to be valid (i.e., have executed BackwardsCompatibleFixup).
	TArray<TObjectPtr<UEdGraphNode>> NodesCopy = Nodes; // Copy to be able to remove nodes inside the PostBackwardsCompatibleFixup.
	for (UEdGraphNode* Node : NodesCopy)
	{
		if (UCustomizableObjectNode* CustomizableObjectNode = Cast<UCustomizableObjectNode>(Node))
		{
			CustomizableObjectNode->PostBackwardsCompatibleFixup();
		}
	}
}


void UCustomizableObjectGraph::NotifyNodeIdChanged(const FGuid& OldGuid, const FGuid& NewGuid)
{
	NotifiedNodeIdsMap.FindOrAdd(OldGuid) = NewGuid;

	if (TSet<FGuid>* NodesToNotify = NodesToNotifyMap.Find(OldGuid))
	{
		for (FGuid& NodeId : *NodesToNotify)
		{
			for (int32 i = 0; i < Nodes.Num(); ++i)
			{
				if (Nodes[i]->NodeGuid == NodeId)
				{
					if (UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(Nodes[i]))
					{
						Node->UpdateReferencedNodeId(NewGuid);
					}
				}
			}
		}

		NodesToNotify->Empty();
	}
}

FGuid UCustomizableObjectGraph::RequestNotificationForNodeIdChange(const FGuid& OldGuid, const FGuid& NodeToNotifyGuid)
{
	if (const FGuid* Value = NotifiedNodeIdsMap.Find(OldGuid))
	{
		return *Value;
	}

	NodesToNotifyMap.FindOrAdd(OldGuid).Add(NodeToNotifyGuid);
	return OldGuid;
}


void UCustomizableObjectGraph::PostRename(UObject* OldOuter, const FName OldName)
{
	// Regenerate the Base Object Guid
	TArray<UCustomizableObjectNodeObject*> ObjectNodes;
	GetNodesOfClass<UCustomizableObjectNodeObject>(ObjectNodes);

	for (UCustomizableObjectNodeObject* ObjectNode : ObjectNodes)
	{
		if (ObjectNode->bIsBase)
		{
			ObjectNode->Identifier = FGuid::NewGuid();
			break;
		}
	}
}


bool UCustomizableObjectGraph::IsEditorOnly() const
{
	return true;
}


void UCustomizableObjectGraph::PostDuplicate(bool bDuplicateForPIE)
{
	// In BeginPostDuplicate, nodes can call RequestNotificationForNodeIdChange
	for (UEdGraphNode* Node : Nodes)
	{
		if (UCustomizableObjectNode* CustomizableObjectNode = Cast<UCustomizableObjectNode>(Node))
		{
			CustomizableObjectNode->BeginPostDuplicate(bDuplicateForPIE);
		}
	}

	TMap<FGuid, FGuid> NewGuids;

	for (UEdGraphNode* Node : Nodes)
	{
		// Generate new Guid
		const FGuid NewGuid = FGuid::NewGuid();
		NewGuids.Add(Node->NodeGuid) = NewGuid;

		// Notify Guid is going to change. Recive the new Guid
		NotifyNodeIdChanged(Node->NodeGuid, NewGuid);
	}

	// Chenge all nodes Guids
	for (UEdGraphNode* Node : Nodes)
	{
		Node->NodeGuid = NewGuids[Node->NodeGuid];
	}

	Super::PostDuplicate(bDuplicateForPIE);
}


void UCustomizableObjectGraph::AddEssentialGraphNodes()
{
	if (!IsMacro())
	{
		// Check whether the graph has a base node and create one if it doesn't since it is required
		bool bGraphHasBase = false;

		for (const TObjectPtr<UEdGraphNode>& AuxNode : Nodes)
		{
			UCustomizableObjectNodeObject* CustomizableObjectNodeObject = Cast<UCustomizableObjectNodeObject>(AuxNode);

			if (CustomizableObjectNodeObject && CustomizableObjectNodeObject->bIsBase)
			{
				bGraphHasBase = true;
				break;
			}
		}

		if (!bGraphHasBase)
		{
			UCustomizableObjectNodeObject* NodeTemplate = NewObject<UCustomizableObjectNodeObject>();
			FCustomizableObjectSchemaAction_NewNode::CreateNode(this, nullptr, FVector2D::ZeroVector, Cast<UEdGraphNode>(NodeTemplate));
		}
	}
	else
	{
		UCustomizableObjectMacro* ParentMacro = Cast<UCustomizableObjectMacro>(GetOuter());
		check(ParentMacro);

		UCustomizableObjectNodeTunnel* InputNode = NewObject<UCustomizableObjectNodeTunnel>();
		InputNode->bIsInputNode = true;
		InputNode->ParentMacro = ParentMacro;

		UCustomizableObjectNodeTunnel* OutputNode = NewObject<UCustomizableObjectNodeTunnel>();
		OutputNode->bIsInputNode = false;
		OutputNode->ParentMacro = ParentMacro;

		FVector2D InputNodePos = FVector2D(-100.0f, 0.0f);
		FVector2D OutputNodePos = FVector2D(100.0f, 0.0f);
		FCustomizableObjectSchemaAction_NewNode::CreateNode(this, nullptr, InputNodePos, Cast<UEdGraphNode>(InputNode));
		FCustomizableObjectSchemaAction_NewNode::CreateNode(this, nullptr, OutputNodePos, Cast<UEdGraphNode>(OutputNode));
	}
}


bool UCustomizableObjectGraph::IsMacro() const
{
	UObject* Outer = GetOuter();
	check(Outer);

	return Cast<UCustomizableObjectMacro>(Outer) != nullptr;
}
