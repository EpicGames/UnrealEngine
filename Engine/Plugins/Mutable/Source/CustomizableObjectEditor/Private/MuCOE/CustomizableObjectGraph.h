// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"

#include "CustomizableObjectGraph.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UObject;

UCLASS(MinimalAPI)
class UCustomizableObjectGraph : public UEdGraph
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectGraph();

	// UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API void PostRename(UObject * OldOuter, const FName OldName) override;

	UE_API virtual bool IsEditorOnly() const override;

	// Own Interface
	UE_API void NotifyNodeIdChanged(const FGuid& OldGuid, const FGuid& NewGuid);

	UE_API FGuid RequestNotificationForNodeIdChange(const FGuid& OldGuid, const FGuid& NodeToNotifyGuid);

	UE_API void PostDuplicate(bool bDuplicateForPIE) override;

	/** Adds the necessary nodes for a CO to work */
	UE_API void AddEssentialGraphNodes();

	UE_API void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion);

	UE_API void PostBackwardsCompatibleFixup();

	UE_API bool IsMacro() const;

private:

	// Request Node Id Update Map
	TMap<FGuid, TSet<FGuid>> NodesToNotifyMap;

	// Guid map with the key beeing the old Guid and the Value the new one, filled after duplicating COs
	TMap<FGuid, FGuid> NotifiedNodeIdsMap;
};

#undef UE_API
