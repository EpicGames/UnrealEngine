// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectIdentifierCustomization.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "PropertyCustomizationHelpers.h"
#include "MuCO/LoadUtils.h"
#include "MuCO/UnrealPortabilityHelpers.h"

#define LOCTEXT_NAMESPACE "FCustomizableObjectIdentifierCustomization"


TSharedRef<IPropertyTypeCustomization> FCustomizableObjectIdentifierCustomization::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectIdentifierCustomization());
}


void FCustomizableObjectIdentifierCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		const TSharedRef< IPropertyHandle > ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		FProperty* Property = ChildHandle->GetProperty();

		if (Property->GetName() == TEXT("Guid"))
		{
			GuidUPropertyHandle = ChildHandle;
		}
	}

	check(GuidUPropertyHandle.IsValid());

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> OutAssetData;
	AssetRegistryModule.Get().GetAssetsByClass(UE_MUTABLE_TOPLEVELASSETPATH(TEXT("/Script/CustomizableObject"), TEXT("CustomizableObject")), OutAssetData);
	SelectedCustomizableObject = nullptr;

	FString StringGuid;
	GuidUPropertyHandle->GetValue(StringGuid);
	FGuid Guid;
	FGuid::Parse(StringGuid, Guid);

	if (Guid.IsValid())
	{
		for (auto Itr = OutAssetData.CreateIterator(); Itr; ++Itr)
		{
			UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(*Itr));

			if (CustomizableObject)
			{
				UCustomizableObjectNodeObject* RootNode = GetRootNode(CustomizableObject);

				if (RootNode && RootNode->Identifier == Guid)
				{
					SelectedCustomizableObject = CustomizableObject;
					break;
				}
			}
		}
	}

	HeaderRow.NameContent()
		[
			//StructPropertyHandle->CreatePropertyNameWidget(FText::FromString(TEXT("New property header name")))
			StructPropertyHandle->CreatePropertyNameWidget()
		].
		ValueContent().
		MinDesiredWidth(500)
		[
			SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCustomizableObject::StaticClass())
				.ObjectPath(this, &FCustomizableObjectIdentifierCustomization::OnGetObjectPath)
				.OnObjectChanged(this, &FCustomizableObjectIdentifierCustomization::OnEventChanged)
		];
}


FString FCustomizableObjectIdentifierCustomization::OnGetObjectPath() const
{
	if (SelectedCustomizableObject)
	{
		return SelectedCustomizableObject->GetPathName();
	}

	return FString();
}


void FCustomizableObjectIdentifierCustomization::OnEventChanged(const FAssetData& InAssetData)
{
	SelectedCustomizableObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(InAssetData));

	if (SelectedCustomizableObject)
	{
		if (UCustomizableObjectNodeObject* RootNode = GetRootNode(SelectedCustomizableObject))
		{
			if (!RootNode->Identifier.IsValid())
			{
				RootNode->Identifier = FGuid::NewGuid();
				RootNode->MarkPackageDirty();
			}

			GuidUPropertyHandle->SetValue(RootNode->Identifier.ToString());
		}
	}
}


void FCustomizableObjectIdentifierCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	//Create further customization here
}


#undef LOCTEXT_NAMESPACE
