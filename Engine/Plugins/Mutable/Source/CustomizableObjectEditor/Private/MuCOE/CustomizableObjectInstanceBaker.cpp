// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectInstanceBaker.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCOE/CustomizableObjectInstanceBakingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectInstanceBaker)


void UCustomizableObjectInstanceBaker::BakeInstance(UCustomizableObjectInstance* InTargetInstance, const FBakingConfiguration& InBakingConfig, const TSharedPtr<const FOnBakerFinishedWork> InOnBakerFinishedWork)
{
	// Cache a delegate that we will later invoke once the baking operation has finished all the work (so we can safely delete this object )
	if (InOnBakerFinishedWork)
	{
		OnWorkFinished = InOnBakerFinishedWork;
	}
	
	// Cache the configuration and the callback used to report the baking results
	Configuration = InBakingConfig;

	if (Configuration.OutputPath.IsEmpty())
	{
		UE_LOGF(LogMutable,Error, "No target save path has been provided.")
		FinishBakingOperation(false, nullptr);
		return;
	}
	
	if (!InTargetInstance)
	{
		UE_LOGF(LogMutable,Error, "No instance has been provided.")
		FinishBakingOperation(false, nullptr);
		return;
	}

	if (!InTargetInstance->GetCustomizableObject())
	{
		UE_LOGF(LogMutable,Error, "The provided instance does not have a Customizable Object defined.")
		FinishBakingOperation(false, nullptr);
		return;
	}
	
	// We clone the InTargetInstance to ensure that the baked it is not open in a COI editor.
	InstanceToBake = InTargetInstance->Clone();
	
	if (ensure(InstanceToBake))
	{
		// Request the async compilation of the CO so we can later perform the update safely
		FCompileNativeDelegate OnCompilationCallbackNative;
		OnCompilationCallbackNative.BindUObject(this, &UCustomizableObjectInstanceBaker::OnCompilationEnd);
		ScheduleCOCompilationForBaking(*InstanceToBake, OnCompilationCallbackNative );
	}
	else
	{
		FinishBakingOperation(false, nullptr);
	}
}


void UCustomizableObjectInstanceBaker::OnCompilationEnd(const FCompileCallbackParams& InCompileCallbackParams)
{
	if (InCompileCallbackParams.bRequestFailed || InCompileCallbackParams.bErrors)
	{
		// Exit if the compilation failed
		UE_LOGF(LogMutable, Display, "The Customizable Object compilation for baking failed. Aborting instance baking operation.");
		FinishBakingOperation(false, nullptr);
		return;
	}

	// Schedule update of the instance for baking
	UpdateInstance();
}


void UCustomizableObjectInstanceBaker::UpdateInstance()
{
	// Call the instance update async method
	FInstanceUpdateNativeDelegate UpdateDelegate;
	UpdateDelegate.AddUObject(this, &UCustomizableObjectInstanceBaker::OnInstanceUpdated);
	
	// Once the update finishes successfully the baking operation will engage
	if (ensure(InstanceToBake))
	{
	ScheduleInstanceUpdateForBaking(*InstanceToBake, UpdateDelegate);
}
	else
	{
		FinishBakingOperation(false, nullptr);
	}
}


void UCustomizableObjectInstanceBaker::OnInstanceUpdated(const FUpdateContext& Result) const
{
	const EUpdateResult InstanceUpdateResult = Result.UpdateResult;
	UE_LOGF(LogMutable, Display, "Instance finished update with state : %ls.", *UEnum::GetValueAsString(InstanceUpdateResult));
	
	if (ensure(InstanceToBake) && InstanceToBake->GetPrivate()->SkeletalMeshStatus == ESkeletalMeshStatus::Success)
	{
		Bake();
	}
	else
	{
		// Report a failed baking operation.
		UE_LOGF(LogMutable, Display, "Instance updating for baking failed. Aborting instance baking operation.");
		FinishBakingOperation(false, nullptr);
	}
}


void UCustomizableObjectInstanceBaker::Bake() const
{
	// Set this flag so we ensure that the systems that make use of it know that should not display interaction messages to the user
	const bool GIsRunningUnattendedScript_Cached = GIsRunningUnattendedScript;
	GIsRunningUnattendedScript = true;
	
	// Array with all the packages that did get saved. Packages that failed the saving procedure will not appear in this array.
	TMap<UPackage*, const FResourceBakingData> SavedPackages;

	bool bBakeWasSuccessful = false;
	if (ensure(InstanceToBake))
	{
		// Ensures no interaction from the user will be required (alongside GIsRunningUnattendedScript)
		constexpr bool bIsUnhandledExecution = true;		// TODO: UE-314349
		bBakeWasSuccessful = BakeCustomizableObjectInstance(*InstanceToBake, Configuration, bIsUnhandledExecution, SavedPackages);
	}
	else
	{
		FinishBakingOperation(false, nullptr);
		return;
	}
	
	// Generate the array of structs to output based on the packages reported as saved
	TArray<FBakedResourceData> SavedPackagesData;
	{
		// Display a list with the paths of the assets saved
		UE_LOGF(LogMutable, Log, "Saved assets (%d) :", SavedPackages.Num());
		SavedPackagesData.Reserve(SavedPackages.Num());
		for (const TPair<UPackage*, const FResourceBakingData>& SavedPackagePair : SavedPackages)
		{
			FBakedResourceData BakeData;
			BakeData.AssetPath = SavedPackagePair.Key->GetPathName();
			
			const FResourceBakingData& PackageData = SavedPackagePair.Value;
			BakeData.SaveType = PackageData.SaveResolutionType;
			BakeData.Prefix = PackageData.GetPrefixSection();
			BakeData.UserGivenName = PackageData.GetUserGivenNameSection();
			BakeData.ComponentName = PackageData.GetComponentNameSection();
			BakeData.ResourceName = PackageData.GetResourceNameSection();
			BakeData.UniqueAssetNameSuffix = PackageData.GetUniqueAssetNameSuffixSection();

			// Sanity checks
			check(BakeData.SaveType != EPackageSaveResolutionType::None);
			check(!BakeData.AssetPath.IsEmpty());
			
			// Log the path for the caller to know
			UE_LOGF(LogMutable, Log, "\t%ls --- %ls",  *UEnum::GetValueAsString(BakeData.SaveType), *BakeData.AssetPath);
			
			SavedPackagesData.Push(BakeData);
		}
	}

	// Report that the baking operation has been completed
	UE_LOGF(LogMutable, Display, "Finishing COI baking procedure.");
	FinishBakingOperation(bBakeWasSuccessful, &SavedPackagesData);
	
	// Reset the GIsRunningUnattendedScript flag 
	GIsRunningUnattendedScript = GIsRunningUnattendedScript_Cached;
}


void UCustomizableObjectInstanceBaker::FinishBakingOperation(const bool bBakeWasSuccessful, TArray<FBakedResourceData>* InSavedPackagesData) const
{
	// Report that the baking operation has been completed
	if (Configuration.OnBakeOperationCompletedCallback.IsBound())
	{
		FCustomizableObjectInstanceBakeOutput Output;
		Output.bWasBakeSuccessful = bBakeWasSuccessful;
		if (InSavedPackagesData && !InSavedPackagesData->IsEmpty())
		{
			Output.SavedPackages = MoveTemp(*InSavedPackagesData);
		}
		Configuration.OnBakeOperationCompletedCallback.Execute(Output);
	}
	
	// Notify whatever caller that the baker completed it's work and therefore is safe to destroy/discard
	if (OnWorkFinished && OnWorkFinished->IsBound())
	{
		OnWorkFinished->Execute();
	}
}

