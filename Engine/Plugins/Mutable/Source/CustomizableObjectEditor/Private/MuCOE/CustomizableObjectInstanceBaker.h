// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"

#include "CustomizableObjectInstanceBaker.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

// Forward declarations
class UCustomizableObjectInstance;
struct FUpdateContext;


DECLARE_DELEGATE(FOnBakerFinishedWork);

/**
 * Utility class designed to allow tbe baking of the resources of a target Mutable Customizable Object instance onto disk.
 *
 * The actual baking operation does require some work done before the baking itself.
 *		- Compilation of the instance's Customizable Object : The mutable CO should be compiled to allow for the update of the instance.
 *		- Updating of the instance (async) : The mutable COInstance gets updated prior to the bake to make sure the instance resources to bake are the latest
 *		- Baking of the instance resources (sync) : The resources of the instance get baked onto disk.
 *
 * Once the whole operation gets completed a callback gets executed (OnBakeOperationCompletedCallback) that will provide the caller with the bake operation result
 * (true for success and false for failure) and a list of asset paths that got saved onto disk.
 *
 * Since the update of the instance for baking requires of some static changes in the instance updating system so the parallel update of instances
 * for baking is not allowed. This prohibition is enforced by the update methods that we call prior to the baking operation. No more than one update for baking can
 * be run at the same time.
 */
UCLASS(MinimalAPI)
class UCustomizableObjectInstanceBaker : public UObject
{
public:
	GENERATED_BODY()
	
	/**
	 * Execute this method in order to bake the provided instance. It will schedule a special type of instance update before proceeding with the bake itself.
	 * @param InTargetInstance The instance we want to bake
	 * @param InBakingConfig Configuration structure that determines how the baking is going to be made
	 * @param InOnBakerFinishedWork Delegate used to determine if the baker has completed its operation. Does not provide any other data than that so we can perform some post-bake actions.
	 */
	UE_API void BakeInstance(UCustomizableObjectInstance* InTargetInstance, const FBakingConfiguration& InBakingConfig, const TSharedPtr<const FOnBakerFinishedWork> InOnBakerFinishedWork = nullptr);
	
private:
	
	/**
	 * Callback executed once the compilation of the CO has finished.
	 * @param InCompileCallbackParams The compilation parameters that the compilation callback invocation provides to the caller.
	 */
	UE_API void OnCompilationEnd(const FCompileCallbackParams& InCompileCallbackParams);
	
	/**
	 * Takes care of updating and generating the instance resources so we can later bake them.
	 */
	UE_API void UpdateInstance();
	
	/**
	 * Callback executed when the instance finishes its update. It does not matter if it is successful or not
	 * @param Result The result returned by the instance updating operation once the operation is completed.
	 */
	UE_API void OnInstanceUpdated(const FUpdateContext& Result) const;

	/**
	 * Actual baking method. It will take care of baking the already updated instance and saving it's resources to disk. It will also log the paths
	 * to the generated packages. 
	 */
	UE_API void Bake() const;

	/**
	 * Closes the baking operation by reporting the assets saved ans also , if bound, running the callback that reports that the baker has completed
	 * all the work that had to be done.
	 * @param bBakeWasSuccessful Was the baking operation successful or not?
	 * @param InSavedPackagesData Data from the baked packages. Can be null if no package was saved.
	 */
	UE_API void FinishBakingOperation(const bool bBakeWasSuccessful, TArray<FBakedResourceData>* InSavedPackagesData) const;
	
	/**
	 * Cached configuration provided by the user
	 */
	FBakingConfiguration Configuration;
	
	/**
	 * The instance we want to update and then bake
	 */
	UPROPERTY()
	TObjectPtr<UCustomizableObjectInstance> InstanceToBake;	
	
	/**
	 * Optional delegate used to report when the baker has completed it's work (and it is not running or making other systems run)
	 * It will be called after we have given the end user the data about the success state of the baking and also the resources saved.
	 */
	TSharedPtr<const FOnBakerFinishedWork> OnWorkFinished;
};

#undef UE_API
