// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectInstanceBakingUtils.h"

#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"

#include "Misc/MessageDialog.h"
#include "FileHelpers.h"
#include "ObjectTools.h"
#include "UnrealBakeHelpers.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/Material.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/CustomizableObjectInstanceAssetUserData.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObjectMipDataProvider.h"
#include "MuT/UnrealPixelFormatOverride.h"
#include "Rendering/SkeletalMeshModel.h"
#include "MeshDescription.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor" 


FString FResourceBakingData::GetCachedComposedResourceName() const
{
	return CachedComposedResourceName;
}

FString FResourceBakingData::GetResourceComposedName()
{
	ensure(!Prefix.IsEmpty());
	ensure(!ResourceName.IsEmpty());
	
	// Prepare the strings that will form the composed string. Cache their values for external reference
	PrefixSection = Prefix;																			// Ex : "T_" or "SKM_"
	ComponentNameSection = ComponentName.IsEmpty() ? ComponentName : ComponentName + TEXT("_");		// Ex : "Head_" or "Body1_"			
	UserGivenNameSection = UserGivenName.IsEmpty() ? UserGivenName : UserGivenName + TEXT("_");		// Ex : "MyObject_" or ""
	ResourceNameSection =  ResourceName;															// Ex : "Texture_0000000..."
	UniqueAssetNameSuffixSection = UniqueAssetNameValueSuffix;										// Ex : "_0"
	
	// const FString ObjectBaseName = UserGivenNameSection.IsEmpty() ? ComponentNameSection : UserGivenNameSection + ComponentNameSection;
	CachedComposedResourceName = PrefixSection + UserGivenNameSection + ComponentNameSection + ResourceNameSection + UniqueAssetNameSuffixSection;

	if (bReplaceRestrictedChars)
	{
		CachedComposedResourceName = ObjectTools::SanitizeObjectName(CachedComposedResourceName);
	}

	return CachedComposedResourceName;
}


FString FResourceBakingData::GetResourceComposedFullPath()
{
	return AssetPath + TEXT("/") + GetResourceComposedName();
}


bool FResourceBakingData::Validate() const
{
	if (ResourceName.IsEmpty() || Prefix.IsEmpty() || AssetPath.IsEmpty())
	{
		return false;
	}
	if (GetCachedComposedResourceName().IsEmpty())
	{
		return false;
	}
	if (SaveResolutionType == EPackageSaveResolutionType::None)
	{
		return false;
	}

	return true;
}


int32 FResourceBakingData::GetObjectWithNameIndex(const FString& InName, const TArray<UObject*>& Objects) const
{
	int32 Index = 0;
	for (UObject* CachedResource : Objects)
	{
		if (CachedResource && CachedResource->GetName() == InName)
		{
			return Index;
		}
		Index++;
	}

	return INDEX_NONE;
}


void FResourceBakingData::MakeComposedResourceNameUnique(const TArray<UObject*>& InCachedResources)
{
	// FString Output;
	const FString TempComposedResourceName = GetResourceComposedName();
	
	// Look for the resource name provided to see if we have already worked with it.
	int32 FindResult = GetObjectWithNameIndex(TempComposedResourceName, InCachedResources);
	if (FindResult != INDEX_NONE)
	{
		// Add an integer suffix to create the unique name
		uint32 Count = 0;
		while (FindResult != INDEX_NONE)
		{
			FindResult = GetObjectWithNameIndex(TempComposedResourceName + "_" + FString::FromInt(Count), InCachedResources);
			Count++;
		}

		UniqueAssetNameValueSuffix =  "_" + FString::FromInt(--Count);
	}
}


bool FResourceBakingData::DoesPackageExist(const FString& InPackageFullName, const FString& InAssetName) const
{
	// Check if the package already exists
	UPackage* ExistingPackage = FindPackage(nullptr, *InPackageFullName);
	if (!ExistingPackage)
	{
		const FString PackageFilePath = InPackageFullName + "." + InAssetName;
		
		FString PackageFileName;
		if (FPackageName::DoesPackageExist(PackageFilePath, &PackageFileName))
		{
			ExistingPackage = LoadPackage(nullptr, *PackageFileName, LOAD_EditorOnly);
		}
	}

	return ExistingPackage != nullptr;
}


void FResourceBakingData::ComputePackageSaveResolutionType()
{
	SaveResolutionType = EPackageSaveResolutionType::None;
	if (DoesPackageExist(GetResourceComposedFullPath(), GetResourceComposedName()))
	{
		// File found, instead of creating a new package use this one and update the contents it has
		SaveResolutionType = EPackageSaveResolutionType::ReusedFile;
	}
	else
	{
		// The package could not be found. It is safe to create a package at this path and with the given name
		SaveResolutionType = EPackageSaveResolutionType::NewFile;
	}

	// Just to be sure this case never happens even with future changes
	check(SaveResolutionType == EPackageSaveResolutionType::ReusedFile || SaveResolutionType == EPackageSaveResolutionType::NewFile);

	const UEnum* OptimizationLevelEnum = StaticEnum<EPackageSaveResolutionType>();
	check(OptimizationLevelEnum);
}


FString FResourceBakingData::GetPrefixSection() const
{
	const FString ComposedResourceName = GetCachedComposedResourceName();
	ensure(ComposedResourceName.StartsWith(PrefixSection) && PrefixSection.EndsWith(TEXT("_")));
	return PrefixSection;
}


FString FResourceBakingData::GetComponentNameSection() const
{
	const FString ComposedResourceName = GetCachedComposedResourceName();
	ensure(ComposedResourceName.Contains(ComponentNameSection));
	return ComponentNameSection;
}


FString FResourceBakingData::GetUserGivenNameSection() const
{
	const FString ComposedResourceName = GetCachedComposedResourceName();
	ensure(ComposedResourceName.Contains(UserGivenNameSection));
	return UserGivenNameSection;
}


FString FResourceBakingData::GetResourceNameSection() const
{
	const FString ComposedResourceName = GetCachedComposedResourceName();
	ensure(ComposedResourceName.Contains(ResourceNameSection));
	return ResourceNameSection;
}


FString FResourceBakingData::GetUniqueAssetNameSuffixSection() const
{
	if (!UniqueAssetNameSuffixSection.IsEmpty())
	{
		const FString ComposedResourceName = GetCachedComposedResourceName();
		ensure(ComposedResourceName.EndsWith(UniqueAssetNameSuffixSection));
	}

	return UniqueAssetNameSuffixSection;
}


/**
 * Removes the "_n" string found at the end of the resource and skeletal mesh name.
 * @warning It does not check if that part is there or not.
 * @param InMutableResourceName The name of the resource that you want not to be unique.
 */
void MakeAssetNameNotUnique(FString& InMutableResourceName)
{
	// This should be a resource generated by mutable so try to remove the value that makes it unique
	int32 UniquenessValueSeparatorIndex = INDEX_NONE;
	InMutableResourceName.FindLastChar(TEXT('_'),UniquenessValueSeparatorIndex);
	if (UniquenessValueSeparatorIndex != INDEX_NONE)
	{
		InMutableResourceName = InMutableResourceName.Left(UniquenessValueSeparatorIndex);
	}
}


/**
 * Simple wrapper to be able to invoke the generation of a popup or log message depending on the execution context in which this code is being ran
 * @param InMessage The message to display
 * @param InTitle The title to be used for the popup or the log generated
 */
void ShowErrorNotification(const FText& InMessage, const FText& InTitle = LOCTEXT("CustomizableObjectInstanceBakingUtils_GenericBakingError","Baking Error") )
{
	if (!FApp::IsUnattended())
	{
		FMessageDialog::Open(EAppMsgType::Ok, InMessage, InTitle);
	}
	else
	{
		UE_LOGF(LogMutable, Error, "%ls - %ls", *InTitle.ToString(), *InMessage.ToString());
	}
}


/**
 * Utility functions for the baking operation.
 */


/**
 * Validates the filename chosen for the baking data
 * @param InFileName The filename chosen by the user
 * @return True if validation was successful, false otherwise
 */
bool ValidateProvidedFileName(const FString& InFileName)
{
	// Check for invalid characters in the name of the object to be serialized
	TCHAR InvalidCharacter = '0';
	{
		FString InvalidCharacters = FPaths::GetInvalidFileSystemChars();
		for (int32 InvalidCharIndex = 0; InvalidCharIndex < InvalidCharacters.Len(); ++InvalidCharIndex)
		{
			TCHAR Char = InvalidCharacters[InvalidCharIndex];
			FString SearchedChar = FString::Chr(Char);
			if (InFileName.Contains(SearchedChar))
			{
				InvalidCharacter = InvalidCharacters[InvalidCharIndex];
				break;
			}
		}
	}

	if (InvalidCharacter != '0')
	{
		const FText InvalidCharacterText = FText::FromString(FString::Chr(InvalidCharacter));
		const FText ErrorText = FText::Format(LOCTEXT("CustomizableObjectInstanceBakingUtils_InvalidCharacter", "The selected contains an invalid character ({0})."), InvalidCharacterText);

		ShowErrorNotification(ErrorText);
		
		return false;
	}

	return true;
}


/**
 * Validates the AssetPath chosen for the baking data
 * @param FileName The filename chosen by the user
 * @param AssetPath The AssetPath chosen by the user
 * @param InstanceCO The CustomizableObject from the provided COI
 * @return True if validation was successful, false otherwise
 */
bool ValidateProvidedAssetPath(const FString& FileName, const FString& AssetPath, const UCustomizableObject* InstanceCO)
{
	if (AssetPath.IsEmpty())
	{
		UE_LOGF(LogMutable, Error, "The AssetPath can not be empty!");
		return false;
	}

	// Ensure we are not overriding the parent CO
	const FString FullAssetPath = AssetPath + FString("/") + FileName + FString(".") + FileName;		// Full asset path to the new asset we want to create
	if (const bool bWouldOverrideParentCO = InstanceCO->GetPathName() == FullAssetPath)
	{
		const FText ErrorText = LOCTEXT("CustomizableObjectInstanceBakingUtils_OverwriteCO", "The selected path would overwrite the instance's parent Customizable Object.");

		ShowErrorNotification(ErrorText);
		
		return false;
	}

	return true;
}


/**
 * Given a collection of UObjects return the index of the one that matches the name also provided
 * @param InName The name of the object we are looking for
 * @param Objects The collection of objects we want to check for an element with the given name
 * @return The index of the element whose name matches the one provided or INDEX_NONE if the element could not be found
 */
int32 GetObjectWithNameIndex(const FString& InName, const TArray<UObject*>& Objects )
{
	int32 Index = 0;
	for (UObject* CachedResource : Objects)
	{
		if (CachedResource && CachedResource->GetName() == InName)
		{
			return Index;
		}
		Index++;
	}

	return INDEX_NONE;
}


bool TryRemovePrefix (FString& InOutString, const TArray<FString>& ToCheckPrefixes)
{
	for (const FString& PrefixToRemove : ToCheckPrefixes)
	{
		if (InOutString.Find(PrefixToRemove, ESearchCase::CaseSensitive) == 0)
		{
			InOutString = InOutString.RightChop(PrefixToRemove.Len());
			return true;
		}
	}

	return false;
}


static bool bIsBakeOperationAlreadyScheduled = false;


void OnInstanceUpdateFinish(const FUpdateContext& Result)
{
	FCustomizableObjectEditorLogger::CreateLog(
	LOCTEXT("CustomizableObjectInstanceBakingUtils_UpdateFinished", "The COInstance Update operation for baking has finished."))
	.Category(ELoggerCategory::COInstanceBaking)
	.CustomNotification()
	.Notification(true)
	.CustomizableObject(Result.Instance->GetCustomizableObject())
	.Log();

	// Allow the baking of more instances once the bake of this one has completed (or at least until the callbacks have been broadcast)
	bIsBakeOperationAlreadyScheduled = false;
}


void ScheduleCOCompilationForBaking(UCustomizableObjectInstance& InTargetInstance, const FCompileNativeDelegate& InCustomizableObjectCompilationDelegate, const bool bPerformPartialCompilation /*= false*/)
{
	// This prevents the queue of updates to have more than one instance for baking.
	if (bIsBakeOperationAlreadyScheduled)
	{
		UE_LOGF(LogMutable, Error, "The CO compilation for baking could not be scheduled. Another instance is being processed at this time.");
		InCustomizableObjectCompilationDelegate.ExecuteIfBound({true,false,false,false, false});
		return;
	}

	TObjectPtr<UCustomizableObject> CustomizableObject = InTargetInstance.GetCustomizableObject();
	if (!CustomizableObject)
	{
		UE_LOGF(LogMutable, Error, "The CO compilation for baking could not be scheduled. The provided instance does not have a CO set.");
		InCustomizableObjectCompilationDelegate.ExecuteIfBound({true,false,false,false, false});
		return;
	}
	
	// Tell the baking system that an instance for baking will be processed
	bIsBakeOperationAlreadyScheduled = true;

	FCompileParams CompileParams;
	CompileParams.bAsync = true;
	CompileParams.CallbackNative = InCustomizableObjectCompilationDelegate;
	CompileParams.CompileOnlySelectedInstance = bPerformPartialCompilation ? &InTargetInstance : nullptr;
	CompileParams.TextureCompression =  ECustomizableObjectTextureCompression::HighQuality;
	
	const UCustomizableObjectPrivate* CustomizableObjectPrivate = CustomizableObject->GetPrivate();
	if (ensure(CustomizableObjectPrivate))
	{
		if (const UModelResources* ModelResources = CustomizableObjectPrivate->GetModelResources())
		{
			// Force the recompilation if the last compilation was not using high quality texture compression settings
			CompileParams.bSkipIfNotOutOfDate = ModelResources->bCompiledWithHDTextureCompression;
		}
	}

	CustomizableObject->Compile(CompileParams);
}


void ScheduleInstanceUpdateForBaking(UCustomizableObjectInstance& InInstance, FInstanceUpdateNativeDelegate& InInstanceUpdateDelegate)
{
	check(InInstance.GetCustomizableObject() && InInstance.GetCustomizableObject()->IsCompiled());
	ensureMsgf(bIsBakeOperationAlreadyScheduled, TEXT("In order to perform an update for baking you first need to schedule a compilation for baking. That will ensure the compilation settings are the appropaite bor a bake."));	

	InInstanceUpdateDelegate.AddStatic(&OnInstanceUpdateFinish);
	
	PrepareUnrealCompression();
	
	// Schedule the update
	const TSharedRef<FUpdateContextPrivate> Context = MakeShared<FUpdateContextPrivate>();
	Context->Instance = &InInstance;
	Context->bForce = true;
	Context->bBake = true;
	Context->bIsHighPriority = true;
	Context->UpdateNativeCallback = InInstanceUpdateDelegate;
	Context->PixelFormatOverride = UnrealPixelFormatFunc;

	UCustomizableObjectSystemPrivate* SystemPrivate = UCustomizableObjectSystem::GetInstance()->GetPrivate();
	SystemPrivate->EnqueueUpdateSkeletalMesh(Context);

	FCustomizableObjectEditorLogger::CreateLog(
	LOCTEXT("CustomizableObjectInstanceBakingUtils_UpdateScheduled", "The COInstance Update operation for baking has been scheduled. Please hold."))
	.Category(ELoggerCategory::COInstanceBaking)
	.CustomNotification()
	.Notification(true)
	.CustomizableObject(InInstance.GetCustomizableObject())
	.Log();
}


/**
 * Checks if the texture was generated by mutable or if it was not
 * @param InTexture The texture we want to check
 * @return true if the texture was generated by mutable, false otherwise
 */
bool IsAMutableTexture (const UTexture2D* InTexture)
{
	bool bIsMutableTexture = false;
	for (UAssetUserData* UserData : *InTexture->GetAssetUserDataArray())
	{
		if (Cast<UMutableTextureMipDataProviderFactory>(UserData))
		{
			bIsMutableTexture = true;
			break;
		}
	}
	return bIsMutableTexture;
}


/**
 * Add the package and save resolution only once to prevent the overriding of packages marked as New with Reuse ones if processed multiple times
 * @param NewObjectPair The Pair of data we want to add to the map. Only if the package is not present in the map the action will get performed.
 * @param OutSavedPackages The map you want to add data into.
 */
void AddUniqueToSavePackage(const TPair<UPackage*, const FResourceBakingData>& NewObjectPair, TMap<UPackage*,const FResourceBakingData>& OutSavedPackages)
{
	if (!OutSavedPackages.Contains(NewObjectPair.Key))
	{
		OutSavedPackages.Add(NewObjectPair);
	}
}


bool BakeCustomizableObjectInstance(
	UCustomizableObjectInstance& InInstance,
	const FBakingConfiguration& Configuration,
	bool bIsUnattendedExecution,
	TMap<UPackage*,const FResourceBakingData>& OutSavedPackages)
{
	OutSavedPackages.Reset();

	// Extract some baking settings from the configuration file provided
	const FString& UserGivenFilename = Configuration.OutputFilesBaseName;
	const FString& AssetPath = Configuration.OutputPath;
	
	// Resource prefixes to be used. If an invalid prefix has been provided by the configuration object use the UE default one instead
	const FString SkeletalMeshAssetPrefix = Configuration.SkeletalMeshAssetPrefix.IsEmpty() ? TEXT("SK_") :  Configuration.SkeletalMeshAssetPrefix;
	const FString SkeletonAssetPrefix = Configuration.SkeletonAssetPrefix.IsEmpty() ? TEXT("SKEL_") : Configuration.SkeletonAssetPrefix;
	const FString PhysicsAssetPrefix = Configuration.PhysicsAssetPrefix.IsEmpty() ? TEXT("PHYS_") : Configuration.PhysicsAssetPrefix;
	const FString MaterialAssetPrefix = Configuration.MaterialAssetPrefix.IsEmpty() ? TEXT("M_") : Configuration.MaterialAssetPrefix;
	const FString TextureAssetPrefix = Configuration.TextureAssetPrefix.IsEmpty() ? TEXT("T_") : Configuration.TextureAssetPrefix;
	const FString MaterialInstanceAssetPrefix = Configuration.MaterialInstanceAssetPrefix.IsEmpty() ? TEXT("MI_") : Configuration.MaterialInstanceAssetPrefix;
	const FString MaterialDynamicInstanceAssetPrefix = Configuration.MaterialDynamicInstanceAssetPrefix.IsEmpty() ? TEXT("MID_") : Configuration.MaterialDynamicInstanceAssetPrefix;
	const FString MaterialConstantInstanceAssetPrefix = Configuration.MaterialConstantInstanceAssetPrefix.IsEmpty() ? TEXT("MIC_") : Configuration.MaterialConstantInstanceAssetPrefix;

	
	// Ensure that the state of the COI provided is valid --------------------------------------------------------------------------------------------
	UCustomizableObject* InstanceCO = InInstance.GetCustomizableObject();

	// Ensure the CO of the COI is accessible 
	if (!InstanceCO || InstanceCO->GetPrivate()->IsLocked())
	{
		FCustomizableObjectEditorLogger::CreateLog(
		LOCTEXT("CustomizableObjectInstanceBakingUtils_LockedObject", "Please wait until the Customizable Object is compiled"))
		.Category(ELoggerCategory::COInstanceBaking)
		.CustomNotification()
		.Notification(true)
		.CustomizableObject(InstanceCO)
		.Log();

		return false;
	}
	
	if (InstanceCO->GetPrivate()->Status.Get() == FCustomizableObjectStatus::EState::Loading)
	{
		FCustomizableObjectEditorLogger::CreateLog(
			LOCTEXT("CustomizableObjectInstanceBakingUtils_LoadingObject","Please wait until the Customizable Object is loaded"))
		.Category(ELoggerCategory::COInstanceBaking)
		.CustomNotification()
		.Notification(true)
		.CustomizableObject(InstanceCO)
		.Log();

		return false;
	}
	
	if (!ValidateProvidedFileName(UserGivenFilename))
	{
		UE_LOGF(LogMutable, Error, "The FileName for the instance baking is not valid.");
		return false;
	}

	if (!ValidateProvidedAssetPath(UserGivenFilename,AssetPath,InstanceCO))
	{
		UE_LOGF(LogMutable, Error, "The AssetPath for the instance baking is not valid.");
		return false;
	}
	
	// Exit early if the provided instance does not have a skeletal mesh
	if (!InInstance.HasAnySkeletalMesh())
	{
		UE_LOGF(LogMutable, Error, "The provided instance does not have an skeletal mesh.");
		return false;
	}

	// Early out if the Instance ModelResources is not valid.
	const UModelResources* ModelResources = InstanceCO->GetPrivate()->GetModelResources();
	if (!ModelResources)
	{
		UE_LOGF(LogMutable, Error, "The ModelResources from the Customizable Object is not valid.");
		return false;
	}

	// COI Validation completed : Proceed with the baking operation ----------------------------------------------------------------------------------
	
	// Notify of better configuration -> Continue operation normally
	if (ModelResources->bCompiledWithHDTextureCompression == false)
	{
		FCustomizableObjectEditorLogger::CreateLog(
		LOCTEXT("CustomizableObjectInstanceBakingUtils_LowQualityTextures", "The Customizable Object wasn't compiled with high quality textures. For the best baking results, change the Texture Compression setting and recompile it."))
		.Category(ELoggerCategory::COInstanceBaking)
		.CustomNotification()
		.Notification(true)
		.CustomizableObject(InstanceCO)
		.Log();
	}
	
	// Prefixes used for UMaterial assets. 
	const TArray<FString> MaterialResourcePrefixes =
	{
		MaterialAssetPrefix,
		MaterialInstanceAssetPrefix,
		MaterialDynamicInstanceAssetPrefix,
		MaterialConstantInstanceAssetPrefix
	};

	// Array with the already processed resource names and resources (UObjects)
	TArray<UObject*> DuplicatedObjects;
	TArray<UObject*> HandledSourceObjects;
	
	
	for (UE::Mutable::Private::FComponentId ComponentId = 0; ComponentId < InstanceCO->GetComponentCount(); ++ComponentId)
	{
		check(DuplicatedObjects.Num() == HandledSourceObjects.Num())
		
		const FName ComponentName = InstanceCO->GetPrivate()->GetComponentName(ComponentId);
		USkeletalMesh* Mesh = InInstance.GetComponentMeshSkeletalMesh(ComponentName);

		if (!Mesh)
		{
			continue;
		}
		
		TMap<UObject*, UObject*> ReplacementMap;

		if (Configuration.bExportAllResourcesOnBake)
		{
			UMaterial* Material;

			// Each element of the array corresponds to a material of the mesh
			// The key is the parameter index and the value the texture to be used there
			TArray<TMap<int, UTexture*>> TextureReplacementMaps;

			// Duplicate Textures found in the Material Instances of the SkeletalMesh so we can later assign them to the
			// duplicates of those material instances. At the end of the baking we will have a series of materials with the 
			// parameters set as the material instances they are based of.
			for (int32 MaterialIndex = 0; MaterialIndex < Mesh->GetMaterials().Num(); ++MaterialIndex)
			{
				UMaterialInterface* Interface = Mesh->GetMaterials()[MaterialIndex].MaterialInterface;
				Material = Interface->GetMaterial();
				UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Interface);

				TextureReplacementMaps.AddDefaulted();
				
				if (Material != nullptr && MaterialInstance != nullptr)
				{
					TArray<FGuid> ParameterIds;
					TArray<FMaterialParameterInfo> ParameterInfoObjects;
					Material->GetAllTextureParameterInfo(ParameterInfoObjects, ParameterIds);
					
					for (int32 ParameterInfoIndex = 0; ParameterInfoIndex < ParameterInfoObjects.Num(); ParameterInfoIndex++)
					{
						UTexture* Texture = nullptr;
						const FName& ParameterName = ParameterInfoObjects[ParameterInfoIndex].Name;
						if (MaterialInstance->GetTextureParameterValue(ParameterName, Texture))
						{
							UTexture2D* SourceTexture = Cast<UTexture2D>(Texture);
							if (!SourceTexture)
							{
								continue;
							}
							
							// Update the name adding to it the name of the component alongside other data
							FResourceBakingData TextureData {TextureAssetPrefix, UserGivenFilename, SourceTexture->GetName(), FString{}, AssetPath, Configuration.bReplaceRestrictedCharacters};
							if (TryRemovePrefix(TextureData.ResourceName, {FBakingConfiguration::BakedResourcePrefix}))
							{
								MakeAssetNameNotUnique(TextureData.ResourceName);
							}
							TryRemovePrefix(TextureData.ResourceName, {TextureAssetPrefix});
							
							// Experimental
							// If we know the resource has already been processed...
							if (HandledSourceObjects.Contains(SourceTexture))
							{
								// Instead of just creating a new name for the texture to be stored in disk, a better approach could be using
								// the already duplicated file and assigning it to the material we are working with.
								// The issue this has is that the texture may be named as from one component and now will be used for multiple ones
								for (UObject* DuplicatedObject : DuplicatedObjects)
								{
									if (DuplicatedObject &&
										DuplicatedObject->IsA<UTexture>() &&
										DuplicatedObject->GetName() == TextureData.GetResourceComposedName())
									{
										UTexture* DupTexture = Cast<UTexture>(DuplicatedObject);
										TextureReplacementMaps[MaterialIndex].Add(ParameterInfoIndex, DupTexture);

										UE_LOGF(LogMutable, Verbose, "Matching texture found with name %ls in component %ls", *TextureData.GetResourceComposedName(), *ComponentName.ToString())
										break;
									}
								}
							}
							
							// Here we will know if the asset already exists (ReusedFile) or if it is new (NewFile)
							TextureData.ComputePackageSaveResolutionType();
							
							// Duplicating mutable generated textures
							if (IsAMutableTexture(SourceTexture))
							{
								if (SourceTexture->GetPlatformData() && SourceTexture->GetPlatformData()->Mips.Num() > 0)
								{
									// Recover original name of the texture parameter value, now substituted by the generated Mutable texture
									UTexture* OriginalTexture = nullptr;
									MaterialInstance->Parent->GetTextureParameterValue(FName(*ParameterName.GetPlainNameString()), OriginalTexture);

									UTexture2D* DuplicatedTexture = FUnrealBakeHelpers::BakeHelper_CreateAssetTexture(SourceTexture, OriginalTexture, nullptr, TextureData);
									
									HandledSourceObjects.Add(SourceTexture);
									DuplicatedObjects.Add(DuplicatedTexture);
									
									AddUniqueToSavePackage({DuplicatedTexture->GetPackage(), TextureData},OutSavedPackages);

									if (OriginalTexture != nullptr)
									{
										TextureReplacementMaps[MaterialIndex].Add(ParameterInfoIndex, DuplicatedTexture);
									}
								}
							}
							else
							{
								// Duplicate the non-mutable textures of the Material instance (pass-through textures)
								UObject* DuplicatedTexture = FUnrealBakeHelpers::BakeHelper_DuplicateAsset(Texture, nullptr, false, TextureData);

								HandledSourceObjects.Add(Texture);
								DuplicatedObjects.Add(DuplicatedTexture);
								
								AddUniqueToSavePackage({DuplicatedTexture->GetPackage(), TextureData},OutSavedPackages);

								UTexture* DupTexture = Cast<UTexture>(DuplicatedTexture);
								TextureReplacementMaps[MaterialIndex].Add(ParameterInfoIndex, DupTexture);
							}
						}
					}
				}
			}
			
			// At this point we have an array where each element represents one material index.
			// Each element is formed by a map where the key is the parameter index and the value the texture to be used in that index so
			// Later we will be able to update all materials of the component with the duplicated textures we have just created
			
			// Duplicate the materials used by each material instance so that the replacement map has proper information 
			// when duplicating the material instances
			// Each material will get filled with the data of the interface it is related to.
			for (int32 MaterialIndex = 0; MaterialIndex < Mesh->GetMaterials().Num(); ++MaterialIndex)
			{
				UMaterialInterface* Interface = Mesh->GetMaterials()[MaterialIndex].MaterialInterface;
				Material = Interface ? Interface->GetMaterial() : nullptr;
				
				if (Material)
				{
					// If the interface has already been processed (so it is already part of the ReplacementMap) just skip it as we only need
					// to update the Material Interface once (Material Interface <- Material)
					if (HandledSourceObjects.Contains(Interface))
					{
						// should be the case if the interface has already been processed
						check(ReplacementMap.Contains(Interface));	
						continue;
					}

					FResourceBakingData MaterialData {MaterialInstanceAssetPrefix, UserGivenFilename, Interface->GetName(), ComponentName.ToString(), AssetPath, Configuration.bReplaceRestrictedCharacters};
				 	// If this is a material marked with the BAKE prefix then it should include a mark to make it unique. Remove it for the baking
				 	// operation
				 	if (TryRemovePrefix(MaterialData.ResourceName,{FBakingConfiguration::BakedResourcePrefix}))
				 	{
				 		MakeAssetNameNotUnique(MaterialData.ResourceName);
				 	}
					TryRemovePrefix (MaterialData.ResourceName,MaterialResourcePrefixes);
					
					// Give it a unique name or may happen that another material already stored on disk during this bake will get updated
					// with data form this material. We want to avoid that from happening as we do not want to require the name to always be a hash of
					// the contents (they are not always produced by mutable in this case)
					MaterialData.MakeComposedResourceNameUnique(DuplicatedObjects);
					MaterialData.ComputePackageSaveResolutionType();
					
					UObject* DuplicatedObject = FUnrealBakeHelpers::BakeHelper_DuplicateAsset(Material, nullptr,
						Configuration.bGenerateConstantMaterialInstancesOnBake, MaterialData);

					HandledSourceObjects.Add(Interface);
					DuplicatedObjects.Add(DuplicatedObject);

					// Trying to add the same interface twice would mean something is not being done correctly in this method
					check(!ReplacementMap.Contains(Interface));		
					// Tell the system that the Interface object will be replaced by the Duplicated one
					ReplacementMap.Add(Interface, DuplicatedObject);
					
					AddUniqueToSavePackage({DuplicatedObject->GetPackage(), MaterialData},OutSavedPackages);

					// Copy the texture parameters from the interface to the material
					if (UMaterial* DuplicatedMaterial = Cast<UMaterial>(DuplicatedObject))
					{
						FUnrealBakeHelpers::CopyAllMaterialParameters<UMaterial>(*DuplicatedMaterial, *Interface, TextureReplacementMaps[MaterialIndex]);
					}
				}
			}
		}
		else
		{
			// Export only the mutable generated resources
			
			// Duplicate the material instances
			for (int32 MaterialIndex = 0; MaterialIndex < Mesh->GetMaterials().Num(); ++MaterialIndex)
			{
				UMaterialInterface* Interface = Mesh->GetMaterials()[MaterialIndex].MaterialInterface;
				
				// If not a mutable material/instance continue as we do not want to duplicate resources that have been untouched by mutable
				if (Interface->HasAnyFlags(RF_Standalone))
				{
					continue;
				}

				// Check that the resource has not yet been processed, and if so, skip it as we only want to duplicate each interface once
				if (HandledSourceObjects.Contains(Interface))
				{
					continue;
				}
				
				// Use the name of the interface as it is unique because of the hash it contains
				FResourceBakingData MaterialData {FString{}, UserGivenFilename, Interface->GetName(), ComponentName.ToString(), AssetPath, Configuration.bReplaceRestrictedCharacters};
				{
					if (TryRemovePrefix(MaterialData.ResourceName, {FBakingConfiguration::BakedResourcePrefix}))
					{
						MakeAssetNameNotUnique(MaterialData.ResourceName);
					}
					TryRemovePrefix(MaterialData.ResourceName,MaterialResourcePrefixes);
					
					if (Configuration.bGenerateConstantMaterialInstancesOnBake && Interface->IsA<UMaterialInstance>())
					{
						// Change the prefix to the Material Constant Instance since in this situation the new asset based on the Interface
						// will be of the Material Constant Instance type
						MaterialData.Prefix = MaterialConstantInstanceAssetPrefix;
					}
					else
					{
						if (Interface->IsA<UMaterial>())
						{
							MaterialData.Prefix = MaterialAssetPrefix;
						}
						else if (Interface->IsA<UMaterialInstanceConstant>())
						{
							MaterialData.Prefix = MaterialConstantInstanceAssetPrefix;
						}
						else if (Interface->IsA<UMaterialInstanceDynamic>())
						{
							MaterialData.Prefix = MaterialDynamicInstanceAssetPrefix;
						}
						else
						{
							checkNoEntry();		// Invalid material type.
						}
					}
				}

				// One material could be used by multiple material instances so, to be sure, make sure the name is unique so we do not reuse it during the bake
				// We could be more specific in which cases we do this but I left it this way to keep the complexity as low as possible
				MaterialData.MakeComposedResourceNameUnique(DuplicatedObjects);
				MaterialData.ComputePackageSaveResolutionType();
				
				UObject* DuplicatedMaterial = FUnrealBakeHelpers::BakeHelper_DuplicateAsset(Interface, &ReplacementMap, Configuration.bGenerateConstantMaterialInstancesOnBake, MaterialData);
				HandledSourceObjects.Add(Interface);
				DuplicatedObjects.Add(DuplicatedMaterial);
				
				AddUniqueToSavePackage({DuplicatedMaterial->GetPackage(), MaterialData},OutSavedPackages);

				// Only need to duplicate the generate textures if the original material is a dynamic instance
				// If the material has Mutable textures, then it will be a dynamic material instance for sure
				if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Interface))
				{
					// Duplicate generated textures
					UMaterialInstanceDynamic* InstDynamic = Cast<UMaterialInstanceDynamic>(DuplicatedMaterial);
					UMaterialInstanceConstant* InstConstant = Cast<UMaterialInstanceConstant>(DuplicatedMaterial);

					if (InstDynamic || InstConstant)
					{
						for (int32 TextureIndex = 0; TextureIndex < MaterialInstance->TextureParameterValues.Num(); ++TextureIndex)
						{
							if (MaterialInstance->TextureParameterValues[TextureIndex].ParameterValue)
							{
								if (MaterialInstance->TextureParameterValues[TextureIndex].ParameterValue->HasAnyFlags(RF_Transient))
								{
									if (UTexture2D* SourceTexture = Cast<UTexture2D>(MaterialInstance->TextureParameterValues[TextureIndex].ParameterValue))
									{
										// If this source texture has already been processed during this bake operation, instead of creating a duplicate,
										// just use the resource cached in the DuplicatedObjects array
										if (HandledSourceObjects.Contains(SourceTexture))
										{
											UTexture* PrevTexture = Cast<UTexture>(DuplicatedObjects[HandledSourceObjects.Find(SourceTexture)]);
											check(PrevTexture);
											
											if (InstDynamic)
											{
												InstDynamic->SetTextureParameterValue(MaterialInstance->TextureParameterValues[TextureIndex].ParameterInfo.Name, PrevTexture);
											}
											else if (InstConstant)
											{
												InstConstant->SetTextureParameterValueEditorOnly(MaterialInstance->TextureParameterValues[TextureIndex].ParameterInfo.Name, PrevTexture);
											}
											
											continue;
										}

										// This should never happen as we handle this in the previous conditional block where we early out
										check(!HandledSourceObjects.Contains(SourceTexture))
										
										// The source texture has not yet been processed so duplicate it
										
										FResourceBakingData TextureData {TextureAssetPrefix, UserGivenFilename, SourceTexture->GetName(), FString{}, AssetPath, Configuration.bReplaceRestrictedCharacters};
										if (TryRemovePrefix(TextureData.ResourceName ,{FBakingConfiguration::BakedResourcePrefix}))
										{
											MakeAssetNameNotUnique(TextureData.ResourceName);
										}
										TryRemovePrefix(TextureData.ResourceName , {TextureAssetPrefix});
										
										// Check if the file is already in disk, so we can later decide what we do with that file
										TextureData.ComputePackageSaveResolutionType();
										
										UTexture2D* DuplicatedTexture = FUnrealBakeHelpers::BakeHelper_CreateAssetTexture(SourceTexture, nullptr, nullptr, TextureData);

										// Cache the file we just have handled and it's duplicate
										HandledSourceObjects.Add(SourceTexture);
										DuplicatedObjects.Add(DuplicatedTexture);
										
										AddUniqueToSavePackage({DuplicatedTexture->GetPackage(), TextureData},OutSavedPackages);

										if (InstDynamic)
										{
											InstDynamic->SetTextureParameterValue(MaterialInstance->TextureParameterValues[TextureIndex].ParameterInfo.Name, DuplicatedTexture);
										}
										else if(InstConstant)
										{
											InstConstant->SetTextureParameterValueEditorOnly(MaterialInstance->TextureParameterValues[TextureIndex].ParameterInfo.Name, DuplicatedTexture);
										}
									}
									else
									{
										UE_LOGF(LogMutable, Error, "A Mutable texture that is not a Texture2D has been found while baking a CustomizableObjectInstance.");
									}
								}
								else
								{
									// If it's not transient it's not a mutable texture, it's a pass-through texture
									// Just set the original texture
									if (InstDynamic)
									{
										InstDynamic->SetTextureParameterValue(MaterialInstance->TextureParameterValues[TextureIndex].ParameterInfo.Name, MaterialInstance->TextureParameterValues[TextureIndex].ParameterValue);
									}
									else if (InstConstant)
									{
										InstConstant->SetTextureParameterValueEditorOnly(MaterialInstance->TextureParameterValues[TextureIndex].ParameterInfo.Name, MaterialInstance->TextureParameterValues[TextureIndex].ParameterValue);
									}
								}
							}
						}
					}
				}
			}
		}
		
		FResourceBakingData SkeletalMeshData {SkeletalMeshAssetPrefix, UserGivenFilename, Mesh->GetName(), FString{}, AssetPath, Configuration.bReplaceRestrictedCharacters};
		if (Mesh->HasAnyFlags(RF_Transient))
		{
			MakeAssetNameNotUnique(SkeletalMeshData.ResourceName);
		}
		
		TryRemovePrefix(SkeletalMeshData.ResourceName,{SkeletalMeshAssetPrefix});
		SkeletalMeshData.MakeComposedResourceNameUnique(DuplicatedObjects);			// This ensures we get only one SKM per component

		// Skeletal Mesh's Skeleton
		if (USkeleton* Skeleton = Mesh->GetSkeleton())
		{
			const bool bTransient = Skeleton->GetPackage() == GetTransientPackage();
			
			// Duplicate only if transient or export all assets.
			if (bTransient || Configuration.bExportAllResourcesOnBake)
			{
				// Use the ResourceName already curated for the Skeletal Mesh
				FResourceBakingData SkeletonData {SkeletonAssetPrefix, UserGivenFilename, SkeletalMeshData.ResourceName, FString{}, AssetPath, Configuration.bReplaceRestrictedCharacters};

				// If the skeleton was already duplicated use the already duplicated one
				if (HandledSourceObjects.Contains(Skeleton))
				{
					for (UObject* DuplicatedObject : DuplicatedObjects)
					{
						if (DuplicatedObject &&
							DuplicatedObject->IsA<USkeleton>() &&
							DuplicatedObject->GetName() == SkeletonData.GetResourceComposedName())
						{
							ReplacementMap.Add(Skeleton, DuplicatedObject);

							UE_LOGF(LogMutable, Verbose, "Matching skeleton found with name %ls in component %ls", *SkeletonData.GetResourceComposedName(), *ComponentName.ToString())
							break;
						}
					}
				}

				SkeletonData.ComputePackageSaveResolutionType();
				UObject* DuplicatedSkeleton = FUnrealBakeHelpers::BakeHelper_DuplicateAsset(Skeleton, &ReplacementMap, false, SkeletonData);

				HandledSourceObjects.Add(Skeleton);
				DuplicatedObjects.Add(DuplicatedSkeleton);

				AddUniqueToSavePackage({DuplicatedSkeleton->GetPackage(), SkeletonData},OutSavedPackages);

				ReplacementMap.Add(Skeleton, DuplicatedSkeleton);
			}
		}

		// Skeletal Mesh's Physics Asset
		bool bNewPhysicsAssetCreated = false;
		if (UPhysicsAsset* PhysicsAsset = Mesh->GetPhysicsAsset())
		{
			const bool bTransient = PhysicsAsset->GetPackage() == GetTransientPackage();

			// Duplicate only if transient or export all assets.
			if (bTransient || Configuration.bExportAllResourcesOnBake)
			{
				// Use the ResourceName already curated for the Skeletal Mesh
				FResourceBakingData PhysicsAssetData {PhysicsAssetPrefix, UserGivenFilename, SkeletalMeshData.ResourceName, FString{}, AssetPath, Configuration.bReplaceRestrictedCharacters};

				// If the PhysicsAsset was already duplicated use the already duplicated one
				if (HandledSourceObjects.Contains(PhysicsAsset))
				{
					for (UObject* DuplicatedObject : DuplicatedObjects)
					{
						if (DuplicatedObject &&
							DuplicatedObject->IsA<UPhysicsAsset>() &&
							DuplicatedObject->GetName() == PhysicsAssetData.GetResourceComposedName())
						{
							ReplacementMap.Add(PhysicsAsset, DuplicatedObject);

							UE_LOGF(LogMutable, Verbose, "Matching Physics Asset found with name %ls in component %ls", *PhysicsAssetData.GetResourceComposedName(), *ComponentName.ToString())
							break;
						}
					}
				}

				PhysicsAssetData.ComputePackageSaveResolutionType();
				UObject* DuplicatedPhysicsAsset = FUnrealBakeHelpers::BakeHelper_DuplicateAsset(PhysicsAsset, &ReplacementMap, false, PhysicsAssetData);

				HandledSourceObjects.Add(PhysicsAsset);
				DuplicatedObjects.Add(DuplicatedPhysicsAsset);

				AddUniqueToSavePackage({DuplicatedPhysicsAsset->GetPackage(), PhysicsAssetData},OutSavedPackages);

				ReplacementMap.Add(PhysicsAsset, DuplicatedPhysicsAsset);

				bNewPhysicsAssetCreated = true;
			}
		}

		SkeletalMeshData.ComputePackageSaveResolutionType();
		UObject* DuplicatedMesh = FUnrealBakeHelpers::BakeHelper_DuplicateAsset(Mesh, &ReplacementMap, false, SkeletalMeshData);
		
		HandledSourceObjects.Add(Mesh);
		DuplicatedObjects.Add(DuplicatedMesh);
		
		AddUniqueToSavePackage({DuplicatedMesh->GetPackage(), SkeletalMeshData},OutSavedPackages);

		Mesh->Build();

		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(DuplicatedMesh))
		{
			const int32 NumLODs = Mesh->GetLODNum();

			SkeletalMesh->SetNumSourceModels(0);
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
        		SkeletalMesh->AddLODInfo(*Mesh->GetLODInfo(LODIndex));
        	}
		
			SkeletalMesh->GetImportedModel()->SkeletalMeshModelGUID = FGuid::NewGuid();

			// Duplicate AssetUserData
			{
				const TArray<UAssetUserData*>* AssetUserDataArray = Mesh->GetAssetUserDataArray();
				for (const UAssetUserData* AssetUserData : *AssetUserDataArray)
				{
					if (AssetUserData)
					{
						// Duplicate to change ownership
						UAssetUserData* NewAssetUserData = Cast<UAssetUserData>(StaticDuplicateObject(AssetUserData, SkeletalMesh));
						SkeletalMesh->AddAssetUserData(NewAssetUserData);
					}
				}
			}

			// Copy LODSettings from the Reference Skeletal Mesh
			{
				if (ModelResources->ReferenceSkeletalMeshesData.IsValidIndex(ComponentId))
				{
					USkeletalMeshLODSettings* LODSettings = ModelResources->ReferenceSkeletalMeshesData[ComponentId].SkeletalMeshLODSettings;
					SkeletalMesh->SetLODSettings(LODSettings);
				}
			}

			// Set the physics asset preview mesh if the SkeletalMesh physics assets has been generated as part of the bake.
			if (SkeletalMesh->GetPhysicsAsset() && bNewPhysicsAssetCreated)
			{
				SkeletalMesh->GetPhysicsAsset()->SetPreviewMesh(SkeletalMesh);
			}

			// Copy MeshDescriptions
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				if (Mesh->HasMeshDescription(LODIndex))
				{
					FMeshDescription MeshDescription;
					Mesh->CloneMeshDescription(LODIndex, MeshDescription);
					SkeletalMesh->CreateMeshDescription(LODIndex, MoveTemp(MeshDescription));
					SkeletalMesh->CommitMeshDescription(LODIndex);

					const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
					LODModel.BuildStringID = LODModel.GetLODModelDeriveDataKey();
				}
			}

			// Generate render data
			SkeletalMesh->Build();
		}

		check(DuplicatedObjects.Num() == HandledSourceObjects.Num());
		
		// Remove duplicated UObjects from Root (previously added to avoid objects from being GC in the middle of the bake process)
		for (UObject* Obj : DuplicatedObjects)
		{
			Obj->RemoveFromRoot();
		}
	}

	// Save the packages generated during the baking operation  --------------------------------------------------------------------------------------
	
	// Complete the baking by saving the packages we have cached during the baking operation
	if (OutSavedPackages.Num())
	{
		// Prepare the list of assets we want to provide to "PromptForCheckoutAndSave" for saving
		TArray<UPackage*> PackagesToSaveProxy;
		PackagesToSaveProxy.Reserve(OutSavedPackages.Num());
		for (TPair<UPackage*, FResourceBakingData> DataToSave : OutSavedPackages)
		{
			// Ensure we have no duplicates here
			check(!PackagesToSaveProxy.Contains(DataToSave.Key));
			check(DataToSave.Value.Validate());
			
			PackagesToSaveProxy.Push(DataToSave.Key);
		}

		// List of packages that could not be saved
		TArray<UPackage*> FailedToSavePackages;
		const bool bWasSavingSuccessful = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSaveProxy, false, !bIsUnattendedExecution, &FailedToSavePackages, false, false) == FEditorFileUtils::EPromptReturnCode::PR_Success;

		// Remove all packages that were going to be saved but failed to do so
		uint32 RemovedPackagesCount = 0;
		for (UPackage* Package : FailedToSavePackages)
		{
			if (OutSavedPackages.Remove(Package))
			{
				RemovedPackagesCount++;
			}
		}
		OutSavedPackages.Shrink();

		return RemovedPackagesCount > 0 ? false : bWasSavingSuccessful;
	}
	
	// The operation will fail if no packages are there to save
	return false;
}


bool BakeCustomizableObjectInstance(UCustomizableObjectInstance& InInstance, const FBakingConfiguration& Configuration, bool bIsUnattendedExecution)
{
	TMap<UPackage*,const FResourceBakingData> DummyMap = TMap<UPackage*,const FResourceBakingData>{};
	return BakeCustomizableObjectInstance(InInstance, Configuration, bIsUnattendedExecution, DummyMap);
}

#undef LOCTEXT_NAMESPACE 
