// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectInstanceEditor.h"

#include "Animation/PoseAsset.h"
#include "ContentBrowserModule.h"
#include "CustomizableObjectEditor.h"
#include "DetailsViewArgs.h"
#include "FileHelpers.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableSkeletalComponent.h"
#include "MuCOE/CustomizableObjectEditorActions.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "MuCOE/CustomizableObjectInstanceEditorActions.h"
#include "MuCOE/CustomizableObjectPreviewScene.h"
#include "MuCOE/SCustomizableObjectEditorTextureAnalyzer.h"
#include "PropertyEditorModule.h"
#include "SCustomizableObjectEditorAdvancedPreviewSettings.h"
#include "SCustomizableObjectEditorViewport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectInstanceEditor)

class FAdvancedPreviewScene;
class FProperty;

#define LOCTEXT_NAMESPACE "CustomizableObjectInstanceEditor"


DEFINE_LOG_CATEGORY_STATIC(LogCustomizableObjectInstanceEditor, Log, All);

const FName FCustomizableObjectInstanceEditor::ViewportTabId( TEXT( "CustomizableObjectInstanceEditor_Viewport" ) );
const FName FCustomizableObjectInstanceEditor::InstancePropertiesTabId( TEXT( "CustomizableObjectInstanceEditor_InstanceProperties" ) );
const FName FCustomizableObjectInstanceEditor::AdvancedPreviewSettingsTabId(TEXT("CustomizableObjectEditor_AdvancedPreviewSettings"));
const FName FCustomizableObjectInstanceEditor::TextureAnalyzerTabId(TEXT("CustomizableObjectInstanceEditor_TextureAnalyzer"));

void UUpdateClassWrapperClass::DelegatedCallback(UCustomizableObjectInstance* Instance)
{
	Delegate.ExecuteIfBound();
}


UProjectorParameter::UProjectorParameter()
{
	SetFlags(RF_Transactional);
}


void UProjectorParameter::SelectProjector(const FString& InParamName, const int32 InRangeIndex)
{
	ParamName = InParamName;
	RangeIndex = InRangeIndex;
}


void UProjectorParameter::UnselectProjector()
{
	ParamName = "";
	RangeIndex = -1;
}


bool UProjectorParameter::IsProjectorSelected(const FString& InParamName, const int32 InRangeIndex) const
{
	return ParamName == InParamName && RangeIndex == InRangeIndex;
}


FVector UProjectorParameter::GetPosition() const
{
	return Position;
}


void UProjectorParameter::SetPosition(const FVector& InPosition)
{
	Position = InPosition;
}


FVector UProjectorParameter::GetDirection() const
{
	return Direction;
}


void UProjectorParameter::SetDirection(const FVector& InDirection)
{
	Direction = InDirection;
}


FVector UProjectorParameter::GetUp() const
{
	return Up;
}


void UProjectorParameter::SetUp(const FVector& InUp)
{
	Up = InUp;
}


FVector UProjectorParameter::GetScale() const
{
	return Scale;
}


void UProjectorParameter::SetScale(const FVector& InScale)
{
	Scale = InScale;
}


void FCustomizableObjectInstanceEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	InTabManager->RegisterTabSpawner( ViewportTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectInstanceEditor::SpawnTab_Viewport) )
		.SetDisplayName( LOCTEXT( "ViewportTab", "Viewport" ) )
		.SetGroup( MenuStructure.GetToolsCategory() );
	
	InTabManager->RegisterTabSpawner( InstancePropertiesTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectInstanceEditor::SpawnTab_InstanceProperties) )
		.SetDisplayName( LOCTEXT( "InstancePropertiesTab", "Instance Properties" ) )
		.SetGroup( MenuStructure.GetToolsCategory() );

	InTabManager->RegisterTabSpawner(AdvancedPreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectInstanceEditor::SpawnTab_AdvancedPreviewSettings))
		.SetDisplayName(LOCTEXT("AdvancedPreviewSettingsTab", "Advanced Preview Settings"))
		.SetGroup(MenuStructure.GetToolsCategory());

	InTabManager->RegisterTabSpawner(TextureAnalyzerTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectInstanceEditor::SpawnTab_TextureAnalyzer))
		.SetDisplayName(LOCTEXT("InstanceTextureAnalyzer", "Texture Analyzer"))
		.SetGroup(MenuStructure.GetToolsCategory());
}

void FCustomizableObjectInstanceEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner( ViewportTabId );
	InTabManager->UnregisterTabSpawner( InstancePropertiesTabId );
	InTabManager->UnregisterTabSpawner(AdvancedPreviewSettingsTabId);
	InTabManager->UnregisterTabSpawner(TextureAnalyzerTabId);
}


void UCustomSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);
	
	if (!PropertyChangedEvent.MemberProperty)
	{
		return;
	}
	
	if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomSettings, Animation))
	{
		TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return;
		}
		
		Editor->GetViewport()->GetViewportClient()->SetAnimation(Animation);
	}
}


ULightComponent* UCustomSettings::GetSelectedLight() const
{
	return SelectedLight;
}


void UCustomSettings::SetSelectedLight(ULightComponent* Light)
{
	SelectedLight = Light;
}


UCustomizableObjectEditorViewportLights* UCustomSettings::GetLightsPreset() const
{
	return LightsPreset;
}


void UCustomSettings::SetLightsPreset(UCustomizableObjectEditorViewportLights& InLightsPreset)
{
	LightsPreset = &InLightsPreset;
}


TWeakPtr<ICustomizableObjectInstanceEditor> UCustomSettings::GetEditor() const
{
	return WeakEditor;
}


void UCustomSettings::SetEditor(TSharedPtr<ICustomizableObjectInstanceEditor> InEditor)
{
	WeakEditor = InEditor;
}


FCustomizableObjectInstanceEditor::FCustomizableObjectInstanceEditor()
{
	CustomizableObjectInstance = nullptr;
	HelperCallback = nullptr;
	PoseAsset = nullptr;
}


FCustomizableObjectInstanceEditor::~FCustomizableObjectInstanceEditor()
{
	if (HelperCallback) 
	{
		CustomizableObjectInstance->UpdatedDelegate.RemoveDynamic(HelperCallback, &UUpdateClassWrapperClass::DelegatedCallback);
		HelperCallback->Delegate.Unbind();
		HelperCallback = nullptr;
	}

	PreviewSkeletalMeshComponents.Reset();

	CustomizableInstanceDetailsView.Reset();
	Viewport.Reset();

	FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedHandle);

	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}


void FCustomizableObjectInstanceEditor::InitCustomizableObjectInstanceEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCustomizableObjectInstance* InCustomizableObjectInstance )
{
	ProjectorParameter = NewObject<UProjectorParameter>();
	
	CustomSettings = NewObject<UCustomSettings>();
	CustomSettings->SetEditor(SharedThis(this));

	EditorProperties = NewObject<UCustomizableObjectEditorProperties>();

	// Register our commands. This will only register them if not previously registered
	FCustomizableObjectInstanceEditorCommands::Register();
	FCustomizableObjectEditorViewportCommands::Register();

	BindCommands();

	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowObjectLabel = false;
	
	CustomizableInstanceDetailsView = PropPlugin.CreateDetailView( DetailsViewArgs );
	
	TextureAnalyzer = SNew(SCustomizableObjecEditorTextureAnalyzer).CustomizableObjectInstanceEditor(this).CustomizableObjectEditor(nullptr);

	// Create EditorModeManager before viewport construction so viewport clients
	// receive the editor's ModeTools instead of creating their own.
	CreateEditorModeManager();

	Viewport = SNew(SCustomizableObjectEditorViewportTabBody)
		.CustomizableObjectEditor(SharedThis(this));
	
	// Set the instance
	check(InCustomizableObjectInstance);
	CustomizableObjectInstance = InCustomizableObjectInstance;
	CustomizableObjectInstance->UpdatedNativeDelegate.AddSP(SharedThis(this), &FCustomizableObjectInstanceEditor::OnUpdatePreviewInstance);
	CustomizableObjectInstance->SetBuildParameterRelevancy(true);
	
	bOnlyRelevantParameters = InCustomizableObjectInstance->GetPrivate()->bShowOnlyRelevantParameters;
	bOnlyRuntimeParameters = InCustomizableObjectInstance->GetPrivate()->bShowOnlyRuntimeParameters;
	
	TSharedPtr<FAdvancedPreviewScene> AdvancedPreviewScene = StaticCastSharedPtr<FAdvancedPreviewScene>(Viewport->GetPreviewScene());

	AdvancedPreviewSettingsWidget = SNew(SCustomizableObjectEditorAdvancedPreviewSettings, AdvancedPreviewScene.ToSharedRef())
		.CustomSettings(CustomSettings)
		.CustomizableObjectEditor(SharedThis(this).ToWeakPtr());
	
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_CustomizableObjectInstanceEditor_Layout_v2.2" );

	StandaloneDefaultLayout->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.7f)
			->SetHideTabWell(true)
			->AddTab(ViewportTabId, ETabState::OpenedTab)
		)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(0.3f)
			->AddTab(InstancePropertiesTabId, ETabState::OpenedTab)
			->AddTab(AdvancedPreviewSettingsTabId, ETabState::OpenedTab)
			->SetForegroundTab(InstancePropertiesTabId)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, CustomizableObjectInstanceEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, CustomizableObjectInstance );

	CustomizableInstanceDetailsView->SetObject(InCustomizableObjectInstance, true); // Can only be called after initializing the Asset Editor
	
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// Clears selection highlight.
	OnInstancePropertySelectionChanged(nullptr);
	
	if (UCustomizableObject* CustomizableObject = CustomizableObjectInstance->GetCustomizableObject())
	{
		UCustomizableObjectPrivate* CustomizableObjectPrivate = CustomizableObject->GetPrivate();

		CustomizableObjectPrivate->Status.GetOnStateChangedDelegate().AddRaw(this, &FCustomizableObjectInstanceEditor::OnCustomizableObjectStatusChanged);
		OnCustomizableObjectStatusChanged(FCustomizableObjectStatusTypes::EState::Loading, CustomizableObjectPrivate->Status.Get()); // Fake we are still in the loading phase.

		int32 StateParameterCount = CustomizableObject->GetStateParameterCount(CustomizableObjectInstance->GetCurrentState());

		if (StateParameterCount == 0)
		{
			bOnlyRuntimeParameters = false;
			InCustomizableObjectInstance->GetPrivate()->bShowOnlyRuntimeParameters = false;
		}
		
		CustomizableObject->GetPostCompileDelegate().AddSP(this, &FCustomizableObjectInstanceEditor::OnPostCompile);
	}
}


FName FCustomizableObjectInstanceEditor::GetToolkitFName() const
{
	return FName("CustomizableObjectInstanceEditor");
}


FText FCustomizableObjectInstanceEditor::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Customizable Object Instance Editor");
}


void FCustomizableObjectInstanceEditor::SaveAsset_Execute()
{
	if (!CustomizableObjectInstance)
	{
		return;
	}

	if (UPackage* Package = CustomizableObjectInstance->GetOutermost())
	{
		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(Package);

		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
	}
}

bool FCustomizableObjectInstanceEditor::CanOpenOrShowParent()
{
	if (!CustomizableObjectInstance)
	{
		return false;
	}

	return CustomizableObjectInstance->GetCustomizableObject() != nullptr;
}


void FCustomizableObjectInstanceEditor::ShowParentInContentBrowser()
{
	check(CustomizableObjectInstance);
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().SyncBrowserToAssets(TArray<FAssetData>{FAssetData(CustomizableObjectInstance->GetCustomizableObject())});
}


void FCustomizableObjectInstanceEditor::OpenParentInEditor()
{
	check(CustomizableObjectInstance);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CustomizableObjectInstance->GetCustomizableObject());
}


void FCustomizableObjectInstanceEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( CustomizableObjectInstance );
	Collector.AddReferencedObject( HelperCallback );
	Collector.AddReferencedObject( ProjectorParameter );
	Collector.AddReferencedObject( CustomSettings );
	Collector.AddReferencedObject(EditorProperties);
}


TSharedRef<SDockTab> FCustomizableObjectInstanceEditor::SpawnTab_Viewport( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == ViewportTabId );

	return SNew(SDockTab)
		//.Icon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.Preview"))
		.Label( FText::FromString( GetTabPrefix() + LOCTEXT("CustomizableObjectViewport_TabTitle", "Instance Viewport").ToString() ) )
		[
			Viewport.ToSharedRef()
		];
}


TSharedRef<SDockTab> FCustomizableObjectInstanceEditor::SpawnTab_InstanceProperties( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId() == InstancePropertiesTabId );

	return SNew(SDockTab)
		//.Icon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.CustomizableInstanceProperties"))
		.Label( FText::FromString( GetTabPrefix() + LOCTEXT( "CustomizableInstanceProperties_TabTitle", "Instance Properties" ).ToString() ) )
		[
			CustomizableInstanceDetailsView.ToSharedRef()
		];
}


TSharedRef<SDockTab> FCustomizableObjectInstanceEditor::SpawnTab_AdvancedPreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == AdvancedPreviewSettingsTabId);
	return SNew(SDockTab)
		//.Icon(FCustomizableObjectEditorStyle::Get().GetBrush("CustomizableObjectEditor.Tabs.PreviewSettings"))
		.Label(LOCTEXT("StaticMeshPreviewScene_TabTitle", "Preview Scene Settings"))
		[
			AdvancedPreviewSettingsWidget.ToSharedRef()
		];
}


UCustomizableObjectInstance* FCustomizableObjectInstanceEditor::GetPreviewInstance()
{
	return CustomizableObjectInstance;
}


void FCustomizableObjectInstanceEditor::BindCommands()
{
	const FCustomizableObjectInstanceEditorCommands& Commands = FCustomizableObjectInstanceEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();
	
	UICommandList->MapAction(
		Commands.ShowParentCO,
		FExecuteAction::CreateSP(this, &FCustomizableObjectInstanceEditor::ShowParentInContentBrowser),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectInstanceEditor::CanOpenOrShowParent),
		FIsActionChecked()
	);

	UICommandList->MapAction(
		Commands.EditParentCO,
		FExecuteAction::CreateSP(this, &FCustomizableObjectInstanceEditor::OpenParentInEditor),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectInstanceEditor::CanOpenOrShowParent),
		FIsActionChecked()
	);

	// Texture Analyzer
	UICommandList->MapAction(
		Commands.TextureAnalyzer,
		FExecuteAction::CreateSP(this, &FCustomizableObjectInstanceEditor::OpenTextureAnalyzerTab),
		FCanExecuteAction(),
		FIsActionChecked());
}


void FCustomizableObjectInstanceEditor::ExtendToolbar()
{
	TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FCustomizableObjectInstanceEditor* Editor, TSharedPtr<FUICommandList> CommandList)
		{
			ToolbarBuilder.BeginSection("Utilities");
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectInstanceEditorCommands::Get().ShowParentCO);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectInstanceEditorCommands::Get().EditParentCO);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectInstanceEditorCommands::Get().TextureAnalyzer);
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this, CommandList));

	AddToolbarExtender(ToolbarExtender);

	ICustomizableObjectEditorModule* CustomizableObjectEditorModule = &FModuleManager::LoadModuleChecked<ICustomizableObjectEditorModule>("CustomizableObjectEditor");
	AddToolbarExtender(CustomizableObjectEditorModule->GetCustomizableObjectEditorToolBarExtensibilityManager()->GetAllExtenders());
}


FText FCustomizableObjectInstanceEditor::GetToolkitName() const
{
	return FText::FromString(GetEditingObject()->GetName());
}


FString FCustomizableObjectInstanceEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT( "WorldCentricTabPrefix", "CustomizableObjectInstance " ).ToString();
}


FLinearColor FCustomizableObjectInstanceEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.3f, 0.2f, 0.5f, 0.5f );
}

FEditorModeTools* FCustomizableObjectInstanceEditor::GetEditorModeTools()
{
	return EditorModeManager.IsValid() ? EditorModeManager.Get() : nullptr;
}

void FCustomizableObjectInstanceEditor::RefreshTool()
{
	Viewport->GetViewportClient()->Invalidate();
}


TSharedPtr<SCustomizableObjectEditorViewportTabBody> FCustomizableObjectInstanceEditor::GetViewport()
{
	return Viewport;		
}


UProjectorParameter* FCustomizableObjectInstanceEditor::GetProjectorParameter()
{
	return ProjectorParameter;
}


UCustomSettings* FCustomizableObjectInstanceEditor::GetCustomSettings()
{
	return CustomSettings;
}


void FCustomizableObjectInstanceEditor::HideGizmo()
{
	HideGizmo(SharedThis(this), Viewport, CustomizableInstanceDetailsView);
}


void FCustomizableObjectInstanceEditor::ShowGizmoProjectorParameter(const FString& ParamName, int32 RangeIndex)
{
	HideGizmo();

	ShowGizmoProjectorParameter(ParamName, RangeIndex, SharedThis(this), Viewport, CustomizableInstanceDetailsView, ProjectorParameter, CustomizableObjectInstance);
}


void FCustomizableObjectInstanceEditor::HideGizmoProjectorParameter()
{
	HideGizmoProjectorParameter(SharedThis(this), Viewport, CustomizableInstanceDetailsView);
}


UCustomizableObjectEditorProperties* FCustomizableObjectInstanceEditor::GetEditorProperties()
{
	return EditorProperties;
}


TSharedPtr<SCustomizableObjectEditorAdvancedPreviewSettings> FCustomizableObjectInstanceEditor::GetAdvancedPreviewSettings()
{
	return AdvancedPreviewSettingsWidget;
}


bool FCustomizableObjectInstanceEditor::ShowLightingSettings()
{
	return false;
}


bool FCustomizableObjectInstanceEditor::ShowProfileManagementOptions()
{
	return false;
}


const UObject* FCustomizableObjectInstanceEditor::GetObjectBeingEdited()
{
	check(GetObjectsCurrentlyBeingEdited()->Num());

	return (*GetObjectsCurrentlyBeingEdited())[0];
}


void FCustomizableObjectInstanceEditor::OnInstancePropertySelectionChanged(FProperty* InProperty)
{
	Viewport->GetViewportClient()->Invalidate();
}


void FCustomizableObjectInstanceEditor::OnUpdatePreviewInstance(UCustomizableObjectInstance* Instance)
{
	if (TextureAnalyzer.IsValid())
	{
		TextureAnalyzer.Get()->RefreshTextureAnalyzerTable(CustomizableObjectInstance);
	}
}


bool FCustomizableObjectInstanceEditor::IsTickable(void) const
{
	return true;
}


void FCustomizableObjectInstanceEditor::Tick(float InDeltaTime)
{
	check(CustomizableObjectInstance);
	
	// If we want to show the Relevant/Runtime parameters, we need to refresh the details view to make sure that the scroll bar appears		
	if (bOnlyRelevantParameters != CustomizableObjectInstance->GetPrivate()->bShowOnlyRelevantParameters)
	{
		bOnlyRelevantParameters = CustomizableObjectInstance->GetPrivate()->bShowOnlyRelevantParameters;
		CustomizableInstanceDetailsView->ForceRefresh();
	}

	if (bOnlyRuntimeParameters != CustomizableObjectInstance->GetPrivate()->bShowOnlyRuntimeParameters)
	{
		bOnlyRuntimeParameters = CustomizableObjectInstance->GetPrivate()->bShowOnlyRuntimeParameters;
		CustomizableInstanceDetailsView->ForceRefresh();
	}
}


TStatId FCustomizableObjectInstanceEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCustomizableObjectInstanceEditor, STATGROUP_Tickables);
}


void FCustomizableObjectInstanceEditor::OnCustomizableObjectStatusChanged(FCustomizableObjectStatus::EState PreviousState, const FCustomizableObjectStatus::EState CurrentState)
{
	if (PreviousState == FCustomizableObjectStatusTypes::EState::Loading)
	{
		if (CurrentState == FCustomizableObjectStatusTypes::EState::ModelLoaded)
		{
			if (Viewport)
			{
				Viewport->CreatePreviewActor(CustomizableObjectInstance);
			}
			CustomizableObjectInstance->UpdateSkeletalMeshAsync(true, true);
		}
		else if (CurrentState == FCustomizableObjectStatusTypes::EState::NoModel)
		{
			if (UCustomizableObject* CustomizableObject = CustomizableObjectInstance->GetCustomizableObject())
			{
				CustomizableObject->ConditionalAutoCompile();
			}
		}
	}
}


void FCustomizableObjectInstanceEditor::OpenTextureAnalyzerTab()
{
	TabManager->TryInvokeTab(TextureAnalyzerTabId);
}


void FCustomizableObjectInstanceEditor::OnPostCompile()
{
	Viewport->CreatePreviewActor(CustomizableObjectInstance);
	CustomizableObjectInstance->UpdateSkeletalMeshAsync(true, true);
}


void FCustomizableObjectInstanceEditor::HideGizmo(const TSharedPtr<ICustomizableObjectInstanceEditor>& Editor,
                                                  const TSharedPtr<SCustomizableObjectEditorViewportTabBody>& Viewport,
                                                  const TSharedPtr<IDetailsView>& InstanceDetailsView)
{
	HideGizmoProjectorParameter(Editor, Viewport, InstanceDetailsView);
}


void FCustomizableObjectInstanceEditor::ShowGizmoProjectorParameter(const FString& ParamName, int32 RangeIndex,
	const TSharedPtr<ICustomizableObjectInstanceEditor>& Editor, const TSharedPtr<SCustomizableObjectEditorViewportTabBody>& Viewport,
	const TSharedPtr<IDetailsView>& InstanceDetailsView, UProjectorParameter* ProjectorParameter,
	UCustomizableObjectInstance* Instance)
{	
	ProjectorParameter->SelectProjector(ParamName, RangeIndex);

	const FCustomizableObjectProjector ProjectorData = Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex);
	ProjectorParameter->SetPosition(static_cast<FVector>(ProjectorData.Position));
	ProjectorParameter->SetDirection(static_cast<FVector>(ProjectorData.Direction));
	ProjectorParameter->SetUp(static_cast<FVector>(ProjectorData.Up));
	ProjectorParameter->SetScale(static_cast<FVector>(ProjectorData.Scale));

	TWeakObjectPtr<UCustomizableObjectInstance> WeakInstance = Instance;

	const TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor = Editor.ToWeakPtr();
	
	FProjectorTypeDelegate  ProjectorTypeDelegate;
	ProjectorTypeDelegate.BindLambda([WeakInstance, ParamName, RangeIndex]() -> ECustomizableObjectProjectorType
	{
		const UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return {};
		}

		return Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex).ProjectionType;
	});
	
	FWidgetColorDelegate WidgetColorDelegate;
	WidgetColorDelegate.BindLambda([]() { return FColor::Green; });
		
	// Position
	FWidgetLocationDelegate WidgetLocationDelegate;
	WidgetLocationDelegate.BindLambda([WeakEditor, WeakInstance]() -> FVector
	{
		const UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return {};
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return {};
		}
		
		return Editor->GetProjectorParameter()->GetPosition(); // We are not getting the value directly from the parameters since they get updated on ReloadParameters, making the gizmo jittery.
	});

	FOnWidgetLocationChangedDelegate OnWidgetLocationChangedDelegate;
	OnWidgetLocationChangedDelegate.BindLambda([WeakEditor, WeakInstance, ParamName, RangeIndex](const FVector& Location)
	{		
		UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return;
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return;
		}
		
		Editor->GetProjectorParameter()->SetPosition(Location);

		FCustomizableObjectProjector ProjectorData = Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex);
		ProjectorData.Position = static_cast<FVector3f>(Location);
		Instance->SetProjectorParameterSelectedOption(ParamName, ProjectorData, RangeIndex);
		Instance->UpdateSkeletalMeshAsync(true, true);
	});

	// Direction
	FWidgetDirectionDelegate WidgetDirectionDelegate;
	WidgetDirectionDelegate.BindLambda([WeakEditor, WeakInstance]() -> FVector
	{
		const UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return {};
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return {};
		}
		
		return Editor->GetProjectorParameter()->GetDirection();
	});
	
	FOnWidgetDirectionChangedDelegate OnWidgetDirectionChangedDelegate;
	OnWidgetDirectionChangedDelegate.BindLambda([WeakEditor, WeakInstance, ParamName, RangeIndex](const FVector& Direction)
	{
		UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return;
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return;
		}
		
		Editor->GetProjectorParameter()->SetDirection(Direction);

		FCustomizableObjectProjector ProjectorData = Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex);
		ProjectorData.Direction = static_cast<FVector3f>(Direction);
		Instance->SetProjectorParameterSelectedOption(ParamName, ProjectorData, RangeIndex);
		Instance->UpdateSkeletalMeshAsync(true, true);
	});

	// Up
	FWidgetUpDelegate WidgetUpDelegate;
	WidgetUpDelegate.BindLambda([WeakEditor, WeakInstance]() -> FVector
	{
		const UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return {};
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return {};
		}
		
		return Editor->GetProjectorParameter()->GetUp();
	});

	FOnWidgetUpChangedDelegate OnWidgetUpChangedDelegate;
	OnWidgetUpChangedDelegate.BindLambda([WeakEditor, WeakInstance, ParamName, RangeIndex](const FVector& Up)
	{
		UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return;
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return;
		}
		
		Editor->GetProjectorParameter()->SetUp(Up);

		FCustomizableObjectProjector ProjectorData = Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex);
		ProjectorData.Up = static_cast<FVector3f>(Up);
		Instance->SetProjectorParameterSelectedOption(ParamName, ProjectorData, RangeIndex);
		Instance->UpdateSkeletalMeshAsync(true, true);
	});

	// Scale
	FWidgetScaleDelegate WidgetScaleDelegate;
	WidgetScaleDelegate.BindLambda([WeakEditor, WeakInstance]() -> FVector
	{
		const UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return {};
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return {};
		}
		
		return Editor->GetProjectorParameter()->GetScale();
	});
	
	FOnWidgetScaleChangedDelegate OnWidgetScaleChangedDelegate;
	OnWidgetScaleChangedDelegate.BindLambda([WeakEditor, WeakInstance, ParamName, RangeIndex](const FVector& Scale)
	{
		UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return;
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return;
		}
		
		Editor->GetProjectorParameter()->SetScale(Scale);

		FCustomizableObjectProjector ProjectorData = Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex);
		ProjectorData.Scale = static_cast<FVector3f>(Scale);
		Instance->SetProjectorParameterSelectedOption(ParamName, ProjectorData, RangeIndex);
		Instance->UpdateSkeletalMeshAsync(true, true);
	});

	// Angle
	FWidgetAngleDelegate WidgetAngleDelegate;
	WidgetAngleDelegate.BindLambda([WeakInstance, ParamName, RangeIndex]() -> float
	{
		const UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return {};
		}

		return Instance->GetProjectorParameterSelectedOption(ParamName, RangeIndex).Angle;
	});

	// UObject transactions
	FWidgetTrackingStartedDelegate WidgetTrackingStartedDelegate;
	WidgetTrackingStartedDelegate.BindLambda([WeakEditor, WeakInstance]()
	{
		UCustomizableObjectInstance* Instance = WeakInstance.Get();
		if (!Instance)
		{
			return;
		}

		const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
		if (!Editor)
		{
			return;
		}
		
		Instance->Modify();
		Editor->GetProjectorParameter()->Modify();
	});

	Viewport->ShowGizmoProjector(WidgetLocationDelegate, OnWidgetLocationChangedDelegate,
		WidgetDirectionDelegate, OnWidgetDirectionChangedDelegate,
		WidgetUpDelegate, OnWidgetUpChangedDelegate,
		WidgetScaleDelegate, OnWidgetScaleChangedDelegate,
		WidgetAngleDelegate,
		ProjectorTypeDelegate,
		WidgetColorDelegate,
		WidgetTrackingStartedDelegate);
	
	InstanceDetailsView->ForceRefresh();
}


void FCustomizableObjectInstanceEditor::HideGizmoProjectorParameter(const TSharedPtr<ICustomizableObjectInstanceEditor>& Editor,
		const TSharedPtr<SCustomizableObjectEditorViewportTabBody>& Viewport,
		const TSharedPtr<IDetailsView>& InstanceDetailsView)
{
	Viewport->HideGizmoProjector();

	UProjectorParameter* ProjectorParameter = Editor->GetProjectorParameter();
	
	ProjectorParameter->UnselectProjector();
	InstanceDetailsView->ForceRefresh();		
}


TSharedRef<SDockTab> FCustomizableObjectInstanceEditor::SpawnTab_TextureAnalyzer(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TextureAnalyzerTabId);
	return SNew(SDockTab)
		.Label(LOCTEXT("Texture Analyzer", "Texture Analyzer"))
		[
			TextureAnalyzer.ToSharedRef()
		];
}


#undef LOCTEXT_NAMESPACE
