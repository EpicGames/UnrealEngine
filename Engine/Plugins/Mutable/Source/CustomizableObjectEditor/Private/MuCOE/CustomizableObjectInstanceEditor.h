// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailCategoryBuilder.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/ICustomizableObjectInstanceEditor.h"
#include "MuCOE/CustomizableObjectEditorViewportLights.h"

#include "TickableEditorObject.h"
#include "GameplayTagContainer.h"

#include "CustomizableObjectInstanceEditor.generated.h"

class FProperty;
class FSpawnTabArgs;
class SDockTab;
class SWidget;
class UCustomizableObject;
class UCustomizableObjectInstance;
class UPoseAsset;
class UAnimationAsset;
class SLevelOfDetailSettings;
class SCustomizableObjectEditorViewportTabBody;
class SCustomizableObjecEditorTextureAnalyzer;
class FCustomizableObjectCompiler;
class UCustomizableSkeletalComponent;
class UDebugSkelMeshComponent;
class SDockableTab;
struct FFrame;

DECLARE_DELEGATE(FCreatePreviewInstanceFlagDelegate);


/**
* Wrapper UObject class for the UCustomizableObjectInstance::FObjectInstanceUpdatedDelegate dynamic multicast delegate
*/
UCLASS()
class UUpdateClassWrapperClass : public UObject
{
public:
	GENERATED_BODY()

	/** Method to assign for the callback */
	UFUNCTION()
	void DelegatedCallback(UCustomizableObjectInstance* Instance);

	FCreatePreviewInstanceFlagDelegate Delegate;
};


/** Currently selected projector parameter and its position.
 *
 * Only used by Instance Parameters. Default values projector values modify the node directly. */
UCLASS(Transient)
class UProjectorParameter : public UObject
{
	GENERATED_BODY()
	
public:
	UProjectorParameter();

	void SelectProjector(const FString& ParamName, int32 RangeIndex = -1);
	
	void UnselectProjector();

	bool IsProjectorSelected(const FString& InParamName, int32 InRangeIndex = -1) const;
	
	FVector GetPosition() const;

	void SetPosition(const FVector& Position);

	FVector GetDirection() const;

	void SetDirection(const FVector& Direction);

	FVector GetUp() const;

	void SetUp(const FVector& Up);

	FVector GetScale() const;

	void SetScale(const FVector& Scale);
	
private:
	UPROPERTY()
	FString ParamName = {};

	UPROPERTY()
	int32 RangeIndex = -1;
	
	UPROPERTY()
	FVector Position = FVector::ZeroVector;

	UPROPERTY()
	FVector Direction = FVector::ForwardVector;

	UPROPERTY()
	FVector Up = FVector::UpVector;

	UPROPERTY()
	FVector Scale = FVector::OneVector;
};


UCLASS(Transient)
class UCustomSettings : public UObject
{
	GENERATED_BODY()

public:
	// UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	ULightComponent* GetSelectedLight() const;

	void SetSelectedLight(ULightComponent* Light);

	UCustomizableObjectEditorViewportLights* GetLightsPreset() const;

	void SetLightsPreset(UCustomizableObjectEditorViewportLights& LightsPreset);

	TWeakPtr<ICustomizableObjectInstanceEditor> GetEditor() const;

	void SetEditor(TSharedPtr<ICustomizableObjectInstanceEditor> Editor);
	
	UPROPERTY(Category = Animation, EditAnywhere)
	TObjectPtr<UAnimationAsset> Animation;
	
private:
	UPROPERTY()
	TObjectPtr<ULightComponent> SelectedLight = nullptr;

	UPROPERTY()
	TObjectPtr<UCustomizableObjectEditorViewportLights> LightsPreset;

	TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor;
};

USTRUCT()
struct FCustomizableObjectGameplayTagsFilter
{
	GENERATED_BODY();

public:
	/** Filter Gameplay tags to match with. */
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Tags"), Category = NoCategory)
	FGameplayTagContainer GameplayTagsFilter;

	/** Filter match type. */
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Match type"), Category = NoCategory)
	EGameplayContainerMatchType GameplayTagsFilterType = EGameplayContainerMatchType::Any;
};

UCLASS(Transient)
class UCustomizableObjectEditorProperties : public UObject
{
	GENERATED_BODY()

public:

	/** Gameplay tags based parameter options filter.
	  * Tags may be added to the Editor Gameplay Tags container in the parameters metadata.
	  * Only applies to the options shown in drop downs. Child object parameters and data table rows. */
	UPROPERTY(meta = (DisplayName = "Gameplay Tags Filter"))
	FCustomizableObjectGameplayTagsFilter Filter;
};


/**
 * CustomizableObject Editor class
 */
class FCustomizableObjectInstanceEditor : 
	public ICustomizableObjectInstanceEditor,
	public FAssetEditorToolkit,
	public FGCObject,
	public FTickableEditorObject
{
public:
	FCustomizableObjectInstanceEditor();
	virtual ~FCustomizableObjectInstanceEditor() override;

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified object
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit			The object to edit
	 */
	void InitCustomizableObjectInstanceEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UCustomizableObjectInstance* ObjectToEdit );

	// FSerializableObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FCustomizableObjectInstanceEditor");
	}
	// End of FSerializableObject interface

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** ICustomizableObjectInstanceEditor interface */
	virtual FEditorModeTools* GetEditorModeTools() override;
	virtual UCustomizableObjectInstance* GetPreviewInstance() override;
	virtual void RefreshTool() override;
	virtual TSharedPtr<SCustomizableObjectEditorViewportTabBody> GetViewport() override;
	virtual UProjectorParameter* GetProjectorParameter() override;
	virtual UCustomSettings* GetCustomSettings() override;
	virtual void HideGizmo() override;
	virtual void ShowGizmoProjectorParameter(const FString& ParamName, int32 RangeIndex) override;
	virtual void HideGizmoProjectorParameter() override;
	virtual UCustomizableObjectEditorProperties* GetEditorProperties() override;
	virtual TSharedPtr<SCustomizableObjectEditorAdvancedPreviewSettings> GetAdvancedPreviewSettings() override;
	virtual bool ShowLightingSettings() override;
	virtual bool ShowProfileManagementOptions() override;
	virtual const UObject* GetObjectBeingEdited() override;

	/** Callback to notify the editor when the PreviewInstance has been updated */
	void OnUpdatePreviewInstance(UCustomizableObjectInstance* Instance);

	/** FTickableGameObject interface */
	virtual bool IsTickable(void) const override;
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;

	void OnCustomizableObjectStatusChanged(FCustomizableObjectStatus::EState PreviousState, FCustomizableObjectStatus::EState CurrentState);
	
private:

	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_InstanceProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AdvancedPreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_TextureAnalyzer(const FSpawnTabArgs& Args);

	/** Binds commands associated with the Static Mesh Editor. */
	void BindCommands();
		
	/** Callback when selection changes in the Property Tree. */
	void OnInstancePropertySelectionChanged(FProperty* InProperty);

	/** Save Customizable Object Instance open in the editor */
	void SaveAsset_Execute() override;

	/** Says if the customizable Object can be shown or be opened in the editor */
	bool CanOpenOrShowParent();

	/** Show Customizable Object Instance's Parent In Content Browser */
	void ShowParentInContentBrowser();

	/** Open Customizable Object Instance's Parent In Editor */
	void OpenParentInEditor();

	/** Open the Texture Analyzer tab */
	void OpenTextureAnalyzerTab();

public:
	/** Static to reuse code with FCustomizableObjectEditor. */
	static void HideGizmo(const TSharedPtr<ICustomizableObjectInstanceEditor>& Editor,
		const TSharedPtr<SCustomizableObjectEditorViewportTabBody>& Viewport,
		const TSharedPtr<IDetailsView>& InstanceDetailsView);

	/** Static to reuse code with FCustomizableObjectEditor. */
	static void ShowGizmoProjectorParameter(
		const FString& ParamName, int32 RangeIndex,
		const TSharedPtr<ICustomizableObjectInstanceEditor>& Editor, const TSharedPtr<SCustomizableObjectEditorViewportTabBody>& Viewport, const TSharedPtr<IDetailsView>& InstanceDetailsView,
		UProjectorParameter* ProjectorParameter, UCustomizableObjectInstance* Instance);

	/** Static to reuse code with FCustomizableObjectEditor. */
	static void HideGizmoProjectorParameter(const TSharedPtr<ICustomizableObjectInstanceEditor>& Editor,
		const TSharedPtr<SCustomizableObjectEditorViewportTabBody>& Viewport,
		const TSharedPtr<IDetailsView>& InstanceDetailsView);
	
private:
	
	void OnPostCompile();
	
	/** The currently viewed object. */
	TObjectPtr<UCustomizableObjectInstance> CustomizableObjectInstance;

	/** Preview Actor. All preview components are attached to this actor. */
	TStrongObjectPtr<AActor> Actor;
	
	TArray<TWeakObjectPtr<UDebugSkelMeshComponent>> PreviewSkeletalMeshComponents;

	/** List of open tool panels; used to ensure only one exists at any one time */
	TMap<FName, TWeakPtr<SDockableTab>> SpawnedToolPanels;

	/** Preview Viewport widget */
	TSharedPtr<SCustomizableObjectEditorViewportTabBody> Viewport;
	TSharedPtr<IDetailsView> CustomizableInstanceDetailsView;

	
	/** Level of Details Settings widget. */
	TSharedPtr<SLevelOfDetailSettings> LevelOfDetailSettings;

	/** Widget for displaying the available UV Channels. */
	TSharedPtr<STextComboBox> UVChannelCombo;

	/** List of available UV Channels. */
	TArray<TSharedPtr<FString>> UVChannels;

	/** Widget for displaying the available LOD. */
	TSharedPtr<STextComboBox> LODLevelCombo;

	/** List of LODs. */
	TArray<TSharedPtr<FString>> LODLevels;

	/**	The tab ids for all the tabs used */
	static const FName ViewportTabId;
	static const FName InstancePropertiesTabId;
	static const FName AdvancedPreviewSettingsTabId;
	static const FName TextureAnalyzerTabId;

	/** Handle for the OnObjectModified event */
	FDelegateHandle OnObjectModifiedHandle;

	/** UObject class to be able to use the update callback */
	TObjectPtr<UUpdateClassWrapperClass> HelperCallback;

	/** Scene preview settings widget */
	TSharedPtr<SCustomizableObjectEditorAdvancedPreviewSettings> AdvancedPreviewSettingsWidget;

	/** Pose asset when doing drag and drop of an UPoseAsset to the viewport */
	UPoseAsset* PoseAsset;

	/** Texture Analyzer table widget which shows the information of the transient textures used in the customizable object instance */
	TSharedPtr<SCustomizableObjecEditorTextureAnalyzer> TextureAnalyzer;

	/** Variables used to force the refresh of the details view widget. These are needed because sometimes the scrollbar of the window doesn't appear
	    until we force the refresh */
	bool bOnlyRuntimeParameters;
	bool bOnlyRelevantParameters;

	TObjectPtr<UProjectorParameter> ProjectorParameter = nullptr;

	TObjectPtr<UCustomSettings> CustomSettings = nullptr;

	TObjectPtr<UCustomizableObjectEditorProperties> EditorProperties = nullptr;

	/** Adds the customizable Object Instance Editor commands to the default toolbar */
	void ExtendToolbar();
};
