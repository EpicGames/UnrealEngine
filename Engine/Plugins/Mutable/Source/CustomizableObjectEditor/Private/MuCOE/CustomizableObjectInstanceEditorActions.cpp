// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectInstanceEditorActions.h"

#include "MuCOE/CustomizableObjectEditorStyle.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FCustomizableObjectInstanceEditorCommands::FCustomizableObjectInstanceEditorCommands() 
	: TCommands<FCustomizableObjectInstanceEditorCommands>
(
	"CustomizableObjectInstanceEditor", // Context name for fast lookup
	NSLOCTEXT("Contexts", "CustomizableObjecInstancetEditor", "CustomizableObjectInstance Editor"), // Localized context name for displaying
	NAME_None, // Parent
	FCustomizableObjectEditorStyle::GetStyleSetName() // Icon Style Set
	)
{
}


void FCustomizableObjectInstanceEditorCommands::RegisterCommands()
{
	UI_COMMAND( SetDrawUVs, "UV", "Toggles display of the static mesh's UVs for the specified channel.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowGrid, "Grid", "Displays the viewport grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( SetShowSky, "Sky", "Displays the viewport sky.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowBounds, "Bounds", "Toggles display of the bounds of the static mesh.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowCollision, "Collision", "Toggles display of the simplified collision mesh of the static mesh, if one has been assigned.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetCameraLock, "Camera Lock", "Toggles viewport navigation between orbit and freely moving about.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SaveThumbnail, "Save Thumbnail", "Saves the viewpoint position and orientation in the Preview Pane for use as the thumbnail preview in the Content Browser.", EUserInterfaceActionType::Button, FInputChord() );

	UI_COMMAND(ShowParentCO, "Show Parent", "Show the Parent of the Customizable Object in the Content Browser", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditParentCO, "Edit Parent", "Open the Parent of the Customizable Object to Edit", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TextureAnalyzer, "Texture Memory Analyzer", "Open the Texture Analyzer window to check all the information of the textures created by Mutable.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
