// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FUICommandInfo;

/**
 * Unreal StaticMesh editor actions
 */
class FCustomizableObjectInstanceEditorCommands : public TCommands<FCustomizableObjectInstanceEditorCommands>
{

public:
	FCustomizableObjectInstanceEditorCommands();

	TSharedPtr< FUICommandInfo > SetDrawUVs;
	TSharedPtr< FUICommandInfo > SetShowGrid;
	TSharedPtr< FUICommandInfo > SetShowSky;
	TSharedPtr< FUICommandInfo > SetShowBounds;
	TSharedPtr< FUICommandInfo > SetShowCollision;
	TSharedPtr< FUICommandInfo > SetCameraLock;
	TSharedPtr< FUICommandInfo > SaveThumbnail;

	//Toolbar Commands
	TSharedPtr< FUICommandInfo > ShowParentCO;
	TSharedPtr< FUICommandInfo > EditParentCO;
	TSharedPtr< FUICommandInfo > TextureAnalyzer;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

};
