// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectInstanceFactory.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableSkeletalComponent.h"
#include "MuCO/CustomizableSkeletalMeshActor.h"
#include "MuCO/CustomizableSkeletalMeshActorPrivate.h"
#include "MuCO/LoadUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectInstanceFactory)

#define LOCTEXT_NAMESPACE "CustomizableObjectInstanceFactory"


UCustomizableObjectInstanceFactory::UCustomizableObjectInstanceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
    DisplayName = LOCTEXT("CustomizableObjectInstanceDisplayName", "Customizable Object Instance");
    NewActorClass = ACustomizableSkeletalMeshActor::StaticClass();
    bUseSurfaceOrientation = true;
}


AActor* UCustomizableObjectInstanceFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	AActor* NewActor = Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams);
	
	ACustomizableSkeletalMeshActor* NewCSMActor = Cast<ACustomizableSkeletalMeshActor>(NewActor);
    if (!NewCSMActor)
    {
    	return nullptr;
    }
	
	UCustomizableObjectInstance* Instance = Cast<UCustomizableObjectInstance>(InAsset);
	if (!Instance)
	{
		return nullptr;
	}

	UCustomizableObject* Object = Instance->GetCustomizableObject();
	if (!Object)
	{
		return nullptr;
	}
	
	NewCSMActor->GetPrivate()->Init(Instance);
	
	return NewActor;
}


UObject* UCustomizableObjectInstanceFactory::GetAssetFromActorInstance(AActor* ActorInstance)
{
	if (ACustomizableSkeletalMeshActor* CSMActor = CastChecked<ACustomizableSkeletalMeshActor>(ActorInstance))
	{
		const TArray<TObjectPtr<UCustomizableSkeletalComponent>>& Components = CSMActor->GetPrivate()->GetComponents();
		if (Components.IsValidIndex(0))
		{
			return Components[0]; 
		}
	}
	
	return nullptr;
}


bool UCustomizableObjectInstanceFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
    if (!AssetData.IsValid() ||
        (!AssetData.GetClass()->IsChildOf(UCustomizableObjectInstance::StaticClass())))
    {
        OutErrorMsg = LOCTEXT("NoCOISeq", "A valid customizable object instance must be specified.");
        return false;
    }

    FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
    FAssetData CustomizableObjectInstanceData;

    if (AssetData.GetClass()->IsChildOf(UCustomizableObjectInstance::StaticClass()))
    {
        if (UCustomizableObjectInstance* CustomizableObjectInstance = Cast<UCustomizableObjectInstance>(UE::Mutable::Private::LoadObject(AssetData)))
        {
			TArray<FName> ComponentNames = CustomizableObjectInstance->GetComponentNames();

			// If there is a valid component with skeletal mesh
			// TODO: This is not accurate in the future, if we have components that have grooms or clothing instead.
            for (FName ComponentName : ComponentNames)
            {
				if (USkeletalMesh* SkeletalMesh = CustomizableObjectInstance->GetComponentMeshSkeletalMesh(ComponentName))
				{
					return true;
				}
            }

			{
                if (UCustomizableObject* CustomizableObject = CustomizableObjectInstance->GetCustomizableObject())
                {
                    return true;
                }
                else
                {
                    OutErrorMsg = LOCTEXT("NoCustomizableObjectInstance", "The UCustomizableObjectInstance does not have a customizableObject.");
                    return false;
                }
            }
        }
        else
        {
            OutErrorMsg = LOCTEXT("NoCustomizableObjectInstanceIsNull", "The CustomizableObjectInstance is null.");
        }
    }

    if (USkeletalMesh* SkeletalMeshCDO = Cast<USkeletalMesh>(AssetData.GetClass()->GetDefaultObject()))
    {
        if (SkeletalMeshCDO->HasCustomActorFactory())
        {
            return false;
        }
    }

    return true;
}


FQuat UCustomizableObjectInstanceFactory::AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const
{
    // Meshes align the Z (up) axis with the surface normal
    return FindActorAlignmentRotation(ActorRotation, FVector(0.f, 0.f, 1.f), InSurfaceNormal);
}

#undef LOCTEXT_NAMESPACE
