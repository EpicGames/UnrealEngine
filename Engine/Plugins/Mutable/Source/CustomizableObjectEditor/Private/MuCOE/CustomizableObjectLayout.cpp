// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectLayout.h"

#include "Engine/StaticMesh.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCOE/CompilationOptions.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceLayout.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/MutableUtils.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectLayout)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

UCustomizableObjectLayout::UCustomizableObjectLayout()
{
	GridSize = FIntPoint(4, 4);
	MaxGridSize = FIntPoint(4, 4);

	FCustomizableObjectLayoutBlock Block(FIntPoint(0, 0), FIntPoint(4, 4));
	Blocks.Add(Block);

	PackingStrategy = ECustomizableObjectTextureLayoutPackingStrategy::Resizable;
	BlockReductionMethod = ECustomizableObjectLayoutBlockReductionMethod::Halve;
}


void UCustomizableObjectLayout::SetLayout(int32 LODIndex, int32 MatIndex, int32 UVIndex)
{
	LOD = LODIndex;
	Material = MatIndex;
	UVChannel = UVIndex;
}

UE::Mutable::Private::EAutoBlocksStrategy ConvertAutomaticBlocksStrategy(const ECustomizableObjectLayoutAutomaticBlocksStrategy InAutomaticBlockStrategy)
{
	UE::Mutable::Private::EAutoBlocksStrategy AutoBlockStrategy = UE::Mutable::Private::EAutoBlocksStrategy::Rectangles;

	switch (InAutomaticBlockStrategy)
	{
	case ECustomizableObjectLayoutAutomaticBlocksStrategy::Rectangles:
		AutoBlockStrategy = UE::Mutable::Private::EAutoBlocksStrategy::Rectangles;
		break;

	case ECustomizableObjectLayoutAutomaticBlocksStrategy::UVIslands:
		AutoBlockStrategy = UE::Mutable::Private::EAutoBlocksStrategy::UVIslands;
		break;

	case ECustomizableObjectLayoutAutomaticBlocksStrategy::Ignore:
		AutoBlockStrategy = UE::Mutable::Private::EAutoBlocksStrategy::Ignore;
		break;

	default:
		checkNoEntry();
	}

	return AutoBlockStrategy;
}



UE::Mutable::Private::EPackStrategy ConvertLayoutStrategy(const ECustomizableObjectTextureLayoutPackingStrategy LayoutPackStrategy)
{
	UE::Mutable::Private::EPackStrategy PackStrategy = UE::Mutable::Private::EPackStrategy::Fixed;

	switch (LayoutPackStrategy)
	{
	case ECustomizableObjectTextureLayoutPackingStrategy::Fixed:
		PackStrategy = UE::Mutable::Private::EPackStrategy::Fixed;
		break;

	case ECustomizableObjectTextureLayoutPackingStrategy::Resizable:
		PackStrategy = UE::Mutable::Private::EPackStrategy::Resizeable;
		break;

	case ECustomizableObjectTextureLayoutPackingStrategy::Overlay:
		PackStrategy = UE::Mutable::Private::EPackStrategy::Overlay;
		break;

	default:
		checkNoEntry();
	}

	return PackStrategy;
}


void UCustomizableObjectLayout::SetGridSize(FIntPoint Size)
{
	GridSize = Size;
}


void UCustomizableObjectLayout::SetMaxGridSize(FIntPoint Size)
{
	MaxGridSize = Size;
}


void UCustomizableObjectLayout::SetLayoutName(FString Name)
{
	LayoutName = Name;
}


void UCustomizableObjectLayout::GenerateAutomaticBlocksFromUVs()
{
	UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(GetOuter());
	TSoftObjectPtr<const UObject> Mesh = GetMesh();

	if (!Node || !Mesh)
	{
		return;
	}

	TSharedPtr<FCustomizableObjectEditor> Editor = StaticCastSharedPtr<FCustomizableObjectEditor>(Node->GetGraphEditor());

	if (AutomaticBlocksStrategy == ECustomizableObjectLayoutAutomaticBlocksStrategy::Ignore || !Editor)
	{
		return;
	}

	// Create a GenerationContext
	TSharedRef<FCustomizableObjectCompiler> Compiler = MakeShared<FCustomizableObjectCompiler>();
	UCustomizableObject* Object = Editor->GetCustomizableObject();
	check(Object);

	FCompilationOptions Options = GetCompilationOptions(*Object);

	TSharedPtr<FMutableCompilationContext> CompilationContext = MakeShared<FMutableCompilationContext>(Object, Compiler, Options);
	FMutableGraphGenerationContext GenerationContext(CompilationContext);

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode = CreateMutableLayoutNode(this, false);
	check(LayoutNode);

	if (PackingStrategy == ECustomizableObjectTextureLayoutPackingStrategy::Overlay)
		{
			// Legacy behavior
			if (Blocks.IsEmpty())
			{
				LayoutNode->Blocks.SetNum(1);
				LayoutNode->Blocks[0].Min = { 0,0 };
				LayoutNode->Blocks[0].Size = LayoutNode->Size;
				LayoutNode->Blocks[0].Priority = 0;
				LayoutNode->Blocks[0].bReduceBothAxes = false;
				LayoutNode->Blocks[0].bReduceByTwo = false;
			}
		}
	else // Convert Mesh and generate blocks
	{
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh;

		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(GenerationContext.LoadObject(GetMesh())))
		{
			// We don't need all the data to generate the blocks
			const EMutableMeshConversionFlags ShapeFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreAUD;

			GenerationContext.MeshGenerationFlags.Push(ShapeFlags);

			bool bForceImmediateConversion = true;
			FMutableSourceMeshData Source;
			Source.Mesh = SkeletalMesh;
			MutableMesh = ConvertSkeletalMeshToMutable(Source, GetLOD(), GetMaterial(), GenerationContext, nullptr, bForceImmediateConversion);

			GenerationContext.MeshGenerationFlags.Pop();
		}
		else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(GenerationContext.LoadObject(GetMesh())))
		{
			MutableMesh = ConvertStaticMeshToMutable(StaticMesh, GetLOD(), GetMaterial(), GenerationContext, nullptr);
		}

		if (MutableMesh)
		{
			// Generating blocks with the mutable mesh
			if (AutomaticBlocksStrategy == ECustomizableObjectLayoutAutomaticBlocksStrategy::Rectangles)
			{
				LayoutNode->GenerateLayoutBlocks(MutableMesh, GetUVChannel());
			}
			else if (AutomaticBlocksStrategy == ECustomizableObjectLayoutAutomaticBlocksStrategy::UVIslands)
			{
				LayoutNode->GenerateLayoutBlocksFromUVIslands(MutableMesh, GetUVChannel());
			}
			else
			{
				// Unimplemented
				check(false);
			}
		}
	}

	AutomaticBlocks.Empty();

	// Generate the editor layout blocks from the mutable layout
	for (int32 i = 0; i < LayoutNode->Blocks.Num(); ++i)
	{
		FIntPoint Min = FIntPoint(LayoutNode->Blocks[i].Min.X, LayoutNode->Blocks[i].Min.Y);
		FIntPoint Size = FIntPoint(LayoutNode->Blocks[i].Size.X, LayoutNode->Blocks[i].Size.Y);

		// Ignore blocks contained inside any block in the initial block set.
		bool bContainedInInitialSet = false;
		for (const FCustomizableObjectLayoutBlock& Block : Blocks)
		{
			FInt32Rect ExistingRect(Block.Min, Block.Max + FIntPoint(1));
			if (ExistingRect.Contains(Min) && ExistingRect.Contains(Min + Size))
			{
				bContainedInInitialSet = true;
				break;
			}
		}
		if (bContainedInInitialSet)
		{
			continue;
		}

		FCustomizableObjectLayoutBlock Block(FIntPoint(Min.X, Min.Y), FIntPoint(Min.X + Size.X, Min.Y + Size.Y));
		Block.bIsAutomatic = true;
		Block.Id = FGuid::NewGuid();

		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Mask = LayoutNode->Blocks[i].Mask;
		if (Mask)
		{
			UTexture2D* UnrealImage = NewObject<UTexture2D>(UTexture2D::StaticClass());

			FMutableModelImageProperties Props;
			Props.Filter = TF_Nearest;
			Props.SRGB = true;
			Props.LODBias = 0;
			ConvertImage(UnrealImage, Mask, Props);
			UnrealImage->NeverStream = true;
			UnrealImage->UpdateResource();

			Block.Mask = UnrealImage;
		}
		AutomaticBlocks.Add(Block);
	}

	Node->PostEditChange();
	Node->GetGraph()->MarkPackageDirty();
}


void UCustomizableObjectLayout::ConsolidateAutomaticBlocks()
{
	Blocks.Append(AutomaticBlocks);
	AutomaticBlocks.Empty();
	for (FCustomizableObjectLayoutBlock& Block : Blocks)
	{
		Block.Id = FGuid::NewGuid();
		Block.bIsAutomatic = false;
	}
}


void UCustomizableObjectLayout::GetUVs(TArray<FVector2f>& UVs) const
{
	if (const UObject* Mesh = UE::Mutable::Private::LoadObject(GetMesh()))
	{
		if (const USkeletalMesh* SkeletalMesh = Cast<const USkeletalMesh>(Mesh))
		{
			UVs = GetUV(*SkeletalMesh, LOD, Material, UVChannel);
		}
		else if (const UStaticMesh* StaticMesh = Cast<const UStaticMesh>(Mesh))
		{
			UVs = GetUV(*StaticMesh, LOD, Material, UVChannel);
		}
	}
}


int32 UCustomizableObjectLayout::FindBlock(const FGuid& InId) const
{
	for (int32 Index = 0; Index < Blocks.Num(); ++Index)
	{
		if (Blocks[Index].Id == InId)
		{
			return Index;
		}
	}

	return -1;
}


void UCustomizableObjectLayout::SetIgnoreVertexLayoutWarnings(bool bValue)
{
	bIgnoreUnassignedVertexWarning = bValue;
}


void UCustomizableObjectLayout::SetIgnoreWarningsLOD(int32 LODValue)
{
	FirstLODToIgnore = LODValue;
}


TSoftObjectPtr<UStreamableRenderAsset> UCustomizableObjectLayout::GetMesh() const
{
	if (UObject* Node = GetOuter())
	{
		if (const ICustomizableObjectNodeMeshInterface* TypedNodeSkeletalMesh = Cast<ICustomizableObjectNodeMeshInterface>(Node))
		{
			return TypedNodeSkeletalMesh->GetMesh();
		}
		else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
		{
			return TypedNodeTable->GetDefaultMeshForLayout(this);
		}
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
