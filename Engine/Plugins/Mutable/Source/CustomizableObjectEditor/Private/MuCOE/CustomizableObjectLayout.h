// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MuT/NodeLayout.h"
#include "MuR/Image.h"
#include "CustomizableObjectLayout.generated.h"

class UStreamableRenderAsset;

UENUM()
enum class ECustomizableObjectTextureLayoutPackingStrategy : uint8
{
	// The layout increases its size to fit all the blocks.
	Resizable = 0 UMETA(DisplayName = "Resizable Layout"),
	// The layout resizes the blocks to keep its size.
	Fixed = 1 UMETA(DisplayName = "Fixed Layout"),
	// The layout is not modified and blocks are ignored. Extend material nodes just add their layouts on top of the base one.
	Overlay = 2 UMETA(DisplayName = "Overlay Layout")
};

UE::Mutable::Private::EPackStrategy ConvertLayoutStrategy(const ECustomizableObjectTextureLayoutPackingStrategy LayoutPackStrategy);

UENUM()
enum class ECustomizableObjectLayoutAutomaticBlocksStrategy : uint8
{
	// Create rectangles on a grid splitting the UV space if possible
	Rectangles = 0 UMETA(DisplayName = "Rectangles"),
	// Detect UV islands and create blocks for them with masks
	UVIslands = 1 UMETA(DisplayName = "UV Islands"),
	// Don't create automatic blocks, and ignore UVs that don't have a manual block already. They get assigned to the first avilable block or ignored if none. This is the legacy behavior.
	Ignore = 2 UMETA(DisplayName = "Ignore"),
};

UE::Mutable::Private::EAutoBlocksStrategy ConvertAutomaticBlocksStrategy(const ECustomizableObjectLayoutAutomaticBlocksStrategy LayoutPackStrategy);

UENUM()
enum class ECustomizableObjectLayoutAutomaticBlocksMergeStrategy : uint8
{
	// Don't merge the blocks
	DontMerge = 0 UMETA(DisplayName = "Don't merge"),
	// Merge the block if a block is entirely included in another block
	MergeChildBlocks = 1 UMETA(DisplayName = "Child blocks"),
	// TODO: this option would merge only when the child falls entirely inside the mask
	// MergeChildInsideMask = 2 UMETA(DisplayName = "Child in mask"),
};


// Fixed Layout reduction methods
UENUM()
enum class ECustomizableObjectLayoutBlockReductionMethod : uint8
{
	// Layout blocks will be reduced by halves
	Halve = 0 UMETA(DisplayName = "Reduce by Half"),
	// Layout blocks will be reduced by a grid unit
	Unitary = 1 UMETA(DisplayName = "Reduce by Unit")
};

USTRUCT()
struct FCustomizableObjectLayoutBlock
{
	GENERATED_USTRUCT_BODY()

	FCustomizableObjectLayoutBlock(FIntPoint InMin = FIntPoint(0, 0), FIntPoint InMax = FIntPoint(1, 1))
	{
		Min = InMin;
		Max = InMax;
		Priority = 0;
		Id = FGuid::NewGuid();
		bReduceBothAxes = false;
		bReduceByTwo = false;
	}

	/** Top left coordinate. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FIntPoint Min;

	/** Bottom right coordinate. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FIntPoint Max;

	/** Priority to be reduced. Only functional in fixed layouts. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	uint32 Priority;

	/** Unique unchangeable id used to reference this block from other nodes. */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid Id;

	/** Block will be reduced on both sizes at the same time on each reduction. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bReduceBothAxes = false;

	/** Block will be reduced by two in an Unitary Layout reduction. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bReduceByTwo = false;

	/** Block mask to use to filter the UVs when assigning them to the block. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TObjectPtr<class UTexture2D> Mask = nullptr;

	/** Transient flag used in the UI to differentiate between manual and automatic blocks. */
	bool bIsAutomatic = false;
};

UCLASS()
class UCustomizableObjectLayout : public UObject
{
	GENERATED_BODY()

public:

	UCustomizableObjectLayout();

	// Sets the layout parameters
	void SetLayout(int32 LODIndex, int32 MatIndex, int32 UVIndex);
	void SetGridSize(FIntPoint Size);
	void SetMaxGridSize(FIntPoint Size);
	void SetLayoutName(FString Name);
	void SetIgnoreVertexLayoutWarnings(bool bValue);
	void SetIgnoreWarningsLOD(int32 LODValue);

	int32 GetLOD() const { return LOD; }
	int32 GetMaterial() const { return Material; }
	int32 GetUVChannel() const { return UVChannel; }
	FString GetLayoutName() const { return LayoutName; }
	TSoftObjectPtr<UStreamableRenderAsset> GetMesh() const;
	FIntPoint GetGridSize() const { return GridSize; }
	FIntPoint GetMaxGridSize() const { return MaxGridSize; }
	bool GetIgnoreVertexLayoutWarnings() const { return bIgnoreUnassignedVertexWarning; };
	int32 GetFirstLODToIgnoreWarnings() const { return FirstLODToIgnore; };

	void GetUVs(TArray<FVector2f>& UVs) const;

	/** Get a block index in the array from its id.Return - 1 if not found. */
	int32 FindBlock(const FGuid& InId) const;

	/** Generate all the transient UV layout automatic blocks with unassigned UVs. */
	void GenerateAutomaticBlocksFromUVs();

	/** Convert the transient automatic blocks into real blocks. */
	void ConsolidateAutomaticBlocks();

	/** List of blocks manually defined in the layout. */
	UPROPERTY()
	TArray<FCustomizableObjectLayoutBlock> Blocks;

	/** List of blocks automatically defined (in the layout) for preview. */
	UPROPERTY(Transient)
	TArray<FCustomizableObjectLayoutBlock> AutomaticBlocks;

	/** List of UVs to highlight in the layout because they have issues. */
	TArray< TArray<FVector2f> > UnassignedUVs;

	UPROPERTY()
	ECustomizableObjectTextureLayoutPackingStrategy PackingStrategy = ECustomizableObjectTextureLayoutPackingStrategy::Resizable;

	UPROPERTY()
	ECustomizableObjectLayoutAutomaticBlocksStrategy AutomaticBlocksStrategy = ECustomizableObjectLayoutAutomaticBlocksStrategy::Rectangles;

	UPROPERTY()
	ECustomizableObjectLayoutAutomaticBlocksMergeStrategy AutomaticBlocksMergeStrategy = ECustomizableObjectLayoutAutomaticBlocksMergeStrategy::MergeChildBlocks;

	UPROPERTY()
	ECustomizableObjectLayoutBlockReductionMethod BlockReductionMethod = ECustomizableObjectLayoutBlockReductionMethod::Halve;

private:

	UPROPERTY()
	int32 LOD;

	UPROPERTY()
	int32 Material;

	UPROPERTY()
	int32 UVChannel;

	UPROPERTY()
	FIntPoint GridSize;

	/** Maximum grid size the layout can grow to. Used with the fixed layout strategy. */
	UPROPERTY()
	FIntPoint MaxGridSize;

	UPROPERTY()
	FString LayoutName;

	/* If true, vertex warning messages will be ignored */
	UPROPERTY()
	bool bIgnoreUnassignedVertexWarning = false;

	/* First LOD from which unassigned vertices warning will be ignored */
	UPROPERTY()
	int32 FirstLODToIgnore = 0;

};
