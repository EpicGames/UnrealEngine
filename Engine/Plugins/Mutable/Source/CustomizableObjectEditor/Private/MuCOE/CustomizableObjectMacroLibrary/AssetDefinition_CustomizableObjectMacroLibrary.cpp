// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectMacroLibrary/AssetDefinition_CustomizableObjectMacroLibrary.h"

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibraryEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CustomizableObjectMacroLibrary)

#define LOCTEXT_NAMESPACE "AssetDefinition_CustomizableObjectMacroLibrary"

FText UAssetDefinition_CustomizableObjectMacroLibrary::GetAssetDisplayName() const
{ 
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CustomizableObjectMacroLibrary", "Customizable Object Macro Library"); 
}


FLinearColor UAssetDefinition_CustomizableObjectMacroLibrary::GetAssetColor() const
{
	return FLinearColor(FColor(100, 100, 100)); 
}


TSoftClassPtr<UObject> UAssetDefinition_CustomizableObjectMacroLibrary::GetAssetClass() const
{
	return UCustomizableObjectMacroLibrary::StaticClass(); 
}


EAssetCommandResult UAssetDefinition_CustomizableObjectMacroLibrary::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	return EAssetCommandResult::Unhandled; 
}


EAssetCommandResult UAssetDefinition_CustomizableObjectMacroLibrary::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UCustomizableObjectMacroLibrary* Object : OpenArgs.LoadObjects<UCustomizableObjectMacroLibrary>())
	{
		const TSharedPtr<FCustomizableObjectMacroLibraryEditor> Editor = MakeShared<FCustomizableObjectMacroLibraryEditor>();
		Editor->InitEditor(Mode, OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}


TConstArrayView<FAssetCategoryPath> UAssetDefinition_CustomizableObjectMacroLibrary::GetAssetCategories() const
{
	static const std::initializer_list<FAssetCategoryPath> Categories =
	{
		// Asset can be found inside the Mutable submenu 
		NSLOCTEXT("AssetTypeActions", "Mutable", "Mutable")
	};

	return Categories;
}


EAssetCommandResult UAssetDefinition_CustomizableObjectMacroLibrary::ActivateAssets(const FAssetActivateArgs& ActivateArgs) const
{
	return EAssetCommandResult::Unhandled;
}


FAssetOpenSupport UAssetDefinition_CustomizableObjectMacroLibrary::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(EAssetOpenMethod::Edit, true, EToolkitMode::Standalone);
}

#undef LOCTEXT_NAMESPACE

