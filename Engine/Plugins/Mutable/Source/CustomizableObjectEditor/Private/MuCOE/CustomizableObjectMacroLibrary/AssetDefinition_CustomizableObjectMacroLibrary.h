// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_CustomizableObjectMacroLibrary.generated.h"


UCLASS()
class UAssetDefinition_CustomizableObjectMacroLibrary final : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:

	// UAssetDefinition Interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;

};
