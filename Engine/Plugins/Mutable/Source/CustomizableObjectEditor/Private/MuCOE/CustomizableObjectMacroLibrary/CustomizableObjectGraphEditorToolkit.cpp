// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformApplicationMisc.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "MaterialGraph/MaterialGraphNode_Comment.h"
#include "MuCOE/CustomizableObjectEditorNodeContextCommands.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "SAssetSearchBox.h"
#include "SNodePanel.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectGraphEditorToolkit"

bool FCustomizableObjectGraphEditorToolkit::CanPasteNodes() const 
{
	if (!GraphEditor.IsValid() || !GraphEditor->GetCurrentGraph())
	{
		return false;
	}

	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(GraphEditor->GetCurrentGraph(), ClipboardContent);
}


void FCustomizableObjectGraphEditorToolkit::PasteNodes()
{
	if (!GraphEditor.IsValid())
	{
		return;
	}

	PasteNodesHere(GraphEditor->GetPasteLocation2f());
}


void FCustomizableObjectGraphEditorToolkit::PasteNodesHere(const FVector2D& Location) 
{
	if (!GraphEditor.IsValid() || !GraphEditor->GetCurrentGraph())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("CustomizableObjectEditorPaste", "Customizable Object Editor Editor: Paste"));
	GraphEditor->GetCurrentGraph()->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	GraphEditor->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(GraphEditor->GetCurrentGraph(), TextToImport, PastedNodes);

	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AvgNodePosition(0.0f, 0.0f);

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;
		AvgNodePosition.X += Node->NodePosX;
		AvgNodePosition.Y += Node->NodePosY;
	}

	if (PastedNodes.Num() > 0)
	{
		float InvNumNodes = 1.0f / float(PastedNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;

		// Select the newly pasted stuff
		GraphEditor->SetNodeSelection(Node, true);

		Node->NodePosX = (Node->NodePosX - AvgNodePosition.X) + Location.X;
		Node->NodePosY = (Node->NodePosY - AvgNodePosition.Y) + Location.Y;

		Node->SnapToGrid(SNodePanel::GetSnapGridSize());

		// Give new node a different Guid from the old one
		Node->CreateNewGuid();
	}

	for (UEdGraphNode* PastedNode : PastedNodes)
	{
		if (UCustomizableObjectNode* TypedNode = Cast<UCustomizableObjectNode>(PastedNode))
		{
			TypedNode->PostBackwardsCompatibleFixup();
		}
	}

	// Update UI
	GraphEditor->NotifyGraphChanged();
	GraphEditor->GetCurrentGraph()->MarkPackageDirty();
}


void FCustomizableObjectGraphEditorToolkit::SelectNode(const UEdGraphNode* Node) 
{
	if (!GraphEditor.IsValid())
	{
		return;
	}

	GraphEditor->JumpToNode(Node);
}


void FCustomizableObjectGraphEditorToolkit::SelectSingleNode(UCustomizableObjectNode& Node)
{
	FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	if (SelectedNodes.Num() != 1 || Cast<UCustomizableObjectNode>(*SelectedNodes.CreateIterator()) != &Node)
	{
		GraphEditor->ClearSelectionSet();
		GraphEditor->SetNodeSelection(&Node, true);
	}
}


void FCustomizableObjectGraphEditorToolkit::DeleteSelectedNodes()
{
	if (!GraphEditor.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("UEdGraphSchema_CustomizableObject", "Delete Nodes"));

	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	GraphEditor->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Node->CanUserDeleteNode())
			{
				if (const UEdGraph* GraphObj = Node->GetGraph())
				{
					if (const UEdGraphSchema* Schema = GraphObj->GetSchema())
					{
						Schema->BreakNodeLinks(*Node);  // Required to notify to all connected nodes (UEdGraphNode::PinConnectionListChanged() and UEdGraphNode::PinConnectionListChanged(...))
					}
				}

				Node->DestroyNode();
			}
		}
	}
}


bool FCustomizableObjectGraphEditorToolkit::CanDeleteNodes() const
{
	if (GraphEditor.IsValid() && GraphEditor->GetSelectedNodes().Num() > 0)
	{
		for (auto Itr = GraphEditor->GetSelectedNodes().CreateConstIterator(); Itr; ++Itr)
		{
			UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(*Itr);

			if (Node && !Node->CanUserDeleteNode())
			{
				return false;
			}
		}

		return true;
	}

	return false;
}


void FCustomizableObjectGraphEditorToolkit::DuplicateSelectedNodes()
{
	CopySelectedNodes();
	PasteNodes();
}


bool FCustomizableObjectGraphEditorToolkit::CanDuplicateSelectedNodes() const
{
	return CanCopyNodes();
}


void FCustomizableObjectGraphEditorToolkit::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}


void FCustomizableObjectGraphEditorToolkit::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	if (Node->CanJumpToDefinition())
	{
		Node->JumpToDefinition();
	}
}


void FCustomizableObjectGraphEditorToolkit::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();

	FString ExportedText;

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			Node->PrepareForCopying();
		}
	}

	FEdGraphUtilities::ExportNodesToText(SelectedNodes, /*out*/ ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	//TODO(Max): This is weird... review it
	// Make sure Material remains the owner of the copied nodes
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if (UMaterialGraphNode* Node = Cast<UMaterialGraphNode>(*SelectedIter))
		{
			Node->PostCopyNode();
		}
		else if (UMaterialGraphNode_Comment* Comment = Cast<UMaterialGraphNode_Comment>(*SelectedIter))
		{
			Comment->PostCopyNode();
		}
	}
}


bool FCustomizableObjectGraphEditorToolkit::CanCopyNodes() const
{
	// If any of the nodes can be duplicated then we should allow copying
	const FGraphPanelSelectionSet SelectedNodes = GraphEditor->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			return true;
		}
	}

	return false;
}


void FCustomizableObjectGraphEditorToolkit::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedNodes();
}


bool FCustomizableObjectGraphEditorToolkit::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}


void FCustomizableObjectGraphEditorToolkit::OnRenameNode()
{
	if (GraphEditor.IsValid())
	{
		const FGraphPanelSelectionSet& SelectedNodes = GraphEditor->GetSelectedNodes();

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			// Rename only the first valid selected node
			UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt);
			if (SelectedNode && SelectedNode->GetCanRenameNode())
			{
				GraphEditor->IsNodeTitleVisible(SelectedNode, true);
				break;
			}
		}
	}
}


bool FCustomizableObjectGraphEditorToolkit::CanRenameNodes() const
{
	if (GraphEditor.IsValid())
	{
		const FGraphPanelSelectionSet& SelectedNodes = GraphEditor->GetSelectedNodes();

		for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
		{
			if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
			{
				return Node->GetCanRenameNode();
			}
		}
	}

	return false;
}


void FCustomizableObjectGraphEditorToolkit::CreateCommentBoxFromKey()
{
	CreateCommentBox(GraphEditor->GetPasteLocation2f());
}


UEdGraphNode* FCustomizableObjectGraphEditorToolkit::CreateCommentBox(const FVector2D& NodePos)
{
	if (!GraphEditor.IsValid() || !GraphEditor->GetCurrentGraph())
	{
		return nullptr;
	}

	//const FScopedTransaction Transaction(LOCTEXT("UEdGraphSchema_CustomizableObject", "Add Comment Box"));
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();
	UEdGraphNode_Comment* NewComment = nullptr;

	{
		// const FGraphPanelSelectionSet& SelectionSet = GraphEditor->GetSelectedNodes();
		FSlateRect Bounds;
		FVector2D Location;
		FVector2D Size;

		if (GraphEditor->GetBoundsForSelectedNodes(Bounds, 50.f))
		{
			Location.X = Bounds.Left;
			Location.Y = Bounds.Top;
			Size = Bounds.GetSize();
		}
		else
		{
			Location.X = NodePos.X;
			Location.Y = NodePos.Y;
			Size.X = 400;
			Size.Y = 100;
		}

		NewComment = FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(GraphEditor->GetCurrentGraph(), CommentTemplate, NodePos, true);
		NewComment->NodePosX = Location.X;
		NewComment->NodePosY = Location.Y;
		NewComment->NodeWidth = Size.X;
		NewComment->NodeHeight = Size.Y;
		NewComment->NodeComment = FString(TEXT("Comment"));
	}

	GraphEditor->GetCurrentGraph()->MarkPackageDirty();
	GraphEditor->NotifyGraphChanged();

	return NewComment;
}


void FCustomizableObjectGraphEditorToolkit::OnEnterText(const FText& NewText, ETextCommit::Type TextType)
{
	if (TextType != ETextCommit::OnEnter)
	{
		return;
	}

	if (!GraphEditor)
	{
		return;
	}

	const UEdGraph* Graph = GraphEditor->GetCurrentGraph();
	if (!Graph)
	{
		return;
	}

	bool bFound = false;
	const FString FindString = NewText.ToString();
	UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(Graph->GetOutermostObject()); // Null if this is a Macro

	for (TObjectPtr<UEdGraphNode> Node : Graph->Nodes)
	{
		if (!Node)
		{
			continue;
		}

		// Node names are not in the reflection system
		const FString NodeName = Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString().Replace(TEXT("\n"), TEXT(" "));
		if (NodeName.Contains(NewText.ToString(), ESearchCase::IgnoreCase))
		{
			LogSearchResult(*Node, "Node", bFound, NodeName, CustomizableObject);
		}

		// Pins are not in the reflection system
		for (const UEdGraphPin* Pin : Node->GetAllPins())
		{
			const FString PinFriendlyName = Pin->PinFriendlyName.ToString();
			if (PinFriendlyName.Contains(FindString))
			{
				LogSearchResult(*Node, "Pin", bFound, PinFriendlyName, CustomizableObject);
			}
		}

		// Find anything marked as a UPROPERTY
		for (TFieldIterator<FProperty> It(Node->GetClass()); It; ++It)
		{
			FindProperty(*It, Node, FindString, *Node, bFound, CustomizableObject);
		}
	}

	const FText Text = bFound ?
		LOCTEXT("SearchCompleted", "Search completed") :
		FText::FromString("No Results for: " + NewText.ToString());

	FCustomizableObjectEditorLogger::CreateLog(Text)
		.Category(ELoggerCategory::GraphSearch)
		.CustomNotification()
		.CustomizableObject(CustomizableObject)
		.Log();
}


void FCustomizableObjectGraphEditorToolkit::FindProperty(const FProperty* Property, const void* InContainer, const FString& FindString, const UObject& Context, bool& bFound, UCustomizableObject* Object)
{
	if (!Property || !InContainer)
	{
		return;
	}

	const FString PropertyName = Property->GetDisplayNameText().ToString();
	if (PropertyName.Contains(FindString))
	{
		LogSearchResult(Context, "Property Name", bFound, *PropertyName, Object);
	}

	for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
	{
		const uint8* ValuePtr = Property->ContainerPtrToValuePtr<uint8>(InContainer, Index);

		if (const FStrProperty* StringProperty = CastField<FStrProperty>(Property))
		{
			const FString* StringResult = StringProperty->GetPropertyValuePtr(ValuePtr);
			if (StringResult->Contains(FindString))
			{
				LogSearchResult(Context, "Property Value", bFound, *StringResult, Object);
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			const UEnum* EnumResult = EnumProperty->GetEnum();

			const FString StringResult = EnumResult->GetDisplayNameTextByIndex(*ValuePtr).ToString();
			if (StringResult.Contains(FindString))
			{
				LogSearchResult(Context, "Property Value", bFound, StringResult, Object);
			}
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
		{
			const FString ObjectPath = SoftObjectProperty->GetPropertyValuePtr(ValuePtr)->ToString();
			if (ObjectPath.Contains(FindString))
			{
				LogSearchResult(Context, "Property Value", bFound, ObjectPath, Object);
			}
		}
		else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			if (const UObject* ObjectValue = ObjectProperty->GetObjectPropertyValue(ValuePtr))
			{
				const FString Name = ObjectValue->GetName();

				if (ObjectValue->GetName().Contains(FindString))
				{
					LogSearchResult(Context, "Property Value", bFound, Name, Object);
				}
			}
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				FindProperty(*It, ValuePtr, FindString, Context, bFound, Object);
			}
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, ValuePtr);
			for (int32 ValueIdx = 0; ValueIdx < ArrayHelper.Num(); ++ValueIdx)
			{
				FindProperty(ArrayProperty->Inner, ArrayHelper.GetRawPtr(ValueIdx), FindString, Context, bFound, Object);
			}
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			FScriptSetHelper SetHelper(SetProperty, ValuePtr);
			for (FScriptSetHelper::FIterator SetIt = SetHelper.CreateIterator(); SetIt; ++SetIt)
			{
				FindProperty(SetProperty->ElementProp, SetHelper.GetElementPtr(SetIt), FindString, Context, bFound, Object);
			}
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			FScriptMapHelper MapHelper(MapProperty, ValuePtr);
			for (FScriptMapHelper::FIterator MapIt = MapHelper.CreateIterator(); MapIt; ++MapIt)
			{
				const uint8* MapValuePtr = MapHelper.GetPairPtr(MapIt);
				FindProperty(MapProperty->KeyProp, MapValuePtr, FindString, Context, bFound, Object);
				FindProperty(MapProperty->ValueProp, MapValuePtr, FindString, Context, bFound, Object);
			}
		}
	}
}


void FCustomizableObjectGraphEditorToolkit::LogSearchResult(const UObject& Context, const FString& Type, bool& bIsFirst, const FString& Result, UCustomizableObject* Object) const
{
	if (!bIsFirst)
	{
		bIsFirst = true;

		if (Object && Object->GetPrivate()->CompilationLogger)
		{
			Object->GetPrivate()->CompilationLogger->ClearLogMessageList();
		}

		FCustomizableObjectEditorLogger::CreateLog(LOCTEXT("SearchResults", "Search Results:"))
			.Notification(false)
			.CustomizableObject(Object)
			.Log();
	}

	FCustomizableObjectEditorLogger::CreateLog(FText::FromString(Type + ": " + Result))
		.Context(Context)
		.BaseObject()
		.Notification(false)
		.CustomizableObject(Object)
		.Log();
}


/** Create new tab for the supplied graph - don't call this directly, call SExplorer->FindTabForGraph.*/
void FCustomizableObjectGraphEditorToolkit::CreateGraphEditorWidget(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents)
{
	UCustomizableObjectGraph* CustomizableObjectGraph = Cast<UCustomizableObjectGraph>(InGraph);
	check(CustomizableObjectGraph);

	GraphEditorCommands = MakeShareable(new FUICommandList);

	TSharedRef<SWidget> TitleBarWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.FillWidth(10.f)
		.Padding(5.f)
		[
			SNew(SSearchBox)
				.HintText(LOCTEXT("Search", "Search..."))
				.ToolTipText(LOCTEXT("Search Nodes, Properties or Values that contain the inserted words", "Search Nodes, Properties or Values that contain the inserted words"))
				.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::OnEnterText))
				.SelectAllTextWhenFocused(true)
		];

	// Create the appearance info
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = CustomizableObjectGraph->IsMacro() ? LOCTEXT("ApperanceCornerMacroText", "MUTABLE MACRO") : LOCTEXT("ApperanceCornerText", "MUTABLE");

	// Add toolkit common events
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::OnNodeTitleCommitted);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::OnNodeDoubleClicked);

	// Make full graph editor
	GraphEditor = SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.Appearance(AppearanceInfo)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents)
		.TitleBar(TitleBarWidget)
		.ShowGraphStateOverlay(false); // Removes graph state overlays (border and text) such as "SIMULATING" and "READ-ONLY"

	// Ensure commands are registered
	FGraphEditorCommands::Register();
	FCustomizableObjectEditorNodeContextCommands::Register();

	// Editing commands
	GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CanDeleteNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CanCopyNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::PasteNodes),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CanPasteNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CanCutNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::DuplicateSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CanDuplicateSelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::OnRenameNode),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CanRenameNodes));

	GraphEditorCommands->MapAction(FCustomizableObjectEditorNodeContextCommands::Get().CreateComment,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::CreateCommentBoxFromKey));

	// Alignment Commands
	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnAlignTop));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnAlignMiddle));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnAlignBottom));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnAlignLeft));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnAlignCenter));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnAlignRight));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnStraightenConnections));

	// Distribution Commands
	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnDistributeNodesH));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
		FExecuteAction::CreateSP(GraphEditor.Get(), &SGraphEditor::OnDistributeNodesV));
}


void FCustomizableObjectGraphEditorToolkit::UpdateGraphNodeProperties()
{
	// Cache a copy of the selected nodes so we can later restore them
	const FGraphPanelSelectionSet PreClearingSelectedNodes = GraphEditor->GetSelectedNodes();

	OnSelectedGraphNodesChanged(FGraphPanelSelectionSet());
	OnSelectedGraphNodesChanged(PreClearingSelectedNodes);
}


void FCustomizableObjectGraphEditorToolkit::BindGraphCommands()
{
	// Undo-Redo
	ToolkitCommands->MapAction(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::UndoGraphAction));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &FCustomizableObjectGraphEditorToolkit::RedoGraphAction));
}


void FCustomizableObjectGraphEditorToolkit::PostUndo(bool bSuccess)
{
	GraphEditor->NotifyGraphChanged();
}


void FCustomizableObjectGraphEditorToolkit::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess); 
}


void FCustomizableObjectGraphEditorToolkit::UndoGraphAction()
{
	GEditor->UndoTransaction();
}


void FCustomizableObjectGraphEditorToolkit::RedoGraphAction()
{
	// Clear selection, to avoid holding refs to nodes that go away
	GraphEditor->ClearSelectionSet();

	GEditor->RedoTransaction();
}

#undef LOCTEXT_NAMESPACE