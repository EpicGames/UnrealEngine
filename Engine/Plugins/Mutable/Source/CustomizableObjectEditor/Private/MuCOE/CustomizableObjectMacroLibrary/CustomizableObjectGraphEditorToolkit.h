// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintUtilities.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphUtilities.h"
#include "EditorUndoClient.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "Misc/NotifyHook.h"
#include "ScopedTransaction.h"
#include "Toolkits/AssetEditorToolkit.h"

class UClass;
class UCustomizableObject;
class UCustomizableObjectNode;

// Public interface to Customizable Object Editor
class FCustomizableObjectGraphEditorToolkit : public FAssetEditorToolkit, public FNotifyHook, public FSelfRegisteringEditorUndoClient
{
public:

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	// Own Interface ---

	/** Searches a node that contains the inserted word */
	void OnEnterText(const FText& NewText, ETextCommit::Type TextType);
	/** Recursively find any property that its name or value contains the given string.
	  * @param Property Root property.
	  * @param Container Root property container (address of the property value).
	  * @param FindString String to find for.
	  * @param Context UObject Context where this string has been found.
	  * @param bFound Mark as true if any property has been found. */
	void FindProperty(const FProperty* Property, const void* InContainer, const FString& FindString, const UObject& Context, bool& bFound, UCustomizableObject* Object);
	/** Logs the search results of the search
	 * @param Context The UObject we have found to be related with the searched string.
	 * @param Type The type of relation with the searched word. It is a node, a value or maybe a variable?
	 * @param bIsFirst Is this the first time we encountered something during our search?
	 * @param Result The string containing the search word we are looking for in Node
	 */
	void LogSearchResult(const UObject& Context, const FString& Type, bool& bIsFirst, const FString& Result, UCustomizableObject* Object) const;

	/** Bind common graph commands */
	void BindGraphCommands();

	// Graph Node Actions --------------------------
	void SelectNode(const UEdGraphNode* Node);

	/** Select only and this node only. Do nothing if already was only selected. */
	void SelectSingleNode(UCustomizableObjectNode& Node);
		
	void DeleteSelectedNodes();
	bool CanDeleteNodes() const;
	
	void DuplicateSelectedNodes();
	bool CanDuplicateSelectedNodes() const;

	/**
	 * Called when a node's title is committed for a rename
	 *
	 * @param	NewText				New title text
	 * @param	CommitInfo			How text was committed
	 * @param	NodeBeingChanged	The node being changed
	 */
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);
	
	void CopySelectedNodes();
	bool CanCopyNodes() const;
	
	void PasteNodes();
	void PasteNodesHere(const FVector2D& Location);
	bool CanPasteNodes() const;
	
	void CutSelectedNodes();
	bool CanCutNodes() const;

	void OnRenameNode();
	bool CanRenameNodes() const;

	void CreateCommentBoxFromKey();
	UEdGraphNode* CreateCommentBox(const FVector2D& NodePos);

	void OnNodeDoubleClicked(UEdGraphNode* Node);

	//Undo/Redo graph Actions
	void UndoGraphAction();
	void RedoGraphAction();

	// ------------------------------------------

	/** Creates a new graph editor widget 
	* @param InGraph is the graph that will be represented in the graph editor.
	* @param InEvents are custom events occurring in/on the graph.
	*/
	void CreateGraphEditorWidget(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents);

	/** Graph Editor callbacks */
	virtual void OnSelectedGraphNodesChanged(const FGraphPanelSelectionSet& NewSelection) = 0;

	/** Reconstructs all child the nodes that match the given type.
	 * @param StartNode Root node to start the graph traversal. This one also will be reconstructed.
	 * @param NodeType Node types to reconstruct. */
	virtual void ReconstructAllChildNodes(UCustomizableObjectNode& StartNode, const UClass& NodeType) = 0;

	virtual void UpdateGraphNodeProperties();

public:

	/** Pointer to the graph editor widget */
	TSharedPtr<SGraphEditor> GraphEditor;

private:

	/** Command list for the graph editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;
};
