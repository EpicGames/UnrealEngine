// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "PropertyCustomizationHelpers.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectMacroDetails"

TSharedRef<IDetailCustomization> FCustomizableObjectMacroDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectMacroDetails);
}


void FCustomizableObjectMacroDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	DetailBuilderPtr = DetailBuilder;

	TSharedPtr<IDetailsView> DetailsView = DetailBuilder->GetDetailsViewSharedPtr();

	Macro = Cast<UCustomizableObjectMacro>(DetailsView->GetSelectedObjects()[0].Get());

	if (!Macro || !Macro->Graph)
	{
		return;
	}

	TArray<UCustomizableObjectNodeTunnel*> IONodes;
	TObjectPtr<UCustomizableObjectNodeTunnel> InputNode;
	TObjectPtr<UCustomizableObjectNodeTunnel> OutputNode;

	Macro->Graph->GetNodesOfClass<UCustomizableObjectNodeTunnel>(IONodes);
	check(IONodes.Num() == 2);

	for (UCustomizableObjectNodeTunnel* IONode : IONodes)
	{
		if (IONode->bIsInputNode)
		{
			InputNode = IONode;
		}
		else
		{
			OutputNode = IONode;
		}
	}

	IDetailCategoryBuilder& GraphCategory = DetailBuilder->EditCategory("Macro");
	IDetailCategoryBuilder& InputsCategory = DetailBuilder->EditCategory("Inputs");
	IDetailCategoryBuilder& OutputsCategory = DetailBuilder->EditCategory("Outputs");

	InputsCategory.HeaderContent(GenerateCategoryButtonWidget(ECOMacroIOType::COMVT_Input, InputNode));
	OutputsCategory.HeaderContent(GenerateCategoryButtonWidget(ECOMacroIOType::COMVT_Output, OutputNode));

	GenerateVariableList(InputsCategory, InputNode, ECOMacroIOType::COMVT_Input);
	GenerateVariableList(OutputsCategory, OutputNode, ECOMacroIOType::COMVT_Output);
}


void FCustomizableObjectMacroDetails::GenerateVariableList(IDetailCategoryBuilder& IOCategory, UCustomizableObjectNodeTunnel* IONode, ECOMacroIOType IOType)
{
	if (!Macro || !Macro->Graph)
	{
		return;
	}

	const UEdGraphSchema_CustomizableObject* Schema = Cast<UEdGraphSchema_CustomizableObject>(Macro->Graph->GetSchema());
	check(Schema);

	if (!IONode->Pins.Num())
	{
		// Add a text widget to let the user know to hit the + icon to add parameters.
		IOCategory.AddCustomRow(FText::GetEmpty()).WholeRowContent()
		.MaxDesiredWidth(980.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoArgumentsAddedForBlueprint", "Press the + icon above to add a new Variable"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

		return;
	}

	for (UCustomizableObjectMacroInputOutput* InputOutput : Macro->InputOutputs)
	{
		if (InputOutput->Type == IOType)
		{
			IOCategory.AddCustomRow(FText::FromName(InputOutput->Name))
			.NameContent()
			[
				SNew(SEditableTextBox)
				.Text(this, &FCustomizableObjectMacroDetails::GetVariableName, InputOutput)
				.OnTextChanged(this, &FCustomizableObjectMacroDetails::OnVariableNameChanged, InputOutput)
				.OnTextCommitted(this, &FCustomizableObjectMacroDetails::OnVariableNameCommitted, InputOutput, IONode)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SCOMacroPinTypeSelector)
				.Variable(InputOutput)
				.IONode(IONode)
				.OnVariableRemoved(this, &FCustomizableObjectMacroDetails::OnRemoveVariable)
			];
		}
	}
}


TSharedRef<SWidget> FCustomizableObjectMacroDetails::GenerateCategoryButtonWidget(ECOMacroIOType IOType, UCustomizableObjectNodeTunnel* IONode)
{
	FText TooltipText = IONode->bIsInputNode ? LOCTEXT("MacroInputTooltip", "Create a new Input variable") : LOCTEXT("MacroOutputTooltip", "Create a new Output variable");

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(FMargin(1, 0))
				.OnClicked(this, &FCustomizableObjectMacroDetails::AddNewVariable, IOType, IONode)
				.HAlign(HAlign_Right)
				.ToolTipText(TooltipText)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
				]
		];
}


FReply FCustomizableObjectMacroDetails::AddNewVariable(ECOMacroIOType VarType, UCustomizableObjectNodeTunnel* IONode)
{
	if (Macro && IONode)
	{
		if (Macro->AddVariable(VarType))
		{
			IONode->ReconstructNode();
			DetailBuilderPtr.Pin()->ForceRefreshDetails();
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FText FCustomizableObjectMacroDetails::GetVariableName(UCustomizableObjectMacroInputOutput* Variable) const
{
	if (Variable)
	{
		return FText::FromName(Variable->Name);
	}

	return FText();
}


void FCustomizableObjectMacroDetails::OnVariableNameChanged(const FText& InNewText, UCustomizableObjectMacroInputOutput* Variable)
{
	if (Variable)
	{
		//TODO(Max): Check for repeated names and notify somehow
		Variable->Name = *InNewText.ToString();
	}
}


void FCustomizableObjectMacroDetails::OnVariableNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit, UCustomizableObjectMacroInputOutput* Variable, UCustomizableObjectNodeTunnel* IONode)
{
	if (Variable && IONode)
	{
		//TODO(Max): Check for repeated names and notify somehow
		Variable->Name = *InNewText.ToString();
		IONode->ReconstructNode();
	}
}


void FCustomizableObjectMacroDetails::OnRemoveVariable(UCustomizableObjectNodeTunnel* IONode)
{
	if (IONode)
	{
		IONode->ReconstructNode();
	}

	DetailBuilderPtr.Pin()->ForceRefreshDetails();
}


// PinTypeSelector Widget --------------
void SCOMacroPinTypeSelector::Construct(const FArguments& InArgs)
{
	Variable = InArgs._Variable;
	IONode = InArgs._IONode;
	Macro = Cast< UCustomizableObjectMacro>(Variable.GetOuter());
	OnVariableRemoved = InArgs._OnVariableRemoved;

	if (!Macro || !Macro->Graph || !Variable || !IONode)
	{
		ChildSlot
		[
			SNew(STextBlock).Text(LOCTEXT("NullMacroErrorPin","Variable Error!"))
		];

		return;
	}

	const UEdGraphSchema_CustomizableObject* Schema = Cast<UEdGraphSchema_CustomizableObject>(Macro->Graph->GetSchema());
	check(Schema);

	TSharedPtr<FPinNameRowData> InitialSelection;

	for (const FName& PinType : Schema->SupportedMacroPinTypes)
	{
		TSharedPtr<FPinNameRowData> NewSourceData = MakeShareable(new FPinNameRowData());
		NewSourceData->PinCategory = PinType;
		NewSourceData->PinFriendlyName = Schema->GetPinCategoryFriendlyName(PinType).ToString();

		ComboBoxSource.Add(NewSourceData);

		if (Variable->PinCategoryType == PinType)
		{
			InitialSelection = ComboBoxSource.Last();
		}
	}

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(0.7f)
		[
			// Combo button that summons the dropdown menu
			SNew(SComboBox<TSharedPtr<FPinNameRowData>>)
			.OptionsSource(&ComboBoxSource)
			.OnGenerateWidget(this, &SCOMacroPinTypeSelector::OnGenerateRow)
			.OnSelectionChanged_Lambda([this](TSharedPtr<FPinNameRowData> Item, ESelectInfo::Type)
			{
				if (Item.IsValid())
				{
					Variable->PinCategoryType = Item->PinCategory;
					IONode->ReconstructNode();
				}
			})
			.Content()
			[
				GenerateSelectedContent()
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			PropertyCustomizationHelpers::MakeClearButton(
				FSimpleDelegate::CreateSP(this, &SCOMacroPinTypeSelector::OnRemoveClicked), LOCTEXT("RemoveVariableTooltip", "Remove Variable."))
		]
	];
}


TSharedRef<SWidget> SCOMacroPinTypeSelector::OnGenerateRow(TSharedPtr<FPinNameRowData> Option)
{
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));

	TSharedPtr<SHorizontalBox> RowWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SImage)
		.Image(IconBrush)
		.ColorAndOpacity(UEdGraphSchema_CustomizableObject::GetPinTypeColor(Option->PinCategory))
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(7.5f, 0.0f, 0.0f, 0.0f)
	[
		SNew(STextBlock)
		.Text(FText::FromString(Option.IsValid() ? Option->PinFriendlyName : "Invalid"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];

	return RowWidget.ToSharedRef();
}


TSharedRef<SWidget> SCOMacroPinTypeSelector::GenerateSelectedContent() const
{
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));

	TSharedPtr<SHorizontalBox> RowWidget = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SImage)
		.Image(IconBrush)
		.ColorAndOpacity_Lambda([this]() 
		{
			return UEdGraphSchema_CustomizableObject::GetPinTypeColor(Variable->PinCategoryType); 
		})
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(7.5f, 0.0f, 0.0f, 0.0f)
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text_Lambda([this]() 
		{
			return Variable ? UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(Variable->PinCategoryType) : FText::FromString("Invalid");
		})
	];

	return RowWidget.ToSharedRef();
}


void SCOMacroPinTypeSelector::OnRemoveClicked()
{
	if (Macro)
	{
		Macro->RemoveVariable(Variable);
		OnVariableRemoved.ExecuteIfBound(IONode);
	}
}


#undef LOCTEXT_NAMESPACE
