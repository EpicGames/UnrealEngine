// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"

class IDetailLayoutBuilder;
class IDetailCategoryBuilder;
class ITableRow;
class STableViewBase;
class UCustomizableObjectMacro;
class UCustomizableObjectMacroInputOutput;
class UCustomizableObjectNodeTunnel;


enum class ECOMacroIOType : uint8;

struct FPinNameRowData
{
	FString PinFriendlyName;
	FName PinCategory;
};

/** Widget to edit the type of a macro's variable. */
class SCOMacroPinTypeSelector : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnVariableRemoved, UCustomizableObjectNodeTunnel*);

	SLATE_BEGIN_ARGS(SCOMacroPinTypeSelector) {}
		SLATE_ARGUMENT(TObjectPtr<UCustomizableObjectMacroInputOutput>, Variable)
		SLATE_ARGUMENT(TObjectPtr<UCustomizableObjectNodeTunnel>, IONode)
		SLATE_EVENT(FOnVariableRemoved, OnVariableRemoved)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Generates the widget of a List view row. */
	TSharedRef<SWidget> OnGenerateRow(TSharedPtr<FPinNameRowData> Option);

	/** Returns the name of the pin type of the variable.*/
	TSharedRef<SWidget> GenerateSelectedContent() const;

	/** Callback of the remove variable button. */
	void OnRemoveClicked();

private:

	/** Pointer to the variablethas is being edited */
	TObjectPtr<UCustomizableObjectMacroInputOutput> Variable;

	/** Pointer to the macro where the variable belongs. */
	TObjectPtr<UCustomizableObjectMacro> Macro;

	/** Pointer to the node that will expose this variable to the graph. */
	TObjectPtr<UCustomizableObjectNodeTunnel> IONode;

	/** Array that contains all the pin types showed in the list view widget.*/
	TArray<TSharedPtr<FPinNameRowData>> ComboBoxSource;

	/** Callback to indicate that a variable has been removed */
	FOnVariableRemoved OnVariableRemoved;
};


/** Details View of UCustomizableObjectMacros */
class FCustomizableObjectMacroDetails : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance();

	// ILayoutDetails interface
	/** Do not use. Add details customization in the other CustomizeDetails signature. */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override {};
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

private:

	/** Creates a widget list to create and visualize macro variables. */
	void GenerateVariableList(IDetailCategoryBuilder& InputsCategory, UCustomizableObjectNodeTunnel* IONode, ECOMacroIOType VariableType);
	
	/** Creates the button that allows to add new macro variables. */
	TSharedRef<SWidget> GenerateCategoryButtonWidget(ECOMacroIOType VariableType, UCustomizableObjectNodeTunnel* IONode);
	
	/** Returns the name of the specified variable. Needed to update automatically the text if the name of the variable changes. */
	FText GetVariableName(UCustomizableObjectMacroInputOutput* Variable) const;
	
	/** Editable Text Box Callbacks */
	void OnVariableNameChanged(const FText& InNewText, UCustomizableObjectMacroInputOutput* Variable);
	void OnVariableNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit, UCustomizableObjectMacroInputOutput* Variable, UCustomizableObjectNodeTunnel* IONode);

	/** Callback to comunicate the editor that the name of a variable has changes. Updates the variable List too. */
	void OnRemoveVariable(UCustomizableObjectNodeTunnel* IONode);

	/** Adds a new variable to a Macro. */
	FReply AddNewVariable(ECOMacroIOType VarType, UCustomizableObjectNodeTunnel* IONode);

private:

	/** Details builder pointer */
	TWeakPtr<IDetailLayoutBuilder> DetailBuilderPtr;

	/** Pointer to the macro thas is being edited */
	TObjectPtr<UCustomizableObjectMacro> Macro;
};
