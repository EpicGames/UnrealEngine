// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCO/CustomizableObjectCustomVersion.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectMacroLibrary)

#define LOCTEXT_NAMESPACE "CustomizableObjectMacroLibrary"


void UCustomizableObjectMacroLibrary::PostLoad()
{
	Super::PostLoad();

	const int32 CustomVersion = FMath::Max(GetLinkerCustomVersion(FCustomizableObjectCustomVersion::GUID), (int32)FCustomizableObjectCustomVersion::EnableMutableMacros);

	for (TObjectPtr<UCustomizableObjectMacro>& Macro : Macros)
	{
		if (ensure(Macro))
		{
			Macro->ConditionalPostLoad();
		}
	}

	if (ICustomizableObjectEditorModule* Module = ICustomizableObjectEditorModule::Get())
	{
		for (TObjectPtr<UCustomizableObjectMacro>& Macro : Macros)
		{
			UEdGraph* MacroGraph = Macro ? Macro->Graph : nullptr;
			if (ensure(MacroGraph))
			{
				MacroGraph->ConditionalPostLoad();

				for (int32 Version = CustomVersion + 1; Version <= FCustomizableObjectCustomVersion::LatestVersion; ++Version)
				{
					// Execute backwards compatible code for all nodes. It requires all nodes to be loaded.
					Module->BackwardsCompatibleFixup(*MacroGraph, Version);
				}

				Module->PostBackwardsCompatibleFixup(*MacroGraph);
			}
		}
	}
}

void UCustomizableObjectMacroLibrary::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);
}

UCustomizableObjectMacro* UCustomizableObjectMacroLibrary::AddMacro()
{
	UCustomizableObjectMacro* NewMacro = nullptr;

	FName BaseName = "NewMacro";
	FName MacroName = MakeUniqueObjectName(this, UCustomizableObjectMacro::StaticClass(), FName(BaseName));
	
	NewMacro = NewObject<UCustomizableObjectMacro>(this, MacroName, RF_Transactional | RF_Public);
	UCustomizableObjectGraph* NewGraph = NewObject<UCustomizableObjectGraph>(NewMacro, NAME_None, RF_Transactional);

	NewGraph->AddEssentialGraphNodes();

	NewMacro->Graph = NewGraph;
	NewMacro->Name = MacroName;
	Macros.Add(NewMacro);

	return NewMacro;
}


void UCustomizableObjectMacroLibrary::RemoveMacro(UCustomizableObjectMacro* MacroToRemove)
{
	if (Macros.Contains(MacroToRemove))
	{
		Macros.Remove(MacroToRemove);
	}
}


UCustomizableObjectMacroInputOutput* UCustomizableObjectMacro::AddVariable(ECOMacroIOType VarType)
{
	const UEdGraphSchema_CustomizableObject* Schema = Cast<UEdGraphSchema_CustomizableObject>(Graph->GetSchema());
	check(Schema);

	FName BaseName = "NewVar";
	FName BaseType = Schema->PC_Mesh;
	FName VariableName = MakeUniqueObjectName(this, UCustomizableObjectMacroInputOutput::StaticClass(), FName(BaseName));

	UCustomizableObjectMacroInputOutput* NewVariable = NewObject<UCustomizableObjectMacroInputOutput>(this, VariableName, RF_Transactional);
	NewVariable->Type = VarType;
	NewVariable->PinCategoryType = BaseType;
	NewVariable->Name = VariableName;
	NewVariable->UniqueId = FGuid::NewGuid();

	InputOutputs.Add(NewVariable);

	return NewVariable;
}


void UCustomizableObjectMacro::RemoveVariable(UCustomizableObjectMacroInputOutput* Variable)
{
	if (InputOutputs.Contains(Variable))
	{
		InputOutputs.Remove(Variable);
	}
}

UCustomizableObjectNodeTunnel* UCustomizableObjectMacro::GetIONode(ECOMacroIOType Type) const
{
	UCustomizableObjectNodeTunnel* OutNode = nullptr;

	if (Graph)
	{
		TArray<UCustomizableObjectNodeTunnel*> IONodes;
		Graph->GetNodesOfClass<UCustomizableObjectNodeTunnel>(IONodes);
		check(IONodes.Num() == 2);

		for (UCustomizableObjectNodeTunnel* IONode : IONodes)
		{
			if (IONode->bIsInputNode && Type == ECOMacroIOType::COMVT_Input)
			{
				OutNode = IONode;
				break;
			}
			else if(!IONode->bIsInputNode && Type == ECOMacroIOType::COMVT_Output)
			{
				OutNode = IONode;
				break;
			}
		}
	}

	return OutNode;
}

#undef LOCTEXT_NAMESPACE
