// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

#include "CustomizableObjectMacroLibrary.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


class UEdGraph;
class UCustomizableObjectNodeTunnel;

UENUM()
enum class ECOMacroIOType : uint8
{
	COMVT_Input,
	COMVT_Output
};

UCLASS()
class UCustomizableObjectMacroInputOutput : public UObject
{
	GENERATED_BODY()

public:

	/** Name of the variable and pin that represents it. */
	UPROPERTY()
	FName Name;

	/** Whether it is an input or output variable. */
	UPROPERTY()
	ECOMacroIOType Type = ECOMacroIOType::COMVT_Input;

	/** Type of the variable and pin. */
	UPROPERTY()
	FName PinCategoryType;

	/** Unique id that identifies this variable. Usefull for pin reconstruction and name repetitions. */
	UPROPERTY()
	FGuid UniqueId;
};

UCLASS(MinimalAPI)
class UCustomizableObjectMacro : public UObject
{
	GENERATED_BODY()

public:

	/** Adds a new variable to the specified Macro. */
	UE_API UCustomizableObjectMacroInputOutput* AddVariable(ECOMacroIOType VarType);

	/** Removes a variable from the specified Macro. */
	UE_API void RemoveVariable(UCustomizableObjectMacroInputOutput* Variable);

	/** Returns the input or output node */
	UE_API UCustomizableObjectNodeTunnel* GetIONode(ECOMacroIOType Type) const;

public:

	/** Name of the Macro. */
	UPROPERTY(EditAnywhere, Category = Macro)
	FName Name;

	/** Description of what this macro does. */
	UPROPERTY(EditAnywhere, Category = Macro)
	FString Description = "Macro Description";

	/** Container of all input and output variables of this Macro. */
	UPROPERTY()
	TArray<TObjectPtr<UCustomizableObjectMacroInputOutput>> InputOutputs;

	/** Graph of the Macro. */
	UPROPERTY()
	TObjectPtr<UEdGraph> Graph;

	//TODO(Max): Add a Callback to refresh all nodes that instantiate this macro when this macro changes
};

/** A Macro Library is an asset that stores reusable mutable graphs. */
UCLASS(MinimalAPI)
class UCustomizableObjectMacroLibrary : public UObject
{
	GENERATED_BODY()

public:

	UCustomizableObjectMacroLibrary() {};

	// UObject interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	/** Creates a new macro with the basic nodes and adds it to the library. */
	UE_API UCustomizableObjectMacro* AddMacro();

	/** Removes the specified macro from the library. */
	UE_API void RemoveMacro(UCustomizableObjectMacro* MacroToRemove);

public:

	/** List of macros. */
	UPROPERTY();
	TArray<TObjectPtr<UCustomizableObjectMacro>> Macros;
};

#undef UE_API
