// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableObjectMacroLibraryEditor.h"

#include "Editor/Transactor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Misc/ITransaction.h"
#include "MuCOE/CustomizableObjectEditorModule.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/Widgets/SCustomizableObjectMacroLibraryList.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "PropertyEditorModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SNullWidget.h"


#define LOCTEXT_NAMESPACE "FCustomizableObjectMacroLibraryEditor"

/** Tab Ids */
const FName FCustomizableObjectMacroLibraryEditor::DetailsTabId(TEXT("CustomizableObjectMacroLibrary_Details"));
const FName FCustomizableObjectMacroLibraryEditor::MacroSelectorTabId(TEXT("CustomizableObjectMacroLibrary_MacroSelector"));
const FName FCustomizableObjectMacroLibraryEditor::GraphTabId(TEXT("CustomizableObjectMacroLibrary_Graph"));


void FCustomizableObjectMacroLibraryEditor::InitEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit)
{
	MacroLibrary = Cast<UCustomizableObjectMacroLibrary>(ObjectToEdit);

	// Bind Commands
	BindGraphCommands();

	// Tab Generation
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_CustomizableObjectMacroLibraryEditor_Layout_v0.1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.6f)
				->SetHideTabWell(true)
				->AddTab(GraphTabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.4f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(DetailsTabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(MacroSelectorTabId, ETabState::OpenedTab)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, CustomizableObjectMacroLibraryEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit);
}


void FCustomizableObjectMacroLibraryEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CustomizableObjectEditor", "Customizable Object Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(DetailsTabId,
		FOnSpawnTab::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::SpawnTab_Details))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(MacroSelectorTabId,
		FOnSpawnTab::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::SpawnTab_MacroSelector))
		.SetDisplayName(LOCTEXT("MacroSelectorTab", "MacroSelector"))
		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(GraphTabId,
		FOnSpawnTab::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::SpawnTab_Graph))
		.SetDisplayName(LOCTEXT("GraphTab", "Graph"))
		.SetGroup(WorkspaceMenuCategoryRef);
}


void FCustomizableObjectMacroLibraryEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(DetailsTabId);
	InTabManager->UnregisterTabSpawner(MacroSelectorTabId);
	InTabManager->UnregisterTabSpawner(GraphTabId);
}


TSharedRef<SDockTab> FCustomizableObjectMacroLibraryEditor::SpawnTab_Details(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DetailsTabId);

	if (!DetailsView.IsValid())
	{
		FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bShowScrollBar = false;
		DetailsViewArgs.ExternalScrollbar = SNew(SScrollBar);

		DetailsView = PropPlugin.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(SelectedMacro, true);
	}

	return SNew(SDockTab)
	.Label(LOCTEXT("DetailsDockTab", "Details"))
	.TabColorScale(GetTabColorScale())
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			DetailsView.ToSharedRef()
		]
	];
}


TSharedRef<SDockTab> FCustomizableObjectMacroLibraryEditor::SpawnTab_MacroSelector(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == MacroSelectorTabId);

	if (!MacroSelector.IsValid())
	{
		SAssignNew(MacroSelector, SCustomizableObjectMacroLibraryList)
		.MacroLibrary(MacroLibrary)
		.OnAddMacroButtonClicked(FExecuteAction::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::OnAddMacroButtonClicked))
		.OnSelectMacro(SCustomizableObjectMacroLibraryList::FOnSelectMacroDelegate::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::OnMacroSelectionChanged))
		.OnRemoveMacro(SCustomizableObjectMacroLibraryList::FOnRemoveMacroDelegate::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::OnRemoveMacroButtonClicked));
	}

	return SNew(SDockTab)
	.Label(LOCTEXT("MacroLibraryDockTab", "Macro Selector"))
	.TabColorScale(GetTabColorScale())
	[
		MacroSelector.ToSharedRef()
	];
}


TSharedRef<SDockTab> FCustomizableObjectMacroLibraryEditor::SpawnTab_Graph(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == GraphTabId);

	if (!GraphEditor.IsValid() && SelectedMacro)
	{
		// Add Editor custom events
		SGraphEditor::FGraphEditorEvents InEvents;
		InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::OnSelectedGraphNodesChanged);

		CreateGraphEditorWidget(SelectedMacro->Graph, InEvents);
	}

	GraphTab = SNew(SDockTab)
		.Label(LOCTEXT("GraphDockTab", "Graph"))
		.TabColorScale(GetTabColorScale());

	if (GraphEditor.IsValid())
	{
		GraphTab->SetContent(GraphEditor.ToSharedRef());
	}
	else
	{
		GraphTab->SetContent(SNullWidget::NullWidget);
	}

	return GraphTab.ToSharedRef();
}


void FCustomizableObjectMacroLibraryEditor::OnSelectedGraphNodesChanged(const FGraphPanelSelectionSet& NewSelection)
{
	if (DetailsView.IsValid())
	{
		if (NewSelection.Num() == 0)
		{
			DetailsView->SetObject(SelectedMacro, true);
		}
		else if (NewSelection.Num() == 1)
		{
			if (Cast<UCustomizableObjectNodeTunnel>(NewSelection.Array()[0]) && SelectedMacro)
			{
				DetailsView->SetObject(SelectedMacro, true);
			}
			else
			{
				DetailsView->SetObject(NewSelection.Array()[0], true);
			}
		}
		else if (NewSelection.Num() > 1)
		{
			DetailsView->SetObjects(NewSelection.Array(), true);
		}
	}
}


void FCustomizableObjectMacroLibraryEditor::ReconstructAllChildNodes(UCustomizableObjectNode& StartNode, const UClass& NodeType)
{
	// TODO(Max)
}


FName FCustomizableObjectMacroLibraryEditor::GetToolkitFName() const
{
	return FName("CustomizableObjectMacroLibraryEditor");
}


FText FCustomizableObjectMacroLibraryEditor::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Customizable Object Macro Library Editor");
}


FText FCustomizableObjectMacroLibraryEditor::GetToolkitName() const
{
	if (GetEditingObjects().Num() == 1)
	{
		return FAssetEditorToolkit::GetToolkitName();
	}

	return GetBaseToolkitName();
}


FString FCustomizableObjectMacroLibraryEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "CustomizableObjectMacroLibraryEditor ").ToString();
}


FLinearColor FCustomizableObjectMacroLibraryEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
}


void FCustomizableObjectMacroLibraryEditor::PostUndo(bool bSuccess) 
{
	FCustomizableObjectGraphEditorToolkit::PostUndo(bSuccess);

	if (bSuccess)
	{
		//TODO(Max): Select the correct macro after an undo.
		/*FTransactionContext TransactionContext = GEditor->GetTransactionName();
		UE_LOGF(LogTemp, Warning, "Transaction: %ls", *TransactionContext.Title.ToString());*/

		MacroSelector->GenerateRowView();
	}
}


void FCustomizableObjectMacroLibraryEditor::PostRedo(bool bSuccess)
{
	FCustomizableObjectGraphEditorToolkit::PostRedo(bSuccess);

	if (bSuccess)
	{
		//TODO(Max): Select the correct macro after a redo.
		if (MacroSelector.IsValid())
		{
			MacroSelector->GenerateRowView();
		}
	}
}


void FCustomizableObjectMacroLibraryEditor::OnAddMacroButtonClicked()
{	
	if (MacroLibrary)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("AddMAcroScopTransaction", "Add Macro"));
		MacroLibrary->Modify();

		UCustomizableObjectMacro* NewMacro = MacroLibrary->AddMacro();
	}
}


void FCustomizableObjectMacroLibraryEditor::OnRemoveMacroButtonClicked(UCustomizableObjectMacro* MacroToRemove)
{
	if (MacroLibrary)
	{
		if (SelectedMacro == MacroToRemove)
		{
			SetSelectedMacro(nullptr);
		}

		MacroLibrary->RemoveMacro(MacroToRemove);
	}
}


void FCustomizableObjectMacroLibraryEditor::OnMacroSelectionChanged(UCustomizableObjectMacro* NewSelection)
{
	SetSelectedMacro(NewSelection);
}


void FCustomizableObjectMacroLibraryEditor::SetSelectedMacro(UCustomizableObjectMacro* NewSelection, bool bRefreshMacroSelection)
{
	if (!NewSelection)
	{
		// Removing references
		SelectedMacro = nullptr;
		GraphEditor.Reset();
		GraphTab->SetContent(SNullWidget::NullWidget);
		DetailsView->SetObject(nullptr, true);
	}
	else
	{
		if (NewSelection != SelectedMacro)
		{
			// Generate a new graph editor
			GraphEditor.Reset();
			SelectedMacro = NewSelection;

			SGraphEditor::FGraphEditorEvents InEvents;
			InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FCustomizableObjectMacroLibraryEditor::OnSelectedGraphNodesChanged);

			CreateGraphEditorWidget(NewSelection->Graph, InEvents);

			GraphTab->SetContent(GraphEditor.ToSharedRef());

			if (bRefreshMacroSelection)
			{
				MacroSelector->SetSelectedMacro(*NewSelection);
			}
		}
	}

	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(SelectedMacro, true);
	}
}

#undef LOCTEXT_NAMESPACE
