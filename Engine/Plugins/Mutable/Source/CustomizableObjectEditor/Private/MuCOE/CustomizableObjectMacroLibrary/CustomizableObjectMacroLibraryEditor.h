// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

class SCustomizableObjectMacroLibraryList;
class SDockTab;

/** Macro Library Editor. */
class FCustomizableObjectMacroLibraryEditor : public FCustomizableObjectGraphEditorToolkit
{
public:

	void InitEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UObject* ObjectToEdit);

	/** Destructor */
	virtual ~FCustomizableObjectMacroLibraryEditor() = default;

	/** IToolkit interface */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** FEditorUndoClient Interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** Sets which macro is edited in the editor.
		@param NewSelection pointer to the new macro to select
		@param bRefreshMacroSelection force to refresh the selection of the Macros List. This is only needed if the selection is done by code and not by the user. */
	void SetSelectedMacro(UCustomizableObjectMacro* NewSelection, bool bRefreshMacroSelection = false);

private:

	/** Tab Spawners */
	TSharedRef<SDockTab> SpawnTab_Details(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_MacroSelector(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Graph(const FSpawnTabArgs& Args);

	// CO Graph Editor Toolkit Interface
	/** Graph Editor callbacks */
	virtual void OnSelectedGraphNodesChanged(const FGraphPanelSelectionSet& NewSelection) override;
	virtual void ReconstructAllChildNodes(UCustomizableObjectNode& StartNode, const UClass& NodeType) override;

	/** Macro List View callbacks */
	void OnAddMacroButtonClicked();
	void OnRemoveMacroButtonClicked(UCustomizableObjectMacro* NewSelection);
	void OnMacroSelectionChanged(UCustomizableObjectMacro* NewSelection);

public:

	/** Tab Ids */
	static const FName DetailsTabId;
	static const FName MacroSelectorTabId;
	static const FName GraphTabId;

private:

	/** Macro Library being edited */
	TObjectPtr<UCustomizableObjectMacroLibrary> MacroLibrary = nullptr;
	
	/** The currently viewed Macro Graph. */
	TObjectPtr<UCustomizableObjectMacro> SelectedMacro;

	/** Details view */
	TSharedPtr<IDetailsView> DetailsView;

	/** Macro List View widget */
	TSharedPtr<SCustomizableObjectMacroLibraryList> MacroSelector;

	/** Pointer to the tab that contains the graph editor. Needed to refresh its content. */
	TSharedPtr<SDockTab> GraphTab;

};

