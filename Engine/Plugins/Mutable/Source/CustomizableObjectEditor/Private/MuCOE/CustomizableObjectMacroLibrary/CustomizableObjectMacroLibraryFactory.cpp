// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibraryFactory.h"

#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectMacroLibraryFactory)

UCustomizableObjectMacroLibraryFactory::UCustomizableObjectMacroLibraryFactory() 
{
	SupportedClass = UCustomizableObjectMacroLibrary::StaticClass();
	bEditAfterNew = true;
}

UObject* UCustomizableObjectMacroLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UCustomizableObjectMacroLibrary* NewMacroLibrary = NewObject<UCustomizableObjectMacroLibrary>(InParent, Class, Name, Flags, Context);
	
	if (NewMacroLibrary)
	{
		NewMacroLibrary->AddMacro();
	}

	return NewMacroLibrary;
}

bool UCustomizableObjectMacroLibraryFactory::CanCreateNew() const
{
	return true;
}
