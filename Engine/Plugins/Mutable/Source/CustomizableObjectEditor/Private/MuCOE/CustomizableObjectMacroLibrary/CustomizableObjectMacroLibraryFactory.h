// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "CustomizableObjectMacroLibraryFactory.generated.h"


UCLASS()
class UCustomizableObjectMacroLibraryFactory : public UFactory
{
	GENERATED_BODY()

public:

	UCustomizableObjectMacroLibraryFactory();

	// UFactory Interface
	UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;

};

