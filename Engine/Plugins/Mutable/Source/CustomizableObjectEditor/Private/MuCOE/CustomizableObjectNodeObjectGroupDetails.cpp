// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectNodeObjectGroupDetails.h"

#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "Misc/Attribute.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/STextComboBox.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectGroupDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeObjectGroupDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeObjectGroupDetails);
}


void FCustomizableObjectNodeObjectGroupDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();

	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		if (DetailsView->GetSelectedObjects()[0].Get()->IsA(UCustomizableObjectNodeObjectGroup::StaticClass()))
		{
			NodeGroup = Cast<UCustomizableObjectNodeObjectGroup>(DetailsView->GetSelectedObjects()[0].Get());
		}
	}

	if (NodeGroup)
	{
		DetailBuilder.HideProperty("DefaultValue");

		IDetailCategoryBuilder& GroupInfoCategory = DetailBuilder.EditCategory("GroupInfo");

		// Forcing property order
		{
			DetailBuilder.HideProperty("GroupName");
			GroupInfoCategory.AddProperty("GroupType");
		}


		// Getting group node children names
		GenerateChildrenObjectNames();

		GroupInfoCategory.AddCustomRow(LOCTEXT("NodeObjectGroupDetails_ComboBox", "Default Value Selector"))
			.Visibility(TAttribute<EVisibility>(this, &FCustomizableObjectNodeObjectGroupDetails::DefaultValueSelectorVisibility))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NodeObjectGroupDetails_ComboBox_Text", "Default Value"))
			.ToolTipText(LOCTEXT("NodeObjectGroupDetails_ComboBox_Tooltip", "Select the default value of the group."))
			.Font(DetailBuilder.GetDetailFont())
			]
		.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SAssignNew(DefaultValueSelector, STextComboBox)
				.InitiallySelectedItem(InitialNameOption)
			.OptionsSource(&ChildrenNameOptions)
			.OnComboBoxOpening(this, &FCustomizableObjectNodeObjectGroupDetails::GenerateChildrenObjectNames)
			.OnSelectionChanged(this, &FCustomizableObjectNodeObjectGroupDetails::OnSetDefaultValue)
			.Font(DetailBuilder.GetDetailFont())
			.ToolTipText(this, &FCustomizableObjectNodeObjectGroupDetails::DefaultValueComboBoxTooltip)
			];

		if (TSharedPtr<FCustomizableObjectEditor> GraphEditor = StaticCastSharedPtr<FCustomizableObjectEditor>(NodeGroup->GetGraphEditor()))
		{
			if (UCustomizableObject* NodeGroupCO = CastChecked<UCustomizableObject>(NodeGroup->GetCustomizableObjectGraph()->GetOuter()))
			{
				IDetailCategoryBuilder& BlocksCategory = DetailBuilder.EditCategory("External Objects");

				TMultiMap<FGuid, UCustomizableObjectNodeObject*> ObjectNodesObjects = GetNodeGroupObjectNodeMapping(NodeGroupCO);
				TArray<UCustomizableObjectNodeObject*> ChildNodes;

				ObjectNodesObjects.MultiFind(NodeGroup->NodeGuid, ChildNodes);

				for (const UCustomizableObjectNodeObject* ChildObjectNode : ChildNodes)
				{
					if (ChildObjectNode)
					{
						BlocksCategory.AddCustomRow(LOCTEXT("FCustomizableObjectNodeObjectGroupDetails", "External Customizable Objects in this Group"))
						[
							SNew(SObjectPropertyEntryBox)
							.ObjectPath(ChildObjectNode->GetOutermostObject()->GetPathName())
							.AllowedClass(UCustomizableObject::StaticClass())
							.AllowClear(false)
							.DisplayUseSelected(false)
							.DisplayBrowse(true)
							.EnableContentPicker(false)
							.DisplayThumbnail(true)
						];
					}
				}
			}
		}
	}
}


void FCustomizableObjectNodeObjectGroupDetails::GenerateChildrenObjectNames()
{
	ChildrenNameOptions.Reset();
	InitialNameOption = nullptr;

	// if needed add a None option and set it as the default value
	if (NodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE_OR_NONE)
	{
		ChildrenNameOptions.Add(MakeShareable(new FString("None")));
		InitialNameOption = ChildrenNameOptions.Last();
	}
	else
	{
		ChildrenNameOptions.Add(MakeShareable(new FString("- Not Selected -")));
		InitialNameOption = ChildrenNameOptions.Last();
	}

	// Adding linked children names
	const TArray<UEdGraphPin*> ConnectedChildrenPins = FollowInputPinArray(*NodeGroup->ObjectsPin());
	for (const UEdGraphPin* ChildPin : ConnectedChildrenPins)
	{
		if (ChildPin)
		{
			if (const UCustomizableObjectNodeObject* ChildObjectNode = Cast<UCustomizableObjectNodeObject>(ChildPin->GetOwningNode()))
			{
				ChildrenNameOptions.Add(MakeShareable(new FString(ChildObjectNode->GetObjectName())));

				if (ChildObjectNode->GetObjectName().Equals(NodeGroup->DefaultValue))
				{
					InitialNameOption = ChildrenNameOptions.Last();
				}
			}
			else if (const UCustomizableObjectNodeObjectGroup* ChildGroupObjectNode = Cast<UCustomizableObjectNodeObjectGroup>(ChildPin->GetOwningNode()))
			{
				ChildrenNameOptions.Add(MakeShareable(new FString(ChildGroupObjectNode->GetGroupName())));

				if (ChildGroupObjectNode->GetGroupName().Equals(NodeGroup->DefaultValue))
				{
					InitialNameOption = ChildrenNameOptions.Last();
				}
			}
		}
	}

	// Adding external children names
	if (TSharedPtr<FCustomizableObjectEditor> GraphEditor = StaticCastSharedPtr<FCustomizableObjectEditor>(NodeGroup->GetGraphEditor()))
	{
		if (UCustomizableObject* NodeGroupCO = CastChecked<UCustomizableObject>(NodeGroup->GetCustomizableObjectGraph()->GetOuter()))
		{
			TMultiMap<FGuid, UCustomizableObjectNodeObject*> ObjectNodesObjects = GetNodeGroupObjectNodeMapping(NodeGroupCO);
			TArray<UCustomizableObjectNodeObject*> ChildNodes;

			ObjectNodesObjects.MultiFind(NodeGroup->NodeGuid, ChildNodes);

			for (const UCustomizableObjectNodeObject* ChildObjectNode : ChildNodes)
			{
				if (ChildObjectNode)
				{
					ChildrenNameOptions.Add(MakeShareable(new FString(ChildObjectNode->GetObjectName())));

					if (ChildObjectNode->GetObjectName().Equals(NodeGroup->DefaultValue))
					{
						InitialNameOption = ChildrenNameOptions.Last();
					}
				}
			}
		}
	}

	if (DefaultValueSelector.IsValid())
	{
		DefaultValueSelector->RefreshOptions();
		DefaultValueSelector->SetSelectedItem(InitialNameOption);
	}
}


void FCustomizableObjectNodeObjectGroupDetails::OnSetDefaultValue(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (NewSelection.IsValid() && (SelectInfo == ESelectInfo::OnMouseClick || SelectInfo == ESelectInfo::OnKeyPress))
	{
		NodeGroup->DefaultValue = *NewSelection;
	}
}


EVisibility FCustomizableObjectNodeObjectGroupDetails::DefaultValueSelectorVisibility() const
{
	if (NodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE || NodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE_OR_NONE)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}


FText FCustomizableObjectNodeObjectGroupDetails::DefaultValueComboBoxTooltip() const
{
	if (DefaultValueSelector.IsValid())
	{
		if (ChildrenNameOptions.Num() && DefaultValueSelector->GetSelectedItem() == ChildrenNameOptions[0] && NodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE)
		{
			return FText::FromString("When nothing selected, the first compiled option will be used as the default value.");
		}
		else if (DefaultValueSelector->GetSelectedItem().IsValid())
		{
			return FText::FromString(*DefaultValueSelector->GetSelectedItem());
		}
	}

	return FText();
}

#undef LOCTEXT_NAMESPACE
