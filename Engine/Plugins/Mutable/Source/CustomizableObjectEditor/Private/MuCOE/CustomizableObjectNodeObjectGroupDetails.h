// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Nodes/CustomizableObjectNodeDetails.h"

struct EVisibility;
namespace ESelectInfo { enum Type : int; }
class IDetailLayoutBuilder;
class STextComboBox;
class UCustomizableObjectNodeObjectGroup;

class FCustomizableObjectNodeObjectGroupDetails : public FCustomizableObjectNodeDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	/** Fills the name options array */
	void GenerateChildrenObjectNames();

	/** Sets the selected default value */
	void OnSetDefaultValue(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	
	/** Determines the visibility of the default value selector */
	EVisibility DefaultValueSelectorVisibility() const;

	FText DefaultValueComboBoxTooltip() const;

private:

	/** Pointer to the original group node */
	UCustomizableObjectNodeObjectGroup* NodeGroup = nullptr;

	/** Combobox to select the defaul value of the node group */
	TSharedPtr<STextComboBox> DefaultValueSelector;

	/** Combobox options */
	TArray< TSharedPtr<FString> > ChildrenNameOptions;

	/** Initially selected option */
	TSharedPtr<FString> InitialNameOption;

};
