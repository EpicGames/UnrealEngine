// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectPin.h"

#include "EdGraph/EdGraphPin.h"

bool IsPinOrphan(const UEdGraphPin &Pin)
{
	return Pin.bOrphanedPin;
}


void OrphanPin(UEdGraphPin& Pin)
{
	Pin.bOrphanedPin = true;
	Pin.SetSavePinIfOrphaned(true);
}