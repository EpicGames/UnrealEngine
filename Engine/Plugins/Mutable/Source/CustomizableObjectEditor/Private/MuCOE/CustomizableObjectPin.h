// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class UEdGraphPin;

/** Returns true if the pin is orphan. */
bool IsPinOrphan(const UEdGraphPin &Pin);


/** Allows to perform work when orphaning a pin. */
void OrphanPin(UEdGraphPin& Pin);
