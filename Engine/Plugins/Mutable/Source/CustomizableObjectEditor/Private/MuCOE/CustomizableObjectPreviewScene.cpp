// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/CustomizableObjectPreviewScene.h"


FCustomizableObjectPreviewScene::FCustomizableObjectPreviewScene(ConstructionValues CVS, float InFloorOffset)
	: FAdvancedPreviewScene(CVS, InFloorOffset)
{

}


FCustomizableObjectPreviewScene::~FCustomizableObjectPreviewScene()
{

}


UStaticMeshComponent* FCustomizableObjectPreviewScene::GetSkyComponent()
{
	return SkyComponent;
}
