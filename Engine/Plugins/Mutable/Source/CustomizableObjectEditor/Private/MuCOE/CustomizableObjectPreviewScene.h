// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"

class UStaticMeshComponent;

class FCustomizableObjectPreviewScene : public FAdvancedPreviewScene
{
public:
	/** Parameter constructor */
	FCustomizableObjectPreviewScene(ConstructionValues CVS, float InFloorOffset = 0.0f);

	/** Destructor */
	virtual ~FCustomizableObjectPreviewScene();

	/** Getter of SkyComponent */
	UStaticMeshComponent* GetSkyComponent();
};