// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectProjectSettings.generated.h"


/** COE Module, Project Settings. */
UCLASS(config = Mutable, defaultconfig)
class UCustomizableObjectProjectSettings : public UObject
{
	GENERATED_BODY()

public:
};
