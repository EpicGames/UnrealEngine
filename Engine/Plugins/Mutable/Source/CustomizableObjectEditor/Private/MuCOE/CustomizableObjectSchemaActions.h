// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2_Actions.h"
#include "Types/SlateVector2.h"

#include "CustomizableObjectSchemaActions.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


/** Action to add a node to the graph */
USTRUCT()
struct FCustomizableObjectSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UEdGraphNode> NodeTemplate;


	FCustomizableObjectSchemaAction_NewNode() 
		: FEdGraphSchemaAction()
		, NodeTemplate(NULL)
	{}

	FCustomizableObjectSchemaAction_NewNode(const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, const int32 InGrouping, const FText& InKeywords = FText(), int32 InSectionID = 0)
		: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGrouping, InKeywords, InSectionID)
		, NodeTemplate(NULL)
	{}

	// FEdGraphSchemaAction interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	/** Reimplementaiton of EdGraphSchema::CreateNode(...). Performs the overlap calculation before calling AutowireNewNode(...). AutowireNewNode can induce a call to a ReconstrucNode() which removes pins required for the calculation. */
	static UE_API UEdGraphNode* CreateNode(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, class UEdGraphNode* InNodeTemplate);
	// End of FEdGraphSchemaAction interface

	template <typename NodeType>
	static NodeType* InstantSpawn(class UEdGraph* ParentGraph, NodeType* InTemplateNode, const UE::Slate::FDeprecateVector2DParameter& Location)
	{
		FEdGraphSchemaAction_K2NewNode Action;
		Action.NodeTemplate = InTemplateNode;

		return Cast<NodeType>(Action.PerformAction(ParentGraph, NULL, Location));
	}
};


/** Action to paste clipboard contents into the graph */
USTRUCT()
struct FCustomizableObjectSchemaAction_Paste : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY();

	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FCustomizableObjectSchemaAction_Paste"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FCustomizableObjectSchemaAction_Paste()
		: FEdGraphSchemaAction()
	{}

	FCustomizableObjectSchemaAction_Paste(const FText& InNodeCategory, const FText& InMenuDesc, const FString& InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, FText::FromString(InToolTip), InGrouping)
	{}

	// FEdGraphSchemaAction interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

#undef UE_API
