// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include "AssetDefinitionRegistry.h"
#include "MuCOE/CustomizableObjectSchemaActions.h"

#include "CustomizableObjectConnectionDrawingPolicy.h"
#include "CustomizableObjectEditorSettings.h"
#include "EdGraphNode_Comment.h"
#include "Editor.h"
#include "Editor/AssetReferenceFilter.h"
#include "Editor/EditorEngine.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "MuCOE/ICustomizableObjectEditorModulePrivate.h"
#include "Materials/MaterialInterface.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/Nodes/CONodeMaterialBreak.h"
#include "MuCOE/Nodes/CONodeMaterialConstant.h"
#include "MuCOE/Nodes/CONodeMaterialModify.h"
#include "MuCOE/Nodes/CONodeMaterialVariation.h"
#include "MuCOE/Nodes/CONodeModifierTransformWithBone.h"
#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorArithmeticOp.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorFromFloats.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorToSRGB.h"
#include "MuCOE/Nodes/CustomizableObjectNodeCurve.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExposePin.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatArithmeticOp.h"
#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMaterialParameter.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSectionVariation.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierEditSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocks.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipDeform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithUVMask.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierTransformInMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackApplication.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackDefinition.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshape.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectChild.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentPassthroughMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTexture.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureBinarise.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureColorMap.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromChannels.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromColor.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromFloats.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureInterpolate.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureInvert.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureLayer.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureProject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureSample.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureToChannels.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureTransform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureSaturate.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodePassThroughTextureVariation.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeReroute.h"
#include "Toolkits/ToolkitManager.h"
#include "PropertyEditorModule.h"
#include "Animation/PoseAsset.h"
#include "MuCO/ICustomizableObjectModulePrivate.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeModifierSkeletalMeshMerge.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTransformConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTransformParameter.h"
#include "MuR/External/FloatAdapter.h"
#include "MuR/External/MaterialAdapter.h"
#include "MuR/External/MeshAdapter.h"
#include "MuR/External/Operation.h"
#include "MuR/External/TextureAdapter.h"
#include "MuR/External/VectorAdapter.h"
#include "Nodes/CONodeClipMeshWithMesh.h"
#include "Nodes/CONodeClipSkeletalMeshWithSkeletalMesh.h"
#include "Nodes/CONodeExternalOperation.h"
#include "Nodes/CONodeExternalTypeParameter.h"
#include "Nodes/CONodeRemoveMesh.h"
#include "Nodes/CONodeSkeletalMeshObjectBreak.h"
#include "Nodes/CONodeSkeletalMeshMake_V2.h"
#include "Nodes/CONodeSkeletalMeshVariation.h"
#include "Nodes/CONodeSkeletalMeshMerge.h"
#include "Nodes/CONodeSkeletalMeshModify.h"
#include "Nodes/CONodeSkeletalMeshMorph.h"
#include "Nodes/CONodeSkeletalMeshReshape.h"
#include "Nodes/CONodeSkeletalMeshTransformWithBone.h"
#include "Nodes/CONodeSkeletalMeshBreak_V2.h"
#include "Nodes/CONodeTransformWithBone.h"
#include "Nodes/CONodeTransformInMesh.h"
#include "Nodes/CONodeSwitch.h"
#include "UObject/Class.h"
#include "Widgets/SMutablePinTypeSelector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_CustomizableObject)

class UCustomizableObjectEditorSettings;
class IToolkit;


#define LOCTEXT_NAMESPACE "CustomizableObjectSchema"

#define SNAP_GRID (16) // @todo ensure this is the same as SNodePanel::GetSnapGridSize()


namespace UE::Mutable::Private
{
	// Maximum distance a drag can be off a node edge to require 'push off' from node
	const int32 NodeDistance = 60;
}


TSharedPtr<FCustomizableObjectGraphEditorToolkit> GetCustomizableObjectEditor(const UEdGraph* ParentGraph)
{
	// Find the associated Editor
	const UCustomizableObjectGraph* CustomizableObjectGraph = Cast<UCustomizableObjectGraph>(ParentGraph);

	if (CustomizableObjectGraph)
	{
		// An CO or a Macro Library
		UObject* AssetBeingEdited;

		if (CustomizableObjectGraph->IsMacro() && CustomizableObjectGraph->GetOuter()) //Macro Library
		{
			AssetBeingEdited = CustomizableObjectGraph->GetOuter()->GetOuter();
		}
		else // CO
		{
			AssetBeingEdited = CustomizableObjectGraph->GetOuter();
		}

		if (AssetBeingEdited)
		{
			TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(AssetBeingEdited);
			if (FoundAssetEditor.IsValid())
			{
				return StaticCastSharedPtr<FCustomizableObjectGraphEditorToolkit>(FoundAssetEditor);
			}
		}
	}

	return nullptr;
}


UEdGraphNode* FCustomizableObjectSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode* ResultNode = nullptr;

	// If there is a template, we actually use it
	if (NodeTemplate)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
		ParentGraph->Modify();

		if (FromPin)
		{
			FromPin->Modify();
		}

		ResultNode = FCustomizableObjectSchemaAction_NewNode::CreateNode(ParentGraph, FromPin, FDeprecateSlateVector2D(Location), NodeTemplate);
	}

	return ResultNode;
}


UEdGraphNode* FCustomizableObjectSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode) 
{
	UEdGraphNode* ResultNode = 0;

	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location,bSelectNewNode);

		// Try autowiring the rest of the pins
		for (int32 Index = 1; Index < FromPins.Num(); ++Index)
		{
			ResultNode->AutowireNewNode(FromPins[Index]);
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, NULL, Location, bSelectNewNode);
	}

	return ResultNode;
}


UEdGraphNode* FCustomizableObjectSchemaAction_NewNode::CreateNode(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, class UEdGraphNode* InNodeTemplate)
{
	// UE Code from FSchemaAction_NewNode::CreateNode(...). Overlap calculations performed before AutowireNewNode(...).
	
	// Duplicate template node to create new node
	UEdGraphNode* ResultNode = DuplicateObject<UEdGraphNode>(InNodeTemplate, ParentGraph);

	ResultNode->SetFlags(RF_Transactional);

	ParentGraph->AddNode(ResultNode, true);

	ResultNode->CreateNewGuid();
	ResultNode->PostPlacedNewNode();
	if (UCustomizableObjectNode* TypedResultNode = Cast<UCustomizableObjectNode>(ResultNode))
	{
		TypedResultNode->BeginConstruct();
		TypedResultNode->PostBackwardsCompatibleFixup();
	}
	ResultNode->ReconstructNode(); // Mutable node lifecycle always starts at ReconstructNode.

	// For input pins, new node will generally overlap node being dragged off
	// Work out if we want to visually push away from connected node
	int32 XLocation = Location.X;
	if (FromPin && FromPin->Direction == EGPD_Input)
	{
		UEdGraphNode* PinNode = FromPin->GetOwningNode();
		const float XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

		if (XDelta < UE::Mutable::Private::NodeDistance)
		{
			// Set location to edge of current node minus the max move distance
			// to force node to push off from connect node enough to give selection handle
			XLocation = PinNode->NodePosX - UE::Mutable::Private::NodeDistance;
		}
	}

	ResultNode->AutowireNewNode(FromPin);

	ResultNode->NodePosX = XLocation;
	ResultNode->NodePosY = Location.Y;
	ResultNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

	return ResultNode;
}


void FCustomizableObjectSchemaAction_NewNode::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( NodeTemplate );
}


////////////////////////////////////////
// FCustomizableObjectSchemaAction_Paste

UEdGraphNode* FCustomizableObjectSchemaAction_Paste::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	TSharedPtr<FCustomizableObjectGraphEditorToolkit> CustomizableObjectEditor = GetCustomizableObjectEditor(ParentGraph);

	if (CustomizableObjectEditor.IsValid() && CustomizableObjectEditor->CanPasteNodes())
	{
		CustomizableObjectEditor->PasteNodesHere(FDeprecateSlateVector2D(Location));
	}
	
	return nullptr;
}


//////////////////////////////////////////////////////////////////////////
// DO NOT change the values because it will break the external pin nodes!
const FName UEdGraphSchema_CustomizableObject::PC_Object("object");
const FName UEdGraphSchema_CustomizableObject::PC_Component("component");
const FName UEdGraphSchema_CustomizableObject::PC_MeshSection("material");
const FName UEdGraphSchema_CustomizableObject::PC_Modifier("modifier");
const FName UEdGraphSchema_CustomizableObject::PC_Mesh("mesh");
const FName UEdGraphSchema_CustomizableObject::PC_SkeletalMesh("skeletalMesh");
const FName UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough("passThroughSkeletalMesh");
const FName UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object("passThroughParameterSkeletalMesh");
const FName UEdGraphSchema_CustomizableObject::PC_Texture("image");
const FName UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough("passThroughImage");
const FName UEdGraphSchema_CustomizableObject::PC_Texture_Object("passThroughParameterImage");
const FName UEdGraphSchema_CustomizableObject::PC_Projector("projector");
const FName UEdGraphSchema_CustomizableObject::PC_GroupProjector("groupProjector");
const FName UEdGraphSchema_CustomizableObject::PC_Color("color");
const FName UEdGraphSchema_CustomizableObject::PC_Float("float");
const FName UEdGraphSchema_CustomizableObject::PC_Bool("bool");
const FName UEdGraphSchema_CustomizableObject::PC_Enum("enum");
const FName UEdGraphSchema_CustomizableObject::PC_Stack("stack");
const FName UEdGraphSchema_CustomizableObject::PC_Material("materialAsset");
const FName UEdGraphSchema_CustomizableObject::PC_Wildcard("wildcard");
const FName UEdGraphSchema_CustomizableObject::PC_PoseAsset("poseAsset");
const FName UEdGraphSchema_CustomizableObject::PC_Transform("transform");
const FName UEdGraphSchema_CustomizableObject::PC_String("string");
const FName UEdGraphSchema_CustomizableObject::PC_StaticMesh("staticMesh");

// DO NOT change the values because it will break the external pin nodes!
const FName UEdGraphSchema_CustomizableObject::PSC_Material_Override("overrideMaterial");
const FName UEdGraphSchema_CustomizableObject::PSC_Material_Overlay("overlayMaterial");


// Add more pin types to this array if needed
const TArray<FName> UEdGraphSchema_CustomizableObject::SupportedMacroPinTypes = { PC_Object, PC_Component, PC_MeshSection, PC_Modifier, PC_Mesh, PC_Texture,
	PC_Texture_Passthrough, PC_Projector, PC_GroupProjector, PC_Color, PC_Float, PC_Enum, PC_Stack, PC_Material, PC_PoseAsset, PC_Transform, PC_String };

// Node categories
const FText UEdGraphSchema_CustomizableObject::NC_Experimental(FText::FromString("Experimental"));
const FText UEdGraphSchema_CustomizableObject::NC_Material(FText::FromString("Material"));


TSharedPtr<FCustomizableObjectSchemaAction_NewNode> AddNewNodeAction(FGraphActionListBuilderBase& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const FText& Tooltip, const int32 Grouping = 0, const FString& Keywords = FString())
{
	TSharedPtr<FCustomizableObjectSchemaAction_NewNode> NewActionNode = MakeShared<FCustomizableObjectSchemaAction_NewNode>(Category, MenuDesc, Tooltip, Grouping, FText::FromString(Keywords));

	ContextMenuBuilder.AddAction( NewActionNode );

	return NewActionNode;
}


TSharedPtr<FCustomizableObjectSchemaAction_NewNode> AddNewNodeAction(TArray< TSharedPtr<FEdGraphSchemaAction> >& OutTypes, const FText& Category, const FText& MenuDesc, const FText& Tooltip)
{
	return *(new (OutTypes) TSharedPtr<FCustomizableObjectSchemaAction_NewNode>(new FCustomizableObjectSchemaAction_NewNode(Category, MenuDesc, Tooltip, 0)));
}


namespace 
{
	bool PinRelevancyFilter(UCustomizableObjectNode* TemplateNode, FGraphContextMenuBuilder& ContextMenuBuilder)
	{
		const UEdGraphPin* FromPin = ContextMenuBuilder.FromPin;
		if (!FromPin)
		{
			return true;
		}

		if (TemplateNode->ProvidesCustomPinRelevancyTest())
		{
			return TemplateNode->IsPinRelevant(ContextMenuBuilder.FromPin);
		}

		TemplateNode->BeginConstruct();
		TemplateNode->ReconstructNode();
		
		for (const UEdGraphPin* Pin : TemplateNode->GetAllNonOrphanPins())
		{
			const UEdGraphPin* InputPin = nullptr;
			const UEdGraphPin* OutputPin = nullptr;

			if (!UEdGraphSchema_CustomizableObject::CategorizePinsByDirection(Pin, FromPin, InputPin, OutputPin))
			{
				continue;
			}

			const UCustomizableObjectNode* InputNode = Cast<UCustomizableObjectNode>(InputPin->GetOwningNode());
			bool bOtherNodeIsBlocklisted = false;
			bool bArePinsCompatible = false;
			if (InputNode->CanConnect(InputPin, OutputPin, bOtherNodeIsBlocklisted, bArePinsCompatible))
			{
				return true;
			}
		}

		return false;
	}

	template<class FilterFn>
	void AddNewNodeActionFiltered(UCustomizableObjectNode* TemplateNode, FGraphContextMenuBuilder& ContextMenuBuilder, const FText& Category, const FText& MenuDesc, const int32 Grouping, FilterFn&& Filter)
	{
		if (!Filter(TemplateNode, ContextMenuBuilder))
		{
			return;
		}

		TSharedPtr<FCustomizableObjectSchemaAction_NewNode> Action = AddNewNodeAction(ContextMenuBuilder, Category, MenuDesc, FText(), Grouping);
		Action->NodeTemplate = TemplateNode;
	}

	template<class FilterFn>
	void AddNewNodeActionFiltered(UCustomizableObjectNode* TemplateNode, FGraphContextMenuBuilder& ContextMenuBuilder, const FText& Category, const int32 Grouping, FilterFn&& Filter)
	{
		AddNewNodeActionFiltered(TemplateNode, ContextMenuBuilder, Category, TemplateNode->GetNodeTitle(ENodeTitleType::ListView), Grouping, Forward<FilterFn>(Filter));
	}

	template<size_t N, class FilterFn>
	void AddNewNodeCategoryActionsFiltered(UCustomizableObjectNode* (&TemplateNodes)[N], FGraphContextMenuBuilder& ContextMenuBuilder, const FText& Category, const int32 Grouping, FilterFn&& Filter)
	{
		for (size_t i = 0; i < N; ++i)
		{
			AddNewNodeActionFiltered(TemplateNodes[i], ContextMenuBuilder, Category, Grouping, Forward<FilterFn>(Filter));
		}
	}
}


void UEdGraphSchema_CustomizableObject::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	//const UCustomizableObjectGraph* CustomizableObjectGraph = CastChecked<UCustomizableObjectGraph>(ContextMenuBuilder.CurrentGraph);
	//TSharedPtr<FCustomizableObjectSchemaAction_NewNode> Action;
	constexpr int32 GeneralGrouping = 3;

	constexpr  bool bDisableFilter = false;

	const UCustomizableObjectEditorSettings* EditorSettings = GetMutableDefault<UCustomizableObjectEditorSettings>();
	
	// return true if Filter is passed.
	const auto Filter = [bDisableFilter](UCustomizableObjectNode* TemplateNode, FGraphContextMenuBuilder& ContextMenuBuilder) -> bool
	{
		const UCustomizableObjectGraph* CustomizableObjectGraph = Cast<UCustomizableObjectGraph>(ContextMenuBuilder.CurrentGraph);

		if (CustomizableObjectGraph && TemplateNode && CustomizableObjectGraph->IsMacro())
		{
			if (!TemplateNode->IsNodeSupportedInMacros())
			{
				return false;
			}
		}
		
		if (!ContextMenuBuilder.FromPin || bDisableFilter)
		{
			return true;
		}

		return PinRelevancyFilter(TemplateNode, ContextMenuBuilder);
	};

	{
		UCustomizableObjectNode* Node = ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeObject>();
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Object), LOCTEXT("Base_Group", "Base Object"), GeneralGrouping, 
			[&Filter](UCustomizableObjectNode* TemplateNode, FGraphContextMenuBuilder& ContextMenuBuilder) -> bool
			{
				// Only let user add a base node if there isn't one in the graph
				for (const TObjectPtr<UEdGraphNode>& AuxNode : ContextMenuBuilder.CurrentGraph->Nodes)
				{
					UCustomizableObjectNodeObject* CustomizableObjectNodeObject = Cast<UCustomizableObjectNodeObject>(AuxNode);

					if (CustomizableObjectNodeObject && CustomizableObjectNodeObject->bIsBase)
					{
						return false;
					}
				}

				return Filter(TemplateNode, ContextMenuBuilder);
			}	
		);
	}

	{
		UCustomizableObjectNode* Node = NewObject<UCustomizableObjectNodeObjectGroup>(); //ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeObjectGroup>();
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Object), LOCTEXT("Child_Group", "Object Group"), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* Node = ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeObjectChild>();
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Object), LOCTEXT("Child_Object", "Child Object"), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* ObjectTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTable>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMacroInstance>(),
		};

		AddNewNodeCategoryActionsFiltered(ObjectTemplateNodes, ContextMenuBuilder, LOCTEXT("Category_Other", "Other"), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* ObjectTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeStaticString>()
		};

		AddNewNodeCategoryActionsFiltered(ObjectTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_String), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* ObjectTemplateNodes[]
		{
				ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshSection>(),
				ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshSectionVariation>(),
		};

		AddNewNodeCategoryActionsFiltered(ObjectTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_MeshSection), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* ModifierTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCONodeModifierExtendSkeletalMeshSection>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierRemoveMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierRemoveMeshBlocks>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeModifierEditSkeletalMeshSection>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierMorphMeshSection>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierClipMorph>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierClipWithMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierClipWithUVMask>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeModifierTransformWithBone>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierTransformInMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeModifierMutableSkeletalMeshMerge>(),
		};

		AddNewNodeCategoryActionsFiltered(ModifierTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Modifier), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* ComponentTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCONodeComponentSkeletalMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeComponentVariation>(),
		};

		AddNewNodeCategoryActionsFiltered(ComponentTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Component), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* MeshTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMeshMorph>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMeshVariation>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMeshMorphStackApplication>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeAnimationPose>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeClipMeshWithMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeRemoveMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeTransformWithBone>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeTransformInMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMeshReshape>(),
		};

		AddNewNodeCategoryActionsFiltered(MeshTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Mesh), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* MeshTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMeshMorphStackDefinition>(),
		};

		AddNewNodeCategoryActionsFiltered(MeshTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Stack), GeneralGrouping, Filter);
	}	

	{
		UCustomizableObjectNode* DeveloperModeMeshTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeSkeletalMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshModify>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshMorph>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshReshape>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshVariation>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshTransformWithBone>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeClipSkeletalMeshWithSkeletalMesh>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeSkeletalMeshParameter>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshMake_V2>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshMerge>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshBreak_V2>(),
		};
			
		AddNewNodeCategoryActionsFiltered(DeveloperModeMeshTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_SkeletalMesh), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* DeveloperModeMeshTemplateNodes[]
		{
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshObjectBreak>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeSkeletalMeshObjectMake>(),
		};
			
		AddNewNodeCategoryActionsFiltered(DeveloperModeMeshTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_SkeletalMesh_Object), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* DeveloperModeMeshTemplateNodes[]
		{
		
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeStaticMesh>(),
		};
			
		AddNewNodeCategoryActionsFiltered(DeveloperModeMeshTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_StaticMesh), GeneralGrouping, Filter);
	}
		
	{
		UCustomizableObjectNode* TextureTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTexture>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureBinarise>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureInterpolate>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureLayer>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureVariation>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureToChannels>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureFromChannels>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureFromColor>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureFromFloats>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureProject>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureInvert>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureColorMap>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureTransform>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureSaturate>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureParameter>(),
		};

		AddNewNodeCategoryActionsFiltered(TextureTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Texture), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* TextureTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodePassThroughTextureVariation>(),
		};

		AddNewNodeCategoryActionsFiltered(TextureTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Texture_Object), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* ColorTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeColorConstant>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeColorParameter>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeColorVariation>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTextureSample>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeColorArithmeticOp>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeColorFromFloats>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeColorToSRGB>(),
		};

		AddNewNodeCategoryActionsFiltered(ColorTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Color), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* EnumTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeEnumParameter>(),
		};

		AddNewNodeCategoryActionsFiltered(EnumTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Enum), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* FloatTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeFloatConstant>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeFloatParameter>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeFloatArithmeticOp>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeFloatVariation>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeCurve>(),
		};


		AddNewNodeCategoryActionsFiltered(FloatTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Float), GeneralGrouping, Filter);

		UCustomizableObjectNodeFloatArithmeticOp* Node = NewObject<UCustomizableObjectNodeFloatArithmeticOp>();
		Node->Operation = EFloatArithmeticOperation::E_Add;
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Float), LOCTEXT("Float_Addition", "Float Addition +"), GeneralGrouping, Filter);

		Node = NewObject<UCustomizableObjectNodeFloatArithmeticOp>();
		Node->Operation = EFloatArithmeticOperation::E_Sub;
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Float), LOCTEXT("Float_Subtraction", "Float Subtraction -"), GeneralGrouping, Filter);

		Node = NewObject<UCustomizableObjectNodeFloatArithmeticOp>();
		Node->Operation = EFloatArithmeticOperation::E_Mul;
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Float), LOCTEXT("Float_Multiplication", "Float Multiplication x"), GeneralGrouping, Filter);

		Node = NewObject<UCustomizableObjectNodeFloatArithmeticOp>();
		Node->Operation = EFloatArithmeticOperation::E_Div;
		AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Float), LOCTEXT("Float_Division", "Float Division /"), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* TransformTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTransformConstant>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeTransformParameter>(),
		};

		AddNewNodeCategoryActionsFiltered(TransformTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Transform), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* ProjectorTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeGroupProjectorParameter>(),
		};

		AddNewNodeCategoryActionsFiltered(ProjectorTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_GroupProjector), GeneralGrouping, Filter);
	}

	{
		UCustomizableObjectNode* ProjectorTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeProjectorConstant>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeProjectorParameter>(),
		};

		AddNewNodeCategoryActionsFiltered(ProjectorTemplateNodes, ContextMenuBuilder, GetPinCategoryFriendlyName(PC_Projector), GeneralGrouping, Filter);
	}
	
	{
		UCustomizableObjectNode* MaterialTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCONodeMaterialConstant>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeMaterialVariation>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeMaterialBreak>(),
			ContextMenuBuilder.CreateTemplateNode<UCONodeMaterialModify>(),
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeMaterialParameter>(),
		};

		AddNewNodeCategoryActionsFiltered(MaterialTemplateNodes, ContextMenuBuilder, NC_Material, GeneralGrouping, Filter);
	}
	
	if (EditorSettings->bEnableDeveloperOptions)
	{
		UCustomizableObjectNode* ExperimentalTemplateNodes[] =
		{
			ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeModifierClipDeform>(),
		};

		AddNewNodeCategoryActionsFiltered(ExperimentalTemplateNodes, ContextMenuBuilder, NC_Experimental, 2, Filter);
	}
	
	// Extensions
	ICustomizableObjectEditorModulePrivate& EditorModule = ICustomizableObjectEditorModulePrivate::GetChecked();

	const TMap<FName, TStrongObjectPtr<const UScriptStruct>>& ExternalPinTypes = EditorModule.GetExternalPinTypes();
	
	{
		for (const TStrongObjectPtr<const UScriptStruct>& OperationType : ICustomizableObjectModulePrivate::Get().GetRegisteredExternalOperations())
		{
			FInstancedStruct OperationInstancedStruct;
			OperationInstancedStruct.InitializeAs(OperationType.Get());

			if (const UE::Mutable::FExternalOperation* Operation = OperationInstancedStruct.GetPtr<UE::Mutable::FExternalOperation>())
			{
				// External Operation
				{
					UCONodeExternalOperation* Node = ContextMenuBuilder.CreateTemplateNode<UCONodeExternalOperation>();
					Node->OperationInstancedStruct = OperationInstancedStruct;

					AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(GetPinCategoryName(Operation->GetOutput().Value)), GeneralGrouping, Filter);
				}
			}
		}
		
		// Parameter
		for (const TTuple<FName, TStrongObjectPtr<const UScriptStruct>>& Pair : ExternalPinTypes)
		{
			if (IsTypeNative(Pair.Value.Get()))
			{
				continue;
			}
			
			UCONodeExternalTypeParameter* Node = ContextMenuBuilder.CreateTemplateNode<UCONodeExternalTypeParameter>();
			Node->DefaultValue.InitializeAs(Pair.Value.Get());

			AddNewNodeActionFiltered(Node, ContextMenuBuilder, GetPinCategoryFriendlyName(Pair.Key), GeneralGrouping, Filter);
		}
	}
	
	{
		TArray<FName> PinTypes({ PC_MeshSection, PC_Mesh, PC_Texture, PC_Projector, PC_Color, PC_Float, PC_Transform, PC_Texture_Passthrough, PC_Material, PC_Component, PC_SkeletalMesh, PC_SkeletalMesh_Passthrough });
		
		// External Types
		for (const TTuple<FName, TStrongObjectPtr<const UScriptStruct>>& Pair : ExternalPinTypes)
		{
			PinTypes.AddUnique(Pair.Key);
		}

		for (const FName& PinCategory : PinTypes)
		{
			UCONodeSwitch* NodeSwitch = ContextMenuBuilder.CreateTemplateNode<UCONodeSwitch>();
			NodeSwitch->PinType = PinCategory;
			
			const FText CategoryFriendlyName = GetPinCategoryFriendlyName(PinCategory);
			
			AddNewNodeActionFiltered(NodeSwitch, ContextMenuBuilder, CategoryFriendlyName, GeneralGrouping, Filter);
		}
		
		PinTypes.Append({  PC_Stack, PC_Modifier, PC_Projector, PC_GroupProjector, PC_PoseAsset, PC_Enum });
		
		// Extensions
		for (const FRegisteredCustomizableObjectPinType& PinType : ICustomizableObjectModule::Get().GetExtendedPinTypes())
		{
			PinTypes.AddUnique(PinType.PinType.Name);
		}
		
		for (const FName& PinCategory : PinTypes)
		{
			const FText CategoryFriendlyName = GetPinCategoryFriendlyName(PinCategory);

			UCustomizableObjectNodeExternalPin* CustomizableObjectNodeExternalPin = ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeExternalPin>();
			CustomizableObjectNodeExternalPin->PinType = PinCategory;
			
			AddNewNodeActionFiltered(CustomizableObjectNodeExternalPin, ContextMenuBuilder, CategoryFriendlyName, GeneralGrouping, Filter);
			
			UCustomizableObjectNodeExposePin* CustomizableObjectNodeExposePin = ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNodeExposePin>();
			CustomizableObjectNodeExposePin->PinType = PinCategory;
			
			AddNewNodeActionFiltered(CustomizableObjectNodeExposePin, ContextMenuBuilder, CategoryFriendlyName, GeneralGrouping, Filter);
		}
	}

	// Search for all subclasses of UCustomizableObjectNode
	//
	// Iterate over the Class Default Objects instead of their corresponding UClasses, as this allows
	// us to filter the TObjectIterator to UCustomizableObjectNode instead of UClass, which should
	// produce far fewer results to iterate through.
	for (TObjectIterator<UCustomizableObjectNode> It(RF_NoFlags); It; ++It)
	{
		const UCustomizableObjectNode* Node = *It;
		if (!IsValid(Node) || !Node->HasAllFlags(RF_ClassDefaultObject) || Node->GetClass()->HasAnyClassFlags(CLASS_Abstract))
		{
			// Only interested in non-abstract CDOs
			continue;
		}

		FText Category;
		if (Node->ShouldAddToContextMenu(Category))
		{
			UCustomizableObjectNode* TemplateNode = ContextMenuBuilder.CreateTemplateNode<UCustomizableObjectNode>(Node->GetClass());
			AddNewNodeActionFiltered(TemplateNode, ContextMenuBuilder, Category, GeneralGrouping, Filter);
		}
	}

	if (!ContextMenuBuilder.FromPin)
	{
		UEdGraphNode_Comment* Node = NewObject<UEdGraphNode_Comment>();
		TSharedPtr<FCustomizableObjectSchemaAction_NewNode> Action = AddNewNodeAction(ContextMenuBuilder, {}, Node->GetNodeTitle(ENodeTitleType::ListView), FText(), 1);
		Action->NodeTemplate = Node;
	}
	
	{
		UCustomizableObjectNodeReroute* Node = NewObject<UCustomizableObjectNodeReroute>();
		TSharedPtr<FCustomizableObjectSchemaAction_NewNode> Action = AddNewNodeAction(ContextMenuBuilder, {}, Node->GetNodeTitle(ENodeTitleType::ListView), FText(), 1);
		Action->NodeTemplate = Node;
	}

	// Add Paste here if appropriate
	if (!ContextMenuBuilder.FromPin)
	{
		const FText PasteDesc = LOCTEXT("PasteDesc", "Paste Here");
		const FText PasteToolTip = LOCTEXT("PasteToolTip", "Pastes copied items at this location.");
		TSharedPtr<FCustomizableObjectSchemaAction_Paste> PasteAction(new FCustomizableObjectSchemaAction_Paste(FText::GetEmpty(), PasteDesc, PasteToolTip.ToString(), 0));
		ContextMenuBuilder.AddAction(PasteAction);
	}
}


const FPinConnectionResponse UEdGraphSchema_CustomizableObject::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are on the same node"));
	}

	// Check both pins support connections
	if(PinA->bNotConnectable || PinB->bNotConnectable)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Pin doesn't support connections"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = NULL;
	const UEdGraphPin* OutputPin = NULL;

	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Directions are not compatible"));
	}

	// Type categories must match and the nodes need to be compatible with each other
	bool bArePinsCompatible = false;
	bool bIsOtherNodeBlocklisted = false;

	UCustomizableObjectNode* InputNode = CastChecked<UCustomizableObjectNode>(InputPin->GetOwningNode());
	if (!InputNode->CanConnect(InputPin, OutputPin, bIsOtherNodeBlocklisted,bArePinsCompatible))
	{
		if (!bArePinsCompatible)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Types are not compatible"));
		}
		else if (bIsOtherNodeBlocklisted)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Direct connections between these nodes are not allowed"));
		}
	}

	// Some special nodes can only have one output
	bool bBreakExistingDueToDataOutput = false;
	if (UCustomizableObjectNode* n = Cast<UCustomizableObjectNode>(OutputPin->GetOwningNode()))
	{
		bBreakExistingDueToDataOutput = (OutputPin->LinkedTo.Num() > 0) && n->ShouldBreakExistingConnections(InputPin, OutputPin);
	}

	// See if we want to break existing connections (if its an input with an existing connection)
	const bool bBreakExistingDueToDataInput = (InputPin->LinkedTo.Num() > 0) && !InputPin->PinType.IsArray();

	if (bBreakExistingDueToDataOutput&&bBreakExistingDueToDataInput)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_BREAK_OTHERS_AB, TEXT("Replace connections at both ends"));
	}
	
	if (bBreakExistingDueToDataInput)
	{
		const ECanCreateConnectionResponse ReplyBreakInputs = (PinA == InputPin) ? CONNECT_RESPONSE_BREAK_OTHERS_A : CONNECT_RESPONSE_BREAK_OTHERS_B;
		return FPinConnectionResponse(ReplyBreakInputs, TEXT("Replace existing input connections"));
	}

	if (bBreakExistingDueToDataOutput)
	{
		const ECanCreateConnectionResponse ReplyBreakOutputs = (PinA == OutputPin) ? CONNECT_RESPONSE_BREAK_OTHERS_A : CONNECT_RESPONSE_BREAK_OTHERS_B;
		return FPinConnectionResponse(ReplyBreakOutputs, TEXT("Replace existing output connections"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}


FLinearColor UEdGraphSchema_CustomizableObject::GetPinTypeColor(const FName& Type)
{
	static TMap<FName, FLinearColor> PinCategoryColors;

	// Cache colors
	if (PinCategoryColors.IsEmpty())
	{
		UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get();
		
		PinCategoryColors.Add(PC_Enum, FLinearColor(0.004f, 0.42f, 0.384f, 1.000000f));
		PinCategoryColors.Add(PC_Float, FLinearColor(0.357667f, 1.000000f, 0.060000f, 1.000000f));
		PinCategoryColors.Add(PC_Color, FLinearColor(1.000000f, 0.591255f, 0.016512f, 1.000000f));
		PinCategoryColors.Add(PC_Bool, FLinearColor(0.470000f, 0.0f, 0.000000f, 1.000000f));
		PinCategoryColors.Add(PC_Projector, FLinearColor(FColorList::Aquamarine));
		PinCategoryColors.Add(PC_GroupProjector, FLinearColor(FColorList::DarkTan));
		PinCategoryColors.Add(PC_Mesh, FLinearColor(FColorList::MediumOrchid)); 
		PinCategoryColors.Add(PC_MeshSection, FLinearColor(0.000000f, 0.100000f, 0.600000f, 1.000000f));
		PinCategoryColors.Add(PC_Modifier, FLinearColor(FColorList::LightGrey)); 
		PinCategoryColors.Add(PC_Object, FLinearColor(0.000000f, 0.400000f, 0.910000f, 1.000000f));
		PinCategoryColors.Add(PC_Component, FLinearColor(FColorList::DarkOrchid)); 
		PinCategoryColors.Add(PC_Stack, FLinearColor(1.000000f, 0.000000f, 0.800000f, 1.000000f));
		PinCategoryColors.Add(PC_Wildcard, FLinearColor(1.000000f, 1.000000f, 1.000000f, 1.000000f));
		PinCategoryColors.Add(PC_Transform, FLinearColor(FColorList::Orange)); 
		PinCategoryColors.Add(PC_String, FLinearColor(0.700000f, 0.010000f, 0.660000f, 1.000000f));

		{
			const FLinearColor Color = AssetDefinitionRegistry->GetAssetDefinitionForClass(UMaterialInterface::StaticClass())->GetAssetColor();
			PinCategoryColors.Add(PC_Material, Color);
		}
		
		{
			const FLinearColor Color = AssetDefinitionRegistry->GetAssetDefinitionForClass(USkeletalMesh::StaticClass())->GetAssetColor();
			PinCategoryColors.Add(PC_SkeletalMesh, Color);
			PinCategoryColors.Add(PC_SkeletalMesh_Passthrough, Color); 	
			PinCategoryColors.Add(PC_SkeletalMesh_Object, Color);
		}
		
		{
			const FLinearColor Color = AssetDefinitionRegistry->GetAssetDefinitionForClass(UStaticMesh::StaticClass())->GetAssetColor();
			PinCategoryColors.Add(PC_StaticMesh, Color);
		}

		{
			const FLinearColor Color = AssetDefinitionRegistry->GetAssetDefinitionForClass(UTexture::StaticClass())->GetAssetColor();
			PinCategoryColors.Add(PC_Texture, Color);
			PinCategoryColors.Add(PC_Texture_Passthrough, Color);	
			PinCategoryColors.Add(PC_Texture_Object, Color);	
		}

		{
			const FLinearColor Color = AssetDefinitionRegistry->GetAssetDefinitionForClass(UPoseAsset::StaticClass())->GetAssetColor();
			PinCategoryColors.Add(PC_PoseAsset, Color);
		}
	}
		
	if (const FLinearColor* Color = PinCategoryColors.Find(Type))
	{
		return *Color;		
	}

	for (const FRegisteredCustomizableObjectPinType& PinType : ICustomizableObjectModule::Get().GetExtendedPinTypes())
	{
		if (PinType.PinType.Name == Type)
		{
			return PinType.PinType.Color;
		}
	}
	
	const TMap<FName, FLinearColor>& ExternalPinColors = ICustomizableObjectEditorModulePrivate::Get()->GetExternalPinTypeColors();
	if (const FLinearColor* Result = ExternalPinColors.Find(Type))
	{
		return *Result;
	}

	return FLinearColor::Black;
}


FLinearColor UEdGraphSchema_CustomizableObject::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	const FName TypeName = PinType.PinCategory;

	return GetPinTypeColor(TypeName);
}


bool UEdGraphSchema_CustomizableObject::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	check(Pin != NULL);

	if (Pin->bDefaultValueIsIgnored)
	{
		return true;
	}

	return false;
}


void GetContextMenuActionsReconstructAllChildNodes(UToolMenu* Menu, TWeakObjectPtr<UGraphNodeContextMenuContext> WeakContext)
{
	FToolMenuSection& SubSection = Menu->AddSection("Section");

	TArray<UClass*> NodeTypes;

	for (TObjectIterator<UCustomizableObjectNode> It(RF_NoFlags); It; ++It)
	{
		const UCustomizableObjectNode* Node = *It;
		if (!IsValid(Node) || !Node->HasAllFlags(RF_ClassDefaultObject))
		{
			continue; // Only interested in CDOs
		}

		UClass* Class = Node->GetClass();
		if (Class->HasAnyClassFlags(CLASS_Abstract))
		{
			continue;
		}
		
		NodeTypes.Add(Class);
	}
	
	NodeTypes.Sort([](UClass& A, UClass& B) -> bool
	{
		const UCustomizableObjectNode* NodeA = CastChecked<UCustomizableObjectNode>(A.GetDefaultObject());
		const UCustomizableObjectNode* NodeB = CastChecked<UCustomizableObjectNode>(B.GetDefaultObject());
		
		return NodeA->GetNodeTitle(ENodeTitleType::ListView).CompareTo(NodeB->GetNodeTitle(ENodeTitleType::ListView)) < 0;
	});
	
	for (UClass* NodeType : NodeTypes)
	{
		const UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(NodeType->GetDefaultObject());

		auto Call = [](TWeakObjectPtr<UGraphNodeContextMenuContext> WeakContext, const TWeakObjectPtr<UClass>& WeakNodeType)
		{
			const UGraphNodeContextMenuContext* Context = WeakContext.Get();
			if (!Context)
			{
				return;
			}

			UCustomizableObjectNode* Node = const_cast<UCustomizableObjectNode*>(Cast<UCustomizableObjectNode>(Context->Node));
			if (!Node)
			{
				return;
			}

			const UClass* NodeType = WeakNodeType.Get();
			if (!NodeType)
			{
				return;
			}

			if (TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = GetCustomizableObjectEditor(Context->Graph))
			{
				Editor->ReconstructAllChildNodes(*Node, *NodeType);
			};
		};

		SubSection.AddMenuEntry(
			Node->GetFName(),
			Node->GetNodeTitle(ENodeTitleType::ListView),
			FText(),
			FSlateIcon(),
			FExecuteAction::CreateLambda(Call, WeakContext, MakeWeakObjectPtr(NodeType)));
	}
}


void UEdGraphSchema_CustomizableObject::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context || !Context->Node)
	{
		return;
	}
	
	if (!Context->Pin) // On Node right click
	{
		if (!Context->bIsDebugging)
		{
			// Node contextual actions
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
			Section.AddMenuEntry(FGenericCommands::Get().Delete);
			Section.AddMenuEntry(FGenericCommands::Get().Cut);
			Section.AddMenuEntry(FGenericCommands::Get().Copy);
			Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
			Section.AddMenuEntry(FGraphEditorCommands::Get().ReconstructNodes);
			Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);

			Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* AlignmentMenu)
			{
				{
					FToolMenuSection& SubSection = AlignmentMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
				}

				{
					FToolMenuSection& SubSection = AlignmentMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
					SubSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
				}
			}));
			
			Section.AddSubMenu(
				"ReconstructAllNodes",
				LOCTEXT("ReconstructChildAllNodes", "Refresh All Child Nodes"),
				LOCTEXT("ReconstructAllChildNodes_Tooltip", "Refresh all child nodes from the selected ones (inclusive)."),
				FNewToolMenuDelegate::CreateStatic(&GetContextMenuActionsReconstructAllChildNodes, MakeWeakObjectPtr(Context)));
		}

		struct SCommentUtility
		{
			static void CreateComment(const UEdGraphSchema_CustomizableObject* Schema, UEdGraph* Graph)
			{
				if (Schema && Graph)
				{
					if (TSharedPtr<FCustomizableObjectGraphEditorToolkit> CustomizableObjectEditor = GetCustomizableObjectEditor(Graph))
					{
						CustomizableObjectEditor->CreateCommentBox(FVector2D::ZeroVector);
					}
				}
			}
		};

		FToolMenuSection& section = Menu->AddSection("SchemaActionComment", LOCTEXT("MultiCommentHeader", "Comment Group"));
		section.AddMenuEntry("MultiCommentDesc", LOCTEXT("MultiCommentDesc", "Create Comment from Selection"),
			LOCTEXT("CommentToolTip", "Create a resizable comment box around selection."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateStatic(SCommentUtility::CreateComment, this, const_cast<UEdGraph*>(ToRawPtr(Context->Graph)))));
	}
	else // On Pin right click
	{
		if (UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(const_cast<UEdGraphNode*>(Context->Node.Get())))
		{
			if (const UEdGraphPin* Pin = Context->Pin)
			{
				if (Node->CanPinBeHidden(*Pin))
				{
					FToolMenuSection& Section = Menu->FindOrAddSection("EdGraphSchemaPinActions");
					Section.AddMenuEntry("HidePin",
						LOCTEXT("HidePin_Label", "Hide Pin"),
						LOCTEXT("HidePin_Tooltip", "Hides the selected pin."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Node = const_cast<UCustomizableObjectNode*>(Node), Pin]()
						{
							if (!Node || !Pin)
							{
								return;
							}
								
							Node->SetPinHidden(*const_cast<UEdGraphPin*>(Pin), true);
						})));
				}

				if (Node->CanPinBeRemoved(*Pin))
				{
					FToolMenuSection& Section = Menu->FindOrAddSection("EdGraphSchemaPinActions");
					Section.AddMenuEntry("RemovePin",
						LOCTEXT("RemovePin_Label", "Remove Pin"),
						LOCTEXT("RemovePin_Tooltip", "Removes the selected pin."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Node = const_cast<UCustomizableObjectNode*>(Node), Pin]()
						{
							if (!Node || !Pin)
							{
								return;
							}
								
							FScopedTransaction Transaction(LOCTEXT("TryRemovePinTransaction", "Try removing a Pin"));
							Node->Modify();

							// Make sure the pin that is requested for removal was actually added by the UI.
							if (Node->CanPinBeRemoved(*Pin) && Node->CustomRemovePin(*const_cast<UEdGraphPin*>(Pin)))
							{
								// Notify that a pin has been removed so the PinViewer reacts to the new pin layout
								if (UEdGraph* Graph = Node->GetGraph())
								{
									Graph->NotifyNodeChanged(Node);
								}
							}

						})));
				}
				
				const TArray<FName> AllowedTypes = Node->GetPinAllowedTypes(*Pin);
				if (AllowedTypes.Num() > 1)
				{
					FEdGraphPinReference PinReference = FEdGraphPinReference(const_cast<UEdGraphPin*>(Pin));
					TSharedRef<SMutablePinTypeSelector> TypeSelectionSlate = SNew(SMutablePinTypeSelector).PinReference(PinReference);
					
					// In order to not access to invalid node data close the widget containing the SMutablePinTypeSelector
					Node->PostReconstructNodeDelegate.AddLambda([WeakMenu = TypeSelectionSlate->AsWeak()](){
						if (TSharedPtr<SWidget> Menu = WeakMenu.Pin())
						{
							FSlateApplication::Get().DismissMenuByWidget(Menu.ToSharedRef());
						}
					});
					
					FToolMenuSection& Section = Menu->FindOrAddSection("EdGraphSchemaPinActions");
					Section.AddEntry(FToolMenuEntry::InitWidget("Pin Type Selector", TypeSelectionSlate,  LOCTEXT("SetPinType_Label", "Pin Type"), true));
				}
				
				const TArray<FName> AllowedSubTypes = Node->GetPinAllowedSubTypes(*Pin);
				if (AllowedSubTypes.Num() > 1)
				{
					FEdGraphPinReference PinReference = FEdGraphPinReference(const_cast<UEdGraphPin*>(Pin));
					TSharedRef<SMutablePinSubTypeSelector> SubTypeSelectionSlate = SNew(SMutablePinSubTypeSelector).PinReference(PinReference);
					
					// In order to not access to invalid node data close the widget containing the SubTypeSelectionSlate
					Node->PostReconstructNodeDelegate.AddLambda([WeakMenu = SubTypeSelectionSlate->AsWeak()](){
						if (TSharedPtr<SWidget> Menu = WeakMenu.Pin())
						{
							FSlateApplication::Get().DismissMenuByWidget(Menu.ToSharedRef());
						}
					});
					
					FToolMenuSection& Section = Menu->FindOrAddSection("EdGraphSchemaPinActions");
					Section.AddEntry(FToolMenuEntry::InitWidget("Pin Subtype Selector", SubTypeSelectionSlate,  LOCTEXT("SetPinSubtype_Label", "Pin Subtype"), true));
				}
				
				if (TSharedPtr<IDetailsView> Widget = Node->CustomizePinDetails(*Pin))
				{
					// In order to not access to invalid node data close the widget containing the custom widget
					Node->PostReconstructNodeDelegate.AddLambda([WeakMenu = Widget->AsWeak()](){
						if (TSharedPtr<SWidget> Menu = WeakMenu.Pin())
						{
							FSlateApplication::Get().DismissMenuByWidget(Menu.ToSharedRef());
						}
					});
					
					FToolMenuSection& Section = Menu->FindOrAddSection("EdGraphSchemaPinActions");
					Section.AddSeparator("Pin Viewer");
					Section.AddEntry(FToolMenuEntry::InitWidget("Pin Viewer", Widget.ToSharedRef(), {}));
				}
			}
		}
	}
}


void UEdGraphSchema_CustomizableObject::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
#if WITH_EDITOR
	TArray<UEdGraphPin*> Pins;

	for (UEdGraphPin* Pin : TargetNode.Pins)
	{
		Pins.Append(ReverseFollowPinArray(*Pin, false));
	}
#endif
	
	Super::BreakNodeLinks(TargetNode);

#if WITH_EDITOR
	NodePinConnectionListChanged(Pins);
#endif
}


void UEdGraphSchema_CustomizableObject::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links"));

	TArray<UEdGraphPin*> Pins;
	Pins.Append(FollowPinArray(TargetPin, false));
	Pins.Append(ReverseFollowPinArray(TargetPin, false));
	
	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);

#if WITH_EDITOR
	NodePinConnectionListChanged(Pins);
#endif
}


void UEdGraphSchema_CustomizableObject::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link"));

	TArray<UEdGraphPin*> SourceConnectedPins = ReverseFollowPinArray(*SourcePin, false);
	TArray<UEdGraphPin*> TargetConnectedPins = ReverseFollowPinArray(*TargetPin, false);
	
	Super::BreakSinglePinLink(SourcePin, TargetPin);

#if WITH_EDITOR
	NodePinConnectionListChanged(SourceConnectedPins);
	NodePinConnectionListChanged(TargetConnectedPins);
#endif
}


/** Enum containing all the object types that we are able to convert onto a node when dragging
	 * and dropping and asset of that type onto the CO graph.
	 * Each value of this enumeration will, in practice, have a CO node to be represented by.
	 */
// TODO: Replace the usage of this enum class with something similar to typeID (not casting)
enum class ESpawnableObjectType : int32
{
	None = -1,		// Invalid value
		
	UTexture2D,
	USkeletalMesh,
	UStaticMesh,
	UMaterialInterface,
};


bool IsSpawnableAsset(const FAssetData& InAsset, ESpawnableObjectType& OutObjectType)
{
	UObject* Object = UE::Mutable::Private::LoadObject(InAsset);

	// Type used to know what kind of UObject this asset is
	OutObjectType = ESpawnableObjectType::None;

	// Check if the provided object can be casted to any of the UObject types we can spawn as CO Nodes
	if (Cast<UTexture2D>(Object))
	{
		OutObjectType = ESpawnableObjectType::UTexture2D;
		return true;
	}
	else if (Cast<USkeletalMesh>(Object))
	{
		OutObjectType = ESpawnableObjectType::USkeletalMesh;
		return true;
	}
	else if (Cast<UStaticMesh>(Object))
	{
		OutObjectType = ESpawnableObjectType::UStaticMesh;
		return true;
	}
	else if (Cast<UMaterialInterface>(Object))
	{
		OutObjectType = ESpawnableObjectType::UMaterialInterface;
		return true;
	}
	// Add more compatible types here, sync it up with ESpawnableObjectType
	else
	{
		// Non spawnable object
		return false;
	}
}


void UEdGraphSchema_CustomizableObject::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	// To prevent overlapping when multiple assets are dropped at the same time on the graph
	const int PixelOffset = 20;
	int CurrentOffset = 0;
	
	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
	for (const FAssetData& Asset : Assets)
	{
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(Asset))
		{
			continue;
		}

		// If it is not a valid asset to be spawned then just skip it
		ESpawnableObjectType ObjectType;
		if (!IsSpawnableAsset(Asset,ObjectType))
		{
			continue;
		}

		// At this point we know we are working with an asset we can spawn as a mutable node.
		
		UObject* Object = UE::Mutable::Private::LoadObject(Asset);
		UEdGraphNode* GraphNode = nullptr;

		// Depending on the UObjectType spawn one or another mutable node
		switch (ObjectType)
		{
		case ESpawnableObjectType::UTexture2D:
			{
				UTexture2D* Texture = Cast<UTexture2D>(Object);
				UCustomizableObjectNodeTexture* Node = NewObject<UCustomizableObjectNodeTexture>(Graph);
				Node->Texture = Texture;
				GraphNode = Node;
				break;
			}
		case ESpawnableObjectType::USkeletalMesh: 
			{
				USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object);
				UCustomizableObjectNodeSkeletalMesh* Node = NewObject<UCustomizableObjectNodeSkeletalMesh>(Graph);
				Node->SkeletalMesh = SkeletalMesh;
				GraphNode = Node;
				break;
			}
		case ESpawnableObjectType::UStaticMesh:
			{
				UStaticMesh* Mesh = Cast<UStaticMesh>(Object);
				UCustomizableObjectNodeStaticMesh* Node = NewObject<UCustomizableObjectNodeStaticMesh>(Graph);
				Node->StaticMesh = Mesh;
				GraphNode = Node;
				break;
			}
		case ESpawnableObjectType::UMaterialInterface:
			{
				UMaterialInterface* Material = Cast<UMaterialInterface>(Object);
				UCONodeSkeletalMeshSection* Node = NewObject<UCONodeSkeletalMeshSection>(Graph);
				Node->SetMaterial(Material);
				GraphNode = Node;
				break;
			}
		// Error : A new compatible type set on UEdGraphSchema_CustomizableObject::IsSpawnableAsset is not providing a valid ESpawnableObjectType value
		case ESpawnableObjectType::None:
		// Error : a switch entry is missing for a ESpawnableObjectType value
		default:
			{
				UE_LOGF(LogTemp,Error, "Unable to create new mutable node for target asset : Invalid ESpawnableObjectType value. ");
				checkNoEntry();
				break;
			}
		}

		// A node must have been spawned at this point.
		if (GraphNode)
		{
			// A new node has been instanced, add it to the graph
			GraphNode->CreateNewGuid();
			GraphNode->PostPlacedNewNode();
			GraphNode->AllocateDefaultPins();
			GraphNode->NodePosX = GraphPosition.X + CurrentOffset;
			GraphNode->NodePosY = GraphPosition.Y + CurrentOffset;
			Graph->AddNode(GraphNode, true);
			CurrentOffset += PixelOffset;	
		}
		else
		{
			UE_LOGF(LogTemp,Error, "Unable to add null node to graph. ")
		}
	}
}


void UEdGraphSchema_CustomizableObject::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(LOCTEXT("CreateRerouteNodeOnWire", "Create Reroute Node"));

	UEdGraph* ParentGraph = PinA->GetOwningNode()->GetGraph();
	ParentGraph->Modify();

	// This constant is duplicated from inside of SGraphNodeKnot
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	// Create a new knot
	UCustomizableObjectNodeReroute* DefaultNodeReroute = CastChecked<UCustomizableObjectNodeReroute>(UCustomizableObjectNodeReroute::StaticClass()->GetDefaultObject());
	UCustomizableObjectNodeReroute* NodeReroute = CastChecked<UCustomizableObjectNodeReroute>(FCustomizableObjectSchemaAction_NewNode::CreateNode(ParentGraph, nullptr, FDeprecateSlateVector2D(KnotTopLeft), DefaultNodeReroute));

	PinA->BreakLinkTo(PinB);
	PinA->MakeLinkTo((PinA->Direction == EGPD_Output) ? NodeReroute->GetInputPin() : NodeReroute->GetOutputPin());
	PinB->MakeLinkTo((PinB->Direction == EGPD_Output) ? NodeReroute->GetInputPin() : NodeReroute->GetOutputPin());
	NodeReroute->UCustomizableObjectNode::ReconstructNode();
}


FConnectionDrawingPolicy* UEdGraphSchema_CustomizableObject::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj) const
{
	return new FCustomizableObjectConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}


void UEdGraphSchema_CustomizableObject::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets,
                                                                   const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
	uint32 AmountOfIncompatibleAssets = 0;

	// Iterate over the assets
	for (const FAssetData& Asset : Assets)
	{
		// On first fail abort the consequent checks and tell the user
		ESpawnableObjectType ObjectType;
		if (IsSpawnableAsset(Asset, ObjectType))
		{
			if (AssetReferenceFilter)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(Asset, &FailureReason))
				{
					if (OutTooltipText.IsEmpty())
					{
						OutTooltipText = FailureReason.ToString();
					}
					continue;
				}
			}

			OutTooltipText.Reset();
			OutOkIcon = true;
			break;
		}
		else
		{
			AmountOfIncompatibleAssets++;

			// Stop checking once we know that more than one asset is not compatible, the UI output will be the same
			if (AmountOfIncompatibleAssets > 1)
			{
				break;
			}
		}
	}

	// Output debug message depending on the quantity of incompatible objects
	if (!OutOkIcon && OutTooltipText.IsEmpty())
	{
		if (Assets.Num() == 1)
		{
			OutTooltipText = LOCTEXT("IncompatibleAsset", "Incompatible asset selected : No node can be created for this type of asset.").ToString();
		}
		else if (Assets.Num() > 1 )
		{
			if (Assets.Num() == AmountOfIncompatibleAssets)
			{
				OutTooltipText = LOCTEXT("AllIncompatibleAssets", "Incompatible assets selected : No node can be created for any of the selected assets.").ToString();
			}
			else 
			{
				OutTooltipText = LOCTEXT("SomeIncompatibleAssets", "Incompatible asset selected : Some assets will not be placed as nodes on the graph.").ToString();
			}
		}
	}
}


bool UEdGraphSchema_CustomizableObject::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	if (!PinA || !PinB)
	{
		return false;
	}
	
	UEdGraphNode* PinAOwningNode = PinA->GetOwningNode(); // TryCreateConnection can reconstruct the node invalidating the FromPin. Get the OwningNode before.
	UEdGraphNode* PinBOwningNode = PinB->GetOwningNode();

	bool bResult = Super::TryCreateConnection(PinA, PinB);

	if (bResult)
	{
		PinAOwningNode->NodeConnectionListChanged();
		PinBOwningNode->NodeConnectionListChanged();
	}
	
	if (!PinA || PinA->bWasTrashed || !PinB || PinB->bWasTrashed)
	{
		return bResult;
	}
	
	UEdGraphPin* InputPin;
	UEdGraphPin* OutputPin;
	if (!CategorizePinsByDirection(PinA, PinB, InputPin, OutputPin))
	{
		return bResult;
	}
	
#if WITH_EDITOR
	if (bResult)
	{
		NodePinConnectionListChanged(ReverseFollowPinArray(*PinA, false));
		NodePinConnectionListChanged(ReverseFollowPinArray(*PinB, false));
	}
#endif

	return bResult;
}


FPinConnectionResponse UEdGraphSchema_CustomizableObject::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
    // Mutable graph and its super does not use it. If we ever want to use it we should call NotifyIndirectConnections.
	unimplemented();
	return {};
}


FName UEdGraphSchema_CustomizableObject::GetPinCategoryName(const FName& PinCategory)
{
	if (PinCategory == PC_Object)
	{
		return TEXT("Object");
	}
	else if (PinCategory == PC_Component)
	{
		return TEXT("Component");
	}
	else if (PinCategory == PC_MeshSection)
	{
		return TEXT("Material");
	}
	else if (PinCategory == PC_Modifier)
	{
		return TEXT("Modifier");
	}
	else if (PinCategory == PC_Mesh)
	{
		return TEXT("Mesh");
	}
	else if (PinCategory == PC_SkeletalMesh)
	{
		return TEXT("Skeletal Mesh");
	}
	else if (PinCategory == PC_StaticMesh)
	{
		return TEXT("Static Mesh");
	}
	else if (PinCategory == PC_SkeletalMesh_Passthrough)
	{
		return TEXT("Passthrough Skeletal Mesh");
	}
	else if (PinCategory == PC_SkeletalMesh_Object)
	{
		return TEXT("Passthrough Skeletal Mesh");
	}
	else if (PinCategory == PC_Texture)
	{
		return TEXT("Texture");
	}
	else if (PinCategory == PC_Texture_Passthrough)
	{
		return TEXT("Passthrough Texture");
	}
	else if (PinCategory == PC_Texture_Object)
	{
		return TEXT("Passthrough Texture");
	}
	else if (PinCategory == PC_Projector)
	{
		return TEXT("Projector");
	}
	else if (PinCategory == PC_GroupProjector)
	{
		return TEXT("Group Projector");
	}
	else if (PinCategory == PC_Color)
	{
		return TEXT("Color");
	}
	else if (PinCategory == PC_Float)
	{
		return TEXT("Float");
	}
	else if (PinCategory == PC_Bool)
	{
		return TEXT("Bool");
	}
	else if (PinCategory == PC_Enum)
	{
		return TEXT("Enum");
	}
	else if (PinCategory == PC_Stack)
	{
		return TEXT("Morph Stack");
	}
	else if (PinCategory == PC_Material)
	{
		return TEXT("Material");
	}
	else if (PinCategory == PC_Wildcard)
	{
		return TEXT("Wildcard");
	}
	else if (PinCategory == PC_PoseAsset)
	{
		return TEXT("PoseAsset");
	}
	else if (PinCategory == PC_Transform)
	{
		return TEXT("Transform");
	}
	else if (PinCategory == PC_String)
	{
		return TEXT("String");
	}
	else
	{
		for (const FRegisteredCustomizableObjectPinType& PinType : ICustomizableObjectModule::Get().GetExtendedPinTypes())
		{
			if (PinType.PinType.Name == PinCategory)
			{
				return PinType.PinType.Name;
			}
		}

		return PinCategory;
	}
}


FName UEdGraphSchema_CustomizableObject::GetPinCategoryName(const UScriptStruct* Type)
{
	if (Type == UE::Mutable::FMeshAdapter::StaticStruct())
	{
		return PC_Mesh;
	}
	else if (Type == UE::Mutable::FTextureAdapter::StaticStruct())
	{
		return PC_Texture;
	}
	else if (Type == UE::Mutable::FFloatAdapter::StaticStruct())
	{
		return PC_Float;
	}
	else if (Type == UE::Mutable::FVectorAdapter::StaticStruct())
	{
		return PC_Color;
	}
	else if (Type == UE::Mutable::FMaterialAdapter::StaticStruct())
	{
		return PC_Material;
	}
	else
	{
		return Type->GetFName();
	}
}

FName UEdGraphSchema_CustomizableObject::GetPinSubCategoryName(const FName& PinSubCategory)
{
	if (PinSubCategory == PSC_Material_Overlay)
	{
		return TEXT("Overlay");
	}
	else if (PinSubCategory == PSC_Material_Override)
	{
		return TEXT("Override");
	}
	
	return {};
}


FText UEdGraphSchema_CustomizableObject::GetPinFriendlyName(const FName& PinCategory)
{
	if (PinCategory == PC_SkeletalMesh_Passthrough)
	{
		return LOCTEXT("Skeletal_Mesh_Pin_FriendlyName", "Skeletal Mesh");
	}
	else if (PinCategory == PC_SkeletalMesh_Object)
	{
		return LOCTEXT("Passthrough_Skeletal_Mesh_Pin_FriendlyName", "Skeletal Mesh");
	}
	else if (PinCategory == PC_Texture_Passthrough)
	{
		return LOCTEXT("Texture_Pin_FriendlyName", "Texture");
	}
	else if (PinCategory == PC_Texture_Object)
	{
		return LOCTEXT("Passthrough_Texture_Pin_FriendlyName", "Texture");
	}
	else
	{
		return GetPinCategoryFriendlyName(PinCategory);
	}
}


FText UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(const FName& PinCategory)
{
	if (PinCategory == PC_Object)
	{
		return LOCTEXT("Object_Pin_Category", "Object");
	}
	else if (PinCategory == PC_Component)
	{
		return LOCTEXT("Component_Pin_Category", "Component");
	}
	else if (PinCategory == PC_MeshSection)
	{
		return LOCTEXT("MeshSection_Pin_Category", "Mesh Section");
	}
	else if (PinCategory == PC_Modifier)
	{
		return LOCTEXT("Modifier_Pin_Category", "Modifier");
	}
	else if (PinCategory == PC_Mesh)
	{
		return LOCTEXT("Mesh_Pin_Category", "Mesh");
	}
	else if (PinCategory == PC_SkeletalMesh)
	{
		return LOCTEXT("Skeletal_Mesh_Pin_Category", "Skeletal Mesh");
	}
	else if (PinCategory == PC_StaticMesh)
	{
		return LOCTEXT("Static_Mesh_Pin_Category", "Static Mesh");
	}
	else if (PinCategory == PC_SkeletalMesh_Passthrough)
	{
		return LOCTEXT("Passthrough_Skeletal_Mesh_Pin_Category", "Skeletal Mesh Object");
	}
	else if (PinCategory == PC_SkeletalMesh_Object)
	{
		return LOCTEXT("Passthrough_Parameter_Skeletal_Mesh_Pin_Category", "Skeletal Mesh Object");
	}
	else if (PinCategory == PC_Texture)
	{
		return LOCTEXT("Image_Pin_Category", "Texture");
	}
	else if (PinCategory == PC_Texture_Passthrough)
	{
		return LOCTEXT("Passthrough_Texture_Pin_Category", "Texture Object");
	}
	else if (PinCategory == PC_Texture_Object)
	{
		return LOCTEXT("Passthrough_Texture_Pin_Category", "Texture Object");
	}
	else if (PinCategory == PC_Projector)
	{
		return LOCTEXT("Projector_Pin_Category", "Projector");
	}
	else if (PinCategory == PC_GroupProjector)
	{
		return LOCTEXT("Group_Projector_Pin_Category", "Group Projector");
	}
	else if (PinCategory == PC_Color)
	{
		return LOCTEXT("Color_Pin_Category", "Color");
	}
	else if (PinCategory == PC_Float)
	{
		return LOCTEXT("Float_Pin_Category", "Float");
	}
	else if (PinCategory == PC_Bool)
	{
		return LOCTEXT("Bool_Pin_Category", "Bool");
	}
	else if (PinCategory == PC_Enum)
	{
		return LOCTEXT("Enum_Pin_Category", "Enum");
	}
	else if (PinCategory == PC_Stack)
	{
		return LOCTEXT("Stack_Pin_Category", "Morph Stack");
	}
	else if (PinCategory == PC_Material)
	{
		return LOCTEXT("Material_Asset_Pin_Category", "Material");
	}
	else if (PinCategory == PC_Wildcard)
	{
		return LOCTEXT("Wildcard_Pin_Category", "Wildcard");
	}
	else if (PinCategory == PC_PoseAsset)
	{
		return LOCTEXT("Pose_Pin_Category", "Pose Asset");
	}
	else if (PinCategory == PC_Transform)
	{
		return LOCTEXT("Transform_Pin_Category", "Transform");
	}
	else if (PinCategory == PC_String)
	{
		return LOCTEXT("String_Pin_Category", "String");
	}
	
	for (const FRegisteredCustomizableObjectPinType& PinType : ICustomizableObjectModule::Get().GetExtendedPinTypes())
	{
		if (PinType.PinType.Name == PinCategory)
		{
			return PinType.PinType.DisplayName;
		}
	}
	
	const TMap<FName, FText>& ExternalPinFriendlyName = ICustomizableObjectEditorModulePrivate::Get()->GetExternalPinFriendlyName();
	if (const FText* Result = ExternalPinFriendlyName.Find(PinCategory))
	{
		return *Result;
	}
	
	return {};
}


FText UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(const FName& PinSubcategory)
{
	if (PinSubcategory == PSC_Material_Overlay)
	{
		return LOCTEXT("Material_Overlay_Pin_Subcategory", "Overlay");
	}
	else if (PinSubcategory == PSC_Material_Override)
	{
		return LOCTEXT("Material_Override_Pin_Subcategory", "Override");
	}
	
	return {};
}


bool UEdGraphSchema_CustomizableObject::IsPassthrough(const FName& PinCategory)
{
	return PinCategory == PC_Texture_Passthrough || 
		PinCategory == PC_Texture_Object ||
		PinCategory == PC_SkeletalMesh_Passthrough ||
		PinCategory == PC_SkeletalMesh_Object;
}

TSharedPtr<IAssetReferenceFilter> UEdGraphSchema_CustomizableObject::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (Graph)
	{
		if (const UCustomizableObject* CustomizableObject = Cast<const UCustomizableObject>(Graph->GetOuter()))
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(CustomizableObject);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}
	}

	return {};
}

bool UEdGraphSchema_CustomizableObject::IsTypeNative(const UScriptStruct* Type)
{
	return Type == UE::Mutable::FMeshAdapter::StaticStruct() ||
		Type == UE::Mutable::FTextureAdapter::StaticStruct() ||
		Type == UE::Mutable::FFloatAdapter::StaticStruct() ||
		Type == UE::Mutable::FVectorAdapter::StaticStruct() ||
		Type == UE::Mutable::FMaterialAdapter::StaticStruct();
}


FText UEdGraphSchema_CustomizableObject::GetSubCategoryTooltipText(FName PinSubType)
{
	if (PinSubType == PSC_Material_Override)
	{
		return LOCTEXT("PSC_Material_Override_Tooltip", "Replaces one material with another one.");
	}
	else if (PinSubType == PSC_Material_Overlay)
	{
		return LOCTEXT("PSC_Material_Overlay_Tooltip", "Renders a secondary material on top of the already set one.");
	}

	return {};
}


#undef LOCTEXT_NAMESPACE
