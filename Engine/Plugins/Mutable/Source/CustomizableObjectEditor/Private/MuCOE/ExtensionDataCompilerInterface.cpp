// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/ExtensionDataCompilerInterface.h"

#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuR/ExtensionData.h"

FExtensionDataCompilerInterface::FExtensionDataCompilerInterface(FMutableGraphGenerationContext& InGenerationContext)
	: GenerationContext(InGenerationContext)
{
}


void FExtensionDataCompilerInterface::AddGeneratedNode(const UCustomizableObjectNode* InNode)
{
	check(InNode);

	// A const_cast here is required because the new node needs to be added in the GeneratedNodes list so mutable can
	// discover new parameters that can potentially be attached to the extension node, however, this
	// function is called as ICustomizableObjectExtensionNode::GenerateMutableNode(this), so we need to cast the const away here.
	// Decided to do the case here so the use of AddGeneratedNode is as clean as possible
	GenerationContext.GeneratedNodes.Add(const_cast<UCustomizableObjectNode*>(InNode));
}


UE::Mutable::Private::PASSTHROUGH_ID FExtensionDataCompilerInterface::MakeExtensionData(UObject& ExtensionData, bool bDuplicate)
{
	return GenerationContext.CompilationContext->PassthroughObjectFactory.Add(ExtensionData, bDuplicate);
}


void FExtensionDataCompilerInterface::CompilerLog(const FText& InLogText, const UCustomizableObjectNode* InNode)
{
	GenerationContext.Log(InLogText, InNode);
}

