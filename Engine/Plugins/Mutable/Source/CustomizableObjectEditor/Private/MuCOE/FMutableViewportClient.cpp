// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/FMutableViewportClient.h"

#include "AdvancedPreviewScene.h"
#include "UnrealWidget.h"

FMutableMeshViewportClient::FMutableMeshViewportClient(const TSharedRef<FAdvancedPreviewScene>& InPreviewScene)
	: FEditorViewportClient(nullptr, &InPreviewScene.Get())
{
	// Remove the initial gizmo
	Widget->SetDefaultVisibility(false);
	
	FAdvancedPreviewScene* PreviewSceneCasted = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	//PreviewSceneCasted->SetEnvironmentVisibility(false);
	PreviewSceneCasted->SetFloorVisibility(false);

	// Set standard FOV
	const float FOVMin = 5.f;
	const float FOVMax = 170.f;
	ViewFOV = FMath::Clamp<float>(53.43f, FOVMin, FOVMax);

}
