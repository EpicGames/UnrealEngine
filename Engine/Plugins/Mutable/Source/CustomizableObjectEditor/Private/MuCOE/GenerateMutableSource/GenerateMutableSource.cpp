// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"

#include "GenerateMutableSourceComponent.h"
#include "GenerateMutableSourceModifier.h"
#include "Animation/AnimInstance.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/TextureLODSettings.h"
#include "TextureCompressorModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "MuCO/CustomizableObjectUIData.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMacro.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/ICustomizableObjectExtensionNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTexture.h"
#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureBinarise.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureColorMap.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromChannels.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureInterpolate.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureInvert.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureLayer.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureProject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureToChannels.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureTransform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureSaturate.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuT/NodeModifierSkeletalMeshMerge.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeObjectGroup.h"
#include "MuT/NodeObjectNew.h"
#include "MuR/Mesh.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PlatformInfo.h"
#include "Math/NumericLimits.h"
#include "Hash/CityHash.h"
#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"
#include "MuCOE/Nodes/CONodeSwitch.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


uint32 GetTypeHash(const FGraphCycleKey& Key)
{
	return HashCombine(HashCombine(GetTypeHash(&Key.Pin), GetTypeHash(Key.Id)), GetTypeHash(Key.MacroContext));
}


FGraphCycle::FGraphCycle(const FGraphCycleKey&& Key, FMutableGraphGenerationContext& Context) :
	Key(Key),
	Context(Context)
{
}


FGraphCycle::~FGraphCycle()
{
	Context.VisitedPins.Remove(Key);
}


bool FGraphCycle::FoundCycle() const
{
	const UCustomizableObjectNode& Node = *Cast<UCustomizableObjectNode>(Key.Pin.GetOwningNode());

	if (const UCustomizableObject** Result = Context.VisitedPins.Find(Key))
	{
		Context.CompilationContext->Log(LOCTEXT("CycleFoundNode", "Cycle detected."), &Node, EMessageSeverity::Error, true);
		Context.CustomizableObjectWithCycle = *Result;
		return true;
	}
	else
	{
		Context.VisitedPins.Add(Key, Node.GetGraph()->GetTypedOuter<UCustomizableObject>());
		return false;	
	}
}


/** Warn if the node has more outputs than it is meant to have. */
void CheckNode(const UEdGraphPin& Pin, const FMutableGraphGenerationContext& GenerationContext)
{
	if (const UCustomizableObjectNode* Typed = Cast<UCustomizableObjectNode>(Pin.GetOwningNode()))
	{
		if (Typed->IsSingleOutputNode())
		{
			int32 numOutLinks = 0;
			for (UEdGraphPin* NodePin : Typed->GetAllNonOrphanPins())
			{
				if (NodePin->Direction == EGPD_Output)
				{
					numOutLinks += NodePin->LinkedTo.Num();
				}
			}

			if (numOutLinks > 1)
			{
				GenerationContext.CompilationContext->Log(LOCTEXT("MultipleOutgoing", "The node has several outgoing connections, but it should be limited to 1."), Typed);
			}
		}
		
		if (Typed->IsDeprecated())
		{
			const FText Text = FText::Format(LOCTEXT("Deprecated", "Deprecated {0} node. Please replace it."), Typed->GetNodeTitle(ENodeTitleType::ListView));
			GenerationContext.CompilationContext->Log(Text, Typed);
		}
	}
}


uint32 GetTypeHash(const FGeneratedGroupProjectorsKey& Key)
{
	uint32 Hash = GetTypeHash(Key.Node);

	Hash = HashCombine(Hash, GetTypeHash(Key.CurrentComponent));
	
	return Hash; 
}


void FMutableCompilationContext::Log(const FText& Message, const TArray<const UObject*>& Context, const EMessageSeverity::Type MessageSeverity, const bool bAddBaseObjectInfo, const ELoggerSpamBin SpamBin) const
{
	if (TSharedPtr<FCustomizableObjectCompiler> Compiler = WeakCompiler.Pin())
	{
		Compiler->CompilerLog(Message, Context, MessageSeverity, bAddBaseObjectInfo, SpamBin);
	}
}


void FMutableCompilationContext::Log(const FText& Message, const UObject* Context, const EMessageSeverity::Type MessageSeverity, const bool bAddBaseObjectInfo, const ELoggerSpamBin SpamBin) const
{
	if (TSharedPtr<FCustomizableObjectCompiler> Compiler = WeakCompiler.Pin())
	{
		Compiler->CompilerLog(Message, Context, MessageSeverity, bAddBaseObjectInfo, SpamBin);
	}
}


FMutableCompilationContext::FMutableCompilationContext(const UCustomizableObject* InRootObject, const TSharedPtr<FCustomizableObjectCompiler>& InCompiler, const FCompilationOptions& InOptions)
	: RootObject(InRootObject)
	, Options(InOptions)
	, WeakCompiler(InCompiler)
{
}


FSkeletalMeshComponentInfo* FMutableCompilationContext::GetComponentInfo(const UCONodeComponentSkeletalMesh* NodeComponentMesh)
{
	return ComponentInfos.FindByPredicate([&](const FSkeletalMeshComponentInfo& ComponentInfo)
	{
		return ComponentInfo.Node == NodeComponentMesh;
	});
}


FSkeletalMeshComponentInfo* FMutableCompilationContext::GetComponentInfo(UE::Mutable::Private::FComponentId ComponentId)
{
	return ComponentInfos.FindByPredicate([&](const FSkeletalMeshComponentInfo& ComponentInfo)
	{
		return ComponentInfo.ComponentId == ComponentId;
	});
}


FString FMutableCompilationContext::GetObjectName() const
{
	return GetNameSafe(RootObject.Get());
}


FMutableGraphGenerationContext::FMutableGraphGenerationContext(const TSharedPtr<FMutableCompilationContext>& InCompilationContext)
	: CompilationContext(InCompilationContext)
	, ExtensionDataCompilerInterface(*this)
{
	// Default flags for mesh generation nodes.
	MeshGenerationFlags.Push(EMutableMeshConversionFlags::None);

	// Default flags for mesh layout generation.
	LayoutGenerationFlags.Push(FLayoutGenerationFlags());

	// Default SocketPriority for mesh generation.
	SocketPriorityStack.Push(0);

	// Default BonePosePriority for mesh generation.
	BonePosePriorityStack.Push(0);
}


FString FMutableGraphGenerationContext::GetObjectName() const
{
	return CompilationContext->GetObjectName();
}


void FMutableGraphGenerationContext::Log(const FText& Message, const TArray<const UObject*>& Context, const EMessageSeverity::Type MessageSeverity, const bool bAddBaseObjectInfo, const ELoggerSpamBin SpamBin) const
{
	CompilationContext->Log(Message, Context, MessageSeverity, bAddBaseObjectInfo, SpamBin);
}


void FMutableGraphGenerationContext::Log(const FText& Message, const UObject* Context, const EMessageSeverity::Type MessageSeverity, const bool bAddBaseObjectInfo, const ELoggerSpamBin SpamBin) const
{
	CompilationContext->Log(Message, Context, MessageSeverity, bAddBaseObjectInfo, SpamBin);
}


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> FMutableGraphGenerationContext::FindGeneratedMesh( const FGeneratedMeshData::FKey& Key )
{
	for (const FGeneratedMeshData& d : GeneratedMeshes)
	{
		if (d.Key == Key )
		{
			return d.Generated;
		}
	}

	return nullptr;
}


const FGuid FMutableGraphGenerationContext::GetNodeIdUnique(const UCustomizableObjectNode* Node)
{
	FGuid NodeID = GetNodeIdUnchecked(Node);
	TArray<const UObject*>* ArrayResult = NodeIdsMap.Find(NodeID);

	if (ArrayResult == nullptr)
	{
		TArray<const UObject*> ArrayTemp;
		ArrayTemp.Add(Node);
		NodeIdsMap.Add(NodeID, ArrayTemp);
		return NodeID;
	}

	ArrayResult->AddUnique(Node);

	if (ArrayResult->Num() == 1)
	{
		return NodeID;
	}

	return FGuid::NewGuid();
}


const FGuid FMutableGraphGenerationContext::GetNodeIdUnchecked(const UCustomizableObjectNode* Node)
{
	FGuid NodeID = Node->NodeGuid;

	if (Node->IsInMacro())
	{
		check(MacroNodesStack.Num()); // To ensure that we only enter here when compiling

		for (int32 MacroIndex = 0; MacroIndex < MacroNodesStack.Num(); ++MacroIndex)
		{
			NodeID = FGuid::Combine(NodeID, MacroNodesStack[MacroIndex]->NodeGuid);
		}
	}

	return NodeID;
}


UObject* FMutableGraphGenerationContext::LoadObject(const FSoftObjectPtr& SoftObject)
{
	return  UE::Mutable::Private::LoadObject(SoftObject);
}


FMaterialBreakParameter FMutableGraphGenerationContext::GetCurrentMaterialBreakParameter()
{
	if (MaterialBreakParameterStack.Num() == 0)
	{
		return FMaterialBreakParameter();
	}
	return MaterialBreakParameterStack.Last();
}


FGeneratedKey::FGeneratedKey(void* InFunctionAddress, const UEdGraphPin& InPin, const UCustomizableObjectNode& Node, FMutableGraphGenerationContext& GenerationContext,  const bool UseMesh, const bool InbOnlyConnectedLOD, const bool bUseMaterialBreakStack)
{
	FunctionAddress = InFunctionAddress;
	Pin = &InPin;
	LOD = Node.IsAffectedByLOD() ? GenerationContext.CurrentLOD : 0;
	FromLOD = Node.IsAffectedByLOD() ? GenerationContext.FromLOD : 0;

	if (bUseMaterialBreakStack)
	{
		MaterialBreakParameterStack = GenerationContext.MaterialBreakParameterStack;
	}

	if (const UCustomizableObjectNodeParameter* ParameterNode = Cast<UCustomizableObjectNodeParameter>(&Node))
	{
		// Do not use the Stack of macros as Key if the Node is a Parameter Node. 
		// By doing so, we can define parameters inside mutable macros and reuse them without having repeated nodes.
		MacroContext = TArray<const UCustomizableObjectNodeMacroInstance*>{};
		ParameterName = ParameterNode->GetParameterName(&GenerationContext.MacroNodesStack);
	}
	else
	{
		MacroContext = GenerationContext.MacroNodesStack;

		if (UseMesh)
		{
			Flags = GenerationContext.MeshGenerationFlags.Last();
			LayoutFlags = GenerationContext.LayoutGenerationFlags.Last();
			bOnlyConnectedLOD = InbOnlyConnectedLOD;
		}
	}
}


uint32 GetTypeHash(const FGeneratedKey& Key)
{
	uint32 Hash = GetTypeHash(Key.FunctionAddress);
	Hash = HashCombine(Hash, GetTypeHash(Key.Pin));
	Hash = HashCombine(Hash, GetTypeHash(Key.LOD));
	Hash = HashCombine(Hash, GetTypeHash(Key.FromLOD));
	Hash = HashCombine(Hash, GetTypeHash(Key.Flags));
	//Hash = HashCombine(Hash, GetTypeHash(Key.LayoutFlags)); // Does not support array
	//Hash = HashCombine(Hash, GetTypeHash(Key.MeshMorphStack)); // Does not support array
	Hash = HashCombine(Hash, GetTypeHash(Key.bOnlyConnectedLOD));
	Hash = HashCombine(Hash, GetTypeHash(Key.CurrentMeshComponent));
	Hash = HashCombine(Hash, GetTypeHash(Key.ReferenceTextureSize));
	Hash = HashCombine(Hash, GetTypeHash(Key.MaterialBreakParameterStack));

	for (int32 MacroIndex = 0; MacroIndex < Key.MacroContext.Num(); ++MacroIndex)
	{
		Hash = HashCombine(Hash, GetTypeHash(Key.MacroContext[MacroIndex]));
	}
	
	return Hash;
}


uint32 GetTypeHash(const FMaterialBreakParameter& MaterialBreakParameter)
{
	uint32 Hash = GetTypeHash(MaterialBreakParameter.ParameterKey);
	Hash = HashCombine(Hash, GetTypeHash(MaterialBreakParameter.ParameterType));
	return Hash;
}


uint32 GetTypeHash(const FGeneratedSourceExternalKey& Key)
{
	uint32 Hash = GetTypeHash(Key.Pin);

	Hash = HashCombine(Hash, GetTypeHash(Key.Options.MeshOptions));
	Hash = HashCombine(Hash, GetTypeHash(Key.Options.bLinkedToExtendMaterial));
	Hash = HashCombine(Hash, GetTypeHash(Key.Options.bOnlyConnectedLOD));
	Hash = HashCombine(Hash, GetTypeHash(Key.Options.ReferenceTextureSize));
	
	return Hash;
}


uint32 GetTypeHash(const FSourceSkeletalMeshObjectOptions& Key)
{
	uint32 Hash = 0;

	Hash = HashCombine(Hash, GetTypeHash(Key.ReferenceSkeletalMesh));
	
	return Hash;
}


uint32 GetTypeHash(const FSourceSkeletalMeshOptions& Key)
{
	uint32 Hash = 0;

	Hash = HashCombine(Hash, GetTypeHash(Key.NumLODs));
	Hash = HashCombine(Hash, GetTypeHash(Key.FirstLODAvailable));

	return Hash;
}


uint32 GetTypeHash(const FGeneratedSourceSkeletalMeshKey& Key)
{
	uint32 Hash = 0;

	Hash = HashCombine(Hash, GetTypeHash(Key.Pin));
	Hash = HashCombine(Hash, GetTypeHash(Key.Options));

	return Hash;
}


uint32 GetTypeHash(const FGeneratedSourceSkeletalMeshObjectKey& Key)
{
	uint32 Hash = 0;

	Hash = HashCombine(Hash, GetTypeHash(Key.Pin));
	Hash = HashCombine(Hash, GetTypeHash(Key.Options));

	return Hash;
}


uint32 GetTypeHash(const FGeneratedParameterKey& Key)
{
	uint32 Hash = GetTypeHash(Key.NodeId);
	Hash = HashCombine(Hash, GetTypeHash(Key.ParameterName));

	return Hash;
}


FGraphCycleKey::FGraphCycleKey(const UEdGraphPin& Pin, const FString& Id, const UCustomizableObjectNodeMacroInstance* MacroContext) :
	Pin(Pin),
	Id(Id),
	MacroContext(MacroContext)
{
}


bool FGraphCycleKey::operator==(const FGraphCycleKey& Other) const
{
	return &Pin == &Other.Pin && Id == Other.Id && MacroContext == Other.MacroContext;
}


UTexture2D* FindReferenceImage(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	const UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	UTexture2D* Result = nullptr;

	if (const UCustomizableObjectNodeTexture* TypedNodeTex = Cast<UCustomizableObjectNodeTexture>(Node))
	{
		Result = Cast<UTexture2D>(TypedNodeTex->Texture);
	}

	else if (const UCustomizableObjectNodeTextureParameter* ParamNodeTex = Cast<UCustomizableObjectNodeTextureParameter>(Node))
	{
		Result = ParamNodeTex->ReferenceValue;
	}

	else if (const ICustomizableObjectNodeMeshInterface* TypedNodeMesh = Cast<ICustomizableObjectNodeMeshInterface>(Node))
	{
		Result = TypedNodeMesh->FindTextureForPin(Pin);
	}

	else if (const UCustomizableObjectNodeTextureInterpolate* TypedNodeInterp = Cast<UCustomizableObjectNodeTextureInterpolate>(Node))
	{
		for (int32 LayerIndex = 0; !Result && LayerIndex < TypedNodeInterp->GetNumTargets(); ++LayerIndex)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeInterp->Targets(LayerIndex)))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureLayer* TypedNodeLayer = Cast<UCustomizableObjectNodeTextureLayer>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeLayer->BasePin.Get()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}

		for (int32 LayerIndex = 0; !Result && LayerIndex < TypedNodeLayer->GetNumLayers(); ++LayerIndex)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeLayer->GetLayerPin(LayerIndex)))
			{
				if (ConnectedPin->PinType.PinCategory == Schema->PC_Texture)
				{
					Result = FindReferenceImage(ConnectedPin, GenerationContext);
				}
			}
		}
	}

	else if (const UCONodeSwitch* TypedNodeSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Texture))
	{
		for (int32 SelectorIndex = 0; !Result && SelectorIndex < TypedNodeSwitch->SwitchPins.Num(); ++SelectorIndex)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeSwitch->SwitchPins[SelectorIndex].Get()))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
	}

	else if (const UCONodeSwitch* TypedNodePassThroughSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough))
	{
		for (int32 SelectorIndex = 0; !Result && SelectorIndex < TypedNodePassThroughSwitch->SwitchPins.Num(); ++SelectorIndex)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodePassThroughSwitch->SwitchPins[SelectorIndex].Get()))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureVariation* TypedNodeVariation = Cast<UCustomizableObjectNodeTextureVariation>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeVariation->DefaultPin()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}

		for (int32 SelectorIndex = 0; !Result && SelectorIndex < TypedNodeVariation->GetNumVariations(); ++SelectorIndex)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeVariation->VariationPin(SelectorIndex)))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureFromChannels* TypedNodeFrom = Cast<UCustomizableObjectNodeTextureFromChannels>(Node))
	{
		UE::Mutable::Private::NodeImagePtr RNode, GNode, BNode, ANode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->RPin()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}
		if (!Result)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->GPin()))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
		if (!Result)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->BPin()))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
			
		}
		if (!Result)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->APin()))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureToChannels* TypedNodeTo = Cast<UCustomizableObjectNodeTextureToChannels>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTo->InputPin()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTextureProject* TypedNodeProj = Cast<UCustomizableObjectNodeTextureProject>(Node))
	{
		if (TypedNodeProj->ReferenceTexture)
		{
			Result = TypedNodeProj->ReferenceTexture;
		}
		else
		{
			int32 TexIndex = -1;// TypedNodeProj->OutputPins.Find((UEdGraphPin*)Pin);
			for (int32 i = 0; i < TypedNodeProj->GetNumOutputs(); ++i)
			{
				if (TypedNodeProj->OutputPins(i) == Pin)
				{
					TexIndex = i;
				}
			}

			check(TexIndex >= 0 && TexIndex < TypedNodeProj->GetNumTextures());

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProj->TexturePins(TexIndex)))
			{
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureBinarise* TypedNodeBin = Cast<UCustomizableObjectNodeTextureBinarise>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeBin->GetBaseImagePin()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTextureInvert* TypedNodeInv = Cast<UCustomizableObjectNodeTextureInvert>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeInv->GetBaseImagePin()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTextureColorMap* TypedNodeColorMap = Cast<UCustomizableObjectNodeTextureColorMap>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorMap->GetBasePin()))
		{
			Result = FindReferenceImage(ConnectedPin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTextureTransform* TypedNodeTransform = Cast<UCustomizableObjectNodeTextureTransform>(Node))
	{
		if ( UEdGraphPin* BaseImagePin = FollowInputPin(*TypedNodeTransform->GetBaseImagePin()) )
		{
			Result = FindReferenceImage(BaseImagePin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTextureSaturate* TypedNodeSaturate = Cast<UCustomizableObjectNodeTextureSaturate>(Node))
	{
		if ( UEdGraphPin* BaseImagePin = FollowInputPin(*TypedNodeSaturate->GetBaseImagePin()) )
		{
			Result = FindReferenceImage(BaseImagePin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		if (Pin->PinType.PinCategory == Schema->PC_Material)
		{
			Result = TypedNodeTable->FindReferenceTextureParameter(Pin, GenerationContext.CurrentMaterialTableParameter);
		}
		else
		{
			Result = TypedNodeTable->GetColumnDefaultAssetByType<UTexture2D>(Pin);
		}
	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		if (const UEdGraphPin* OutputPin = TypedNodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin->PinName))
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*OutputPin))
			{
				GenerationContext.MacroNodesStack.Push(TypedNodeMacro);
				Result = FindReferenceImage(ConnectedPin, GenerationContext);
				GenerationContext.MacroNodesStack.Pop();
			}
		}
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		check(TypedNodeTunnel->bIsInputNode);
		check(GenerationContext.MacroNodesStack.Num());

		const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = GenerationContext.MacroNodesStack.Pop();
		check(MacroInstanceNode);

		if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin->PinName, EEdGraphPinDirection::EGPD_Input))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
			{
				Result = FindReferenceImage(FollowPin, GenerationContext);
			}
		}

		// Push the Macro again even if the result is null
		GenerationContext.MacroNodesStack.Push(MacroInstanceNode);
	}
	
	return Result;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshApplyPose> CreateNodeMeshApplyPose(FMutableGraphGenerationContext& GenerationContext, UE::Mutable::Private::NodeMeshPtr InputMeshNode, const TArray<FName>& ArrayBoneName, const TArray<FTransform>& ArrayTransform)
{
	check(ArrayBoneName.Num() == ArrayTransform.Num());

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = 
			UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshConstant> PoseNodeMesh = new UE::Mutable::Private::NodeMeshConstant;
	PoseNodeMesh->Value = MutableMesh;

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeleton> MutableSkeleton = 
			UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FSkeleton>();

	MutableMesh->SetSkeleton(MutableSkeleton);
	MutableMesh->SetBonePoseCount(ArrayBoneName.Num());
	MutableSkeleton->BoneNames.Reserve(ArrayBoneName.Num());
	MutableSkeleton->BoneParents.Reserve(ArrayBoneName.Num());

	for (int32 i = 0; i < ArrayBoneName.Num(); ++i)
	{
		constexpr int16 ParentIndex = -1; // Skeletons from poses don't need bone parents?

		UE::Mutable::Private::FBoneIdOrIndex MutableBoneIndex;
		MutableBoneIndex.Index = MutableSkeleton->AddBone(ArrayBoneName[i], ParentIndex);
		MutableMesh->SetBonePose(i, MutableBoneIndex, (FTransform3f)ArrayTransform[i], UE::Mutable::Private::EBoneUsageFlags::Skinning);
	}

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshApplyPose> NodeMeshApplyPose = new UE::Mutable::Private::NodeMeshApplyPose;
	NodeMeshApplyPose->Base = InputMeshNode;
	NodeMeshApplyPose->Pose = PoseNodeMesh;

	return NodeMeshApplyPose;
}


const FSkeletalMaterial* GetSkeletalMaterial(const USkeletalMesh* SkeletalMesh, uint8 LODIndex, uint8 SectionIndex)
{
	if (!SkeletalMesh)
	{
		return nullptr;
	}

	// We assume that LODIndex and MaterialIndex are valid for the imported model
	int32 MaterialIndex = INDEX_NONE;

	// Check if we have lod info map to get the correct material index
	if (const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LODIndex))
	{
		if (LodInfo->LODMaterialMap.IsValidIndex(SectionIndex))
		{
			MaterialIndex = LodInfo->LODMaterialMap[SectionIndex];
		}
	}

	if (MaterialIndex == INDEX_NONE)
	{
		const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		if (ImportedModel && ImportedModel->LODModels.IsValidIndex(LODIndex) && ImportedModel->LODModels[LODIndex].Sections.IsValidIndex(SectionIndex))
		{
			MaterialIndex = ImportedModel->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex;
		}
	}

	if (SkeletalMesh->GetMaterials().IsValidIndex(MaterialIndex))
	{
		return &SkeletalMesh->GetMaterials()[MaterialIndex];
	}
		
	return nullptr;
}


bool DoOptionsMatchEnum(const UCONodeSwitch& InEditorSwitchNode,
	const UE::Mutable::Private::NodeScalarEnumParameter& MutableScalarParameter)
{
	// todo: use the actual UEditor CO node once the issue UE-294591 is resolved. Currently we do need to use the core node as GenerateMutableSource is
	// capable of navigating over the macro objects while FindInputPin is not
	
	const int32 EditorNodeSwitchPinsCount = InEditorSwitchNode.SwitchPins.Num();
	if (MutableScalarParameter.Options.Num() != EditorNodeSwitchPinsCount)
	{
		return false;
	}
		
	// Compare the name of the enum options. It expects to be able to compare the name of the switch pin with the value of the Enum Parameter
	for (int32 ValueIndex = 0; ValueIndex < EditorNodeSwitchPinsCount; ++ValueIndex)
	{
		FString SwitchNodeValueName;
		const UEdGraphPin* SwitchPin = InEditorSwitchNode.SwitchPins[ValueIndex].Get();
		check(SwitchPin)

		SwitchNodeValueName = SwitchPin->PinName.ToString();
		const FString EnumValueName = MutableScalarParameter.Options[ValueIndex].Name;

		if (EnumValueName.Compare(SwitchNodeValueName) != 0)
		{
			return false;
		}
	}
				
	return true;
}


// Convert a CustomizableObject Source Graph into a mutable source graph  
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> GenerateMutableSource(const UEdGraphPin * Pin, FMutableGraphGenerationContext & GenerationContext)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSource);

	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());
	
	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSource), *Pin, *Node, GenerationContext, true);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeObject*>(Generated->Node.get());
	}

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For exampel, MacroInstanceNodes
	bool bCacheNode = true;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> Result;
	
	if (const UCustomizableObjectNodeObject* TypedNodeObj = Cast<UCustomizableObjectNodeObject>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObjectNew> ObjectNode = new UE::Mutable::Private::NodeObjectNew();
		Result = ObjectNode;

		ObjectNode->SetMessageContext(Node);
		ObjectNode->SetName(TypedNodeObj->GetObjectName(&GenerationContext.MacroNodesStack));
		FGuid FinalGuid = GenerationContext.GetNodeIdUnique(TypedNodeObj);
		if (FinalGuid != TypedNodeObj->NodeGuid && !TypedNodeObj->IsInMacro())
		{
			GenerationContext.Log(FText::FromString(TEXT("Warning: Node has a duplicated GUID. A new ID has been generated, but cooked data will not be deterministic.")), Node, EMessageSeverity::Warning);
		}
		ObjectNode->SetUid(FinalGuid.ToString());
		
		// States
		int32 NumStates = TypedNodeObj->States.Num();
		ObjectNode->SetStateCount(NumStates);

		// In a partial compilation we will filter the states of the root object
		bool bFilterStates = true;

		if (GenerationContext.bPartialCompilation)
		{
			if (!TypedNodeObj->ParentObject)
			{
				bFilterStates = false;
			}
		}

		for (int32 StateIndex = 0; StateIndex < NumStates && bFilterStates; ++StateIndex)
		{
			const FCustomizableObjectState& State = TypedNodeObj->States[StateIndex];
			ObjectNode->SetStateName(StateIndex, State.Name);
			for (int32 ParamIndex = 0; ParamIndex < State.RuntimeParameters.Num(); ++ParamIndex)
			{
				ObjectNode->AddStateParam(StateIndex, State.RuntimeParameters[ParamIndex]);
			}

			ObjectNode->SetStateProperties(StateIndex, State.TextureCompressionStrategy, State.bBuildOnlyFirstLOD, 0);

			// UI Data
			FMutableStateData StateUIData;
			StateUIData.StateUIMetadata = State.UIMetadata;
			StateUIData.bDisableMeshStreaming = State.bDisableMeshStreaming;
			StateUIData.bDisableTextureStreaming = State.bDisableTextureStreaming;
			StateUIData.bLiveUpdateMode = State.bLiveUpdateMode;
			StateUIData.ForcedParameterValues = State.ForcedParameterValues;

			GenerationContext.StateUIDataMap.Add(State.Name, StateUIData);
		}
		
		// Process components.
		//-------------------------------------------------------------------
		const UEdGraphPin* ComponentsPin = TypedNodeObj->ComponentsPin();
		if (ComponentsPin)
		{
			TArray<UEdGraphPin*> ConnectedComponentPins = FollowInputPinArray(*ComponentsPin);
			for (const UEdGraphPin* ComponentNodePin : ConnectedComponentPins)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> ComponentNode = GenerateMutableSourceComponent(ComponentNodePin, GenerationContext);
				ObjectNode->Components.Add(ComponentNode);
			}
		}

		// Process modifiers.
		//-------------------------------------------------------------------
		if (const UEdGraphPin* ModifierPin = TypedNodeObj->ModifiersPin())
		{
			TArray<UEdGraphPin*> ConnectedModifierPins = FollowInputPinArray(*ModifierPin);
			for (UEdGraphPin* const ConnectedModifier : ConnectedModifierPins)
			{
				TArray<UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeModifier>> ModifierNodes = UE::Mutable::Private::GenerateMutableSourceModifier(ConnectedModifier, GenerationContext);
				for (const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeModifier>& ModifierNode : ModifierNodes)
				{
					ObjectNode->Modifiers.Add(ModifierNode);
				}
			}
		}
		
		// Generate inputs to Object node pins added by extensions
		//-------------------------------------------------------------------
		for (const FRegisteredObjectNodeInputPin& ExtensionInputPin : ICustomizableObjectModule::Get().GetAdditionalObjectNodePins())
		{
			const UEdGraphPin* GraphPin = TypedNodeObj->FindPin(ExtensionInputPin.GlobalPinName, EGPD_Input);
			if (!GraphPin)
			{
				continue;
			}

			TArray<UEdGraphPin*> ConnectedPins = FollowInputPinArray(*GraphPin);

			// If the pin isn't supposed to take more than one connection, ignore all but the first
			// incoming connection.
			if (!ExtensionInputPin.InputPin.bIsArray && ConnectedPins.Num() > 1)
			{
				FString Msg = FString::Printf(TEXT("Extension input %s has multiple incoming connections but is only expecting one connection."),
					*ExtensionInputPin.InputPin.DisplayName.ToString());

				GenerationContext.Log(FText::FromString(Msg), Node, EMessageSeverity::Warning);
			}

			for (const UEdGraphPin* ConnectedPin : ConnectedPins)
			{
				const UEdGraphNode* ConnectedNode = ConnectedPin->GetOwningNode();
				
				if (const ICustomizableObjectExtensionNode* ExtensionNode = Cast<ICustomizableObjectExtensionNode>(ConnectedNode))
				{
					if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> GeneratedNode = ExtensionNode->GenerateMutableNode(GenerationContext.ExtensionDataCompilerInterface))
					{
						ObjectNode->AddExtensionDataNode(GeneratedNode);
					}
				}
			}
		}

		// Children
		//-------------------------------------------------------------------
		TArray<UEdGraphPin*> ConnectedChildrenPins = FollowInputPinArray(*TypedNodeObj->ChildrenPin());
		ObjectNode->Children.Reserve(ConnectedChildrenPins.Num());
		for (int32 ChildIndex = 0; ChildIndex < ConnectedChildrenPins.Num(); ++ChildIndex)
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> ChildNode = GenerateMutableSource(ConnectedChildrenPins[ChildIndex], GenerationContext);
			ObjectNode->Children.Add(ChildNode);
		}
	}

	else if (const UCustomizableObjectNodeObjectGroup* TypedNodeGroup = Cast<UCustomizableObjectNodeObjectGroup>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObjectGroup> GroupNode = new UE::Mutable::Private::NodeObjectGroup();
		Result = GroupNode;

		// All sockets from all mesh parts plugged into this group node will have the following priority when there's a socket name clash
		GenerationContext.SocketPriorityStack.Push(TypedNodeGroup->SocketPriority);

		// All bone poses from all mesh parts plugged into this group node will have the following priority when there's a EBoneUsageFlags clash
		GenerationContext.BonePosePriorityStack.Push(TypedNodeGroup->BonePosePriority);

		// Getting node Id. We may need a different ID if the group node is inside a Macro and this Macro is used more than once.
		// Group nodes of Macros Can Not have external children so we can change their ids here safetely.
		FGuid GroupNodeId = GenerationContext.GetNodeIdUnchecked(TypedNodeGroup);

		GroupNode->SetMessageContext(Node);
		GroupNode->SetName(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack));
		GroupNode->SetUid(GroupNodeId.ToString());
		
		TArray<UCustomizableObjectNodeGroupProjectorParameter*> GroupProjectors;
		if (UEdGraphPin* ProjectorsPin = TypedNodeGroup->GroupProjectorsPin())
		{
			for (const UEdGraphPin* ConnectedPin : FollowInputPinArray(*ProjectorsPin))
			{
				// Checking if it's linked to a Macro or tunnel node
				if (const UEdGraphPin* GroupProjectorPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, &GenerationContext.MacroNodesStack))
				{
					if (UCustomizableObjectNodeGroupProjectorParameter* GroupProjectorNode = Cast<UCustomizableObjectNodeGroupProjectorParameter>(GroupProjectorPin->GetOwningNode()))
					{
						GroupProjectors.Add(GroupProjectorNode);
					}
				}
			}
		}

		GenerationContext.CurrentGroupProjectors.Push(GroupProjectors);
		ON_SCOPE_EXIT
		{
			GenerationContext.CurrentGroupProjectors.Pop();
		};

		UE::Mutable::Private::NodeObjectGroup::EChildSelection Type = UE::Mutable::Private::NodeObjectGroup::CS_ALWAYS_ALL;
		switch (TypedNodeGroup->GroupType)
		{
		case ECustomizableObjectGroupType::COGT_ALL: Type = UE::Mutable::Private::NodeObjectGroup::CS_ALWAYS_ALL; break;
		case ECustomizableObjectGroupType::COGT_TOGGLE: Type = UE::Mutable::Private::NodeObjectGroup::CS_TOGGLE_EACH; break;
		case ECustomizableObjectGroupType::COGT_ONE: Type = UE::Mutable::Private::NodeObjectGroup::CS_ALWAYS_ONE; break;
		case ECustomizableObjectGroupType::COGT_ONE_OR_NONE: Type = UE::Mutable::Private::NodeObjectGroup::CS_ONE_OR_NONE; break;
		default:
			GenerationContext.Log(LOCTEXT("UnsupportedGroupType", "Object Group Type not supported. Setting to 'ALL'."), Node);
			break;
		}
		GroupNode->Type = Type;

		// External children
		TArray<UCustomizableObjectNodeObject*> ExternalChildNodes;

		GenerationContext.GroupIdToExternalNodeMap.MultiFind(GroupNodeId, ExternalChildNodes);
		FMutableGraphGenerationContext::FParamInfo ParamInfo(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack),
															 TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_TOGGLE);
		GenerationContext.GuidToParamNameMap.Add(GroupNodeId,ParamInfo);

		// Children
		TArray<const UEdGraphPin*> ConnectedChildrenPins;

		for (const UEdGraphPin* LinkedPin : FollowInputPinArray(*TypedNodeGroup->ObjectsPin()))
		{
			// Checking if it's linked to a macro node and the macro node is linked to something
			if (const UEdGraphPin* MacroCheckPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, &GenerationContext.MacroNodesStack))
			{
				// We use the original pins to generate the children objects. GenerateMutableSourceX functions already go through macro nodes.
				ConnectedChildrenPins.Add(LinkedPin);
			}
		}

		const int32 NumChildren = ConnectedChildrenPins.Num();
		const int32 TotalNumChildren = NumChildren + ExternalChildNodes.Num();

		GroupNode->Children.SetNum(TotalNumChildren);
		GroupNode->DefaultValue = (Type == UE::Mutable::Private::NodeObjectGroup::CS_ONE_OR_NONE ? -1 : 0);
		int32 ChildIndex = 0;

		// UI data
		FMutableParameterData ParameterUIData(
			TypedNodeGroup->ParamUIMetadata,
			EMutableParameterType::Int);

		ParameterUIData.IntegerParameterGroupType = TypedNodeGroup->GroupType;

		// In the case of a partial compilation, make sure at least one child is connected so that the param is no optimized
		bool bAtLeastOneConnected = false;

		for (; ChildIndex < NumChildren; ++ChildIndex)
		{
			bool bLastChildNode = (ChildIndex == NumChildren - 1) && (ExternalChildNodes.Num() == 0);
			bool bConnectAtLeastTheLastChild = bLastChildNode && !bAtLeastOneConnected;

			// Check if this node is linked to a MacroNode
			const UEdGraphPin* MacroContextPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedChildrenPins[ChildIndex], &GenerationContext.MacroNodesStack);
			check(MacroContextPin);

			UCustomizableObjectNodeObject* CustomizableObjectNodeObject = Cast<UCustomizableObjectNodeObject>(MacroContextPin->GetOwningNode());

			const FString* SelectedOptionName = GenerationContext.CompilationContext->Options.ParamNamesToSelectedOptions.Find(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack)); // If the param is in the map restrict to only the selected option
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> ChildNode;

			if (bConnectAtLeastTheLastChild || !SelectedOptionName || (CustomizableObjectNodeObject && *SelectedOptionName == CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack)) )
			{
				bAtLeastOneConnected = true;

				ChildNode = GenerateMutableSource(ConnectedChildrenPins[ChildIndex], GenerationContext);
				GroupNode->Children[ChildIndex] = ChildNode;

				if (CustomizableObjectNodeObject)
				{
					FString LeftSplit = CustomizableObjectNodeObject->GetPathName();
					LeftSplit.Split(".", &LeftSplit, nullptr);
					GenerationContext.CustomizableObjectPathMap.Add(CustomizableObjectNodeObject->Identifier.ToString(), LeftSplit);
					GenerationContext.GroupNodeMap.Add(CustomizableObjectNodeObject->Identifier.ToString(), FCustomizableObjectIdPair(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack), ChildNode->GetName()));
					ParameterUIData.ArrayIntegerParameterOption.Add(
						CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack),
				FIntegerParameterUIData(CustomizableObjectNodeObject->ParamUIMetadata));

					if (TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_TOGGLE)
					{
						// UI Data is only relevant when the group node is set to Toggle
						GenerationContext.ParameterUIDataMap.Add(CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack), FMutableParameterData(
							CustomizableObjectNodeObject->ParamUIMetadata,
							EMutableParameterType::Int));
					}
				}
			}
			else
			{
				ChildNode = new UE::Mutable::Private::NodeObjectNew;
				ChildNode->SetName(CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack));
				GroupNode->Children[ChildIndex] = ChildNode;
			}

			if ((TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE ||
				TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE_OR_NONE)
				&& TypedNodeGroup->DefaultValue == ChildNode->GetName())
			{
				GroupNode->DefaultValue = ChildIndex;
			}
		}

		const bool bCollapseUnderParent = TypedNodeGroup->ParamUIMetadata.ExtraInformation.Find(FString("CollapseUnderParent")) != nullptr;
		constexpr bool bHideWhenNotSelected = true; //TypedNodeGroup->ParamUIMetadata.ExtraInformation.Find(FString("HideWhenNotSelected"));
		
		if (bCollapseUnderParent || bHideWhenNotSelected)
		{
			if (const UEdGraphPin* ConnectedPin = FollowOutputPin(*Pin))
			{
				// Checking if it's linked to a Macro or tunnel node
				const UEdGraphPin* NodeLinkedPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, &GenerationContext.MacroNodesStack);

				if (NodeLinkedPin)
				{
					const FGuid* ParentId = nullptr;
					FString ValueName;

					if (UCustomizableObjectNodeObject* NodeObject = Cast<UCustomizableObjectNodeObject>(NodeLinkedPin->GetOwningNode()))
					{
						ParentId = GenerationContext.GroupIdToExternalNodeMap.FindKey(NodeObject);
						ValueName = NodeObject->GetObjectName(&GenerationContext.MacroNodesStack);

						// Group objects in the same graph aren't in the GroupIdToExternalNodeMap, so follow the pins instead
						if (!ParentId && NodeObject->OutputPin())
						{
							if (const UEdGraphPin* ConnectedPinToObject = FollowOutputPin(*NodeObject->OutputPin()))
							{
								if (UCustomizableObjectNodeObjectGroup* ParentGroupNode = Cast<UCustomizableObjectNodeObjectGroup>(ConnectedPinToObject->GetOwningNode()))
								{
									FGuid ParentGroupNodeId = GenerationContext.GetNodeIdUnchecked(ParentGroupNode);
									ParentId = &ParentGroupNodeId;
								}
							}
						}
					}
					else if (UCustomizableObjectNodeObjectGroup* NodeObjectGroup = Cast<UCustomizableObjectNodeObjectGroup>(NodeLinkedPin->GetOwningNode()))
					{
						FGuid ParentGroupNodeId = GenerationContext.GetNodeIdUnchecked(NodeObjectGroup);
						ParentId = &ParentGroupNodeId;

						ValueName = TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack);
					}

					if (ParentId)
					{
						FMutableGraphGenerationContext::FParamInfo* ParentParamInfo = GenerationContext.GuidToParamNameMap.Find(*ParentId);

						if (ParentParamInfo)
						{
							FString ParentParamName = ParentParamInfo->ParamName;

							ParameterUIData.ParamUIMetadata.ExtraInformation.Add(FString("__ParentParamName"),
								ParentParamInfo->bIsToggle ? ValueName : *ParentParamName);

							if (bHideWhenNotSelected)
							{
								ParameterUIData.ParamUIMetadata.ExtraInformation.Add(FString("__DisplayWhenParentValueEquals"),
									ParentParamInfo->bIsToggle ? FString("1") : ValueName);
							}

							if (bCollapseUnderParent)
							{
								ParameterUIData.ParamUIMetadata.ExtraInformation.Add(FString("CollapseUnderParent"));

								FMutableParameterData ParentParameterUIData;
								ParentParameterUIData.ParamUIMetadata.ExtraInformation.Add(FString("__HasCollapsibleChildren"));
								GenerationContext.ParameterUIDataMap.Add(*ParentParamName, ParentParameterUIData);
							}
						}
					}
				}
			}
		}

		// Build external objects that reference this object as parent
		const int32 NumExternalChildren = FMath::Max(0, TotalNumChildren - NumChildren);
		for (int32 ExternalChildIndex = 0; ExternalChildIndex < NumExternalChildren; ++ExternalChildIndex)
		{
			const UCustomizableObjectNodeObject* ExternalChildNode = ExternalChildNodes[ExternalChildIndex];
			bool bLastExternalChildNode = ExternalChildIndex == ExternalChildNodes.Num() - 1;
			bool bConnectAtLeastTheLastChild = bLastExternalChildNode && !bAtLeastOneConnected;

			UCustomizableObjectNodeObject* CustomizableObjectNodeObject = Cast<UCustomizableObjectNodeObject>(ExternalChildNode->OutputPin()->GetOwningNode());

			const FString* SelectedOptionName = GenerationContext.CompilationContext->Options.ParamNamesToSelectedOptions.Find(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack)); // If the param is in the map restrict to only the selected option
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> ChildNode;

			if (bConnectAtLeastTheLastChild || !SelectedOptionName || (CustomizableObjectNodeObject  && *SelectedOptionName == CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack)) )
			{
				bAtLeastOneConnected = true;

				ChildNode = GenerateMutableSource(ExternalChildNode->OutputPin(), GenerationContext);
				GroupNode->Children[ChildIndex] = ChildNode;

				if (CustomizableObjectNodeObject)
				{
					FString LeftSplit = ExternalChildNode->GetPathName();
					LeftSplit.Split(".", &LeftSplit, nullptr);
					GenerationContext.CustomizableObjectPathMap.Add(CustomizableObjectNodeObject->Identifier.ToString(), LeftSplit);
					GenerationContext.GroupNodeMap.Add(CustomizableObjectNodeObject->Identifier.ToString(), FCustomizableObjectIdPair(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack), ChildNode->GetName()));
					ParameterUIData.ArrayIntegerParameterOption.Add(
						CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack),
						FIntegerParameterUIData(CustomizableObjectNodeObject->ParamUIMetadata));

					if (CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack).IsEmpty())
					{
						GenerationContext.NoNameNodeObjectArray.AddUnique(CustomizableObjectNodeObject);
					}

					if (TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_TOGGLE)
					{
						// UI Data is only relevant when the group node is set to Toggle
						GenerationContext.ParameterUIDataMap.Add(CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack), FMutableParameterData(
							CustomizableObjectNodeObject->ParamUIMetadata,
							EMutableParameterType::Int));
					}
				}
			}
			else
			{
				ChildNode = new UE::Mutable::Private::NodeObjectNew;
				ChildNode->SetName(CustomizableObjectNodeObject->GetObjectName(&GenerationContext.MacroNodesStack));
				GroupNode->Children[ChildIndex] = ChildNode;
			}

			if ((TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE ||
				TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_ONE_OR_NONE)
				&& TypedNodeGroup->DefaultValue == ChildNode->GetName())
			{
				GroupNode->DefaultValue = ChildIndex;
			}

			ChildIndex++;
		}

		const FMutableParameterData* ChildFilledUIData = GenerationContext.ParameterUIDataMap.Find(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack));
		if (ChildFilledUIData && ChildFilledUIData->ParamUIMetadata.ExtraInformation.Find(FString("__HasCollapsibleChildren")))
		{
			// Some child param filled the HasCollapsibleChildren UI info, refill it so it's not lost
			ParameterUIData.ParamUIMetadata.ExtraInformation.Add(FString("__HasCollapsibleChildren"));
		}

		if (TypedNodeGroup->GroupType == ECustomizableObjectGroupType::COGT_TOGGLE)
		{
			for (const TTuple<FString, FIntegerParameterUIData>& BooleanParam : ParameterUIData.ArrayIntegerParameterOption)
			{
				FMutableParameterData ParameterUIDataBoolean(
					BooleanParam.Value.ParamUIMetadata,
					EMutableParameterType::Bool);

				ParameterUIDataBoolean.ParamUIMetadata.ExtraInformation = ParameterUIData.ParamUIMetadata.ExtraInformation;

				GenerationContext.ParameterUIDataMap.Add(BooleanParam.Key, ParameterUIDataBoolean);
			}
		}
		else
		{
			GenerationContext.ParameterUIDataMap.Add(TypedNodeGroup->GetGroupName(&GenerationContext.MacroNodesStack), ParameterUIData);
		}

		// Go back to the parent group node's socket priority if it exists
		ensure(GenerationContext.SocketPriorityStack.Num() > 0);
		GenerationContext.SocketPriorityStack.Pop();

		// Go back to the parent group node's bone pose priority if it exists
		ensure(GenerationContext.BonePosePriorityStack.Num() > 0);
		GenerationContext.BonePosePriorityStack.Pop();
	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeObject>(*Pin, GenerationContext, GenerateMutableSource);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeObject>(*Pin, GenerationContext, GenerateMutableSource);
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	return Result;
}


void PopulateReferenceSkeletalMeshesData(FMutableGraphGenerationContext& GenerationContext)
{
	const FString PlatformName = GenerationContext.CompilationContext->Options.TargetPlatform->IniPlatformName();
	
	const uint32 ComponentCount = GenerationContext.CompilationContext->ComponentInfos.Num();

	GenerationContext.ReferenceSkeletalMeshesData.AddDefaulted(ComponentCount);
	for (uint32 ComponentIndex = 0; ComponentIndex < ComponentCount; ++ComponentIndex)
	{
		const FSkeletalMeshComponentInfo& ComponentInfo = GenerationContext.CompilationContext->ComponentInfos[ComponentIndex];
		
		USkeletalMesh* RefSkeletalMesh = ComponentInfo.RefSkeletalMesh.Get();
		if (!RefSkeletalMesh)
		{
			continue;
		}

		FMutableRefSkeletalMeshData& Data = GenerationContext.ReferenceSkeletalMeshesData[ComponentIndex];

		// Set the RefSkeletalMesh
		if (!GenerationContext.CompilationContext->Options.TargetPlatform->IsClientOnly()
			|| GenerationContext.CompilationContext->RootObject->bEnableUseRefSkeletalMeshAsPlaceholder)
		{
			Data.SkeletalMesh = RefSkeletalMesh;
		}

		Data.SoftSkeletalMesh = RefSkeletalMesh;

		// Set the optional USkeletalMeshLODSettings that will be applied to the generated transient meshes or the baked meshes
		Data.SkeletalMeshLODSettings = RefSkeletalMesh->GetLODSettings();

		// Gather SkeletalMesh Sockets;
		const TArray<USkeletalMeshSocket*>& RefSkeletonSockets = RefSkeletalMesh->GetMeshOnlySocketList();
		const uint32 SocketCount = RefSkeletonSockets.Num();

		Data.Sockets.AddDefaulted(SocketCount);
		for(uint32 SocketIndex = 0 ; SocketIndex < SocketCount; ++SocketIndex)
		{
			const USkeletalMeshSocket* RefSocket = RefSkeletonSockets[SocketIndex];
			check(RefSocket);

			FMutableRefSocket& Socket = Data.Sockets[SocketIndex];
			Socket.SocketName = RefSocket->SocketName;
			Socket.BoneName = RefSocket->BoneName;
			Socket.RelativeLocation = RefSocket->RelativeLocation;
			Socket.RelativeRotation = RefSocket->RelativeRotation;
			Socket.RelativeScale = RefSocket->RelativeScale;
			Socket.bForceAlwaysAnimated = RefSocket->bForceAlwaysAnimated;
		}

		// TODO: Generate Bounds?
		// Gather Bounds
		Data.Bounds = RefSkeletalMesh->GetBounds();
		
		// Additional Settings
		Data.Settings.bEnablePerPolyCollision = RefSkeletalMesh->GetEnablePerPolyCollision();

		const TArray<FSkeletalMaterial>& Materials = RefSkeletalMesh->GetMaterials();
		for (const FSkeletalMaterial& Material : Materials)
		{
			if (Material.UVChannelData.bInitialized)
			{
				for (int32 UVIndex = 0; UVIndex < TEXSTREAM_MAX_NUM_UVCHANNELS; ++UVIndex)
				{
					Data.Settings.DefaultUVChannelDensity = FMath::Max(Data.Settings.DefaultUVChannelDensity, Material.UVChannelData.LocalUVDensities[UVIndex]);
				}
			}
		}

		// Skeleton
		if(USkeleton* Skeleton = RefSkeletalMesh->GetSkeleton())
		{
			Data.Skeleton = Skeleton;
		}
		
		// Physics Asset
		if (UPhysicsAsset* PhysicsAsset = RefSkeletalMesh->GetPhysicsAsset())
		{
			Data.PhysicsAsset = PhysicsAsset;
		}

		// Post ProcessAnimInstance
		if(const TSubclassOf<UAnimInstance> PostProcessAnimInstance = RefSkeletalMesh->GetPostProcessAnimBlueprint())
		{
			Data.PostProcessAnimInst = PostProcessAnimInstance;
		}
		
		// Shadow Physics Asset
		if (UPhysicsAsset* PhysicsAsset = RefSkeletalMesh->GetShadowPhysicsAsset())
		{
			Data.ShadowPhysicsAsset = PhysicsAsset;
		}
	}
}


uint32 GetBaseTextureSize(const FMutableGraphGenerationContext& GenerationContext, const UCONodeSkeletalMeshSection* MeshSectionNode, const UE::Mutable::Private::FParameterKey& ImageKey)
{
	const FGeneratedImageProperties* ImageProperties = GenerationContext.ImageProperties.Find({ MeshSectionNode, ImageKey });
	return ImageProperties ? ImageProperties->TextureSize : 0;
}


// Find the LODBias to apply to stay within the MaxTextureSize limit of the TargetPlatform
int32 GetPlatformLODBias(int32 TextureSize, int32 NumMips, int32 MaxPlatformSize)
{
	if (MaxPlatformSize > 0 && MaxPlatformSize < TextureSize)
	{
		const int32 MaxMipsAllowed = FMath::CeilLogTwo(MaxPlatformSize) + 1;
		return NumMips - MaxMipsAllowed;
	}

	return 0;
}


uint32 ComputeLODBiasForTexture(const FMutableGraphGenerationContext& GenerationContext, const UTexture2D& Texture, const UTexture2D* ReferenceTexture, int32 BaseTextureSize)
{
	constexpr int32 MaxAllowedLODBias = 6;

	// Force a large LODBias for debug
	if (GenerationContext.CompilationContext->Options.bForceLargeLODBias)
	{
		return FMath::Min(GenerationContext.CompilationContext->Options.DebugBias, MaxAllowedLODBias);
	}

	// Max size and number of mips from Texture. 
	const int32 SourceSize = (int32)FMath::Max3(Texture.Source.GetSizeX(),Texture.Source.GetSizeY(),(int64)1);
	const int32 NumMipsSource = FMath::CeilLogTwo(SourceSize) + 1;

	// When the BaseTextureSize is known, skip mips until the texture is equal or smaller.
	if (BaseTextureSize > 0)
	{
		if (BaseTextureSize < SourceSize)
		{
			const int32 MaxNumMipsInGame = FMath::CeilLogTwo(BaseTextureSize) + 1;
			return FMath::Max(NumMipsSource - MaxNumMipsInGame, 0);
		}

		return 0;
	}

	const UTextureLODSettings& LODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();

	// Get the MaxTextureSize for the TargetPlatform.
	const int32 MaxTextureSize = GetMaxTextureSize(ReferenceTexture ? *ReferenceTexture : Texture, LODSettings);

	if (ReferenceTexture)
	{
		// Max size and number of mips from ReferenceTexture. 
		const int32 MaxRefSourceSize = (uint32)FMath::Max3(ReferenceTexture->Source.GetSizeX(), ReferenceTexture->Source.GetSizeY(), (int64)1);
		const int32 NumMipsRefSource = FMath::CeilLogTwo(MaxRefSourceSize) + 1;

		// Find the LODBias to apply to stay within the MaxTextureSize limit of the TargetPlatform
		const int32 PlatformLODBias = GetPlatformLODBias(MaxRefSourceSize, NumMipsRefSource, MaxTextureSize);

		// TextureSize in-game without any additional LOD bias.
		const int64 ReferenceTextureSize = MaxRefSourceSize >> PlatformLODBias;

		// Additional LODBias of the Texture
		const int32 ReferenceTextureLODBias = LODSettings.CalculateLODBias(ReferenceTextureSize, ReferenceTextureSize, 0, 0, ReferenceTexture->LODGroup,
			ReferenceTexture->LODBias, 0, ReferenceTexture->MipGenSettings, ReferenceTexture->IsCurrentlyVirtualTextured(), ReferenceTexture->IsStreamable());

		return FMath::Max(NumMipsSource - NumMipsRefSource + PlatformLODBias + ReferenceTextureLODBias, 0);
	}

	// Find the LODBias to apply to stay within the MaxTextureSize limit of the TargetPlatform
	const int32 PlatformLODBias = GetPlatformLODBias(SourceSize, NumMipsSource, MaxTextureSize);

	// TextureSize in-game without any additional LOD bias.
	const int64 TextureSize = SourceSize >> PlatformLODBias;

	// Additional LODBias of the Texture
	const int32 TextureLODBias = LODSettings.CalculateLODBias(TextureSize, TextureSize, 0, 0, Texture.LODGroup, Texture.LODBias, 0, Texture.MipGenSettings, Texture.IsCurrentlyVirtualTextured(), Texture.IsStreamable());

	return FMath::Max(PlatformLODBias + TextureLODBias, 0);
}


int32 GetMaxTextureSize(const UTexture2D& ReferenceTexture, const UTextureLODSettings& LODSettings)
{
	// Setting the maximum texture size
	FTextureLODGroup TextureGroupSettings = LODSettings.GetTextureLODGroup(ReferenceTexture.LODGroup);

	if (TextureGroupSettings.MaxLODSize > 0)
	{
		return ReferenceTexture.MaxTextureSize == 0 ? TextureGroupSettings.MaxLODSize : FMath::Min(TextureGroupSettings.MaxLODSize, ReferenceTexture.MaxTextureSize);
	}

	return ReferenceTexture.MaxTextureSize;
}


int32 GetTextureSizeInGame(const UTexture2D& Texture, const UTextureLODSettings& LODSettings)
{
	const int32 SourceSize = (uint32)FMath::Max3(Texture.Source.GetSizeX(), Texture.Source.GetSizeY(), (int64)1);
	const int32 NumMipsSource = FMath::CeilLogTwo(SourceSize) + 1;

	// Max size allowed on the TargetPlatform
	const int32 MaxTextureSize = GetMaxTextureSize(Texture, LODSettings);

	// Find the LODBias to apply to stay within the MaxTextureSize limit of the TargetPlatform
	const int32 PlatformLODBias = GetPlatformLODBias(SourceSize, NumMipsSource, MaxTextureSize);
	
	// MaxTextureSize in-game without any additional LOD bias.
	const int32 MaxTextureSizeAllowed = SourceSize >> PlatformLODBias;

	// Calculate the LODBias specific for this texture 
	const int32 TextureLODBias = LODSettings.CalculateLODBias(MaxTextureSizeAllowed, MaxTextureSizeAllowed, 0, 0, Texture.LODGroup, Texture.LODBias, 0, Texture.MipGenSettings, Texture.IsCurrentlyVirtualTextured(), Texture.IsStreamable());

	return MaxTextureSizeAllowed >> TextureLODBias;
}


void GetCookedTextureSize(const FMutableGraphGenerationContext& GenerationContext, const UTexture2D* Texture, uint32& OutTextureSizeX, uint32& OutTextureSizeY, bool bSkipCinematicMips)
{
	if (!Texture)
	{
		ensureAlwaysMsgf(false, TEXT("Called GetCookedTextureSize for null texture."));
		OutTextureSizeX = 0;
		OutTextureSizeY = 0;
		return;
	}

	const ITargetPlatform* TargetPlatform = GenerationContext.CompilationContext->Options.TargetPlatform;

	// We replicate the same two-step computation the Texture Editor's Platforms Sizes.
	FIntPoint CookedTopMip(Texture->Source.GetSizeX(), Texture->Source.GetSizeY());

	TArray<TArray<FTextureBuildSettings>> BuildSettingsPerFormatPerLayer;
	Texture->GetTargetPlatformBuildSettings(TargetPlatform, BuildSettingsPerFormatPerLayer);

	if (!BuildSettingsPerFormatPerLayer.IsEmpty() && !BuildSettingsPerFormatPerLayer[0].IsEmpty())
	{
		// Layer 0 of the first supported format, matching the Platforms tab (Settings[0][0]).
		const FTextureBuildSettings& BuildSettings = BuildSettingsPerFormatPerLayer[0][0];

		int32 OutSizeX = 0, OutSizeY = 0, OutNumSlices = 0, OutMipCount = 0;
		if (BuildSettings.GetOutputMipInfo(
			Texture->Source.GetSizeX(), Texture->Source.GetSizeY(),
			Texture->Source.GetNumSlices(), Texture->Source.GetNumMips(),
			OutSizeX, OutSizeY, OutNumSlices, OutMipCount))
		{
			CookedTopMip.X = OutSizeX;
			CookedTopMip.Y = OutSizeY;
		}
	}

	// Pass NumCinematicMipLevels = 0 to match the Platforms tab. Cinematic mips are cooked and serialized,
	// so they are part of the on-disk top mip and must not be stripped from the reported size here.
	const int32 LODBias = TargetPlatform->GetTextureLODSettings().CalculateLODBias(
		CookedTopMip.X, CookedTopMip.Y,
		Texture->GetMinTextureSize(), Texture->GetMaxTextureSize(),
		Texture->LODGroup, Texture->LODBias,
		bSkipCinematicMips ? Texture->NumCinematicMipLevels : 0, // 0 will return cooked texture size with cinematic mips
		Texture->MipGenSettings,
		false, Texture->IsStreamable());

	OutTextureSizeX = FMath::Max(CookedTopMip.X >> LODBias, 1);
	OutTextureSizeY = FMath::Max(CookedTopMip.Y >> LODBias, 1);
}


void GetLayoutBlockSizeInPixels(const FMutableGraphGenerationContext& GenerationContext, const UTexture2D* ReferenceTexture, const int32 NumBlocksX, const int32 NumBlocksY, uint16& BlockSizeX, uint16& BlockSizeY)
{
	BlockSizeX = 0;
	BlockSizeY = 0;

	if (ReferenceTexture && NumBlocksX > 0 && NumBlocksY > 0)
	{
		const uint32 LODBias = ComputeLODBiasForTexture(GenerationContext, *ReferenceTexture);

		int32 TextureSizeX = FMath::Max(ReferenceTexture->Source.GetSizeX() >> LODBias, 1);
		int32 TextureSizeY = FMath::Max(ReferenceTexture->Source.GetSizeY() >> LODBias, 1);

		BlockSizeX = FMath::DivideAndRoundUp(TextureSizeX, NumBlocksX);
		BlockSizeY = FMath::DivideAndRoundUp(TextureSizeY, NumBlocksY);
	}
}


UE::Mutable::Private::FImageDesc GenerateImageDescriptor(UTexture* Texture)
{
	check(Texture);
	UE::Mutable::Private::FImageDesc ImageDesc;

	ImageDesc.m_size[0] = Texture->Source.GetSizeX();
	ImageDesc.m_size[1] = Texture->Source.GetSizeY();
	ImageDesc.m_lods = Texture->Source.GetNumMips();

	UE::Mutable::Private::EImageFormat MutableFormat = UE::Mutable::Private::EImageFormat::RGBA_UByte;
	ETextureSourceFormat SourceFormat = Texture->Source.GetFormat();
	switch (SourceFormat)
	{
	case ETextureSourceFormat::TSF_G8:
	case ETextureSourceFormat::TSF_G16:
	case ETextureSourceFormat::TSF_R16F:
	case ETextureSourceFormat::TSF_R32F:
		MutableFormat = UE::Mutable::Private::EImageFormat::L_UByte;
		break;

	default:
		break;
	}

	ImageDesc.m_format = MutableFormat;

	return ImageDesc;
}


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> GenerateImageConstant(UTexture* Texture, FMutableGraphGenerationContext& GenerationContext, bool bIsReference)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateImageConstant);

	if (!Texture)
	{
		return nullptr;
	}

	bool bForceLoad = false;
	bool bIsCompileTime = false;
	if (!bIsReference)
	{
		bForceLoad = true;
		if (GenerationContext.CompilationContext->Options.OptimizationLevel == 0)
		{
			bIsCompileTime = false;
		}
		else
		{
			bIsCompileTime = true;
		}
	}

	// Create a descriptor for the image.
	// \TODO: If passthrough (bIsReference) we should apply lod bias, and max texture size to this desc.
	// For now it is not a problem because passthrough textures shouldn't mix with any other operations.
	UE::Mutable::Private::FImageDesc ImageDesc = GenerateImageDescriptor(Texture);

	UE::Mutable::Private::PASSTHROUGH_ID Id;

	if (bIsReference)
	{
		Id = GenerationContext.CompilationContext->PassthroughObjectFactory.Add(*Texture, false);
	}
	else if (bIsCompileTime)
	{
		FMutableGraphGenerationContext::FGeneratedReferencedTexture InvalidEntry;
		InvalidEntry.ID = UE::Mutable::Private::PASSTHROUGH_ID_INVALID;

        FMutableGraphGenerationContext::FGeneratedReferencedTexture* Entry = &GenerationContext.CompileTimeTextureMap.FindOrAdd(CastChecked<UTexture2D>(Texture), InvalidEntry);
		int32 Num = GenerationContext.CompileTimeTextureMap.Num();

		if (Entry->ID == UE::Mutable::Private::PASSTHROUGH_ID_INVALID)
		{
			Entry->ID = Num - 1;
		}

		Id = Entry->ID;
	}
	else
	{
		FMutableGraphGenerationContext::FGeneratedReferencedTexture InvalidEntry;
		InvalidEntry.ID = UE::Mutable::Private::PASSTHROUGH_ID_INVALID;

		FMutableGraphGenerationContext::FGeneratedReferencedTexture* Entry = &GenerationContext.RuntimeReferencedTextureMap.FindOrAdd(CastChecked<UTexture2D>(Texture), InvalidEntry);
		int32 Num = GenerationContext.RuntimeReferencedTextureMap.Num();

		if (Entry->ID == UE::Mutable::Private::PASSTHROUGH_ID_INVALID)
		{
			Entry->ID = Num - 1;
		}

		Id = Entry->ID;
	}
	
	// Compile-time references that are left should be resolved immediately (should only happen in editor).
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> Result = UE::Mutable::Private::FImage::CreateAsReference(UE::Mutable::Private::TPassthroughObjectPtr<UTexture>(Id), ImageDesc, bForceLoad); // TODO GMT If bIsCompileTime it should not create a TPassthroughObjectPtr since it is not a Passthrough
	return Result;
}


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> GenerateMeshConstant(
	const FMutableSourceMeshData& Source,
	FMutableGraphGenerationContext& GenerationContext)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateMeshConstant)

	if (Source.Mesh.IsNull())
	{
		return nullptr;
	}

	UE::Mutable::Private::PASSTHROUGH_ID Id;

	FMutableGraphGenerationContext::FGeneratedReferencedMesh InvalidEntry;
	InvalidEntry.ID = UE::Mutable::Private::PASSTHROUGH_ID_INVALID;

	FMutableGraphGenerationContext::FGeneratedReferencedMesh* Entry = &GenerationContext.CompileTimeMeshMap.FindOrAdd(Source, InvalidEntry);
	int32 Num = GenerationContext.CompileTimeMeshMap.Num();

	if (Entry->ID == UE::Mutable::Private::PASSTHROUGH_ID_INVALID)
	{
		Entry->ID = Num - 1;
	}

	Id = Entry->ID;

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> Result = UE::Mutable::Private::FMesh::CreateAsReference(UE::Mutable::Private::TPassthroughObjectPtr<USkeletalMesh>(Id));
	return Result;
}

#undef LOCTEXT_NAMESPACE


FGeneratedMutableDataTableKey::FGeneratedMutableDataTableKey(FString TableName, FName VersionColumn, const TArray<FTableNodeCompilationFilter>& CompilationFilterOptions)
	: TableName(TableName), VersionColumn(VersionColumn), CompilationFilterOptions(CompilationFilterOptions)
{}


uint32 GetTypeHash(const FGeneratedMutableDataTableKey& Key)
{
	uint32 Hash = GetTypeHash(Key.TableName);
	Hash = HashCombine(Hash, GetTypeHash(Key.VersionColumn));

	for (int32 FilterIndex = 0; FilterIndex < Key.CompilationFilterOptions.Num(); ++FilterIndex)
	{
		Hash = HashCombine(Hash, GetTypeHash(Key.CompilationFilterOptions[FilterIndex].FilterColumn));

		for (int32 NameIndex = 0; NameIndex < Key.CompilationFilterOptions[FilterIndex].Filters.Num(); ++NameIndex)
		{
			Hash = HashCombine(Hash, GetTypeHash(Key.CompilationFilterOptions[FilterIndex].Filters[NameIndex]));
		}

		Hash = HashCombine(Hash, GetTypeHash(Key.CompilationFilterOptions[FilterIndex].OperationType));
	}

	return Hash;
}


FSharedSurfaceKey::FSharedSurfaceKey(const UCONodeSkeletalMeshSection* NodeMaterial, const TArray<const UCustomizableObjectNodeMacroInstance*>& MacroContext)
{
	check(NodeMaterial);

	MaterialGuid = NodeMaterial->NodeGuid;

	MacroContextGuid = FGuid();
	for (const UCustomizableObjectNodeMacroInstance* Macro : MacroContext)
	{
		if (ensure(Macro))
		{
			MacroContextGuid = FGuid::Combine(MacroContextGuid, Macro->NodeGuid);
		}
	}
}


uint32 GetTypeHash(const FSharedSurfaceKey& Key)
{
	return HashCombine(GetTypeHash(Key.MaterialGuid), GetTypeHash(Key.MacroContextGuid));
}


