// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/Skeleton.h"
#include "Animation/AnimInstance.h"
#include "Engine/DataTable.h"
#include "Engine/SkeletalMesh.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectIdentifier.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/ExtensionDataCompilerInterface.h"
#include "MuCOE/PassthroughObjectFactory.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuT/NodeComponentNew.h"
#include "MuT/NodeImageConstant.h"
#include "MuT/NodeImageMaterialBreak.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeMeshApplyPose.h"
#include "MuT/NodeSurfaceModifierMeshClipWithMesh.h"
#include "MuT/NodeObject.h"
#include "MuT/NodeProjector.h"
#include "MuT/NodeScalarEnumParameter.h"
#include "MuT/NodeScalarParameter.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeMaterialConstant.h"
#include "MuT/NodeMaterialParameter.h"
#include "MuT/NodeExternal.h"
#include "MuT/Table.h"
#include "MuR/Skeleton.h"
#include "MuR/Mesh.h"
#include "MuR/Ptr.h"
#include "MuT/NodeImageObjectParameter.h"
#include "MuT/NodeSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshObjectParameter.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "Tasks/Task.h"

class UCONodeComponentSkeletalMesh;
class UCustomizableObjectNodeGroupProjectorParameter;
class UCONodeSkeletalMeshSection;
class UCONodeSwitch;
class UCustomizableObjectNodeModifierClipWithMesh;

class FCustomizableObjectCompiler;
class UTextureLODSettings;
class UAnimInstance;
class UCompositeDataTable;
class UCONodeMaterialConstant;
class UCustomizableObjectNodeMacroInstance;
class UCONodeSkeletalMeshSection;
class UCustomizableObjectNodeMaterialParameter;
class UCustomizableObjectNodeMeshMorph;
class UCustomizableObjectNodeMeshMorphStackDefinition;
class UCustomizableObjectNodeObjectGroup;
class UCustomizableObjectNodeVariation;
class UEdGraphNode;
class UMaterialInterface;
class UObject;
class UPhysicsAsset;
class UTexture2D;
struct FAnimBpOverridePhysicsAssetsInfo;
struct FMutableGraphGenerationContext;
struct FMutableParameterData;
struct FMutableRefSkeletalMeshData;
struct FMorphTargetVertexData;
enum class EPinMode;
enum class ETableCompilationFilterOperationType : uint8;

struct FGeneratedImageProperties
{
	/** Name in the Material. */
	FString TextureParameterName;

	/** Name in the UE::Mutable::Private::Surface. */
	int32 ImagePropertiesIndex = INDEX_NONE;

	TEnumAsByte<TextureCompressionSettings> CompressionSettings = TC_Default;

	TEnumAsByte<TextureFilter> Filter = TF_Bilinear;

	uint32 SRGB = 0;

	uint32 bFlipGreenChannel = 0;

	int32 LODBias = 0;

	TEnumAsByte<TextureMipGenSettings> MipGenSettings = TMGS_SimpleAverage;

	int32 MaxTextureSize = 0;

	TEnumAsByte<TextureGroup> LODGroup = TEnumAsByte<TextureGroup>(TMGS_FromTextureGroup);

	TEnumAsByte<TextureAddress> AddressX = TA_Clamp;
	TEnumAsByte<TextureAddress> AddressY = TA_Clamp;

	bool bIsPassThrough = false;

	// ReferenceTexture source size.
	int32 TextureSize = 0;

	// TODO: Remove base LODBias added to root.
	int32 BaseLODBias = 0;

};

struct FLayoutGenerationFlags
{
	bool operator==(const FLayoutGenerationFlags& Other) const = default;

	// Texture pin mode per UV Channel
	TArray<EPinMode> TexturePinModes;
};

/** 
	Struct to store the necessary data to generate the morphs of a skeletal mesh 
	This struct allows the stack morph nodes to use the same functions as the mesh morph nodes
*/
struct FMorphNodeData
{
	// Pointer to the node that owns this morph data
	UCustomizableObjectNode* OwningNode;

	// Name of the morph that will be applied
	FName MorphTargetName;

	// Pin to the node that generates the factor of the morph
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FactorNode;

	// Pin of the mesh where the morphs will ble applied
	const UEdGraphPin* MeshPin;

	bool operator==(const FMorphNodeData& Other) const
	{
		return OwningNode == Other.OwningNode && MorphTargetName == Other.MorphTargetName
			&& FactorNode == Other.FactorNode && MeshPin == Other.MeshPin;
	}
};


/** Representation of the root of a Material Break node compilation branch. */
struct FMaterialBreakParameter 
{
	/** Name and Layer Index of the material parameter. */
	UE::Mutable::Private::FParameterKey ParameterKey;
	
	/** The type of data this parameter relates to. */
	FName ParameterType;

	bool operator==(const FMaterialBreakParameter& OtherItem) const = default;
};


// Key for the data stored for each processed unreal graph node.
class FGeneratedKey
{
	friend uint32 GetTypeHash(const FGeneratedKey& Key);

public:
	FGeneratedKey(void* InFunctionAddress, const UEdGraphPin& InPin, const UCustomizableObjectNode& Node, FMutableGraphGenerationContext& GenerationContext, const bool UseMesh = false, bool bOnlyConnectedLOD = false, const bool bUseMaterialBreakStack = false);
	
	bool operator==(const FGeneratedKey& Other) const = default;

	/** Used to differentiate pins being cached from different functions (e.g. a PC_Color pin cached from GenerateMutableSourceImage and GenerateMutableSourceColor). */
	void* FunctionAddress;
	
	const UEdGraphPin* Pin;
	
	int32 LOD;
	int32 FromLOD;

	/** Flag used to generate this mesh. Bit mask of EMutableMeshConversionFlags */
	EMutableMeshConversionFlags Flags = EMutableMeshConversionFlags::None;

	/** UV Layout modes */
	FLayoutGenerationFlags LayoutFlags;
	
	UE::Mutable::Private::FComponentId CurrentMeshComponent = INDEX_NONE;
	
	/** When caching a generated mesh, true if we force to generate the connected LOD when using Automatic LODs From Mesh. */
	bool bOnlyConnectedLOD = false;

	/** Pointer to control if this is a node inside a Mutable Macro. */
	TArray<const UCustomizableObjectNodeMacroInstance*> MacroContext;

	/** Stack of Material Break Parameters. It may contain one or multiple entries where the last one is the Material Break Parameter being processed
	 * while compiling the branches of the graph.
	 */
	TArray<FMaterialBreakParameter> MaterialBreakParameterStack;
	
	int32 ReferenceTextureSize = -1;

	/** Name of the parameter. A parameter node can be redefined if it is inside a macro and the name comes from outside the graph. */
	FString ParameterName;
};


uint32 GetTypeHash(const FMaterialBreakParameter& Key);

uint32 GetTypeHash(const FGeneratedKey& Key);

// Cache key of the generated parameter nodes
struct FGeneratedParameterKey
{
	// ID of the node
	FGuid NodeId;

	// Name of the parameter.
	FString ParameterName;

	bool operator==(const FGeneratedParameterKey& Other) const = default;
};

uint32 GetTypeHash(const FGeneratedParameterKey& Key);


// Struct used for external options of the GenerateMutableSourceX functions.
struct FSourceExternalOptions
{
	// GenerateMutableSourceMesh
	FMutableSourceMeshData MeshOptions;
	bool bLinkedToExtendMaterial = false;
	bool bOnlyConnectedLOD = false;
	int8 CurrentLOD = 0;

	// GenerateMutableSourceImage
	int32 ReferenceTextureSize = 0;

	// GenerateMutableSourceMaterial
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurfaceNew> SurfaceNode = nullptr;

	bool operator==(const FSourceExternalOptions& Other) const = default;
};


struct FGeneratedSourceExternalKey
{
	const UEdGraphPin* Pin = nullptr;
	
	FSourceExternalOptions Options;

	bool operator==(const FGeneratedSourceExternalKey& Other) const = default;
};


uint32 GetTypeHash(const FGeneratedSourceExternalKey& Key);


struct FSourceSkeletalMeshObjectOptions
{
	USkeletalMesh* ReferenceSkeletalMesh = nullptr;
	
	bool operator==(const FSourceSkeletalMeshObjectOptions& Other) const = default;
};


uint32 GetTypeHash(const FSourceSkeletalMeshObjectOptions& Key);


struct FSourceSkeletalMeshOptions
{
	int32 NumLODs = 0;
	
	int32 FirstLODAvailable = 0;
	
	bool operator==(const FSourceSkeletalMeshOptions& Other) const = default;
};


uint32 GetTypeHash(const FSourceSkeletalMeshOptions& Key);


struct FGeneratedSourceSkeletalMeshKey
{
	const UEdGraphPin* Pin = nullptr;
	
	FSourceSkeletalMeshOptions Options;

	/** Pointer to control if this is a node inside a Mutable Macro. */
	TArray<const UCustomizableObjectNodeMacroInstance*> MacroContext;

	bool operator==(const FGeneratedSourceSkeletalMeshKey& Other) const = default;
};


uint32 GetTypeHash(const FGeneratedSourceSkeletalMeshKey& Key);


struct FGeneratedSourceSkeletalMeshObjectKey
{
	const UEdGraphPin* Pin = nullptr;
	
	FSourceSkeletalMeshObjectOptions Options;
	
	/** Pointer to control if this is a node inside a Mutable Macro. */
	TArray<const UCustomizableObjectNodeMacroInstance*> MacroContext;

	bool operator==(const FGeneratedSourceSkeletalMeshObjectKey& Other) const = default;
};


uint32 GetTypeHash(const FGeneratedSourceSkeletalMeshObjectKey& Key);


struct FImageKey
{
	FImageKey(const UEdGraphPin* InPin)
	{
		Pin = InPin;
	}

	bool operator==(const FImageKey& Other) const
	{
		return Pin == Other.Pin;
	}

	const UEdGraphPin* Pin;
};


struct FGeneratedImagePropertiesKey
{
	FGeneratedImagePropertiesKey(const UCustomizableObjectNode* InMaterial, UE::Mutable::Private::FParameterKey InImageKey)
	{
		MaterialReferenceId = (PTRINT)InMaterial;
		ImageKey = InImageKey;
	}

	bool operator==(const FGeneratedImagePropertiesKey& Other) const
	{
		return MaterialReferenceId == Other.MaterialReferenceId && ImageKey == Other.ImageKey;
	}


	PTRINT MaterialReferenceId = 0;
	UE::Mutable::Private::FParameterKey ImageKey;
};

// Structure storing results to propagate up when generating mutable mesh node expressions.
struct FMutableGraphMeshGenerationData
{
	// Did we find any mesh with vertex colors in the expression?
	bool bHasVertexColors = false;
	bool bHasRealTimeMorphs = false;
	bool bHasClothing = false;

	// Maximum number of texture channels found in the expression.
	int32 NumTexCoordChannels = 0;

	// Maximum number of bones per vertex found in the expression.
	int32 MaxNumBonesPerVertex = 0;

	// Maximum size of the vertex bone index type in the expression.
	int32 MaxBoneIndexTypeSizeBytes = 0;

	int32 MaxNumTriangles = 0;
	int32 MinNumTriangles = TNumericLimits<int32>::Max();

	TArray<int32> SkinWeightProfilesSemanticIndices;
};


// Data stored for each processed unreal graph node, stored in the cache.
struct FGeneratedData
{
	FGeneratedData(const UEdGraphNode* InSource, const TArray<UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeModifier>>& InModifierNodes)
	{
		Source = InSource;
		ModifierNodes = InModifierNodes;
	}
	
	FGeneratedData(const UEdGraphNode* InSource, UE::Mutable::Private::NodePtr InNode)
	{
		Source = InSource;
		Node = InNode;
	}

	const UEdGraphNode* Source;
	UE::Mutable::Private::NodePtr Node;
	TArray<UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeModifier>> ModifierNodes;
};


struct FGeneratedSourceExternalData
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternal> Node;
};


struct FGeneratedSourceSkeletalMeshData
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSkeletalMesh> Node;
};


struct FGeneratedSourceSkeletalMeshObjectData
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSkeletalMeshObject> Node;
};


inline uint32 GetTypeHash(const FImageKey& Key)
{
	uint32 GuidHash = GetTypeHash(Key.Pin->PinId);

	return GuidHash;
}


inline uint32 GetTypeHash(const FGeneratedImagePropertiesKey& Key)
{
	uint32 GuidHash = HashCombineFast(GetTypeHash(Key.MaterialReferenceId), GetTypeHash(Key.ImageKey));

	return GuidHash;
}

struct FPoseBoneData
{
	TArray<FName> ArrayBoneName;
	TArray<FTransform> ArrayTransform;
};

struct FGroupProjectorTempData
{
	class UCustomizableObjectNodeGroupProjectorParameter* CustomizableObjectNodeGroupProjectorParameter;
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjectorParameter> NodeProjectorParameterPtr;
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> NodeImagePtr;
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeRange> NodeRange;
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter> NodeOpacityParameter;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter> PoseOptionsParameter;
	TArray<FPoseBoneData> PoseBoneDataArray;

	bool bAlternateResStateNameWarningDisplayed = false; // Used to display this warning only once

	int32 TextureSize = 512;
};


struct FGroupNodeIdsTempData
{
	FGroupNodeIdsTempData(FGuid OldGuid, FGuid NewGuid = FGuid()) :
		OldGroupNodeId(OldGuid),
		NewGroupNodeId(NewGuid)
	{

	}

	FGuid OldGroupNodeId;
	FGuid NewGroupNodeId;

	bool operator==(const FGroupNodeIdsTempData& Other) const
	{
		return OldGroupNodeId == Other.OldGroupNodeId;
	}
};


/** Struct used to store info specific to each component during compilation */
struct FSkeletalMeshComponentInfo
{
	UE::Mutable::Private::FComponentId ComponentId = INDEX_NONE;
	
	// Each component must have a reference SkeletalMesh with a valid Skeleton
	TStrongObjectPtr<USkeletalMesh> RefSkeletalMesh;
	
	UCONodeComponentSkeletalMesh* Node = nullptr;
};


/** Graph cycle key.
 *
 * Pin is not enough since we can call multiple recursive functions with the same pin.
 * Each function has to have an unique identifier.
 */
struct FGraphCycleKey
{
	friend uint32 GetTypeHash(const FGraphCycleKey& Key);

	FGraphCycleKey(const UEdGraphPin& Pin, const FString& Id, const UCustomizableObjectNodeMacroInstance* MacroContext);

	bool operator==(const FGraphCycleKey& Other) const;
	
	/** Valid pin. */
	const UEdGraphPin& Pin;

	/** Unique id. */
	FString Id;

	const UCustomizableObjectNodeMacroInstance* MacroContext;
};

/** Graph Cycle scope.
 *
 * Detect a cycle during the graph traversal.
 */
class FGraphCycle
{
public:
	explicit FGraphCycle(const FGraphCycleKey&& Key, FMutableGraphGenerationContext &Context);
	~FGraphCycle();

	/** Return true if there is a cycle. */
	bool FoundCycle() const;
	
private:
	/** Graph traversal key. */
	FGraphCycleKey Key;

	/** Generation context. */
	FMutableGraphGenerationContext& Context;
};

/** Return the default value if there is a cycle. */
#define RETURN_ON_CYCLE(Pin, GenerationContext) \
	FGraphCycle GraphCycle(FGraphCycleKey(Pin, TEXT(__FILE__ UE_STRINGIZE(__LINE__)),GenerationContext.MacroNodesStack.Num() ? GenerationContext.MacroNodesStack.Top() : nullptr), GenerationContext); \
	if (GraphCycle.FoundCycle()) \
	{ \
		return {}; \
	} \



struct FGeneratedGroupProjectorsKey
{
	UCustomizableObjectNodeGroupProjectorParameter* Node = nullptr;
	UE::Mutable::Private::FComponentId CurrentComponent;

	bool operator==(const FGeneratedGroupProjectorsKey&) const = default;
};

uint32 GetTypeHash(const FGeneratedGroupProjectorsKey& Key);


/** This structure stores the information that is used during the CustomizableObject compilation. 
* This includes graph generation, core compilation and data storage.
* This context should have nothing to do with CO-level nodes.
* It should only be accessed from the game thread.
*/
struct FMutableCompilationContext
{
	/** */
	TStrongObjectPtr<const UCustomizableObject> RootObject;
		
	/** Compilation options, including target platform. */
	FCompilationOptions Options;

	/** Global morph selection overrides. */
	TArray<FRealTimeMorphSelectionOverride> RealTimeMorphTargetsOverrides;

	/** Stores the physics assets gathered from the SkeletalMesh nodes during compilation, to be used in mesh generation in-game */
	TArray<TSoftObjectPtr<UPhysicsAsset>> PhysicsAssets;

	TArray<FAnimBpOverridePhysicsAssetsInfo> AnimBpOverridePhysicsAssetsInfo;

	
	/** Only Mesh Components (no passthrough). */
	TArray<FSkeletalMeshComponentInfo> ComponentInfos;

	UE::Mutable::Private::FPassthroughObjectFactory PassthroughObjectFactory;

private:

	/** Non-owned reference to the compiler object */
	TWeakPtr<FCustomizableObjectCompiler> WeakCompiler;

public:

	FMutableCompilationContext(const UCustomizableObject* InRootObject, const TSharedPtr<FCustomizableObjectCompiler>& InCompiler, const FCompilationOptions& InOptions);

	/** Return the name of the current customizable object, for logging purposes. */
	FString GetObjectName() const;

	/** Message logging */
	void Log(const FText& Message, const TArray<const UObject*>& UObject, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const bool bAddBaseObjectInfo = true, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll) const;
	void Log(const FText& Message, const UObject* Context = nullptr, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const bool bAddBaseObjectInfo = true, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll) const;

	/** Component access (for Mesh components only). */
	FSkeletalMeshComponentInfo* GetComponentInfo(const UCONodeComponentSkeletalMesh* NodeComponentMesh);

	FSkeletalMeshComponentInfo* GetComponentInfo(UE::Mutable::Private::FComponentId ComponentName);

};


struct FGeneratedMutableDataTableKey
{
	FGeneratedMutableDataTableKey(FString TableName, FName VersionColumn, const TArray<FTableNodeCompilationFilter>& CompilationFilterOptions);

	// Name of the Data Table Asset
	FString TableName;

	// Compilation Restrictions:
	// Name of the column that determines de version control
	FName VersionColumn;

	// Compilation Filters
	TArray<FTableNodeCompilationFilter> CompilationFilterOptions;

	bool operator==(const FGeneratedMutableDataTableKey&) const = default;
};

uint32 GetTypeHash(const FGeneratedMutableDataTableKey& Key);

struct FSharedSurfaceKey
{
	FSharedSurfaceKey(const UCONodeSkeletalMeshSection* NodeMaterial, const TArray<const UCustomizableObjectNodeMacroInstance*>& CurrentMacroContext);

	bool operator==(const FSharedSurfaceKey& o) const = default;

	FGuid MaterialGuid;
	FGuid MacroContextGuid;
};

uint32 GetTypeHash(const FSharedSurfaceKey& Key);


/** This structure stores the information that is used only during the graph generation stage of the CustomizableObject compilation. 
* Data needed for node graph processing goes here.
*/
struct FMutableGraphGenerationContext
{
	FMutableGraphGenerationContext(const TSharedPtr<FMutableCompilationContext>& InCompilationContext);
	
	TSharedPtr<FMutableCompilationContext> CompilationContext;

	/** Full hierarchy root. */
	UCustomizableObjectNodeObject* Root = nullptr;
	
	// Cache of generated pins per LOD
	TMap<FGeneratedKey, FGeneratedData> Generated;
	TMap<FGeneratedSourceExternalKey, FGeneratedSourceExternalData> GeneratedSourceExternals;
	TMap<FGeneratedSourceSkeletalMeshKey, FGeneratedSourceSkeletalMeshData> GeneratedSourceSkeletalMesh;
	TMap<FGeneratedSourceSkeletalMeshObjectKey, FGeneratedSourceSkeletalMeshObjectData> GeneratedSourceSkeletalMeshObject;

	/** Set of all generated nodes. */
	TSet<UCustomizableObjectNode*> GeneratedNodes;

	/** Struct that stores the relevant information of a data table generated during the compilation. 
	* e.g. all data tables must have the same compilation restrictions 
	*/
	struct FGeneratedDataTablesData
	{
		// Pointer to the generated mutable Table
		UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> GeneratedTable;

		// Table Node used to fill this info
		const UCustomizableObjectNodeTable* ReferenceNode;

		// Stores the names of the rows that will be compiled
		TArray<FName> RowNames;
		TArray<uint32> RowIds;
	};

	// Cache of generated Node Tables
	TMap<FGeneratedMutableDataTableKey, FGeneratedDataTablesData> GeneratedTables;

	TMap<FGeneratedGroupProjectorsKey, FGroupProjectorTempData> GeneratedGroupProjectors;

	/** Key is the Node Uid and the parameter name. */
	TMap<FGeneratedParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter>> GeneratedScalarParameters;

	/** Key is the Node Uid and the parameter name. */
	TMap<FGeneratedParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter>> GeneratedEnumParameters;

	/** Key is the Node Uid and the parameter name. */
	TMap<FGeneratedParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter>> GeneratedImageParameters;

	/** Key is the Node Uid and the parameter name. */
	TMap<FGeneratedParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSkeletalMeshObjectParameter>> GeneratedSkeletalMeshParameters;

	/** Key is the Node Uid and the parameter name. */
	TMap<FGeneratedParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter>> GeneratedMaterialParameters;
	
	struct FGeneratedCompositeDataTablesData
	{
		UScriptStruct* ParentStruct = nullptr;
		TArray<FName> FilterPaths;
		UCompositeDataTable* GeneratedDataTable = nullptr;

		bool operator==(const FGeneratedCompositeDataTablesData& Other) const
		{
			return ParentStruct == Other.ParentStruct && FilterPaths == Other.FilterPaths;
		}
	};

	// Cache of generated Composited Data Tables
	TArray<FGeneratedCompositeDataTablesData> GeneratedCompositeDataTables;

	// Cache of generated images, because sometimes they are reused by LOD, we use this as a second
	// level cache
	TMap<FImageKey, UE::Mutable::Private::NodeImagePtr> GeneratedImages;

	/** Data stored per-generated passthrough texture. */
	struct FGeneratedReferencedTexture
	{
		UE::Mutable::Private::PASSTHROUGH_ID ID;
		//UE::Mutable::Private::FImageDesc ImageDesc;
	};

	/** Data stored per-generated passthrough mesh. */
	struct FGeneratedReferencedMesh
	{
		UE::Mutable::Private::PASSTHROUGH_ID ID;
	};

	struct FParamInfo
	{
		FString ParamName;
		bool bIsToggle = false;

		FParamInfo(const FString& InParamName, bool bInIsToggle) : ParamName(InParamName), bIsToggle(bInIsToggle) {}
	};
	
	// Cache of runtime pass-through meshes and their IDs used in the core to identify them
	// These meshes will become mutable meshes in the compiled model.
	TMap<FMutableSourceMeshData, FGeneratedReferencedMesh> CompileTimeMeshMap;

	// Cache of runtime images and their IDs used in the core to identify them.
	// These textures will remain as external references even in optimized models.
	TMap<TSoftObjectPtr<UTexture2D>, FGeneratedReferencedTexture> RuntimeReferencedTextureMap;
	
	// Cache of runtime pass-through images and their IDs used in the core to identify them
	// These textures will become mutable images in the compiled model.
	TMap<TSoftObjectPtr<UTexture2D>, FGeneratedReferencedTexture> CompileTimeTextureMap;
	
	// Mutable meshes already build for source UStaticMesh or USkeletalMesh.
	struct FGeneratedMeshData
	{
		struct FKey
		{
			/** Source mesh data. */
			TSoftObjectPtr<const UStreamableRenderAsset> Mesh;
			int32 LOD = 0; // Mesh Data LOD (i.e., LOD where we are getting the vertices from)
			int32 CurrentLOD = 0; // Derived data LOD (i.e., LOD where we are generating the non-Core Data like morphs)
			int32 FromLOD = 0; // Connected data LOD
			int32 MaterialIndex = 0;

			/** Flag used to generate this mesh. Bit mask of EMutableMeshConversionFlags */
			EMutableMeshConversionFlags Flags = EMutableMeshConversionFlags::None;

			/**
			* SkeletalMeshNode is needed to disambiguate realtime morph selection from different nodes.
			* TODO: Consider using the actual selection.
			*/
			const UCustomizableObjectNode* SkeletalMeshNode = nullptr;

			FGameplayTagContainer GameplayTags;
			FName AnimBPSlotName;
			TSoftClassPtr<UAnimInstance> AnimInstance;

			bool operator==(const FKey& OtherKey) const = default;
		};

		FKey Key;

		/** Generated mesh. */
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> Generated;
	};
	TArray<FGeneratedMeshData> GeneratedMeshes;

	struct FGeneratedTableImageData
	{
		FString PinName;
		FName PinType;
		const UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> Table;
		const UCustomizableObjectNodeTable* TableNode;

		bool operator==(const FGeneratedTableImageData& Other) const
		{
			return PinName == Other.PinName && Table == Other.Table;
		}
	};
	TArray<FGeneratedTableImageData> GeneratedTableImages;

	// Stack of mesh generation flags. The last one is the currently valid.
	// The value is a bit mask of EMutableMeshConversionFlags
	TArray<EMutableMeshConversionFlags> MeshGenerationFlags;

	// Stack of Layout generation flags. The last one is the currently valid.
	TArray<FLayoutGenerationFlags> LayoutGenerationFlags;

	/** Stack of Group Projector nodes. Each time a Group Object node is visited, a set of Group Projector nodes get pushed
	 * When a Mesh Section node is found, it will compile all Group Projector nodes in the stack. */
	TArray<TArray<UCustomizableObjectNodeGroupProjectorParameter*>> CurrentGroupProjectors;

	/** Message logging */
	FString GetObjectName() const;
	void Log(const FText& Message, const TArray<const UObject*>& Context, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const bool bAddBaseObjectInfo = true, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll) const;
	void Log(const FText& Message, const UObject* Context = nullptr, const EMessageSeverity::Type MessageSeverity = EMessageSeverity::Warning, const bool bAddBaseObjectInfo = true, const ELoggerSpamBin SpamBin = ELoggerSpamBin::ShowAll) const;

	/** Find a mesh if already generated for a given source and flags. */
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> FindGeneratedMesh(const FGeneratedMeshData::FKey& Key);

	// Check if the Id of the node Node already exists, if it's new adds it to NodeIds array, otherwise, returns new Id
	const FGuid GetNodeIdUnique(const UCustomizableObjectNode* Node);

	/** Same as GetNodeIdUnique but does not triggers any warning on repeated IDs.
	* Use only if GetNodeIdUnique is used in another part of the code.
	*/
	const FGuid GetNodeIdUnchecked(const UCustomizableObjectNode* Node);

	UObject* LoadObject(const FSoftObjectPtr& SoftObject);
	
	template<typename T>
	T* LoadObject(const TSoftObjectPtr<T>& SoftObject)
	{
		return UE::Mutable::Private::LoadObject<T>(SoftObject);
	}

	template<typename T>
	UClass* LoadClass(const TSoftClassPtr<T>& SoftClass)
	{
		return UE::Mutable::Private::LoadClass<T>(SoftClass);
	}

	/**
	 * Get the Current Material Break Parameter that roots the current CO node graph branch. 
	 * @return The current Material Break Parameter. Will return a default object if there is no Material Break Parameter in the current compilation scope.
	 */
	FMaterialBreakParameter GetCurrentMaterialBreakParameter();
	
	/** Only compiled components. All component types. Index is the ComponentId. */
	TArray<FName> ComponentNames;
	
	TArray<FMutableRefSkeletalMeshData> ReferenceSkeletalMeshesData;
	
	TMap<FGeneratedImagePropertiesKey, FGeneratedImageProperties> ImageProperties;
	TArray<const UCustomizableObjectNode*> NoNameNodeObjectArray;
	TMap<FString, FCustomizableObjectIdPair> GroupNodeMap;
	TMap<FString, FString> CustomizableObjectPathMap;
	TMap<FString, FMutableParameterData> ParameterUIDataMap;
	TMap<FString, FMutableStateData> StateUIDataMap;
	TMap<FIntegerParameterOptionKey, FIntegerParameterOptionDataTable> IntParameterOptionDataTable;


	// Used to aviod Nodes with duplicated ids
	TMap<FGuid, TArray<const UObject*>> NodeIdsMap;
	TMultiMap<const UCustomizableObject*, FGroupNodeIdsTempData> DuplicatedGroupNodeIds;
	
	uint8 FromLOD = 0; // LOD to append to the CurrentLOD when using AutomaticLODs. 
	uint8 CurrentLOD = 0;
	UE::Mutable::Private::FComponentId CurrentSkeletalMeshComponent = INDEX_NONE;

	/** If this is set, we are generating materials for a "passthrough" component, with a fixed mesh. */
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> ComponentMeshOverride;

	const uint8 MaxNumLODs = 16; // Ceiling. Core will discard unnecessary ones. 
	
	bool bPartialCompilation = false;

	// Stores external graph root nodes to be added to the specified group nodes
	TMultiMap<FGuid, UCustomizableObjectNodeObject*> GroupIdToExternalNodeMap;

	// Easily retrieve a parameter name from its node guid
	TMap<FGuid, FParamInfo> GuidToParamNameMap;

	// Graph cycle detection
	/** Visited nodes during the DAC recursion traversal.
	 * It acts like stack, pushing pins when recursively exploring a new pin an popping it when exiting the recursion. */
	TMap<FGraphCycleKey, const UCustomizableObject*> VisitedPins;
	const UCustomizableObject* CustomizableObjectWithCycle = nullptr;

	/** Stores the anim BP assets gathered from the SkeletalMesh nodes during compilation, to be used in mesh generation in-game */
	TArray<TSoftClassPtr<UAnimInstance>> AnimBPAssets;

	/** Used to propagate the socket priority defined in group nodes to their child skeletal mesh nodes
	* It's a stack because group nodes are recursive
	*/
	TArray<int8> SocketPriorityStack;
	
	/** Used to propagate the BonePose priority defined in group nodes to their child skeletal mesh nodes
	* It's a stack because group nodes are recursive
	*/
	TArray<int8> BonePosePriorityStack;

	// Stores what param names use a certain table as a table can be used from multiple table nodes, useful for partial compilations to restrict params
	TMap<FString, FMutableParamNameSet> TableToParamNames;

	TArray<const UEdGraphNode*> LimitedParameters;
	int32 ParameterLimitationCount = 0;

	// Current material parameter name to find the corresponding column in a mutable table
	FString CurrentMaterialTableParameter;

	/** */
	UE::Mutable::Private::ETableColumnType CurrentTableColumnType = UE::Mutable::Private::ETableColumnType::None;

	// Current material parameter id to find the corresponding column in a mutable table
	FString CurrentMaterialParameterId;

	// Material to SharedSurfaceId
	TMap<FSharedSurfaceKey, FGuid> SurfaceGuids;

	/** Extension Data constants are collected here */
	FExtensionDataCompilerInterface ExtensionDataCompilerInterface;

	/** Map to relate a Composite Data Table Row and its original DataTable */
	TMap<UDataTable*,TMap<FName, TArray<UDataTable*>>> CompositeDataTableRowToOriginalDataTableMap;

	/** Version Bridge of the root object */
	TObjectPtr<UObject> RootVersionBridge;

	/** Index of the Referenced Material being generated */
	TStrongObjectPtr<UMaterialInterface> CurrentReferencedMaterial;
	
	FGeneratedImageProperties* CurrentImageProperties = nullptr;

	// This should be automatized (create a struct that each time that a Macro is stacked, it adds 1 to the count.
	// it also substracts 1 each time we remove a Macro from the stack)
	TArray<const UCustomizableObjectNodeMacroInstance*> MacroNodesStack;

	// TODO GMT Get Default Values from Core. We only need compiled ones
	TMap<FString, TObjectPtr<UTexture>> TextureParameterDefaultValues;
	TMap<FString, TObjectPtr<USkeletalMesh>> SkeletalMeshParameterDefaultValues;
	TMap<FString, TObjectPtr<UMaterialInterface>> MaterialParameterDefaultValues;
	TMap<FString, FInstancedStruct> ExternalTypeParameterDefaultValues;

	TMap<FString, TObjectPtr<const UScriptStruct>> ExternalTypeParameterTypes;
	
	/** Name of the material break Node that is being compiled.
	NOTE: This is set when the material pin of a material break node is compiled and then it must be reset to empty. */
	TArray<FMaterialBreakParameter> MaterialBreakParameterStack;
};


namespace GenerateMutableSourcePrivate
{
	template<class HashableType, class HashDataSetType, class HashFuncType, class CompareFuncType>
	uint32 GenerateUniquePersistentHash(const HashableType& HashableData, const HashDataSetType& HashDataSet, HashFuncType&& HashFunc, CompareFuncType&& CompareFunc)
	{
		constexpr uint32 InvalidResourceId = 0;
		
		const uint32 DataHash = HashFunc(HashableData);

		uint32 UniqueHash = DataHash == InvalidResourceId ? DataHash + 1 : DataHash;

		const HashableType* FoundHash = HashDataSet.Find(UniqueHash);

		bool bIsDataAlreadyCollected = false;
		
		if (FoundHash)
		{
			bIsDataAlreadyCollected = CompareFunc(*FoundHash, HashableData); 
		}

		// NOTE: This way of unique hash generation guarantees all valid values can be used but given its 
		// sequential nature a cascade of changes can occur if new meshes are added. Not many hash collisions 
		// are expected so it should not be problematic.
		if (FoundHash && !bIsDataAlreadyCollected)
		{
			uint32 NumTries = 0;
			for (; NumTries < TNumericLimits<uint32>::Max(); ++NumTries)
			{
				FoundHash = HashDataSet.Find(UniqueHash);
				
				if (!FoundHash)
				{
					break;
				}

				bIsDataAlreadyCollected = CompareFunc(*FoundHash, HashableData);

				if (bIsDataAlreadyCollected)
				{
					break;
				}

				UniqueHash = UniqueHash + 1 == InvalidResourceId ? InvalidResourceId + 1 : UniqueHash + 1;
			}

			if (NumTries == TNumericLimits<uint32>::Max())
			{
				UniqueHash = InvalidResourceId;
			}	
		}

		return UniqueHash;
	}
}

//
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeObject> GenerateMutableSource(const class UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);

/** Populate an array with all the information related to the reference skeletal meshes we might need in-game to generate instances */
void PopulateReferenceSkeletalMeshesData(FMutableGraphGenerationContext& GenerationContext);


void CheckNode(const UEdGraphPin& Pin, const FMutableGraphGenerationContext& GenerationContext);


// TODO FutureGMT Remove generation context dependency and move to GraphTraversal.
UTexture2D* FindReferenceImage(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshApplyPose> CreateNodeMeshApplyPose(FMutableGraphGenerationContext& GenerationContext, UE::Mutable::Private::NodeMeshPtr InputMeshNode, const TArray<FName>& ArrayBoneName, const TArray<FTransform>& ArrayTransform);


uint32 GetBaseTextureSize(const FMutableGraphGenerationContext& GenerationContext, const UCONodeSkeletalMeshSection* MeshSectionNode, const UE::Mutable::Private::FParameterKey& ImageKey);

// Computes the LOD bias for a texture given the current mesh LOD and automatic LOD settings, the reference texture settings
// and whether it's being built for a server or not
uint32 ComputeLODBiasForTexture(const FMutableGraphGenerationContext& GenerationContext, const UTexture2D& Texture, const UTexture2D* ReferenceTexture = nullptr, int32 MaxTextureSizeInGame = 0);

// Max texture size to set on the ImageProperties
int32 GetMaxTextureSize(const UTexture2D& ReferenceTexture, const UTextureLODSettings& LODSettings);

// Max texture size of the texture with per platform MaxTextureSize and LODBias applied.
int32 GetTextureSizeInGame(const UTexture2D& Texture, const UTextureLODSettings& LODSettings);

// Compute the size Texture will have in-game on the TargetPlatform (cooked) using Source data only.
void GetCookedTextureSize(const FMutableGraphGenerationContext& GenerationContext, const UTexture2D* Texture, uint32& OutTextureSizeX, uint32& OutTextureSizeY, bool bSkipCinematicMips);

// Get the BlockSize in pixels in-game for an image with ReferenceTexture and NumBlocksX and NumBlocksY blocks. 
void GetLayoutBlockSizeInPixels(const FMutableGraphGenerationContext& GenerationContext, const UTexture2D* ReferenceTexture, const int32 NumBlocksX, const int32 NumBlocksY, uint16& BlockSizeX, uint16& BlockSizeY);

UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> GenerateImageConstant(UTexture*, FMutableGraphGenerationContext&, bool bIsReference);
UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> GenerateMeshConstant(const FMutableSourceMeshData& Source, FMutableGraphGenerationContext&);

/** Generates a mutable image descriptor from an unreal engine texture */
UE::Mutable::Private::FImageDesc GenerateImageDescriptor(UTexture* Texture);

/** Convert a mesh to Mutable format.
*/
UE::Tasks::FTask ConvertSkeletalMeshToMutable(TSharedRef<FMeshConversionContext> MeshConversionContext);

/**
 * Compares the Amount and names of the options exposed in the provided Enum Parameter Node with the options set in this node.
* @param MutableScalarParameter The enum parameter node we want to compare the options against.
 * @return true if the amount of options does match and false otherwise.
 */
bool DoOptionsMatchEnum(const UCONodeSwitch& InEditorSwitchNode, const UE::Mutable::Private::NodeScalarEnumParameter& MutableScalarParameter);

const FSkeletalMaterial* GetSkeletalMaterial(const USkeletalMesh* SkeletalMesh, uint8 LODIndex, uint8 SectionIndex);
