// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceColor.h"

#include "GenerateMutableSourceExternal.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceImage.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMacro.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMaterial.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeExternalOperation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorArithmeticOp.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorFromFloats.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorToSRGB.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CONodeMaterialBreak.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureSample.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuT/NodeColorArithmeticOperation.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeColorParameter.h"
#include "MuT/NodeColorSampleImage.h"
#include "MuT/NodeColorSwitch.h"
#include "MuT/NodeColorTable.h"
#include "MuT/NodeColorVariation.h"
#include "MuT/NodeColorToSRGB.h"
#include "MuT/NodeColorExternal.h"
#include "MuT/NodeColorMaterialBreak.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> GenerateMutableSourceColor(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceColor), *Pin, *Node, GenerationContext);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeColor*>(Generated->Node.get());
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For exampel, MacroInstanceNodes
	bool bCacheNode = true;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> Result;

	if (const UCustomizableObjectNodeColorConstant* TypedNodeColorConst = Cast<UCustomizableObjectNodeColorConstant>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorConstant> ColorNode = new UE::Mutable::Private::NodeColorConstant();
		Result = ColorNode;

		ColorNode->Value = TypedNodeColorConst->Value;
	}

	else if (const UCustomizableObjectNodeColorParameter* TypedNodeColorParam = Cast<UCustomizableObjectNodeColorParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorParameter> ColorNode = new UE::Mutable::Private::NodeColorParameter();
		Result = ColorNode;

		ColorNode->Name = TypedNodeColorParam->GetParameterName(&GenerationContext.MacroNodesStack);
		ColorNode->Uid = GenerationContext.GetNodeIdUnique(Node).ToString();
		ColorNode->DefaultValue = TypedNodeColorParam->DefaultValue;

		GenerationContext.ParameterUIDataMap.Add(TypedNodeColorParam->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
			TypedNodeColorParam->ParamUIMetadata,
			EMutableParameterType::Color));
	}

	else if (const UCONodeSwitch* TypedNodeColorSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Color))
	{
		Result = [&]() -> UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorSwitch>
		{
			if (const int32 NumParameters = FollowInputPinArray(*TypedNodeColorSwitch->SwitchParameterPinReference.Get()).Num();
				NumParameters != 1)
			{
				const FText Message = NumParameters
					? LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node.")
					: LOCTEXT("InvalidEnumInSwitch", "Switch nodes must have a single enum with all the options inside. Please remove all the enums but one and refresh the switch node.");

				GenerationContext.Log(Message, Node);
				return nullptr;
			}

			const UEdGraphPin* EnumPin = FollowInputPin(*TypedNodeColorSwitch->SwitchParameterPinReference.Get());
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = EnumPin ? GenerateMutableSourceFloat(EnumPin, GenerationContext) : nullptr;

			// Switch Param not generated
			if (!SwitchParam)
			{
				// Warn about a failure.
				if (EnumPin)
				{
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
					GenerationContext.Log(Message, Node);
				}

				return nullptr;
			}

			if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
			{
				const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
				GenerationContext.Log(Message, Node);

				return nullptr;
			}
				
			{
				const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());
				
				if (!DoOptionsMatchEnum(*TypedNodeColorSwitch, *EnumParameter))
				{
					const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
					GenerationContext.Log(Message, Node);
					Node->SetRefreshNodeWarning();
				}
			}

			const int32 NumSwitchOptions = TypedNodeColorSwitch->SwitchPins.Num();
				
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorSwitch> SwitchNode = new UE::Mutable::Private::NodeColorSwitch;
			SwitchNode->Parameter = SwitchParam;
			SwitchNode->Options.SetNum(NumSwitchOptions);

			for (int SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
			{
				const UEdGraphPin* const ColorPin = TypedNodeColorSwitch->SwitchPins[SelectorIndex].Get();
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*ColorPin))
				{
					SwitchNode->Options[SelectorIndex] = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
				}
			}

			return SwitchNode;
		}(); // invoke lambda;
	}

	else if (const UCustomizableObjectNodeTextureSample* TypedNodeTexSample = Cast<UCustomizableObjectNodeTextureSample>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorSampleImage> ColorNode = new UE::Mutable::Private::NodeColorSampleImage();
		Result = ColorNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTexSample->TexturePin()))
		{
			UE::Mutable::Private::NodeImagePtr TextureNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, 0);
			ColorNode->Image = TextureNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTexSample->XPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> XNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			ColorNode->X = XNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTexSample->YPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> YNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			ColorNode->Y = YNode;
		}
	}

	else if (const UCustomizableObjectNodeColorArithmeticOp* TypedNodeColorArith = Cast<UCustomizableObjectNodeColorArithmeticOp>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorArithmeticOperation> OpNode = new UE::Mutable::Private::NodeColorArithmeticOperation();
		Result = OpNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorArith->XPin()))
		{
			OpNode->A = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorArith->YPin()))
		{
			OpNode->B = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
		}

		switch (TypedNodeColorArith->Operation)
		{
		case EColorArithmeticOperation::E_Add:
			OpNode->Operation = UE::Mutable::Private::NodeColorArithmeticOperation::EOperation::Add;
			break;

		case EColorArithmeticOperation::E_Sub:
			OpNode->Operation = UE::Mutable::Private::NodeColorArithmeticOperation::EOperation::Subtract;
			break;

		case EColorArithmeticOperation::E_Mul:
			OpNode->Operation = UE::Mutable::Private::NodeColorArithmeticOperation::EOperation::Multiply;
			break;

		case EColorArithmeticOperation::E_Div:
			OpNode->Operation = UE::Mutable::Private::NodeColorArithmeticOperation::EOperation::Divide;
			break;
		}
	}

	else if (const UCustomizableObjectNodeColorFromFloats* TypedNodeFrom = Cast<UCustomizableObjectNodeColorFromFloats>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorFromScalars> OpNode = new UE::Mutable::Private::NodeColorFromScalars();
		Result = OpNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->RPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FloatNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			OpNode->X = FloatNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->GPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FloatNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			OpNode->Y = FloatNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->BPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FloatNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			OpNode->Z = FloatNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->APin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FloatNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			OpNode->W = FloatNode;
		}
	}

	else if (const UCustomizableObjectNodeColorVariation* TypedNodeColorVar = Cast<const UCustomizableObjectNodeColorVariation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorVariation> ColorNode = new UE::Mutable::Private::NodeColorVariation();
		Result = ColorNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorVar->DefaultPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ChildNode = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
			if (ChildNode)
			{
				ColorNode->DefaultColor = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("ColorFailed", "Color generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeColorVar->GetNumVariations();
		ColorNode->Variations.SetNum(NumVariations);
		for (int VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			const UEdGraphPin* VariationPin = TypedNodeColorVar->VariationPin(VariationIndex);
			if (!VariationPin) continue;

			ColorNode->Variations[VariationIndex].Tag = TypedNodeColorVar->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ChildNode = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
				ColorNode->Variations[VariationIndex].Color = ChildNode;
			}
		}
	}

	else if (const UCustomizableObjectNodeColorToSRGB* TypedNodeColorToSRGB = Cast<UCustomizableObjectNodeColorToSRGB>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorToSRGB> ColorNode = new UE::Mutable::Private::NodeColorToSRGB();
		Result = ColorNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorToSRGB->GetInputPin()))
		{
			ColorNode->Color = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
		}
	}

	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		//This node will add a white color in case of error
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorConstant> WhiteColorNode = new UE::Mutable::Private::NodeColorConstant();
		WhiteColorNode->Value = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

		Result = WhiteColorNode;

		bool bSuccess = true;
		UDataTable* DataTable = GetDataTable(TypedNodeTable, GenerationContext);

		if (DataTable)
		{
			FString ColumnName = TypedNodeTable->GetPinColumnName(Pin);
			FProperty* Property = TypedNodeTable->FindPinProperty(*Pin);

			if (!Property)
			{
				FString Msg = FString::Printf(TEXT("Couldn't find the column [%s] in the data table's struct."), *ColumnName);
				GenerationContext.Log(FText::FromString(Msg), Node);

				bSuccess = false;
			}

			if (bSuccess)
			{
				// Generating a new data table if not exists
				UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> Table;
				Table = GenerateMutableSourceTable(DataTable, TypedNodeTable, GenerationContext);

				if (Table)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorTable> ColorTableNode = new UE::Mutable::Private::NodeColorTable();

					// Generating a new Color column if not exists
					if (Table->FindColumn(ColumnName) == INDEX_NONE)
					{
						bSuccess = GenerateTableColumn(TypedNodeTable, Pin, Table, ColumnName, Property, FMutableSourceMeshData(), GenerationContext);

						if (!bSuccess)
						{
							FString Msg = FString::Printf(TEXT("Failed to generate the mutable table column [%s]"), *ColumnName);
							GenerationContext.Log(FText::FromString(Msg), Node);
						}
					}

					if (bSuccess)
					{
						Result = ColorTableNode;

						ColorTableNode->Table = Table;
						ColorTableNode->ColumnName = ColumnName;
						ColorTableNode->ParameterName = TypedNodeTable->ParameterName;
						ColorTableNode->bNoneOption = TypedNodeTable->bAddNoneOption;
						ColorTableNode->DefaultRowName = TypedNodeTable->DefaultRowName.ToString();
					}
				}
				else
				{
					FString Msg = FString::Printf(TEXT("Couldn't generate a mutable table."));
					GenerationContext.Log(FText::FromString(Msg), Node);
				}
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("ColorTableError", "Couldn't find the data table of the node."), Node);
		}
	}
	
	else if (Cast<UCONodeExternalOperation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorExternal> MeshExtension = new UE::Mutable::Private::NodeColorExternal();

		FSourceExternalOptions Options;
		MeshExtension->Node = GenerateMutableSourceExternal(Pin, GenerationContext, Options);
		
		Result = MeshExtension;

	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeColor>(*Pin, GenerationContext, GenerateMutableSourceColor);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeColor>(*Pin, GenerationContext, GenerateMutableSourceColor);
	}

	else if (const UCONodeMaterialBreak* TypedNodeMaterialBreak = Cast<UCONodeMaterialBreak>(Node))
	{
		check(TypedNodeMaterialBreak->MaterialPinRef.Get());

		if (const UEdGraphPin* LinkedMaterial = FollowInputPin(*TypedNodeMaterialBreak->MaterialPinRef.Get()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorMaterialBreak> MaterialBreak = new UE::Mutable::Private::NodeColorMaterialBreak();

			FName ParameterName = TypedNodeMaterialBreak->GetPinParameterName(*Pin);
			if (!ParameterName.IsNone())
			{
				const int32 LayerIndex = TypedNodeMaterialBreak->GetPinParameterLayerIndex(*Pin);
				check(LayerIndex <= MAX_int8);

				// Set the key of the parameter that represents the pin
				MaterialBreak->ParameterKey = {ParameterName, (int8)LayerIndex};

				// Generate the material that is going to be broken into parameters
				GenerationContext.MaterialBreakParameterStack.Add(FMaterialBreakParameter{ MaterialBreak->ParameterKey, Pin->PinType.PinCategory });
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> LinkedNode = GenerateMutableSourceMaterial(LinkedMaterial, GenerationContext, FGenerationMaterialOptions());
				MaterialBreak->MaterialSource = LinkedNode;
				GenerationContext.MaterialBreakParameterStack.Pop();

				if (LinkedNode)
				{
					Result = MaterialBreak;
				}
				else
				{
					GenerationContext.Log(LOCTEXT("MaterialBreakNodeError", "An error occurred generating the Material Source."), Node, EMessageSeverity::Error);
				}
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoneParameterName", "'None' is not a valid Material Parameter name."), Node, EMessageSeverity::Warning);
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MaterialBreakPinError", "Material Break Nodes must be linked to an input Material Node."), Node);
		}
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	if (bCacheNode)
	{
		FGeneratedData CacheData = FGeneratedData(Node, Result);
		GenerationContext.Generated.Add(Key, CacheData);
		GenerationContext.GeneratedNodes.Add(Node);
	}
	
	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
