// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeColor.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;

/** Convert a CustomizableObject Source Graph into a mutable source graph. */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> GenerateMutableSourceColor(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);
