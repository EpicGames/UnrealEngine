// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateMutableSourceComponent.h"

#include "GenerateMutableSourceSkeletalMeshObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceSurface.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMacro.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMaterial.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeModifierSkeletalMeshMerge.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentPassthroughMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/Nodes/CONodeComponentSkeletalMesh.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeModifierSkeletalMeshMerge.h"
#include "MuT/NodeComponentNew.h"
#include "MuT/NodeComponentSwitch.h"
#include "MuT/NodeComponentVariation.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeSurfaceNew.h"

#include "Rendering/SkeletalMeshLODModel.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CONodeSwitch.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> GenerateMutableSourceComponent(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceComponent), *Pin, *Node, GenerationContext, false);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeComponent*>(Generated->Node.get());
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}
	
	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For example, MacroInstanceNodes
	bool bCacheNode = true;
	
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> Result;
	
	if (const UCONodeComponentSkeletalMesh* NodeComponentMesh = Cast<UCONodeComponentSkeletalMesh>(Node))
	{
		if (FSkeletalMeshComponentInfo* ComponentInfo = GenerationContext.CompilationContext->GetComponentInfo(NodeComponentMesh))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponentNew> NodeComponentNew = new UE::Mutable::Private::NodeComponentNew();
			NodeComponentNew->Id = ComponentInfo->ComponentId;
			NodeComponentNew->SetMessageContext(Node);

			// Skeletal Mesh
			const UEdGraphPin* SkeletalMeshPin = NodeComponentMesh->SkeletalMeshPin.Get();
			check(SkeletalMeshPin);
			if (const UEdGraphPin* ConnectedSkeletalMeshPin = FollowInputPin(*SkeletalMeshPin))
			{
				GenerationContext.CurrentSkeletalMeshComponent = ComponentInfo->ComponentId;
				
				FSourceSkeletalMeshObjectOptions Options;
				Options.ReferenceSkeletalMesh = NodeComponentMesh->ReferenceSkeletalMesh;
				NodeComponentNew->SkeletalMeshObject = UE::Mutable::Private::GenerateMutableSourceSkeletalMeshObject(ConnectedSkeletalMeshPin, GenerationContext, Options);
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
			
			// Overlay material  (applied to the whole component)
			const UEdGraphPin* MaterialAssetPin = NodeComponentMesh->GetOverlayMaterialAssetPin().Get();
			check(MaterialAssetPin);
			if (const UEdGraphPin* ConnectedMaterialPin = FollowInputPin(*MaterialAssetPin))
			{
				GenerationContext.CurrentMaterialParameterId = ConnectedMaterialPin->PinId.ToString();
				NodeComponentNew->OverlayMaterial = GenerateMutableSourceMaterial(ConnectedMaterialPin, GenerationContext, FGenerationMaterialOptions());
			}
			
			// Override/Overlay Material slots
			TArray<FName> ProcessedOverrideMaterialSlotNames;
			TArray<FName> ProcessedOverlayMaterialSlotNames;
			for (const FEdGraphPinReference& InputMaterialPinReference : NodeComponentMesh->MaterialSlotPins)
			{
				// Material node
				if (const UEdGraphPin* MaterialOverridePin = InputMaterialPinReference.Get())
				{
					if (const UEdGraphPin* ConnectedPin = FollowInputPin(*MaterialOverridePin))
					{
						const UCONodeComponentSkeletalMeshMaterialPinData& OverrideMaterialPinData =
							NodeComponentMesh->GetPinData<UCONodeComponentSkeletalMeshMaterialPinData>(*MaterialOverridePin);
						
						const FName TargetMaterialSlotName = OverrideMaterialPinData.TargetMaterialSlotName;
						
						// Skip invalid slot names (None)
						if (TargetMaterialSlotName.IsNone())
						{
							GenerationContext.Log(LOCTEXT("NoneMaterialSlotName", "'None' is not a valid Material slot name."), Node, EMessageSeverity::Error);
							continue;
						}
						
						const FName TargetMaterialOperationName = InputMaterialPinReference.Get()->PinType.PinSubCategory;
						
						bool bSlotMaterialOperationAlreadyFound = false;
						if (TargetMaterialOperationName == UEdGraphSchema_CustomizableObject::PSC_Material_Override)
						{
							bSlotMaterialOperationAlreadyFound = ProcessedOverrideMaterialSlotNames.Contains(TargetMaterialSlotName);
						}
						else if (TargetMaterialOperationName == UEdGraphSchema_CustomizableObject::PSC_Material_Overlay)
						{
							bSlotMaterialOperationAlreadyFound = ProcessedOverlayMaterialSlotNames.Contains(TargetMaterialSlotName);
						}
					
						// Skip repeated slot operations. 
						if (bSlotMaterialOperationAlreadyFound)
						{
							const FText FriendlyOperationTypeName = UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(TargetMaterialOperationName);
							GenerationContext.Log(FText::Format(LOCTEXT("RepeatedMaterialSlot", "The slot with '{0}' name and '{1}' type has been defined multiple times."), FText::FromName(TargetMaterialSlotName), FriendlyOperationTypeName), Node, EMessageSeverity::Error);
							continue;
						}
				
						const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> MaterialNode = 
							GenerateMutableSourceMaterial(ConnectedPin, GenerationContext, FGenerationMaterialOptions());
				
						if (MaterialNode)
						{
							if (TargetMaterialOperationName == UEdGraphSchema_CustomizableObject::PSC_Material_Override)
							{
								NodeComponentNew->OverrideMaterials.Add(TargetMaterialSlotName, MaterialNode);
								ProcessedOverrideMaterialSlotNames.Add(TargetMaterialSlotName);
							}
							else if (TargetMaterialOperationName == UEdGraphSchema_CustomizableObject::PSC_Material_Overlay)
							{
								NodeComponentNew->OverlayMaterials.Add(TargetMaterialSlotName, MaterialNode);
								ProcessedOverlayMaterialSlotNames.Add(TargetMaterialSlotName);
							}
						}
					}
				}
			}
			
			Result = NodeComponentNew;
		}
	}
	
	else if (const UCONodeSwitch* TypedNodeSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Component))
	{
		// Using a lambda so control flow is easier to manage.
		Result = [&]()
			{
				const UEdGraphPin* SwitchParameter = TypedNodeSwitch->SwitchParameterPinReference.Get();

				// Check Switch Parameter arity preconditions.
				if (const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter))
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

					// Switch Param not generated
					if (!SwitchParam)
					{
						// Warn about a failure.
						const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
						GenerationContext.Log(Message, Node);

						return Result;
					}

					if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
					{
						const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
						GenerationContext.Log(Message, Node);

						return Result;
					}

					{
						const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());
						
						if (!DoOptionsMatchEnum(*TypedNodeSwitch, *EnumParameter))
						{
							const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
							GenerationContext.Log(Message, Node);
							Node->SetRefreshNodeWarning();
						}
					}
					
					const int32 NumSwitchOptions = TypedNodeSwitch->SwitchPins.Num();

					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponentSwitch> SwitchNode = new UE::Mutable::Private::NodeComponentSwitch;
					SwitchNode->Parameter = SwitchParam;
					SwitchNode->Options.SetNum(NumSwitchOptions);

					for (int32 SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
					{
						if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeSwitch->SwitchPins[SelectorIndex].Get()))
						{
							UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> ChildNode = GenerateMutableSourceComponent(ConnectedPin, GenerationContext);
							if (ChildNode)
							{
								SwitchNode->Options[SelectorIndex] = ChildNode;
							}
							else
							{
								// Probably ok
							}
						}
					}

					Result = SwitchNode;
					return Result;
				}
				else
				{
					GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), Node);
					return Result;
				}
			}(); // invoke lambda.
	}

	else if (const UCustomizableObjectNodeComponentVariation* TypedNodeVar = Cast<UCustomizableObjectNodeComponentVariation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponentVariation> SurfNode = new UE::Mutable::Private::NodeComponentVariation();
		Result = SurfNode;

		for (const UEdGraphPin* ConnectedPin : FollowInputPinArray(*TypedNodeVar->DefaultPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> ChildNode = GenerateMutableSourceComponent(ConnectedPin, GenerationContext);
			if (ChildNode)
			{
				SurfNode->DefaultComponent = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("ComponentFailed", "Component generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeVar->GetNumVariations();
		SurfNode->Variations.SetNum(NumVariations);
		for (int VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			UE::Mutable::Private::NodeSurfacePtr VariationSurfaceNode;

			if (UEdGraphPin* VariationPin = TypedNodeVar->VariationPin(VariationIndex))
			{
				SurfNode->Variations[VariationIndex].Tag = TypedNodeVar->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);
				for (const UEdGraphPin* ConnectedPin : FollowInputPinArray(*VariationPin))
				{
					// Is it a modifier?
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> ChildNode = GenerateMutableSourceComponent(ConnectedPin, GenerationContext);
					if (ChildNode)
					{
						SurfNode->Variations[VariationIndex].Component = ChildNode;
					}
					else
					{
						GenerationContext.Log(LOCTEXT("ComponentFailed", "Component generation failed."), Node);
					}
				}
			}
		}
	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeComponent>(*Pin, GenerationContext, GenerateMutableSourceComponent);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeComponent>(*Pin, GenerationContext, GenerateMutableSourceComponent);
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
		ensure(false);
	}

	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}


void FirstPass(UCONodeComponentSkeletalMesh& ComponentNode, FMutableGraphGenerationContext& GenerationContext)
{	
	USkeletalMesh* RefSkeletalMesh = ComponentNode.ReferenceSkeletalMesh;
	const USkeleton* RefSkeleton = nullptr;
	if (RefSkeletalMesh)
	{
		RefSkeleton = RefSkeletalMesh->GetSkeleton();
	}
	
	const FName ComponentName = ComponentNode.GetComponentName(&GenerationContext.MacroNodesStack);
	UE::Mutable::Private::FComponentId ComponentId = GenerationContext.ComponentNames.Add(ComponentName);
	
	// Add a new entry to the list of Component Infos
	FSkeletalMeshComponentInfo ComponentInfo;
	ComponentInfo.ComponentId = ComponentId;
	ComponentInfo.Node = &ComponentNode;
	
	if (RefSkeletalMesh && RefSkeleton)
	{
		ComponentInfo.RefSkeletalMesh = TStrongObjectPtr(RefSkeletalMesh);
	}

	GenerationContext.CompilationContext->ComponentInfos.Add(ComponentInfo);
}


#undef LOCTEXT_NAMESPACE

