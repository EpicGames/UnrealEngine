// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CONodeComponentSkeletalMesh.h"
#include "MuR/Ptr.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;

namespace UE::Mutable::Private
{
	class NodeComponent;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeComponent> GenerateMutableSourceComponent(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);


void FirstPass(UCONodeComponentSkeletalMesh& ComponentNode, FMutableGraphGenerationContext& GenerationContext);

