// Copyright Epic Games, Inc. All Rights Reserved.


#include "GenerateMutableSourceExternal.h"

#include "GenerateMutableSourceColor.h"
#include "GenerateMutableSourceFloat.h"
#include "GenerateMutableSourceImage.h"
#include "GenerateMutableSourceMacro.h"
#include "GenerateMutableSourceMaterial.h"
#include "GenerateMutableSourceMesh.h"
#include "MuCO/ICustomizableObjectModulePrivate.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeExternalOperation.h"
#include "MuCOE/Nodes/CONodeExternalTypeParameter.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuCOE/Nodes/ExternalTypeCustomization.h"
#include "MuR/External/FloatAdapter.h"
#include "MuR/External/MaterialAdapter.h"
#include "MuR/External/MeshAdapter.h"
#include "MuR/External/Operation.h"
#include "MuR/External/TextureAdapter.h"
#include "MuR/External/VectorAdapter.h"
#include "MuT/NodeExternalOperation.h"
#include "MuT/NodeExternalParameter.h"
#include "MuT/NodeExternalSwitch.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternal> GenerateMutableSourceExternal(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FSourceExternalOptions& Options)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	FGeneratedSourceExternalKey Key;
	Key.Pin = Pin;
	Key.Options = Options;

	// TODO: limit to external mesh operations.
	Key.Options.CurrentLOD = Node->IsAffectedByLOD() ? GenerationContext.CurrentLOD : 0;
	Key.Options.MeshOptions.LODOffset = Node->IsAffectedByLOD() ? GenerationContext.FromLOD - GenerationContext.CurrentLOD : 0;

	if (const FGeneratedSourceExternalData* Generated = GenerationContext.GeneratedSourceExternals.Find(Key))
	{
		return Generated->Node;
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}
	
	UCustomizableObjectNode* EditorNode = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternal> Result;
	bool bCacheNode = true;
	
	if (const UCONodeExternalOperation* TypedNodeExternal = Cast<UCONodeExternalOperation>(EditorNode))
	{
		[&]()
		{
			const UE::Mutable::FExternalOperation* Operation = TypedNodeExternal->OperationInstancedStruct.GetPtr<UE::Mutable::FExternalOperation>();
			if (!Operation)
			{
				FText Msg = FText::Format(LOCTEXT("ExternalNotLoaded", "Node is using an external operation which has not been loaded: {0}"), TypedNodeExternal->CachedOperationName);
				GenerationContext.Log(Msg, TypedNodeExternal, EMessageSeverity::Error);
				return;
			}
				
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternalOperation> ExternalNode = new UE::Mutable::Private::NodeExternalOperation();
            				
            ExternalNode->OperationInstancedStruct = TypedNodeExternal->OperationInstancedStruct;

            TArray<TPair<FText, const UScriptStruct*>> Inputs = Operation->GetInputs();
            for (const TPair<FText, const UScriptStruct*>& Pair : Inputs)
            {
            	const FText& InputName = Pair.Key;
            
            	const FEdGraphPinReference* InputResult = TypedNodeExternal->InputPins.FindByPredicate([&](const FEdGraphPinReference& Element)
            	{
            		return Element.Get()->PinFriendlyName.EqualTo(InputName);
            	});
        
            	if (!InputResult)
            	{
            		FText Msg = FText::Format(LOCTEXT("ExternalInputNotFound", "Can not find input in external operation: {0}"), InputName);
            		GenerationContext.Log(Msg, TypedNodeExternal, EMessageSeverity::Error);
            		continue;
            	}

            	const UEdGraphPin* ConnectedPin = FollowInputPin(*InputResult->Get());
            	if (!ConnectedPin)
            	{
            		FText Msg = FText::Format(LOCTEXT("ExternalInputNotConnected", "External operation input not connected: {0}"), InputName);
            		GenerationContext.Log(Msg, TypedNodeExternal, EMessageSeverity::Error);
            		continue;
            	}

            	const UScriptStruct* InputType = Pair.Value;
            
            	if (InputType == UE::Mutable::FMeshAdapter::StaticStruct())
            	{
            		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> SourceNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, Options.MeshOptions, Options.bLinkedToExtendMaterial, Options.bOnlyConnectedLOD);
            		ExternalNode->Inputs.Add(SourceNode);
            	}
            	else if (InputType == UE::Mutable::FTextureAdapter::StaticStruct())
            	{
            		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> SourceNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, Options.ReferenceTextureSize);
            		ExternalNode->Inputs.Add(SourceNode);
            	}
            	else if (InputType == UE::Mutable::FFloatAdapter::StaticStruct())
            	{
            		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SourceNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
            		ExternalNode->Inputs.Add(SourceNode);
            	}
            	else if (InputType == UE::Mutable::FVectorAdapter::StaticStruct())
            	{
            		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> SourceNode = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
            		ExternalNode->Inputs.Add(SourceNode);
            	}
            	else if (InputType == UE::Mutable::FMaterialAdapter::StaticStruct())
            	{
					FGenerationMaterialOptions MaterialOptions;
					MaterialOptions.SurfaceNode = Options.SurfaceNode;

            		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> SourceNode = GenerateMutableSourceMaterial(ConnectedPin, GenerationContext, MaterialOptions);
            		ExternalNode->Inputs.Add(SourceNode);
            	}
            	else
            	{
            		UE::Mutable::Private::Ptr<UE::Mutable::Private::Node> SourceNode = GenerateMutableSourceExternal(ConnectedPin, GenerationContext, Options);
            		ExternalNode->Inputs.Add(SourceNode);
            	}	
            }

			Result = ExternalNode;
		}();		
	}

	if (const UCONodeSwitch* NodeSwitch = Cast<UCONodeSwitch>(EditorNode))
	{
		// Using a lambda so control flow is easier to manage.
		Result = [&]() -> UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternal>
		{
			const UEdGraphPin* SwitchParameter = NodeSwitch->SwitchParameterPinReference.Get();

			// Check Switch Parameter arity preconditions.
			if (const int32 NumParameters = FollowInputPinArray(*SwitchParameter).Num();
				NumParameters != 1)
			{
				const FText Message = NumParameters
					? LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node.")
					: LOCTEXT("InvalidEnumInSwitch", "Switch nodes must have a single enum with all the options inside. Please remove all the enums but one and refresh the switch node.");

				GenerationContext.Log(Message, Node);
				return nullptr;
			}

			const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter);
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

			// Switch Param not generated
			if (!SwitchParam)
			{
				// Warn about a failure.
				if (EnumPin)
				{
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
					GenerationContext.Log(Message, Node);
				}

				return nullptr;
			}

			if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
			{
				const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
				GenerationContext.Log(Message, Node);

				return nullptr;
			}

			{
				const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());
				
				if (!DoOptionsMatchEnum(*NodeSwitch, *EnumParameter))
				{
					const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
					GenerationContext.Log(Message, Node);
					Node->SetRefreshNodeWarning();
				}
			}

			const int32 NumSwitchOptions = NodeSwitch->SwitchPins.Num();
				
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternalSwitch> SwitchNode = new UE::Mutable::Private::NodeExternalSwitch;
			SwitchNode->Parameter = SwitchParam;
			SwitchNode->Options.SetNum(NumSwitchOptions);

			for (int SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
			{
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeSwitch->SwitchPins[SelectorIndex].Get()))
				{
					SwitchNode->Options[SelectorIndex] = GenerateMutableSourceExternal(ConnectedPin, GenerationContext, Options);
				}
			}

			return SwitchNode;
		}(); // invoke lambda.
	}
	
	if (const UCONodeExternalTypeParameter* TypeNodeExternalParameter = Cast<UCONodeExternalTypeParameter>(EditorNode))
	{
		FString ParameterName = TypeNodeExternalParameter->GetParameterName(&GenerationContext.MacroNodesStack);

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternalParameter> ExternalParameterNode = new UE::Mutable::Private::NodeExternalParameter();
		ExternalParameterNode->Name = ParameterName;
		ExternalParameterNode->UID = GenerationContext.GetNodeIdUnique(Node).ToString();
		ExternalParameterNode->DefaultValue = UE::Mutable::Private::MakeManaged<FInstancedStruct>(TypeNodeExternalParameter->DefaultValue);

		GenerationContext.ExternalTypeParameterDefaultValues.Add(ParameterName, TypeNodeExternalParameter->DefaultValue);
		GenerationContext.ExternalTypeParameterTypes.Add(ParameterName, TypeNodeExternalParameter->DefaultValue.GetScriptStruct());
		
		Result = ExternalParameterNode;
	}
	
	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeExternal>(*Pin, GenerationContext, GenerateMutableSourceExternal, Options);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;

		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeExternal>(*Pin, GenerationContext, GenerateMutableSourceExternal, Options);
	}
	
	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	if (bCacheNode)
	{
		FGeneratedSourceExternalData CacheData;
		CacheData.Node = Result;
	
		GenerationContext.GeneratedSourceExternals.Add(Key, CacheData);
		GenerationContext.GeneratedNodes.Add(Node);	
	}

	return Result;
}


#undef LOCTEXT_NAMESPACE
