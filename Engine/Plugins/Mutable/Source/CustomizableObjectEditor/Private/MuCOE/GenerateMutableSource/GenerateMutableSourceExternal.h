// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeExternal.h"

class UEdGraphPin;
struct FSourceExternalOptions;
struct FMutableGraphGenerationContext;


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExternal> GenerateMutableSourceExternal(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FSourceExternalOptions& Options);

