// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"

#include "GenerateMutableSourceExternal.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceImage.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMacro.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMaterial.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeExternalOperation.h"
#include "MuCOE/Nodes/CONodeMaterialBreak.h"
#include "MuCOE/Nodes/CustomizableObjectNodeCurve.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatArithmeticOp.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/NodeScalarCurve.h"
#include "MuT/NodeScalarSwitch.h"
#include "MuT/NodeScalarTable.h"
#include "MuT/NodeScalarVariation.h"
#include "MuT/NodeScalarArithmeticOperation.h"
#include "MuT/NodeScalarExternal.h"
#include "MuT/NodeScalarMaterialBreak.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> GenerateMutableSourceFloat(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceFloat), *Pin, *Node, GenerationContext);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeScalar*>(Generated->Node.get());
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For exampel, MacroInstanceNodes
	bool bCacheNode = true;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> Result;
	
	if (const UCustomizableObjectNodeFloatConstant* FloatConstantNode = Cast<UCustomizableObjectNodeFloatConstant>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> ScalarNode = new UE::Mutable::Private::NodeScalarConstant();
		Result = ScalarNode;

		ScalarNode->Value = FloatConstantNode->Value;
	}

	else if (const UCustomizableObjectNodeFloatParameter* FloatParameterNode = Cast<UCustomizableObjectNodeFloatParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter> ScalarNode = new UE::Mutable::Private::NodeScalarParameter();
		Result = ScalarNode;

		ScalarNode->Name = FloatParameterNode->GetParameterName(&GenerationContext.MacroNodesStack);
		ScalarNode->UID = GenerationContext.GetNodeIdUnique(Node).ToString();
		ScalarNode->DefaultValue = FloatParameterNode->DefaultValue;

		GenerationContext.ParameterUIDataMap.Add(FloatParameterNode->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
			FloatParameterNode->ParamUIMetadata,
			EMutableParameterType::Float));
	}

	else if (const UCustomizableObjectNodeEnumParameter* EnumParamNode = Cast<UCustomizableObjectNodeEnumParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter> EnumParameterNode = new UE::Mutable::Private::NodeScalarEnumParameter;

		const int32 NumSelectors = EnumParamNode->Values.Num();

		int32 DefaultValue = FMath::Clamp(EnumParamNode->DefaultIndex, 0, NumSelectors - 1);

		EnumParameterNode->Name = EnumParamNode->GetParameterName(&GenerationContext.MacroNodesStack);
		EnumParameterNode->UID = GenerationContext.GetNodeIdUnique(Node).ToString();
		EnumParameterNode->Options.SetNum(NumSelectors);
		EnumParameterNode->DefaultValue = DefaultValue;

		FMutableParameterData ParameterUIData(EnumParamNode->ParamUIMetadata, EMutableParameterType::Int);
		ParameterUIData.IntegerParameterGroupType = ECustomizableObjectGroupType::COGT_ONE;

		for (int SelectorIndex = 0; SelectorIndex < NumSelectors; ++SelectorIndex)
		{
			EnumParameterNode->Options[SelectorIndex].Name = EnumParamNode->Values[SelectorIndex].Name;
			EnumParameterNode->Options[SelectorIndex].Value = (float)SelectorIndex;

			ParameterUIData.ArrayIntegerParameterOption.Add(
				EnumParamNode->Values[SelectorIndex].Name,
				FIntegerParameterUIData(EnumParamNode->Values[SelectorIndex].ParamUIMetadata));
		}

		Result = EnumParameterNode;

		GenerationContext.ParameterUIDataMap.Add(EnumParamNode->GetParameterName(&GenerationContext.MacroNodesStack), ParameterUIData);
	}

	else if (const UCONodeSwitch* TypedNodeFloatSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Float))
	{
		// Using a lambda so control flow is easier to manage.
		Result = [&]() -> UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar>
		{
			const UEdGraphPin* SwitchParameter = TypedNodeFloatSwitch->SwitchParameterPinReference.Get();

			// Check Switch Parameter arity preconditions.
			if (const int32 NumParameters = FollowInputPinArray(*SwitchParameter).Num();
				NumParameters != 1)
			{
				const FText Message = NumParameters
					? LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node.")
					: LOCTEXT("InvalidEnumInSwitch", "Switch nodes must have a single enum with all the options inside. Please remove all the enums but one and refresh the switch node.");

				GenerationContext.Log(Message, Node);
				return nullptr;
			}

			const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter);
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

			// Switch Param not generated
			if (!SwitchParam)
			{
				// Warn about a failure.
				if (EnumPin)
				{
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
					GenerationContext.Log(Message, Node);
				}

				return nullptr;
			}

			if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
			{
				const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
				GenerationContext.Log(Message, Node);

				return nullptr;
			}

			{
				const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());
				
				if (!DoOptionsMatchEnum(*TypedNodeFloatSwitch, *EnumParameter))
				{
					const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
					GenerationContext.Log(Message, Node);
					Node->SetRefreshNodeWarning();
				}
			}

			const int32 NumSwitchOptions = TypedNodeFloatSwitch->SwitchPins.Num();
				
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarSwitch> SwitchNode = new UE::Mutable::Private::NodeScalarSwitch;
			SwitchNode->Parameter = SwitchParam;
			SwitchNode->Options.SetNum(NumSwitchOptions);

			for (int SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
			{
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFloatSwitch->SwitchPins[SelectorIndex].Get()))
				{
					SwitchNode->Options[SelectorIndex] = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
				}
			}

			return SwitchNode;
		}(); // invoke lambda.
	}

	else if (const UCustomizableObjectNodeCurve* TypedNodeCurve = Cast<UCustomizableObjectNodeCurve>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarCurve> CurveNode = new UE::Mutable::Private::NodeScalarCurve();
		Result = CurveNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeCurve->InputPin()))
		{
			CurveNode->CurveSampleValue = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
		}

		if (UCurveBase* CurveAsset = TypedNodeCurve->CurveAsset)
		{
			int32 PinIndex = -1;

			for (int32 i = 0; i < TypedNodeCurve->GetNumCurvePins(); ++i)
			{
				if (TypedNodeCurve->CurvePins(i) == Pin)
				{
					PinIndex = i;
					break;
				}
			}

			if (const UCurveLinearColor* const CurveColor = Cast<UCurveLinearColor>(CurveAsset))
			{
				if (PinIndex >= 0 && PinIndex <= 3)
				{
					CurveNode->Curve = CurveColor->FloatCurves[PinIndex];
				}
			}
			else if (const UCurveVector* const CurveVector = Cast<UCurveVector>(CurveAsset))
			{
				if (PinIndex >= 0 && PinIndex <= 2)
				{
					CurveNode->Curve = CurveVector->FloatCurves[PinIndex];
				}
			}
			else if (const UCurveFloat* const CurveFloat = Cast<UCurveFloat>(CurveAsset))
			{
				if (PinIndex == 0)
				{
					CurveNode->Curve = CurveFloat->FloatCurve;
				}
			}
		}
	}

	else if (const UCustomizableObjectNodeFloatVariation* TypedNodeFloatVar = Cast<const UCustomizableObjectNodeFloatVariation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarVariation> FloatNode = new UE::Mutable::Private::NodeScalarVariation();
		Result = FloatNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFloatVar->DefaultPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ChildNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			if (ChildNode)
			{
				FloatNode->DefaultScalar = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("FloatFailed", "Float generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeFloatVar->GetNumVariations();
		FloatNode->Variations.SetNum(NumVariations);
		for (int VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			UEdGraphPin* VariationPin = TypedNodeFloatVar->VariationPin(VariationIndex);
			if (!VariationPin) continue;

			FloatNode->Variations[VariationIndex].Tag = TypedNodeFloatVar->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ChildNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
				FloatNode->Variations[VariationIndex].Scalar = ChildNode;
			}
		}
	}

	else if (const UCustomizableObjectNodeFloatArithmeticOp* TypedNodeFloatArith = Cast<UCustomizableObjectNodeFloatArithmeticOp>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarArithmeticOperation> OpNode = new UE::Mutable::Private::NodeScalarArithmeticOperation();
		Result = OpNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFloatArith->XPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> XNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			OpNode->A = XNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFloatArith->YPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> YNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			OpNode->B = YNode;
		}

		switch (TypedNodeFloatArith->Operation)
		{
		case EFloatArithmeticOperation::E_Add:
			OpNode->Operation = UE::Mutable::Private::NodeScalarArithmeticOperation::EOperation::AO_ADD;
			break;

		case EFloatArithmeticOperation::E_Sub:
			OpNode->Operation = UE::Mutable::Private::NodeScalarArithmeticOperation::EOperation::AO_SUBTRACT;
			break;

		case EFloatArithmeticOperation::E_Mul:
			OpNode->Operation = UE::Mutable::Private::NodeScalarArithmeticOperation::EOperation::AO_MULTIPLY;
			break;

		case EFloatArithmeticOperation::E_Div:
			OpNode->Operation = UE::Mutable::Private::NodeScalarArithmeticOperation::EOperation::AO_DIVIDE;
			break;

		default:
			unimplemented();
		}
	}

	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		//This node will add a default value in case of error
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> ConstantValue = new UE::Mutable::Private::NodeScalarConstant();
		ConstantValue->Value = 1.0f;

		Result = ConstantValue;

		if (Pin->PinType.PinCategory == Schema->PC_Material)
		{
			// Material pins have to skip the cache of nodes or they will return always the same column node
			bCacheNode = false;
		}

		bool bSuccess = true;
		UDataTable* DataTable = GetDataTable(TypedNodeTable, GenerationContext);

		if (DataTable)
		{
			FString ColumnName = TypedNodeTable->GetPinColumnName(Pin);
			FProperty* Property = TypedNodeTable->FindPinProperty(*Pin);

			if (!Property)
			{
				FString Msg = FString::Printf(TEXT("Couldn't find the column [%s] in the data table's struct."), *ColumnName);
				GenerationContext.Log(FText::FromString(Msg), Node);

				bSuccess = false;
			}

			if (bSuccess)
			{
				// Generating a new data table if not exists
				UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> Table;
				Table = GenerateMutableSourceTable(DataTable, TypedNodeTable, GenerationContext);

				if (Table)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarTable> ScalarTableNode = new UE::Mutable::Private::NodeScalarTable();

					// Generating a new Float column if not exists
					if (Table->FindColumn(ColumnName) == INDEX_NONE)
					{
						bSuccess = GenerateTableColumn(TypedNodeTable, Pin, Table, ColumnName, Property, FMutableSourceMeshData(), GenerationContext);

						if (!bSuccess)
						{
							FString Msg = FString::Printf(TEXT("Failed to generate the mutable table column [%s]"), *ColumnName);
							GenerationContext.Log(FText::FromString(Msg), Node);
						}
					}

					if (bSuccess)
					{
						Result = ScalarTableNode;

						ScalarTableNode->Table = Table;
						ScalarTableNode->ColumnName = ColumnName;
						ScalarTableNode->ParameterName = TypedNodeTable->ParameterName;
						ScalarTableNode->bNoneOption = TypedNodeTable->bAddNoneOption;
						ScalarTableNode->DefaultRowName = TypedNodeTable->DefaultRowName.ToString();
					}
				}
				else
				{
					FString Msg = FString::Printf(TEXT("Couldn't generate a mutable table."));
					GenerationContext.Log(FText::FromString(Msg), Node);
				}
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("ScalarTableError", "Couldn't find the data table of the node."), Node);
		}
	}

	else if (Cast<UCONodeExternalOperation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarExternal> MeshExtension = new UE::Mutable::Private::NodeScalarExternal();

		FSourceExternalOptions Options;
		MeshExtension->Node = GenerateMutableSourceExternal(Pin, GenerationContext, Options);
		
		Result = MeshExtension;

	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeScalar>(*Pin, GenerationContext, GenerateMutableSourceFloat);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeScalar>(*Pin, GenerationContext, GenerateMutableSourceFloat);
	}

	else if (const UCONodeMaterialBreak* TypedNodeMaterialBreak = Cast<UCONodeMaterialBreak>(Node))
	{
		check(TypedNodeMaterialBreak->MaterialPinRef.Get());

		if (const UEdGraphPin* LinkedMaterial = FollowInputPin(*TypedNodeMaterialBreak->MaterialPinRef.Get()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarMaterialBreak> MaterialBreak = new UE::Mutable::Private::NodeScalarMaterialBreak();

			FName ParameterName = TypedNodeMaterialBreak->GetPinParameterName(*Pin);
			if (!ParameterName.IsNone())
			{
				const int32 LayerIndex = TypedNodeMaterialBreak->GetPinParameterLayerIndex(*Pin);
				check(LayerIndex <= MAX_int8);

				// Set the key of the parameter that represents the pin
				MaterialBreak->ParameterKey = {ParameterName, (int8)LayerIndex};

				// Generate the material that is going to be broken into parameters
				GenerationContext.MaterialBreakParameterStack.Add(FMaterialBreakParameter{ MaterialBreak->ParameterKey, Pin->PinType.PinCategory});
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> LinkedNode = GenerateMutableSourceMaterial(LinkedMaterial, GenerationContext, FGenerationMaterialOptions());
				MaterialBreak->MaterialSource = LinkedNode;
				GenerationContext.MaterialBreakParameterStack.Pop();

				if (LinkedNode)
				{
					Result = MaterialBreak;
				}
				else
				{
					GenerationContext.Log(LOCTEXT("MaterialBreakNodeError", "An error occurred generating the Material Source."), Node, EMessageSeverity::Error);
				}
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoneParameterName", "'None' is not a valid Material Parameter name."), Node, EMessageSeverity::Warning);
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MaterialBreakPinError", "Material Break Nodes must be linked to an input Material Node."), Node);
		}
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE

