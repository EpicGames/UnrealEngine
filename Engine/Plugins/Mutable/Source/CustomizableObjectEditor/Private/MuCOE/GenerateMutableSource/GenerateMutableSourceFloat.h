// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeScalar.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;

/** Convert a CustomizableObject Source Graph into a mutable source graph. */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> GenerateMutableSourceFloat(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);
