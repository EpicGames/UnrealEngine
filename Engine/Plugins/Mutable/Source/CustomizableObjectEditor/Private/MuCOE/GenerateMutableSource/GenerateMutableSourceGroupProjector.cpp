// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceGroupProjector.h"

#include "Engine/TextureLODSettings.h"
#include "Interfaces/ITargetPlatform.h"

#include "GenerateMutableSourceImage.h"
#include "Materials/MaterialInterface.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/MultilayerProjector.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceProjector.h"
#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeImageLayerColor.h"
#include "MuT/NodeImageMultiLayer.h"
#include "MuT/NodeImagePlainColor.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeImageResize.h"
#include "MuT/NodeImageSwitch.h"
#include "MuT/NodeImageSwizzle.h"
#include "MuT/NodeMeshSwitch.h"
#include "MuT/NodeRangeFromScalar.h"
#include "MuT/NodeScalarConstant.h"

class UPoseAsset;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::NodeImagePtr GenerateMutableSourceGroupProjector(const int32 NodeLOD, const int32 ImageIndex, UE::Mutable::Private::NodeMeshPtr MeshNode, FMutableGraphGenerationContext& GenerationContext,
	UCONodeSkeletalMeshSection* TypedNodeMat, UCONodeModifierExtendSkeletalMeshSection* TypedNodeExt, bool& bIsGroupProjectorImage,
	UTexture2D*& GroupProjectionReferenceTexture)
{
	check(static_cast<bool>(TypedNodeMat) != static_cast<bool>(TypedNodeExt)); // XOr. TypedNodeMat valid or TypedNodeExt valid. At least one valid.

	if (!MeshNode.get())
	{
		return UE::Mutable::Private::NodeImagePtr();
	}

	TArray<UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageProject>> ImageNodes;
	TArray<FGroupProjectorTempData> ImageNodes_ProjectorTempData;

	int32 TextureSize = 512;
	
	for (TArray<UCustomizableObjectNodeGroupProjectorParameter*> GroupProjectors : GenerationContext.CurrentGroupProjectors)
	{
		for (UCustomizableObjectNodeGroupProjectorParameter* ProjParamNode : GroupProjectors)
		{
			TOptional<FGroupProjectorTempData> OptionalProjectorTempData = GenerateMutableGroupProjector(ProjParamNode, GenerationContext);
			if (!OptionalProjectorTempData.IsSet())
			{
				continue;
			}

			FGroupProjectorTempData& ProjectorTempData = OptionalProjectorTempData.GetValue();
				
			const int32& DropProjectionTextureAtLOD = ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->DropProjectionTextureAtLOD;
			if (DropProjectionTextureAtLOD >= 0 && NodeLOD >= DropProjectionTextureAtLOD)
			{
				continue;
			}

			if (!GroupProjectionReferenceTexture)
			{
				GroupProjectionReferenceTexture = ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->ReferenceTexture;
			}

			const bool bProjectToImage = [&]
			{
				FString ParameterName;
				
				if (TypedNodeMat)
				{
					ParameterName = TypedNodeMat->GetParameterName(EMaterialParameterType::Texture, ImageIndex).ToString();
				}
				else
				{
					const FNodeMaterialParameterId ImageId = TypedNodeExt->GetParameterId(EMaterialParameterType::Texture, ImageIndex);
					return TypedNodeExt->UsesImage(ImageId);
				}

				return ParameterName == ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->MaterialChannelNameToConnect;
			}();
			
			if (bProjectToImage)
			{
				const bool bWarningReplacedImage = [&]
				{
					if (TypedNodeMat)
					{
						return TypedNodeMat->IsImageMutableMode(ImageIndex);
					}
					else
					{
						const FNodeMaterialParameterId ImageId = TypedNodeExt->GetParameterId(EMaterialParameterType::Texture, ImageIndex);
						return TypedNodeExt->UsesImage(ImageId);
					}
				}();
				
				if (bWarningReplacedImage)
				{
					const FString ImageName = [&]
					{
						if (TypedNodeMat)
						{
							return TypedNodeMat->GetParameterName(EMaterialParameterType::Texture, ImageIndex).ToString();
						}
						else
						{
							return TypedNodeExt->GetParameterName(EMaterialParameterType::Texture, ImageIndex).ToString();
						}
					}();
					
					FString msg = FString::Printf(TEXT("Material image [%s] is connected to an image but will be replaced by a Group Projector."), *ImageName);
					GenerationContext.Log(FText::FromString(msg), TypedNodeMat);
					continue;
				}
				
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageProject> ImageNode = new UE::Mutable::Private::NodeImageProject();
				bIsGroupProjectorImage = true;
				ImageNode->Layout = ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->UVLayout;

				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> ScalarNode = new UE::Mutable::Private::NodeScalarConstant;
					ScalarNode->Value = 120.f;
					ImageNode->AngleFadeStart = ScalarNode;
				}

				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> ScalarNode = new UE::Mutable::Private::NodeScalarConstant;
					ScalarNode->Value = 150.f;
					ImageNode->AngleFadeEnd = ScalarNode;
				}

				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshSwitch> MeshSwitchNode = new UE::Mutable::Private::NodeMeshSwitch;
				MeshSwitchNode->Parameter = ProjectorTempData.PoseOptionsParameter;
				MeshSwitchNode->Options.SetNum(ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->OptionPoses.Num() + 1);
				MeshSwitchNode->Options[0] = MeshNode;

				for (int32 SelectorIndex = 0; SelectorIndex < ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->OptionPoses.Num(); ++SelectorIndex)
				{
					if (ProjectorTempData.CustomizableObjectNodeGroupProjectorParameter->OptionPoses[SelectorIndex].OptionPose)
					{
						UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshApplyPose> NodeMeshApplyPose = CreateNodeMeshApplyPose(GenerationContext, MeshNode,
							ProjectorTempData.PoseBoneDataArray[SelectorIndex].ArrayBoneName, ProjectorTempData.PoseBoneDataArray[SelectorIndex].ArrayTransform);

						if (!NodeMeshApplyPose)
						{
							FString msg = FString::Printf(TEXT("Couldn't get bone transform information from a Pose Asset."));
							GenerationContext.Log(FText::FromString(msg), TypedNodeMat);
						}

						MeshSwitchNode->Options[SelectorIndex + 1] = NodeMeshApplyPose;
					}
					else
					{
						MeshSwitchNode->Options[SelectorIndex + 1] = MeshNode;
					}
				}

				ImageNode->Mesh = MeshSwitchNode;
				ImageNode->Projector = ProjectorTempData.NodeProjectorParameterPtr;
				ImageNode->Image = ProjectorTempData.NodeImagePtr;

				TextureSize = ProjectorTempData.TextureSize;
				ImageNode->ImageSize = FUintVector2(TextureSize, TextureSize);

				ImageNodes.Add(ImageNode);
				ImageNodes_ProjectorTempData.Add(ProjectorTempData);
			}
		}
	}
	
	if (ImageNodes.Num() == 0)
	{
		return UE::Mutable::Private::NodeImagePtr();
	}
	
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorConstant> ZeroColorNode = new UE::Mutable::Private::NodeColorConstant();
	ZeroColorNode->Value = FVector4f(0.f, 0.f, 0.f, 1.0f);

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImagePlainColor> ZeroPlainColorNode = new UE::Mutable::Private::NodeImagePlainColor;
	ZeroPlainColorNode->SizeX = TextureSize;
	ZeroPlainColorNode->SizeY = TextureSize;
	ZeroPlainColorNode->Color = ZeroColorNode;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> ZeroChannelNode = new UE::Mutable::Private::NodeImageSwizzle;
	ZeroChannelNode->SetFormat( UE::Mutable::Private::EImageFormat::L_UByte );
	ZeroChannelNode->Sources[0] = ZeroPlainColorNode;
	ZeroChannelNode->SourceChannels[0] = 2; // Just take a zeroed channel for the base alpha

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> OneConstantNode = new UE::Mutable::Private::NodeScalarConstant;
	OneConstantNode->Value = 1.f;

	UE::Mutable::Private::NodeImagePtr ResultAlpha = ZeroChannelNode;
	UE::Mutable::Private::NodeImagePtr ResultImage = ZeroPlainColorNode;

	for (int32 i = 0; i < ImageNodes.Num(); ++i)
	{
		if (i > 0) // Resize the projection texture if necessary after the first iteration
		{
			int32 NewTextureSize = ImageNodes_ProjectorTempData.IsValidIndex(i) ? ImageNodes_ProjectorTempData[i].CustomizableObjectNodeGroupProjectorParameter->ProjectionTextureSize : TextureSize;

			if (NewTextureSize <= 0 || !FMath::IsPowerOfTwo(NewTextureSize))
			{
				NewTextureSize = TextureSize;
			}

			if (NewTextureSize != TextureSize)
			{
				TextureSize = NewTextureSize;

				ZeroPlainColorNode = new UE::Mutable::Private::NodeImagePlainColor;
				ZeroPlainColorNode->SizeX = TextureSize;
				ZeroPlainColorNode->SizeY = TextureSize;
				ZeroPlainColorNode->Color = ZeroColorNode;
			}
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> ImageNodesAlphaChannelNode = new UE::Mutable::Private::NodeImageSwizzle;
		ImageNodesAlphaChannelNode->SetFormat( UE::Mutable::Private::EImageFormat::L_UByte );
		ImageNodesAlphaChannelNode->Sources[0] = ImageNodes[i];
		ImageNodesAlphaChannelNode->SourceChannels[0] = 3;

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorFromScalars> ColorFromScalars = new UE::Mutable::Private::NodeColorFromScalars;
		ColorFromScalars->X = ImageNodes_ProjectorTempData[i].NodeOpacityParameter;
		ColorFromScalars->Y = ImageNodes_ProjectorTempData[i].NodeOpacityParameter;
		ColorFromScalars->Z = ImageNodes_ProjectorTempData[i].NodeOpacityParameter;
		ColorFromScalars->W = OneConstantNode;

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageLayerColor> OpacityMultiLayerNode = new UE::Mutable::Private::NodeImageLayerColor;
		OpacityMultiLayerNode->Type = UE::Mutable::Private::EBlendType::BT_MULTIPLY;
		OpacityMultiLayerNode->Color = ColorFromScalars;
		OpacityMultiLayerNode->Base = ImageNodesAlphaChannelNode;
		//OpacityMultiLayerNode->SetMask(OneChannelNode); // No mask needed

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> MultiplySwizzleNode = new UE::Mutable::Private::NodeImageSwizzle;
		MultiplySwizzleNode->SetFormat( UE::Mutable::Private::EImageFormat::L_UByte );
		MultiplySwizzleNode->Sources[0] = OpacityMultiLayerNode;
		MultiplySwizzleNode->SourceChannels[0] = 0;

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageMultiLayer> BaseAlphaMultiLayerNode = new UE::Mutable::Private::NodeImageMultiLayer;
		BaseAlphaMultiLayerNode->Range = ImageNodes_ProjectorTempData[i].NodeRange;
		BaseAlphaMultiLayerNode->Type = UE::Mutable::Private::EBlendType::BT_LIGHTEN;
		BaseAlphaMultiLayerNode->Base = ResultAlpha;
		BaseAlphaMultiLayerNode->Blended = MultiplySwizzleNode;
		//BaseAlphaMultiLayerNode->SetMask(MultiplySwizzleNode); // No mask needed
		ResultAlpha = BaseAlphaMultiLayerNode;

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageMultiLayer> BaseMultiLayerNode = new UE::Mutable::Private::NodeImageMultiLayer;
		BaseMultiLayerNode->Range = ImageNodes_ProjectorTempData[i].NodeRange;
		BaseMultiLayerNode->Type = UE::Mutable::Private::EBlendType::BT_BLEND;
		BaseMultiLayerNode->Base = ResultImage;
		BaseMultiLayerNode->Blended = ImageNodes[i];
		BaseMultiLayerNode->Mask = MultiplySwizzleNode;
		ResultImage = BaseMultiLayerNode;
	}

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> SwizzleNodeR = new UE::Mutable::Private::NodeImageSwizzle;
	SwizzleNodeR->SetFormat(UE::Mutable::Private::EImageFormat::L_UByte);
	SwizzleNodeR->Sources[0] = ResultImage;
	SwizzleNodeR->SourceChannels[0] = 0;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> SwizzleNodeG = new UE::Mutable::Private::NodeImageSwizzle;
	SwizzleNodeG->SetFormat(UE::Mutable::Private::EImageFormat::L_UByte);
	SwizzleNodeG->Sources[0] = ResultImage;
	SwizzleNodeG->SourceChannels[0] = 1;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> SwizzleNodeB = new UE::Mutable::Private::NodeImageSwizzle;
	SwizzleNodeB->SetFormat(UE::Mutable::Private::EImageFormat::L_UByte);
	SwizzleNodeB->Sources[0] = ResultImage;
	SwizzleNodeB->SourceChannels[0] = 2;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> FinalSwizzleNode = new UE::Mutable::Private::NodeImageSwizzle;
	FinalSwizzleNode->SetFormat(UE::Mutable::Private::EImageFormat::RGBA_UByte);
	FinalSwizzleNode->Sources[0] = SwizzleNodeR;
	FinalSwizzleNode->SourceChannels[0] = 0;
	FinalSwizzleNode->Sources[1] = SwizzleNodeG;
	FinalSwizzleNode->SourceChannels[1] = 0;
	FinalSwizzleNode->Sources[2] = SwizzleNodeB;
	FinalSwizzleNode->SourceChannels[2] = 0;
	FinalSwizzleNode->Sources[3] = ResultAlpha;
	FinalSwizzleNode->SourceChannels[3] = 0;

	return FinalSwizzleNode;
}


TOptional<FGroupProjectorTempData> GenerateMutableGroupProjector(UCustomizableObjectNodeGroupProjectorParameter* ProjParamNode, FMutableGraphGenerationContext& GenerationContext)
{
	FGeneratedGroupProjectorsKey Key;
	Key.Node = ProjParamNode;
	Key.CurrentComponent = GenerationContext.CurrentSkeletalMeshComponent;
	
	if (FGroupProjectorTempData* Result = GenerationContext.GeneratedGroupProjectors.Find(Key))
	{
		return *Result;
	}

	FGroupProjectorTempData GroupProjectorTempData;
	
	const FCompilationOptions& CompilationOptions = GenerationContext.CompilationContext->Options;

	// The static cast works here because it's already known to be a UE::Mutable::Private::NodeProjectorParameter* because of the UE5 Cast in the previous line
	GroupProjectorTempData.NodeProjectorParameterPtr = static_cast<UE::Mutable::Private::NodeProjectorParameter*>(GenerateMutableSourceProjector(&ProjParamNode->OutputPin(), GenerationContext).get());

	if (GroupProjectorTempData.NodeProjectorParameterPtr)
	{
		FString ParameterName = ProjParamNode->GetParameterName(&GenerationContext.MacroNodesStack);

		// Use the projector parameter uid + num to identify parameters derived from this node
		// TODO(Max UE-247783): Get Node UID taking into account the macro context
		FGuid NumLayersParamUid = ProjParamNode->NodeGuid;
		NumLayersParamUid.D += 1;
		FGeneratedParameterKey NumLayerParamKey = { NumLayersParamUid, ParameterName };

		FGuid SelectedPoseParamUid = ProjParamNode->NodeGuid;
		SelectedPoseParamUid.D += 2;
		FGeneratedParameterKey SelectedPoseParamKey = { SelectedPoseParamUid, ParameterName };

		FGuid OpacityParamUid = ProjParamNode->NodeGuid;
		OpacityParamUid.D += 3;
		FGeneratedParameterKey OpacityParamKey = { OpacityParamUid, ParameterName };

		FGuid SelectedImageParamUid = ProjParamNode->NodeGuid;
		SelectedImageParamUid.D += 4;
		FGeneratedParameterKey SelectedImageParamKey = { SelectedImageParamUid, ParameterName };

		// Add to UCustomizableObjectNodeGroupProjectorParameter::OptionTextures those textures that are present in
		// UCustomizableObjectNodeGroupProjectorParameter::OptionTexturesDataTable avoiding any repeated element
		TArray<FGroupProjectorParameterImage> ArrayOptionTexture = ProjParamNode->GetFinalOptionTexturesNoRepeat();

		if ((ProjParamNode->OptionTexturesDataTable != nullptr) &&
			(ProjParamNode->DataTableTextureColumnName.ToString().IsEmpty() || (ProjParamNode->DataTableTextureColumnName.ToString() == "None")))
		{
			FString msg = FString::Printf(TEXT("The group projection node has a table assigned to the Option Images Data Table property, but no column to read textures is specified at the Data Table Texture Column Name property."));
			GenerationContext.Log(FText::FromString(msg), ProjParamNode, EMessageSeverity::Error, true);
		}

		GroupProjectorTempData.CustomizableObjectNodeGroupProjectorParameter = ProjParamNode;

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter> NodeScalarParam;
		if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter>* FindResult = GenerationContext.GeneratedScalarParameters.Find(NumLayerParamKey))
		{
			NodeScalarParam = *FindResult;
		}
		else
		{
			NodeScalarParam = new UE::Mutable::Private::NodeScalarParameter;
			FString NodeScalarParamName = ProjParamNode->GetParameterName(&GenerationContext.MacroNodesStack) + NUM_LAYERS_PARAMETER_POSTFIX;
			NodeScalarParam->Name = NodeScalarParamName;
			NodeScalarParam->UID = NumLayersParamUid.ToString();
			NodeScalarParam->DefaultValue =0.f;

			GenerationContext.ParameterUIDataMap.Add(NodeScalarParamName,
		FMutableParameterData(ProjParamNode->ParamUIMetadata, EMutableParameterType::Int));
					
			GenerationContext.GeneratedScalarParameters.Add(NumLayerParamKey, NodeScalarParam);
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeRangeFromScalar> NodeRangeFromScalar = new UE::Mutable::Private::NodeRangeFromScalar;
		NodeRangeFromScalar->Size = NodeScalarParam;
		GroupProjectorTempData.NodeRange = NodeRangeFromScalar;
		GroupProjectorTempData.NodeProjectorParameterPtr->SetRangeCount(1);
		GroupProjectorTempData.NodeProjectorParameterPtr->SetRange(0, NodeRangeFromScalar);
		
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter> PoseEnumParameterNode;
		if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter>* FindResult = GenerationContext.GeneratedEnumParameters.Find(SelectedPoseParamKey))
		{
			PoseEnumParameterNode = *FindResult;
		}
		else
		{
			PoseEnumParameterNode = new UE::Mutable::Private::NodeScalarEnumParameter;
			FString PoseNodeEnumParamName = ProjParamNode->GetParameterName(&GenerationContext.MacroNodesStack) + POSE_PARAMETER_POSTFIX;
			PoseEnumParameterNode->Name = PoseNodeEnumParamName;
			PoseEnumParameterNode->UID = SelectedPoseParamUid.ToString();
			PoseEnumParameterNode->Options.SetNum( ProjParamNode->OptionPoses.Num() + 1 );
			PoseEnumParameterNode->DefaultValue = 0;
			GroupProjectorTempData.PoseOptionsParameter = PoseEnumParameterNode;

			GenerationContext.GeneratedEnumParameters.Add(SelectedPoseParamKey, PoseEnumParameterNode);

			GenerationContext.ParameterUIDataMap.Add(PoseNodeEnumParamName,
		FMutableParameterData(ProjParamNode->ParamUIMetadata, EMutableParameterType::Int));
		}
		
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter> OpacityParameterNode;
		if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarParameter>* FindResult = GenerationContext.GeneratedScalarParameters.Find(OpacityParamKey))
		{
			OpacityParameterNode = *FindResult;
		}
		else
		{
			OpacityParameterNode = new UE::Mutable::Private::NodeScalarParameter;
			FString OpacityParameterNodeName = ProjParamNode->GetParameterName(&GenerationContext.MacroNodesStack) + OPACITY_PARAMETER_POSTFIX;
			OpacityParameterNode->Name = OpacityParameterNodeName;
			OpacityParameterNode->UID = OpacityParamUid.ToString();
			OpacityParameterNode->DefaultValue = 0.75f;
			OpacityParameterNode->Ranges.SetNum(1);
			OpacityParameterNode->Ranges[0] = NodeRangeFromScalar;

			GenerationContext.GeneratedScalarParameters.Add(OpacityParamKey, OpacityParameterNode);
					
			FMutableParamUIMetadata OpacityMetadata = ProjParamNode->ParamUIMetadata;
			OpacityMetadata.ObjectFriendlyName = FString("Opacity");
			GroupProjectorTempData.NodeOpacityParameter = OpacityParameterNode;
					
			FMutableParameterData ParameterUIData;
			ParameterUIData.ParamUIMetadata = OpacityMetadata;
			ParameterUIData.Type = EMutableParameterType::Float;
				
			GenerationContext.ParameterUIDataMap.Add(OpacityParameterNodeName, ParameterUIData);
		}
		
		if (ArrayOptionTexture.Num() == 0)
		{
			FString msg = FString::Printf(TEXT("The group projection node must have at least one option image connected to a texture or at least one valid element in Option Images Data Table."));
			GenerationContext.Log(FText::FromString(msg), ProjParamNode, EMessageSeverity::Error, true);
			return {};
		}

		PoseEnumParameterNode->Options[0].Value = 0.f;
		PoseEnumParameterNode->Options[0].Name = "Default pose";

		for (int32 PoseIndex = 0; PoseIndex < ProjParamNode->OptionPoses.Num(); ++PoseIndex)
		{
			PoseEnumParameterNode->Options[PoseIndex + 1].Value = (float)PoseIndex + 1.f;
			PoseEnumParameterNode->Options[PoseIndex + 1].Name = ProjParamNode->OptionPoses[PoseIndex].PoseName;

			TArray<FString> ArrayBoneName;
			TArray<FTransform> ArrayTransform;
			UPoseAsset* PoseAsset = ProjParamNode->OptionPoses[PoseIndex].OptionPose;

			if (PoseAsset == nullptr) // Check if the slot has a selected pose. Could be left empty by the user
			{
				FString msg = FString::Printf(TEXT("The group projection node must have a pose assigned on each Option Poses element."));
				GenerationContext.Log(FText::FromString(msg), ProjParamNode, EMessageSeverity::Error, true);
				return {};
			}

			check(GroupProjectorTempData.PoseBoneDataArray.Num() == PoseIndex);
			GroupProjectorTempData.PoseBoneDataArray.AddDefaulted(1);
			UCustomizableObjectNodeAnimationPose::StaticRetrievePoseInformation(PoseAsset, ProjParamNode->ReferenceSkeletalMesh,
				GroupProjectorTempData.PoseBoneDataArray[PoseIndex].ArrayBoneName, GroupProjectorTempData.PoseBoneDataArray[PoseIndex].ArrayTransform);
		}
	
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter> EnumParameterNode;
		if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarEnumParameter>* FindResult = GenerationContext.GeneratedEnumParameters.Find(SelectedImageParamKey))
		{
			EnumParameterNode = *FindResult;
		}
		else
		{
			EnumParameterNode = new UE::Mutable::Private::NodeScalarEnumParameter;

			FString NodeEnumParamName = ProjParamNode->GetParameterName(&GenerationContext.MacroNodesStack) + IMAGE_PARAMETER_POSTFIX;
			EnumParameterNode->Name = NodeEnumParamName;
			EnumParameterNode->UID = SelectedImageParamUid.ToString();
			EnumParameterNode->Options.SetNum(ArrayOptionTexture.Num());
			EnumParameterNode->DefaultValue = 0;
			EnumParameterNode->Ranges.SetNum(1);
			EnumParameterNode->Ranges[0] = NodeRangeFromScalar;

			GenerationContext.GeneratedEnumParameters.Add(SelectedImageParamKey, EnumParameterNode);

			FMutableParameterData ParameterUIData;
			ParameterUIData.ParamUIMetadata = ProjParamNode->ParamUIMetadata;
			ParameterUIData.Type = EMutableParameterType::Int;
			ParameterUIData.IntegerParameterGroupType = ECustomizableObjectGroupType::COGT_ONE;
			ParameterUIData.ParamUIMetadata.ExtraInformation.Add(FString("UseThumbnails"));

			for (int OptionTextureIndex = 0; OptionTextureIndex < ArrayOptionTexture.Num(); ++OptionTextureIndex)
			{
				EnumParameterNode->Options[OptionTextureIndex].Value = (float)OptionTextureIndex;
				EnumParameterNode->Options[OptionTextureIndex].Name = ArrayOptionTexture[OptionTextureIndex].OptionName;

				FMutableParamUIMetadata OptionMetadata = ParameterUIData.ParamUIMetadata;
				OptionMetadata.UIThumbnail = ArrayOptionTexture[OptionTextureIndex].OptionTexture;
				ParameterUIData.ArrayIntegerParameterOption.Add(
					ArrayOptionTexture[OptionTextureIndex].OptionName,
					FIntegerParameterUIData(OptionMetadata));
			}

			GenerationContext.ParameterUIDataMap.Add(NodeEnumParamName, ParameterUIData);
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwitch> SwitchNode = new UE::Mutable::Private::NodeImageSwitch;
		SwitchNode->Parameter = EnumParameterNode;
		SwitchNode->Options.SetNum(ArrayOptionTexture.Num());

		for (int32 SelectorIndex = 0; SelectorIndex < ArrayOptionTexture.Num(); ++SelectorIndex)
		{
			if (const TObjectPtr<UTexture2D>& Texture = ArrayOptionTexture[SelectorIndex].OptionTexture)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ImageNode = new UE::Mutable::Private::NodeImageConstant();
				ImageNode->Image = GenerateImageConstant(ArrayOptionTexture[SelectorIndex].OptionTexture, GenerationContext, false);

				const uint32 MipsToSkip = ComputeLODBiasForTexture(GenerationContext, *Texture, ProjParamNode->ReferenceTexture);
				SwitchNode->Options[SelectorIndex] = ResizeTextureByNumMips(ImageNode, MipsToSkip);

				// Calculate the number of mips to tag as high res for this image.
				if (ProjParamNode->ReferenceTexture)
				{
					const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
					const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(ProjParamNode->ReferenceTexture->LODGroup);

					ImageNode->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
					ImageNode->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
					ImageNode->SourceDataDescriptor.NumNonOptionalLODs = CompilationOptions.MinDiskMips;

					const FString TextureName = GetNameSafe(Texture).ToLower();
					ImageNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*TextureName), TextureName.Len() * sizeof(FString::ElementType));
				}
			}
			else
			{
				FString msg = FString::Printf(TEXT("The group projection node must have a texture for all the options. Please set a texture for all the options."));
				GenerationContext.Log(FText::FromString(msg), ProjParamNode);
			}
		}

		int32 TextureSize = ProjParamNode->ProjectionTextureSize > 0 ? ProjParamNode->ProjectionTextureSize : 512;

		// If TextureSize is not power of two, round up to the next power of two 
		if (!FMath::IsPowerOfTwo(TextureSize))
		{
			TextureSize = FMath::RoundUpToPowerOfTwo(TextureSize);
		}

		GroupProjectorTempData.TextureSize = TextureSize;
		GroupProjectorTempData.NodeImagePtr = SwitchNode;
	}

	GenerationContext.GeneratedGroupProjectors.Add(Key, GroupProjectorTempData);
	
	return GroupProjectorTempData;
}

#undef LOCTEXT_NAMESPACE

