// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "MuT/NodeSurfaceNew.h"

class UCustomizableObjectNodeGroupProjectorParameter;
struct FGroupProjectorTempData;
class UCONodeModifierExtendSkeletalMeshSection;
class FString;
class UCONodeSkeletalMeshSection;
class UCONodeSkeletalMeshSection;
class UCustomizableObjectNodeObjectGroup;
class UEdGraphPin;
class UTexture2D;
struct FMutableGraphGenerationContext;


UE::Mutable::Private::NodeImagePtr GenerateMutableSourceGroupProjector(const int32 NodeLOD, const int32 ImageIndex, UE::Mutable::Private::NodeMeshPtr MeshNode, FMutableGraphGenerationContext& GenerationContext,
	UCONodeSkeletalMeshSection* TypedNodeMat, UCONodeModifierExtendSkeletalMeshSection* TypedNodeExt, bool& bIsGroupProjectorImage,
	UTexture2D*& GroupProjectionReferenceTexture);


TOptional<FGroupProjectorTempData> GenerateMutableGroupProjector(UCustomizableObjectNodeGroupProjectorParameter* ProjParamNode, FMutableGraphGenerationContext& GenerationContext);

