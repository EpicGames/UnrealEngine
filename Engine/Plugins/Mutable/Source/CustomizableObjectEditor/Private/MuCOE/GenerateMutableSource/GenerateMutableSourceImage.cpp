// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceImage.h"

#include "Engine/TextureDefines.h"
#include "GenerateMutableSourceExternal.h"
#include "Engine/TextureLODSettings.h"
#include "Modules/ModuleManager.h"

#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceColor.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMaterial.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceProjector.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeMaterialBreak.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTexture.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureBinarise.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureColorMap.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromChannels.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromColor.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromFloats.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureInterpolate.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureInvert.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureLayer.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureProject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureToChannels.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureTransform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureSaturate.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCOE/Nodes/CONodeExternalOperation.h"
#include "MuT/NodeImageBinarise.h"
#include "MuT/NodeImageColorMap.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeImageConform.h"
#include "MuT/NodeImageInterpolate.h"
#include "MuT/NodeImageInvert.h"
#include "MuT/NodeImageLayer.h"
#include "MuT/NodeImageLayerColor.h"
#include "MuT/NodeImageObjectParameter.h"
#include "MuT/NodeImagePlainColor.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeImageResize.h"
#include "MuT/NodeImageSaturate.h"
#include "MuT/NodeImageSwitch.h"
#include "MuT/NodeImageSwizzle.h"
#include "MuT/NodeImageTable.h"
#include "MuT/NodeImageTransform.h"
#include "MuT/NodeImageVariation.h"
#include "MuT/NodeImageConstant.h"
#include "MuT/NodeImageExternal.h"
#include "MuT/NodeImageConvert.h"
#include "MuT/NodeScalarConstant.h"

#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuT/UnrealPixelFormatOverride.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ResizeTextureByNumMips(const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage>& ImageConstant, int32 MipsToSkip)
{
	if (MipsToSkip > 0)
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageResize> ImageResize = new UE::Mutable::Private::NodeImageResize();
		ImageResize->Base = ImageConstant;
		ImageResize->bRelative = true;
		const float Factor = FMath::Pow(0.5f, MipsToSkip);
		ImageResize->SizeX = Factor;
		ImageResize->SizeY = Factor;
		return ImageResize;
	}

	return ImageConstant;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> GenerateMutableSourceImage(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, int32 ReferenceTextureSize)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceImage), *Pin, *Node, GenerationContext, true); // Reinterpret cast to void required on some compilers.
	Key.ReferenceTextureSize = ReferenceTextureSize;
	
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeImage*>(Generated->Node.get());
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For example, MacroInstanceNodes
	bool bCacheNode = true;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> Result;

	const FCompilationOptions& CompilationOptions = GenerationContext.CompilationContext->Options;

	if (const UCustomizableObjectNodeTexture* TypedNodeTex = Cast<UCustomizableObjectNodeTexture>(Node))
	{
		if (Pin == TypedNodeTex->TexturePin.Get())
		{
			if (TypedNodeTex->Texture)
			{
				if (UTexture2D* BaseTexture = Cast<UTexture2D>(TypedNodeTex->Texture))
				{
					// Check the specific image cache
					FImageKey ImageKey = FImageKey(Pin);
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ImageNode;
				
					const uint32 MipsToSkip = ComputeLODBiasForTexture(GenerationContext, *BaseTexture, nullptr, ReferenceTextureSize);

					if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage>* Cached = GenerationContext.GeneratedImages.Find(ImageKey))
					{
						ImageNode = *Cached;
					}
					else
					{
						UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ConstantImageNode = new UE::Mutable::Private::NodeImageConstant;
						ConstantImageNode->Image = GenerateImageConstant(BaseTexture, GenerationContext, false);
					
						const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
						const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(BaseTexture->LODGroup);

						ConstantImageNode->SourceDataDescriptor.bNoMipmaps = BaseTexture->MipGenSettings == TextureMipGenSettings::TMGS_NoMipmaps;
						ConstantImageNode->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
						ConstantImageNode->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
						ConstantImageNode->SourceDataDescriptor.NumNonOptionalLODs = CompilationOptions.MinDiskMips;

						const FString TextureName = GetNameSafe(BaseTexture).ToLower();
						ConstantImageNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*TextureName), TextureName.Len() * sizeof(FString::ElementType));

						ImageNode = ConstantImageNode;

						GenerationContext.GeneratedImages.Add(ImageKey, ImageNode);
					}

					Result = ResizeTextureByNumMips(ImageNode, MipsToSkip);
				}
				else
				{
					GenerationContext.Log(LOCTEXT("TextureNot2D", "Texture is not a Texture 2D."), Node, EMessageSeverity::Warning);
				}
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MissingImage", "Missing Texture."), Node);
			}
		}
		else if (Pin == TypedNodeTex->PassthroughTexturePin.Get())
		{
			if (TypedNodeTex->Texture)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ImageNode = new UE::Mutable::Private::NodeImageConstant();
				ImageNode->Image = GenerateImageConstant(TypedNodeTex->Texture, GenerationContext, true);

				Result = ImageNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MissingImage", "Missing Texture."), Node);
			}
		}
		else
		{
			unimplemented();
		}
	}
	
	else if (const UCustomizableObjectNodeTextureParameter* TypedNodeParam = Cast<UCustomizableObjectNodeTextureParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter> ImageParameterNode = GenerateMutableSourceImageParameter(Pin, GenerationContext);

		const bool bPassthrough = Pin == TypedNodeParam->PassthroughPin.Get();
		if (bPassthrough)
		{
			Result = ImageParameterNode;
		}
		else
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConvert> ImagePassthroughConvertNode = new UE::Mutable::Private::NodeImageConvert();
			ImagePassthroughConvertNode->ImageParameter = ImageParameterNode;

			FUintVector2 TextureSize(TypedNodeParam->TextureSizeX, TypedNodeParam->TextureSizeY);

			if (const UTexture2D* ReferenceTexture = TypedNodeParam->ReferenceValue)
			{
				// Get the cooked texture size with Cinematic Mips since that's what will be cooked for the input textures.
				constexpr bool bSkipCinematicMips = false;
				GetCookedTextureSize(GenerationContext, ReferenceTexture, TextureSize.X, TextureSize.Y, bSkipCinematicMips);

				// TODO: Remove Hack
				// Hack - Due to legacy reasons, Mutable adds a LODBias at the top of the root and propagates it to all leaves. 
				// This is wrong and has to be removed. Until then we must compensate the LODBias added at the root to avoid size mismatches.
				if (GenerationContext.CurrentImageProperties)
				{
					TextureSize.X = TextureSize.X << GenerationContext.CurrentImageProperties->BaseLODBias;
					TextureSize.Y = TextureSize.Y << GenerationContext.CurrentImageProperties->BaseLODBias;
				}
			}
			else
			{
				const int32 MaxNodeTextureSize = FMath::Max(TypedNodeParam->TextureSizeX, TypedNodeParam->TextureSizeY);
				if (MaxNodeTextureSize <= 0)
				{
					TextureSize.X = TextureSize.Y = 1;
					GenerationContext.Log(LOCTEXT("TextureParameterSize0", "Texture size not specified. Add a reference texture or set a valid value to the Texture Size variables."), Node);
				}
				else if (ReferenceTextureSize > 0 && ReferenceTextureSize < MaxNodeTextureSize)
				{
					const uint32 MipsToSkip = FMath::CeilLogTwo(MaxNodeTextureSize) - FMath::CeilLogTwo(ReferenceTextureSize);
					TextureSize.X = FMath::Max(TextureSize.X >> MipsToSkip, (uint32)1);
					TextureSize.Y = FMath::Max(TextureSize.Y >> MipsToSkip, (uint32)1);
				}
			}

			// This node is discarded, only used to get the formats.
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageFormat> ImageFormat = GenerateMutableImageFormat(ImagePassthroughConvertNode, TypedNodeParam->ReferenceValue, GenerationContext, TypedNodeParam);
		
			ImagePassthroughConvertNode->ImageDescriptor.m_format = ImageFormat->Format;
			ImagePassthroughConvertNode->ImageDescriptor.FormatIfAlpha = ImageFormat->FormatIfAlpha;
			
			// Set expected image formats and sizes in game.
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConform> ImageConform = new UE::Mutable::Private::NodeImageConform();

			ImageConform->Source = ImagePassthroughConvertNode;

			ImageConform->ConformDescriptor = UE::Mutable::Private::FImageDesc(
				UE::Mutable::Private::FImageSize(TextureSize.X, TextureSize.Y),
				ImageFormat->Format,
				ImageFormat->FormatIfAlpha,
				0);


			Result = ImageConform;
		}
	}

	else if (const ICustomizableObjectNodeMeshInterface* TypedNodeMesh = Cast<ICustomizableObjectNodeMeshInterface>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ImageNode = new UE::Mutable::Private::NodeImageConstant();
		Result = ImageNode;

		if (UTexture2D* Texture = TypedNodeMesh->FindTextureForPin(Pin))
		{
			ImageNode->Image = GenerateImageConstant(Texture, GenerationContext, false);

			const uint32 MipsToSkip = ComputeLODBiasForTexture(GenerationContext, *Texture, nullptr, ReferenceTextureSize);
			Result = ResizeTextureByNumMips(ImageNode, MipsToSkip);

			const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
			const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(Texture->LODGroup);

			ImageNode->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
			ImageNode->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
			ImageNode->SourceDataDescriptor.NumNonOptionalLODs = CompilationOptions.MinDiskMips;

			const FString TextureName = GetNameSafe(Texture).ToLower();
			ImageNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*TextureName), TextureName.Len() * sizeof(FString::ElementType));
		}
		else
		{
			Result = UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage>();
		}
	}

	else if (const UCustomizableObjectNodeTextureInterpolate* TypedNodeInterp = Cast<UCustomizableObjectNodeTextureInterpolate>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageInterpolate> ImageNode = new UE::Mutable::Private::NodeImageInterpolate();
		Result = ImageNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeInterp->FactorPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FactorNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			ImageNode->Factor = FactorNode;
		}

		int currentTarget = 0;
		for (int LayerIndex = 0; LayerIndex < TypedNodeInterp->GetNumTargets(); ++LayerIndex)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeInterp->Targets(LayerIndex)))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> TargetNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);

				if (TargetNode)
				{
					ImageNode->Targets.SetNum(currentTarget + 1);
					ImageNode->Targets[currentTarget] = TargetNode;
					++currentTarget;
				}
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureLayer* TypedNodeLayer = Cast<UCustomizableObjectNodeTextureLayer>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeLayer->BasePin.Get()))
		{
			Result = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
		}

		for (int LayerIndex = 0; LayerIndex < TypedNodeLayer->GetNumLayers(); ++LayerIndex)
		{
			if (const UEdGraphPin* OtherPin = FollowInputPin(*TypedNodeLayer->GetLayerPin(LayerIndex)))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> MaskNode = nullptr;
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeLayer->GetMaskPin(LayerIndex)))
				{
					MaskNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
				}

				UE::Mutable::Private::EBlendType Type = UE::Mutable::Private::EBlendType::BT_BLEND;
				switch (TypedNodeLayer->Layers[LayerIndex].Effect)
				{
				case COTLE_MODULATE: Type = UE::Mutable::Private::EBlendType::BT_BLEND; break;
				case COTLE_MULTIPLY: Type = UE::Mutable::Private::EBlendType::BT_MULTIPLY; break;
				case COTLE_SOFTLIGHT: Type = UE::Mutable::Private::EBlendType::BT_SOFTLIGHT; break;
				case COTLE_HARDLIGHT: Type = UE::Mutable::Private::EBlendType::BT_HARDLIGHT; break;
				case COTLE_DODGE: Type = UE::Mutable::Private::EBlendType::BT_DODGE; break;
				case COTLE_BURN: Type = UE::Mutable::Private::EBlendType::BT_BURN; break;
				case COTLE_SCREEN: Type = UE::Mutable::Private::EBlendType::BT_SCREEN; break;
				case COTLE_OVERLAY: Type = UE::Mutable::Private::EBlendType::BT_OVERLAY; break;
				case COTLE_ALPHA_OVERLAY: Type = UE::Mutable::Private::EBlendType::BT_LIGHTEN; break;
				case COTLE_NORMAL_COMBINE: Type = UE::Mutable::Private::EBlendType::BT_NORMAL_COMBINE; break;
				default:
					GenerationContext.Log(LOCTEXT("UnsupportedImageEffect", "Texture layer effect not supported. Setting to 'Blend'."), Node);
					break;
				}

				if (Type == UE::Mutable::Private::EBlendType::BT_BLEND && !MaskNode)
				{
					GenerationContext.Log(LOCTEXT("ModulateWithoutMask", "Texture layer effect uses Modulate without a mask. It will replace everything below it!"), Node);
				}
				
				if (OtherPin->PinType.PinCategory == Schema->PC_Texture)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> BlendNode = GenerateMutableSourceImage(OtherPin, GenerationContext, ReferenceTextureSize);

					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageLayer> LayerNode = new UE::Mutable::Private::NodeImageLayer;
					LayerNode->Type = Type;
					LayerNode->Base = Result;
					LayerNode->Blended = BlendNode;
					LayerNode->Mask = MaskNode;
					Result = LayerNode;
				}
				else if (OtherPin->PinType.PinCategory == Schema->PC_Color)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ColorNode = GenerateMutableSourceColor(OtherPin, GenerationContext);

					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageLayerColor> LayerNode = new UE::Mutable::Private::NodeImageLayerColor;
					LayerNode->Type = Type;
					LayerNode->Base = Result;
					LayerNode->Color = ColorNode;
					LayerNode->Mask = MaskNode;
					Result = LayerNode;
				}			
				else
				{	
					unimplemented();
				}

				// We need it here because we create multiple nodes.
				Result->SetMessageContext(Node);
			}
		}
	}

	else if (const UCONodeSwitch* TypedNodeTextureSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Texture))
	{
		Result = [&]()
		{
			const UEdGraphPin* SwitchParameter = TypedNodeTextureSwitch->SwitchParameterPinReference.Get();

			// Check Switch Parameter arity preconditions.
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SwitchParameter))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
				// Switch Param not generated
				if (!SwitchParam)
				{
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
					GenerationContext.Log(Message, Node);
					
					return Result;
				}

				if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
				{
					const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
					GenerationContext.Log(Message, Node);

					return Result;
				}

				{
					const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());

					if (!DoOptionsMatchEnum(*TypedNodeTextureSwitch, *EnumParameter))
					{
						const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
						GenerationContext.Log(Message, Node);
						Node->SetRefreshNodeWarning();
					}
				}

				const int32 NumSwitchOptions = TypedNodeTextureSwitch->SwitchPins.Num();
				
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwitch> SwitchNode = new UE::Mutable::Private::NodeImageSwitch;
				SwitchNode->Parameter = SwitchParam;
				SwitchNode->Options.SetNum(NumSwitchOptions);

				for (int SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
				{
					if (const UEdGraphPin* TexturePin = FollowInputPin(*TypedNodeTextureSwitch->SwitchPins[SelectorIndex].Get()))
					{
						SwitchNode->Options[SelectorIndex] = GenerateMutableSourceImage(TexturePin, GenerationContext, ReferenceTextureSize);
					}
					else
					{
						const FText Message = LOCTEXT("MissingTexture", "Unable to generate texture switch node. Required connection not found.");
						GenerationContext.Log(Message, Node);
						return Result;
					}
				}

				Result = SwitchNode;
				return Result;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), Node);
				return Result;
			}
		}(); // invoke lambda.
	}

	else if (const UCustomizableObjectNodeTextureVariation* TypedNodeImageVar = Cast<const UCustomizableObjectNodeTextureVariation>(Node))
	{
		// UCustomizableObjectNodePassThroughTextureVariation nodes are also handled here
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageVariation> TextureNode = new UE::Mutable::Private::NodeImageVariation();
		Result = TextureNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeImageVar->DefaultPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ChildNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			if (ChildNode)
			{
				TextureNode->DefaultImage = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("TextureFailed", "Texture generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeImageVar->GetNumVariations();
		TextureNode->Variations.SetNum(NumVariations);
		
		for (int32 VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			const UEdGraphPin* VariationPin = TypedNodeImageVar->VariationPin(VariationIndex);
			if (!VariationPin)
			{
				continue;
			}

			TextureNode->Variations[VariationIndex].Tag = TypedNodeImageVar->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ChildNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
				TextureNode->Variations[VariationIndex].Image = ChildNode;
			}
		}
	}

	else if (const UCustomizableObjectNodeTextureFromColor* TypedNodeFromColor = Cast<UCustomizableObjectNodeTextureFromColor>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> color;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFromColor->ColorPin.Get()))
		{
			color = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImagePlainColor> ImageFromColor = new UE::Mutable::Private::NodeImagePlainColor;
		Result = ImageFromColor;

		if (color)
		{
			ImageFromColor->Color = color;
		}

		if (ReferenceTextureSize > 0)
		{
			ImageFromColor->SizeX = ReferenceTextureSize;
			ImageFromColor->SizeY = ReferenceTextureSize;
		}
	}

	else if (const UCustomizableObjectNodeTextureFromFloats* TypedNodeFromFloats = Cast<UCustomizableObjectNodeTextureFromFloats>(Node))
	{		
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColorFromScalars> Color = new UE::Mutable::Private::NodeColorFromScalars();

		if (const UEdGraphPin* RPin = FollowInputPin(*TypedNodeFromFloats->RPin()))
		{
			Color->X = GenerateMutableSourceFloat(RPin, GenerationContext);
		}

		if (const UEdGraphPin* GPin = FollowInputPin(*TypedNodeFromFloats->GPin()))
		{
			Color->Y = GenerateMutableSourceFloat(GPin, GenerationContext);
		}

		if (const UEdGraphPin* BPin = FollowInputPin(*TypedNodeFromFloats->BPin()))
		{
			Color->Z = GenerateMutableSourceFloat(BPin, GenerationContext);
		}

		if (const UEdGraphPin* APin = FollowInputPin(*TypedNodeFromFloats->APin()))
		{
			Color->W = GenerateMutableSourceFloat(APin, GenerationContext);
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImagePlainColor> ImageFromColor = new UE::Mutable::Private::NodeImagePlainColor;
		Result = ImageFromColor;

		if (Color)
		{
			ImageFromColor->Color = Color;
		}

		if (ReferenceTextureSize > 0)
		{
			ImageFromColor->SizeX = ReferenceTextureSize;
			ImageFromColor->SizeY = ReferenceTextureSize;
		}
	}

	else if (const UCustomizableObjectNodeTextureFromChannels* TypedNodeFrom = Cast<UCustomizableObjectNodeTextureFromChannels>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> RNode, GNode, BNode, ANode;
		bool RGB = false;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->RPin()))
		{
			RNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			RGB = true;
		}
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->GPin()))
		{
			GNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			RGB = true;
		}
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->BPin()))
		{
			BNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			RGB = true;
		}
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeFrom->APin()))
		{
			ANode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> SwizzleNode = new UE::Mutable::Private::NodeImageSwizzle;
		Result = SwizzleNode;

		if (RGB && ANode)
		{
			SwizzleNode->SetFormat(UE::Mutable::Private::EImageFormat::RGBA_UByte);
			SwizzleNode->Sources[0] = RNode;
			SwizzleNode->SourceChannels[0] = 0;
			SwizzleNode->Sources[1] = GNode;
			SwizzleNode->SourceChannels[1] = 0;
			SwizzleNode->Sources[2] = BNode;
			SwizzleNode->SourceChannels[2] = 0;
			SwizzleNode->Sources[3] = ANode;
			SwizzleNode->SourceChannels[3] = 0;
		}
		else if (RGB)
		{
			SwizzleNode->SetFormat(UE::Mutable::Private::EImageFormat::RGB_UByte);
			SwizzleNode->Sources[0] = RNode;
			SwizzleNode->SourceChannels[0] = 0;
			SwizzleNode->Sources[1] = GNode;
			SwizzleNode->SourceChannels[1] = 0;
			SwizzleNode->Sources[2] = BNode;
			SwizzleNode->SourceChannels[2] = 0;
		}
		else if (RNode)
		{
			SwizzleNode->SetFormat(UE::Mutable::Private::EImageFormat::L_UByte);
			SwizzleNode->Sources[0] = RNode;
			SwizzleNode->SourceChannels[0] = 0;
		}
		else if (ANode)
		{
			SwizzleNode->SetFormat(UE::Mutable::Private::EImageFormat::L_UByte);
			SwizzleNode->Sources[0] = ANode;
			SwizzleNode->SourceChannels[0] = 0;
		}
	}

	else if (const UCustomizableObjectNodeTextureToChannels* TypedNodeTo = Cast<UCustomizableObjectNodeTextureToChannels>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> BaseNode;
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTo->InputPin()))
		{
			BaseNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwizzle> SwizzleNode = new UE::Mutable::Private::NodeImageSwizzle;
		Result = SwizzleNode;
		SwizzleNode->SetFormat(UE::Mutable::Private::EImageFormat::L_UByte);
		SwizzleNode->Sources[0] = BaseNode;

		if (Pin == TypedNodeTo->RPin())
		{
			SwizzleNode->SourceChannels[0] = 0;
		}
		else if (Pin == TypedNodeTo->GPin())
		{
			SwizzleNode->SourceChannels[0] = 1;
		}
		else if (Pin == TypedNodeTo->BPin())
		{
			SwizzleNode->SourceChannels[0] = 2;
		}
		else if (Pin == TypedNodeTo->APin())
		{
			SwizzleNode->SourceChannels[0] = 3;
		}
		else
		{
			check(false);
		}
	}

	else if (const UCustomizableObjectNodeTextureProject* TypedNodeProject = Cast<UCustomizableObjectNodeTextureProject>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageProject> ImageNode = new UE::Mutable::Private::NodeImageProject();
		Result = ImageNode;

		if (!FollowInputPin(*TypedNodeProject->MeshPin()))
		{
			GenerationContext.Log(LOCTEXT("MissingMeshInProjector", "Texture projector does not have a Mesh. It will be ignored. "), Node, EMessageSeverity::Warning);
			Result = nullptr;
		}

		ImageNode->Layout = TypedNodeProject->Layout;

		// Calculate the max TextureSize allowed using the ReferenceTextureSize and the Reference texture from the node
		int32 MaxReferenceTextureSizeInGame = ReferenceTextureSize;
		if (TypedNodeProject->ReferenceTexture)
		{
			//const UTextureLODSettings& TextureLODSettings = CompilationOptions.TargetPlatform->GetTextureLODSettings();
			//MaxReferenceTextureSizeInGame = GetTextureSizeInGame(*TypedNodeProject->ReferenceTexture, TextureLODSettings);

			FUintVector2 TextureSize(0, 0);
			constexpr bool bSkipCinematicMips = false;
			GetCookedTextureSize(GenerationContext, TypedNodeProject->ReferenceTexture, TextureSize.X, TextureSize.Y, bSkipCinematicMips);
			
			MaxReferenceTextureSizeInGame = FMath::Max(TextureSize.X, TextureSize.Y);
		}

		FUintVector2 TextureSize(TypedNodeProject->TextureSizeX, TypedNodeProject->TextureSizeY);

		// Max TextureSize allowed
		const int32 MaxProjectedTextureSizeInGame = ReferenceTextureSize > 0 && ReferenceTextureSize < MaxReferenceTextureSizeInGame ? ReferenceTextureSize : MaxReferenceTextureSizeInGame;

		const int32 ProjectorNodeTextureSize = FMath::Max(TextureSize.X, TextureSize.Y);
		if (ProjectorNodeTextureSize > 0 && MaxProjectedTextureSizeInGame > 0 && ProjectorNodeTextureSize > MaxProjectedTextureSizeInGame)
		{
			const int32 NumMips = FMath::CeilLogTwo(ProjectorNodeTextureSize) + 1;
			const int32 MaxNumMips = FMath::CeilLogTwo(MaxProjectedTextureSizeInGame) + 1;

			TextureSize.X = TextureSize.X >> (NumMips - MaxNumMips);
			TextureSize.Y = TextureSize.Y >> (NumMips - MaxNumMips);
		}

		ImageNode->ImageSize = TextureSize;

		ImageNode->bEnableTextureSeamCorrection = TypedNodeProject->bEnableTextureSeamCorrection;
		ImageNode->bIsRGBFadingEnabled = TypedNodeProject->bEnableAngleFadeOutForRGB;
		ImageNode->bIsAlphaFadingEnabled = TypedNodeProject->bEnableAngleFadeOutForAlpha;
		ImageNode->SamplingMethod = Invoke(
			[](ETextureProjectSamplingMethod Method) -> UE::Mutable::Private::ESamplingMethod 
			{
				switch(Method)
				{
				case ETextureProjectSamplingMethod::Point:    return UE::Mutable::Private::ESamplingMethod::Point;
				case ETextureProjectSamplingMethod::BiLinear: return UE::Mutable::Private::ESamplingMethod::BiLinear;
				default: { check(false); return UE::Mutable::Private::ESamplingMethod::Point; }
				}
			}, 
			TypedNodeProject->SamplingMethod);

		ImageNode->MinFilterMethod = Invoke(
			[](ETextureProjectMinFilterMethod Method) -> UE::Mutable::Private::EMinFilterMethod 
			{
				switch(Method)
				{
				case ETextureProjectMinFilterMethod::None:               return UE::Mutable::Private::EMinFilterMethod::None;
				case ETextureProjectMinFilterMethod::TotalAreaHeuristic: return UE::Mutable::Private::EMinFilterMethod::TotalAreaHeuristic;
				default: { check(false); return UE::Mutable::Private::EMinFilterMethod::None; }
				}
			}, 
			TypedNodeProject->MinFilterMethod);

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProject->AngleFadeStartPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ScalarNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			ImageNode->AngleFadeStart = ScalarNode;
		}
		else
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> ScalarNode = new UE::Mutable::Private::NodeScalarConstant;
			float value = FCString::Atof(*TypedNodeProject->AngleFadeStartPin()->DefaultValue);
			ScalarNode->Value = value;
			ImageNode->AngleFadeStart = ScalarNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProject->AngleFadeEndPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ScalarNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			ImageNode->AngleFadeEnd = ScalarNode;
		}
		else
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalarConstant> ScalarNode = new UE::Mutable::Private::NodeScalarConstant;
			float value = FCString::Atof(*TypedNodeProject->AngleFadeEndPin()->DefaultValue);
			ScalarNode->Value = value;
			ImageNode->AngleFadeEnd = ScalarNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProject->MeshPin()))
		{
			FLayoutGenerationFlags LayoutGenerationFlags;
			LayoutGenerationFlags.TexturePinModes.Init(EPinMode::Mutable, TEXSTREAM_MAX_NUM_UVCHANNELS);
			GenerationContext.LayoutGenerationFlags.Push(LayoutGenerationFlags);
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> MeshNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, false);
			GenerationContext.LayoutGenerationFlags.Pop();
			ImageNode->Mesh = MeshNode;
		}

		// TODO: Remove with the Texture Parameters lod bias hack. 
		// In this case the projected image has no relation with the source image. The properties should be discarded.
		// The only place where the context generation properties is used is to revert the LodBias applied at root for
		// image parameters.
		FGeneratedImageProperties* PrevImageProperties = GenerationContext.CurrentImageProperties;
		GenerationContext.CurrentImageProperties = nullptr;
		ON_SCOPE_EXIT
		{
			GenerationContext.CurrentImageProperties = PrevImageProperties;
		};

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProject->MeshMaskPin()))
		{		
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> MeshMaskNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			ImageNode->Mask = MeshMaskNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProject->ProjectorPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjector> ProjectorNode = GenerateMutableSourceProjector(ConnectedPin, GenerationContext);
			ImageNode->Projector = ProjectorNode;
		}

		int TexIndex = -1;// TypedNodeProject->OutputPins.Find((UEdGraphPin*)Pin);

		for (int32 i = 0; i < TypedNodeProject->GetNumOutputs(); ++i)
		{
			if (TypedNodeProject->OutputPins(i) == Pin)
			{
				TexIndex = i;
			}
		}

		check(TexIndex >= 0 && TexIndex < TypedNodeProject->GetNumTextures());

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeProject->TexturePins(TexIndex)))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> SourceNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, FMath::Max(TextureSize.X, TextureSize.Y));
			ImageNode->Image = SourceNode;
		}

	}

	else if (const UCustomizableObjectNodeTextureBinarise* TypedNodeTexBin = Cast<UCustomizableObjectNodeTextureBinarise>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageBinarise> BinariseNode = new UE::Mutable::Private::NodeImageBinarise();
		Result = BinariseNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTexBin->GetBaseImagePin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> BaseImageNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			BinariseNode->Base = BaseImageNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTexBin->GetThresholdPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ThresholdNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
			BinariseNode->Threshold = ThresholdNode;
		}
	}

	else if (const UCustomizableObjectNodeTextureInvert* TypedNodeTexInv = Cast<UCustomizableObjectNodeTextureInvert>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageInvert> InvertNode = new UE::Mutable::Private::NodeImageInvert();
		Result = InvertNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeTexInv->GetBaseImagePin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> BaseImageNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			InvertNode->Base = BaseImageNode;
		}
	}

	else if (const UCustomizableObjectNodeTextureColorMap* TypedNodeColorMap = Cast<UCustomizableObjectNodeTextureColorMap>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageColorMap> ColorMapNode = new UE::Mutable::Private::NodeImageColorMap();

		Result = ColorMapNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorMap->GetMapPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> GradientImageNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			ColorMapNode->Map = GradientImageNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorMap->GetMaskPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> GradientImageNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			ColorMapNode->Mask = GradientImageNode;
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeColorMap->GetBasePin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> SourceImageNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
			ColorMapNode->Base = SourceImageNode;
		}
	}

	else if (const UCustomizableObjectNodeTextureTransform* TypedNodeTransform = Cast<UCustomizableObjectNodeTextureTransform>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageTransform> TransformNode = new UE::Mutable::Private::NodeImageTransform();
		Result = TransformNode;
		
		if ( UEdGraphPin* BaseImagePin = FollowInputPin(*TypedNodeTransform->GetBaseImagePin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ImageNode = GenerateMutableSourceImage( BaseImagePin, GenerationContext, ReferenceTextureSize);
			TransformNode->Base = ImageNode;
		}

		if ( UEdGraphPin* OffsetXPin = FollowInputPin(*TypedNodeTransform->GetOffsetXPin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> OffsetXNode = GenerateMutableSourceFloat(OffsetXPin, GenerationContext);
			TransformNode->OffsetX = OffsetXNode;
		}

		if ( UEdGraphPin* OffsetYPin = FollowInputPin(*TypedNodeTransform->GetOffsetYPin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> OffsetYNode = GenerateMutableSourceFloat(OffsetYPin, GenerationContext);
			TransformNode->OffsetY = OffsetYNode;
		}

		if ( UEdGraphPin* ScaleXPin = FollowInputPin(*TypedNodeTransform->GetScaleXPin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ScaleXNode = GenerateMutableSourceFloat(ScaleXPin, GenerationContext);
			TransformNode->ScaleX = ScaleXNode;
		}

		if ( UEdGraphPin* ScaleYPin = FollowInputPin(*TypedNodeTransform->GetScaleYPin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ScaleYNode = GenerateMutableSourceFloat(ScaleYPin, GenerationContext);
			TransformNode->ScaleY = ScaleYNode; 
		}

		if ( UEdGraphPin* RotationPin = FollowInputPin(*TypedNodeTransform->GetRotationPin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> RotationNode = GenerateMutableSourceFloat(RotationPin, GenerationContext);
			TransformNode->Rotation = RotationNode; 
		}

		TransformNode->AddressMode = Invoke([&]() 
		{
			switch (TypedNodeTransform->AddressMode)
			{
			case ETextureTransformAddressMode::Wrap:		 return UE::Mutable::Private::EAddressMode::Wrap;
			case ETextureTransformAddressMode::ClampToEdge:  return UE::Mutable::Private::EAddressMode::ClampToEdge;
			case ETextureTransformAddressMode::ClampToBlack: return UE::Mutable::Private::EAddressMode::ClampToBlack;
			default: { check(false); return UE::Mutable::Private::EAddressMode::None; }
			}
		});

		FUintVector2 TextureSize(TypedNodeTransform->TextureSizeX, TypedNodeTransform->TextureSizeY);

		// Calculate the max TextureSize allowed using the ReferenceTextureSize and the Reference texture from the node
		int32 MaxReferenceTextureSizeInGame = ReferenceTextureSize;
		if (TypedNodeTransform->ReferenceTexture)
		{
			const UTextureLODSettings& TextureLODSettings = CompilationOptions.TargetPlatform->GetTextureLODSettings();
			MaxReferenceTextureSizeInGame = GetTextureSizeInGame(*TypedNodeTransform->ReferenceTexture, TextureLODSettings);
		}

		// Max TextureSize allowed
		const int32 MaxTransformTextureSizeInGame = ReferenceTextureSize > 0 && ReferenceTextureSize < MaxReferenceTextureSizeInGame ? ReferenceTextureSize : MaxReferenceTextureSizeInGame;

		const int32 TransformNodeTextureSize = FMath::Max(TextureSize.X, TextureSize.Y);
		if (TransformNodeTextureSize > 0 && TransformNodeTextureSize > MaxTransformTextureSizeInGame)
		{
			const int32 NumMips = FMath::CeilLogTwo(TransformNodeTextureSize) + 1;
			const int32 MaxNumMips = FMath::CeilLogTwo(MaxTransformTextureSizeInGame) + 1;

			TextureSize.X = TextureSize.X >> (NumMips - MaxNumMips);
			TextureSize.Y = TextureSize.Y >> (NumMips - MaxNumMips);
		}

		TransformNode->bKeepAspectRatio = TypedNodeTransform->bKeepAspectRatio;
		TransformNode->SizeX = TextureSize.X;
		TransformNode->SizeY = TextureSize.Y;
	}

	else if (const UCustomizableObjectNodeTextureSaturate* TypedNodeSaturate = Cast<UCustomizableObjectNodeTextureSaturate>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSaturate> SaturateNode = new UE::Mutable::Private::NodeImageSaturate();
		Result = SaturateNode;
	
		if ( UEdGraphPin* BaseImagePin = FollowInputPin(*TypedNodeSaturate->GetBaseImagePin()) )
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ImageNode = GenerateMutableSourceImage(BaseImagePin, GenerationContext, ReferenceTextureSize);
			SaturateNode->Source = ImageNode;
		}

		if ( UEdGraphPin* FactorPin = FollowInputPin(*TypedNodeSaturate->GetFactorPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FactorNode = GenerateMutableSourceFloat(FactorPin, GenerationContext);
			SaturateNode->Factor = FactorNode;
		}
	}

	else if (const UCONodeSwitch* TypedNodePassThroughTextureSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough))
	{
		Result = [&]()
		{
			const UEdGraphPin* SwitchParameter = TypedNodePassThroughTextureSwitch->SwitchParameterPinReference.Get();

			// Check Switch Parameter arity preconditions.
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SwitchParameter))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
				// Switch Param not generated
				if (!SwitchParam)
				{
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
					GenerationContext.Log(Message, Node);

					return Result;
				}

				if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
				{
					const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
					GenerationContext.Log(Message, Node);

					return Result;
				}

				{
					const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());

					if (!DoOptionsMatchEnum(*TypedNodePassThroughTextureSwitch, *EnumParameter))
					{
						const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
						GenerationContext.Log(Message, Node);
						Node->SetRefreshNodeWarning();
					}
				}


				const int32 NumSwitchOptions = TypedNodePassThroughTextureSwitch->SwitchPins.Num();
				
				// TODO: Implement Mutable core pass-through switch nodes
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageSwitch> SwitchNode = new UE::Mutable::Private::NodeImageSwitch;
				SwitchNode->Parameter = SwitchParam;
				SwitchNode->Options.SetNum(NumSwitchOptions);

				for (int32 SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
				{
					if (const UEdGraphPin* TexturePin = FollowInputPin(*TypedNodePassThroughTextureSwitch->SwitchPins[SelectorIndex].Get()))
					{
						UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> PassThroughImage = GenerateMutableSourceImage(TexturePin, GenerationContext, ReferenceTextureSize);
						SwitchNode->Options[SelectorIndex] = PassThroughImage;
					}
					else
					{
						const FText Message = LOCTEXT("MissingPassThroughTexture", "Unable to generate pass-through texture switch node. Required connection not found.");
						GenerationContext.Log(Message, Node);
						return Result;
					}
				}

				Result = SwitchNode;
				return Result;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), Node);
				return Result;
			}
		}(); // invoke lambda.
	}

	// If the node is a plain color node, generate an image out of it
	else if (Pin->PinType.PinCategory == Schema->PC_Color)
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ColorNode = GenerateMutableSourceColor(Pin, GenerationContext);

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImagePlainColor> ImageNode = new UE::Mutable::Private::NodeImagePlainColor;
		ImageNode->SizeX = 16;
		ImageNode->SizeY = 16;
		ImageNode->Color = ColorNode;
		Result = ImageNode;
	}

	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		if (Pin->PinType.PinCategory == Schema->PC_Color)
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ColorNode = GenerateMutableSourceColor(Pin, GenerationContext);
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImagePlainColor> ImageNode = new UE::Mutable::Private::NodeImagePlainColor;

			ImageNode->SizeX = 16;
			ImageNode->SizeY = 16;
			ImageNode->Color = ColorNode;

			Result = ImageNode;
		}
		else
		{
			TObjectPtr<UDataTable> Table = UE::Mutable::Private::LoadObject(TypedNodeTable->Table);
			TObjectPtr<UScriptStruct> Structure = UE::Mutable::Private::LoadObject(TypedNodeTable->Structure);
			const FString TableName = Table ? GetNameSafe(Table).ToLower() : GetNameSafe(Structure).ToLower();
			const uint32 TableId = CityHash32(reinterpret_cast<const char*>(*TableName), TableName.Len() * sizeof(FString::ElementType));

			// This node will add a checker texture in case of error
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> EmptyNode = new UE::Mutable::Private::NodeImageConstant();
			Result = EmptyNode;

			bool bSuccess = true;

			if (Pin->PinType.PinCategory == Schema->PC_Material)
			{
				// Material pins have to skip the cache of nodes or they will return always the same column node
				bCacheNode = false;
			}

			UDataTable* DataTable = GetDataTable(TypedNodeTable, GenerationContext);

			if (DataTable)
			{
				FString ColumnName = TypedNodeTable->GetPinColumnName(Pin);
				FProperty* Property = TypedNodeTable->FindPinProperty(*Pin);

				if (!Property)
				{
					FString Msg = FString::Printf(TEXT("Couldn't find the column [%s] in the data table's struct."), *ColumnName);
					GenerationContext.Log(FText::FromString(Msg), Node);

					bSuccess = false;
				}

				UTexture* DefaultTexture = TypedNodeTable->GetColumnDefaultAssetByType<UTexture>(Pin);

				if (bSuccess && Pin->PinType.PinCategory != Schema->PC_Material && !DefaultTexture)
				{
					FString Msg = FString::Printf(TEXT("Couldn't find a default value in the data table's struct for the column [%s]. The default value is null or not a supported Texture"), *ColumnName);
					GenerationContext.Log(FText::FromString(Msg), Node);
					
					bSuccess = false;
				}

				if (bSuccess)
				{
					// Generating a new data table if not exists
					UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> GeneratedTable = nullptr;
					GeneratedTable = GenerateMutableSourceTable(DataTable, TypedNodeTable, GenerationContext);

					if (GeneratedTable)
					{
						UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageTable> ImageTableNode = new UE::Mutable::Private::NodeImageTable();

						if (Pin->PinType.PinCategory == Schema->PC_Material)
						{
							// Material parameters use the Data Table Column Name + Parameter id as mutable column Name to aboid duplicated names (i.e. two MI columns with the same parents but different values).
							ColumnName = Property->GetDisplayNameText().ToString() + GenerationContext.CurrentMaterialParameterId;
						}
						else
						{
							// Checking if this pin texture has been used in another table node but with a different texture mode
							const int32 ImageDataIndex = GenerationContext.GeneratedTableImages.Find({ ColumnName, Pin->PinType.PinCategory, GeneratedTable });

							if (ImageDataIndex != INDEX_NONE )
							{
								if (GenerationContext.GeneratedTableImages[ImageDataIndex].PinType != Pin->PinType.PinCategory)
								{
									TArray<const UObject*> Nodes;
									Nodes.Add(TypedNodeTable);
									Nodes.Add(GenerationContext.GeneratedTableImages[ImageDataIndex].TableNode);

									FString Msg = FString::Printf(TEXT("Texture pin [%s] with different texture modes found in more than one table node. This will add multiple times the texture reseource in the final cook."), *ColumnName);
									GenerationContext.Log(FText::FromString(Msg), Nodes);
								}
							}
							else
							{
								GenerationContext.GeneratedTableImages.Add({ColumnName, Pin->PinType.PinCategory, GeneratedTable, TypedNodeTable});
							}

							// Encoding the texture mode of the type to allow different texture modes from the same pin
							if (Pin->PinType.PinCategory == Schema->PC_Texture_Passthrough)
							{
								ColumnName += "--PassThrough";
							}
						}

						// Generating a new Texture column if not exists
						if (GeneratedTable->FindColumn(ColumnName) == INDEX_NONE)
						{
							GenerationContext.CurrentTableColumnType = UE::Mutable::Private::ETableColumnType::Image;
							bSuccess = GenerateTableColumn(TypedNodeTable, Pin, GeneratedTable, ColumnName, Property, FMutableSourceMeshData(), GenerationContext);

							if (!bSuccess)
							{
								FString Msg = FString::Printf(TEXT("Failed to generate the mutable table column [%s]"), *ColumnName);
								GenerationContext.Log(FText::FromString(Msg), Node);
							}
						}

						if (bSuccess)
						{
							Result = ImageTableNode;

							ImageTableNode->Table = GeneratedTable;
							ImageTableNode->ColumnName = ColumnName;
							ImageTableNode->ParameterName = TypedNodeTable->ParameterName;
							ImageTableNode->bNoneOption = TypedNodeTable->bAddNoneOption;
							ImageTableNode->DefaultRowName = TypedNodeTable->DefaultRowName.ToString();

							bool bIsPassthrough = (Pin->PinType.PinCategory == Schema->PC_Texture_Passthrough);

							UTexture2D* DefaultTexture2D = Cast<UTexture2D>(DefaultTexture);
							if (!bIsPassthrough && DefaultTexture2D)
							{
								UE::Mutable::Private::FImageDesc ImageDesc = GenerateImageDescriptor(DefaultTexture2D);

								uint32 LODBias = ReferenceTextureSize > 0 ? ComputeLODBiasForTexture(GenerationContext, *DefaultTexture2D, nullptr, ReferenceTextureSize) : 0;
								ImageDesc.m_size[0] = ImageDesc.m_size[0] >> LODBias;
								ImageDesc.m_size[1] = ImageDesc.m_size[1] >> LODBias;
								
								const uint16 MaxTextureSize = FMath::Max3(ImageDesc.m_size[0], ImageDesc.m_size[1], (uint16)1);
								ImageTableNode->MaxTextureSize = MaxTextureSize;
								ImageTableNode->ReferenceImageDesc = ImageDesc;
				
								const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
								const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(DefaultTexture2D->LODGroup);

								ImageTableNode->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
								ImageTableNode->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
								ImageTableNode->SourceDataDescriptor.NumNonOptionalLODs = CompilationOptions.MinDiskMips;

								ImageTableNode->SourceDataDescriptor.SourceId = TableId; // It will be combined with the RowId when generating constants
							}
						}
					}
					else
					{
						FString Msg = FString::Printf(TEXT("Couldn't generate a mutable table."));
						GenerationContext.Log(FText::FromString(Msg), Node);
					}
				}
			}
			else
			{
				GenerationContext.Log(LOCTEXT("ImageTableError", "Couldn't find the data table of the node."), Node);
			}
		}
	}

	else if (Cast<UCONodeExternalOperation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageExternal> MeshExtension = new UE::Mutable::Private::NodeImageExternal();

		FSourceExternalOptions Options;
		Options.ReferenceTextureSize = ReferenceTextureSize;
		MeshExtension->Node = GenerateMutableSourceExternal(Pin, GenerationContext, Options);
		
		Result = MeshExtension;
	}
	
	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		// Can't use the GenerateMutableSourceMacro function here because GenerateMutableSourceImage needs the extra parameter ReferenceTextureSize 
		bCacheNode = false;

		if (const UEdGraphPin* OutputPin = TypedNodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin->PinName))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*OutputPin))
			{
				GenerationContext.MacroNodesStack.Push(TypedNodeMacro);
				Result = GenerateMutableSourceImage(FollowPin, GenerationContext, ReferenceTextureSize);
				GenerationContext.MacroNodesStack.Pop();
			}
			else
			{
				FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNotLinked_Image", "Macro Output node Pin {0} not linked."), FText::FromName(Pin->PinName));
				GenerationContext.Log(Msg, Node);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNameNotFound_Image", "Macro Output node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
			GenerationContext.Log(Msg, Node);
		}
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		// Can't use the GenerateMutableSourceMacro function here because GenerateMutableSourceImage needs the extra parameter ReferenceTextureSize 
		check(TypedNodeTunnel->bIsInputNode);
		check(GenerationContext.MacroNodesStack.Num());

		bCacheNode = false;

		const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = GenerationContext.MacroNodesStack.Pop();
		check(MacroInstanceNode);

		if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin->PinName, EEdGraphPinDirection::EGPD_Input))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
			{
				Result = GenerateMutableSourceImage(FollowPin, GenerationContext, ReferenceTextureSize);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroTunnelError_PinNameNotFound_Image", "Macro Instance Node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
			GenerationContext.Log(Msg, Node);
		}

		// Push the Macro again even if the result is null
		GenerationContext.MacroNodesStack.Push(MacroInstanceNode);
	}

	else if (const UCONodeMaterialBreak* TypedNodeMaterialBreak = Cast<UCONodeMaterialBreak>(Node))
	{
		check(TypedNodeMaterialBreak->MaterialPinRef.Get());

		if (const UEdGraphPin* LinkedMaterial = FollowInputPin(*TypedNodeMaterialBreak->MaterialPinRef.Get()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageMaterialBreak> MaterialBreak = new UE::Mutable::Private::NodeImageMaterialBreak();

			FName ParameterName = TypedNodeMaterialBreak->GetPinParameterName(*Pin);
			if (!ParameterName.IsNone())
			{
				const int32 LayerIndex = TypedNodeMaterialBreak->GetPinParameterLayerIndex(*Pin);
				check(LayerIndex <= MAX_int8);

				// Set the key of the parameter that represents the pin
				MaterialBreak->ParameterKey = {ParameterName, (int8)LayerIndex};

				// Store the reference texture into the material generation options to generate all the textures with the correct size
				FGenerationMaterialOptions GenerationOptions;
				GenerationOptions.ReferenceTextureSize = ReferenceTextureSize;

				// Generate the material that is going to be broken into parameters
				GenerationContext.MaterialBreakParameterStack.Add(FMaterialBreakParameter{ MaterialBreak->ParameterKey, Pin->PinType.PinCategory});
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> LinkedNode = GenerateMutableSourceMaterial(LinkedMaterial, GenerationContext, GenerationOptions);
				MaterialBreak->MaterialSource = LinkedNode;
				GenerationContext.MaterialBreakParameterStack.Pop();

				if (LinkedNode)
				{
					Result = MaterialBreak;
				}
				else
				{
					GenerationContext.Log(LOCTEXT("MaterialBreakNodeError", "An error occurred generating the Material Source."), Node, EMessageSeverity::Error);
				}
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoneParameterName", "'None' is not a valid Material Parameter name."), Node, EMessageSeverity::Warning);

			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MaterialBreakPinError", "Material Break Nodes must be linked to an input Material Node."), Node);
		}
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter> GenerateMutableSourceImageParameter(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	UCustomizableObjectNodeParameter* ParameterNode = CastChecked<UCustomizableObjectNodeParameter>(Pin->GetOwningNode());

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter> Result;

	const FGeneratedParameterKey Key = { ParameterNode->NodeGuid, ParameterNode->GetParameterName(&GenerationContext.MacroNodesStack) };
	if (const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter>* GeneratedImageParameter = GenerationContext.GeneratedImageParameters.Find(Key))
	{
		return GeneratedImageParameter->get();
	}
	
	if (const UCustomizableObjectNodeTextureParameter* TypedNodeParam = Cast<UCustomizableObjectNodeTextureParameter>(ParameterNode))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter> TextureNode = new UE::Mutable::Private::NodeImageObjectParameter();

		TextureNode->Name = TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack);
		TextureNode->UID = GenerationContext.GetNodeIdUnique(ParameterNode).ToString();
		
		GenerationContext.TextureParameterDefaultValues.Add(TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack), TypedNodeParam->DefaultValue);

		GenerationContext.ParameterUIDataMap.Add(TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
			TypedNodeParam->ParamUIMetadata,
			EMutableParameterType::Texture));

		Result = TextureNode;
	}
	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), ParameterNode);
	}

	if (Result)
	{
		Result->SetMessageContext(ParameterNode);
	}
	
	GenerationContext.GeneratedImageParameters.Add(Key, Result);
	GenerationContext.GeneratedNodes.Add(ParameterNode);
	
	return Result;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageFormat> GenerateMutableImageFormat(
	const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage>& InSource,
	const UTexture2D* ReferenceTexture,
	const FMutableGraphGenerationContext& GenerationContext,
	const UCustomizableObjectNode* Node)
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageFormat> ImageFormat = new UE::Mutable::Private::NodeImageFormat();
	ImageFormat->Source = InSource.get();
	ImageFormat->SetMessageContext(Node);

	// Use an "easy format" as the default value.
	ImageFormat->Format = UE::Mutable::Private::EImageFormat::RGBA_UByte;

	if (ReferenceTexture)
	{
		TArray<TArray<FTextureBuildSettings>> BuildSettingsPerFormatPerLayer;
		if (GenerationContext.CompilationContext->Options.TargetPlatform)
		{
			ReferenceTexture->GetTargetPlatformBuildSettings(GenerationContext.CompilationContext->Options.TargetPlatform, BuildSettingsPerFormatPerLayer);

			bool bIsServerOnly = GenerationContext.CompilationContext->Options.TargetPlatform->IsServerOnly();
			// Suppress the message for server only platforms. Images are discarded is ServerOnly. 
			if (!bIsServerOnly && BuildSettingsPerFormatPerLayer.IsEmpty())
			{
				const FString ReplacedImageFormatMsg = FString::Printf(TEXT("In object [%s] for platform [%s] the unsupported image format of texture [%s] is used, RGBA_UByte will be used instead."),
					*GenerationContext.GetObjectName(),
					*GenerationContext.CompilationContext->Options.TargetPlatform->PlatformName(),
					*ReferenceTexture->GetName());
				const FText ReplacedImageFormatText = FText::FromString(ReplacedImageFormatMsg);
				GenerationContext.Log(ReplacedImageFormatText, Node, EMessageSeverity::Info);
				UE_LOGF(LogMutable, Log, "%ls", *ReplacedImageFormatMsg);
			}
			else if (BuildSettingsPerFormatPerLayer.Num() > 1)
			{
				const FString ReplacedImageFormatMsg = FString::Printf(TEXT("In object [%s] for platform [%s] the image format of texture [%s] has multiple target formats. Only one will be used.."),
					*GenerationContext.GetObjectName(),
					*GenerationContext.CompilationContext->Options.TargetPlatform->PlatformName(),
					*ReferenceTexture->GetName());
				const FText ReplacedImageFormatText = FText::FromString(ReplacedImageFormatMsg);
				GenerationContext.Log(ReplacedImageFormatText, Node, EMessageSeverity::Info);
				UE_LOGF(LogMutable, Log, "%ls", *ReplacedImageFormatMsg);
			}
		}

		if (BuildSettingsPerFormatPerLayer.IsValidIndex(0) && BuildSettingsPerFormatPerLayer[0].IsValidIndex(0))
		{
			const TArray<FTextureBuildSettings>& BuildSettingsPerLayer = BuildSettingsPerFormatPerLayer[0];

			if (GenerationContext.CompilationContext->Options.TextureCompression != ECustomizableObjectTextureCompression::None)
			{
				static ITextureFormatManagerModule* TextureFormatManager = nullptr;
				if (!TextureFormatManager)
				{
					TextureFormatManager = &FModuleManager::LoadModuleChecked<ITextureFormatManagerModule>("TextureFormat");
					check(TextureFormatManager);
				}

				EPixelFormat UnrealTargetPlatformFormat = EPixelFormat::PF_Unknown;
				EPixelFormat UnrealTargetPlatformFormatAlpha = EPixelFormat::PF_Unknown;

				if (ensure(TextureFormatManager))
				{
					const ITextureFormat* TextureFormat = TextureFormatManager->FindTextureFormat(BuildSettingsPerLayer[0].TextureFormatName);
					if (ensure(TextureFormat))
					{
						UnrealTargetPlatformFormat = TextureFormat->GetEncodedPixelFormat(BuildSettingsPerLayer[0], false);
						UnrealTargetPlatformFormatAlpha = TextureFormat->GetEncodedPixelFormat(BuildSettingsPerLayer[0], true);
					}
				}

				// \TODO: The QualityFix filter is used while the internal mutable runtime compression doesn't provide enough quality for some large block formats.
				UE::Mutable::Private::EImageFormat MutableFormat = QualityAndPerformanceFix(UnrealToMutablePixelFormat(UnrealTargetPlatformFormat, false));
				UE::Mutable::Private::EImageFormat MutableFormatIfAlpha = QualityAndPerformanceFix(UnrealToMutablePixelFormat(UnrealTargetPlatformFormatAlpha, true));

				// Temp hack to enable RG->LA 
				if (GenerationContext.CompilationContext->Options.TargetPlatform)
				{
					bool bUseLA = GenerationContext.CompilationContext->Options.TargetPlatform->SupportsFeature(ETargetPlatformFeatures::NormalmapLAEncodingMode);
					if (bUseLA)
					{
						// We'll have to trust the reference texture because the actual internal settings are opaque.
						// See GetQualityFormat in TextureFormatASTC.cpp to understand how it works, but it depends on some inaccessible texture name constants like this:
						// bool bIsNormalMapFormat = TextureFormatName == GTextureFormatNameASTC_NormalAG ||
						//		TextureFormatName == GTextureFormatNameASTC_NormalRG ||
						//		TextureFormatName == GTextureFormatNameASTC_NormalLA ||
						//		TextureFormatName == GTextureFormatNameASTC_NormalRG_Precise;
						ImageFormat->bNeedsNormalmapLAEncoding = ReferenceTexture->IsNormalMap();
					}
				}

				// Unsupported format: look for something generic
				if (MutableFormat == UE::Mutable::Private::EImageFormat::None)
				{
					const FString ReplacedImageFormatMsg = FString::Printf(TEXT("In object [%s] the unsupported image format %d is used, RGBA_UByte will be used instead."), *GenerationContext.GetObjectName(), UnrealTargetPlatformFormat);
					const FText ReplacedImageFormatText = FText::FromString(ReplacedImageFormatMsg);
					GenerationContext.Log(ReplacedImageFormatText, Node, EMessageSeverity::Info);
					UE_LOGF(LogMutable, Log, "%ls", *ReplacedImageFormatMsg);
					MutableFormat = UE::Mutable::Private::EImageFormat::RGBA_UByte;
				}
				if (MutableFormatIfAlpha == UE::Mutable::Private::EImageFormat::None)
				{
					const FString ReplacedImageFormatMsg = FString::Printf(TEXT("In object [%s] the unsupported image format %d is used, RGBA_UByte will be used instead."), *GenerationContext.GetObjectName(), UnrealTargetPlatformFormatAlpha);
					const FText ReplacedImageFormatText = FText::FromString(ReplacedImageFormatMsg);
					GenerationContext.Log(ReplacedImageFormatText, Node, EMessageSeverity::Info);
					UE_LOGF(LogMutable, Log, "%ls", *ReplacedImageFormatMsg);
					MutableFormatIfAlpha = UE::Mutable::Private::EImageFormat::RGBA_UByte;
				}

				ImageFormat->Format = MutableFormat;
				ImageFormat->FormatIfAlpha = MutableFormatIfAlpha;
			}
		}
	}

	return ImageFormat;
}


#undef LOCTEXT_NAMESPACE

