// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MuR/Ptr.h"
#include "HAL/Platform.h"

class FCustomizableObjectCompiler;
class UCustomizableObjectNode;
class UEdGraphPin;
class UTexture2D;
struct FMutableGraphGenerationContext;

namespace UE::Mutable::Private
{
	class NodeImage;
	class NodeImageObjectParameter;
	class NodeImageFormat;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ResizeTextureByNumMips(const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage>& Image, int32 MipsToSkip);

/** Convert a CustomizableObject Source Graph from an Image pin into a mutable source graph. */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> GenerateMutableSourceImage(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, int32 ReferenceTextureSize);

UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageObjectParameter> GenerateMutableSourceImageParameter(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);

UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageFormat> GenerateMutableImageFormat(
	const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage>& InSource,
	const UTexture2D* ReferenceTexture,
	const FMutableGraphGenerationContext& GenerationContext,
	const UCustomizableObjectNode* Node);