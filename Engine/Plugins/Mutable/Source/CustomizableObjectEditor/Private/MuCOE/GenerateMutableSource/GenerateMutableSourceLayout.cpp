// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceLayout.h"

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeLayoutBlocks.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "Engine/StaticMesh.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> CreateDefaultLayout()
{
	constexpr int32 GridSize = 4;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode = new UE::Mutable::Private::NodeLayout();
	LayoutNode->Size = { GridSize, GridSize };
	LayoutNode->MaxSize = { GridSize, GridSize };
	LayoutNode->AutoBlockStrategy = UE::Mutable::Private::EAutoBlocksStrategy::Ignore;
	LayoutNode->Strategy = UE::Mutable::Private::EPackStrategy::Resizeable;
	LayoutNode->ReductionMethod = UE::Mutable::Private::EReductionMethod::Halve;
	LayoutNode->TexCoordsIndex = 0;
	LayoutNode->Blocks.SetNum(1);
	LayoutNode->Blocks[0].Min = { 0, 0 };
	LayoutNode->Blocks[0].Size = { GridSize, GridSize };
	LayoutNode->Blocks[0].Priority = 0;
	LayoutNode->Blocks[0].bReduceBothAxes = false;
	LayoutNode->Blocks[0].bReduceByTwo = false;

	return LayoutNode;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> CreateMutableLayoutNode(const UCustomizableObjectLayout* UnrealLayout, bool bIgnoreLayoutWarnings)
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode = new UE::Mutable::Private::NodeLayout;

	LayoutNode->Size = FIntVector2(UnrealLayout->GetGridSize().X, UnrealLayout->GetGridSize().Y);
	LayoutNode->MaxSize = FIntVector2(UnrealLayout->GetMaxGridSize().X, UnrealLayout->GetMaxGridSize().Y);
	LayoutNode->TexCoordsIndex = UnrealLayout->GetUVChannel();

	LayoutNode->AutoBlockStrategy = ConvertAutomaticBlocksStrategy(UnrealLayout->AutomaticBlocksStrategy);
	LayoutNode->bMergeChildBlocks = UnrealLayout->AutomaticBlocksMergeStrategy == ECustomizableObjectLayoutAutomaticBlocksMergeStrategy::MergeChildBlocks;
	LayoutNode->Strategy = ConvertLayoutStrategy(UnrealLayout->PackingStrategy);
	LayoutNode->ReductionMethod = UnrealLayout->BlockReductionMethod == ECustomizableObjectLayoutBlockReductionMethod::Halve ? UE::Mutable::Private::EReductionMethod::Halve : UE::Mutable::Private::EReductionMethod::Unitary;

	if (bIgnoreLayoutWarnings)
	{
		// Layout warnings can be safely ignored in this case. Vertices that do not belong to any layout block will be removed (Extend Materials only)
		LayoutNode->FirstLODToIgnoreWarnings = 0;
	}
	else
	{
		LayoutNode->FirstLODToIgnoreWarnings = UnrealLayout->GetIgnoreVertexLayoutWarnings() ? UnrealLayout->GetFirstLODToIgnoreWarnings() : -1;
	}

	LayoutNode->Blocks.SetNum(UnrealLayout->Blocks.Num());
	for (int32 BlockIndex = 0; BlockIndex < UnrealLayout->Blocks.Num(); ++BlockIndex)
	{
		LayoutNode->Blocks[BlockIndex] = ToMutable(UnrealLayout->Blocks[BlockIndex]);
	}

	return LayoutNode;
}


UE::Mutable::Private::FSourceLayoutBlock ToMutable(const FCustomizableObjectLayoutBlock& UnrealBlock)
{
	UE::Mutable::Private::FSourceLayoutBlock MutableBlock;

	MutableBlock.Min = { uint16(UnrealBlock.Min.X), uint16(UnrealBlock.Min.Y) };
	FIntPoint Size = UnrealBlock.Max - UnrealBlock.Min;
	MutableBlock.Size = { uint16(Size.X), uint16(Size.Y) };

	MutableBlock.Priority = UnrealBlock.Priority;
	MutableBlock.bReduceBothAxes = UnrealBlock.bReduceBothAxes;
	MutableBlock.bReduceByTwo = UnrealBlock.bReduceByTwo;

	if (UnrealBlock.Mask)
	{
		// In the editor the src data can be directly accessed
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> MaskImage = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>();

		FMutableSourceTextureData Texture(*UnrealBlock.Mask);

		constexpr int32 MipLevel = 0;
		constexpr int32 bLoadMipTail = false;
		EUnrealToMutableConversionError Error = ConvertTextureUnrealSourceToMutable(MaskImage.Get(), Texture, MipLevel, false);
		if (Error != EUnrealToMutableConversionError::Success)
		{
			// This should never happen, so details are not necessary.
			UE_LOGF(LogMutable, Warning, "Failed to convert layout block mask texture.");
		}
		else
		{
			MutableBlock.Mask = MaskImage;
		}
	}

	return MutableBlock;
}


#undef LOCTEXT_NAMESPACE

