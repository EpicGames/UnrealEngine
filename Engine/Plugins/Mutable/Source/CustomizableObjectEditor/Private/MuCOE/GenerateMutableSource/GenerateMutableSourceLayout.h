// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeLayout.h"
#include "MuCOE/CustomizableObjectLayout.h"

class UEdGraphPin;


extern UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> CreateDefaultLayout();

extern UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> CreateMutableLayoutNode(const UCustomizableObjectLayout*, bool bIgnoreLayoutWarnings);

extern UE::Mutable::Private::FSourceLayoutBlock ToMutable(const FCustomizableObjectLayoutBlock&);
