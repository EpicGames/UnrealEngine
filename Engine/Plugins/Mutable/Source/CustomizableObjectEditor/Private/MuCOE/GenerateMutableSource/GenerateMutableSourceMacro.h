// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/GraphTraversal.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


template<typename T, typename... ArgTypes>
UE::Mutable::Private::Ptr<T> GenerateMutableSourceMacro(const UEdGraphPin& Pin, FMutableGraphGenerationContext& GenerationContext, UE::Mutable::Private::Ptr<T> (*GenerationFunction)(const UEdGraphPin*, FMutableGraphGenerationContext&, ArgTypes...), ArgTypes&&... Args)
{
	UE::Mutable::Private::Ptr<T> Result;
	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin.GetOwningNode());

	if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		if (const UEdGraphPin* OutputPin = TypedNodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin.PinName))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*OutputPin))
			{
				GenerationContext.MacroNodesStack.Push(TypedNodeMacro);
				Result = GenerationFunction(FollowPin, GenerationContext, Forward<ArgTypes>(Args)...);
				GenerationContext.MacroNodesStack.Pop();
			}
			else
			{
				FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNotLinked", "Macro Output node Pin {0} not linked."), FText::FromName(Pin.PinName));
				GenerationContext.Log(Msg, Node);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNameNotFound", "Macro Output node does not contain a pin with name {0}."), FText::FromName(Pin.PinName));
			GenerationContext.Log(Msg, Node);
		}
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		check(TypedNodeTunnel->bIsInputNode);
		check(GenerationContext.MacroNodesStack.Num());

		const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = GenerationContext.MacroNodesStack.Pop();
		check(MacroInstanceNode);

		if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin.PinName, EEdGraphPinDirection::EGPD_Input))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
			{
				Result = GenerationFunction(FollowPin, GenerationContext, Forward<ArgTypes>(Args)...);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroTunnelError_PinNameNotFound", "Macro Instance Node does not contain a pin with name {0}."), FText::FromName(Pin.PinName));
			GenerationContext.Log(Msg, Node);
		}

		// Push the Macro again even if the result is null
		GenerationContext.MacroNodesStack.Push(MacroInstanceNode);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
