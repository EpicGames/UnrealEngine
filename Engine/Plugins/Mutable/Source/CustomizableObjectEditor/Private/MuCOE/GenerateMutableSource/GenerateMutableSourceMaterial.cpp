// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMaterial.h"

#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITextureFormat.h"

#include "GenerateMutableSourceExternal.h"
#include "GenerateMutableSourceMesh.h"
#include "GenerateMutableSourceSkeletalMesh.h"
#include "GenerateMutableSourceSkeletalMeshObject.h"
#include "Engine/TextureLODSettings.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceColor.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceGroupProjector.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceImage.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMacro.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMaterialParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/Nodes/CONodeMaterialBreak.h"
#include "MuCOE/Nodes/CONodeMaterialConstant.h"
#include "MuCOE/Nodes/CONodeMaterialModify.h"
#include "MuCOE/Nodes/CONodeMaterialVariation.h"
#include "MuCO/LoadUtils.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeImageFromMaterialParameter.h"
#include "MuT/NodeImageMipmap.h"
#include "MuT/NodeImageNormalComposite.h"
#include "MuT/NodeImageResize.h"
#include "MuT/NodeMaterialConstant.h"
#include "MuT/NodeMaterialSwitch.h"
#include "MuT/NodeMaterialTable.h"
#include "MuT/NodeMaterialVariation.h"
#include "MuT/NodeMaterialParameter.h"
#include "MuT/NodeMaterialModify.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstance.h"
#include "MuCOE/Nodes/CONodeExternalOperation.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshBreak_V2.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshObjectBreak.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuT/NodeMaterialExternal.h"
#include "MuT/NodeMaterialSkeletalMeshBreak.h"
#include "MuT/NodeMaterialSkeletalMeshObjectBreak.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter> GenerateMutableSourceMaterialParameter(const UEdGraphPin* Pin,
	FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	UCustomizableObjectNodeParameter* ParameterNode = CastChecked<UCustomizableObjectNodeParameter>(Pin->GetOwningNode());

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter> Result;

	const FGeneratedParameterKey Key = { ParameterNode->NodeGuid, ParameterNode->GetParameterName(&GenerationContext.MacroNodesStack) };
	if (const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter>* GeneratedMaterialParameter = GenerationContext.GeneratedMaterialParameters.Find(Key))
	{
		return GeneratedMaterialParameter->get();
	}
	
	if (const UCustomizableObjectNodeMaterialParameter* MaterialParameterNode = Cast<UCustomizableObjectNodeMaterialParameter>(ParameterNode))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter> MaterialNode = new UE::Mutable::Private::NodeMaterialParameter();
		MaterialNode->Name = MaterialParameterNode->GetParameterName(&GenerationContext.MacroNodesStack);
		MaterialNode->UID = GenerationContext.GetNodeIdUnique(ParameterNode).ToString();
		
		GenerationContext.MaterialParameterDefaultValues.Add(MaterialParameterNode->GetParameterName(&GenerationContext.MacroNodesStack), UE::Mutable::Private::LoadObject(MaterialParameterNode->DefaultValue));
		GenerationContext.ParameterUIDataMap.Add(MaterialParameterNode->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
			MaterialParameterNode->ParamUIMetadata,
			EMutableParameterType::Material));

		Result = MaterialNode;
	}
	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), ParameterNode);
	}

	if (Result)
	{
		Result->SetMessageContext(ParameterNode);
	}
	
	GenerationContext.GeneratedMaterialParameters.Add(Key, Result);
	GenerationContext.GeneratedNodes.Add(ParameterNode);
	
	return Result;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> GenerateMutableSourceMaterial(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FGenerationMaterialOptions& MaterialOptions)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceMaterial), *Pin, *Node, GenerationContext, false, false, true);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeMaterial*>(Generated->Node.get());
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For example, MacroInstanceNodes
	bool bCacheNode = true;

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> Result;
	
	if (UCONodeSkeletalMeshSection* TypedNodeMaterialBase = Cast<UCONodeSkeletalMeshSection>(Node))
	{
		check(MaterialOptions.SurfaceNode.get());

		//Base Materials are a Material Modify
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialModify> BaseMaterial = new UE::Mutable::Private::NodeMaterialModify();
		Result = BaseMaterial;

		const UEdGraphPin* ConnectedMaterialAssetPin = nullptr;

		// Generate the Material linked to the Mesh Section first, if any
		if (const UEdGraphPin* MaterialAssetPin = TypedNodeMaterialBase->GetMaterialAssetPin())
		{
			ConnectedMaterialAssetPin = FollowInputPin(*MaterialAssetPin);

			if (ConnectedMaterialAssetPin)
			{
				//Check if the pin goes through a macro or tunnel node
				ConnectedMaterialAssetPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedMaterialAssetPin, &GenerationContext.MacroNodesStack);

				GenerationContext.CurrentMaterialParameterId = MaterialAssetPin->PinId.ToString();
				BaseMaterial->MaterialSource = GenerateMutableSourceMaterial(ConnectedMaterialAssetPin, GenerationContext, MaterialOptions);
			}
		}

		if (!BaseMaterial->MaterialSource)
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialConstant> MaterialNode = new UE::Mutable::Private::NodeMaterialConstant();
			MaterialNode->MaterialId = GenerationContext.CompilationContext->PassthroughObjectFactory.Add(*GenerationContext.CurrentReferencedMaterial.Get(), false);

			BaseMaterial->MaterialSource = MaterialNode;
		}

		// Generate images
		const int32 NumImages = TypedNodeMaterialBase->GetNumParameters(EMaterialParameterType::Texture);

		for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
		{
			const UEdGraphPin* ImagePin = TypedNodeMaterialBase->GetParameterPin(EMaterialParameterType::Texture, ImageIndex);
			const FName ImageName = TypedNodeMaterialBase->GetParameterName(EMaterialParameterType::Texture, ImageIndex);
			const int32 LayerIndex = TypedNodeMaterialBase->GetParameterLayerIndex(EMaterialParameterType::Texture, ImageIndex);
			check(LayerIndex <= MAX_int8);

			UE::Mutable::Private::FParameterKey ImageKey = { ImageName, (int8)LayerIndex };

			const bool bIsImagePinLinked = ImagePin && FollowInputPin(*ImagePin);

			if (bIsImagePinLinked && !TypedNodeMaterialBase->IsImageMutableMode(ImageIndex))
			{
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*ImagePin))
				{
					FGeneratedImageProperties Props = GenerateImageProperties(TypedNodeMaterialBase, nullptr, ImageKey, true, GenerationContext);

					// This is a connected pass-through texture that simply has to be passed to the core
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> PassThroughImagePtr = GenerateMutableSourceImage(ConnectedPin, GenerationContext, 0);
					UE::Mutable::Private::FImageParameterData Image;

					check(Props.ImagePropertiesIndex != INDEX_NONE);

					Image.ImageNode = PassThroughImagePtr;
					Image.ImagePropertyIndex = Props.ImagePropertiesIndex;
					Image.LayoutIndex = -1;
					Image.bIsPassthrough = true;

					BaseMaterial->ImageParameters.Add(ImageKey, Image);
				}
			}
			else
			{
				bool bIsGroupProjectorImage = false;
					
				UTexture2D* GroupProjectionReferenceTexture = nullptr;

				UE::Mutable::Private::NodeImagePtr GroupProjectionImg = GenerateMutableSourceGroupProjector(GenerationContext.CurrentLOD, ImageIndex, MaterialOptions.SurfaceNode->Mesh, GenerationContext,
					TypedNodeMaterialBase, nullptr, bIsGroupProjectorImage,
					GroupProjectionReferenceTexture);

				if (GroupProjectionImg.get() || TypedNodeMaterialBase->IsImageMutableMode(ImageIndex))
				{
					// Get the reference texture
					UTexture2D* ReferenceTexture = nullptr;
					{
						const FNodeMaterialParameterId ImageId = TypedNodeMaterialBase->GetParameterId(EMaterialParameterType::Texture, ImageIndex);
						
						//TODO(Max) UE-220247: Add support for multilayer materials
						GenerationContext.CurrentMaterialTableParameter = ImageName.ToString();
						GenerationContext.CurrentMaterialParameterId = ImageId.ParameterId.ToString();

						ReferenceTexture = GroupProjectionImg.get() ? GroupProjectionReferenceTexture : nullptr;

						if (!ReferenceTexture)
						{
							ReferenceTexture = TypedNodeMaterialBase->GetImageReferenceTexture(ImageIndex);
						}

						// In case of group projector, don't follow the pin to find the reference texture.
						if (!GroupProjectionImg.get() && !ReferenceTexture && ImagePin)
						{
							if (const UEdGraphPin* ConnectedPin = FollowInputPin(*ImagePin))
							{
								ReferenceTexture = FindReferenceImage(ConnectedPin, GenerationContext);
							}
						}

						if (!ReferenceTexture &&
							ConnectedMaterialAssetPin && Cast<UCustomizableObjectNodeTable>(ConnectedMaterialAssetPin->GetOwningNode()) != nullptr)
						{
							ReferenceTexture = FindReferenceImage(ConnectedMaterialAssetPin, GenerationContext);
						}

						if (!ReferenceTexture)
						{
							ReferenceTexture = TypedNodeMaterialBase->GetImageValue(ImageIndex);
						}
					}

					// Generate image properties
					FGeneratedImageProperties Props = GenerateImageProperties(TypedNodeMaterialBase, ReferenceTexture, ImageKey, false, GenerationContext);
					GenerationContext.CurrentImageProperties = &Props;

					// Generate the texture nodes
					UE::Mutable::Private::NodeImagePtr ImageNode = [&]()
						{
							if (TypedNodeMaterialBase->IsImageMutableMode(ImageIndex))
							{
								if (ImagePin)
								{
									if (const UEdGraphPin* ConnectedPin = FollowInputPin(*ImagePin))
									{
										return GenerateMutableSourceImage(ConnectedPin, GenerationContext, Props.TextureSize);
									}
								}

								// If the table material pin is linked to a table node, get all the textures of the current material parameter (CurrentMaterialTableParameter) from the Material Instances of the specified data table column.
								// Then Generate a mutable table column with all these textures.
								if (ConnectedMaterialAssetPin && Cast<UCustomizableObjectNodeTable>(ConnectedMaterialAssetPin->GetOwningNode()) != nullptr)
								{
									return GenerateMutableSourceImage(ConnectedMaterialAssetPin, GenerationContext, Props.TextureSize);
								}

								// Else
								{
									UTexture2D* Texture2D = TypedNodeMaterialBase->GetImageValue(ImageIndex);

									if (Texture2D)
									{
										UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ConstImageNode = new UE::Mutable::Private::NodeImageConstant();
										ConstImageNode->Image = GenerateImageConstant(Texture2D, GenerationContext, false);

										const uint32 MipsToSkip = ComputeLODBiasForTexture(GenerationContext, *Texture2D, nullptr, Props.TextureSize);
										UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> Result = ResizeTextureByNumMips(ConstImageNode, MipsToSkip);

										const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
										const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(Texture2D->LODGroup);

										ConstImageNode->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
										ConstImageNode->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
										ConstImageNode->SourceDataDescriptor.NumNonOptionalLODs = GenerationContext.CompilationContext->Options.MinDiskMips;

										const FString TextureName = GetNameSafe(Texture2D).ToLower();
										ConstImageNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*TextureName), TextureName.Len() * sizeof(FString::ElementType));

										return Result;
									}
									else
									{
										return UE::Mutable::Private::NodeImagePtr();
									}
								}
							}
							else
							{
								return UE::Mutable::Private::NodeImagePtr();
							}
						}();

					GenerationContext.CurrentImageProperties = nullptr;

					if (GroupProjectionImg.get())
					{
						ImageNode = GroupProjectionImg;
					}

					// NOTE: Important step to generate a texture with the correct format and size
					ImageNode = GenerateMipMapAndFormatNodes(Node, GenerationContext, ImageNode, ReferenceTexture);

					// If we are generating an implicit component (with a passthrough mesh) we don't apply any layout.
					bool bGeneratingImplicitComponent = GenerationContext.ComponentMeshOverride.get() != nullptr;

					int32 UVLayout = -1;
					if (!bGeneratingImplicitComponent)
					{
						UVLayout = TypedNodeMaterialBase->GetImageUVLayout(ImageIndex);
					}

					// Generate image data
					UE::Mutable::Private::FImageParameterData ImageData;
					ImageData.ImageNode = ImageNode;
					ImageData.ImagePropertyIndex = Props.ImagePropertiesIndex;
					ImageData.LayoutIndex = UVLayout;
					ImageData.bIsPassthrough = false;

					if (ImageData.LayoutIndex != INDEX_NONE && GenerationContext.CompilationContext->Options.bUseLegacyLayouts == false)
					{
						int32 MaxBlocksX = static_cast<int32>(TypedNodeMaterialBase->UVPackagingSettings[ImageData.LayoutIndex].MaxGridSizeX);
						int32 MaxBlocksY = static_cast<int32>(TypedNodeMaterialBase->UVPackagingSettings[ImageData.LayoutIndex].MaxGridSizeY);
						GetLayoutBlockSizeInPixels(GenerationContext, ReferenceTexture, MaxBlocksX, MaxBlocksY, ImageData.BlockSizeX, ImageData.BlockSizeY);
					}

					//Store the image inside the current generated material
					BaseMaterial->ImageParameters.Add(ImageKey, ImageData);
				}
			}
		}

		// Generate color parameters
		const int32 NumVectors = TypedNodeMaterialBase->GetNumParameters(EMaterialParameterType::Vector);
		for (int32 VectorIndex = 0; VectorIndex < NumVectors; ++VectorIndex)
		{
			const UEdGraphPin* VectorPin = TypedNodeMaterialBase->GetParameterPin(EMaterialParameterType::Vector, VectorIndex);
			bool bVectorPinConnected = VectorPin && FollowInputPin(*VectorPin);

			if (bVectorPinConnected)
			{
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VectorPin))
				{
					FName VectorName = TypedNodeMaterialBase->GetParameterName(EMaterialParameterType::Vector, VectorIndex);
					FNodeMaterialParameterId VectorId = TypedNodeMaterialBase->GetParameterId(EMaterialParameterType::Vector, VectorIndex);
					const int32 LayerIndex = TypedNodeMaterialBase->GetParameterLayerIndex(EMaterialParameterType::Vector, VectorIndex);
					check(LayerIndex <= MAX_int8);

					// Store info to generate colors from the material of a table node
					GenerationContext.CurrentMaterialTableParameter = VectorName.ToString();
					GenerationContext.CurrentMaterialParameterId = VectorId.ParameterId.ToString();

					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ColorNode = GenerateMutableSourceColor(ConnectedPin, GenerationContext);
					BaseMaterial->ColorParameters.Add({ VectorName, (int8)LayerIndex }, ColorNode);
				}
			}
		}

		// Generate scalar parameters
		const int32 NumScalar = TypedNodeMaterialBase->GetNumParameters(EMaterialParameterType::Scalar);
		for (int32 ScalarIndex = 0; ScalarIndex < NumScalar; ++ScalarIndex)
		{
			const UEdGraphPin* ScalarPin = TypedNodeMaterialBase->GetParameterPin(EMaterialParameterType::Scalar, ScalarIndex);
			bool bScalarPinConnected = ScalarPin && FollowInputPin(*ScalarPin);

			if (bScalarPinConnected)
			{
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*ScalarPin))
				{
					FName ScalarName = TypedNodeMaterialBase->GetParameterName(EMaterialParameterType::Scalar, ScalarIndex);
					FNodeMaterialParameterId ScalarId = TypedNodeMaterialBase->GetParameterId(EMaterialParameterType::Scalar, ScalarIndex);
					const int32 LayerIndex = TypedNodeMaterialBase->GetParameterLayerIndex(EMaterialParameterType::Scalar, ScalarIndex);
					check(LayerIndex <= MAX_int8);
					
					// Store info to generate scalars from the material of a table
					GenerationContext.CurrentMaterialTableParameter = ScalarName.ToString();
					GenerationContext.CurrentMaterialParameterId = ScalarId.ParameterId.ToString();

					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ScalarNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
					BaseMaterial->ScalarParameters.Add({ ScalarName, (int8)LayerIndex }, ScalarNode);
				}
			}
		}
	}
	
	else if (const UCONodeMaterialConstant* TypedNodeMaterialConstant = Cast<UCONodeMaterialConstant>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialConstant> MaterialNode;

		if (UMaterialInterface* Material = GenerationContext.LoadObject(TypedNodeMaterialConstant->Material))
		{
			MaterialNode = new UE::Mutable::Private::NodeMaterialConstant();
			GenerationContext.CurrentReferencedMaterial = TStrongObjectPtr(Material);
			MaterialNode->MaterialId = GenerationContext.CompilationContext->PassthroughObjectFactory.Add(*Material, false);

			// Store the parameter that needs to be processed
			const FMaterialBreakParameter CurrentBreakParameter = GenerationContext.GetCurrentMaterialBreakParameter();
			if (!CurrentBreakParameter.ParameterType.IsNone())
			{
				FHashedMaterialParameterInfo ParameterInfo;
				ParameterInfo.Name = FScriptName(CurrentBreakParameter.ParameterKey.ParameterName);
				ParameterInfo.Index = (int32)CurrentBreakParameter.ParameterKey.LayerIndex;
				ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

				if (!TypedNodeMaterialConstant->Material)
				{
					GenerationContext.Log(LOCTEXT("NoReferenceMaterialToBreak", "Could not find a Reference Material to break."), TypedNodeMaterialConstant);
				}
				else
				{
					if (CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Texture)
					{
						UTexture* Texture = nullptr;
						bool bParameterFound = Material->GetTextureParameterValue(ParameterInfo, Texture);
						UTexture2D* BaseTexture = Cast<UTexture2D>(Texture);

						if (bParameterFound && BaseTexture)
						{
							UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ConstantImageNode = new UE::Mutable::Private::NodeImageConstant;
							ConstantImageNode->Image = GenerateImageConstant(BaseTexture, GenerationContext, false);

							const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
							const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(BaseTexture->LODGroup);
							const FCompilationOptions& CompilationOptions = GenerationContext.CompilationContext->Options;

							ConstantImageNode->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
							ConstantImageNode->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
							ConstantImageNode->SourceDataDescriptor.NumNonOptionalLODs = CompilationOptions.MinDiskMips;

							const FString TextureName = GetNameSafe(BaseTexture).ToLower();
							ConstantImageNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*TextureName), TextureName.Len() * sizeof(FString::ElementType));

							const uint32 MipsToSkip = ComputeLODBiasForTexture(GenerationContext, *BaseTexture, nullptr, MaterialOptions.ReferenceTextureSize);

							if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> FinalNodeImag = ResizeTextureByNumMips(ConstantImageNode, MipsToSkip))
							{
								MaterialNode->ImageValues.Add(CurrentBreakParameter.ParameterKey, FinalNodeImag);
							}
						}
						else
						{
							FText Message = FText::Format(LOCTEXT("NoReferenceTextureToBreak", "Could not find a Reference Texture for break parameter {0}."), FText::FromName(CurrentBreakParameter.ParameterKey.ParameterName));
							GenerationContext.Log(Message, TypedNodeMaterialConstant);

							MaterialNode = nullptr;
						}
					}
					
					else if (CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
					{
						UTexture* PassthroughTexture = nullptr;
						const bool bPassthroughParameterFound = Material->GetTextureParameterValue(ParameterInfo, PassthroughTexture);
						const UTexture2D* PassthroughBaseTexture = Cast<UTexture2D>(PassthroughTexture);

						if (bPassthroughParameterFound && PassthroughBaseTexture)
						{
							UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> ImageNode = new UE::Mutable::Private::NodeImageConstant();
							ImageNode->Image = GenerateImageConstant(PassthroughTexture, GenerationContext, true);
							MaterialNode->ImageValues.Add(CurrentBreakParameter.ParameterKey, ImageNode);
						}
					}

					else if(CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Color)
					{
						FLinearColor Color;
						if (Material->GetVectorParameterValue(ParameterInfo, Color))
						{
							MaterialNode->ColorValues.Add(CurrentBreakParameter.ParameterKey, Color);
						}
					}

					else if(CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Float)
					{
						float Scalar;
						if (Material->GetScalarParameterValue(ParameterInfo, Scalar))
						{
							MaterialNode->ScalarValues.Add(CurrentBreakParameter.ParameterKey, Scalar);
						}
					}
				}
			}
		}

		Result = MaterialNode;
	}

	else if (const UCONodeSwitch* TypedNodeMaterialSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Material))
	{
		Result = [&]()
			{
				const UEdGraphPin* SwitchParameter = TypedNodeMaterialSwitch->SwitchParameterPinReference.Get();

				// Check Switch Parameter arity preconditions.
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SwitchParameter))
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);

					// Switch Param not generated
					if (!SwitchParam)
					{
						const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
						GenerationContext.Log(Message, Node);

						return Result;
					}

					if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
					{
						const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
						GenerationContext.Log(Message, Node);

						return Result;
					}

					{
						const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());

						if (!DoOptionsMatchEnum(*TypedNodeMaterialSwitch, *EnumParameter))
						{
							const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
							GenerationContext.Log(Message, Node);
							Node->SetRefreshNodeWarning();
						}
					}

					const int32 NumSwitchOptions = TypedNodeMaterialSwitch->SwitchPins.Num();
					
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialSwitch> SwitchNode = new UE::Mutable::Private::NodeMaterialSwitch;
					SwitchNode->Parameter = SwitchParam;
					SwitchNode->Options.SetNum(NumSwitchOptions);

					for (int SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
					{
						if (const UEdGraphPin* MaterialPin = FollowInputPin(*TypedNodeMaterialSwitch->SwitchPins[SelectorIndex].Get()))
						{
							SwitchNode->Options[SelectorIndex] = GenerateMutableSourceMaterial(MaterialPin, GenerationContext, MaterialOptions);
						}
						else
						{
							const FText Message = LOCTEXT("MissingMaterial", "Unable to generate material switch node. Required connection not found.");
							GenerationContext.Log(Message, Node);
							return Result;
						}
					}

					Result = SwitchNode;
					return Result;
				}
				else
				{
					GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), Node);
					return Result;
				}
			}(); // invoke lambda.
	}

	else if (const UCONodeMaterialVariation* TypedNodeMaterialVariation = Cast<UCONodeMaterialVariation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialVariation> MaterialNode = new UE::Mutable::Private::NodeMaterialVariation();
		Result = MaterialNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMaterialVariation->DefaultPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> ChildNode = GenerateMutableSourceMaterial(ConnectedPin, GenerationContext, MaterialOptions);
			if (ChildNode)
			{
				MaterialNode->DefaultMaterial = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MaterialVariationFailed", "Material variation generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeMaterialVariation->GetNumVariations();
		MaterialNode->Variations.SetNum(NumVariations);
		for (int VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			if (const UEdGraphPin* VariationPin = TypedNodeMaterialVariation->VariationPin(VariationIndex))
			{
				MaterialNode->Variations[VariationIndex].Tag = TypedNodeMaterialVariation->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);

				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> ChildNode = GenerateMutableSourceMaterial(ConnectedPin, GenerationContext, MaterialOptions);
					MaterialNode->Variations[VariationIndex].Material = ChildNode;
				}
			}
		}
	}

	else if (const UCustomizableObjectNodeMaterialParameter* MaterialParameterNode = Cast<UCustomizableObjectNodeMaterialParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter> MaterialNode = GenerateMutableSourceMaterialParameter(Pin, GenerationContext);

		// Store the parameter that needs to be processed
		const FMaterialBreakParameter CurrentBreakParameter = GenerationContext.GetCurrentMaterialBreakParameter();
		
		if (!CurrentBreakParameter.ParameterType.IsNone() && CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Texture)
		{
			if (!MaterialParameterNode->ReferenceValue)
			{
				GenerationContext.Log(LOCTEXT("NoReferenceMaterialToBreak", "Could not find a Reference Material to break."), MaterialParameterNode);
			}
			else
			{
				// Data to find the parameter value inside the material
				FHashedMaterialParameterInfo ParameterInfo;
				ParameterInfo.Name = FScriptName(CurrentBreakParameter.ParameterKey.ParameterName);
				ParameterInfo.Index = (int32)CurrentBreakParameter.ParameterKey.LayerIndex;
				ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

				//TODO(Max - GM): Move this to the generatemutablesourceimage?
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageFromMaterialParameter> TextureNode = new UE::Mutable::Private::NodeImageFromMaterialParameter();
				TextureNode->ImageParameterKey = CurrentBreakParameter.ParameterKey;

				// Resize image node.
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageResize> ResizeNode = new UE::Mutable::Private::NodeImageResize();
				ResizeNode->Base = TextureNode;
				ResizeNode->bRelative = false;

				UE::Mutable::Private::FImageSize TextureSize(1);

				//Find reference Texture from the material parameter
				UTexture* Texture = nullptr;
				bool bParameterFound = MaterialParameterNode->ReferenceValue->GetTextureParameterValue(ParameterInfo, Texture);
				UTexture2D* ReferenceTexture = Cast<UTexture2D>(Texture);

				if (bParameterFound && ReferenceTexture)
				{
					const uint32 LODBias = ComputeLODBiasForTexture(GenerationContext, *ReferenceTexture, nullptr, MaterialOptions.ReferenceTextureSize);
					TextureSize.X = FMath::Max(ReferenceTexture->Source.GetSizeX() >> LODBias, 1);
					TextureSize.Y = FMath::Max(ReferenceTexture->Source.GetSizeY() >> LODBias, 1);

					ResizeNode->SizeX = TextureSize.X;
					ResizeNode->SizeY = TextureSize.Y;

					// Force the same format that the default texture if any.
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageFormat> FormatNode = GenerateMutableImageFormat(ResizeNode, ReferenceTexture, GenerationContext, Node);
					MaterialNode->ImageParametersToCompile.Add(CurrentBreakParameter.ParameterKey, FormatNode);
				}
				else
				{
					FText Message = FText::Format(LOCTEXT("NoReferenceTextureToBreak", "Could not find a Reference Texture for break parameter {0}."), FText::FromName(CurrentBreakParameter.ParameterKey.ParameterName));
					GenerationContext.Log(Message, TypedNodeMaterialConstant);

					MaterialNode = nullptr;
				}
			}
		}

		Result = MaterialNode;
	}

	else if (const UCONodeSkeletalMeshObjectBreak* TypedSkeletalMeshObjectBreakNode = Cast<UCONodeSkeletalMeshObjectBreak>(Node))
	{
		const UCONodeSkeletalMeshObjectBreakPinData& OverrideMaterialPinData = TypedSkeletalMeshObjectBreakNode->GetPinData<UCONodeSkeletalMeshObjectBreakPinData>(*Pin);
		const FName TargetSlotName = OverrideMaterialPinData.TargetMaterialSlotName;

		if (!TargetSlotName.IsNone())
		{
			if (const UEdGraphPin* ConnectedMeshPin = FollowInputPin(*TypedSkeletalMeshObjectBreakNode->InputPassthroughMesh.Get()))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialSkeletalMeshObjectBreak> MaterialFromSlotNode = new UE::Mutable::Private::NodeMaterialSkeletalMeshObjectBreak();
				MaterialFromSlotNode->SkeletalMeshObject = UE::Mutable::Private::GenerateMutableSourceSkeletalMeshObject(ConnectedMeshPin, GenerationContext, {});
				MaterialFromSlotNode->SlotName = TargetSlotName;

				Result = MaterialFromSlotNode;

				// Log unsuported textures error if needed
				const FMaterialBreakParameter CurrentBreakParameter = GenerationContext.GetCurrentMaterialBreakParameter();
				if (!CurrentBreakParameter.ParameterType.IsNone() && CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Texture)
				{
					FText Msg = FText::Format(LOCTEXT("NodeSKMBreak_UnsupportedOperation", "Breaking the Skeletal Mesh Material parameter [{0}] into a Mutable Texture is currently unsupported. Use it as a passthrough texture."),
						FText::FromName(CurrentBreakParameter.ParameterKey.ParameterName));
					GenerationContext.Log(Msg, Node, EMessageSeverity::Error);

					Result = nullptr;
				}
			}
		}
	}
	
	else if (const UCONodeSkeletalMeshBreak_V2* TypedSkeletalMeshBreakNode = Cast<UCONodeSkeletalMeshBreak_V2>(Node))
	{
		const UCONodeSkeletalMeshBreakPinData_V2& OverrideMaterialPinData = TypedSkeletalMeshBreakNode->GetPinData<UCONodeSkeletalMeshBreakPinData_V2>(*Pin);
		const FName TargetSlotName = OverrideMaterialPinData.TargetMaterialSlotName;

		if (!TargetSlotName.IsNone())
		{
			if (const UEdGraphPin* ConnectedMeshPin = FollowInputPin(*TypedSkeletalMeshBreakNode->InputPassthroughMesh.Get()))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialSkeletalMeshBreak> MaterialFromSlotNode = new UE::Mutable::Private::NodeMaterialSkeletalMeshBreak();
				FSourceSkeletalMeshOptions Options;
				Options.NumLODs = UINT8_MAX;
				MaterialFromSlotNode->SkeletalMesh = UE::Mutable::Private::GenerateMutableSourceSkeletalMesh(ConnectedMeshPin, GenerationContext, Options);
				MaterialFromSlotNode->SlotName = TargetSlotName;

				Result = MaterialFromSlotNode;

				// Log unsuported textures error if needed
				const FMaterialBreakParameter CurrentBreakParameter = GenerationContext.GetCurrentMaterialBreakParameter();
				if (!CurrentBreakParameter.ParameterType.IsNone() && CurrentBreakParameter.ParameterType == UEdGraphSchema_CustomizableObject::PC_Texture)
				{
					FText Msg = FText::Format(LOCTEXT("NodeSKMBreak_UnsupportedOperation", "Breaking the Skeletal Mesh Material parameter [{0}] into a Mutable Texture is currently unsupported. Use it as a passthrough texture."),
						FText::FromName(CurrentBreakParameter.ParameterKey.ParameterName));
					GenerationContext.Log(Msg, Node, EMessageSeverity::Error);

					Result = nullptr;
				}
			}
		}
	}

	else if (const UCONodeMaterialModify* TypedNodeMaterialModify = Cast<UCONodeMaterialModify>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialModify> MaterialModifyNode = new UE::Mutable::Private::NodeMaterialModify();
		Result = MaterialModifyNode;

		check(TypedNodeMaterialModify->MaterialPinRef.Get())
		const UEdGraphPin* MaterialPin = FollowInputPin(*TypedNodeMaterialModify->MaterialPinRef.Get());

		if (!MaterialPin)
		{
			const FText ErrorMessage = LOCTEXT("ModifyMaterial_MissingMaterialConnection","Missing Material Connection. Modify Material Nodes require a material node.");
			GenerationContext.Log(ErrorMessage, Node, EMessageSeverity::Error);

			return Result;
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> MaterialNode = GenerateMutableSourceMaterial(MaterialPin, GenerationContext, MaterialOptions);

		if (!MaterialNode)
		{
			const FText ErrorMessage = LOCTEXT("ModifyMaterial_FailedMaterialGeneration", "The Generation of the material node has failed.");
			GenerationContext.Log(ErrorMessage, Node, EMessageSeverity::Error);

			return Result;
		}

		MaterialModifyNode->MaterialSource = MaterialNode;

		// Generating the material parameters that will be modified
		//-----------------------------------------------------------------
		// TODOs: 
		// - Add support for table node materials. This will be added when the current hack of the tables is removed. UE-314401
		// - Add support for group projectors.
		// - Remove the material edition part from the mesh section nodes. Talk about this with Gerard since we can lose the capacity to pass an image parameter of the reference material as mutable without linking any texture node.

		// Tracking the index of the image. Needed for the FGeneratedImagePropertyKey
		int32 ImageIndex = 0;
	
		for (const UEdGraphPin* InputPin : TypedNodeMaterialModify->GetAllNonOrphanPins())
		{
			if (FEdGraphPinReference(InputPin) == TypedNodeMaterialModify->MaterialPinRef.Get() || InputPin->Direction == EGPD_Output)
			{
				continue;
			}

			const FName ParameterName = TypedNodeMaterialModify->GetPinParameterName(*InputPin);
			if (ParameterName.IsNone())
			{
				GenerationContext.Log(LOCTEXT("NoneParameterName", "'None' is not a valid Material Parameter name."), Node, EMessageSeverity::Warning);
				continue;
			}
			
			const int32 LayerIndex = TypedNodeMaterialModify->GetPinParameterLayerIndex(*InputPin);
			check(LayerIndex <= MAX_int8);

			UE::Mutable::Private::FParameterKey ParameterKey = { ParameterName, (int8)LayerIndex };

			const FName PinType = InputPin->PinType.PinCategory;
			if (PinType == UEdGraphSchema_CustomizableObject::PC_Texture)
			{
				UE::Mutable::Private::FImageParameterData ImageData;

				if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
				{
					TObjectPtr<UTexture2D> ReferenceTexture = GenerationContext.LoadObject(TypedNodeMaterialModify->GetTexturePinParameterReferenceTexture(*InputPin));
					if (!ReferenceTexture)
					{
						// Fallback if the user did not provide a reference texture
						ReferenceTexture = FindReferenceImage(FollowPin, GenerationContext);
					}
					
					FGeneratedImageProperties Props = GenerateImageProperties(Node, ReferenceTexture, ParameterKey, false, GenerationContext);
					GenerationContext.CurrentImageProperties = &Props;

					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ImageNode = GenerateMutableSourceImage(FollowPin, GenerationContext, Props.TextureSize);

					GenerationContext.CurrentImageProperties = nullptr;

					// NOTE: Important step to generate a texture with the correct format and size
					ImageNode = GenerateMipMapAndFormatNodes(Node, GenerationContext, ImageNode, ReferenceTexture);

					// Fill Image Data
					ImageData.ImageNode = ImageNode;
					ImageData.ImagePropertyIndex = Props.ImagePropertiesIndex;
					ImageData.LayoutIndex = TypedNodeMaterialModify->GetImagePinParameterUVIndex(*InputPin);
					ImageData.bIsPassthrough = false;

					if (ImageData.LayoutIndex != INDEX_NONE && GenerationContext.CompilationContext->Options.bUseLegacyLayouts == false)
					{
						int32 NumBlocksX = static_cast<int32>(TypedNodeMaterialModify->UVPackagingSettings[ImageData.LayoutIndex].MaxGridSizeX);
						int32 NumBlocksY = static_cast<int32>(TypedNodeMaterialModify->UVPackagingSettings[ImageData.LayoutIndex].MaxGridSizeY);
						GetLayoutBlockSizeInPixels(GenerationContext, ReferenceTexture, NumBlocksX, NumBlocksY, ImageData.BlockSizeX, ImageData.BlockSizeY);
					}
				}

				// We do not check Null images since it is possible to not have anything linked
				MaterialModifyNode->ImageParameters.Add(ParameterKey, ImageData);

				ImageIndex++;
			}

			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
			{
				UE::Mutable::Private::FImageParameterData ImageData;

				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*InputPin))
				{
					FGeneratedImageProperties Props = GenerateImageProperties(Node, nullptr, ParameterKey, true, GenerationContext);

					// This is a connected pass-through texture that simply has to be passed to the core
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> PassThroughImagePtr = GenerateMutableSourceImage(ConnectedPin, GenerationContext, 0);

					check(Props.ImagePropertiesIndex != INDEX_NONE);

					ImageData.ImageNode = PassThroughImagePtr;
					ImageData.ImagePropertyIndex = Props.ImagePropertiesIndex;
					ImageData.LayoutIndex = -1;
					ImageData.bIsPassthrough = true;
				}

				// We do not check Null images since it is possible to not have anything linked
				MaterialModifyNode->ImageParameters.Add(ParameterKey, ImageData);

				ImageIndex++;
			}

			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Color)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor> ColorNode;

				if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
				{
					ColorNode = GenerateMutableSourceColor(FollowPin, GenerationContext);
				}

				// We do not check Null images since it is possible to not have anything linked
				MaterialModifyNode->ColorParameters.Add(ParameterKey, ColorNode);
			}

			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Float)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> ScalarNode;

				if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
				{
					ScalarNode = GenerateMutableSourceFloat(FollowPin, GenerationContext);
				}

				// We do not check Null images since it is possible to not have anything linked
				MaterialModifyNode->ScalarParameters.Add(ParameterKey, ScalarNode);
			}

			else
			{
				unimplemented();
			}
		}
	}

	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		//This node will add a default value in case of error
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialConstant> ConstantValue = new UE::Mutable::Private::NodeMaterialConstant();
		ConstantValue->MaterialId = GenerationContext.CompilationContext->PassthroughObjectFactory.Add(*GenerationContext.CurrentReferencedMaterial.Get(), false);

		Result = ConstantValue;

		if (Pin->PinType.PinCategory == Schema->PC_Material)
		{
			// Material pins have to skip the cache of nodes or they will return always the same column node
			bCacheNode = false;
		}

		if (UDataTable* DataTable = GetDataTable(TypedNodeTable, GenerationContext))
		{
			bool bSuccess = true;
			FString ColumnName = TypedNodeTable->GetPinColumnName(Pin);
			FProperty* Property = TypedNodeTable->FindPinProperty(*Pin);

			if (!Property)
			{
				FString Msg = FString::Printf(TEXT("Couldn't find the column [%s] in the data table's struct."), *ColumnName);
				GenerationContext.Log(FText::FromString(Msg), Node);

				bSuccess = false;
			}

			if (bSuccess)
			{
				if (UMaterialInstance* TableMaterial = TypedNodeTable->GetColumnDefaultAssetByType<UMaterialInstance>(Pin))
				{
					UMaterialInterface* BaseMaterial = GenerationContext.CurrentReferencedMaterial.Get();

					const bool bTableMaterialCheckDisabled = GenerationContext.CompilationContext->RootObject->GetPrivate()->IsTableMaterialsParentCheckDisabled();
					const bool bMaterialParentMismatch = !bTableMaterialCheckDisabled && BaseMaterial
						&& TableMaterial->GetMaterial() != BaseMaterial->GetMaterial();

					// Checking if the reference material of the Table Node has the same parent as the material of the Material Node 
					if (bMaterialParentMismatch)
					{
						GenerationContext.Log(LOCTEXT("DifferentParentMaterial", "The Default Material of the Data Table and the Mesh Section must have the same Parent Material."), Node);
						bSuccess = false;
					}
				}
				else
				{
					FString TableColumnName = TypedNodeTable->GetPinColumnName(Pin);
					FText Msg = FText::Format(LOCTEXT("DefaultValueNotFound", "Couldn't find a default value in the data table's struct for the column {0}. The default value is null or not a Material Instance."), FText::FromString(TableColumnName));
					GenerationContext.Log(Msg, Node);
					bSuccess = false;
				}
			}

			if (bSuccess)
			{
				// Generating a new data table if not exists
				UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> Table;
				Table = GenerateMutableSourceTable(DataTable, TypedNodeTable, GenerationContext);

				if (Table)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialTable> MaterialTableNode = new UE::Mutable::Private::NodeMaterialTable();

					// Materials use the parameter id as column names
					ColumnName = GenerationContext.CurrentMaterialParameterId;

					// Generating a new material column if it does not exist
					if (Table->FindColumn(ColumnName) == INDEX_NONE)
					{
						GenerationContext.CurrentTableColumnType = UE::Mutable::Private::ETableColumnType::Material;
						bSuccess = GenerateTableColumn(TypedNodeTable, Pin, Table, ColumnName, Property, FMutableSourceMeshData(), GenerationContext);

						if (!bSuccess)
						{
							FString Msg = FString::Printf(TEXT("Failed to generate the mutable table column [%s]"), *ColumnName);
							GenerationContext.Log(FText::FromString(Msg), Node);
						}
					}

					if (bSuccess)
					{
						Result = MaterialTableNode;

						MaterialTableNode->Table = Table;
						MaterialTableNode->ColumnName = ColumnName;
						MaterialTableNode->ParameterName = TypedNodeTable->ParameterName;
						MaterialTableNode->bNoneOption = TypedNodeTable->bAddNoneOption;
						MaterialTableNode->DefaultRowName = TypedNodeTable->DefaultRowName.ToString();
					}
				}
				else
				{
					FString Msg = FString::Printf(TEXT("Couldn't generate a mutable table."));
					GenerationContext.Log(FText::FromString(Msg), Node);
				}
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MaterialTableError", "Couldn't find the data table of the node."), Node);
		}
	}

	else if (Cast<UCONodeExternalOperation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialExternal> MeshExtension = new UE::Mutable::Private::NodeMaterialExternal();

		FSourceExternalOptions Options;
		Options.SurfaceNode = MaterialOptions.SurfaceNode;

		MeshExtension->Node = GenerateMutableSourceExternal(Pin, GenerationContext, Options);
	
		Result = MeshExtension;
	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeMaterial>(*Pin, GenerationContext, GenerateMutableSourceMaterial, MaterialOptions);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeMaterial>(*Pin, GenerationContext, GenerateMutableSourceMaterial, MaterialOptions);
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}


FGeneratedImageProperties GenerateImageProperties(const UCustomizableObjectNode* Node, const UTexture2D* ReferenceTexture, const UE::Mutable::Private::FParameterKey& ImageKey, bool bIsPassthrough, FMutableGraphGenerationContext& GenerationContext)
{
	// Find or add Image properties
	const FGeneratedImagePropertiesKey PropsKey(Node, ImageKey);
	const bool bNewImageProps = !GenerationContext.ImageProperties.Contains(PropsKey);

	FGeneratedImageProperties& Props = GenerationContext.ImageProperties.FindOrAdd(PropsKey);

	if (!bNewImageProps)
	{
		return Props;
	}

	// Store properties for the generated images
	Props.TextureParameterName = ImageKey.ParameterName.ToString();
	Props.ImagePropertiesIndex = GenerationContext.ImageProperties.Num() - 1;
	Props.bIsPassThrough = bIsPassthrough;

	if (bIsPassthrough)
	{
		// We don't need more info for passthrough textures.
		return Props;
	}

	if (ReferenceTexture)
	{
		Props.CompressionSettings = ReferenceTexture->CompressionSettings;
		Props.Filter = ReferenceTexture->Filter;
		Props.SRGB = ReferenceTexture->SRGB;
		Props.LODBias = 0;
		Props.MipGenSettings = ReferenceTexture->MipGenSettings;
		Props.LODGroup = ReferenceTexture->LODGroup;
		Props.AddressX = ReferenceTexture->AddressX;
		Props.AddressY = ReferenceTexture->AddressY;
		Props.bFlipGreenChannel = ReferenceTexture->bFlipGreenChannel;

		// MaxTextureSize setting. Based on the ReferenceTexture and Platform settings.
		const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
		Props.MaxTextureSize = GetMaxTextureSize(*ReferenceTexture, TextureLODSettings);

		// ReferenceTexture source size. Textures contributing to this Image should be equal to or smaller than TextureSize. 
		// The LOD Bias applied to the root node will be applied on top of it.
		Props.TextureSize = (int32)FMath::Max3(ReferenceTexture->Source.GetSizeX(), ReferenceTexture->Source.GetSizeY(), 1LL);

		// TODO: Remove once the base LODBias added to root gets removed.
		// Keep the Base LODBias that will be applied to the root. Used when generatin texture parameters to compensate the base LODBias. 
		Props.BaseLODBias = ComputeLODBiasForTexture(GenerationContext, *ReferenceTexture);

		// TODO: MTBL-1081
		// TextureGroup::TEXTUREGROUP_UI does not support streaming. If we generate a texture that requires streaming and set this group, it will crash when initializing the resource. 
		// If LODGroup == TEXTUREGROUP_UI, UTexture::IsPossibleToStream() will return false and UE will assume all mips are loaded, when they're not, and crash.
		if (Props.LODGroup == TEXTUREGROUP_UI)
		{
			Props.LODGroup = TextureGroup::TEXTUREGROUP_Character;

			FString msg = FString::Printf(TEXT("The Reference texture [%s] is using TEXTUREGROUP_UI which does not support streaming. Please set a different TEXTURE group."),
				*ReferenceTexture->GetName());
			GenerationContext.Log(FText::FromString(msg), Node, EMessageSeverity::Info);
		}
	}
	else
	{
		// warning!
		FString msg = FString::Printf(TEXT("The Reference texture for material image [%s] is not set and it couldn't be found automatically."), *ImageKey.ParameterName.ToString());
		GenerationContext.Log(FText::FromString(msg), Node);
	}

	return Props;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> GenerateMipMapAndFormatNodes(
	const UCustomizableObjectNode* Node,
	FMutableGraphGenerationContext& GenerationContext,
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ImageNode,
	UTexture2D* ReferenceTexture)
{
	if (!ReferenceTexture)
	{
		return ImageNode;
	}

	// Apply base LODBias. It will be propagated to most images.
	const uint32 BaseLODBias = ComputeLODBiasForTexture(GenerationContext, *ReferenceTexture);
	UE::Mutable::Private::NodeImagePtr LastImage = ResizeTextureByNumMips(ImageNode, BaseLODBias);

	if (ReferenceTexture->MipGenSettings != TextureMipGenSettings::TMGS_NoMipmaps)
	{
		UE::Mutable::Private::EMipmapFilterType MipGenerationFilterType = Invoke([&]()
			{
				if (ReferenceTexture)
				{
					switch (ReferenceTexture->MipGenSettings)
					{
					case TextureMipGenSettings::TMGS_SimpleAverage: return UE::Mutable::Private::EMipmapFilterType::SimpleAverage;
					case TextureMipGenSettings::TMGS_Unfiltered:    return UE::Mutable::Private::EMipmapFilterType::Unfiltered;
					default: return UE::Mutable::Private::EMipmapFilterType::SimpleAverage;
					}
				}

				return UE::Mutable::Private::EMipmapFilterType::SimpleAverage;
			});


		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageMipmap> MipmapImage = new UE::Mutable::Private::NodeImageMipmap();
		MipmapImage->Source = LastImage;
		MipmapImage->Settings.FilterType = MipGenerationFilterType;
		MipmapImage->Settings.AddressMode = UE::Mutable::Private::EAddressMode::None;

		MipmapImage->SetMessageContext(Node);
		LastImage = MipmapImage;
	}

	// Apply composite image. This needs to be computed after mipmaps generation. 	
	if (ReferenceTexture && ReferenceTexture->GetCompositeTexture() && ReferenceTexture->CompositeTextureMode != CTM_Disabled)
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageNormalComposite> CompositedImage = new UE::Mutable::Private::NodeImageNormalComposite();
		CompositedImage->Base = LastImage;
		CompositedImage->Power = ReferenceTexture->CompositePower;

		UE::Mutable::Private::ECompositeImageMode CompositeImageMode = [CompositeTextureMode = ReferenceTexture->CompositeTextureMode]() -> UE::Mutable::Private::ECompositeImageMode
			{
				switch (CompositeTextureMode)
				{
				case CTM_NormalRoughnessToRed: return UE::Mutable::Private::ECompositeImageMode::CIM_NormalRoughnessToRed;
				case CTM_NormalRoughnessToGreen: return UE::Mutable::Private::ECompositeImageMode::CIM_NormalRoughnessToGreen;
				case CTM_NormalRoughnessToBlue: return UE::Mutable::Private::ECompositeImageMode::CIM_NormalRoughnessToBlue;
				case CTM_NormalRoughnessToAlpha: return UE::Mutable::Private::ECompositeImageMode::CIM_NormalRoughnessToAlpha;

				default: return UE::Mutable::Private::ECompositeImageMode::CIM_Disabled;
				}
			}();

		CompositedImage->Mode = CompositeImageMode;

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageConstant> CompositeNormalImage = new UE::Mutable::Private::NodeImageConstant();

		UTexture2D* ReferenceCompositeNormalTexture = Cast<UTexture2D>(ReferenceTexture->GetCompositeTexture());
		if (ReferenceCompositeNormalTexture)
		{
			// GenerationContext.ArrayTextureUnrealToMutableTask.Add(FTextureUnrealToMutableTask(CompositeNormalImage, ReferenceCompositeNormalTexture, Node, true));
			// TODO: The normal composite part is not propagated, so it will be unsupported. Create a task that performs the required transforms at mutable image level, and add the right operations here
			// instead of propagating the flag and doing them on unreal-convert.
			CompositeNormalImage->Image = GenerateImageConstant(ReferenceCompositeNormalTexture, GenerationContext, false);

			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImageMipmap> NormalCompositeMipmapImage = new UE::Mutable::Private::NodeImageMipmap();
			const uint32 MipsToSkip = ComputeLODBiasForTexture(GenerationContext, *ReferenceCompositeNormalTexture, ReferenceTexture);
			NormalCompositeMipmapImage->Source = ResizeTextureByNumMips(CompositeNormalImage, MipsToSkip);
			NormalCompositeMipmapImage->Settings.FilterType = UE::Mutable::Private::EMipmapFilterType::SimpleAverage;
			NormalCompositeMipmapImage->Settings.AddressMode = UE::Mutable::Private::EAddressMode::None;

			CompositedImage->Normal = NormalCompositeMipmapImage;

			CompositeNormalImage->SourceDataDescriptor.OptionalMaxLODSize = 0;
			if (GenerationContext.CompilationContext->Options.TargetPlatform)
			{
				const UTextureLODSettings& TextureLODSettings = GenerationContext.CompilationContext->Options.TargetPlatform->GetTextureLODSettings();
				const FTextureLODGroup& LODGroupInfo = TextureLODSettings.GetTextureLODGroup(ReferenceCompositeNormalTexture->LODGroup);

				CompositeNormalImage->SourceDataDescriptor.OptionalMaxLODSize = LODGroupInfo.OptionalMaxLODSize;
				CompositeNormalImage->SourceDataDescriptor.OptionalLODBias = LODGroupInfo.OptionalLODBias;
				CompositeNormalImage->SourceDataDescriptor.NumNonOptionalLODs = GenerationContext.CompilationContext->Options.MinDiskMips;
			}

			const FString TextureName = GetNameSafe(ReferenceCompositeNormalTexture).ToLower();
			CompositeNormalImage->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*TextureName), TextureName.Len() * sizeof(FString::ElementType));
		}

		LastImage = CompositedImage;
	}

	LastImage = GenerateMutableImageFormat(LastImage, ReferenceTexture, GenerationContext, Node);

	return LastImage;
}


#undef LOCTEXT_NAMESPACE

