// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenerateMutableSource.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeMaterialParameter.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;

/** Generates or retrieves an already generated Material Parameter based on the Pin and Generation Context provided. 
 * @param Pin Does represent the pin from the node we want to use as source for the generation or retrieval of the Mutable Material Parameter Node.
 * @param GenerationContext The compilation context of the current compilation operation.
 * @return A pointer to the generated/retrieved Mutable NodeMaterialParameter
 */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterialParameter> GenerateMutableSourceMaterialParameter(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);

// Add to this struct all the extra parameters that need to be passed to the GenerateMutableSourceMaterial function.
struct FGenerationMaterialOptions 
{
	// Node surface. Only needed for MeshSection nodes.
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurfaceNew> SurfaceNode = nullptr;

	// Size of the reference Texture Size
	int32 ReferenceTextureSize = 0;
};

/** Convert a CustomizableObject Source Graph into a mutable source graph. */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial> GenerateMutableSourceMaterial(
	const UEdGraphPin* Pin,
	FMutableGraphGenerationContext& GenerationContext,
	const FGenerationMaterialOptions& GenerationOptions);

/** Generates an FGeneratedImageProperties from a Material Parameter and (if not null) its Reference Texture. */
FGeneratedImageProperties GenerateImageProperties(const UCustomizableObjectNode* Node, const UTexture2D* ReferenceTexture, const UE::Mutable::Private::FParameterKey& ImageKey, bool bIsPassthrough, FMutableGraphGenerationContext& GenerationContext);

// Generate additional nodes to generate a texture with the correct format and size.
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> GenerateMipMapAndFormatNodes(
	const UCustomizableObjectNode* Node,
	FMutableGraphGenerationContext& GenerationContext,
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeImage> ImageNode,
	UTexture2D* ReferenceTexture);
