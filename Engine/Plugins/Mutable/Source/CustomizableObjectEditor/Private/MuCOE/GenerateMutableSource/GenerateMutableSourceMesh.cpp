// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"

#include "Algo/Count.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "AnimGraphNode_RigidBody.h"
#include "ClothConfigBase.h"
#include "ClothingAsset.h"
#include "GenerateMutableSourceExternal.h"
#include "GenerateMutableSourceImage.h"
#include "GenerateMutableSourceSkeletalMesh.h"
#include "GenerateMutableSourceSkeletalMeshObject.h"
#include "GenerateMutableSourceTransform.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/UnrealConversionUtils.h"
#include "MuR/MeshBufferUtils.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceLayout.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackApplication.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackDefinition.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshape.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeCommon.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeMeshMakeMorph.h"
#include "MuT/NodeMeshMorph.h"
#include "MuT/NodeMeshReshape.h"
#include "MuT/NodeMeshSwitch.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeMeshVariation.h"
#include "MuT/NodeMeshSkeletalMeshObjectBreak.h"
#include "MuR/ConvertData.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/SkeletalMeshSocket.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Hash/CityHash.h"
#include "PackedNormal.h"
#include "MuCOE/Nodes/CONodeClipMeshWithMesh.h"
#include "MuCOE/Nodes/CONodeTransformWithBone.h"
#include "MuCO/CustomizableObjectInstanceAssetUserData.h"
#include "MuCOE/Nodes/CONodeExternalOperation.h"
#include "MuCOE/Nodes/CONodeRemoveMesh.h"
#include "MuCOE/Nodes/CONodeTransformInMesh.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuT/NodeMeshClipWithMesh.h"
#include "MuT/NodeMeshExternal.h"
#include "MuT/NodeMeshRemoveMesh.h"
#include "MuT/NodeMeshTransform.h"
#include "MuT/NodeMeshTransformInMesh.h"
#include "MuT/NodeMeshTransformWithBone.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

struct FMutableConvertSkeletalMeshContext
{
	int32 NumBaseBuffers = 0;

	// Skeleton
	bool bShouldRemoveBonesInfluences = false;
	TArray<FBoneIndexType> BoneMap;
	TArray<FBoneIndexType> RemappedBoneMapIndices;

	// Skinning
	int32 SourceNumBoneInfluences = 0;
	UE::Mutable::Private::EMeshBufferFormat SourceSkinWeightFormat;

	UE::Mutable::Private::EMeshBufferFormat TargetBoneIndexFormat;
	UE::Mutable::Private::EMeshBufferFormat TargetBoneWeightFormat;
	int32 NumBoneInfluences = 0;

};

void SetBuffersFormat(FMutableCompilationContext& Context,
	FMutableConvertSkeletalMeshContext& MeshContext,
	const USkeletalMesh& SourceMesh,
	const FSkeletalMeshModel& SourceImportedModel,
	int32 LODIndex,
	int32 SectionIndex,
	EMutableMeshConversionFlags Flags,
	UE::Mutable::Private::FMesh& MutableMesh)
{
	if (!SourceImportedModel.LODModels.IsValidIndex(LODIndex)
		|| !SourceImportedModel.LODModels[LODIndex].Sections.IsValidIndex(SectionIndex))
	{
		return;
	}

	const bool bIgnoreSkinning = EnumHasAnyFlags(Flags, EMutableMeshConversionFlags::IgnoreSkinning);
	const bool bIgnorePhysics = EnumHasAnyFlags(Flags, EMutableMeshConversionFlags::IgnorePhysics);
	const bool bIgnoreMorphs = EnumHasAnyFlags(Flags, EMutableMeshConversionFlags::IgnoreMorphs);
	const bool bIgnoreTexCoords = EnumHasAnyFlags(Flags, EMutableMeshConversionFlags::IgnoreTexCoords);

	const FSkeletalMeshLODModel& LODModel = SourceImportedModel.LODModels[LODIndex];
	const FSkelMeshSection& MeshSection = LODModel.Sections[SectionIndex];

	check(bIgnoreSkinning || MutableMesh.Skeleton);

	MeshContext.TargetBoneIndexFormat = MeshSection.BoneMap.Num() > 256 ? UE::Mutable::Private::EMeshBufferFormat::UInt16 : UE::Mutable::Private::EMeshBufferFormat::UInt8;
	MeshContext.TargetBoneWeightFormat = Context.Options.b16BitBoneWeightsEnabled ? UE::Mutable::Private::EMeshBufferFormat::NUInt16 : UE::Mutable::Private::EMeshBufferFormat::NUInt8;
	const int32 MaxBoneWeightTypeSizeBytes = Context.Options.b16BitBoneWeightsEnabled ? 2 : 1;
	const int32 MaxBoneIndexTypeSizeBytes = MeshSection.BoneMap.Num() > 256 ? 2 : 1;

	const int32 MaxNumBonesPerVertex = MeshSection.GetMaxBoneInfluences();
	MeshContext.SourceNumBoneInfluences = MaxNumBonesPerVertex;
	// Limit skinning weights if necessary
	// \todo: make it more flexible to support 3 or 5 or 1 weight, since there is support for this in 4.25
	MeshContext.NumBoneInfluences = 0;
	if (!FGPUBaseSkinVertexFactory::GetUnlimitedBoneInfluences(Context.Options.TargetPlatform))
	{
		MeshContext.NumBoneInfluences = FMath::Min(MaxNumBonesPerVertex, (int32)Context.Options.CustomizableObjectNumBoneInfluences, EXTRA_BONE_INFLUENCES);
	}
	else
	{
		MeshContext.NumBoneInfluences = FMath::Min(MaxNumBonesPerVertex, (int32)Context.Options.CustomizableObjectNumBoneInfluences);
	}

	ensure(MeshContext.NumBoneInfluences <= MAX_TOTAL_INFLUENCES);

	if (MeshContext.NumBoneInfluences != MaxNumBonesPerVertex)
	{
		UE_LOGF(LogMutable, Verbose, "In object [%ls] Mesh bone number adjusted from %d to %d.", *Context.RootObject->GetFullName(), MaxNumBonesPerVertex, MeshContext.NumBoneInfluences);
	}
	
	const bool bHasTexCoords = !bIgnoreTexCoords;
	const bool bHasSkinWeights = !bIgnoreSkinning && MeshContext.NumBoneInfluences > 0;
	const bool bHasVertexColors = SourceMesh.GetHasVertexColors();
	const bool bHasClothing = !bIgnoreSkinning && SourceMesh.HasActiveClothingAssetsForLOD(LODIndex);

	int32 MutableBufferCount = 2; // Position & Tangents

	if (bHasTexCoords)
	{
		++MutableBufferCount;
	}

	if (bHasVertexColors)
	{
		++MutableBufferCount;
	}

	if (bHasSkinWeights)
	{
		++MutableBufferCount;
	}

	MeshContext.NumBaseBuffers = MutableBufferCount;

	UE::Mutable::Private::FMeshBufferSet& VertexBuffers = MutableMesh.GetVertexBuffers();
	VertexBuffers.SetBufferCount(MutableBufferCount);

	int32 CurrentVertexBuffer = 0;

	using namespace UE::Mutable::Private;

	// Vertex buffer
	MeshBufferUtils::SetupVertexPositionsBuffer(CurrentVertexBuffer, VertexBuffers);
	check(!VertexBuffers.Buffers[CurrentVertexBuffer].HasPadding());
	++CurrentVertexBuffer;

	// Tangent buffer
	MeshBufferUtils::SetupTangentBuffer(CurrentVertexBuffer, VertexBuffers);
	check(!VertexBuffers.Buffers[CurrentVertexBuffer].HasPadding());
	++CurrentVertexBuffer;

	// Texture coords buffer
	if (bHasTexCoords)
	{
		bool bHighPrecision = true;
		MeshBufferUtils::SetupTexCoordinatesBuffer(CurrentVertexBuffer, LODModel.NumTexCoords, bHighPrecision, VertexBuffers);
		check(!VertexBuffers.Buffers[CurrentVertexBuffer].HasPadding());
		++CurrentVertexBuffer;
	}

	// Skin buffer
	if (bHasSkinWeights)
	{
		MeshBufferUtils::SetupSkinBuffer(CurrentVertexBuffer, MaxBoneIndexTypeSizeBytes, MaxBoneWeightTypeSizeBytes, MeshContext.NumBoneInfluences, VertexBuffers);
		check(!VertexBuffers.Buffers[CurrentVertexBuffer].HasPadding());
		++CurrentVertexBuffer;
	}


	// Color buffer
	if (bHasVertexColors)
	{
		MeshBufferUtils::SetupVertexColorBuffer(CurrentVertexBuffer, VertexBuffers);
		check(!VertexBuffers.Buffers[CurrentVertexBuffer].HasPadding());
		++CurrentVertexBuffer;
	}

	// Index buffer
	MeshBufferUtils::SetupIndexBuffer(MutableMesh.GetIndexBuffers(), EMeshBufferFormat::UInt32);
}


void GetLODAndSection(const FMutableCompilationContext& Context, const FMutableSourceMeshData& Source, const USkeletalMesh& SkeletalMesh, int32& OutLODIndex, int32& OutSectionIndex)
{
	OutLODIndex = Source.BaseLODIndex;
	OutSectionIndex = Source.BaseSectionIndex;
	
	if (Source.bOnlyConnectedLOD || Source.LODOffset == 0)
	{
		return;
	}
	
	const FSkeletalMeshModel* ImportedModel = SkeletalMesh.GetImportedModel();
	if (!ImportedModel)
	{
		return;
	}

	if (!ImportedModel->LODModels.IsValidIndex(Source.BaseLODIndex) ||
		!ImportedModel->LODModels[Source.BaseLODIndex].Sections.IsValidIndex(Source.BaseSectionIndex))
	{
		return;
	}
	
	const FSkelMeshSection& FromSection = ImportedModel->LODModels[Source.BaseLODIndex].Sections[Source.BaseSectionIndex];
	const TArray<int32>& FromMaterialMap = SkeletalMesh.GetLODInfo(Source.BaseLODIndex)->LODMaterialMap;
	
	// Material Index of the connected pin
	const int32 SearchLODMaterialIndex = FromMaterialMap.IsValidIndex(Source.BaseSectionIndex) && SkeletalMesh.GetMaterials().IsValidIndex(FromMaterialMap[Source.BaseSectionIndex]) ?
		FromMaterialMap[Source.BaseSectionIndex] :
		FromSection.MaterialIndex;


	const int32 CompilingLODIndex = Source.BaseLODIndex + Source.LODOffset;
	OutLODIndex = INDEX_NONE;
	OutSectionIndex = INDEX_NONE;

	if (ImportedModel->LODModels.IsValidIndex(CompilingLODIndex))
	{
		const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[CompilingLODIndex];
		const TArray<int32>& MaterialMap = SkeletalMesh.GetLODInfo(CompilingLODIndex)->LODMaterialMap;

		for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
		{
			const int32 MaterialIndex = MaterialMap.IsValidIndex(SectionIndex) && SkeletalMesh.GetMaterials().IsValidIndex(MaterialMap[SectionIndex]) ?
				MaterialMap[SectionIndex] :
				LODModel.Sections[SectionIndex].MaterialIndex; // MaterialMap overrides the MaterialIndex in the section

			if (MaterialIndex == SearchLODMaterialIndex &&
				!LODModel.Sections[SectionIndex].bDisabled)
			{
				if (OutLODIndex == INDEX_NONE && OutSectionIndex == INDEX_NONE)
				{
					OutLODIndex = CompilingLODIndex;
					OutSectionIndex = SectionIndex;
				}
				else 
				{
					Context.Log(FText::Format(LOCTEXT("MeshMultipleMaterialIndex", "Mesh {0} contains multiple sections with the same Material Index"), FText::FromString(SkeletalMesh.GetName())), Source.MessageContext);
				}
			}
		}
	}
}


void BuildRemappedBonesArray(TObjectPtr<const USkeletalMesh> InSkeletalMesh, int32 InLODIndex, const TArray<FBoneIndexType>& InRequiredBones, TArray<FBoneIndexType>& OutRemappedBones)
{
	if (!InSkeletalMesh)
	{
		return;
	}
	
	const FReferenceSkeleton& ReferenceSkeleton = InSkeletalMesh->GetRefSkeleton();
	const int32 NumBones = ReferenceSkeleton.GetNum();

	// Build RemappedBones array
	OutRemappedBones.Init(0, NumBones);

	const TArray<FMeshBoneInfo>& RefBoneInfos = ReferenceSkeleton.GetRefBoneInfo();
	const int32 NumLODInfos = InSkeletalMesh->GetLODNum();

	// Helper to know which bones have been removed
	TBitArray<> RemovedBones(false, NumBones);

	for (const FBoneIndexType& RequiredBoneIndex : InRequiredBones)
	{
		const FMeshBoneInfo& BoneInfo = RefBoneInfos[RequiredBoneIndex];
		FBoneIndexType FinalBoneIndex = RequiredBoneIndex;

		// Remove bone if the parent has been removed, Root can't be removed
		if (BoneInfo.ParentIndex != INDEX_NONE && RemovedBones[BoneInfo.ParentIndex])
		{
			RemovedBones[RequiredBoneIndex] = true;
			FinalBoneIndex = OutRemappedBones[BoneInfo.ParentIndex];
		}

		else
		{
			// Check if it has to be removed
			bool bBoneRemoved = false;

			// If the bone has not been remove yet, check if it's in the BonesToRemove of the SkeletalMesh.
			for (int32 LODIndex = 0; !bBoneRemoved && LODIndex <= InLODIndex && LODIndex < NumLODInfos; ++LODIndex)
			{
				const FBoneReference* BoneToRemove = InSkeletalMesh->GetLODInfo(LODIndex)->BonesToRemove.FindByPredicate(
					[&BoneInfo](const FBoneReference& BoneReference) { return BoneReference.BoneName == BoneInfo.Name; });
				
				bBoneRemoved = BoneToRemove != nullptr;
				RemovedBones[RequiredBoneIndex] |= bBoneRemoved;
			}

			// Fix up FinalBoneIndex if it has been removed. Root can't be removed
			FinalBoneIndex = !bBoneRemoved || BoneInfo.ParentIndex == INDEX_NONE ? RequiredBoneIndex : OutRemappedBones[BoneInfo.ParentIndex];
		}

		OutRemappedBones[RequiredBoneIndex] = FinalBoneIndex;
	}

}


void RemoveBoneInfluences(uint16* InfluenceBones, uint16* InfluenceWeights, const int32 InfluenceCount, const TArray<FBoneIndexType>& RemappedBoneMapIndices)
{
	const int32 BoneMapBoneCount = RemappedBoneMapIndices.Num();

	for (int32 i = 0; i < InfluenceCount; ++i)
	{
		if (InfluenceBones[i] < BoneMapBoneCount)
		{
			bool bParentFound = false;
			FBoneIndexType ParentIndex = RemappedBoneMapIndices[InfluenceBones[i]];
			for (int32 j = 0; j < i; ++j)
			{
				if (InfluenceBones[j] == ParentIndex)
				{
					InfluenceWeights[j] += InfluenceWeights[i];

					InfluenceBones[i] = 0;
					InfluenceWeights[i] = 0.f;
					bParentFound = true;
					break;
				}
			}

			if (!bParentFound)
			{
				InfluenceBones[i] = ParentIndex;
			}
		}
		else
		{
			InfluenceBones[i] = 0;
			InfluenceWeights[i] = 0.f;
		}
	}
}


void NormalizeWeights(uint16* InfluenceBones, uint16* InfluenceWeights, const int32 InfluenceCount, const int32 MutableInfluenceCount, const int32 MaxBoneWeight)
{
	uint16 MaxOrderedBones[MAX_TOTAL_INFLUENCES];
	uint16 MaxOrderedWeights[MAX_TOTAL_INFLUENCES];

	int32 TotalWeight = 0;
	int32 UsedMask = 0;

	// First sort the indices of the influences. Heaviest influences first
	for (int8 i = 0; i < MutableInfluenceCount; ++i)
	{
		uint16 CurrentMaxWeight = 0;
		int8 CurrentMaxIndex = 0;
		
		for (int8 j = 0; j < InfluenceCount; ++j)
		{
			if ((UsedMask & (1 << j)) == 0
				&& InfluenceWeights[j] > CurrentMaxWeight)
			{
				CurrentMaxWeight = InfluenceWeights[j];
				CurrentMaxIndex = j;
			}
		}

		UsedMask |= (1 << CurrentMaxIndex);
		MaxOrderedBones[i] = InfluenceBones[CurrentMaxIndex];
		MaxOrderedWeights[i] = CurrentMaxWeight;
		TotalWeight += CurrentMaxWeight;
	}

	// Copy bone influences
	FMemory::Memcpy(&InfluenceBones[0], &MaxOrderedBones[0], MutableInfluenceCount);

	// Renormalize and copy weights
	if (TotalWeight > 0)
	{
		int32 AssignedWeight = 0;
		for (int32 j = 1; j < MutableInfluenceCount; ++j)
		{
			int32 Res = MaxOrderedWeights[j] * MaxBoneWeight / TotalWeight;
			AssignedWeight += Res;
			InfluenceWeights[j] = Res;
		}

		InfluenceWeights[0] = MaxBoneWeight - AssignedWeight;
		
		for (int32 j = 1; j < MutableInfluenceCount; ++j)
		{
			if (InfluenceWeights[j] == 0)
			{
				FMemory::Memzero(&InfluenceWeights[j], (MutableInfluenceCount - j) * sizeof(uint16));
				break;
			}
		}
	}
	else
	{
		FMemory::Memzero(InfluenceWeights, MutableInfluenceCount * sizeof(InfluenceWeights[0]));
		InfluenceWeights[0] = MaxBoneWeight;
	}
}

void ConvertVertexSkinningData(uint16* InfluenceBones, const UE::Mutable::Private::EMeshBufferFormat DestBonesFormat
	, uint16* InfluenceWeights, const UE::Mutable::Private::EMeshBufferFormat DestWeightsFormat, const int32 DestNumInfluences
	, const int32 MaxWeight, const FMutableConvertSkeletalMeshContext& MeshContext)
{
	bool bNormalize = DestNumInfluences != MeshContext.SourceNumBoneInfluences;

	if (MeshContext.bShouldRemoveBonesInfluences)
	{
		RemoveBoneInfluences(InfluenceBones, InfluenceWeights, MeshContext.SourceNumBoneInfluences, MeshContext.RemappedBoneMapIndices);
		bNormalize = true;
	}

	if (DestWeightsFormat != MeshContext.SourceSkinWeightFormat)
	{
		uint16* DestWeightData = InfluenceWeights;
		if (DestWeightsFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt8
			&& MeshContext.SourceSkinWeightFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16)
		{
			for (int32 Index = 0; Index < MeshContext.SourceNumBoneInfluences; ++Index)
			{
				*DestWeightData /= (MAX_uint16 / MAX_uint8);
				++DestWeightData;
			}
			bNormalize = true;
		}
		else if (DestWeightsFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16
			&& MeshContext.SourceSkinWeightFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt8)
		{
			for (int32 Index = 0; Index < MeshContext.SourceNumBoneInfluences; ++Index)
			{
				*DestWeightData *= (MAX_uint16 / MAX_uint8);
				++DestWeightData;
			}
		}
		else
		{
			check(false);
		}
	}

	if (bNormalize)
	{
		NormalizeWeights(InfluenceBones, InfluenceWeights, MeshContext.SourceNumBoneInfluences, DestNumInfluences, MaxWeight);
	}

	// Remove Padding
	if (DestBonesFormat == UE::Mutable::Private::EMeshBufferFormat::UInt8)
	{
		uint8* DestBoneData = reinterpret_cast<uint8*>(InfluenceBones);
		for (int32 Index = 0; Index < DestNumInfluences; ++Index)
		{
			*DestBoneData = uint8(*InfluenceBones);
			++DestBoneData;
			++InfluenceBones;
		}
	}

	// RemovePadding
	if (DestWeightsFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt8)
	{
		uint8* DestWeightData = reinterpret_cast<uint8*>(InfluenceWeights);
		for (int32 Index = 0; Index < DestNumInfluences; ++Index)
		{
			*DestWeightData = uint8(*InfluenceWeights);
			++DestWeightData;
			++InfluenceWeights;
		}
	}
}


TArray<FName> GetUsedRealTimeMorphsNames(const FMutableSourceMeshData& SourceMeshData, bool bEnableRealTimeMorphTargets, const TArray<FRealTimeMorphSelectionOverride>& Overrides)
{
	TArray<FName> Result;
	
	if (!bEnableRealTimeMorphTargets)
	{
		return Result;
	}

	if (SourceMeshData.TableReferenceSkeletalMesh.IsNull())
	{	
		const USkeletalMesh* SourceSkeletalMesh = Cast<USkeletalMesh>(UE::Mutable::Private::LoadObject(SourceMeshData.Mesh));
		
		if (SourceSkeletalMesh)
		{
			if (SourceMeshData.bUseAllRealTimeMorphs)
			{
				const TArray<UMorphTarget*>& SkeletalMeshMorphTargets = SourceSkeletalMesh->GetMorphTargets();

				for (const UMorphTarget* MorphTarget : SkeletalMeshMorphTargets)
				{
					check(MorphTarget);
					Result.Add(MorphTarget->GetFName());
				}
			}
			else
			{
				for (const FString& MorphName : SourceMeshData.UsedRealTimeMorphTargetNames)
				{
					Result.Emplace(*MorphName);
				}
			}

			for (const FRealTimeMorphSelectionOverride& MorphTargetOverride : Overrides)
			{
				const ECustomizableObjectSelectionOverride OverrideValue = Invoke([&]() -> ECustomizableObjectSelectionOverride
				{
					const int32 FoundMeshIndex = MorphTargetOverride.SkeletalMeshes.IndexOfByPredicate(
					[Name = SourceSkeletalMesh->GetFName()](const FSkeletalMeshMorphTargetOverride& Elem)
					{
						return Name == Elem.SkeletalMeshName;
					});

					if (FoundMeshIndex != INDEX_NONE)
					{
						return MorphTargetOverride.SkeletalMeshes[FoundMeshIndex].SelectionOverride;
					}

					return MorphTargetOverride.SelectionOverride;
				});

				if (OverrideValue == ECustomizableObjectSelectionOverride::Enable)
				{
					Result.AddUnique(MorphTargetOverride.MorphName);
				}
				else if (OverrideValue == ECustomizableObjectSelectionOverride::Disable)
				{
					Result.Remove(MorphTargetOverride.MorphName);
				}
			}
		}
	}
	else
	{
		const USkeletalMesh* TableReferenceSkeletalMesh = Cast<USkeletalMesh>(UE::Mutable::Private::LoadObject(SourceMeshData.TableReferenceSkeletalMesh));

		if (TableReferenceSkeletalMesh)
		{
			for (const UMorphTarget* MorphTarget : TableReferenceSkeletalMesh->GetMorphTargets())
			{
				if (MorphTarget)
				{
					Result.Add(MorphTarget->GetFName());
				}
			}
		}
	}


	return Result;
}

TArray<TTuple<UPhysicsAsset*, int32>> GetPhysicsAssetsFromAnimInstance(const TSoftClassPtr<UAnimInstance>& AnimInstance)
{
	// TODO: Consider caching the result in the GenerationContext.
	TArray<TTuple<UPhysicsAsset*, int32>> Result;

	if (AnimInstance.IsNull())
	{
		return Result;
	}

	// TODO: Get Physics Asset using AssetRegistry
	UClass* AnimInstanceClass = nullptr;
	if (IsInGameThread())
	{
		AnimInstanceClass = UE::Mutable::Private::LoadClass(AnimInstance);
	}
	else
	{
		AnimInstanceClass = AnimInstance.Get();
		check(AnimInstanceClass);
	}

	UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstanceClass);

	if (AnimClass)
	{
		const int32 AnimNodePropertiesNum = AnimClass->AnimNodeProperties.Num();
		for ( int32 PropertyIndex = 0; PropertyIndex < AnimNodePropertiesNum; ++PropertyIndex)
		{
			FStructProperty* StructProperty = AnimClass->AnimNodeProperties[PropertyIndex];

			if (StructProperty->Struct->IsChildOf(FAnimNode_RigidBody::StaticStruct()))
			{
				FAnimNode_RigidBody* Rban = StructProperty->ContainerPtrToValuePtr<FAnimNode_RigidBody>(AnimInstanceClass->GetDefaultObject());
			
				if (Rban && Rban->OverridePhysicsAsset)
				{
					Result.Emplace(Rban->OverridePhysicsAsset, PropertyIndex);
				}
			}
		}
	}		

	return Result;
}

TArray<TTuple<UPhysicsAsset*, int32>> GetPhysicsAssetsFromAnimInstance(FMutableGraphGenerationContext& GenerationContext, const TSoftClassPtr<UAnimInstance>& AnimInstance)
{
	// TODO: Consider caching the result in the GenerationContext.
	TArray<TTuple<UPhysicsAsset*, int32>> Result;

	if (AnimInstance.IsNull())
	{
		return Result;
	}

	UClass* AnimInstanceClass = GenerationContext.LoadClass(AnimInstance);
	if (AnimInstanceClass)
	{
		Result = GetPhysicsAssetsFromAnimInstance(AnimInstance);
	}

	return Result;
}


namespace UE::Mutable::Private
{
	UE::Tasks::FTask ConvertSkeletalMeshDefaultBuffers(UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh, const FSkeletalMeshLODModel& LODModel, const FSkelMeshSection& MeshSection, TSharedPtr<FMutableConvertSkeletalMeshContext>& MeshContext)
	{
		TArrayView<const FSoftSkinVertex> Vertices(MeshSection.SoftVertices);

		const int32 NumVertices = Vertices.Num();
		check(Vertices.Num() == MutableMesh->GetVertexCount());
		if (NumVertices == 0)
		{
			return UE::Tasks::MakeCompletedTask<void>();
		}

		TArray<UE::Tasks::FTask> Prerequisites;

		// Vertices
		const int32 NumVertexBuffers = MutableMesh->GetVertexBuffers().GetBufferCount();
		for (int32 BufferIndex = 0; BufferIndex < MeshContext->NumBaseBuffers; ++BufferIndex)
		{
			Prerequisites.Add(
				UE::Tasks::Launch(TEXT("BufferConversion"),
					[BufferIndex, MutableMesh, MeshContext, Vertices, NumVertices]()
					{
						const FMutableConvertSkeletalMeshContext& MeshContextRef = *MeshContext.Get();

						UE::Mutable::Private::FMeshBuffer& VertexBuffer = MutableMesh->GetVertexBuffers().Buffers[BufferIndex];
						uint8* DestData = VertexBuffer.Data.GetData();
						const int32 DestDataSize = VertexBuffer.ElementSize;

						const int32 NumChannels = VertexBuffer.Channels.Num();
						if (VertexBuffer.Channels[0].Semantic == UE::Mutable::Private::EMeshBufferSemantic::Position)
						{
							MUTABLE_CPUPROFILER_SCOPE(ConvertBuffers_Position);

							check(NumChannels == 1);

							FVector3f* DestPosition = reinterpret_cast<FVector3f*>(VertexBuffer.Data.GetData());
							for (const FSoftSkinVertex& Vertex : Vertices)
							{
								*DestPosition = Vertex.Position;
								++DestPosition;
							}
						}
						else if (NumChannels == 2
							&& VertexBuffer.Channels[0].Semantic == UE::Mutable::Private::EMeshBufferSemantic::Tangent
							&& VertexBuffer.Channels[1].Semantic == UE::Mutable::Private::EMeshBufferSemantic::Normal)
						{
							MUTABLE_CPUPROFILER_SCOPE(ConvertBuffers_Tangent);

							// TODO: ParallelFor x2
							const int32 NumComponents = VertexBuffer.Channels[0].ComponentCount;
							check(VertexBuffer.Channels[1].ComponentCount == NumComponents);

							const UE::Mutable::Private::EMeshBufferFormat TangentFormat = VertexBuffer.Channels[0].Format;
							const UE::Mutable::Private::EMeshBufferFormat NormalFormat = VertexBuffer.Channels[1].Format;

							const int32 TangentDataSize = UE::Mutable::Private::GetMeshFormatData(TangentFormat).SizeInBytes * NumComponents;
							const int32 NormalDataSize = UE::Mutable::Private::GetMeshFormatData(NormalFormat).SizeInBytes * NumComponents;
							check(TangentDataSize == NormalDataSize && TangentDataSize == sizeof(FPackedNormal));

							FPackedNormal* TypedDestData = reinterpret_cast<FPackedNormal*>(DestData);

							for (const FSoftSkinVertex& Vertex : Vertices)
							{
								// Tangent
								*TypedDestData = Vertex.TangentX;
								(*TypedDestData).Vector.W = 0;
								++TypedDestData;

								// Normal
								*TypedDestData = Vertex.TangentZ;
								FMatrix44f Mat(Vertex.TangentX, Vertex.TangentY, FVector3f(Vertex.TangentZ), FVector3f(0, 0, 0));
								(*TypedDestData).Vector.W = Mat.RotDeterminant() < 0 ? -128 : 127;
								++TypedDestData;
							}
						}
						else if (NumChannels == 2
							&& VertexBuffer.Channels[0].Semantic == UE::Mutable::Private::EMeshBufferSemantic::BoneIndices
							&& VertexBuffer.Channels[1].Semantic == UE::Mutable::Private::EMeshBufferSemantic::BoneWeights)
						{
							MUTABLE_CPUPROFILER_SCOPE(ConvertBuffers_Skin);
							// TODO: ParallelFor x4
							const int32 MaxSectionBoneInfluences = MeshContextRef.SourceNumBoneInfluences;
							const int32 NumBoneInfluences = VertexBuffer.Channels[0].ComponentCount;

							const UE::Mutable::Private::EMeshBufferFormat BoneIndicesFormat = VertexBuffer.Channels[0].Format;
							const UE::Mutable::Private::EMeshBufferFormat BoneWeightFormat = VertexBuffer.Channels[1].Format;

							const int32 BoneIndicesSizeBytes = UE::Mutable::Private::GetMeshFormatData(BoneIndicesFormat).SizeInBytes * NumBoneInfluences;
							const int32 BoneWeightsSizeBytes = UE::Mutable::Private::GetMeshFormatData(BoneWeightFormat).SizeInBytes * NumBoneInfluences;

							const uint16 MaxBoneWeightValue = BoneWeightFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16 ? MAX_uint16 : MAX_uint8;

							uint16 InfluenceBones[MAX_TOTAL_INFLUENCES];
							uint16 InfluenceWeights[MAX_TOTAL_INFLUENCES];

							for (const FSoftSkinVertex& Vertex : Vertices)
							{
								FMemory::Memcpy(&InfluenceBones[0], &Vertex.InfluenceBones[0], sizeof(uint16) * MaxSectionBoneInfluences);
								FMemory::Memcpy(&InfluenceWeights[0], &Vertex.InfluenceWeights[0], sizeof(uint16) * MaxSectionBoneInfluences);

								ConvertVertexSkinningData(&InfluenceBones[0], BoneIndicesFormat,
									&InfluenceWeights[0], BoneWeightFormat, NumBoneInfluences, MaxBoneWeightValue, MeshContextRef);

								// Final copy
								FMemory::Memcpy(DestData, &InfluenceBones[0], BoneIndicesSizeBytes);
								DestData += BoneIndicesSizeBytes;
								FMemory::Memcpy(DestData, &InfluenceWeights[0], BoneWeightsSizeBytes);
								DestData += BoneWeightsSizeBytes;
							}
						}
						else if (VertexBuffer.Channels[0].Semantic == UE::Mutable::Private::EMeshBufferSemantic::TexCoords)
						{
							MUTABLE_CPUPROFILER_SCOPE(ConvertBuffers_TexCoords);

							constexpr size_t SoftSkinVertexUVsElemSize = sizeof(TDecay<decltype(DeclVal<FSoftSkinVertex>().UVs[0])>::Type);

							if (DestDataSize == SoftSkinVertexUVsElemSize * NumChannels)
							{
								for (const FSoftSkinVertex& Vertex : Vertices)
								{
									FMemory::Memcpy(DestData, &Vertex.UVs[0], DestDataSize);
									DestData += DestDataSize;
								}
							}
							else
							{
								const UE::Mutable::Private::EMeshBufferFormat DestFormat = VertexBuffer.Channels[0].Format;
								const int32 DataTypeSize = UE::Mutable::Private::GetMeshFormatData(DestFormat).SizeInBytes;

								for (const FSoftSkinVertex& Vertex : Vertices)
								{
									for (int32 TexCoordIndex = 0; TexCoordIndex < NumChannels; ++TexCoordIndex)
									{
										UE::Mutable::Private::ConvertData(0, DestData, DestFormat, &Vertex.UVs[TexCoordIndex].X, UE::Mutable::Private::EMeshBufferFormat::Float32);
										DestData += DataTypeSize;
										UE::Mutable::Private::ConvertData(0, DestData, DestFormat, &Vertex.UVs[TexCoordIndex].Y, UE::Mutable::Private::EMeshBufferFormat::Float32);
										DestData += DataTypeSize;
									}
								}
							}
						}
						else if (VertexBuffer.Channels[0].Semantic == UE::Mutable::Private::EMeshBufferSemantic::Color)
						{
							MUTABLE_CPUPROFILER_SCOPE(ConvertBuffers_Color);

							check(NumChannels == 1);
							check(VertexBuffer.Channels[0].ComponentCount == 4);
							check(VertexBuffer.Channels[0].Format == UE::Mutable::Private::EMeshBufferFormat::NUInt8);

							uint32* TypedDestData = reinterpret_cast<uint32*>(DestData);
							for (const FSoftSkinVertex& Vertex : Vertices)
							{
								*TypedDestData = Vertex.Color.DWColor();
								++TypedDestData;
							}
						}
						else
						{
							unimplemented();
						}
					}
					));
		}

		// Indices
		Prerequisites.Add(
			UE::Tasks::Launch(TEXT("Convert Indices"),
				[&MeshSection, MutableMesh, &LODModel]()
				{
					MUTABLE_CPUPROFILER_SCOPE(ConvertBuffers_Indices);

					const uint32 VertexStart = MeshSection.BaseVertexIndex;
					const uint32 VertexCount = MeshSection.NumVertices;
					const uint32 IndexStart = MeshSection.BaseIndex;
					const uint32 IndexCount = MeshSection.NumTriangles * 3;
					MutableMesh->GetIndexBuffers().SetElementCount(IndexCount);

					check(LODModel.IndexBuffer.IsValidIndex(IndexStart) && LODModel.IndexBuffer.IsValidIndex(IndexStart + IndexCount - 1));
					const uint32* IndexDataPtr = &LODModel.IndexBuffer[IndexStart];

					uint32* pDest = reinterpret_cast<uint32*>(MutableMesh->GetIndexBuffers().GetBufferData(0));

					// 32-bit to 32-bit
					uint32 VertexIndex = 0;
					for (uint32 Index = 0; Index < IndexCount; ++Index)
					{
						VertexIndex = *IndexDataPtr - VertexStart;
						if (VertexIndex < VertexCount)
						{
							*pDest = VertexIndex;
						}
						else
						{
							// Malformed mesh?
							ensure(false);
							*pDest = 0;
						}
						++pDest;
						++IndexDataPtr;
					}
				}));

		return UE::Tasks::Launch(TEXT("ConvertSkeletalMeshBuffers Completed"),
			[]() {}, Prerequisites, LowLevelTasks::ETaskPriority::Inherit);
	}

}


UE::Tasks::FTask ConvertSkeletalMeshToMutable(TSharedRef<FMeshConversionContext> MeshConversionContext)
{
	check(IsInGameThread());

	const FMutableSourceMeshData& Source = MeshConversionContext->Source;
	FMutableCompilationContext& Context = *MeshConversionContext->CompilationContext;
	const FString& MorphName = MeshConversionContext->MorphName;

	const FString MeshName = Source.Mesh.GetLongPackageName().ToLower();
	const uint32 MeshId = CityHash32(reinterpret_cast<const char*>(*MeshName), MeshName.Len() * sizeof(FString::ElementType));

	const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Source.Mesh.Get());
	if (!SkeletalMesh)
	{
		const FString Msg = FString::Printf(TEXT("The SkeletalMesh [%s] failed to load."), *MeshName);
		Context.Log(FText::FromString(Msg), Source.MessageContext);
		return UE::Tasks::MakeCompletedTask<void>();
	}

	const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
	if (!ImportedModel)
	{
		const FString Msg = FString::Printf(TEXT("The SkeletalMesh [%s] doesn't have an imported resource."), *MeshName);
		Context.Log(FText::FromString(Msg), Source.MessageContext);
		return UE::Tasks::MakeCompletedTask<void>();
	}

	// Data tables may allow missing data. Don't throw warnings of missing LODs or sections.
	const bool bIgnoreMissingData = Source.TableReferenceSkeletalMesh || Context.RootObject->GetPrivate()->bDisableTableMissingDataWarning;

	// If we are using automatic LODs and not generating the base LOD generate a mesh even if empty.
	const bool bMeshMustExist = Source.LODOffset == 0;

	// Find the correct LOD and Section indices when using automatic LODs 
	int32 LODIndex = INDEX_NONE;
	int32 SectionIndex = INDEX_NONE;
	GetLODAndSection(Context, Source, *SkeletalMesh, LODIndex, SectionIndex);

	if (!ImportedModel->LODModels.IsValidIndex(LODIndex))
	{
		if (!bMeshMustExist)
		{
			MeshConversionContext->Result = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>(); // Return empty mesh to preserve the layouts
			return UE::Tasks::MakeCompletedTask<void>();
		}
		else
		{
			if (!bIgnoreMissingData)
			{
				const FString Msg = FString::Printf(
					TEXT("The SkeletalMesh [%s] doesn't have the expected number of LODs [need %d, has %d]. Changed after reimporting?"),
					*MeshName,
					LODIndex + 1,
					ImportedModel->LODModels.Num());
				Context.Log(FText::FromString(Msg), Source.MessageContext);
			}
			return UE::Tasks::MakeCompletedTask<void>();
		}
	}


	const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		if (!bMeshMustExist)
		{
			MeshConversionContext->Result = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>(); // Return empty mesh to preserve the layouts
			return UE::Tasks::MakeCompletedTask<void>();
		}
		else
		{
			if (!bIgnoreMissingData)
			{
				const FString Msg = FString::Printf(
					TEXT("The SkeletalMesh [%s] doesn't have the expected structure. Maybe the number of LODs [need %d, has %d] or Materials [need %d, has %d] has changed after reimporting?"),
					*SkeletalMesh->GetName(),
					LODIndex + 1,
					ImportedModel->LODModels.Num(),
					SectionIndex + 1,
					LODModel.Sections.Num());
				Context.Log(FText::FromString(Msg), Source.MessageContext);
			}
			return UE::Tasks::MakeCompletedTask<void>();
		}
	}

	const FSkelMeshSection& MeshSection = LODModel.Sections[SectionIndex];

	// Get the mesh generation flags to use
	const EMutableMeshConversionFlags CurrentFlags = Source.Flags;
	const bool bIgnoreSkinning = EnumHasAnyFlags(CurrentFlags, EMutableMeshConversionFlags::IgnoreSkinning);
	const bool bIgnorePhysics = EnumHasAnyFlags(CurrentFlags, EMutableMeshConversionFlags::IgnorePhysics);
	const bool bIgnoreMorphs = EnumHasAnyFlags(CurrentFlags, EMutableMeshConversionFlags::IgnoreMorphs);
	const bool bIgnoreAUD = EnumHasAnyFlags(CurrentFlags, EMutableMeshConversionFlags::IgnoreAUD);

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();
	TSharedPtr<FMutableConvertSkeletalMeshContext> MeshContext = MakeShared<FMutableConvertSkeletalMeshContext>();
	
	if (!bIgnoreAUD)
	{
		for (UAssetUserData* AssetUserData : *SkeletalMesh->GetAssetUserDataArray())
		{
			if (AssetUserData)
			{
				MutableMesh->AssetUserData.AddUnique(Context.PassthroughObjectFactory.Add(*AssetUserData, true));
			}
		}
	}
	
	const int32 NumGameplayTags = Source.GameplayTags.Num();
	
	TArray<FName> GameplayTags;
	GameplayTags.Reserve(NumGameplayTags);
	for (int32 Index = 0; Index < NumGameplayTags; ++Index)
	{
		GameplayTags.Add(Source.GameplayTags.GetByIndex(Index).GetTagName());
	}
	
	MutableMesh->GameplayTags = GameplayTags;
	if (UClass* AnimInstance = UE::Mutable::Private::LoadClass(Source.AnimInstance))
	{
		MutableMesh->AnimationSlots.Emplace(Source.AnimBPSlotName, Context.PassthroughObjectFactory.Add(*AnimInstance, false));
	}

	bool bBoneMapModified = false;
	TArray<FBoneIndexType> BoneMap;

	// Check if the Skeleton is valid and build the UE::Mutable::Private::FSkeleton
	if (!bIgnoreSkinning)
	{
		const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
		if (!Skeleton)
		{
			FString Msg = FString::Printf(TEXT("No skeleton provided when converting SkeletalMesh [%s]."), *MeshName);
			Context.Log(FText::FromString(Msg), Source.MessageContext);
			return UE::Tasks::MakeCompletedTask<void>();
		}

		MutableMesh->SkeletonObjects.AddUnique(Context.PassthroughObjectFactory.Add(*const_cast<USkeleton*>(Skeleton), false));
		
		// RefSkeleton check
		{
			// Ensure the bones used by the Skeletal Mesh exits in the Mesh's Skeleton
			const TArray<FMeshBoneInfo>& RawRefBoneInfo = SkeletalMesh->GetRefSkeleton().GetRawRefBoneInfo();
			const FReferenceSkeleton& InSkeletonRefSkeleton = Skeleton->GetReferenceSkeleton();

			bool bIsSkeletonMissingBones = false;

			for (const FMeshBoneInfo& BoneInfo : RawRefBoneInfo)
			{
				if (InSkeletonRefSkeleton.FindRawBoneIndex(BoneInfo.Name) == INDEX_NONE)
				{
					bIsSkeletonMissingBones = true;
					FString Msg = FString::Printf(TEXT("SkeletalMesh [%s] uses bone [%s] not present in skeleton [%s]."),
						*MeshName,
						*BoneInfo.ExportName,
						*Skeleton->GetName());
					Context.Log(FText::FromString(Msg), Source.MessageContext);
				}
			}

			// Discard SkeletalMesh if some bones are missing
			if (bIsSkeletonMissingBones)
			{
				FString Msg = FString::Printf(
					TEXT("The Skeleton [%s] is missing bones that SkeletalMesh [%s] needs. The mesh will be discarded! Information about missing bones can be found in the Output Log."),
					*Skeleton->GetName(), *MeshName);
				Context.Log(FText::FromString(Msg), Source.MessageContext);

				return UE::Tasks::MakeCompletedTask<void>();
			}
		}

		const TArray<uint16>& SourceRequiredBones = LODModel.RequiredBones;

		// Remove bones and build an array to remap indices of the BoneMap
		TArray<FBoneIndexType> RemappedBones;
		BuildRemappedBonesArray(SkeletalMesh, LODIndex, SourceRequiredBones, RemappedBones);

		// Build RequiredBones array
		TArray<FBoneIndexType> RequiredBones;
		RequiredBones.Reserve(SourceRequiredBones.Num());

		for (const FBoneIndexType& RequiredBoneIndex : SourceRequiredBones)
		{
			RequiredBones.AddUnique(RemappedBones[RequiredBoneIndex]);
		}

		// Rebuild BoneMap
		const TArray<uint16>& SourceBoneMap = MeshSection.BoneMap;
		const int32 NumBonesInBoneMap = SourceBoneMap.Num();
		const int32 NumRemappedBones = RemappedBones.Num();

		for (int32 BoneIndex = 0; BoneIndex < NumBonesInBoneMap; ++BoneIndex)
		{
			const FBoneIndexType BoneMapBoneIndex = SourceBoneMap[BoneIndex];
			const FBoneIndexType FinalBoneIndex = BoneMapBoneIndex < NumRemappedBones ? RemappedBones[BoneMapBoneIndex] : 0;

			const int32 BoneMapIndex = BoneMap.AddUnique(FinalBoneIndex);
			MeshContext->RemappedBoneMapIndices.Add(BoneMapIndex);

			bBoneMapModified = bBoneMapModified || SourceBoneMap[BoneIndex] != FinalBoneIndex;
		}
		MeshContext->bShouldRemoveBonesInfluences = bBoneMapModified;

		// Create the skeleton, poses, and BoneMap for this mesh
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeleton> MutableSkeleton = 
				UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FSkeleton>();
		MutableMesh->SetSkeleton(MutableSkeleton);

		const int32 NumRequiredBones = RequiredBones.Num();
		MutableMesh->SetBonePoseCount(NumRequiredBones);

		MutableSkeleton->BoneNames.Reserve(NumRequiredBones);
		MutableSkeleton->BoneParents.Reserve(NumRequiredBones);

		// MutableBoneMap will not keep an index to the Skeleton, but to the BoneName
		TArray<UE::Mutable::Private::FBoneIdOrIndex> MutableBoneMap;
		MutableBoneMap.SetNum(BoneMap.Num());

		TArray<FMatrix> ComposedRefPoseMatrices;
		ComposedRefPoseMatrices.SetNum(NumRequiredBones);

		const TArray<FMeshBoneInfo>& RefBoneInfo = SkeletalMesh->GetRefSkeleton().GetRefBoneInfo();
		for (int32 BoneIndex = 0; BoneIndex < NumRequiredBones; ++BoneIndex)
		{
			const int32 RefSkeletonBoneIndex = RequiredBones[BoneIndex];

			const FMeshBoneInfo& BoneInfo = RefBoneInfo[RefSkeletonBoneIndex];
			const int32 ParentBoneIndex = RequiredBones.Find(BoneInfo.ParentIndex);

			UE::Mutable::Private::FBoneIdOrIndex MutableBoneIndex;
			MutableBoneIndex.Index = MutableSkeleton->AddBone(BoneInfo.Name, ParentBoneIndex);

			// BoneMap: Convert RefSkeletonBoneIndex to BoneId
			const int32 BoneMapIndex = BoneMap.Find(RefSkeletonBoneIndex);
			if (BoneMapIndex != INDEX_NONE)
			{
				MutableBoneMap[BoneMapIndex] = MutableBoneIndex;
			}

			if (ParentBoneIndex >= 0)
			{
				ComposedRefPoseMatrices[BoneIndex] = SkeletalMesh->GetRefPoseMatrix(RefSkeletonBoneIndex) * ComposedRefPoseMatrices[ParentBoneIndex];
			}
			else
			{
				ComposedRefPoseMatrices[BoneIndex] = SkeletalMesh->GetRefPoseMatrix(RefSkeletonBoneIndex);
			}

			// Set bone pose
			FTransform3f BoneTransform;
			BoneTransform.SetFromMatrix(FMatrix44f(ComposedRefPoseMatrices[BoneIndex]));

			UE::Mutable::Private::EBoneUsageFlags BoneUsageFlags = UE::Mutable::Private::EBoneUsageFlags::None;
			EnumAddFlags(BoneUsageFlags, BoneMapIndex != INDEX_NONE ? UE::Mutable::Private::EBoneUsageFlags::Skinning : UE::Mutable::Private::EBoneUsageFlags::None);
			EnumAddFlags(BoneUsageFlags, ParentBoneIndex == INDEX_NONE ? UE::Mutable::Private::EBoneUsageFlags::Root : UE::Mutable::Private::EBoneUsageFlags::None);

			MutableMesh->SetBonePose(BoneIndex, MutableBoneIndex, BoneTransform, BoneUsageFlags);
		}

		MutableMesh->SetBoneMap(MutableBoneMap);
	}

	if (!bIgnoreSkinning)
	{
		for (int32 SocketIndex = 0; SocketIndex < SkeletalMesh->NumSockets(); ++SocketIndex)
		{
			const USkeletalMeshSocket* Socket = SkeletalMesh->GetSocketByIndex(SocketIndex);
			check(Socket);

			UE::Mutable::Private::FMeshSocket MutableSocket;
			MutableSocket.SocketName = Socket->SocketName;
			MutableSocket.BoneName = Socket->BoneName;
			MutableSocket.RelativeLocation = Socket->RelativeLocation;
			MutableSocket.RelativeRotation = Socket->RelativeRotation;
			MutableSocket.RelativeScale = Socket->RelativeScale;
			MutableSocket.bForceAlwaysAnimated = Socket->bForceAlwaysAnimated;

			MutableMesh->Sockets.Add(MutableSocket);
		}
	}

	// Set Mutable Mesh's final buffer formats
	SetBuffersFormat(Context, *MeshContext.Get(), *SkeletalMesh, *ImportedModel, LODIndex, SectionIndex, Source.Flags, *MutableMesh.Get());

	// Vertices
	const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
	TArrayView<const FSoftSkinVertex> Vertices(Section.SoftVertices);

	const uint32 VertexCount = MeshSection.GetNumVertices();
	MutableMesh->GetVertexBuffers().SetElementCount(VertexCount);

	const int32 MaxSectionInfluences = MeshSection.MaxBoneInfluences;
	const bool bUseUnlimitedInfluences = FGPUBaseSkinVertexFactory::UseUnlimitedBoneInfluences(MaxSectionInfluences, Context.Options.TargetPlatform);

	int32 NextBufferIndex = MeshContext->NumBaseBuffers;

	if (!bIgnoreMorphs && SkeletalMesh->GetMorphTargets().Num())
	{
		UnrealConversionUtils::SetMorphData(
				MutableMesh.Get(), SkeletalMesh, 
				LODIndex, SectionIndex, Section.BaseVertexIndex, Section.NumVertices, LODModel.NumVertices,
				nullptr);
	}

	if (!bIgnoreSkinning)
	{
		if (MeshSection.NumVertices > 0)
		{
			const FSoftSkinVertex& FirstVertex = Vertices[0];

			uint16 TotalWeight = 0;
			for (int32 InfluenceIndex = 0; InfluenceIndex < MaxSectionInfluences; ++InfluenceIndex)
			{
				TotalWeight += FirstVertex.InfluenceWeights[InfluenceIndex];
			}

			MeshContext->SourceSkinWeightFormat = TotalWeight > MAX_uint8 ? UE::Mutable::Private::EMeshBufferFormat::NUInt16 : UE::Mutable::Private::EMeshBufferFormat::NUInt8;
		}
	}

	UE::Tasks::FTask ConvertSkeletalMeshBuffersTask = UE::Mutable::Private::ConvertSkeletalMeshDefaultBuffers(MutableMesh, LODModel, MeshSection, MeshContext);

	// SkinWeightProfiles vertex info.
	if (!bIgnoreSkinning && Context.Options.bSkinWeightProfilesEnabled)
	{
		const FMutableConvertSkeletalMeshContext& MeshContextRef = *MeshContext.Get();
		const int32 SourceDataSizeBytes = sizeof(uint16) * MeshContextRef.SourceNumBoneInfluences;
		static_assert(sizeof(FBoneIndexType) == sizeof(uint16));

		const TArray<FSkinWeightProfileInfo>& SkinWeightProfilesInfo = SkeletalMesh->GetSkinWeightProfiles();
		MutableMesh->SkinWeightProfiles.Reserve(SkinWeightProfilesInfo.Num());

		for (const FSkinWeightProfileInfo& Profile : SkinWeightProfilesInfo)
		{
			const FImportedSkinWeightProfileData* ImportedProfileData = LODModel.SkinWeightProfiles.Find(Profile.Name);
			if (!ImportedProfileData)
			{
				continue;
			}

			if (ImportedProfileData->SkinWeights.Num() != LODModel.NumVertices)
			{
				UE_LOGF(LogMutable, Display, "Invalid data found in SkeletalMesh [%ls] when converting SkinWeightProfiles. Regenerate and save the SkeletalMesh.", *SkeletalMesh->GetName());
				continue;
			}

			TBitArray<> DifferentWeights;
			DifferentWeights.Init(false, VertexCount);

			int32 NumDifferentWeights = 0;
			for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
			{
				const FRawSkinWeight& SkinWeight = ImportedProfileData->SkinWeights[VertexIndex + MeshSection.BaseVertexIndex];

				if (FMemory::Memcmp(&Vertices[VertexIndex].InfluenceBones[0], &SkinWeight.InfluenceBones[0], SourceDataSizeBytes) != 0
					||
					FMemory::Memcmp(&Vertices[VertexIndex].InfluenceWeights[0], &SkinWeight.InfluenceWeights[0], SourceDataSizeBytes) != 0)
				{
					DifferentWeights[VertexIndex] = true;
					++NumDifferentWeights;
				}
			}

			if (NumDifferentWeights == 0)
			{
				continue;
			}

			UE::Mutable::Private::FSkinWeightProfile& SkinWeightProfile = MutableMesh->SkinWeightProfiles.AddDefaulted_GetRef();
			SkinWeightProfile.Name = Profile.Name;

			const FName PlatformName = *Context.Options.TargetPlatform->PlatformName();
			SkinWeightProfile.bDefaultProfile = Profile.DefaultProfile.GetValueForPlatform(PlatformName);
			SkinWeightProfile.DefaultProfileFromLODIndex = Profile.DefaultProfileFromLODIndex.GetValueForPlatform(PlatformName);

			SkinWeightProfile.NumBoneInfluences = MeshContextRef.NumBoneInfluences;

			const UE::Mutable::Private::EMeshBufferFormat BoneIndexFormat = MeshContext->TargetBoneIndexFormat;
			const UE::Mutable::Private::EMeshBufferFormat BoneWeightFormat = MeshContext->TargetBoneWeightFormat;
			SkinWeightProfile.bUse16BitBoneIndex = BoneIndexFormat == UE::Mutable::Private::EMeshBufferFormat::UInt16;
			SkinWeightProfile.bUse16BitBoneWeight = BoneWeightFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16;

			const uint16 MaxBoneWeightValue = SkinWeightProfile.bUse16BitBoneWeight ? MAX_uint16 : MAX_uint8;
			const int32 MaxSectionBoneInfluences = MeshContextRef.SourceNumBoneInfluences;

			const int32 NumBoneInfluences = MeshContext->NumBoneInfluences;
			const int32 BoneIndicesStride = SkinWeightProfile.bUse16BitBoneIndex ? sizeof(uint16) * NumBoneInfluences : NumBoneInfluences;
			const int32 BoneWeightsStride = SkinWeightProfile.bUse16BitBoneWeight ? sizeof(uint16) * NumBoneInfluences : NumBoneInfluences;


			TMap<uint32, int32> SkinWeightHashToWeightIndex;
			SkinWeightHashToWeightIndex.Reserve(NumDifferentWeights);

			SkinWeightProfile.VertexIndexToInfluenceOffset.Reserve(NumDifferentWeights);

			SkinWeightProfile.BoneIDs.Reserve(NumDifferentWeights * BoneIndicesStride);
			SkinWeightProfile.BoneWeights.Reserve(NumDifferentWeights * BoneWeightsStride);

			uint16 InfluenceBones[MAX_TOTAL_INFLUENCES];
			uint16 InfluenceWeights[MAX_TOTAL_INFLUENCES];

			int32 SkinWeightIndex = 0;
			for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
			{
				if (!DifferentWeights[VertexIndex])
				{
					continue;
				}

				const FRawSkinWeight& SkinWeight = ImportedProfileData->SkinWeights[VertexIndex + MeshSection.BaseVertexIndex];

				FMemory::Memcpy(&InfluenceBones[0], &SkinWeight.InfluenceBones[0], SourceDataSizeBytes);
				FMemory::Memcpy(&InfluenceWeights[0], &SkinWeight.InfluenceWeights[0], SourceDataSizeBytes);

				ConvertVertexSkinningData(&InfluenceBones[0], BoneIndexFormat,
					&InfluenceWeights[0], BoneWeightFormat, NumBoneInfluences, MaxBoneWeightValue, MeshContextRef);

				// TODO: Add proper hashing to avoid collisions
				uint32 SkinWeightVertexHash = 0;
				for (int32 InfluenceIndex = 0; InfluenceIndex < NumBoneInfluences; ++InfluenceIndex)
				{
					SkinWeightVertexHash = HashCombine(SkinWeightVertexHash, InfluenceBones[InfluenceIndex]);
					SkinWeightVertexHash = HashCombine(SkinWeightVertexHash, InfluenceWeights[InfluenceIndex]);
				}

				if (int32* WeightIndex = SkinWeightHashToWeightIndex.Find(SkinWeightVertexHash))
				{
					SkinWeightProfile.VertexIndexToInfluenceOffset.Emplace(VertexIndex, *WeightIndex);
					continue;
				}

				SkinWeightProfile.BoneIDs.SetNumUninitialized((SkinWeightIndex + 1) * BoneIndicesStride);
				SkinWeightProfile.BoneWeights.SetNumUninitialized((SkinWeightIndex + 1) * BoneWeightsStride);

				FMemory::Memcpy(SkinWeightProfile.BoneIDs.GetData() + SkinWeightIndex * BoneIndicesStride, &InfluenceBones[0], BoneIndicesStride);
				FMemory::Memcpy(SkinWeightProfile.BoneWeights.GetData() + SkinWeightIndex * BoneWeightsStride, &InfluenceWeights[0], BoneWeightsStride);

				SkinWeightProfile.VertexIndexToInfluenceOffset.Emplace(VertexIndex, SkinWeightIndex);
				SkinWeightHashToWeightIndex.Add(SkinWeightVertexHash, SkinWeightIndex);

				++SkinWeightIndex;
			}
		}
	}


	if (!bIgnorePhysics && SkeletalMesh->GetPhysicsAsset() && MutableMesh->GetSkeleton() && Context.Options.bPhysicsAssetMergeEnabled)
	{
		UPhysicsAsset& PhysicsAsset = *SkeletalMesh->GetPhysicsAsset();

		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FPhysicsBody> PhysicsBody = UnrealConversionUtils::CreatePhysicsBodyForMesh(PhysicsAsset, *MutableMesh);

		if (PhysicsBody)
		{
			MutableMesh->SetPhysicsBody(PhysicsBody);
			MutableMesh->PhysicsAssets.Emplace(Context.PassthroughObjectFactory.Add(PhysicsAsset, false));

			// Set Physics usage flag
			const int32 NumBodySetups = PhysicsBody->GetBodyCount();
			for (int32 BodyIndex = 0; BodyIndex < NumBodySetups; ++BodyIndex)
			{

				const int32 PoseIndex = MutableMesh->Skeleton->BoneNames.Find(PhysicsBody->BodiesBoneNames[BodyIndex]); // Bone Poses are 1 : 1 with the skeleton bones
				check(PoseIndex != INDEX_NONE);
				EnumAddFlags(MutableMesh->BonePoses[PoseIndex].BoneUsageFlags, UE::Mutable::Private::EBoneUsageFlags::Physics);
			}
		}
	}

	// Set Bone Parenting usages. This has to be done after all primary usages are set.
	UnrealConversionUtils::PropagateBoneUsageFlagsThroughMeshPose(*MutableMesh);

	const bool bAnimPhysicsManipulationEnabled = Context.Options.bAnimBpPhysicsManipulationEnabled;

	if (!bIgnorePhysics && !Source.AnimInstance.IsNull() && MutableMesh->GetSkeleton() && bAnimPhysicsManipulationEnabled)
	{
		using AnimPhysicsInfoType = TTuple<UPhysicsAsset*, int32>;
		const TArray<AnimPhysicsInfoType> AnimPhysicsInfo = GetPhysicsAssetsFromAnimInstance(Source.AnimInstance);

		for (const AnimPhysicsInfoType PropertyInfo : AnimPhysicsInfo)
		{
			UPhysicsAsset* const PropertyAsset = PropertyInfo.Get<UPhysicsAsset*>();
			const int32 PropertyIndex = PropertyInfo.Get<int32>();

			FAnimBpOverridePhysicsAssetsInfo Info;
			{
				Info.AnimInstanceClass = Source.AnimInstance;
				Info.PropertyIndex = PropertyIndex;
			}

			const int32 PhysicsAssetId = Context.AnimBpOverridePhysicsAssetsInfo.AddUnique(Info);

			UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FPhysicsBody> MutableBody = UnrealConversionUtils::CreatePhysicsBodyForMesh(*PropertyAsset, *MutableMesh);

			MutableBody->CustomId = PhysicsAssetId;

			MutableMesh->AddAdditionalPhysicsBody(MutableBody);
			MutableMesh->AdditionalPhysicsAssets.Emplace(Context.PassthroughObjectFactory.Add(*PropertyAsset, false));
		}
	}
	
	// Ensure Surface Data
	UE::Mutable::Private::FMeshSurface& MeshSurface = MutableMesh->Surfaces.Emplace_GetRef();
	MutableMesh->ClothSections.AddDefaulted();
	MeshSurface.BoneMapCount = MutableMesh->BoneMap.Num();
	MeshSurface.bCastShadow = MeshSection.bCastShadow;
	MeshSurface.bRecomputeTangent = MeshSection.bRecomputeTangent;

	MeshSurface.SubMeshes.Emplace(
		UE::Mutable::Private::FSurfaceSubMesh
		{
			0, MeshSection.NumVertices,
			0, (int32)MeshSection.NumTriangles * 3
		});
	
	// Clothing
	if (!bIgnoreSkinning)
	{
		const bool bHasClothData = MeshSection.ClothMappingDataLODs.IsValidIndex(0) && MeshSection.ClothMappingDataLODs[0].Num();
					
		UClothingAssetBase* ClothingAsset = SkeletalMesh->GetClothingAsset(Section.ClothingData.AssetGuid);
		if (bHasClothData && ClothingAsset)
		{
			MutableMesh->ClothSections[0].ClothingAsset = Context.PassthroughObjectFactory.Add(*ClothingAsset, true);
			MutableMesh->ClothSections[0].AssetLODIndex = Section.ClothingData.AssetLodIndex;

			if (MeshSection.ClothMappingDataLODs.Num() > 1)
			{
				UE_LOGF(LogMutable, Error, "\"Mappings to Same LOD\" only supported in Ray Tracing \"Cloth LODBias Mode\" option for [%ls] Skeletal Mesh", *SkeletalMesh->GetName());	
			}
			
			const uint32 NumClothVertices = MeshSection.ClothMappingDataLODs[0].Num();
			
			MutableMesh->ClothSections[0].Data.SetNumUninitialized(NumClothVertices);
			FMemory::Memcpy(MutableMesh->ClothSections[0].Data.GetData(), &MeshSection.ClothMappingDataLODs[0][0], NumClothVertices * sizeof(FMeshToMeshVertData));
		}
	}

	MeshConversionContext->Result = MutableMesh;

	return ConvertSkeletalMeshBuffersTask;
}


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> ConvertSkeletalMeshToMutable(
	FMutableSourceMeshData& Source,
	int32 LODIndex, int32 SectionIndex,
	FMutableGraphGenerationContext& GenerationContext, 
	const UCustomizableObjectNode* CurrentNode,
	bool bForceImmediateConversion)
{
	MUTABLE_CPUPROFILER_SCOPE(ConvertSkeletalMeshToMutable);

	if(Source.Mesh.IsNull())
	{
		return nullptr;
	}

	// Prepare the data that is needed for the core mesh conversion.
	Source.BaseLODIndex = LODIndex;
	Source.BaseSectionIndex = SectionIndex;
	Source.LODOffset = GenerationContext.CurrentLOD - GenerationContext.FromLOD;
	Source.Flags = GenerationContext.MeshGenerationFlags.Last();
	Source.MessageContext = CurrentNode;
	Source.ComponentId = GenerationContext.CurrentSkeletalMeshComponent;

	// Prepare source data for real-time morph targets
	const UCustomizableObjectNodeSkeletalMesh* NodeTypedSkMesh = Cast<UCustomizableObjectNodeSkeletalMesh>(CurrentNode);
	if (NodeTypedSkMesh)
	{
		Source.bUseAllRealTimeMorphs = NodeTypedSkMesh->bUseAllRealTimeMorphs;
		if (!Source.bUseAllRealTimeMorphs)
		{
			Source.UsedRealTimeMorphTargetNames = NodeTypedSkMesh->UsedRealTimeMorphTargetNames;
		}
	}

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh;
	if (!bForceImmediateConversion)
	{
		MutableMesh = GenerateMeshConstant(Source, GenerationContext);
	}
	else
	{
		// At some point this shouldn't happen anymore because all mesh conversion could be moved to the core compilation stage.

		TSharedRef<FMeshConversionContext> MeshConversionContext = MakeShared<FMeshConversionContext>();
		MeshConversionContext->Source = Source;
		//MeshConversionContext->MorphName = FString(); No Morph
		MeshConversionContext->CompilationContext = GenerationContext.CompilationContext;

		MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(UE::Mutable::Private::LoadObject(Source.Mesh)));
		MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(UE::Mutable::Private::LoadObject(Source.TableReferenceSkeletalMesh)));
		MeshConversionContext->ReferencedObjects.Emplace(TStrongObjectPtr<UObject>(UE::Mutable::Private::LoadClass(Source.AnimInstance)));

		UE::Tasks::FTask MeshConversionTask = ConvertSkeletalMeshToMutable(MeshConversionContext);
		MeshConversionTask.Wait();

		MutableMesh = MeshConversionContext->Result;
	}

	return MutableMesh;
}

UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> ConvertStaticMeshToMutable(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex, FMutableGraphGenerationContext& GenerationContext, const UCustomizableObjectNode* CurrentNode)
{
	if (!StaticMesh->GetRenderData() ||
		!StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODIndex) ||
		!StaticMesh->GetRenderData()->LODResources[LODIndex].Sections.IsValidIndex(SectionIndex))
	{
		FString Msg = FString::Printf(TEXT("Degenerated static mesh found for LOD %d Material %d. It will be ignored. "), LODIndex, SectionIndex);
		GenerationContext.Log(FText::FromString(Msg), CurrentNode, EMessageSeverity::Warning);
		return nullptr;
	}

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();

	// Vertices
	int32 VertexStart = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].MinVertexIndex;
	int32 VertexCount = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].MaxVertexIndex - VertexStart + 1;

	MutableMesh->GetVertexBuffers().SetElementCount(VertexCount);
	{
		using namespace UE::Mutable::Private;

		MutableMesh->GetVertexBuffers().SetBufferCount(5);

		// Position buffer
		{
			const FPositionVertexBuffer& VertexBuffer = StaticMesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.PositionVertexBuffer;

			const int32 ElementSize = 12;
			const int32 ChannelCount = 1;
			const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::Position };
			const int32 SemanticIndices[ChannelCount] = { 0 };
			const EMeshBufferFormat Formats[ChannelCount] = { EMeshBufferFormat::Float32 };
			const int32 Components[ChannelCount] = { 3 };
			const int32 Offsets[ChannelCount] = { 0 };

			MutableMesh->GetVertexBuffers().SetBuffer(MUTABLE_VERTEXBUFFER_POSITION, ElementSize, ChannelCount, Semantics, SemanticIndices, Formats, Components, Offsets);
			FMemory::Memcpy(
				MutableMesh->GetVertexBuffers().GetBufferData(MUTABLE_VERTEXBUFFER_POSITION),
				&VertexBuffer.VertexPosition(VertexStart),
				VertexCount * ElementSize);
		}

		// Tangent buffer
		{
			const FStaticMeshVertexBuffer& VertexBuffer = StaticMesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer;

			EMeshBufferSemantic Semantics[2];
			int32 SemanticIndices[2];
			EMeshBufferFormat Formats[2];
			int32 Components[2];
			int32 Offsets[2];

			int32 currentChannel = 0;
			int32 currentOffset = 0;

			Semantics[currentChannel] = EMeshBufferSemantic::Tangent;
			SemanticIndices[currentChannel] = 0;
			Formats[currentChannel] = EMeshBufferFormat::PackedDirS8;
			Components[currentChannel] = 4;
			Offsets[currentChannel] = currentOffset;
			currentOffset += 4;
			++currentChannel;

			Semantics[currentChannel] = EMeshBufferSemantic::Normal;
			SemanticIndices[currentChannel] = 0;
			Formats[currentChannel] = EMeshBufferFormat::PackedDirS8;

			Components[currentChannel] = 4;
			Offsets[currentChannel] = currentOffset;
			currentOffset += 4;
			//++currentChannel;

			MutableMesh->GetVertexBuffers().SetBuffer(MUTABLE_VERTEXBUFFER_TANGENT, currentOffset, 2, Semantics, SemanticIndices, Formats, Components, Offsets);

			const uint8_t* pTangentData = static_cast<const uint8_t*>(VertexBuffer.GetTangentData());
			FMemory::Memcpy(
				MutableMesh->GetVertexBuffers().GetBufferData(MUTABLE_VERTEXBUFFER_TANGENT),
				pTangentData + VertexStart * currentOffset,
				VertexCount * currentOffset);
		}

		// Texture coordinates
		{
			const FStaticMeshVertexBuffer& VertexBuffer = StaticMesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer;

			int32 texChannels = VertexBuffer.GetNumTexCoords();
			int32 ChannelCount = texChannels;

			EMeshBufferSemantic* Semantics = new EMeshBufferSemantic[ChannelCount];
			int32* SemanticIndices = new int[ChannelCount];
			EMeshBufferFormat* Formats = new EMeshBufferFormat[ChannelCount];
			int32* Components = new int[ChannelCount];
			int32* Offsets = new int[ChannelCount];

			int32 currentChannel = 0;
			int32 currentOffset = 0;

			int32 texChannelSize;
			EMeshBufferFormat texChannelFormat;
			if (VertexBuffer.GetUseFullPrecisionUVs())
			{
				texChannelSize = 2 * 4;
				texChannelFormat = EMeshBufferFormat::Float32;
			}
			else
			{
				texChannelSize = 2 * 2;
				texChannelFormat = EMeshBufferFormat::Float16;
			}

			for (int32 c = 0; c < texChannels; ++c)
			{
				Semantics[currentChannel] = EMeshBufferSemantic::TexCoords;
				SemanticIndices[currentChannel] = c;
				Formats[currentChannel] = texChannelFormat;
				Components[currentChannel] = 2;
				Offsets[currentChannel] = currentOffset;
				currentOffset += texChannelSize;
				++currentChannel;
			}

			MutableMesh->GetVertexBuffers().SetBuffer(MUTABLE_VERTEXBUFFER_TEXCOORDS, currentOffset, ChannelCount, Semantics, SemanticIndices, Formats, Components, Offsets);

			const uint8_t* pTextureCoordData = static_cast<const uint8_t*>(VertexBuffer.GetTexCoordData());
			FMemory::Memcpy(
				MutableMesh->GetVertexBuffers().GetBufferData(MUTABLE_VERTEXBUFFER_TEXCOORDS),
				pTextureCoordData + VertexStart * currentOffset,
				VertexCount * currentOffset);

			delete[] Semantics;
			delete[] SemanticIndices;
			delete[] Formats;
			delete[] Components;
			delete[] Offsets;
		}
	}

	// Indices
	{
		int IndexStart = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].FirstIndex;
		int IndexCount = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].NumTriangles * 3;
		MutableMesh->GetIndexBuffers().SetBufferCount(1);
		MutableMesh->GetIndexBuffers().SetElementCount(IndexCount);

		using namespace UE::Mutable::Private;
		const int ElementSize = 2;
		const int ChannelCount = 1;
		const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::VertexIndex };
		const int SemanticIndices[ChannelCount] = { 0 };
		EMeshBufferFormat Formats[ChannelCount] = { EMeshBufferFormat::UInt16 };
		const int Components[ChannelCount] = { 1 };
		const int Offsets[ChannelCount] = { 0 };

		MutableMesh->GetIndexBuffers().SetBuffer(0, ElementSize, ChannelCount, Semantics, SemanticIndices, Formats, Components, Offsets);

		FIndexArrayView Source = StaticMesh->GetRenderData()->LODResources[LODIndex].IndexBuffer.GetArrayView();

		uint16* pDest = reinterpret_cast<uint16*>(MutableMesh->GetIndexBuffers().GetBufferData(0));

		for (int32 i = 0; i < IndexCount; ++i)
		{
			*pDest = Source[IndexStart + i] - VertexStart;
			++pDest;
		}
	}

	return MutableMesh;
}


// Convert a Mesh constant to a mutable format. UniqueTags are the tags that make this Mesh unique that cannot be merged in the cache 
// with the exact same Mesh with other tags
UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> GenerateMutableSkeletalMesh(
	FMutableSourceMeshData& Source,
	int32 LODIndexConnected,
	int32 SectionIndexConnected,
	FMutableGraphGenerationContext & GenerationContext, 
	const UCustomizableObjectNode* CurrentNode,
	const bool bOnlyConnectedLOD)
{
	// Get the mesh generation flags to use
	EMutableMeshConversionFlags CurrentFlags = GenerationContext.MeshGenerationFlags.Last();
	
	FMutableGraphGenerationContext::FGeneratedMeshData::FKey Key = { Source.Mesh, LODIndexConnected, GenerationContext.CurrentLOD, GenerationContext.FromLOD, SectionIndexConnected, CurrentFlags, CurrentNode, Source.GameplayTags, Source.AnimBPSlotName, Source.AnimInstance};
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = GenerationContext.FindGeneratedMesh(Key);
	if (MutableMesh)
	{
		return MutableMesh;
	}

	Source.bOnlyConnectedLOD = bOnlyConnectedLOD;
	MutableMesh = ConvertSkeletalMeshToMutable(Source, LODIndexConnected, SectionIndexConnected, GenerationContext, CurrentNode);

	if (MutableMesh)
	{
		GenerationContext.GeneratedMeshes.Push({ Key, MutableMesh });
	}
	
	return MutableMesh;
}


// Convert a Mesh constant to a mutable format. UniqueTags are the tags that make this Mesh unique that cannot be merged in the cache 
// with the exact same Mesh with other tags
UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> GenerateMutableStaticMesh(
	TSoftObjectPtr<UStreamableRenderAsset> Mesh,
	int32 LODIndex, 
	int32 SectionIndex,
	FMutableGraphGenerationContext& GenerationContext,
	const UCustomizableObjectNode* CurrentNode)
{
	// Get the mesh generation flags to use
	EMutableMeshConversionFlags CurrentFlags = GenerationContext.MeshGenerationFlags.Last();

	FMutableGraphGenerationContext::FGeneratedMeshData::FKey Key = { Mesh, LODIndex, GenerationContext.CurrentLOD, GenerationContext.FromLOD, SectionIndex, CurrentFlags, CurrentNode };
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = GenerationContext.FindGeneratedMesh(Key);
	if (MutableMesh)
	{
		return MutableMesh;
	}

	// When we want to defer the mesh conversion to the core compilation stage, but it is not supported for static meshes yet.
	// Meanwhile:
	UObject* LoadedMesh = UE::Mutable::Private::LoadObject(Mesh);
	if (const UStaticMesh* StaticMesh = Cast<UStaticMesh>(LoadedMesh))
	{
		MutableMesh = ConvertStaticMeshToMutable(StaticMesh, LODIndex, SectionIndex, GenerationContext, CurrentNode);
	}
	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedMesh", "Mesh type not implemented yet."), CurrentNode);
	}

	if (MutableMesh)
	{
		GenerationContext.GeneratedMeshes.Push({ Key, MutableMesh });
	}

	return MutableMesh;
}


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> BuildMorphedMutableMesh(const UEdGraphPin* BaseSourcePin, const FString& MorphTargetName, FMutableGraphGenerationContext& GenerationContext, const bool bOnlyConnectedLOD, const FName& RowName)
{
	check(BaseSourcePin);

	if (!BaseSourcePin)
	{
		GenerationContext.Log(LOCTEXT("NULLBaseSourcePin", "Morph base not set."), nullptr);
		return nullptr;
	}

	int32 BaseLODIndex = -1; // LOD which the pin is connected to
	int32 BaseSectionIndex = -1;

	USkeletalMesh* SkeletalMesh = nullptr;
	UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(BaseSourcePin->GetOwningNode());

	if (Cast<UCustomizableObjectNodeSkeletalMeshParameter>(Node))
	{
		// Do not generate the morphed mesh. Deltas will be included in the Mesh Parameter.
	}

	else if (const ICustomizableObjectNodeMeshInterface* TypedNodeSkeletalMesh = Cast<ICustomizableObjectNodeMeshInterface>(Node))
	{
		TypedNodeSkeletalMesh->GetPinSection(*BaseSourcePin, BaseLODIndex, BaseSectionIndex);
		SkeletalMesh = Cast<USkeletalMesh>(UE::Mutable::Private::LoadObject(TypedNodeSkeletalMesh->GetMesh()));
	}
	
	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		UDataTable* DataTable = GetDataTable(TypedNodeTable, GenerationContext);

		if (DataTable)
		{
			TypedNodeTable->GetPinLODAndSection(BaseSourcePin, BaseLODIndex, BaseSectionIndex);
			SkeletalMesh = Cast<USkeletalMesh>(GenerationContext.LoadObject(TypedNodeTable->GetSkeletalMeshAt(BaseSourcePin, DataTable, RowName)));
		}
	}
	
	else
	{
		unimplemented();	
	}

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MorphedSourceMesh;

	if (SkeletalMesh)
	{
		// Get the base mesh
		FMutableSourceMeshData Source;
		Source.Mesh = SkeletalMesh;
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> BaseSourceMesh = GenerateMutableSkeletalMesh(Source, BaseLODIndex, BaseSectionIndex, GenerationContext, Node, bOnlyConnectedLOD);

		if (BaseSourceMesh)
		{
			check(BaseSourceMesh->IsReference());
		
			// The mesh will be modified and it may come from a cache, so we need to clone it.
			MorphedSourceMesh = BaseSourceMesh->Clone(); 
			MorphedSourceMesh->SetReferencedMorph(MorphTargetName);
		}
	}

	return MorphedSourceMesh;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> GenerateMorphFactor(const UCustomizableObjectNode* Node, const UEdGraphPin& FactorPin, FMutableGraphGenerationContext& GenerationContext)
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FactorNode;

	if (const UEdGraphPin* ConnectedPin = FollowInputPin(FactorPin))
	{
		// Checking if it's linked to a Macro or tunnel node
		const UEdGraphPin* FloatPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, &GenerationContext.MacroNodesStack);
		bool validStaticFactor = true;

		if (FloatPin)
		{
			UEdGraphNode* floatNode = FloatPin->GetOwningNode();

			if (const UCustomizableObjectNodeFloatParameter* floatParameterNode = Cast<UCustomizableObjectNodeFloatParameter>(floatNode))
			{
				if (floatParameterNode->DefaultValue < -1.0f || floatParameterNode->DefaultValue > 1.0f)
				{
					validStaticFactor = false;
					FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the default value of the float parameter node is (%f). Factor will be ignored."), floatParameterNode->DefaultValue);
					GenerationContext.Log(FText::FromString(msg), Node);
				}
				if (floatParameterNode->ParamUIMetadata.MinimumValue < -1.0f)
				{
					validStaticFactor = false;
					FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the minimum UI value for the input float parameter node is (%f). Factor will be ignored."), floatParameterNode->ParamUIMetadata.MinimumValue);
					GenerationContext.Log(FText::FromString(msg), Node);
				}
				if (floatParameterNode->ParamUIMetadata.MaximumValue > 1.0f)
				{
					validStaticFactor = false;
					FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the maximum UI value for the input float parameter node is (%f). Factor will be ignored."), floatParameterNode->ParamUIMetadata.MaximumValue);
					GenerationContext.Log(FText::FromString(msg), Node);
				}
			}

			else if (const UCustomizableObjectNodeFloatConstant* floatConstantNode = Cast<UCustomizableObjectNodeFloatConstant>(floatNode))
			{
				if (floatConstantNode->Value < -1.0f || floatConstantNode->Value > 1.0f)
				{
					validStaticFactor = false;
					FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the value of the float constant node is (%f). Factor will be ignored."), floatConstantNode->Value);
					GenerationContext.Log(FText::FromString(msg), Node);
				}
			}
		}

		// If is a valid factor, continue the Generation
		if (validStaticFactor)
		{
			FactorNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
		}
	}

	return FactorNode;
}


UE::Mutable::Private::NodeMeshPtr GenerateMorphMesh(const UEdGraphPin* Pin,
	TArray<FMorphNodeData> TypedNodeMorphs,
	int32 MorphIndex,
	const UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh>& SourceNode,
	FMutableGraphGenerationContext & GenerationContext,
	const bool bOnlyConnectedLOD,
	const FString& TableColumnName = "")
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateMorphMesh);
	
	check(Pin);
	
	// SkeletalMesh node
	const UEdGraphNode * MeshNode = Pin->GetOwningNode();
	check(MeshNode);
	
	// Current morph node
	check(TypedNodeMorphs.IsValidIndex(MorphIndex));
	UCustomizableObjectNode* MorphNode = TypedNodeMorphs[MorphIndex].OwningNode;
	check(MorphNode);
	
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshMorph> Result = new UE::Mutable::Private::NodeMeshMorph();
	
	// Factor
	//GenerateMorphFactor(MorphNode, , GenerationContext, Result);
	Result->Name = TypedNodeMorphs[MorphIndex].MorphTargetName;
	Result->Factor = TypedNodeMorphs[MorphIndex].FactorNode;

	// Base
	if (MorphIndex == TypedNodeMorphs.Num() - 1)
	{
		Result->Base = SourceNode;
	}
	else
	{
		UE::Mutable::Private::NodeMeshPtr NextMorph = GenerateMorphMesh(Pin, TypedNodeMorphs, MorphIndex + 1, SourceNode, GenerationContext, bOnlyConnectedLOD, TableColumnName); // TODO FutureGMT change to a for. This recursion can be problematic with the production cache
		Result->Base = NextMorph;
	}

	Result->SetMessageContext(MorphNode);
	
	return Result;
}


TArray<FMorphNodeData> GenerateMeshMorphStackDefinition(const UEdGraphPin* Pin, UEdGraphPin* MeshPin, FMutableGraphGenerationContext& GenerationContext)
{
	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());
	TArray<FMorphNodeData> OutMorphData;

	if (const UCustomizableObjectNodeMeshMorphStackDefinition* TypedNodeMeshMorphStackDef = Cast<UCustomizableObjectNodeMeshMorphStackDefinition>(Node))
	{
		if (Node->IsNodeOutDatedAndNeedsRefresh())
		{
			Node->SetRefreshNodeWarning();
		}

		const TArray<FEdGraphPinReference>& MorphPins = TypedNodeMeshMorphStackDef->MorphTargetPinReferences;
		for (const FEdGraphPinReference& MorphPin : MorphPins)
		{
			const UEdGraphPin* MorphPinPointer = MorphPin.Get();
			check(MorphPinPointer);
			
			if (MorphPinPointer->LinkedTo.Num())
			{
				// Generate Factor
				const FName MorphTargetName = TypedNodeMeshMorphStackDef->GetMorphTargetName(*MorphPinPointer);
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FactorNode = GenerateMorphFactor(Node, *MorphPinPointer, GenerationContext);
			
				OutMorphData.Add({ Node, MorphTargetName, FactorNode, MeshPin});
			}
		}
	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		if (const UEdGraphPin* OutputPin = TypedNodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin->PinName))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*OutputPin))
			{
				GenerationContext.MacroNodesStack.Push(TypedNodeMacro);
				OutMorphData = GenerateMeshMorphStackDefinition(FollowPin, MeshPin, GenerationContext);
				GenerationContext.MacroNodesStack.Pop();
			}
			else
			{
				FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNotLinked", "Macro Output node Pin {0} not linked."), FText::FromName(Pin->PinName));
				GenerationContext.Log(Msg, Node);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNameNotFound", "Macro Output node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
			GenerationContext.Log(Msg, Node);
		}
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		check(TypedNodeTunnel->bIsInputNode);
		check(GenerationContext.MacroNodesStack.Num());

		const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = GenerationContext.MacroNodesStack.Pop();
		check(MacroInstanceNode);

		if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin->PinName, EEdGraphPinDirection::EGPD_Input))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
			{
				OutMorphData = GenerateMeshMorphStackDefinition(FollowPin, MeshPin, GenerationContext);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroTunnelError_PinNameNotFound", "Macro Instance Node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
			GenerationContext.Log(Msg, Node);
		}

		// Push the Macro again even if the result is null
		GenerationContext.MacroNodesStack.Push(MacroInstanceNode);
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	return OutMorphData;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> GenerateMutableSourceMesh(const UEdGraphPin* Pin,
	FMutableGraphGenerationContext& GenerationContext,
	const FMutableSourceMeshData& BaseMeshData,
	const bool bLinkedToExtendMaterial,
	const bool bOnlyConnectedLOD)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceMesh);

	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceMesh), *Pin , *Node, GenerationContext, true, bOnlyConnectedLOD);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeMesh*>(Generated->Node.get());
	}
	
	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For example, MacroInstanceNodes
	bool bCacheNode = true;

	//SkeletalMesh Result
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> Result;
	
	if (const UCustomizableObjectNodeSkeletalMesh* TypedNodeSkel = Cast<UCustomizableObjectNodeSkeletalMesh>(Node))
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceMesh_SkeletalMesh);
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshConstant> MeshNode = new UE::Mutable::Private::NodeMeshConstant();
		Result = MeshNode;

		if (!TypedNodeSkel->SkeletalMesh.IsNull())
		{
			int32 BaseLODIndex = -1; // LOD which the pin is connected to
			int32 BaseSectionIndex = -1;
			TypedNodeSkel->GetPinSection(*Pin, BaseLODIndex, BaseSectionIndex);

			FMutableSourceMeshData Source = BaseMeshData;
			Source.Mesh = TypedNodeSkel->SkeletalMesh;
			Source.AnimInstance = TypedNodeSkel->AnimInstance;
			Source.GameplayTags = TypedNodeSkel->AnimationGameplayTags;
			Source.AnimBPSlotName = TypedNodeSkel->AnimBlueprintSlotName;
			
			UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = GenerateMutableSkeletalMesh(Source, BaseLODIndex, BaseSectionIndex, GenerationContext, TypedNodeSkel, bOnlyConnectedLOD);
			MeshNode->Value = MutableMesh;
			
			// Layouts. Always use the base one (connected pin).
			{
				const FLayoutGenerationFlags& LayoutFlags = GenerationContext.LayoutGenerationFlags.Last();

				const TArray<UCustomizableObjectLayout*>& Layouts = TypedNodeSkel->GetLayouts(*Pin);
				const int32 NumLayouts = Layouts.Num();

				MeshNode->Layouts.SetNum(NumLayouts);

				for (int32 LayoutIndex = 0; LayoutIndex < NumLayouts; ++LayoutIndex)
				{
					if (!LayoutFlags.TexturePinModes.IsValidIndex(LayoutIndex) ||
						LayoutFlags.TexturePinModes[LayoutIndex] != EPinMode::Mutable)
					{
						MeshNode->Layouts[LayoutIndex] = CreateDefaultLayout();

						if (Layouts.IsValidIndex(LayoutIndex) && Layouts[LayoutIndex])
						{
							// Keep packing strategy if possible, Overlay can be valid with EPinMode == Passthrough	
							MeshNode->Layouts[LayoutIndex]->Strategy = ConvertLayoutStrategy(Layouts[LayoutIndex]->PackingStrategy);
							MeshNode->Layouts[LayoutIndex]->TexCoordsIndex = Layouts[LayoutIndex]->GetUVChannel();
						}

						// Ignore layout
						continue;
					}

					const UCustomizableObjectLayout* Layout = Layouts.IsValidIndex(LayoutIndex) ? Layouts[LayoutIndex] : nullptr;
					if (ensure(Layout))
					{
						UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode = CreateMutableLayoutNode(Layout, bLinkedToExtendMaterial); // TODO PERE: Figure out
						LayoutNode->SetMessageContext(Node);

						MeshNode->Layouts[LayoutIndex] = LayoutNode;
					}
				}	
			}

			const bool bEnableRealTimeMorphTargets = GenerationContext.CompilationContext->Options.bRealTimeMorphTargetsEnabled;
			MeshNode->RealTimeMorphNames = GetUsedRealTimeMorphsNames(Source, bEnableRealTimeMorphTargets, GenerationContext.CompilationContext->RealTimeMorphTargetsOverrides);

			MeshNode->BonePosePriority = GenerationContext.BonePosePriorityStack.Last();
			MeshNode->SocketPriority = GenerationContext.SocketPriorityStack.Last();

			const FString MeshName = TypedNodeSkel->SkeletalMesh.GetLongPackageName().ToLower();
			MeshNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*MeshName), MeshName.Len() * sizeof(FString::ElementType));
			MeshNode->SourceDataDescriptor.OptionalMaxLODSize = 0;
		}
		else
		{
			GenerationContext.Log(LOCTEXT("NodeSkeletalMeshEmpty", "Node SkeletalMesh without a mesh assigned."), Node);
		}
	}

	else if (const UCustomizableObjectNodeStaticMesh* TypedNodeStatic = Cast<UCustomizableObjectNodeStaticMesh>(Node))
	{
		GenerationContext.LoadObject(TypedNodeStatic->StaticMesh);
		if (TypedNodeStatic->StaticMesh == nullptr)
		{
			FString Msg = FString::Printf(TEXT("The UCustomizableObjectNodeStaticMesh node %s has no static mesh assigned"), *Node->GetName());
			GenerationContext.Log(FText::FromString(Msg), Node, EMessageSeverity::Warning);
			return {};
		}

		if (TypedNodeStatic->StaticMesh->GetNumLODs() == 0)
		{
			FString Msg = FString::Printf(TEXT("The UCustomizableObjectNodeStaticMesh node %s has a static mesh assigned with no RenderData"), *Node->GetName());
			GenerationContext.Log(FText::FromString(Msg), Node, EMessageSeverity::Warning);
			return {};
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshConstant> MeshNode = new UE::Mutable::Private::NodeMeshConstant();
		Result = MeshNode;

		int32 LODIndex = 0; // TODO MTBL-1474
		int32 SectionIndex = 0;

		// Find out what material do we need
		[&LODIndex, &SectionIndex, &TypedNodeStatic, &Pin]()
		{
			for (; LODIndex < TypedNodeStatic->LODs.Num(); ++LODIndex)
			{
				for (SectionIndex = 0; SectionIndex < TypedNodeStatic->LODs[LODIndex].Materials.Num(); ++SectionIndex)
				{
					if (TypedNodeStatic->LODs[LODIndex].Materials[SectionIndex].MeshPinRef.Get() == Pin)
					{
						return;
					}
				}
			}

			LODIndex = -1;
			SectionIndex = -1;
		}();
			
		check(TypedNodeStatic->LODs.IsValidIndex(LODIndex) && SectionIndex < TypedNodeStatic->LODs[LODIndex].Materials.Num());

		constexpr bool bIsPassthrough = false;
		UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = GenerateMutableStaticMesh(TypedNodeStatic->StaticMesh,  LODIndex, SectionIndex, GenerationContext, TypedNodeStatic);
		if (MutableMesh)
		{
			MeshNode->Value = MutableMesh;

			// Layouts
			MeshNode->Layouts.SetNum(1);

			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode;

			const TArray<UCustomizableObjectLayout*>& Layouts = TypedNodeStatic->GetLayouts(*Pin);
				
			if (const UCustomizableObjectLayout* Layout = Layouts.IsValidIndex(0) ? Layouts[0] : nullptr)
			{
				LayoutNode = CreateMutableLayoutNode(Layout, false);
			}
			else
			{
				LayoutNode = CreateDefaultLayout();
			}

			MeshNode->Layouts[0] = LayoutNode;
			LayoutNode->SetMessageContext(Node);  // We need it here because we create multiple nodes.
			
			const FString MeshName = TypedNodeStatic->StaticMesh.GetLongPackageName().ToLower();
			MeshNode->SourceDataDescriptor.SourceId = CityHash32(reinterpret_cast<const char*>(*MeshName), MeshName.Len() * sizeof(FString::ElementType));
			MeshNode->SourceDataDescriptor.OptionalMaxLODSize = 0;
		}
		else
		{
			Result = nullptr;
		}
	}

	else if (UCustomizableObjectNodeMeshMorph* TypedNodeMorph = Cast<UCustomizableObjectNodeMeshMorph>(Node))
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceMesh_MeshMorph);

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMorph->MeshPin()))
		{
			// Base Mesh
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> BaseMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData,
				false, bOnlyConnectedLOD);
			
			if (BaseMesh)
			{
				// Factor
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> FactorNode = GenerateMorphFactor(Node, *TypedNodeMorph->FactorPin(), GenerationContext);
				
				if (FactorNode)
				{
					const FName MorphTargetName = TypedNodeMorph->GetMorphTargetName(GenerationContext);
					FMorphNodeData NewMorphData = { TypedNodeMorph, MorphTargetName, FactorNode, TypedNodeMorph->MeshPin() };
					
					// Apply the morphs to the mesh
					Result = GenerateMorphMesh(ConnectedPin, {NewMorphData}, 0, BaseMesh, GenerationContext, bOnlyConnectedLOD);
				}
			}
		}
		else
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshMorph> MeshNode = new UE::Mutable::Private::NodeMeshMorph();
			Result = MeshNode;
		}
	}

	else if (const UCustomizableObjectNodeMeshMorphStackApplication* TypedNodeMeshMorphStackApp = Cast< UCustomizableObjectNodeMeshMorphStackApplication >(Node))
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceMesh_MeshMorphStack);

		Result = nullptr;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMeshMorphStackApp->GetStackPin()))
		{
			if (const UEdGraphPin* MeshConnectedPin = FollowInputPin(*TypedNodeMeshMorphStackApp->GetMeshPin()))
			{
				UE::Mutable::Private::NodeMeshPtr BaseMesh = GenerateMutableSourceMesh(MeshConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
				
				TArray<FMorphNodeData> MorphDataArray = GenerateMeshMorphStackDefinition(ConnectedPin, TypedNodeMeshMorphStackApp->GetMeshPin(), GenerationContext);
				
				if (MorphDataArray.Num())
				{
					Result = GenerateMorphMesh(ConnectedPin, MorphDataArray, 0, BaseMesh, GenerationContext, bOnlyConnectedLOD);
				}
				else
				{
					GenerationContext.Log(LOCTEXT("MorphStackMorphsNotSet", "No morphs could be found to be used."), Node, EMessageSeverity::Error);
				}
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MorphStackBaseMeshConnectionFailed", "Base Mesh connection not found."), Node, EMessageSeverity::Error);
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MorphStackStackConnectionFailed", "Stack definition connection not found."), Node, EMessageSeverity::Error);
		}
	}

	else if (const UCONodeSwitch* TypedNodeMeshSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Mesh))
	{
		// Using a lambda so control flow is easier to manage.
		Result = [&]()
		{
			const UEdGraphPin* SwitchParameter = TypedNodeMeshSwitch->SwitchParameterPinReference.Get();

			// Check Switch Parameter arity preconditions.
			if (const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

				// Switch Param not generated
				if (!SwitchParam)
				{
					// Warn about a failure.
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
						GenerationContext.Log(Message, Node);

					return Result;
				}

				if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
				{
					const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
					GenerationContext.Log(Message, Node);

					return Result;
				}

				{
					const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());

					if (!DoOptionsMatchEnum(*TypedNodeMeshSwitch, *EnumParameter))
					{
						const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
						GenerationContext.Log(Message, Node);
						Node->SetRefreshNodeWarning();
					}
				}

				const int32 NumSwitchOptions = TypedNodeMeshSwitch->SwitchPins.Num();
				
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshSwitch> SwitchNode = new UE::Mutable::Private::NodeMeshSwitch;
				SwitchNode->Parameter = SwitchParam;
				SwitchNode->Options.SetNum(NumSwitchOptions);

				for (int SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
				{
					if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMeshSwitch->SwitchPins[SelectorIndex].Get()))
					{
						Result = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
						SwitchNode->Options[SelectorIndex] = Result;
					}
				}

				Result = SwitchNode;
				return Result;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), Node);
				return Result;
			}
		}(); // invoke lambda.
	}

	else if (const UCustomizableObjectNodeMeshVariation* TypedNodeMeshVar = Cast<const UCustomizableObjectNodeMeshVariation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshVariation> MeshNode = new UE::Mutable::Private::NodeMeshVariation();
		Result = MeshNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMeshVar->DefaultPin()))
		{
			UE::Mutable::Private::NodeMeshPtr ChildNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
			if (ChildNode)
			{
				MeshNode->DefaultMesh = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MeshFailed", "Mesh generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeMeshVar->GetNumVariations();

		MeshNode->Variations.SetNum(NumVariations);

		for (int VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			const UEdGraphPin* VariationPin = TypedNodeMeshVar->VariationPin(VariationIndex);
			if (!VariationPin) continue;

			FString VariationTag = TypedNodeMeshVar->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);
			MeshNode->Variations[VariationIndex].Tag = StringCast<ANSICHAR>(*VariationTag).Get();

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
			{
				UE::Mutable::Private::NodeMeshPtr ChildNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
				MeshNode->Variations[VariationIndex].Mesh = ChildNode;
			}
		}
	}

	else if (const UCustomizableObjectNodeMeshReshape* TypedNodeReshape = Cast<const UCustomizableObjectNodeMeshReshape>(Node))
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceMesh_Reshape);
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshReshape> MeshNode = new UE::Mutable::Private::NodeMeshReshape();
		Result = MeshNode;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeReshape->BaseMeshPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> ChildNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
			if (ChildNode)
			{
				MeshNode->BaseMesh = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MeshFailed", "Mesh generation failed."), Node);
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MeshReshapeMissingDef", "Mesh reshape node requires a default value."), Node);
		}
	
		{

			MeshNode->bReshapeVertices = TypedNodeReshape->bReshapeVertices;
			MeshNode->bRecomputeNormals = TypedNodeReshape->bRecomputeNormals;
			MeshNode->bApplyLaplacian = TypedNodeReshape->bApplyLaplacianSmoothing;
			MeshNode->bReshapeSkeleton = TypedNodeReshape->bReshapePose;
			MeshNode->bReshapePhysicsVolumes = TypedNodeReshape->bReshapePhysics;

			EMeshReshapeVertexColorChannelUsage ChannelUsages[4] =
			{
				TypedNodeReshape->VertexColorUsage.R,
				TypedNodeReshape->VertexColorUsage.G,
				TypedNodeReshape->VertexColorUsage.B,
				TypedNodeReshape->VertexColorUsage.A
			};

			{
				int32 MaskWeightChannelNum = 0;
				for (int32 I = 0; I < 4; ++I)
				{
					if (ChannelUsages[I] == EMeshReshapeVertexColorChannelUsage::MaskWeight)
					{
						++MaskWeightChannelNum;
					}
				}

				if (MaskWeightChannelNum > 1)
				{
					for (int32 I = 0; I < 4; ++I)
					{
						if (ChannelUsages[I] == EMeshReshapeVertexColorChannelUsage::MaskWeight)
						{
							ChannelUsages[I] = EMeshReshapeVertexColorChannelUsage::None;
						}
					}

					GenerationContext.Log(
						LOCTEXT("MeshReshapeColorUsageMask", 
								"Only one color channel with mask weight usage is allowed, multiple found. Reshape masking disabled."),
						Node);
				}
			}

			auto ConvertColorUsage = [](EMeshReshapeVertexColorChannelUsage Usage) -> UE::Mutable::Private::EVertexColorUsage
			{
				switch (Usage)
				{
				case EMeshReshapeVertexColorChannelUsage::None:			  return UE::Mutable::Private::EVertexColorUsage::None;
				case EMeshReshapeVertexColorChannelUsage::RigidClusterId: return UE::Mutable::Private::EVertexColorUsage::ReshapeClusterId;
				case EMeshReshapeVertexColorChannelUsage::MaskWeight:     return UE::Mutable::Private::EVertexColorUsage::ReshapeMaskWeight;
				default: check(false); return UE::Mutable::Private::EVertexColorUsage::None;
				};
			};

			MeshNode->ColorRChannelUsage = ConvertColorUsage(ChannelUsages[0]);
			MeshNode->ColorGChannelUsage = ConvertColorUsage(ChannelUsages[1]);
			MeshNode->ColorBChannelUsage = ConvertColorUsage(ChannelUsages[2]);
			MeshNode->ColorAChannelUsage = ConvertColorUsage(ChannelUsages[3]);

			MeshNode->bReshapeSkeletonInvertSelection = TypedNodeReshape->SelectionMethod == EBoneDeformSelectionMethod::ALL_BUT_SELECTED;
			MeshNode->bReshapePhysicsVolumesInvertSelection = TypedNodeReshape->PhysicsSelectionMethod == EBoneDeformSelectionMethod::ALL_BUT_SELECTED;

			MeshNode->BonesToDeform = TypedNodeReshape->BonesToDeform_V2;
			MeshNode->PhysicsToDeform = TypedNodeReshape->PhysicsBodiesToDeform_V2;
		}
		// We don't need all the data for the shape meshes
		const EMutableMeshConversionFlags ShapeFlags = 
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;

		GenerationContext.MeshGenerationFlags.Push( ShapeFlags );
			
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeReshape->BaseShapePin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> ChildNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, true);
	
			if (ChildNode)
			{
				MeshNode->BaseShape = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MeshFailed", "Mesh generation failed."), Node);
			}
		}

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeReshape->TargetShapePin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> ChildNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, true);
			
			if (ChildNode)
			{
				MeshNode->TargetShape = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MeshFailed", "Mesh generation failed."), Node);
			}
		}

		GenerationContext.MeshGenerationFlags.Pop();
	}
	
	else if (const UCONodeClipMeshWithMesh* ClipMeshWithMeshNode = Cast<UCONodeClipMeshWithMesh>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshClipWithMesh> ClipNode = new UE::Mutable::Private::NodeMeshClipWithMesh();
		ClipNode->FaceCullStrategy = ClipMeshWithMeshNode->FaceCullStrategy;
		
		// Base Mesh
		const UEdGraphPin* BaseMeshPin = ClipMeshWithMeshNode->BaseMeshPin.Get();
		check(BaseMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BaseMeshPin))
		{
			UE::Mutable::Private::NodeMeshPtr BaseMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);
			ClipNode->Source = BaseMesh;
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("ClipMeshWithMesh_Base_Mesh_missing", "The Clip Mesh with Mesh node requires an input base mesh.");
			GenerationContext.Log(ErrorMsg, ClipMeshWithMeshNode, EMessageSeverity::Error);
		}
		
		// Clip Mesh
		const UEdGraphPin* ClipMeshPin = ClipMeshWithMeshNode->ClipMeshPin.Get();
		check(ClipMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*ClipMeshPin))
		{
			constexpr EMutableMeshConversionFlags ClipMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
			
			GenerationContext.MeshGenerationFlags.Push(ClipMeshFlags);
			
			UE::Mutable::Private::NodeMeshPtr ClipMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);

			if (const FMatrix Matrix = ClipMeshWithMeshNode->Transform.ToMatrixWithScale(); Matrix != FMatrix::Identity)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshTransform> TransformMesh = new UE::Mutable::Private::NodeMeshTransform();
				TransformMesh->Source = ClipMesh;

				TransformMesh->Transform = FMatrix44f(Matrix);
				ClipMesh = TransformMesh;
			}

			ClipNode->ClipMesh = ClipMesh;
			
			GenerationContext.MeshGenerationFlags.Pop();
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("ClipMeshWithMesh_Clipping_Mesh_missing", "The Clip Mesh with Mesh node requires an input clipping mesh.");
			GenerationContext.Log(ErrorMsg, ClipMeshWithMeshNode, EMessageSeverity::Error);
		}
		
		Result = ClipNode;
	}

	else if (const UCONodeRemoveMesh* RemoveMeshNode = Cast<UCONodeRemoveMesh>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshRemoveMesh> RemoveNode = new UE::Mutable::Private::NodeMeshRemoveMesh();
		RemoveNode->FaceCullStrategy = RemoveMeshNode->FaceCullStrategy;

		// Base Mesh
		UEdGraphPin* BaseMeshPin = RemoveMeshNode->BaseMeshPin.Get();
		check(BaseMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BaseMeshPin))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> BaseMeshCoreNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);
			RemoveNode->Source = BaseMeshCoreNode;
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("RemoveMesh_Remove_Mesh_missing", "The Remove Mesh node requires an input base mesh.");
			GenerationContext.Log(ErrorMsg, RemoveMeshNode, EMessageSeverity::Error);
		}
		
		// Remove Mesh
		UEdGraphPin* RemoveMeshPin = RemoveMeshNode->RemoveMeshPin.Get();
		check(RemoveMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*RemoveMeshPin))
		{
			constexpr EMutableMeshConversionFlags RemoveMeshFlags = 
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
			
			GenerationContext.MeshGenerationFlags.Push(RemoveMeshFlags);
			
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> RemoveMeshCoreNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);
			RemoveNode->RemoveMesh = RemoveMeshCoreNode;
			
			GenerationContext.MeshGenerationFlags.Pop();
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("RemoveMesh_Remove_RemoveMesh_missing", "The Remove Mesh node requires an input remove mesh.");
			GenerationContext.Log(ErrorMsg, RemoveMeshNode, EMessageSeverity::Error);
		}
		
		Result = RemoveNode;
	}
	
	else if (const UCONodeTransformWithBone* TransformWithBoneNode = Cast<UCONodeTransformWithBone>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshTransformWithBone> TransformNode = new UE::Mutable::Private::NodeMeshTransformWithBone();
		Result = TransformNode;

		TransformNode->BoneName = *TransformWithBoneNode->BoneName;
		TransformNode->ThresholdFactor = TransformWithBoneNode->ThresholdFactor;

		const UEdGraphPin* BaseMeshPin = TransformWithBoneNode->BaseMeshPin.Get();
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BaseMeshPin))
		{
			TransformNode->Source = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("TransformWithBone_Base_Mesh_missing", "The Transform With Bone node requires an input base mesh.");
			GenerationContext.Log(ErrorMsg, TransformWithBoneNode, EMessageSeverity::Error);
		}
		
		const UEdGraphPin* TransformPin = TransformWithBoneNode->TransformPin.Get();
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TransformPin))
		{
			TransformNode->MatrixNode = GenerateMutableSourceTransform(ConnectedPin, GenerationContext);
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("TransformWithBone_Transform_missing", "The Transform With Bone node requires an input transform.");
			GenerationContext.Log(ErrorMsg, TransformWithBoneNode, EMessageSeverity::Error);
		}
	}
	
	
	else if (const UCONodeTransformInMesh* TransformInMeshNode = Cast<UCONodeTransformInMesh>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshTransformInMesh> TransformNode = new UE::Mutable::Private::NodeMeshTransformInMesh();
		Result = TransformNode;

		// Input mesh
		const UEdGraphPin* BaseMeshPin = TransformInMeshNode->BaseMeshPin.Get();
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BaseMeshPin))
		{
			TransformNode->SourceMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("TransformInMesh_Base_Mesh_missing", "The Transform In mesh node requires an input base mesh.");
			GenerationContext.Log(ErrorMsg, TransformInMeshNode, EMessageSeverity::Error);
		}
		
		// transform to apply to the source mesh
		const UEdGraphPin* TransformPin = TransformInMeshNode->TransformMeshPin.Get();
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TransformPin))
		{
			TransformNode->MatrixNode = GenerateMutableSourceTransform(ConnectedPin, GenerationContext);
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("TransformInMesh_Transform_missing", "The Transform In mesh node requires an input transform.");
			GenerationContext.Log(ErrorMsg, TransformInMeshNode, EMessageSeverity::Error);
		}
		
		// Bounding Mesh
		const UEdGraphPin* BoundingMeshPin = TransformInMeshNode->BoundingMeshPin.Get();
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BoundingMeshPin))
		{
			constexpr EMutableMeshConversionFlags BoundingMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
				
			GenerationContext.MeshGenerationFlags.Push(BoundingMeshFlags);
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> BoundingMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);
				
				// Apply the transform defined within the node to the bounding mesh
				if (const FMatrix Matrix = TransformInMeshNode->BoundingMeshTransform.ToMatrixWithScale(); Matrix != FMatrix::Identity)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshTransform> TransformMesh = new UE::Mutable::Private::NodeMeshTransform();
					TransformMesh->Source = BoundingMesh;

					TransformMesh->Transform = FMatrix44f(Matrix);
					BoundingMesh = TransformMesh;
				}
				
				TransformNode->BoundingMesh = BoundingMesh;
			}
			GenerationContext.MeshGenerationFlags.Pop();
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("TransformInMesh_Bounding_Mesh_missing", "The Transform In mesh node requires an input Bounding Mesh.");
			GenerationContext.Log(ErrorMsg, TransformInMeshNode, EMessageSeverity::Error);
		}
	}
	
	else if (const UCustomizableObjectNodeAnimationPose* TypedNode = Cast<UCustomizableObjectNodeAnimationPose>(Node))
	{
		if (const UEdGraphPin* InputMeshPin = FollowInputPin(*TypedNode->GetInputMeshPin()))
		{
			UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> InputMeshNode = GenerateMutableSourceMesh(InputMeshPin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);

			if (TypedNode->ReferenceSkeletalMesh)
			{
				if (TypedNode->PoseAsset)
				{
					TArray<FName> ArrayBoneName;
					TArray<FTransform> ArrayTransform;
					UCustomizableObjectNodeAnimationPose::StaticRetrievePoseInformation(TypedNode->PoseAsset, TypedNode->ReferenceSkeletalMesh, ArrayBoneName, ArrayTransform);
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshApplyPose> NodeMeshApplyPose = CreateNodeMeshApplyPose(GenerationContext, InputMeshNode, ArrayBoneName, ArrayTransform);

					if (NodeMeshApplyPose)
					{
						Result = NodeMeshApplyPose;
					}
					else
					{
						FString msg = FString::Printf(TEXT("Couldn't get bone transform information from a Pose Asset."));
						GenerationContext.Log(FText::FromString(msg), Node);

						Result = nullptr;
					}
				}
				else if (const UEdGraphPin* TablePosePin = FollowInputPin(*TypedNode->GetTablePosePin()))
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshApplyPose> NodeMeshApplyPose = new UE::Mutable::Private::NodeMeshApplyPose();
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> MeshTableNode = GenerateMutableSourceMesh(TablePosePin, GenerationContext, BaseMeshData, false, bOnlyConnectedLOD);

					NodeMeshApplyPose->Base = InputMeshNode;
					NodeMeshApplyPose->Pose = MeshTableNode;

					Result = NodeMeshApplyPose;
				}
				else
				{
					if (!TypedNode->PoseAsset) // Check if the slot has a selected pose. Could be left empty by the user
					{
						const FString Message = FString::Printf(TEXT("Found pose mesh node without a pose asset assigned."));
						GenerationContext.Log(FText::FromString(Message), TypedNode);
					}

					Result = InputMeshNode;
				}
			}
			
		}
	}

	else if (const UCustomizableObjectNodeTable* TypedNodeTable = Cast<UCustomizableObjectNodeTable>(Node))
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceMesh_NodeTable);

		TObjectPtr<UDataTable> Table = UE::Mutable::Private::LoadObject(TypedNodeTable->Table);
		TObjectPtr<UScriptStruct> Structure = UE::Mutable::Private::LoadObject(TypedNodeTable->Structure);
		const FString TableName = Table ? GetNameSafe(Table).ToLower() : GetNameSafe(Structure).ToLower();
		const uint32 TableId = CityHash32(reinterpret_cast<const char*>(*TableName), TableName.Len() * sizeof(FString::ElementType));

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshConstant> EmptyNode = new UE::Mutable::Private::NodeMeshConstant();
		Result = EmptyNode;
		bool bSuccess = true;

		UDataTable* DataTable = GetDataTable(TypedNodeTable, GenerationContext);

		if (DataTable)
		{
			const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

			// Getting the real name of the data table column
			FString ColumnName = TypedNodeTable->GetPinColumnName(Pin);
			FProperty* Property = TypedNodeTable->FindPinProperty(*Pin);

			if (!Property)
			{
				FString Msg = FString::Printf(TEXT("Couldn't find the column [%s] in the data table's struct."), *ColumnName);
				GenerationContext.Log(FText::FromString(Msg), Node);

				bSuccess = false;
			}

			USkeletalMesh* DefaultSkeletalMesh = TypedNodeTable->GetColumnDefaultAssetByType<USkeletalMesh>(Pin);
			UStaticMesh* DefaultStaticMesh = TypedNodeTable->GetColumnDefaultAssetByType<UStaticMesh>(Pin);
			UPoseAsset* DefaultPoseAsset = TypedNodeTable->GetColumnDefaultAssetByType<UPoseAsset>(Pin);

			if (bSuccess && !DefaultSkeletalMesh && !DefaultStaticMesh && !DefaultPoseAsset)
			{
				FString Msg = FString::Printf(TEXT("Couldn't find a default value in the data table's struct for the column [%s]."), *ColumnName);
				GenerationContext.Log(FText::FromString(Msg), Node);

				bSuccess = false;
			}

			if (bSuccess)
			{
				// Generating a new data table if not exists
				UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> GeneratedTable = nullptr;
				GeneratedTable = GenerateMutableSourceTable(DataTable, TypedNodeTable, GenerationContext);

				if (GeneratedTable)
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshTable> MeshTableNode = new UE::Mutable::Private::NodeMeshTable();
					
					int32 BaseLODIndex = INDEX_NONE; // LOD which the pin is connected to
					int32 BaseSectionIndex = INDEX_NONE;

					// Getting the mutable table mesh column name
					FString MutableColumnName = ColumnName;

					if (Pin->PinType.PinCategory == Schema->PC_Mesh)
					{
						// LOD and sections are relevant for Skeletal and Static meshes but not for Pose Assets
						TypedNodeTable->GetPinLODAndSection(Pin, BaseLODIndex, BaseSectionIndex);

						if (DefaultSkeletalMesh)
						{
							int32 LODIndex = 0;
							int32 SectionIndex = 0;

							FMutableSourceMeshData Source;
							Source.BaseLODIndex = BaseLODIndex;
							Source.BaseSectionIndex = BaseSectionIndex;
							Source.LODOffset = GenerationContext.CurrentLOD - GenerationContext.FromLOD;
							Source.MessageContext = Node;
							Source.bOnlyConnectedLOD = bOnlyConnectedLOD;
							GetLODAndSection(*GenerationContext.CompilationContext, Source, *DefaultSkeletalMesh, LODIndex, SectionIndex);

							MutableColumnName = TypedNodeTable->GenerateSkeletalMeshMutableColumName(ColumnName, LODIndex, SectionIndex);	
						}
						else
						{
							MutableColumnName = TypedNodeTable->GenerateStaticMeshMutableColumName(ColumnName, BaseSectionIndex);
						}
					}
					
					// Generating a new FMesh column if not exists
					if (GeneratedTable->FindColumn(MutableColumnName) == INDEX_NONE)
					{
						FMutableSourceMeshData Source = BaseMeshData;
						Source.BaseLODIndex = BaseLODIndex;
						Source.BaseSectionIndex = BaseSectionIndex;
						Source.LODOffset = GenerationContext.CurrentLOD - GenerationContext.FromLOD;
						Source.bOnlyConnectedLOD = bOnlyConnectedLOD; // True?
						bSuccess = GenerateTableColumn(TypedNodeTable, Pin, GeneratedTable, ColumnName, Property, Source, GenerationContext);
							
						if (!bSuccess)
						{
							FString Msg = FString::Printf(TEXT("Failed to generate the mutable table column [%s]"), *MutableColumnName);
							GenerationContext.Log(FText::FromString(Msg), Node);
						}

					}

					if (bSuccess)
					{
						Result = MeshTableNode;

						MeshTableNode->Table = GeneratedTable;
						MeshTableNode->ColumnName = MutableColumnName;
						MeshTableNode->ParameterName = TypedNodeTable->ParameterName;
						MeshTableNode->bNoneOption = TypedNodeTable->bAddNoneOption;
						MeshTableNode->DefaultRowName = TypedNodeTable->DefaultRowName.ToString();
						MeshTableNode->SourceDataDescriptor.SourceId = TableId;
						MeshTableNode->SourceDataDescriptor.OptionalMaxLODSize = 0;

						// Pose Assets do not need this part of the code
						if (Pin->PinType.PinCategory == Schema->PC_Mesh)
						{
							const FLayoutGenerationFlags& LayoutFlags = GenerationContext.LayoutGenerationFlags.Last();

							// Layouts. Always use the base one (connected pin).
							{
								TArray<UCustomizableObjectLayout*> Layouts = TypedNodeTable->GetLayouts(Pin);
								const int32 NumLayouts = Layouts.Num();

								// Generating node Layouts
								MeshTableNode->Layouts.SetNum(NumLayouts);
								for (int32 LayoutIndex = 0; LayoutIndex < NumLayouts; ++LayoutIndex)
								{
									if (!LayoutFlags.TexturePinModes.IsValidIndex(LayoutIndex) ||
										LayoutFlags.TexturePinModes[LayoutIndex] != EPinMode::Mutable)
									{
										MeshTableNode->Layouts[LayoutIndex] = CreateDefaultLayout();

										// Keep packing strategy if possible, Overlay can be valid with EPinMode == Passthrough	
										if (Layouts[LayoutIndex])
										{
											MeshTableNode->Layouts[LayoutIndex]->Strategy = ConvertLayoutStrategy(Layouts[LayoutIndex]->PackingStrategy);
											MeshTableNode->Layouts[LayoutIndex]->TexCoordsIndex = Layouts[LayoutIndex]->GetUVChannel();
										}

										// Ignore layouts
										continue;
									}

									// In tables, mimic the legacy behaviour and ignore all layout warnings beyond LOD 0.
									bool bIgnoreLayoutWarnings = true;
									UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode = CreateMutableLayoutNode(Layouts[LayoutIndex], bIgnoreLayoutWarnings);

									MeshTableNode->Layouts[LayoutIndex] = LayoutNode;
								}
							}

							// Set real-time morph names.
							{
								const bool bEnableRealTimeMorphTargets = GenerationContext.CompilationContext->Options.bRealTimeMorphTargetsEnabled;
								FMutableSourceMeshData Source;
								Source.TableReferenceSkeletalMesh = DefaultSkeletalMesh;
								MeshTableNode->RealTimeMorphNames = GetUsedRealTimeMorphsNames(Source, bEnableRealTimeMorphTargets, {});
							}
						}
					}
				}
				else
				{
					FString Msg = FString::Printf(TEXT("Couldn't generate a mutable table."));
					GenerationContext.Log(FText::FromString(Msg), Node);
				}
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("ImageTableError", "Couldn't find the data table of the node."), Node);
		}
	}

	else if (const UCustomizableObjectNodeSkeletalMeshParameter* TypedNodeParam = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSkeletalMeshObjectParameter> ParameterNode = UE::Mutable::Private::GenerateMutableSourceSkeletalMeshObjectParameter(*TypedNodeParam, GenerationContext);
		
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshSkeletalMeshObjectBreak> BreakNode = new UE::Mutable::Private::NodeMeshSkeletalMeshObjectBreak();
		BreakNode->SkeletalMeshObject = ParameterNode;
		
		int32 BaseLODIndex = 0;
		int32 BaseSectionIndex = 0;

		{
			TypedNodeParam->GetPinSection(*Pin, BaseLODIndex, BaseSectionIndex);
			BreakNode->BaseLODIndex = BaseLODIndex;
			BreakNode->BaseSectionIndex = BaseSectionIndex;
			BreakNode->LODBias = GenerationContext.CurrentLOD - GenerationContext.FromLOD;
		}
		
		BreakNode->ConversionFlags = (uint8)GenerationContext.MeshGenerationFlags.Last();
		BreakNode->BonePosePriority = GenerationContext.BonePosePriorityStack.Last();
		BreakNode->SocketPriority = GenerationContext.SocketPriorityStack.Last();


		TSoftObjectPtr<UStreamableRenderAsset> NodeMesh = TypedNodeParam->GetMesh();
		FMutableSourceMeshData Source;
		Source.Mesh = NodeMesh;
		// Layouts
		{
			const FLayoutGenerationFlags& LayoutFlags = GenerationContext.LayoutGenerationFlags.Last();

			const TArray<UCustomizableObjectLayout*>& Layouts = TypedNodeParam->GetLayouts(*Pin);
			const int32 NumLayouts = Layouts.Num();

			BreakNode->Layouts.SetNum(NumLayouts);
			for (int32 LayoutIndex = 0; LayoutIndex < NumLayouts; ++LayoutIndex)
			{
				if (!LayoutFlags.TexturePinModes.IsValidIndex(LayoutIndex) ||
					LayoutFlags.TexturePinModes[LayoutIndex] != EPinMode::Mutable)
				{
					BreakNode->Layouts[LayoutIndex] = CreateDefaultLayout();

					if (Layouts.IsValidIndex(LayoutIndex) && Layouts[LayoutIndex])
					{
						// Keep packing strategy if possible, Overlay can be valid with EPinMode == Passthrough	
						BreakNode->Layouts[LayoutIndex]->Strategy = ConvertLayoutStrategy(Layouts[LayoutIndex]->PackingStrategy);
						BreakNode->Layouts[LayoutIndex]->TexCoordsIndex = Layouts[LayoutIndex]->GetUVChannel();
					}

					// Ignore layout
					continue;
				}

				const UCustomizableObjectLayout* Layout = Layouts.IsValidIndex(LayoutIndex) ? Layouts[LayoutIndex] : nullptr;
				if (ensure(Layout))
				{
					UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeLayout> LayoutNode = CreateMutableLayoutNode(Layout, bLinkedToExtendMaterial);
					LayoutNode->SetMessageContext(Node);

					BreakNode->Layouts[LayoutIndex] = LayoutNode;
				}
			}

			if (!Source.Mesh.IsNull())
			{
				const EMutableMeshConversionFlags ModifiersMeshFlags =
					EMutableMeshConversionFlags::IgnoreSkinning |
					EMutableMeshConversionFlags::IgnorePhysics |
					EMutableMeshConversionFlags::IgnoreMorphs |
					EMutableMeshConversionFlags::IgnoreAUD;
				
				GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

				// Get the base mesh
				BreakNode->ReferenceMesh = ConvertSkeletalMeshToMutable(Source, BaseLODIndex, BaseSectionIndex, GenerationContext, Node);

				GenerationContext.MeshGenerationFlags.Pop();

			}
		}

		const bool bEnableRealTimeMorphTargets = GenerationContext.CompilationContext->Options.bRealTimeMorphTargetsEnabled;
		Source.bUseAllRealTimeMorphs = true; // TODO: Remove, this should be selected similarly to the constant case, meanwhile use all morphs from the Reference Mesh.
		BreakNode->RealTimeMorphNames = GetUsedRealTimeMorphsNames(Source, bEnableRealTimeMorphTargets, GenerationContext.CompilationContext->RealTimeMorphTargetsOverrides);

		// For constants we also have:
		// animbp instance?
		// gameplay tags?

		Result = BreakNode;
	}

	else if (Cast<UCONodeExternalOperation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMeshExternal> MeshExtension = new UE::Mutable::Private::NodeMeshExternal();

		FSourceExternalOptions Options;
		Options.MeshOptions = BaseMeshData;
		Options.bLinkedToExtendMaterial = bLinkedToExtendMaterial;
		Options.bOnlyConnectedLOD = bOnlyConnectedLOD;
		MeshExtension->Node = GenerateMutableSourceExternal(Pin, GenerationContext, Options);
		
		Result = MeshExtension;
	}
	
	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		// Can't use the GenerateMutableSourceMacro function here because GenerateMutableSourceMesh needs some extra parameters
		bCacheNode = false;

		if (const UEdGraphPin* OutputPin = TypedNodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin->PinName))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*OutputPin))
			{
				GenerationContext.MacroNodesStack.Push(TypedNodeMacro);
				Result = GenerateMutableSourceMesh(FollowPin, GenerationContext, BaseMeshData, bLinkedToExtendMaterial, bOnlyConnectedLOD);
				GenerationContext.MacroNodesStack.Pop();
			}
			else
			{
				FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNotLinked_Mesh", "Macro Output node Pin {0} not linked."), FText::FromName(Pin->PinName));
				GenerationContext.Log(Msg, Node);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNameNotFound_Mesh", "Macro Output node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
			GenerationContext.Log(Msg, Node);
		}
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		// Can't use the GenerateMutableSourceMacro function here because GenerateMutableSourceMesh needs some extra parameters
		check(TypedNodeTunnel->bIsInputNode);
		check(GenerationContext.MacroNodesStack.Num());

		bCacheNode = false;

		const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = GenerationContext.MacroNodesStack.Pop();
		check(MacroInstanceNode);

		if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin->PinName, EEdGraphPinDirection::EGPD_Input))
		{
			if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
			{
				Result = GenerateMutableSourceMesh(FollowPin, GenerationContext, BaseMeshData, bLinkedToExtendMaterial, bOnlyConnectedLOD);
			}
		}
		else
		{
			FText Msg = FText::Format(LOCTEXT("MacroTunnelError_PinNameNotFound_Mesh", "Macro Instance Node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
			GenerationContext.Log(Msg, Node);
		}

		// Push the Macro again even if the result is null
		GenerationContext.MacroNodesStack.Push(MacroInstanceNode);
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedMeshNode", "Mesh node type not implemented yet."), Node);
	}
	
	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
