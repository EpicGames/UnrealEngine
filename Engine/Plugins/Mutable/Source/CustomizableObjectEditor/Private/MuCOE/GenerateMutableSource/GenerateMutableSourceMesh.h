// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSurfaceModifierMeshClipWithMesh.h"
#include "MuT/Table.h"
#include "Animation/AnimInstance.h"
#include "UObject/SoftObjectPtr.h"

class FName;
class FString;
class UCustomizableObjectNode;
class UEdGraphPin;
class UObject;
class USkeletalMesh;
class UStaticMesh;
class UStreamableRenderAsset;
struct FMutableGraphGenerationContext;
struct FMutableGraphMeshGenerationData;
struct FMutableCompilationContext;
struct FMutableSourceMeshData;

namespace UE::Mutable::Private
{
	class FMeshBufferSet;
}


/** Returns the corrected LOD and Section Index
 *
 * Do not confuse Section Index and Material Index, they are not the same.
 * 
 * @param Context 
 * @param Source 
 * @param SkeletalMesh 
 * @param OutLODIndex Corrected Skeletal Mesh LOD Index.
 * @param OutSectionIndex Corrected Skeletal Mesh Section Index.
 * @return OutLODIndex and OutSectionIndex will return -1 if the section is not found in the currently compiling LOD. */
void GetLODAndSection(const FMutableCompilationContext& Context, const FMutableSourceMeshData& Source, const USkeletalMesh& SkeletalMesh, int32& OutLODIndex, int32& OutSectionIndex);


/** Converts an Unreal Skeletal Mesh to Mutable Mesh.
 *
 * @param Source. Mesh data required to generate the mutable mesh
 * @param LODIndex LOD we are generating. Will be different from LODIndexConnected only when using Automatic LOD From Mesh. 
 * @param SectionIndex Section we are generating. Will be different from SectionIndexConnected only when using Automatic LOD From Mesh.
 * @param GenerationContext 
 * @param CurrentNode The node being compiled, can be a skeletal mesh or a table node.
 * @return Mutable Mesh. Nullptr if there has been an error. Empty mesh if the Skeletal Mesh does not contain the requested LOD + Section. */
UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> ConvertSkeletalMeshToMutable(FMutableSourceMeshData& Source,
                                         int32 LODIndex, int32 SectionIndex,
                                         FMutableGraphGenerationContext& GenerationContext,
                                         const UCustomizableObjectNode* CurrentNode, 
										 bool bForceImmediateConversion = false );


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> ConvertStaticMeshToMutable(const UStaticMesh* StaticMesh, int32 LODIndex, int32 SectionIndex,
	FMutableGraphGenerationContext& GenerationContext, const UCustomizableObjectNode* CurrentNode);


/** 
 * @param Source. Mesh data required to generate the mutable mesh
 * @param LODIndexConnected LOD which the pin is connected to.
 * @param SectionIndexConnected Section which the pin is connected to.
 * @param GenerationContext
 * @param CurrentNode The node being compiled, can be a skeletal mesh or a table node.
 * @param bOnlyConnectedLOD
 * @return Mutable Mesh. Nullptr if there has been an error. Empty mesh if the Skeletal Mesh does not contain the requested LOD + Section. */
UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> GenerateMutableSkeletalMesh(
	FMutableSourceMeshData& Source,
	int32 LODIndexConnected,
	int32 SectionIndexConnected,
	FMutableGraphGenerationContext & GenerationContext, 
	const UCustomizableObjectNode* CurrentNode,
	const bool bOnlyConnectedLOD);


/** */
UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> GenerateMutableStaticMesh(
	TSoftObjectPtr<UStreamableRenderAsset> Mesh,
	int32 LODIndex, 
	int32 SectionIndex,
	FMutableGraphGenerationContext& GenerationContext,
	const UCustomizableObjectNode* CurrentNode);


UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> BuildMorphedMutableMesh(const UEdGraphPin* BaseSourcePin, const FString& MorphTargetName,
	FMutableGraphGenerationContext& GenerationContext, bool bOnlyConnectedLOD, const FName& RowName = "");


/** Compiler recursive function. Mutable Node Mesh.
 *
 * @param Pin 
 * @param GenerationContext 
 * @param BaseMeshData 
 * @param bLinkedToExtendMaterial 
 * @param bOnlyConnectedLOD Corrected LOD and Section will unconditionally always be the connected ones.
 * @return  Mutable Mesh Node. */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> GenerateMutableSourceMesh(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FMutableSourceMeshData& BaseMeshData, bool bLinkedToExtendMaterial, bool bOnlyConnectedLOD);
