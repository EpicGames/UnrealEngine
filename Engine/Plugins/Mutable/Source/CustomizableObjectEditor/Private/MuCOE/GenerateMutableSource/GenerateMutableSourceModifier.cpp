// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceModifier.h"

#include "Engine/StaticMesh.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceImage.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceSurface.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceLayout.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceGroupProjector.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTransform.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/MutableUtils.h"
#include "MuCOE/Nodes/CONodeModifierTransformWithBone.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipDeform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithUVMask.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierEditSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierSkeletalMeshMerge.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocks.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierTransformInMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuR/Mesh.h"
#include "MuT/NodeModifierSkeletalMeshMerge.h"
#include "MuT/NodeMeshTransform.h"
#include "MuT/NodeSurfaceModifierMeshClipDeform.h"
#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"
#include "MuT/NodeSurfaceModifierMeshClipWithUVMask.h"
#include "MuT/NodeSurfaceModifierMeshTransformInMesh.h"
#include "MuT/NodeSurfaceModifierMeshTransformWithBone.h"
#include "MuT/NodeSurfaceModifierSurfaceEdit.h"
#include "Rendering/SkeletalMeshLODModel.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


namespace UE::Mutable::Private
{
	Ptr<NodeModifierSkeletalMeshMerge> GenerateMutableSourceModifierSkeletalMeshMerge(const UCONodeModifierMutableSkeletalMeshMerge& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		Ptr<NodeModifierSkeletalMeshMerge> MergeNode = new NodeModifierSkeletalMeshMerge();

		const FName& ParentSkeletalMeshName = Node.GetParentSkeletalMeshName(&GenerationContext.MacroNodesStack);
		MergeNode->ParentSkeletalMeshName = ParentSkeletalMeshName;

		const UEdGraphPin* SkeletalMeshPin = Node.SkeletalMeshPin.Get();
		check(SkeletalMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SkeletalMeshPin))
		{
			// The mutable Skeletal Mesh Merge modifier only consumes NodeSkeletalMeshNew, which is produced by UCONodeSkeletalMeshMake_V2.
			FSourceSkeletalMeshOptions Options;
			Options.NumLODs = MAX_uint8;
			Options.FirstLODAvailable = 0;

			Ptr<NodeSkeletalMesh> SourceMesh = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
			if (SourceMesh)
			{
				MergeNode->ToAddSkeletalMesh = SourceMesh.get();
			}
		}

		return MergeNode;
	}
	
	
	Ptr<NodeSurfaceModifierMeshClipMorphPlane> GenerateMutableSourceModifierMeshClipMorphPlane(const UCustomizableObjectNodeModifierClipMorph& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
			EMutableMeshConversionFlags::IgnoreSkinning |
			EMutableMeshConversionFlags::IgnorePhysics |
			EMutableMeshConversionFlags::IgnoreMorphs |
			EMutableMeshConversionFlags::IgnoreTexCoords |
			EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		// This modifier can be connected to multiple nodes at the same time and, when that happens and if the cache is being used, only the first node to be processed does work. 
		// By not caching the mutable node we avoid this from even happening
		
		Ptr<NodeSurfaceModifierMeshClipMorphPlane> ClipNode = new NodeSurfaceModifierMeshClipMorphPlane();

		const FVector Origin = Node.GetOriginWithOffset();
		const FVector& Normal = Node.Normal;

		ClipNode->SetPlane(FVector3f(Origin), FVector3f(Normal));
		ClipNode->SetParams(Node.B, Node.Exponent);
		ClipNode->SetMorphEllipse(Node.Radius, Node.Radius2, Node.RotationAngle);

		ClipNode->SetVertexSelectionBone(Node.BoneName, Node.MaxEffectRadius);

		ClipNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		ClipNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		ClipNode->Parameters.FaceCullStrategy = Node.FaceCullStrategy;

		GenerationContext.MeshGenerationFlags.Pop();
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		ClipNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return ClipNode;
	}

	
	Ptr<NodeSurfaceModifierMeshClipDeform> GenerateMutableSourceModifierClipDeform(const UCustomizableObjectNodeModifierClipDeform& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierMeshClipDeform> ClipNode = new NodeSurfaceModifierMeshClipDeform();
	
		ClipNode->FaceCullStrategy = Node.FaceCullStrategy;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.ClipShapePin()))
		{
			Ptr<NodeMesh> ClipMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);

			ClipNode->ClipMesh = ClipMesh;

			EShapeBindingMethod BindingMethod = EShapeBindingMethod::ClipDeformClosestProject;
			switch(Node.BindingMethod)
			{
			case ::EShapeBindingMethod::ClosestProject:
				BindingMethod = EShapeBindingMethod::ClipDeformClosestProject;
				break;
			case ::EShapeBindingMethod::NormalProject:
				BindingMethod = EShapeBindingMethod::ClipDeformNormalProject;
				break;
			case ::EShapeBindingMethod::ClosestToSurface:
				BindingMethod = EShapeBindingMethod::ClipDeformClosestToSurface;
				break;
			default:
				check(false);
				break;
			}

			ClipNode->BindingMethod = BindingMethod;
		}
		else
		{
			FText ErrorMsg = LOCTEXT("ClipDeform mesh", "The clip deform node requires an input clip shape.");
			GenerationContext.Log(ErrorMsg, &Node, EMessageSeverity::Error);
			
			GenerationContext.MeshGenerationFlags.Pop();
			return nullptr;
		}
	
		ClipNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		ClipNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		GenerationContext.MeshGenerationFlags.Pop();
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		ClipNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;
		
		return ClipNode;
	}
	

	Ptr<NodeSurfaceModifierMeshClipWithMesh> GenerateMutableSourceModifierClipWithMesh(const UCustomizableObjectNodeModifierClipWithMesh& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);
		
		Ptr<NodeSurfaceModifierMeshClipWithMesh> ClipNode = new NodeSurfaceModifierMeshClipWithMesh();

		ClipNode->FaceCullStrategy = Node.FaceCullStrategy;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.GetClipMeshPin()))
		{
			NodeMeshPtr ClipMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);

			if (FMatrix Matrix = Node.Transform.ToMatrixWithScale(); Matrix != FMatrix::Identity)
			{
				Ptr<NodeMeshTransform> TransformMesh = new NodeMeshTransform();
				TransformMesh->Source = ClipMesh;

				TransformMesh->Transform = FMatrix44f(Matrix);
				ClipMesh = TransformMesh;
			}

			ClipNode->ClipMesh = ClipMesh;
		}
		else
		{
			FText ErrorMsg = LOCTEXT("Clipping mesh missing", "The clip mesh with mesh node requires an input clip mesh.");
			GenerationContext.Log(ErrorMsg, &Node, EMessageSeverity::Error);
			
			GenerationContext.MeshGenerationFlags.Pop();
			return nullptr;
		}

		ClipNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		ClipNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		GenerationContext.MeshGenerationFlags.Pop();
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		ClipNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;
		
		
		return ClipNode;
	}
	
	Ptr<NodeSurfaceModifierMeshClipWithUVMask> GenerateMutableSourceModifierModifierClipWithUVMask(const UCustomizableObjectNodeModifierClipWithUVMask& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreAUD;
				
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierMeshClipWithUVMask> ClipNode = new NodeSurfaceModifierMeshClipWithUVMask();

		ClipNode->FaceCullStrategy = Node.FaceCullStrategy;

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.ClipMaskPin()))
		{
			Ptr<NodeImage> ClipMask = GenerateMutableSourceImage(ConnectedPin, GenerationContext, 0);

			ClipNode->ClipMask = ClipMask;
		}
		else
		{
			FText ErrorMsg = LOCTEXT("ClipUVMask mesh", "The clip mesh with UV Mask node requires an input texture mask.");
			GenerationContext.Log(ErrorMsg, &Node, EMessageSeverity::Error);
			
			GenerationContext.MeshGenerationFlags.Pop();
			return nullptr;
		}

		ClipNode->LayoutIndex = Node.UVChannelForMask;

		ClipNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		ClipNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		GenerationContext.MeshGenerationFlags.Pop();
				
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		ClipNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;
		
		return ClipNode;
	}
	
	
	Ptr<NodeSurfaceModifierSurfaceEdit> GenerateMutableSourceModifierExtendSkeletalMeshSection(UCONodeModifierExtendSkeletalMeshSection& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags = EMutableMeshConversionFlags::None;
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierSurfaceEdit> SurfNode = new NodeSurfaceModifierSurfaceEdit();
		SurfNode->ModifierGuid = GenerationContext.GetNodeIdUnique(&Node);

		// TODO: This was used in the non-modifier version for group projectors. It may affect the "drop projection from LOD" feature.
		const int32 LOD = Node.IsAffectedByLOD() ? GenerationContext.CurrentLOD : 0;

		SurfNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		SurfNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		// Is this enough? Should we try to narrow down with potential mesh sections modified by this?
		int32 LODCount = GenerationContext.MaxNumLODs;
		SurfNode->LODs.SetNum(LODCount);

		for (int32 LODIndex= Node.FirstLOD; LODIndex<LODCount; ++LODIndex)
		{
			GenerationContext.FromLOD = Node.FirstLOD;
			GenerationContext.CurrentLOD = LODIndex;

			Ptr<NodeMesh> AddMeshNode;
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.AddMeshPin()))
			{
				// Flags to know which UV channels need layout
				FLayoutGenerationFlags LayoutGenerationFlags;
				LayoutGenerationFlags.TexturePinModes.Init(EPinMode::Mutable, TEXSTREAM_MAX_NUM_UVCHANNELS);

				GenerationContext.LayoutGenerationFlags.Push(LayoutGenerationFlags);
			
				FMutableSourceMeshData MeshData;
				AddMeshNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, MeshData, true, false);

				GenerationContext.LayoutGenerationFlags.Pop();
			}

			SurfNode->LODs[LODIndex].MeshAdd = AddMeshNode;

			const int32 NumImages = Node.GetNumParameters(EMaterialParameterType::Texture);
			for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
			{
				NodeImagePtr ImageNode;
				const FName& MaterialParameterName = Node.GetParameterName(EMaterialParameterType::Texture, ImageIndex);
				const int32 MaterialParameterLayerIndex = Node.GetParameterLayerIndex(EMaterialParameterType::Texture, ImageIndex);
				check(MaterialParameterLayerIndex <= MAX_int8);

				if (!ImageNode) // If
				{
					bool bIsGroupProjectorImage = false;
					UTexture2D* GroupProjectionReferenceTexture = nullptr;

					ImageNode = GenerateMutableSourceGroupProjector(LOD, ImageIndex, AddMeshNode, GenerationContext,
						nullptr, &Node, bIsGroupProjectorImage,
						GroupProjectionReferenceTexture);
				}

				if (!ImageNode) // Else if
				{
					const FNodeMaterialParameterId ImageId = Node.GetParameterId(EMaterialParameterType::Texture, ImageIndex);

					if (Node.UsesImage(ImageId))
					{
						// TODO
						//check(ParentMaterialNode->IsImageMutableMode(ImageIndex)); // Ensured at graph time. If it fails, something is wrong.

						if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.GetUsedImagePin(ImageId)))
						{
							// ReferenceTextureSize is used to limit the size of textures contributing to the final image.
							const int32 ReferenceTextureSize = 0; // TODO GetBaseTextureSize(GenerationContext, TypedNodeExt, ImageIndex);

							ImageNode = GenerateMutableSourceImage(ConnectedPin, GenerationContext, ReferenceTextureSize);
						}
					}
				}

				if (ImageNode)
				{
					NodeSurfaceModifierSurfaceEdit::FTexture Texture;
					Texture.Extend = ImageNode;
					Texture.MaterialParameterKey = FParameterKey(MaterialParameterName, (int8)MaterialParameterLayerIndex);

					SurfNode->LODs[LODIndex].Textures.Add(Texture);
				}
			}
		}

		SurfNode->EnableTags = Node.GetEnableTags(&GenerationContext.MacroNodesStack);
		SurfNode->EnableTags.AddUnique(Node.GetInternalTag());

		GenerationContext.MeshGenerationFlags.Pop();
		GenerationContext.FromLOD = 0;
		GenerationContext.CurrentLOD = 0;
				
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		SurfNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return SurfNode;
	}
	
	
	Ptr<NodeSurfaceModifierSurfaceEdit> GenerateMutableSourceModifierRemoveMesh(const UCustomizableObjectNodeModifierRemoveMesh& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierSurfaceEdit> SurfNode = new NodeSurfaceModifierSurfaceEdit();

		SurfNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		SurfNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.RemoveMeshPin()))
		{
			// Is this enough? Should we try to narrow down with potential mesh sections modified by this?
			int32 LODCount = GenerationContext.MaxNumLODs;
			SurfNode->LODs.SetNum(LODCount);

			SurfNode->FaceCullStrategy = Node.FaceCullStrategy;

			for (int32 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
			{
				GenerationContext.FromLOD = 0;
				GenerationContext.CurrentLOD = LODIndex;

				Ptr<NodeMesh> RemoveMeshNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);
				SurfNode->LODs[LODIndex].MeshRemove = RemoveMeshNode;
			}
		}

		GenerationContext.MeshGenerationFlags.Pop();
		GenerationContext.FromLOD = 0;
		GenerationContext.CurrentLOD = 0;
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		SurfNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return SurfNode;
	}
	
	
	Ptr<NodeSurfaceModifierMeshClipWithUVMask> GenerateMutableSourceModifierRemoveMeshBlocks(const UCustomizableObjectNodeModifierRemoveMeshBlocks& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		Ptr<NodeSurfaceModifierMeshClipWithUVMask> ClipNode = new NodeSurfaceModifierMeshClipWithUVMask();

		ClipNode->FaceCullStrategy = Node.FaceCullStrategy;

		ClipNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		ClipNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		check(Node.Layout->GetMesh().IsNull());
		Ptr<NodeLayout> SourceLayout = CreateMutableLayoutNode(Node.Layout, true);
		ClipNode->ClipLayout = SourceLayout;
		ClipNode->LayoutIndex = Node.ParentLayoutIndex;
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		ClipNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return ClipNode;
	}
	
	
	Ptr<NodeSurfaceModifierSurfaceEdit> GenerateMutableSourceModifierEditSkeletalMeshSection(const UCONodeModifierEditSkeletalMeshSection& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierSurfaceEdit> SurfNode = new NodeSurfaceModifierSurfaceEdit();

		SurfNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		SurfNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		// Is this enough? Should we try to narrow down with potential mesh sections modified by this?
		int32 LODCount = GenerationContext.MaxNumLODs;
		SurfNode->LODs.SetNum(LODCount);

		for (int32 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
		{
			GenerationContext.FromLOD = 0;
			GenerationContext.CurrentLOD = LODIndex;

			const int32 NumImages = Node.GetNumParameters(EMaterialParameterType::Texture);

			for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
			{
				const FNodeMaterialParameterId ImageId = Node.GetParameterId(EMaterialParameterType::Texture, ImageIndex);

				if (Node.UsesImage(ImageId))
				{
					// TODO
					//check(ParentMaterialNode->IsImageMutableMode(ImageIndex)); // Ensured at graph time. If it fails, something is wrong.

					const UEdGraphPin* ConnectedImagePin = FollowInputPin(*Node.GetUsedImagePin(ImageId));
					const FName& MaterialParameterName = Node.GetParameterName(EMaterialParameterType::Texture, ImageIndex);
					const int32 MaterialParameterLayerIndex = Node.GetParameterLayerIndex(EMaterialParameterType::Texture, ImageIndex);
					check(MaterialParameterLayerIndex <= MAX_int8);
				
					NodeSurfaceModifierSurfaceEdit::FTexture ImagePatch;
					ImagePatch.MaterialParameterKey = FParameterKey(MaterialParameterName,  (int8)MaterialParameterLayerIndex);
					// \todo: expose these two options?
					ImagePatch.PatchBlendType = EBlendType::BT_BLEND;
					ImagePatch.bPatchApplyToAlpha = true;

					// ReferenceTextureSize is used to limit the size of textures contributing to the final image.
					const int32 ReferenceTextureSize = 0; //TODO GetBaseTextureSize(GenerationContext, ParentMaterialNode, ImageIndex);

					ImagePatch.PatchImage = GenerateMutableSourceImage(ConnectedImagePin, GenerationContext, ReferenceTextureSize);

					const UEdGraphPin* ImageMaskPin = Node.GetUsedImageMaskPin(ImageId);
					check(ImageMaskPin); // Ensured when reconstructing EditMaterial nodes. If it fails, something is wrong.

					if (const UEdGraphPin* ConnectedMaskPin = FollowInputPin(*ImageMaskPin))
					{
						ImagePatch.PatchMask = GenerateMutableSourceImage(ConnectedMaskPin, GenerationContext, ReferenceTextureSize);
					}

					// Add the blocks to patch
					FIntPoint GridSize = Node.Layout->GetGridSize();
					FVector2f GridSizeF = FVector2f(GridSize);
					ImagePatch.PatchBlocks.Reserve(Node.Layout->Blocks.Num());

					for (const FCustomizableObjectLayoutBlock& LayoutBlock : Node.Layout->Blocks)
					{
						FBox2f Rect;
						Rect.Min = FVector2f(LayoutBlock.Min) / GridSizeF;
						Rect.Max = FVector2f(LayoutBlock.Max) / GridSizeF;
						ImagePatch.PatchBlocks.Add(Rect);
					}

					SurfNode->LODs[LODIndex].Textures.Add(ImagePatch);
				}
			}
		}

		GenerationContext.MeshGenerationFlags.Pop();
		GenerationContext.FromLOD = 0;
		GenerationContext.CurrentLOD = 0;
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		SurfNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return SurfNode;
	}
	
	
	Ptr<NodeSurfaceModifierSurfaceEdit> GenerateMutableSourceModifierMorphMeshSection(const UCustomizableObjectNodeModifierMorphMeshSection& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierSurfaceEdit> SurfNode = new NodeSurfaceModifierSurfaceEdit();

		// This modifier needs to be applied right after the mesh constant is generated
		SurfNode->bApplyBeforeNormalOperations = true;

		SurfNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		SurfNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		SurfNode->MeshMorph = Node.GetMorphTargetName(GenerationContext);

		UEdGraphPin* FactorPinPtr = Node.FactorPinReference.Get();
		check(FactorPinPtr);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*FactorPinPtr))
		{
			// Checking if it's linked to a Macro or tunnel node
			const UEdGraphPin* FloatPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, &GenerationContext.MacroNodesStack);
			bool validStaticFactor = true;

			if (FloatPin)
			{
				UEdGraphNode* floatNode = FloatPin->GetOwningNode();
				if (const UCustomizableObjectNodeFloatParameter* floatParameterNode = Cast<UCustomizableObjectNodeFloatParameter>(floatNode))
				{
					if (floatParameterNode->DefaultValue < -1.0f || floatParameterNode->DefaultValue > 1.0f)
					{
						validStaticFactor = false;
						FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the default value of the float parameter node is (%f). Factor will be ignored."), floatParameterNode->DefaultValue);
						GenerationContext.Log(FText::FromString(msg), &Node);
					}
					if (floatParameterNode->ParamUIMetadata.MinimumValue < -1.0f)
					{
						validStaticFactor = false;
						FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the minimum UI value for the input float parameter node is (%f). Factor will be ignored."), floatParameterNode->ParamUIMetadata.MinimumValue);
						GenerationContext.Log(FText::FromString(msg), &Node);
					}
					if (floatParameterNode->ParamUIMetadata.MaximumValue > 1.0f)
					{
						validStaticFactor = false;
						FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the maximum UI value for the input float parameter node is (%f). Factor will be ignored."), floatParameterNode->ParamUIMetadata.MaximumValue);
						GenerationContext.Log(FText::FromString(msg), &Node);
					}
				}
				else if (const UCustomizableObjectNodeFloatConstant* floatConstantNode = Cast<UCustomizableObjectNodeFloatConstant>(floatNode))
				{
					if (floatConstantNode->Value < -1.0f || floatConstantNode->Value > 1.0f)
					{
						validStaticFactor = false;
						FString msg = FString::Printf(TEXT("Mesh morph nodes only accept factors between -1.0 and 1.0 inclusive but the value of the float constant node is (%f). Factor will be ignored."), floatConstantNode->Value);
						GenerationContext.Log(FText::FromString(msg), &Node);
					}
				}
			}

			// If is a valid factor, continue the Generation
			if (validStaticFactor)
			{
				Ptr<NodeScalar> FactorNode = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
				SurfNode->MorphFactor = FactorNode;
			}
		}

		GenerationContext.MeshGenerationFlags.Pop();
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		SurfNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return SurfNode;
	}
	
	
	Ptr<NodeSurfaceModifierMeshTransformInMesh> GenerateMutableSourceModifierTransformInMesh(const UCustomizableObjectNodeModifierTransformInMesh& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		const EMutableMeshConversionFlags ModifiersMeshFlags =
			EMutableMeshConversionFlags::IgnoreSkinning |
			EMutableMeshConversionFlags::IgnorePhysics |
			EMutableMeshConversionFlags::IgnoreMorphs |
			EMutableMeshConversionFlags::IgnoreTexCoords |
			EMutableMeshConversionFlags::IgnoreAUD;
		
		GenerationContext.MeshGenerationFlags.Push(ModifiersMeshFlags);

		Ptr<NodeSurfaceModifierMeshTransformInMesh> TransformNode = new NodeSurfaceModifierMeshTransformInMesh();

		const UEdGraphPin* TransformPin = Node.TransformMeshPin.Get();
		check(TransformPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TransformPin))
		{
			TransformNode->MatrixNode = GenerateMutableSourceTransform(ConnectedPin, GenerationContext);
		}

		// If no bounding mesh is provided, we transform the entire mesh.
		const UEdGraphPin* BoundingMeshPin = Node.BoundingMeshPin.Get();
		check(BoundingMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BoundingMeshPin))
		{
			Ptr<NodeMesh> BoundingMesh = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, FMutableSourceMeshData(), false, true);

			if (FMatrix Matrix = Node.BoundingMeshTransform.ToMatrixWithScale(); Matrix != FMatrix::Identity)
			{
				Ptr<NodeMeshTransform> TransformMesh = new NodeMeshTransform();
				TransformMesh->Source = BoundingMesh;

				TransformMesh->Transform = FMatrix44f(Matrix);
				BoundingMesh = TransformMesh;
			}

			TransformNode->BoundingMesh = BoundingMesh;
		}

		TransformNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		TransformNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);

		GenerationContext.MeshGenerationFlags.Pop();
	
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		TransformNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return TransformNode;
	}
	
	
	Ptr<NodeSurfaceModifierMeshTransformWithBone> GenerateMutableSourceModifierMeshTransformWithBone(const UCONodeModifierTransformWithBone& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		Ptr<NodeSurfaceModifierMeshTransformWithBone> TransformNode = new NodeSurfaceModifierMeshTransformWithBone();

		TransformNode->BoneName = *Node.BoneName;
		TransformNode->ThresholdFactor = Node.ThresholdFactor;

		const UEdGraphPin* TransformPin = Node.TransformPin.Get();
		check(TransformPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TransformPin))
		{
			TransformNode->MatrixNode = GenerateMutableSourceTransform(ConnectedPin, GenerationContext);
		}

		TransformNode->MultipleTagsPolicy = Node.MultipleTagPolicy;
		TransformNode->RequiredTags = Node.GetNodeRequiredTags(&GenerationContext.MacroNodesStack);
		
		check(GenerationContext.CurrentSkeletalMeshComponent != INDEX_NONE);
		TransformNode->RequiredComponentId = GenerationContext.CurrentSkeletalMeshComponent;

		return TransformNode;
	}
	
	
	TArray<Ptr<NodeModifier>> GenerateMutableSourceModifier(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
	{
		check(Pin)
		RETURN_ON_CYCLE(*Pin, GenerationContext)

		CheckNode(*Pin, GenerationContext);
	
		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

		FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceModifier), *Pin, *Node, GenerationContext, true);
		Key.CurrentMeshComponent = GenerationContext.CurrentSkeletalMeshComponent;
	
		if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
		{
			return Generated->ModifierNodes;
		}
	
		TArray<Ptr<NodeModifier>> Result;

		// Bool that determines if a node can be added to the cache of nodes.
		// Most nodes need to be added to the cache but there are some that don't. For exampel, MacroInstanceNodes
		bool bCacheNode = true; 

		if (const UCustomizableObjectNodeModifierClipMorph* TypedNodeClip = Cast<UCustomizableObjectNodeModifierClipMorph>(Node))
		{
			bCacheNode = false;
			
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierMeshClipMorphPlane(*TypedNodeClip, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierClipDeform* TypedNodeClipDeform = Cast<UCustomizableObjectNodeModifierClipDeform>(Node))
		{
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierClipDeform(*TypedNodeClipDeform, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierClipWithMesh* TypedNodeClipMesh = Cast<UCustomizableObjectNodeModifierClipWithMesh>(Node))
		{
			// MeshClipWithMesh can be connected to multiple objects, so the compiled NodeModifierMeshClipWithMesh
			// needs to be different for each object. If it were added to the Generated cache, all the objects would get the same.
			bCacheNode = false;

			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierClipWithMesh(*TypedNodeClipMesh, GenerationContext))
				{
					Result.Add(NodeModifier);
				}

				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierClipWithUVMask* TypedNodeClipUVMask = Cast<UCustomizableObjectNodeModifierClipWithUVMask>(Node))
		{
			// This modifier can be connected to multiple objects, so the compiled node
			// needs to be different for each object. If it were added to the Generated cache, all the objects would get the same.
			bCacheNode = false;
			
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierModifierClipWithUVMask(*TypedNodeClipUVMask, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (UCONodeModifierExtendSkeletalMeshSection* TypedNodeExt = Cast<UCONodeModifierExtendSkeletalMeshSection>(Node))
		{
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierExtendSkeletalMeshSection(*TypedNodeExt, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierRemoveMesh* TypedNodeRem = Cast<UCustomizableObjectNodeModifierRemoveMesh>(Node))
		{
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierRemoveMesh(*TypedNodeRem, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierRemoveMeshBlocks* TypedNodeRemBlocks = Cast<UCustomizableObjectNodeModifierRemoveMeshBlocks>(Node))
		{
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierRemoveMeshBlocks(*TypedNodeRemBlocks, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (UCONodeModifierEditSkeletalMeshSection* TypedNodeEdit = Cast<UCONodeModifierEditSkeletalMeshSection>(Node))
		{
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierEditSkeletalMeshSection(*TypedNodeEdit, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierMorphMeshSection* TypedNodeMorph = Cast<UCustomizableObjectNodeModifierMorphMeshSection>(Node))
		{
			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierMorphMeshSection(*TypedNodeMorph, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCustomizableObjectNodeModifierTransformInMesh* TypedNodeTransformMesh = Cast<UCustomizableObjectNodeModifierTransformInMesh>(Node))
		{
			// MeshTransformInMesh can be connected to multiple objects, so the compiled NodeModifierMeshTransformInMesh
			// needs to be different for each object. If it were added to the Generated cache, all the objects would get the same.
			bCacheNode = false;

			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierTransformInMesh(*TypedNodeTransformMesh, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCONodeModifierTransformWithBone* TypedNodeTransformWithBone = Cast<UCONodeModifierTransformWithBone>(Node))
		{
			bCacheNode = false;

			for (const FSkeletalMeshComponentInfo& Pair : GenerationContext.CompilationContext->ComponentInfos)
			{
				GenerationContext.CurrentSkeletalMeshComponent = Pair.ComponentId;
				
				if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierMeshTransformWithBone(*TypedNodeTransformWithBone, GenerationContext))
				{
					Result.Add(NodeModifier);
				}
				
				GenerationContext.CurrentSkeletalMeshComponent = INDEX_NONE;
			}
		}

		else if (const UCONodeModifierMutableSkeletalMeshMerge* MergeNode = Cast<UCONodeModifierMutableSkeletalMeshMerge>(Node))
		{
			if (Ptr<NodeModifier> NodeModifier = GenerateMutableSourceModifierSkeletalMeshMerge(*MergeNode, GenerationContext))
			{
				Result.Add(NodeModifier);
			}
		}
	
		else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
		{
			bCacheNode = false;
		
			if (const UEdGraphPin* OutputPin = TypedNodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin->PinName))
			{
				if (const UEdGraphPin* FollowPin = FollowInputPin(*OutputPin))
				{
					GenerationContext.MacroNodesStack.Push(TypedNodeMacro);
					Result = GenerateMutableSourceModifier(FollowPin, GenerationContext);
					GenerationContext.MacroNodesStack.Pop();
				}
				else
				{
					FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNotLinked", "Macro Output node Pin {0} not linked."), FText::FromName(Pin->PinName));
					GenerationContext.Log(Msg, Node);
				}
			}
			else
			{
				FText Msg = FText::Format(LOCTEXT("MacroInstanceError_PinNameNotFound", "Macro Output node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
				GenerationContext.Log(Msg, Node);
			}
		}

		else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
		{
			bCacheNode = false;
			check(TypedNodeTunnel->bIsInputNode);
			check(GenerationContext.MacroNodesStack.Num());

			const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = GenerationContext.MacroNodesStack.Pop();
			check(MacroInstanceNode);

			if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin->PinName, EEdGraphPinDirection::EGPD_Input))
			{
				if (const UEdGraphPin* FollowPin = FollowInputPin(*InputPin))
				{
					Result = GenerateMutableSourceModifier(FollowPin, GenerationContext);
				}
			}
			else
			{
				FText Msg = FText::Format(LOCTEXT("MacroTunnelError_PinNameNotFound", "Macro Instance Node does not contain a pin with name {0}."), FText::FromName(Pin->PinName));
				GenerationContext.Log(Msg, Node);
			}

			// Push the Macro again even if the result is null
			GenerationContext.MacroNodesStack.Push(MacroInstanceNode);
		}
		
		else
		{
			GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
		}

		for (Ptr<NodeModifier> NodeModifier : Result)
		{
			NodeModifier->SetMessageContext(Node);
		}

		if (bCacheNode)
		{
			GenerationContext.Generated.Add(Key,FGeneratedData(Node, Result));
		}
		
		GenerationContext.GeneratedNodes.Add(Node);
		return Result;
	}
}


#undef LOCTEXT_NAMESPACE

