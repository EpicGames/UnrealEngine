// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeModifier.h"
#include "MuR/Ptr.h"
#include "Containers/Array.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;


namespace UE::Mutable::Private
{
	TArray<Ptr<NodeModifier>> GenerateMutableSourceModifier(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);
}
