// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceProjector.h"

#include "MuCO/MutableProjectorTypeUtils.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorParameter.h"
#include "MuR/Parameters.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjector> GenerateMutableSourceProjector(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)
	
	CheckNode(*Pin, GenerationContext);
	
	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceProjector), *Pin, *Node, GenerationContext, true);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeProjector*>(Generated->Node.get());
	}

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjector> Result;
	
	if (const UCustomizableObjectNodeProjectorConstant* TypedNodeConst = Cast<UCustomizableObjectNodeProjectorConstant>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjectorConstant> ProjectorNode = new UE::Mutable::Private::NodeProjectorConstant();
		Result = ProjectorNode;

		const UE::Mutable::Private::EProjectorType ProjectorType = ProjectorUtils::GetEquivalentProjectorType(TypedNodeConst->Value.ProjectionType);
		switch (ProjectorType) 
		{ 
			case UE::Mutable::Private::EProjectorType::Planar:
				ProjectorNode->SetValue(
					ProjectorType,
					TypedNodeConst->Value.Position,
					TypedNodeConst->Value.Direction,
					TypedNodeConst->Value.Up,
					TypedNodeConst->Value.Scale,
					TypedNodeConst->Value.Angle);
				break;
			
			case UE::Mutable::Private::EProjectorType::Cylindrical:
				{
					// Apply strange swizzle for scales
					// TODO: try to avoid this
					const float Radius = FMath::Abs(TypedNodeConst->Value.Scale[0] / 2.0f);
					const float Height = TypedNodeConst->Value.Scale[2];
					ProjectorNode->SetValue(
						ProjectorType,
						TypedNodeConst->Value.Position,
						-TypedNodeConst->Value.Direction,
						-TypedNodeConst->Value.Up,
						FVector3f( -Height, Radius, Radius ),
						TypedNodeConst->Value.Angle);
				}
				break;
			
			case UE::Mutable::Private::EProjectorType::Wrapping:
				ProjectorNode->SetValue(
					ProjectorType,
					TypedNodeConst->Value.Position,
					TypedNodeConst->Value.Direction,
					TypedNodeConst->Value.Up,
					TypedNodeConst->Value.Scale,
					TypedNodeConst->Value.Angle);
				break;
			
			case UE::Mutable::Private::EProjectorType::Count: 
			default:
				checkNoEntry();
		}
	}

	else if (const UCustomizableObjectNodeProjectorParameter* TypedNodeParam = Cast<UCustomizableObjectNodeProjectorParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjectorParameter> ProjectorNode = new UE::Mutable::Private::NodeProjectorParameter();
		Result = ProjectorNode;

		ProjectorNode->SetName(TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack));
		ProjectorNode->SetUid(GenerationContext.GetNodeIdUnique(Node).ToString());
		switch ((int)TypedNodeParam->DefaultValue.ProjectionType)
		{
		case 0:
			ProjectorNode->SetDefaultValue(
				UE::Mutable::Private::EProjectorType::Planar,
				TypedNodeParam->DefaultValue.Position,
				TypedNodeParam->DefaultValue.Direction,
				TypedNodeParam->DefaultValue.Up,
				TypedNodeParam->DefaultValue.Scale,
				TypedNodeParam->DefaultValue.Angle);
			break;

		case 1:
		{
			// Apply strange swizzle for scales
			// TODO: try to avoid this
			float Radius = FMath::Abs(TypedNodeParam->DefaultValue.Scale[0] / 2.0f);
			float Height = TypedNodeParam->DefaultValue.Scale[2];
			ProjectorNode->SetDefaultValue(
				UE::Mutable::Private::EProjectorType::Cylindrical,
				TypedNodeParam->DefaultValue.Position,
				-TypedNodeParam->DefaultValue.Direction,
				-TypedNodeParam->DefaultValue.Up,
				FVector3f( -Height, Radius, Radius ),
				TypedNodeParam->DefaultValue.Angle);
			break;
		}

		case 2:
			ProjectorNode->SetDefaultValue(
				UE::Mutable::Private::EProjectorType::Wrapping,
				TypedNodeParam->DefaultValue.Position,
				TypedNodeParam->DefaultValue.Direction,
				TypedNodeParam->DefaultValue.Up,
				TypedNodeParam->DefaultValue.Scale,
				TypedNodeParam->DefaultValue.Angle);
			break;

		default:
			// Not implemented.
			check(false);
		}

		GenerationContext.ParameterUIDataMap.Add(TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
			TypedNodeParam->ParamUIMetadata,
			EMutableParameterType::Projector));
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}

	GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
	GenerationContext.GeneratedNodes.Add(Node);

	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE

