// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeProjector.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;

/** Convert a CustomizableObject Source Graph into a mutable source graph. */
UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeProjector> GenerateMutableSourceProjector(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext);
