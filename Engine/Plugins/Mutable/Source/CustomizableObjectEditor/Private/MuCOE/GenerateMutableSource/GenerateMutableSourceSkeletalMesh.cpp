// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateMutableSourceSkeletalMesh.h"

#include "GenerateMutableSource.h"
#include "GenerateMutableSourceFloat.h"
#include "GenerateMutableSourceMaterial.h"
#include "GenerateMutableSourceSurface.h"
#include "GenerateMutableSourceTransform.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshMerge.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshModify.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshMorph.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshReshape.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshTransformWithBone.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuT/NodeSkeletalMeshSwitch.h"
#include "MuT/NodeSkeletalMeshTransform.h"
#include "MuT/NodeSkeletalMeshVariation.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshVariation.h"

#include "GenerateMutableSourceSkeletalMeshObject.h"
#include "MuCOE/Nodes/CONodeClipSkeletalMeshWithSkeletalMesh.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshMake_V2.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuR/MutableTrace.h"
#include "MuT/NodeSkeletalMeshConvert.h"
#include "MuT/NodeSkeletalMeshClipWithSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshMerge.h"
#include "MuT/NodeSkeletalMeshModify.h"
#include "MuT/NodeSkeletalMeshMorph.h"
#include "MuT/NodeSkeletalMeshNew.h"
#include "MuT/NodeSkeletalMeshReshape.h"
#include "MuT/NodeSkeletalMeshTransformWithBone.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


namespace UE::Mutable::Private
{
	Ptr<NodeSkeletalMeshNew> GenerateMutableSourceSkeletalMeshNew(const UCONodeSkeletalMeshMake_V2& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshNew> SkeletalMeshNode = new NodeSkeletalMeshNew();

		const TArray<FEdGraphPinReference>& LODPins = Node.LODPins;
		
		const uint8 NumLODs = FMath::Min(Options.NumLODs, LODPins.Num());
		
		SkeletalMeshNode->LODs.SetNum(NumLODs);
		
		for (int32 CurrentLOD = Options.FirstLODAvailable; CurrentLOD < NumLODs; ++CurrentLOD)
		{
			GenerationContext.CurrentLOD = CurrentLOD;

			Ptr<NodeLOD> LODNode = new NodeLOD();
			LODNode->SetMessageContext(&Node);

			SkeletalMeshNode->LODs[CurrentLOD] = LODNode;
			
			// Generate all relevant LODs for this object up until the current LODIndex.
			for (int32 LODIndex = 0; LODIndex <= CurrentLOD; ++LODIndex)
			{
				if (!LODPins.IsValidIndex(LODIndex))
				{
					continue;
				}
				
				const UEdGraphPin* LODPin = LODPins[LODIndex].Get();
				check(LODPin);
				
				GenerationContext.FromLOD = LODIndex;

				for (UEdGraphPin* const ChildNodePin : FollowInputPinArray(*LODPin))
				{
					Ptr<NodeSurface> SurfaceNode = GenerateMutableSourceSurface(ChildNodePin, GenerationContext);
					LODNode->Surfaces.Add(SurfaceNode);
				}
			}
		}

		// Clear the context state for LODs
		GenerationContext.CurrentLOD = 0;
		GenerationContext.FromLOD = 0;
		
		return SkeletalMeshNode;
	}

	
	Ptr<NodeSkeletalMeshMerge> GenerateMutableSourceSkeletalMeshMerge(const UCONodeSkeletalMeshMerge& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshMerge> Result = new NodeSkeletalMeshMerge();
		
		const UEdGraphPin* BaseMeshPin = Node.BaseSkeletalMeshPin.Get();
		check(BaseMeshPin);
		if (const UEdGraphPin* ConnectedBaseMeshPin = FollowInputPin(*BaseMeshPin))
		{
			Result->BaseSkeletalMesh = GenerateMutableSourceSkeletalMesh(ConnectedBaseMeshPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("CONodeSkeletalMeshMerge_BaseMeshConnectionNotFoundWarning","No connections could be found to the Base Skeletal Mesh input pin.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
		}
		
		const UEdGraphPin* ToAddMeshPin = Node.ToAddSkeletalMeshPin.Get();
		check(ToAddMeshPin);
		if (const UEdGraphPin* ConnectedToAddMeshPin = FollowInputPin(*ToAddMeshPin))
		{
			Result->ToAddSkeletalMesh = GenerateMutableSourceSkeletalMesh(ConnectedToAddMeshPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("CONodeSkeletalMeshMerge_ToAddMeshConnectionNotFoundWarning","No connections could be found to the To Add Skeletal Mesh input pin.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
		}
		
		return Result;
	}

	
	Ptr<NodeSkeletalMeshModify> GenerateMutableSourceSkeletalMeshModify(const UCONodeSkeletalMeshModify& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshModify> Result = new NodeSkeletalMeshModify();
				
		const UEdGraphPin* BaseSkeletalMeshPin = Node.MutableSkeletalMeshPin.Get();
		check(BaseSkeletalMeshPin);
		if (const UEdGraphPin* ConnectedBaseSkeletalMeshPin = FollowInputPin(*BaseSkeletalMeshPin))
		{
			Result->SkeletalMesh = GenerateMutableSourceSkeletalMesh(ConnectedBaseSkeletalMeshPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("MutableSkeletalMeshModify_MissingSkeletalMeshConnection","Missing Skeletal Mesh Connection. Modify Skeletal Mesh Nodes requires a Skeletal Mesh.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
			return Result;
		}
		
		for (const UEdGraphPin* NodePin : Node.GetAllNonOrphanPins())
		{
			check(NodePin);
			
			if (!Node.IsSlotPin(*NodePin))
			{
				continue;
			}
			
			const FName TargetSlotName = Node.GetTargetSlotName(*NodePin);
		
			if (Result->SlotMaterials.Contains(TargetSlotName))
			{
				const FText Message = FText::Format(LOCTEXT("MutableSkeletalMeshModify_SlotNameAlreadyDefined","Target Slot Name '{0}' was defined multiple times. Only one of the entries will be used."), FText::FromName(TargetSlotName));
				GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
			}
			
			Ptr<NodeMaterial> MaterialNode;
			
			if (const UEdGraphPin* MaterialPin = FollowInputPin(*NodePin))
			{
				MaterialNode = GenerateMutableSourceMaterial(MaterialPin, GenerationContext, FGenerationMaterialOptions());
			}
			else
			{
				MaterialNode = new NodeMaterialConstant();
			}
			
			Result->SlotMaterials.Add(TargetSlotName, MaterialNode);
		}
		
		return Result;
	}


	Ptr<NodeSkeletalMeshMorph> GenerateMutableSourceSkeletalMeshMorph(const UCONodeSkeletalMeshMorph& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshMorph> Result = new NodeSkeletalMeshMorph();

		// Base Skeletal Mesh
		const UEdGraphPin* InputSkeletalMeshPin = Node.MeshPinReference.Get();
		check(InputSkeletalMeshPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*InputSkeletalMeshPin))
		{
			Result->Base = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("MutableSkeletalMeshMorph_MissingSkeletalMeshConnection","Missing Skeletal Mesh Connection. Morph Skeletal Mesh Nodes requires a Skeletal Mesh.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
			return Result;
		}
		
		// Target morph name
		Result->Name = Node.GetMorphTargetName(GenerationContext);
		
		// Target morph factor
		const UEdGraphPin* MorphFactorPin = Node.MorphFactorPinReference.Get();
		check(MorphFactorPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*MorphFactorPin))
		{
			Result->Factor = GenerateMutableSourceFloat(ConnectedPin, GenerationContext);
		}
		else
		{
			const FText Message = LOCTEXT("MutableSkeletalMeshMorph_MissingFactorConnection","Missing Morph Factor Connection. Morph Skeletal Mesh Nodes requires a factor to be used as the value for the morph.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
		}

		return Result;
	}


	Ptr<NodeSkeletalMeshReshape> GenerateMutableSourceSkeletalMeshReshape(const UCONodeSkeletalMeshReshape& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshReshape> Result = new NodeSkeletalMeshReshape();

		// Base skeletal mesh
		const UEdGraphPin* BasePin = Node.BasePinReference.Get();
		check(BasePin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BasePin))
		{
			Result->Base = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("MutableSkeletalMeshReshape_MissingBaseConnection", "Missing Base connection. Skeletal Mesh Reshape requires a base skeletal mesh.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
		}

		// Base shape
		const UEdGraphPin* BaseShapePin = Node.BaseShapePinReference.Get();
		check(BaseShapePin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*BaseShapePin))
		{
			Result->BaseShape = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("MutableSkeletalMeshReshape_MissingBaseShapeConnection", "Missing Base Shape connection. Skeletal Mesh Reshape requires a base shape skeletal mesh.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
		}

		// Target shape
		const UEdGraphPin* TargetShapePin = Node.TargetShapePinReference.Get();
		check(TargetShapePin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TargetShapePin))
		{
			Result->TargetShape = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
		}
		else
		{
			const FText Message = LOCTEXT("MutableSkeletalMeshReshape_MissingTargetShapeConnection", "Missing Target Shape connection. Skeletal Mesh Reshape requires a target shape skeletal mesh.");
			GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
		}

		Result->bReshapeVertices = Node.bReshapeVertices;
		Result->bRecomputeNormals = Node.bRecomputeNormals;
		Result->bApplyLaplacian = Node.bApplyLaplacianSmoothing;
		Result->bReshapeSkeleton = Node.bReshapePose;
		Result->bReshapePhysicsVolumes = Node.bReshapePhysics;

		EMeshReshapeVertexColorChannelUsage ChannelUsages[4] =
		{
			Node.VertexColorUsage.R,
			Node.VertexColorUsage.G,
			Node.VertexColorUsage.B,
			Node.VertexColorUsage.A
		};

		{
			int32 MaskWeightChannelNum = 0;
			for (int32 I = 0; I < 4; ++I)
			{
				if (ChannelUsages[I] == EMeshReshapeVertexColorChannelUsage::MaskWeight)
				{
					++MaskWeightChannelNum;
				}
			}

			if (MaskWeightChannelNum > 1)
			{
				for (int32 I = 0; I < 4; ++I)
				{
					if (ChannelUsages[I] == EMeshReshapeVertexColorChannelUsage::MaskWeight)
					{
						ChannelUsages[I] = EMeshReshapeVertexColorChannelUsage::None;
					}
				}

				const FText Message = LOCTEXT("MutableSkeletalMeshReshape_MultipleMaskWeights", "Only one color channel with mask weight usage is allowed. All mask weight usages disabled.");
				GenerationContext.Log(Message, &Node, EMessageSeverity::Warning);
			}
		}

		auto ConvertColorUsage = [](EMeshReshapeVertexColorChannelUsage Usage) -> EVertexColorUsage
		{
			switch (Usage)
			{
			case EMeshReshapeVertexColorChannelUsage::None:           return EVertexColorUsage::None;
			case EMeshReshapeVertexColorChannelUsage::RigidClusterId: return EVertexColorUsage::ReshapeClusterId;
			case EMeshReshapeVertexColorChannelUsage::MaskWeight:     return EVertexColorUsage::ReshapeMaskWeight;
			default: check(false); return EVertexColorUsage::None;
			};
		};

		Result->ColorRChannelUsage = ConvertColorUsage(ChannelUsages[0]);
		Result->ColorGChannelUsage = ConvertColorUsage(ChannelUsages[1]);
		Result->ColorBChannelUsage = ConvertColorUsage(ChannelUsages[2]);
		Result->ColorAChannelUsage = ConvertColorUsage(ChannelUsages[3]);

		Result->bReshapeSkeletonInvertSelection = Node.SelectionMethod == EBoneDeformSelectionMethod::ALL_BUT_SELECTED;
		Result->bReshapePhysicsVolumesInvertSelection = Node.PhysicsSelectionMethod == EBoneDeformSelectionMethod::ALL_BUT_SELECTED;

		Result->BonesToDeform = Node.BonesToDeform;
		Result->PhysicsToDeform = Node.PhysicsBodiesToDeform;

		return Result;
	}


	Ptr<NodeSkeletalMeshSwitch> GenerateMutableSourceSkeletalMeshSwitch(UCONodeSwitch& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshSwitch> Result = new NodeSkeletalMeshSwitch();

		// Check Switch Parameter arity preconditions.
		const UEdGraphPin* SwitchParameter = Node.SwitchParameterPinReference.Get();
		check(SwitchParameter);
		if (const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter))
		{
			const Ptr<NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

			// Switch Param not generated
			if (!SwitchParam)
			{
				// Warn about a failure.
				const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
				GenerationContext.Log(Message, &Node);

				return Result;
			}

			if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
			{
				const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
				GenerationContext.Log(Message, &Node);

				return Result;
			}

			{
				const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());
				if (!DoOptionsMatchEnum(Node, *EnumParameter))
				{
					const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
					GenerationContext.Log(Message, &Node);
					Node.SetRefreshNodeWarning();
				}
			}
			
			const int32 NumSwitchOptions = Node.SwitchPins.Num();

			Result->Parameter = SwitchParam;
			Result->Options.SetNum(NumSwitchOptions);

			for (int32 SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
			{
				const UEdGraphPin* SwitchPin = Node.SwitchPins[SelectorIndex].Get();
				check(SwitchPin);
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SwitchPin))
				{
					const Ptr<NodeSkeletalMesh> ChildNode = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
					Result->Options[SelectorIndex] = ChildNode;
				}
			}

			return Result;
		}
		else
		{
			GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), &Node);
			return Result;
		}
	}

	
	Ptr<NodeSkeletalMeshVariation> GenerateMutableSourceSkeletalMeshVariation(const UCONodeSkeletalMeshVariation& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshVariation> Result = new UE::Mutable::Private::NodeSkeletalMeshVariation();

		const UEdGraphPin* DefaultPin = Node.DefaultPin();
		check(DefaultPin);
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*DefaultPin))
		{
			if (Ptr<NodeSkeletalMesh> ChildNode = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options))
			{
				Result->DefaultSkeletalMesh = ChildNode;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MeshFailed", "Mesh generation failed."), &Node);
			}
		}

		const int32 NumVariations = Node.GetNumVariations();

		Result->Variations.SetNum(NumVariations);

		for (int32 VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			const UEdGraphPin* VariationPin = Node.VariationPin(VariationIndex);
			if (!VariationPin)
			{
				continue;
			}

			const FString VariationTag = Node.GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);
			Result->Variations[VariationIndex].Tag = StringCast<ANSICHAR>(*VariationTag).Get();

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
			{
				const Ptr<NodeSkeletalMesh> ChildNode = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
				Result->Variations[VariationIndex].SkeletalMesh = ChildNode;
			}
		}
		
		return Result;
	}

	
	Ptr<NodeSkeletalMeshConvert> GenerateMutableSourceSkeletalMeshParameter(const UCustomizableObjectNodeSkeletalMeshParameter& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshConvert> SkeletalMeshConvertNode = new NodeSkeletalMeshConvert();

		EMutableMeshConversionFlags ConversionFlags = GenerationContext.MeshGenerationFlags.Last();

		SkeletalMeshConvertNode->SkeletalMesh = GenerateMutableSourceSkeletalMeshObjectParameter(Node, GenerationContext);
		
		if (GenerationContext.CompilationContext->Options.bRealTimeMorphTargetsEnabled)
		{
			EnumAddFlags(ConversionFlags, EMutableMeshConversionFlags::AddMorphsAsRealTime); 
		}

		SkeletalMeshConvertNode->ConversionFlags = (uint8)ConversionFlags;

		return SkeletalMeshConvertNode;
	}
	

	Ptr<NodeSkeletalMeshTransformWithBone> GenerateMutableSkeletalMeshTransformWithBone(const UCONodeSkeletalMeshTransformWithBone& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshTransformWithBone> TransformNode = new NodeSkeletalMeshTransformWithBone();
		
		TransformNode->BoneName = FName(Node.BoneName);
		TransformNode->ThresholdFactor = Node.ThresholdFactor;
		
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.SkeletalMeshPin.Get()))
		{
			TransformNode->Source = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, Options);
		}
		
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Node.TransformPin.Get()))
		{
			TransformNode->MatrixNode = GenerateMutableSourceTransform(ConnectedPin, GenerationContext);
		}
		
		return TransformNode;
	}


	Ptr<NodeSkeletalMeshClipWithSkeletalMesh> GenerateMutableSourceSkeletalMeshClip(const UCONodeClipSkeletalMeshWithSkeletalMesh& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		Ptr<NodeSkeletalMeshClipWithSkeletalMesh> Result = new UE::Mutable::Private::NodeSkeletalMeshClipWithSkeletalMesh();

		// Base Mesh
		const UEdGraphPin* BaseSkeletalMeshPin = Node.BaseSkeletalMeshPin.Get();
		check(BaseSkeletalMeshPin);
		if (const UEdGraphPin* ConnectedSkeletalMeshPin = FollowInputPin(*BaseSkeletalMeshPin))
		{
			Result->SourceSkeletalMesh = GenerateMutableSourceSkeletalMesh(ConnectedSkeletalMeshPin, GenerationContext, Options);
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("ClipSkeletalMeshWithSkeletalMesh_Base_Mesh_missing", "This node requires an input base skeletal mesh.");
			GenerationContext.Log(ErrorMsg, &Node, EMessageSeverity::Warning);
		}

		// Clipping mesh
		const UEdGraphPin* ClippingSkeletalMeshPin = Node.ClipSkeletalMeshPin.Get();
		check(ClippingSkeletalMeshPin);
		if (const UEdGraphPin* ConnectedSkeletalMeshPin = FollowInputPin(*ClippingSkeletalMeshPin))
		{
			constexpr EMutableMeshConversionFlags ClipMeshFlags =
				EMutableMeshConversionFlags::IgnoreSkinning |
				EMutableMeshConversionFlags::IgnorePhysics |
				EMutableMeshConversionFlags::IgnoreMorphs |
				EMutableMeshConversionFlags::IgnoreTexCoords |
				EMutableMeshConversionFlags::IgnoreAUD;
				
			GenerationContext.MeshGenerationFlags.Push(ClipMeshFlags);
			
			Ptr<NodeSkeletalMesh> ClipMesh = GenerateMutableSourceSkeletalMesh(ConnectedSkeletalMeshPin, GenerationContext, Options);

		 	if (const FMatrix Matrix = Node.Transform.ToMatrixWithScale(); Matrix != FMatrix::Identity)
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSkeletalMeshTransform> TransformSkeletalMesh = new UE::Mutable::Private::NodeSkeletalMeshTransform();
				TransformSkeletalMesh->Source = ClipMesh;
			 
				TransformSkeletalMesh->Transform = FMatrix44f(Matrix);
				ClipMesh = TransformSkeletalMesh;
			}

			Result->ClipSkeletalMesh = ClipMesh;
			
			GenerationContext.MeshGenerationFlags.Pop();
		}
		else
		{
			const FText ErrorMsg = LOCTEXT("ClipSkeletalMeshWithSkeletalMesh_Clipping_Mesh_missing", "This node requires an input clipping skeletal mesh.");
			GenerationContext.Log(ErrorMsg, &Node, EMessageSeverity::Warning);
		}

		Result->FaceCullStrategy = Node.FaceCullStrategy;
		
		return Result;
	}
	

	Ptr<NodeSkeletalMesh> GenerateMutableSourceSkeletalMesh(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceSkeletalMesh);

		check(Pin)
		RETURN_ON_CYCLE(*Pin, GenerationContext)

		CheckNode(*Pin, GenerationContext);

		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());
		
		FGeneratedSourceSkeletalMeshKey Key;
		Key.Pin = Pin;
		Key.Options = Options;
		Key.MacroContext = GenerationContext.MacroNodesStack;

		Ptr<NodeSkeletalMesh> Result;
		
		if (const FGeneratedSourceSkeletalMeshData* Generated = GenerationContext.GeneratedSourceSkeletalMesh.Find(Key))
		{
			return Generated->Node;
		}
		
		if (const UCONodeSkeletalMeshMake_V2* MutableSkeletalMeshNew = Cast<UCONodeSkeletalMeshMake_V2>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshNew(*MutableSkeletalMeshNew, GenerationContext, Options);
		}
		else if (const UCONodeSkeletalMeshMerge* SkeletalMeshMergeNode = Cast<UCONodeSkeletalMeshMerge>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshMerge(*SkeletalMeshMergeNode, GenerationContext, Options);
		}
		else if (const UCONodeSkeletalMeshModify* SkeletalMeshModifyNode = Cast<UCONodeSkeletalMeshModify>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshModify(*SkeletalMeshModifyNode, GenerationContext, Options);
		}
		else if (const UCONodeSkeletalMeshMorph* SkeletalMeshMorphNode = Cast<UCONodeSkeletalMeshMorph>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshMorph(*SkeletalMeshMorphNode, GenerationContext, Options);
		}
		else if (const UCONodeSkeletalMeshReshape* SkeletalMeshReshapeNode = Cast<UCONodeSkeletalMeshReshape>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshReshape(*SkeletalMeshReshapeNode, GenerationContext, Options);
		}
		else if (UCONodeSwitch* SkeletalMeshSwitchNode = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh))
		{
			Result = GenerateMutableSourceSkeletalMeshSwitch(*SkeletalMeshSwitchNode, GenerationContext, Options);
		}
		else if (UCONodeSkeletalMeshVariation* SkeletalMeshVariationNode = Cast<UCONodeSkeletalMeshVariation>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshVariation(*SkeletalMeshVariationNode, GenerationContext, Options);
		}
		else if (UCustomizableObjectNodeSkeletalMeshParameter* SkeletalMeshParameterNode = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshParameter(*SkeletalMeshParameterNode, GenerationContext, Options);
		}
		else if (UCONodeClipSkeletalMeshWithSkeletalMesh* SkeletalMeshClipNode = Cast<UCONodeClipSkeletalMeshWithSkeletalMesh>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshClip(*SkeletalMeshClipNode, GenerationContext, Options);
		}
		else if (UCONodeSkeletalMeshTransformWithBone* SkeletalMeshTransformWithBone = Cast<UCONodeSkeletalMeshTransformWithBone>(Node))
		{
			Result = GenerateMutableSkeletalMeshTransformWithBone(*SkeletalMeshTransformWithBone, GenerationContext, Options);
		}
		else
		{
			GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
		}
		
		if (Result)
		{
			Result->SetMessageContext(Node);
		}

		FGeneratedSourceSkeletalMeshData Data;
		Data.Node = Result;
		
		GenerationContext.GeneratedSourceSkeletalMesh.Add(Key, Data);
		GenerationContext.GeneratedNodes.Add(Node);

		return Result;
	}
}


#undef LOCTEXT_NAMESPACE
