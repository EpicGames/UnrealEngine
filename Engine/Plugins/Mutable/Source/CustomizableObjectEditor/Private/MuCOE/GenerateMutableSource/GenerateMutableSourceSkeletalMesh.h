// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMesh.h"

struct FSourceSkeletalMeshOptions;
struct FMutableGraphGenerationContext;
class UEdGraphPin;

namespace UE::Mutable::Private
{
	Ptr<NodeSkeletalMesh> GenerateMutableSourceSkeletalMesh(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshOptions& Options);	
}
