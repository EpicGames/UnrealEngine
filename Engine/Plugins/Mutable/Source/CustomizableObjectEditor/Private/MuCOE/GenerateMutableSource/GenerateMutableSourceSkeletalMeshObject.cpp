// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateMutableSourceSkeletalMeshObject.h"

#include "GenerateMutableSource.h"
#include "GenerateMutableSourceSkeletalMesh.h"
#include "GenerateMutableSourceSurface.h"
#include "GenerateMutableSourceFloat.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuT/LODInfo.h"
#include "MuR/MutableTrace.h"
#include "MuT/NodeSkeletalMeshObjectConvert.h"
#include "MuT/NodeSkeletalMeshObjectParameter.h"
#include "MuT/NodeSkeletalMeshObjectSwitch.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


namespace UE::Mutable::Private
{
	Ptr<NodeSkeletalMeshObjectConvert> GenerateMutableSourceSkeletalMeshObjectConvert(const UCONodeSkeletalMeshObjectMake& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshObjectOptions& Options)
	{
		if (!Options.ReferenceSkeletalMesh)
		{
			GenerationContext.Log(LOCTEXT("NoReferenceMeshObjectTab", "Missing reference Skeletal Mesh. Check the component node connected to this node."), &Node, EMessageSeverity::Error);
			return nullptr;
		}
	
		if (!Options.ReferenceSkeletalMesh->GetSkeleton())
		{
			const FText Msg = FText::Format(LOCTEXT("NoReferenceSkeleton", "Missing skeleton in the reference mesh [{0}]. Check the component node connected to this node."), FText::FromString(GenerationContext.CustomizableObjectWithCycle->GetPathName()));
			GenerationContext.Log(Msg, &Node, EMessageSeverity::Error);
			return nullptr;
		}
	
		const FString PlatformName = GenerationContext.CompilationContext->Options.TargetPlatform->IniPlatformName();

		Ptr<NodeSkeletalMeshObjectConvert> Result = new NodeSkeletalMeshObjectConvert;

		const uint8 NumLODs = Node.NumLODs;
		
		uint8 FirstLODResident = 0;
		
		// Find the streaming settings for the target platform
		if (Node.LODSettings.bOverrideLODStreamingSettings)
		{
			if (Node.LODSettings.bEnableLODStreaming.GetValueForPlatform(*PlatformName))
			{
				FirstLODResident = Node.LODSettings.NumMaxStreamedLODs.GetValueForPlatform(*PlatformName);
			}
		}
		
		FirstLODResident = FMath::Clamp(FirstLODResident, 0, NumLODs - 1);
		
		Result->MinQualityLevelLODs = Node.LODSettings.MinQualityLevelLOD;

		// Strip quality levels for cook and when compiling for a platform different than the running platform (debugger)
		if (GenerationContext.CompilationContext->Options.bIsCooking ||
			!GenerationContext.CompilationContext->Options.TargetPlatform->IsRunningPlatform())
		{
			Result->MinQualityLevelLODs.StripQualityLevelForCooking(*PlatformName);
			Result->MinQualityLevelLODs.Default = Result->MinQualityLevelLODs.GetValueForPlatform(GenerationContext.CompilationContext->Options.TargetPlatform);
			Result->MinLODs.Default = Node.LODSettings.MinLOD.GetValueForPlatform(*PlatformName);
		}
		else
		{
			Result->MinQualityLevelLODs.Default = FMath::Min(Result->MinQualityLevelLODs.Default, Result->MinQualityLevelLODs.GetValueForPlatform(GenerationContext.CompilationContext->Options.TargetPlatform));
			Result->MinLODs = Node.LODSettings.MinLOD;
		}
		
		uint8 FirstLODAvailable = 0;

		// Find the MinLOD available for the target platform
		if (GEngine && GEngine->UseSkeletalMeshMinLODPerQualityLevels)
		{
			FirstLODAvailable = Result->MinQualityLevelLODs.Default;
		}
		else
		{
			FirstLODAvailable = Result->MinLODs.GetValueForPlatform(*PlatformName);
		}

		FirstLODAvailable = FMath::Clamp(FirstLODAvailable, 0, NumLODs - 1);

		// Adjust MinQualityLevelLODs to be higher than the FirstLODAvailable;
		Result->MinQualityLevelLODs.Default = static_cast<int32>(FirstLODAvailable);
		for (TTuple<int32, int32>& Pair : Result->MinQualityLevelLODs.PerQuality)
		{
			Pair.Value = FMath::Max(Pair.Value, static_cast<int32>(FirstLODAvailable));
		}

		Result->Name = FName(Node.GetSkeletalMeshName(&GenerationContext.MacroNodesStack));
		Result->NumLODs = NumLODs;
		Result->FirstLODResident = FirstLODResident;
		Result->FirstLODAvailable = FirstLODAvailable;

		Result->LODInfos.SetNum(NumLODs);
		for (int32 LODIndex = FirstLODAvailable; LODIndex < NumLODs; ++LODIndex)
		{
			FLODInfo& LODInfo = Result->LODInfos[LODIndex];
			
			const FSkeletalMeshLODInfo* ReferenceLODInfo = Options.ReferenceSkeletalMesh->GetLODInfo(LODIndex);
			if (ReferenceLODInfo)
			{
				// Copy LOD info data from the reference skeletal mesh
				LODInfo.ScreenSize = ReferenceLODInfo->ScreenSize.GetValueForPlatform(*PlatformName);
				LODInfo.LODHysteresis = ReferenceLODInfo->LODHysteresis;
				LODInfo.bSupportUniformlyDistributedSampling = ReferenceLODInfo->bSupportUniformlyDistributedSampling;
				LODInfo.bAllowCPUAccess = ReferenceLODInfo->bAllowCPUAccess;
			}
			else
			{
				LODInfo.ScreenSize = 0.3f / (LODIndex + 1);
				LODInfo.LODHysteresis = 0.02f;				
			}
		}
		
		if (UEdGraphPin* ConnectedPin = FollowInputPin(*Node.SkeletalMeshPin.Get()))
		{
			FSourceSkeletalMeshOptions SkeletalMeshOptions;
			SkeletalMeshOptions.NumLODs = NumLODs;
			SkeletalMeshOptions.FirstLODAvailable = FirstLODAvailable;
			
			Result->SkeletalMesh = GenerateMutableSourceSkeletalMesh(ConnectedPin, GenerationContext, SkeletalMeshOptions);
		}
		
		return Result;
	}
	
	
	Ptr<NodeSkeletalMeshObjectParameter> GenerateMutableSourceSkeletalMeshObjectParameter(const UCustomizableObjectNodeSkeletalMeshParameter& Node, FMutableGraphGenerationContext& GenerationContext)
	{
		Ptr<NodeSkeletalMeshObjectParameter> Result;

		const FGeneratedParameterKey Key = { Node.NodeGuid, Node.GetParameterName(&GenerationContext.MacroNodesStack) };
		if (const Ptr<NodeSkeletalMeshObjectParameter>* GeneratedSkeletalMeshParameter = GenerationContext.GeneratedSkeletalMeshParameters.Find(Key))
		{
			return GeneratedSkeletalMeshParameter->get();
		}
	
		if (const UCustomizableObjectNodeSkeletalMeshParameter* TypedNodeParam = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(&Node))
		{
			Ptr<NodeSkeletalMeshObjectParameter> SkeletalMeshNode = new NodeSkeletalMeshObjectParameter();

			SkeletalMeshNode->Name = TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack);
			SkeletalMeshNode->UID = GenerationContext.GetNodeIdUnique(&Node).ToString();
		
			GenerationContext.SkeletalMeshParameterDefaultValues.Add(TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack), LoadObject(TypedNodeParam->DefaultValue));

			GenerationContext.ParameterUIDataMap.Add(TypedNodeParam->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
				TypedNodeParam->ParamUIMetadata,
				EMutableParameterType::SkeletalMesh));

			Result = SkeletalMeshNode;
		}
		else
		{
			GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), &Node);
		}

		if (Result)
		{
			Result->SetMessageContext(&Node);
		}
	
		GenerationContext.GeneratedSkeletalMeshParameters.Add(Key, Result);
	
		return Result;
	}


	Ptr<NodeSkeletalMeshObjectSwitch> GenerateMutableSourceSkeletalMeshObjectSwitch(UCONodeSwitch& Node, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshObjectOptions& Options)
	{
		Ptr<NodeSkeletalMeshObjectSwitch> Result = new NodeSkeletalMeshObjectSwitch();

		// Check Switch Parameter arity preconditions.
		const UEdGraphPin* SwitchParameter = Node.SwitchParameterPinReference.Get();
		check(SwitchParameter);
		if (const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter))
		{
			const Ptr<NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

			// Switch Param not generated
			if (!SwitchParam)
			{
				// Warn about a failure.
				const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
				GenerationContext.Log(Message, &Node);

				return Result;
			}

			if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
			{
				const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
				GenerationContext.Log(Message, &Node);

				return Result;
			}

			{
				const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());
				if (!DoOptionsMatchEnum(Node, *EnumParameter))
				{
					const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
					GenerationContext.Log(Message, &Node);
					Node.SetRefreshNodeWarning();
				}
			}

			const int32 NumSwitchOptions = Node.SwitchPins.Num();

			Result->Parameter = SwitchParam;
			Result->Options.SetNum(NumSwitchOptions);

			for (int32 SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
			{
				const UEdGraphPin* SwitchPin = Node.SwitchPins[SelectorIndex].Get();
				check(SwitchPin);
				if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SwitchPin))
				{
					const Ptr<NodeSkeletalMeshObject> ChildNode = GenerateMutableSourceSkeletalMeshObject(ConnectedPin, GenerationContext, Options);
					Result->Options[SelectorIndex] = ChildNode;
				}
			}

			return Result;
		}
		else
		{
			GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), &Node);
			return Result;
		}
	}

	
	Ptr<NodeSkeletalMeshObject> GenerateMutableSourceSkeletalMeshObject(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshObjectOptions& Options)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceSkeletalMeshObject);

		check(Pin)
		RETURN_ON_CYCLE(*Pin, GenerationContext)

		CheckNode(*Pin, GenerationContext);

		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());
		
		FGeneratedSourceSkeletalMeshObjectKey Key;
		Key.Options = Options;
		Key.Pin = Pin;
		Key.MacroContext = GenerationContext.MacroNodesStack;

		Ptr<NodeSkeletalMeshObject> Result;
		
		if (const FGeneratedSourceSkeletalMeshObjectData* Generated = GenerationContext.GeneratedSourceSkeletalMeshObject.Find(Key))
		{
			return Generated->Node;
		}

		if (UCONodeSkeletalMeshObjectMake* CONodeSkeletalMeshMake = Cast<UCONodeSkeletalMeshObjectMake>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshObjectConvert(*CONodeSkeletalMeshMake, GenerationContext, Options);
		}
		else if (UCustomizableObjectNodeSkeletalMeshParameter* CONodeSkeletalMeshParameter = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(Node))
		{
			Result = GenerateMutableSourceSkeletalMeshObjectParameter(*CONodeSkeletalMeshParameter, GenerationContext);
		}
		else if (UCONodeSwitch* SkeletalMeshSwitchNode = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough))
		{
			Result = GenerateMutableSourceSkeletalMeshObjectSwitch(*SkeletalMeshSwitchNode, GenerationContext, Options);
		}
		else
		{
			GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
		}
		
		if (Result)
		{
			Result->SetMessageContext(Node);
		}

		FGeneratedSourceSkeletalMeshObjectData Data;
		Data.Node = Result;
		
		GenerationContext.GeneratedSourceSkeletalMeshObject.Add(Key, Data);
		GenerationContext.GeneratedNodes.Add(Node);

		return Result;
	}
}


#undef LOCTEXT_NAMESPACE
