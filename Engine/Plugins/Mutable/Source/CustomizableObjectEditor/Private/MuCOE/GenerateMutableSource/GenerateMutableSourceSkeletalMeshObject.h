// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshMerge.h"
#include "MuT/NodeSkeletalMeshNew.h"
#include "MuT/NodeSkeletalMeshObjectParameter.h"

struct FSourceSkeletalMeshObjectOptions;
class UCustomizableObjectNodeSkeletalMeshParameter;
struct FMutableGraphGenerationContext;
class UEdGraphPin;

namespace UE::Mutable::Private
{
	Ptr<NodeSkeletalMeshObject> GenerateMutableSourceSkeletalMeshObject(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext, const FSourceSkeletalMeshObjectOptions& Options);
	
	Ptr<NodeSkeletalMeshObjectParameter> GenerateMutableSourceSkeletalMeshObjectParameter(const UCustomizableObjectNodeSkeletalMeshParameter& Node, FMutableGraphGenerationContext& GenerationContext);
}
