// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceSurface.h"

#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Modules/ModuleManager.h"
#include "GPUSkinVertexFactory.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "MuCO/UnrealConversionUtils.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceFloat.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceGroupProjector.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMacro.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMaterial.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSectionVariation.h"
#include "MuCOE/Nodes/CONodeSwitch.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeMeshFormat.h"
#include "MuT/NodeSurfaceSwitch.h"
#include "MuT/NodeSurfaceVariation.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurface> GenerateMutableSourceSurface(const UEdGraphPin * Pin, FMutableGraphGenerationContext & GenerationContext)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateMutableSourceSurface);

	check(Pin)
	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());

	// Bool that determines if a node can be added to the cache of nodes.
	// Most nodes need to be added to the cache but there are some that don't. For exampel, MacroInstanceNodes
	bool bCacheNode = true;

	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceSurface), *Pin, *Node, GenerationContext, true);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeSurface*>(Generated->Node.get());
	}
	
	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}
	
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurface> Result;

	if (UCustomizableObjectNode* CustomObjNode = Cast<UCustomizableObjectNode>(Node))
	{
		if (CustomObjNode->IsNodeOutDatedAndNeedsRefresh())
		{
			CustomObjNode->SetRefreshNodeWarning();
		}
	}

	if (UCONodeSkeletalMeshSection* TypedNodeMat = Cast<UCONodeSkeletalMeshSection>(Node))
	{
		if (TypedNodeMat->MaxLOD != INDEX_NONE && TypedNodeMat->MaxLOD < GenerationContext.CurrentLOD)
		{
			return Result;
		}

		bool bGeneratingImplicitComponent = GenerationContext.ComponentMeshOverride.get() != nullptr;

		const UEdGraphPin* ConnectedMaterialPin = FollowInputPin(*TypedNodeMat->GetMeshPin());
		// Warn when texture connections are improperly used by connecting them directly to material inputs when no layout is used
		// TODO: delete the if clause and the warning when static meshes are operational again
		if (ConnectedMaterialPin)
		{
			if (const UEdGraphPin* StaticMeshPin = FindMeshBaseSource(*ConnectedMaterialPin, true, GenerationContext.MacroNodesStack))
			{
				const UCustomizableObjectNode* StaticMeshNode = CastChecked<UCustomizableObjectNode>(StaticMeshPin->GetOwningNode());
				GenerationContext.Log(LOCTEXT("UnsupportedStaticMeshes", "Static meshes are currently not supported as material meshes"), StaticMeshNode);
			}
		}

		UMaterialInterface* Material = TypedNodeMat->GetMaterial();
		if (!Material)
		{
			const FText Message = LOCTEXT("FailedToGenerateMeshSection", "Could not generate a mesh section because it didn't have a material selected. Please assign one and recompile.");
			GenerationContext.Log(Message, Node);
			Result = nullptr;

			return Result;
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurfaceNew> SurfNode = new UE::Mutable::Private::NodeSurfaceNew();
		Result = SurfNode;

		FGuid& SurfaceGuid = GenerationContext.SurfaceGuids.FindOrAdd(FSharedSurfaceKey(TypedNodeMat, GenerationContext.MacroNodesStack));
		if (!SurfaceGuid.IsValid())
		{
			SurfaceGuid = GenerationContext.GetNodeIdUnique(TypedNodeMat);
		}

		SurfNode->SurfaceGuid = SurfaceGuid;

		GenerationContext.CurrentReferencedMaterial = TStrongObjectPtr(Material);

		// Find reference mesh used to generate the surface metadata for this fragment.
		if (ConnectedMaterialPin)
		{
			//NOTE: This is the same is done in GenerateMutableSourceSurface. 
			if (const UEdGraphPin* SkeletalMeshPin = FindMeshBaseSource(*ConnectedMaterialPin, false, GenerationContext.MacroNodesStack))
			{
				TSoftObjectPtr<UStreamableRenderAsset> Mesh;
				
				int32 MetadataLODIndex, MetadataSectionIndex;
				MetadataLODIndex = MetadataSectionIndex = INDEX_NONE;

				if (const UCustomizableObjectNodeSkeletalMesh* SkeletalMeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(SkeletalMeshPin->GetOwningNode()))
				{
					Mesh = SkeletalMeshNode->GetMesh().ToSoftObjectPath();
					SkeletalMeshNode->GetPinSection(*SkeletalMeshPin, MetadataLODIndex, MetadataSectionIndex);
				}
				else if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(SkeletalMeshPin->GetOwningNode()))
				{
					Mesh = TableNode->GetColumnDefaultAssetByType<USkeletalMesh>(SkeletalMeshPin);
					TableNode->GetPinLODAndSection(SkeletalMeshPin, MetadataLODIndex, MetadataSectionIndex);
				}
				
				USkeletalMesh* MetadataBaseMesh = Cast<USkeletalMesh>(UE::Mutable::Private::LoadObject(Mesh));
				if (const FSkeletalMaterial* SkeletalMaterial = GetSkeletalMaterial(MetadataBaseMesh, MetadataLODIndex, MetadataSectionIndex))
				{
					SurfNode->Name =  SkeletalMaterial->MaterialSlotName;
				}
			}
		}

		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMesh> MeshNode;
		
		if (bGeneratingImplicitComponent)
		{
			MeshNode = GenerationContext.ComponentMeshOverride;
			SurfNode->Mesh = MeshNode;

			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMat->GetMeshPin()))
			{
				GenerationContext.Log(LOCTEXT("MeshIgnored", "The mesh nodes connected to a material node will be ignored because it is part of an explicit mesh component."), Node);
			}
		}
		else
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMat->GetMeshPin()))
			{
				// Flags to know which UV channels need layout
				FLayoutGenerationFlags LayoutGenerationFlags;
				
				LayoutGenerationFlags.TexturePinModes.Init(EPinMode::Default, TEXSTREAM_MAX_NUM_UVCHANNELS);

				const int32 NumImages = TypedNodeMat->GetNumParameters(EMaterialParameterType::Texture);
				for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
				{
					if (TypedNodeMat->IsImageMutableMode(ImageIndex))
					{
						const int32 UVChannel = TypedNodeMat->GetImageUVLayout(ImageIndex);
						if (LayoutGenerationFlags.TexturePinModes.IsValidIndex(UVChannel))
						{
							LayoutGenerationFlags.TexturePinModes[UVChannel] = EPinMode::Mutable;
						}
					}
				}

				GenerationContext.LayoutGenerationFlags.Push(LayoutGenerationFlags);

				FMutableSourceMeshData MeshData;
				MeshData.LODOffset = GenerationContext.CurrentLOD - GenerationContext.FromLOD;
				MeshNode = GenerateMutableSourceMesh(ConnectedPin, GenerationContext, MeshData, false, false);

				GenerationContext.LayoutGenerationFlags.Pop();

				if (!MeshNode)
				{
					GenerationContext.Log(LOCTEXT("MeshFailed", "Mesh generation failed."), Node);
				}
				else
				{
					SurfNode->Mesh = MeshNode;
				}
			}
		}

		// Generate the material of this surface. Mesh section nodes contain their own material.
		FGenerationMaterialOptions Options;
		Options.SurfaceNode = SurfNode;
		SurfNode->Material = GenerateMutableSourceMaterial(Pin, GenerationContext, Options);
		
		// Add the tags that this mesh section enables.
		for (const FString& Tag : TypedNodeMat->GetEnableTags(&GenerationContext.MacroNodesStack))
		{
			SurfNode->Tags.AddUnique(Tag);
		}

		SurfNode->Tags.AddUnique(TypedNodeMat->GetInternalTag());
	}

	else if (const UCONodeSkeletalMeshSectionVariation* TypedNodeVar = Cast<UCONodeSkeletalMeshSectionVariation>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurfaceVariation> SurfNode = new UE::Mutable::Private::NodeSurfaceVariation();
		Result = SurfNode;

		UE::Mutable::Private::NodeSurfaceVariation::VariationType muType = UE::Mutable::Private::NodeSurfaceVariation::VariationType::Tag;
		switch (TypedNodeVar->Type)
		{
		case ECustomizableObjectNodeMaterialVariationType::Tag: muType = UE::Mutable::Private::NodeSurfaceVariation::VariationType::Tag; break;
		case ECustomizableObjectNodeMaterialVariationType::State: muType = UE::Mutable::Private::NodeSurfaceVariation::VariationType::State; break;
		default:
			check(false);
			break;
		}
		SurfNode->Type = muType;

		for (const UEdGraphPin* ConnectedPin : FollowInputPinArray(*TypedNodeVar->DefaultPin()))
		{
			// Is it a modifier?
			UE::Mutable::Private::NodeSurfacePtr ChildNode = GenerateMutableSourceSurface(ConnectedPin, GenerationContext);
			if (ChildNode)
			{
				SurfNode->DefaultSurfaces.Add(ChildNode);
			}
			else
			{
				GenerationContext.Log(LOCTEXT("SurfaceFailed", "Surface generation failed."), Node);
			}
		}

		const int32 NumVariations = TypedNodeVar->GetNumVariations();
		SurfNode->Variations.SetNum(NumVariations);
		for (int VariationIndex = 0; VariationIndex < NumVariations; ++VariationIndex)
		{
			UE::Mutable::Private::NodeSurfacePtr VariationSurfaceNode;

			if (UEdGraphPin* VariationPin = TypedNodeVar->VariationPin(VariationIndex))
			{
				SurfNode->Variations[VariationIndex].Tag = TypedNodeVar->GetVariationTag(VariationIndex, &GenerationContext.MacroNodesStack);
				for (const UEdGraphPin* ConnectedPin : FollowInputPinArray(*VariationPin))
				{
					// Is it a modifier?
					UE::Mutable::Private::NodeSurfacePtr ChildNode = GenerateMutableSourceSurface(ConnectedPin, GenerationContext);
					if (ChildNode)
					{
						SurfNode->Variations[VariationIndex].Surfaces.Add( ChildNode );
					}
					else
					{
						GenerationContext.Log(LOCTEXT("SurfaceModifierFailed", "Surface generation failed."), Node);
					}
				}
			}
		}
	}

	else if (const UCONodeSwitch* TypedNodeSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_MeshSection))
	{
		// Using a lambda so control flow is easier to manage.
		Result = [&]()
		{
			const UEdGraphPin* SwitchParameter = TypedNodeSwitch->SwitchParameterPinReference.Get();

			// Check Switch Parameter arity preconditions.
			if (const UEdGraphPin* EnumPin = FollowInputPin(*SwitchParameter))
			{
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar> SwitchParam = GenerateMutableSourceFloat(EnumPin, GenerationContext);

				// Switch Param not generated
				if (!SwitchParam)
				{
					// Warn about a failure.
					const FText Message = LOCTEXT("FailedToGenerateSwitchParam", "Could not generate switch enum parameter. Please refresh the switch node and connect an enum.");
						GenerationContext.Log(Message, Node);

					return Result;
				}

				if (SwitchParam->GetType() != UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
				{
					const FText Message = LOCTEXT("WrongSwitchParamType", "Switch parameter of incorrect type.");
					GenerationContext.Log(Message, Node);

					return Result;
				}

				{
					const UE::Mutable::Private::NodeScalarEnumParameter* EnumParameter = static_cast<UE::Mutable::Private::NodeScalarEnumParameter*>(SwitchParam.get());

					if (!DoOptionsMatchEnum(*TypedNodeSwitch, *EnumParameter))
					{
						const FText Message = LOCTEXT("MismatchedSwitch", "Switch enum and switch node have different options. Please refresh the switch node to make sure the outcomes are labeled properly.");
						GenerationContext.Log(Message, Node);
						Node->SetRefreshNodeWarning();
					}
				}

				const int32 NumSwitchOptions = TypedNodeSwitch->SwitchPins.Num();
				
				UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurfaceSwitch> SwitchNode = new UE::Mutable::Private::NodeSurfaceSwitch;
				SwitchNode->Parameter = SwitchParam;
				SwitchNode->Options.SetNum(NumSwitchOptions);

				for (int32 SelectorIndex = 0; SelectorIndex < NumSwitchOptions; ++SelectorIndex)
				{
					if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeSwitch->SwitchPins[SelectorIndex].Get()))
					{
						UE::Mutable::Private::NodeSurfacePtr ChildNode = GenerateMutableSourceSurface(ConnectedPin, GenerationContext);
						if (ChildNode)
						{
							SwitchNode->Options[SelectorIndex] = ChildNode;
						}
						else
						{
							// Probably ok
							//GenerationContext.Log(LOCTEXT("SurfaceModifierFailed", "Surface generation failed."), Node);
						}
					}
				}

				Result = SwitchNode;
				return Result;
			}
			else
			{
				GenerationContext.Log(LOCTEXT("NoEnumParamInSwitch", "Switch nodes must have an enum switch parameter. Please connect an enum and refresh the switch node."), Node);
				return Result;
			}
		}(); // invoke lambda.
	}

	else if (const UCustomizableObjectNodeMacroInstance* TypedNodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeSurface>(*Pin, GenerationContext, GenerateMutableSourceSurface);
	}

	else if (const UCustomizableObjectNodeTunnel* TypedNodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		bCacheNode = false;
		Result = GenerateMutableSourceMacro<UE::Mutable::Private::NodeSurface>(*Pin, GenerationContext, GenerateMutableSourceSurface);
	}

	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}


	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	if (bCacheNode)
	{
		GenerationContext.Generated.Add(Key, FGeneratedData(Node, Result));
		GenerationContext.GeneratedNodes.Add(Node);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
