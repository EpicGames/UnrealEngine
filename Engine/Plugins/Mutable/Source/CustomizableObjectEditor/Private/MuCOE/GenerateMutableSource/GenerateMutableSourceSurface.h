// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "HAL/Platform.h"

class UEdGraphPin;
struct FMutableGraphGenerationContext;

enum class ECustomizableObjectNumBoneInfluences:uint8;
namespace UE::Mutable::Private
{
	class FMeshBufferSet;
	class NodeSurface;
}

extern UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeSurface> GenerateMutableSourceSurface(const UEdGraphPin* Pin, FMutableGraphGenerationContext & GenerationContext);

