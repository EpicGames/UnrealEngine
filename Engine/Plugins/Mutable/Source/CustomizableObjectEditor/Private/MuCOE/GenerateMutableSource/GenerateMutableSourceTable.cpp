// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GenerateMutableSource/GenerateMutableSourceTable.h"

#include "Animation/AnimInstance.h"
#include "Animation/PoseAsset.h"
#include "AssetRegistry/ARFilter.h"
#include "DataTableUtils.h"
#include "Engine/CompositeDataTable.h"
#include "Engine/StaticMesh.h"
#include "GameplayTagContainer.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectUIData.h"
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuCOE/CustomizableObjectCompiler.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSourceMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"
#include "MuCOE/GraphTraversal.h"
#include "MuR/Mesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "MuCOE/CustomizableObjectVersionBridge.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


bool FillTableColumn(const UCustomizableObjectNodeTable* TableNode, UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable, const FString& ColumnName, const FString& RowName, const uint32 RowId, uint8* CellData, const FProperty* ColumnProperty,
	const FMutableSourceMeshData& BaseMeshData, FMutableGraphGenerationContext& GenerationContext)
{
	int32 CurrentColumn;
	UDataTable* DataTablePtr = GetDataTable(TableNode, GenerationContext);
	check(DataTablePtr);
	
	FString ColumnPropertyName = ColumnProperty->GetAuthoredName();

	const bool bIgnoreMissingData = GenerationContext.CompilationContext->RootObject->GetPrivate()->bDisableTableMissingDataWarning;

	// Getting property type
	if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
	{
		FSoftObjectPtr SoftObject = SoftObjectProperty->GetPropertyValue(CellData);
		
		if (SoftObjectProperty->PropertyClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			// Generating an Empty cell
			
			// Getting reference Mesh
			USkeletalMesh* ReferenceSkeletalMesh = TableNode->GetColumnDefaultAssetByType<USkeletalMesh>(ColumnPropertyName);

			if (!ReferenceSkeletalMesh)
			{
				FString msg = FString::Printf(TEXT("Reference Skeletal Mesh not found for column [%s]."), *ColumnName);
				GenerationContext.Log(FText::FromString(msg), TableNode);

				return false;
			}

			FString MutableColumnName;

			{
				// Current LOD and Section 
				int32 LODIndex = INDEX_NONE;
				int32 SectionIndex = INDEX_NONE;
				GetLODAndSection(*GenerationContext.CompilationContext, BaseMeshData, *ReferenceSkeletalMesh, LODIndex, SectionIndex);

				MutableColumnName = TableNode->GenerateSkeletalMeshMutableColumName(ColumnName, LODIndex, SectionIndex);
			}

			CurrentColumn = MutableTable.get()->FindColumn(MutableColumnName);
			if (CurrentColumn == INDEX_NONE)
			{
				CurrentColumn = MutableTable->AddColumn(MutableColumnName, UE::Mutable::Private::ETableColumnType::Mesh);
			}

			if(SoftObject.IsNull())
			{
				UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> EmptySkeletalMesh = nullptr;
				MutableTable->SetCell(CurrentColumn, RowId, EmptySkeletalMesh);

				return true;
			}

			// Getting Animation information (Anim Blueprints, Animation Slot and Anim Tags)
			FString AnimBP, AnimSlot, GameplayTag, AnimBPAssetTag;
			TableNode->GetAnimationColumns(ColumnName, AnimBP, AnimSlot, GameplayTag);

			FName AnimBPSlotName;
			if (!AnimSlot.IsEmpty())
			{
				if (uint8* AnimRowData = DataTablePtr->FindRowUnchecked(FName(*RowName)))
				{
					// Getting animation slot row value from data table
					if (FProperty* AnimSlotProperty = TableNode->FindColumnProperty(FName(*AnimSlot)))
					{
						if (uint8* AnimSlotData = AnimSlotProperty->ContainerPtrToValuePtr<uint8>(AnimRowData, 0))
						{
							if (const FIntProperty* IntProperty = CastField<FIntProperty>(AnimSlotProperty))
							{
								FString Message = FString::Printf(
									TEXT("The column with name [%s] for the Anim Slot property should be an FName instead of an Integer, it will be internally converted to FName but should probaly be converted in the table itself."), 
									*AnimBP);
								GenerationContext.Log(FText::FromString(Message), TableNode, EMessageSeverity::Info);

								AnimBPSlotName = FName(FString::FromInt(IntProperty->GetPropertyValue(AnimSlotData)));
							}
							else if (const FNameProperty* NameProperty = CastField<FNameProperty>(AnimSlotProperty))
							{
								AnimBPSlotName = NameProperty->GetPropertyValue(AnimSlotData);
							}
						}
					}
				}
			}
			
			TSoftClassPtr<UAnimInstance> AnimInstance;
			if (!AnimBP.IsEmpty())
			{
				if (uint8* AnimRowData = DataTablePtr->FindRowUnchecked(FName(*RowName)))
				{
					// Getting animation instance soft class from data table
					if (FProperty* AnimBPProperty = TableNode->FindColumnProperty(FName(*AnimBP)))
					{
						if (uint8* AnimBPData = AnimBPProperty->ContainerPtrToValuePtr<uint8>(AnimRowData, 0))
						{
							if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(AnimBPProperty))
							{
								AnimInstance = SoftClassProperty->GetPropertyValue(AnimBPData).ToSoftObjectPath();
							}
						}
					}
				}
			}
			
			FGameplayTagContainer GameplayTags;
			if (!GameplayTag.IsEmpty())
			{
				if (DataTablePtr)
				{
					uint8* GameplayRowData = DataTablePtr->FindRowUnchecked(FName(*RowName));

					if (GameplayRowData)
					{
						// Getting animation tag row value from data table
						if (FProperty* GameplayTagProperty = TableNode->FindColumnProperty(FName(*GameplayTag)))
						{
							uint8* GameplayTagData = GameplayTagProperty->ContainerPtrToValuePtr<uint8>(GameplayRowData, 0);

							if (const FStructProperty* StructProperty = CastField<FStructProperty>(GameplayTagProperty))
							{
								if (StructProperty->Struct == TBaseStructure<FGameplayTagContainer>::Get())
								{
									if (GameplayTagData)
									{
										GameplayTags = *reinterpret_cast<FGameplayTagContainer*>(GameplayTagData);
									}
								}
							}
						}
					}
				}
			}
			
			// TODO: Add AnimBp physics to Tables.

			FMutableSourceMeshData Source = BaseMeshData;
			Source.Mesh = SoftObject.ToSoftObjectPath();
			Source.TableReferenceSkeletalMesh = ReferenceSkeletalMesh;
			Source.OptionalMessageInfo = FString::Printf(TEXT(" SkeletalMesh from column [%s] row [%s]"), *ColumnName, *RowName);
			Source.GameplayTags = GameplayTags;
			Source.AnimBPSlotName = AnimBPSlotName;
			Source.AnimInstance = AnimInstance;

			UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = GenerateMutableSkeletalMesh(Source, Source.BaseLODIndex, Source.BaseSectionIndex, GenerationContext, TableNode, Source.bOnlyConnectedLOD);

			if (MutableMesh)
			{
				MutableTable->SetCell(CurrentColumn, RowId, MutableMesh, TableNode);
			}
		}

		else if (SoftObjectProperty->PropertyClass->IsChildOf(UStaticMesh::StaticClass()))
		{
			UObject* Object = GenerationContext.LoadObject(SoftObject);

			UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object);
			if (!StaticMesh)
			{
				return false;
			}

			// Getting reference Mesh
			UStaticMesh* ReferenceStaticMesh = TableNode->GetColumnDefaultAssetByType<UStaticMesh>(ColumnPropertyName);

			if (!ReferenceStaticMesh)
			{
				FString msg = FString::Printf(TEXT("Reference Static Mesh not found for column [%s]."), *ColumnName);
				GenerationContext.Log(FText::FromString(msg), TableNode);

				return false;
			}

			// Parameter used for LOD differences
			int32 CurrentLOD = BaseMeshData.BaseLODIndex;

			int NumLODs = StaticMesh->GetRenderData()->LODResources.Num();

			if (NumLODs <= CurrentLOD)
			{
				CurrentLOD = NumLODs - 1;

				FString msg = FString::Printf(TEXT("Mesh from column [%s] row [%s] needs LOD %d but has less LODs than the reference mesh. LOD %d will be used instead. This can cause some performance penalties."),
					*ColumnName, *RowName, BaseMeshData.BaseLODIndex, CurrentLOD);
				LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, msg, RowName);
			}

			int32 NumMaterials = StaticMesh->GetRenderData()->LODResources[CurrentLOD].Sections.Num();
			int32 ReferenceNumMaterials = ReferenceStaticMesh->GetRenderData()->LODResources[CurrentLOD].Sections.Num();

			if (NumMaterials != ReferenceNumMaterials)
			{
				FString FirstTextOption = NumMaterials > ReferenceNumMaterials ? "more" : "less";
				FString SecondTextOption = NumMaterials > ReferenceNumMaterials ? "Some will be ignored" : "This can cause some compilation errors.";

				FString msg = FString::Printf(TEXT("Mesh from column [%s] row [%s] has %s Sections than the reference mesh. %s"), *ColumnName, *RowName, *FirstTextOption, *SecondTextOption);
				LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, msg, RowName);
			}

			FString MutableColumnName = TableNode->GenerateStaticMeshMutableColumName(ColumnName, BaseMeshData.BaseSectionIndex);

			CurrentColumn = MutableTable.get()->FindColumn(MutableColumnName);

			if (CurrentColumn == -1)
			{
				CurrentColumn = MutableTable->AddColumn(MutableColumnName, UE::Mutable::Private::ETableColumnType::Mesh);
			}

			constexpr bool bIsPassthrough = false;
			UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = GenerateMutableStaticMesh(StaticMesh, CurrentLOD, BaseMeshData.BaseSectionIndex, GenerationContext, TableNode);

			if (MutableMesh)
			{
				MutableTable->SetCell(CurrentColumn, RowId, MutableMesh, StaticMesh);
			}
			else
			{
				FString msg = FString::Printf(TEXT("Error converting static mesh LOD %d, Section %d from column [%s] row [%s] to mutable."),
					BaseMeshData.BaseLODIndex, BaseMeshData.BaseSectionIndex, *ColumnName, *RowName);
				LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, msg, RowName);
			}
		}

		else if (SoftObjectProperty->PropertyClass->IsChildOf(UTexture::StaticClass()))
		{
			UObject* Object = GenerationContext.LoadObject(SoftObject);
			UTexture* Texture = Cast<UTexture>(Object);

			if (!Texture)
			{
				Texture = TableNode->GetColumnDefaultAssetByType<UTexture>(ColumnPropertyName);

				if (!bIgnoreMissingData)
				{
					FString Message = Cast<UObject>(Object) ? "not a suported Texture" : "null";
					FString WarningMessage = FString::Printf(TEXT("Texture from column [%s] row [%s] is %s. The default texture will be used instead."), *ColumnName, *RowName, *Message);
					LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, WarningMessage, RowName);
				}
			}

			// There will be always one of the two options
			check(Texture);

			// Getting column index from column name
			CurrentColumn = MutableTable->FindColumn(ColumnName);

			if (CurrentColumn == INDEX_NONE)
			{
				CurrentColumn = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Image);
			}

			const bool bIsPassthroughTexture = TableNode->GetColumnImageMode(ColumnPropertyName) == ETableTextureType::PASSTHROUGH_TEXTURE;
			MutableTable->SetCell(CurrentColumn, RowId, GenerateImageConstant(Texture, GenerationContext, bIsPassthroughTexture));
		}

		else if (SoftObjectProperty->PropertyClass->IsChildOf(UMaterialInterface::StaticClass()))
		{
			UObject* Object = GenerationContext.LoadObject(SoftObject);

			// Get display name of the column of the data table (name showed in the table and struct editors)
			// Will be used in the warnings to help to identify a column with errors.
			FString MaterialColumnDisplayName = ColumnProperty->GetDisplayNameText().ToString();

			UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(Object);
			UMaterialInstance* ReferenceMaterial = TableNode->GetColumnDefaultAssetByType<UMaterialInstance>(ColumnPropertyName);

			if (!ReferenceMaterial)
			{
				FString msg = FString::Printf(TEXT("Default Material Instance not found for column [%s]."), *MaterialColumnDisplayName);
				GenerationContext.Log(FText::FromString(msg), TableNode);

				return false;
			}

			const bool bTableMaterialCheckDisabled = GenerationContext.CompilationContext->RootObject->GetPrivate()->IsTableMaterialsParentCheckDisabled();
			const bool bMaterialParentMismatch = !bTableMaterialCheckDisabled && MaterialInstance
												 && ReferenceMaterial->GetMaterial() != MaterialInstance->GetMaterial();

			if (bMaterialParentMismatch)
			{
				FText Warning = FText::Format(LOCTEXT("MatInstanceFromDifferentParent", "Material Instance from column [{0}] row [{1}] has a different Material Parent than the Default Material Instance. The Default Material Instance will be used instead."),
					FText::FromString(MaterialColumnDisplayName), FText::FromString(RowName));
				LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, Warning.ToString(), RowName);
				
				MaterialInstance = ReferenceMaterial;
			}

			if (!MaterialInstance)
			{
				if (!bIgnoreMissingData)
				{
					FText Warning;

					if (UMaterial* Material = Cast<UMaterial>(Object))
					{
						Warning = FText::Format(LOCTEXT("IsAMaterial", "Asset from column [{0}] row [{1}] is a Material and not a MaterialInstance. The default Material Instance will be used instead."),
							FText::FromString(MaterialColumnDisplayName), FText::FromString(RowName));
					}
					else
					{
						Warning = FText::Format(LOCTEXT("NullMaterialInstance", "Material Instance from column [{0}] row [{1}] is null. The default Material Instance will be used instead."),
							FText::FromString(MaterialColumnDisplayName), FText::FromString(RowName));
					}

					LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, Warning.ToString(), RowName);
				}

				MaterialInstance = ReferenceMaterial;
			}

			if (GenerationContext.CurrentTableColumnType == UE::Mutable::Private::ETableColumnType::Material)
			{
				CurrentColumn = MutableTable.get()->FindColumn(ColumnName);

				if (CurrentColumn == -1)
				{
					CurrentColumn = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Material);
				}

				int32 ReferenceMaterialId = GenerationContext.CompilationContext->PassthroughObjectFactory.Add(*MaterialInstance, false);
				MutableTable->SetCell(CurrentColumn, RowId, ReferenceMaterialId);

				return true;
			}


			// We get here if a mesh section node has the Table Material pin linked and a Texture pin set to Mutable but nothing linked to it.
			// This part of the code will generate a new Mutable Texture column with all the material instances textures specified in the current texture parameter (CurrentMaterialParameterId)

			int32 ColumnIndex;

			// Getting parameter value
			TArray<FMaterialParameterInfo> ParameterInfos;
			TArray<FGuid> ParameterGuids;

			MaterialInstance->GetMaterial()->GetAllParameterInfoOfType(EMaterialParameterType::Texture, ParameterInfos, ParameterGuids);
			
			FGuid ParameterId(GenerationContext.CurrentMaterialParameterId);
			int32 ParameterIndex = ParameterGuids.Find(ParameterId);

			if (ParameterIndex != INDEX_NONE && ParameterInfos[ParameterIndex].Name == GenerationContext.CurrentMaterialTableParameter)
			{
				// Getting column index from parameter name
				ColumnIndex = MutableTable->FindColumn(ColumnName);

				if (ColumnIndex == INDEX_NONE)
				{
					// If there is no column with the parameters name, we generate a new one
					ColumnIndex = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Image);
				}

				UTexture* ParentTextureValue = nullptr;
				MaterialInstance->GetMaterial()->GetTextureParameterValue(ParameterInfos[ParameterIndex], ParentTextureValue);
				
				UTexture2D* ParentParameterTexture = Cast<UTexture2D>(ParentTextureValue);
				if (!ParentParameterTexture)
				{
					FString ParamName = ParameterInfos[ParameterIndex].Name.ToString();
					FString Message = Cast<UObject>(ParentParameterTexture) ? "not a Texture2D" : "null";
					
					FString msg = FString::Printf(TEXT("Parameter [%s] from Default Material Instance of column [%s] is %s. This parameter will be ignored."), *ParamName, *MaterialColumnDisplayName, *Message);
					LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, msg, RowName);
					 
					 return false;
				}

				UTexture* TextureValue = nullptr;
				MaterialInstance->GetTextureParameterValue(ParameterInfos[ParameterIndex], TextureValue);

				UTexture2D* ParameterTexture = Cast<UTexture2D>(TextureValue);

				if (!ParameterTexture)
				{
					ParameterTexture = ParentParameterTexture;

					FString ParamName = GenerationContext.CurrentMaterialTableParameter;
					FString Message = Cast<UObject>(TextureValue) ? "not a Texture2D" : "null";

					FString msg = FString::Printf(TEXT("Parameter [%s] from material instance of column [%s] row [%s] is %s. The parameter texture of the default material will be used instead."), *ParamName, *MaterialColumnDisplayName, *RowName, *Message);
					LogRowGenerationMessage(TableNode, DataTablePtr, GenerationContext, msg, RowName);
				}

				bool bIsPassthroughTexture = false;
				MutableTable->SetCell(ColumnIndex, RowId, GenerateImageConstant(ParameterTexture, GenerationContext, bIsPassthroughTexture));

				return true;
			}
		}

		else if (SoftObjectProperty->PropertyClass->IsChildOf(UPoseAsset::StaticClass()))
		{
			UObject* Object = GenerationContext.LoadObject(SoftObject);
			
			if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(Object))
			{
				CurrentColumn = MutableTable.get()->FindColumn(ColumnName);

				if (CurrentColumn == -1)
				{
					CurrentColumn = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Mesh);
				}

				TArray<FName> ArrayBoneName;
				TArray<FTransform> ArrayTransform;
				UCustomizableObjectNodeAnimationPose::StaticRetrievePoseInformation(PoseAsset, TableNode->PoseReferenceMesh, ArrayBoneName, ArrayTransform);

				UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FMesh> MutableMesh = 
						UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();

				UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FSkeleton> MutableSkeleton = 
						UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FSkeleton>();

				MutableMesh->SetSkeleton(MutableSkeleton);
				MutableMesh->SetBonePoseCount(ArrayBoneName.Num());

				MutableSkeleton->BoneNames.Reserve(ArrayBoneName.Num());
				MutableSkeleton->BoneParents.Reserve(ArrayBoneName.Num());

				for (int32 i = 0; i < ArrayBoneName.Num(); ++i)
				{
					constexpr int16 ParentIndex = -1; // Skeletons from poses don't need bone parents?

					UE::Mutable::Private::FBoneIdOrIndex MutableBoneIndex;
					MutableBoneIndex.Index = MutableSkeleton->AddBone(ArrayBoneName[i], ParentIndex);
					MutableMesh->SetBonePose(i, MutableBoneIndex, (FTransform3f)ArrayTransform[i], UE::Mutable::Private::EBoneUsageFlags::Skinning);
				}

				MutableTable->SetCell(CurrentColumn, RowId, MutableMesh);
			}
		}

		else
		{
			// Unsuported Variable Type
			FString msg = FString::Printf(TEXT("[%s] is not a supported class for mutable nodes."), *SoftObjectProperty->PropertyClass.GetName());
			GenerationContext.Log(FText::FromString(msg), TableNode);

			return false;
		}
	}

	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ColumnProperty))
	{
		if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
		{
			CurrentColumn = MutableTable->FindColumn(ColumnName);

			if (CurrentColumn == INDEX_NONE)
			{
				CurrentColumn = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Color);
			}

			// Setting cell value
			FLinearColor Value = *(FLinearColor*)CellData;
			MutableTable->SetCell(CurrentColumn, RowId, Value);
		}
		
		else
		{
			// Unsuported Variable Type
			return false;
		}
	}

	else if (const FNumericProperty* FloatNumProperty = CastField<FFloatProperty>(ColumnProperty))
	{
		CurrentColumn = MutableTable->FindColumn(ColumnName);

		if (CurrentColumn == INDEX_NONE)
		{
			CurrentColumn = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Scalar);
		}

		// Setting cell value
		float Value = FloatNumProperty->GetFloatingPointPropertyValue(CellData);
		MutableTable->SetCell(CurrentColumn, RowId, Value);
	}

	else if (const FNumericProperty* DoubleNumProperty = CastField<FDoubleProperty>(ColumnProperty))
	{
		CurrentColumn = MutableTable->FindColumn(ColumnName);
	
		if (CurrentColumn == INDEX_NONE)
		{
			CurrentColumn = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Scalar);
		}
	
		// Setting cell value
		float Value = DoubleNumProperty->GetFloatingPointPropertyValue(CellData);
		MutableTable->SetCell(CurrentColumn, RowId, Value);
	}

	else
	{
		// Unsuported Variable Type
		return false;
	}

	return true;
}


FName GetAnotherOption(FName SelectedOptionName, const TArray<FName>& RowNames)
{
	for (const FName& CandidateOption : RowNames)
	{
		if (CandidateOption != SelectedOptionName)
		{
			return CandidateOption;
		}
	}

	return FName("None");
}


void RestrictRowNamesToSelectedOption(TArray<FName>& InOutRowNames, const UCustomizableObjectNodeTable& TableNode, FMutableGraphGenerationContext& GenerationContext)
{
	if (!GenerationContext.CompilationContext->Options.ParamNamesToSelectedOptions.IsEmpty())
	{
		FMutableParamNameSet* ParamNameSet = GenerationContext.TableToParamNames.Find(TableNode.Table->GetPathName());

		if (ParamNameSet && !ParamNameSet->ParamNames.IsEmpty())
		{
			TSet<FName> SelectedOptionNames;

			for (const FString& ParamName : ParamNameSet->ParamNames)
			{
				// If the param is in the map restrict to only the selected option
				const FString* SelectedOptionString = GenerationContext.CompilationContext->Options.ParamNamesToSelectedOptions.Find(ParamName);

				if (SelectedOptionString)
				{
					if (*SelectedOptionString != FString("None")) // The None option will always be added in the following compilation step.
					{
						SelectedOptionNames.Add(FName(*SelectedOptionString));
					}
				}
			}

			if (!SelectedOptionNames.IsEmpty())
			{
				bool bRowNamesContainsSelectedOptionName = false;

				for (const FName& OptionName : SelectedOptionNames)
				{
					if (InOutRowNames.Contains(OptionName))
					{
						bRowNamesContainsSelectedOptionName = true;
						break;
					}
				}

				if (bRowNamesContainsSelectedOptionName)
				{
					InOutRowNames.Empty(SelectedOptionNames.Num());

					for (const FName& OptionName : SelectedOptionNames)
					{
						InOutRowNames.Add(OptionName);
					}
				}
				else
				{
					InOutRowNames.Empty(0);
				}
			}
		}
	}
}


void RestrictRowContentByVersion( TArray<FName>& InOutRowNames, const UDataTable& DataTable, const UCustomizableObjectNodeTable& TableNode, FMutableGraphGenerationContext& GenerationContext)
{
	FProperty* ColumnProperty = TableNode.FindColumnProperty(TableNode.VersionColumn);

	if (!ColumnProperty)
	{
		return;
	}

	ICustomizableObjectVersionBridgeInterface* CustomizableObjectVersionBridgeInterface = Cast<ICustomizableObjectVersionBridgeInterface>(GenerationContext.RootVersionBridge);
	if (!CustomizableObjectVersionBridgeInterface)
	{
		const FString Message = "Found a data table with at least a row with a Custom Version asset but the Root Object does not have a Version Bridge asset assigned.";
		GenerationContext.Log(FText::FromString(Message), &TableNode, EMessageSeverity::Error);
		return;
	}

	TArray<FName> OutRowNames;
	OutRowNames.Reserve(InOutRowNames.Num());

	for (int32 RowIndex = 0; RowIndex < InOutRowNames.Num(); ++RowIndex)
	{
		if (uint8* CellData = UCustomizableObjectNodeTable::GetCellData(InOutRowNames[RowIndex], DataTable, *ColumnProperty))
		{
			if (!CustomizableObjectVersionBridgeInterface->IsVersionPropertyIncludedInCurrentRelease(*ColumnProperty, CellData))
			{
				continue;
			}

			OutRowNames.Add(InOutRowNames[RowIndex]);
		}
	}

	InOutRowNames = OutRowNames;
}


void GenerateUniqueRowIds(const TArray<FName>& RowNames, TArray<uint32>& OutRowIds)
{
	const int32 NumRows = RowNames.Num();

	OutRowIds.SetNum(NumRows);

	for (int32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
	{
		const FString& RowName = RowNames[RowIndex].ToString();

		uint32 RowId = CityHash32(reinterpret_cast<const char*>(*RowName), RowName.Len() * sizeof(FString::ElementType));

		// Ensure Row Id is unique 
		bool bIsUnique = false;
		while (!bIsUnique)
		{
			bIsUnique = true;
			for (int32 RowIdIndex = 0; RowIdIndex < RowIndex; ++RowIdIndex)
			{
				if (OutRowIds[RowIdIndex] == RowId)
				{
					bIsUnique = false;
					++RowId;
					break;
				}
			}
		}

		OutRowIds[RowIndex] = RowId;
	}
}


TArray<FName> GetRowsToCompile(const UDataTable& DataTable, const UCustomizableObjectNodeTable& TableNode, FMutableGraphGenerationContext& GenerationContext, TArray<uint32>& OutRowIds)
{
	FGeneratedMutableDataTableKey MutableTableKey(DataTable.GetName(), TableNode.VersionColumn, TableNode.CompilationFilterOptions);
	if (FMutableGraphGenerationContext::FGeneratedDataTablesData* Result = GenerationContext.GeneratedTables.Find(MutableTableKey))
	{
		OutRowIds = Result->RowIds;
		return Result->RowNames;
	}
	else
	{
		TArray<FName> RowNames = TableNode.GetEnabledRows(DataTable);

		if (!RowNames.IsEmpty())
		{
			RestrictRowNamesToSelectedOption(RowNames, TableNode, GenerationContext);
			RestrictRowContentByVersion(RowNames, DataTable, TableNode, GenerationContext);
		}

		GenerateUniqueRowIds(RowNames, OutRowIds);

		return RowNames;
	}
}


bool GenerateTableColumn(const UCustomizableObjectNodeTable* TableNode, const UEdGraphPin* Pin, UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable, const FString& DataTableColumnName, const FProperty* ColumnProperty, 
	const FMutableSourceMeshData& BaseMeshData, FMutableGraphGenerationContext& GenerationContext)
{
	MUTABLE_CPUPROFILER_SCOPE(GenerateTableColumn);

	if (!TableNode)
	{
		return false;
	}

	UDataTable* DataTable = GetDataTable(TableNode, GenerationContext);

	if (!DataTable || !DataTable->GetRowStruct())
	{
		return false;
	}

	// Getting names of the rows to access the information
	TArray<uint32> RowIds;
	TArray<FName> RowNames = GetRowsToCompile(*DataTable, *TableNode, GenerationContext, RowIds);

	// Variable to check if something failed during a Cell generation
	bool bCellGenerated = true;
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	check(Schema);

	// At some point we may want more default values types
	TSet<FName> ValidNonePinTypes({ Schema->PC_Color, Schema->PC_Material });
	
	// Only set the "None" value of Material and Color columns
	if (ValidNonePinTypes.Contains(Pin->PinType.PinCategory))
	{
		bCellGenerated = GenerateNoneRow(*TableNode, Pin, DataTableColumnName, MutableTable, GenerationContext);
	}

	// Set the value of each row
	for (int32 RowIndex = 0; RowIndex < RowNames.Num() && bCellGenerated; ++RowIndex)
	{
		if (uint8* CellData = UCustomizableObjectNodeTable::GetCellData(RowNames[RowIndex], *DataTable, *ColumnProperty))
		{
			bCellGenerated = FillTableColumn(TableNode, MutableTable, DataTableColumnName, RowNames[RowIndex].ToString(), RowIds[RowIndex], CellData, ColumnProperty,
				BaseMeshData, GenerationContext);

			// Stop the compilation if something fails
			if (!bCellGenerated)
			{
				return false;
			}
		}
	}

	return true;
}


void GenerateTableParameterUIData(const UDataTable* DataTable, const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext)
{
	TArray<uint32> RowIds;
	TArray<FName> RowNames = GetRowsToCompile(*DataTable, *TableNode, GenerationContext, RowIds);

	for (const FName& Name : RowNames)
	{
		FIntegerParameterOptionDataTable& Data = GenerationContext.IntParameterOptionDataTable.FindOrAdd({ TableNode->ParameterName, Name.ToString() });
		Data.DataTables.Add(TSoftObjectPtr<UDataTable>(const_cast<UDataTable*>(DataTable)));
	}
	
	// Generating Parameter UI MetaData if not exists
	if (!GenerationContext.ParameterUIDataMap.Contains(TableNode->ParameterName))
	{
		FMutableParameterData ParameterUIData(TableNode->ParamUIMetadata, EMutableParameterType::Int);
		ParameterUIData.IntegerParameterGroupType = TableNode->bAddNoneOption ? ECustomizableObjectGroupType::COGT_ONE_OR_NONE : ECustomizableObjectGroupType::COGT_ONE;
		
		FMutableParameterData& ParameterUIDataRef = GenerationContext.ParameterUIDataMap.Add(TableNode->ParameterName, ParameterUIData);
		FProperty* MetadataColumnProperty = TableNode->FindColumnProperty(TableNode->ParamUIMetadataColumn);
		bool bIsValidMetadataColumn = MetadataColumnProperty &&
			CastField<FStructProperty>(MetadataColumnProperty) &&
			CastField<FStructProperty>(MetadataColumnProperty)->Struct == FMutableParamUIMetadata::StaticStruct();

		// Trigger warning only if the name is different than "None"
		if (!TableNode->ParamUIMetadataColumn.IsNone() && !bIsValidMetadataColumn)
		{
			FText LogMessage = FText::Format(LOCTEXT("InvalidParamUIMetadataColumn_Warning",
				"UI Metadata Column [{0}] is not a valid type or does not exist in the Structure of the Node."), FText::FromName(TableNode->ParamUIMetadataColumn));
			GenerationContext.Log(LogMessage, TableNode);
		}

		FProperty* ThumbnailColumnProperty = TableNode->FindColumnProperty(TableNode->ThumbnailColumn);
		bool bIsValidThumbnailColumn = ThumbnailColumnProperty && CastField<FSoftObjectProperty>(ThumbnailColumnProperty);

		// Trigger warning only if the name is different than "None"
		if (!TableNode->ThumbnailColumn.IsNone() && !bIsValidThumbnailColumn)
		{
			FText LogMessage = FText::Format(LOCTEXT("InvalidThumbnailColumn_Warning",
				"Thumbnail Column [{0}] is not an objet type or does not exist in the Structure of the Node."), FText::FromName(TableNode->ThumbnailColumn));
			GenerationContext.Log(LogMessage, TableNode);
		}

		if (!bIsValidMetadataColumn)
		{
			return;
		}

		for (int32 NameIndex = 0; NameIndex < RowNames.Num(); ++NameIndex)
		{
			FName RowName = RowNames[NameIndex];

			if (uint8* MetadataCellData = UCustomizableObjectNodeTable::GetCellData(RowName, *DataTable, *MetadataColumnProperty))
			{
				FMutableParamUIMetadata MetadataValue = *reinterpret_cast<FMutableParamUIMetadata*>(MetadataCellData);

				FIntegerParameterUIData IntegerMetadata = FIntegerParameterUIData(MetadataValue);

				// Add thumbnail
				if (bIsValidThumbnailColumn && MetadataValue.EditorUIThumbnailObject.IsNull())
				{
					if (uint8* ThumbnailCellData = UCustomizableObjectNodeTable::GetCellData(RowName, *DataTable, *ThumbnailColumnProperty))
					{
						FSoftObjectPtr* ObjectPtr = reinterpret_cast<FSoftObjectPtr*>(ThumbnailCellData);
						IntegerMetadata.ParamUIMetadata.EditorUIThumbnailObject = ObjectPtr->ToSoftObjectPath();
					}
				}

				// Add Tags
				if (TableNode->bGatherTags)
				{
					if (const UScriptStruct* Struct = DataTable->GetRowStruct())
					{
						for (TFieldIterator<FProperty> It(Struct); It; ++It)
						{
							FProperty* ColumnProperty = *It;

							if (!ColumnProperty)
							{
								continue;
							}

							if (const FStructProperty* StructProperty = CastField<FStructProperty>(ColumnProperty))
							{
								if (StructProperty->Struct == TBaseStructure<FGameplayTagContainer>::Get())
								{
									if (uint8* TagCellData = UCustomizableObjectNodeTable::GetCellData(RowName, *DataTable, *ColumnProperty))
									{
										FGameplayTagContainer* TagContainer = reinterpret_cast<FGameplayTagContainer*>(TagCellData);
										IntegerMetadata.ParamUIMetadata.GameplayTags.AppendTags(*TagContainer);
									}
								}
							}
						}
					}
				}

				ParameterUIDataRef.ArrayIntegerParameterOption.Add(RowName.ToString(), IntegerMetadata);
			}
		}
	}
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> GenerateMutableSourceTable(const UDataTable* DataTable, const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext)
{
	check(DataTable && TableNode);

	if (GenerationContext.CompilationContext->Options.ParamNamesToSelectedOptions.IsEmpty())
	{
		FMutableParamNameSet* ParamNameSet = GenerationContext.TableToParamNames.Find(DataTable->GetPathName());

		if (!ParamNameSet)
		{
			ParamNameSet = &GenerationContext.TableToParamNames.Add(DataTable->GetPathName());
		}

		ParamNameSet->ParamNames.Add(TableNode->ParameterName);
	}

	// Checking if the table is in the cache
	const FString TableName = DataTable->GetName();

	FGeneratedMutableDataTableKey MutableTableKey(TableName, TableNode->VersionColumn, TableNode->CompilationFilterOptions);
	if (FMutableGraphGenerationContext::FGeneratedDataTablesData* CachedTable = GenerationContext.GeneratedTables.Find(MutableTableKey))
	{
		// Generating Parameter Metadata for parameters that reuse a Table
		GenerateTableParameterUIData(DataTable, TableNode, GenerationContext);

		return CachedTable->GeneratedTable;
	}

	UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable = new UE::Mutable::Private::FTable();

	if (const UScriptStruct* TableStruct = DataTable->GetRowStruct())
	{
		// Getting Table and row names to access the information
		TArray<uint32> RowIds;
		TArray<FName> RowNames = GetRowsToCompile(*DataTable, *TableNode, GenerationContext, RowIds);

		// Check if the generated table will contain anything
		if (RowNames.IsEmpty())
		{
			FText EmptyTableMessage;

			if (DataTable->GetRowNames().IsEmpty())
			{
				EmptyTableMessage = FText::Format(LOCTEXT("EmptyDataTable_Error",
					"Node Table [{0}] has no rows to compile."), FText::FromString(TableNode->ParameterName));
			}
			else
			{
				EmptyTableMessage = FText::Format(LOCTEXT("AllDataTableRowsFiltered_Error",
					"All data table rows from the node [{0}] have been filtered and nothing will be compiled. Please review the compilation filters of the node."), FText::FromString(TableNode->ParameterName));
			}

			GenerationContext.Log(EmptyTableMessage, TableNode);

			return nullptr;
		}

		// Adding the Name Column
		MutableTable->AddColumn("Name", UE::Mutable::Private::ETableColumnType::String);

		// Always generate "None" row
		{
			FString NoneRowName = "None";
			uint32 RowId = CityHash32(reinterpret_cast<const char*>(*NoneRowName), NoneRowName.Len() * sizeof(FString::ElementType));
			MutableTable->AddRow(RowId);
			MutableTable->SetCell(0, RowId, NoneRowName);
		}

		// Adding name rows
		for (int32 RowIndex = 0; RowIndex < RowNames.Num(); ++RowIndex)
		{
			MutableTable->AddRow(RowIds[RowIndex]);
			MutableTable->SetCell(0, RowIds[RowIndex], RowNames[RowIndex].ToString());
		}

		// Generating Parameter Metadata for new table parameters
		GenerateTableParameterUIData(DataTable, TableNode, GenerationContext);

		// Generating data for Table Cache
		FMutableGraphGenerationContext::FGeneratedDataTablesData GeneratedTable;
		GeneratedTable.GeneratedTable = MutableTable;
		GeneratedTable.RowNames = RowNames;
		GeneratedTable.RowIds = RowIds;
		GeneratedTable.ReferenceNode = TableNode;

		// Add table to cache
		GenerationContext.GeneratedTables.Add(MutableTableKey, GeneratedTable);
	}
	else
	{
		FString msg = "Couldn't find the Data Table's Struct asset in the Node.";
		GenerationContext.Log(FText::FromString(msg), DataTable);
		
		return nullptr;
	}

	return MutableTable;
}


UDataTable* GetDataTable(const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext)
{
	UDataTable* OutDataTable = nullptr;

	if (TableNode->TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry)
	{
		OutDataTable = GenerateDataTableFromStruct(TableNode, GenerationContext);
	}
	else
	{
		OutDataTable = UE::Mutable::Private::LoadObject(TableNode->Table);
	}

	return OutDataTable;
}


UDataTable* GenerateDataTableFromStruct(const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext)
{
	TObjectPtr<UScriptStruct> Structure = UE::Mutable::Private::LoadObject(TableNode->Structure);

	if (!Structure)
	{
		GenerationContext.Log(LOCTEXT("EmptyStructureError", "Empty structure asset."), TableNode);
		return nullptr;
	}

	FMutableGraphGenerationContext::FGeneratedCompositeDataTablesData DataTableData;
	DataTableData.ParentStruct = Structure;
	DataTableData.FilterPaths = TableNode->FilterPaths;
	
	//Checking cache of generated data tables
	int32 DataTableIndex = GenerationContext.GeneratedCompositeDataTables.Find(DataTableData);
	if (DataTableIndex != INDEX_NONE)
	{
		// DataTable already generated
		UCompositeDataTable* GeneratedDataTable = GenerationContext.GeneratedCompositeDataTables[DataTableIndex].GeneratedDataTable;
		return Cast<UDataTable>(GeneratedDataTable);
	}

	if (TableNode->FilterPaths.IsEmpty())
	{
		// Preventing load all data tables of the project
		GenerationContext.Log(LOCTEXT("NoFilePathsError", "There are no filter paths selected. This is an error to prevent loading all data table of the project."), TableNode);

		return nullptr;
	}
	
	TArray<FAssetData> DataTableAssets = TableNode->GetParentTables();

	UCompositeDataTable* CompositeDataTable = NewObject<UCompositeDataTable>();
	CompositeDataTable->RowStruct = Structure;

	TArray<UDataTable*> ParentTables;

	for (const FAssetData& DataTableAsset : DataTableAssets)
	{
		if (DataTableAsset.IsValid())
		{
			if (UDataTable* DataTable = Cast<UDataTable>(UE::Mutable::Private::LoadObject(DataTableAsset)))
			{
				ParentTables.Add(DataTable);
			}
		}
	}

	if (ParentTables.IsEmpty())
	{
		GenerationContext.Log(LOCTEXT("NoDataTablesFoundWarning", "Could not find a data table with the specified struct in the selected paths."), TableNode);

		return nullptr;
	}

	// Map to find the original data table of a row
	TMap<FName, TArray<UDataTable*>> OriginalTableRowsMap;

	// Set to iterate faster the repeated rows inside the map
	TSet<FName> RepeatedRowNamesArray;

	// Checking if a row name is repeated in several tables
	for (int32 ParentIndx = 0; ParentIndx < ParentTables.Num(); ++ParentIndx)
	{
		const TArray<FName>& RowNames = ParentTables[ParentIndx]->GetRowNames();

		for (const FName& RowName : RowNames)
		{
			TArray<UDataTable*>* DataTablesNames = OriginalTableRowsMap.Find(RowName);

			if (DataTablesNames == nullptr)
			{
				TArray<UDataTable*> ArrayTemp;
				ArrayTemp.Add(ParentTables[ParentIndx]);
				OriginalTableRowsMap.Add(RowName, ArrayTemp);
			}
			else
			{
				DataTablesNames->Add(ParentTables[ParentIndx]);
				RepeatedRowNamesArray.Add(RowName);
			}
		}
	}

	for (const FName& RowName : RepeatedRowNamesArray)
	{
		const TArray<UDataTable*>& DataTablesNames = OriginalTableRowsMap[RowName];

		FString TableNames;

		for (int32 NameIndx = 0; NameIndx < DataTablesNames.Num(); ++NameIndx)
		{
			TableNames += DataTablesNames[NameIndx]->GetName();

			if (NameIndx + 1 < DataTablesNames.Num())
			{
				TableNames += ", ";
			}
		}

		FString Message = FString::Printf(TEXT("Row with name [%s] repeated in the following Data Tables: [%s]. The last row processed will be used [%s]."),
			*RowName.ToString(), *TableNames, *DataTablesNames.Last()->GetName());
		GenerationContext.Log(FText::FromString(Message), TableNode);
	}

	CompositeDataTable->AppendParentTables(ParentTables);

	// Adding Generated Data Table to the cache
	DataTableData.GeneratedDataTable = CompositeDataTable;
	GenerationContext.GeneratedCompositeDataTables.Add(DataTableData);
	GenerationContext.CompositeDataTableRowToOriginalDataTableMap.Add(CompositeDataTable, OriginalTableRowsMap);
	
	return Cast<UDataTable>(CompositeDataTable);
}


void LogRowGenerationMessage(const UCustomizableObjectNodeTable* TableNode, const UDataTable* DataTable, FMutableGraphGenerationContext& GenerationContext, const FString& Message, const FString& RowName)
{
	FString FinalMessage = Message;

	if (TableNode->TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry)
	{
		TMap<FName, TArray<UDataTable*>>* ParameterDataTableMap = GenerationContext.CompositeDataTableRowToOriginalDataTableMap.Find(DataTable);

		if (ParameterDataTableMap)
		{
			TArray<UDataTable*>* DataTables = ParameterDataTableMap->Find(FName(*RowName));

			if (DataTables)
			{
				FString TableNames;

				for (int32 NameIndx = 0; NameIndx < DataTables->Num(); ++NameIndx)
				{
					TableNames += (*DataTables)[NameIndx]->GetName();

					if (NameIndx + 1 < DataTables->Num())
					{
						TableNames += ", ";
					}
				}

				FinalMessage += " Row from Composite Data Table, original Data Table/s: " + TableNames;
			}
		}
	}

	GenerationContext.Log(FText::FromString(FinalMessage), TableNode);
}


bool GenerateNoneRow(const UCustomizableObjectNodeTable& TableNode, const UEdGraphPin* Pin, const FString& ColumnName, UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable, FMutableGraphGenerationContext& GenerationContext)
{
	if (!Pin || !MutableTable)
	{
		FText Message = FText::Format(LOCTEXT("NoneRowWarning_NullPinTable","Error creating [None] row for Table Node {0}. Null pin or Table."), FText::FromString(TableNode.ParameterName));
		GenerationContext.Log(Message, &TableNode, EMessageSeverity::Error);

		return false;
	}

	FString NoneRowName = "None";
	uint32 RowId = CityHash32(reinterpret_cast<const char*>(*NoneRowName), NoneRowName.Len() * sizeof(FString::ElementType));
	int32 ColumnId = MutableTable->FindColumn(ColumnName);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	check(Schema);

	if (Pin->PinType.PinCategory == Schema->PC_Material && GenerationContext.CurrentTableColumnType == UE::Mutable::Private::ETableColumnType::Material)
	{
		// Add a scalar column if it has not been added yet
		if (ColumnId == INDEX_NONE)
		{
			ColumnId = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Material);
		}

		int32 MaterialId = GenerationContext.CompilationContext->PassthroughObjectFactory.Add(*GenerationContext.CurrentReferencedMaterial.Get(), false);
		MutableTable->SetCell(ColumnId, RowId, MaterialId);
	}
	else if (Pin->PinType.PinCategory == Schema->PC_Color)
	{
		// Add a color column if it has not been added yet
		if (ColumnId == INDEX_NONE)
		{
			ColumnId = MutableTable->AddColumn(ColumnName, UE::Mutable::Private::ETableColumnType::Color);
		}

		// HACK: Encoding an invalid value (Nan) for table option "None".
		// Using Nan avoids that color operations modify this encoded number since any operation returns Nan.
		// Also, QuietNaNs do not trigger errors nor checks.
		// It's checked in the moment that the material of the COI is generated.
		FLinearColor InvalidColor(std::numeric_limits<float>::quiet_NaN(), 0.0f, 0.0f);
		MutableTable->SetCell(ColumnId, RowId, TableNode.bUseMaterialColor ? InvalidColor : FLinearColor::Black);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
