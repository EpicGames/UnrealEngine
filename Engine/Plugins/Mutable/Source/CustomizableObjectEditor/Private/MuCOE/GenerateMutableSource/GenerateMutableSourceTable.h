// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Table.h"

class FProperty;
class FString;
class UCustomizableObjectNodeTable;
class UDataTable;
class UEdGraphPin;
struct FMutableGraphGenerationContext;
struct FMutableSourceMeshData;


/** 
 *
 * @param TableNode 
 * @param Pin 
 * @param MutableTable 
 * @param DataTableColumnName 
 * @param TableProperty 
 * @param BaseMeshData
 * @param GenerationContext 
 * @return  */
bool GenerateTableColumn(const UCustomizableObjectNodeTable* TableNode, const UEdGraphPin* Pin, UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable, const FString& DataTableColumnName, const FProperty* ColumnProperty,
	const FMutableSourceMeshData& BaseMeshData,
	FMutableGraphGenerationContext& GenerationContext);


/**
 *
 * @param TableNode 
 * @param MutableTable 
 * @param ColumnName 
 * @param TableProperty 
 * @param RowName 
 * @param RowIdx 
 * @param CellData 
 * @param Property 
 * @param BaseMeshData 
 * @param GenerationContext 
 * @return  */
bool FillTableColumn(const UCustomizableObjectNodeTable* TableNode, UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable, const FString& ColumnName,
	const FString& RowName, uint32 RowId, uint8* CellData, const FProperty* ColumnProperty,
	const FMutableSourceMeshData& BaseMeshData,	FMutableGraphGenerationContext& GenerationContext);


UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> GenerateMutableSourceTable(const UDataTable* DataTable, const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext);

/** Generates a new FParameterUIData for a new table parameter. Also, adds the UIData to the parameters map. */
void GenerateTableParameterUIData(const UDataTable* DataTable, const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext);

/** Gets the data table needed during the compilation process */
UDataTable* GetDataTable(const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext);

/** Gets all the rows of a data table that are going to be compiled. Some rows can be disabled with a bool column or an asset version system 
	@OutRowIds Deterministic Ids for each row. Used to Add, Get, and Set cell data
	@return Array with all the names of the rows that are going to be compiled */
TArray<FName> GetRowsToCompile(const UDataTable& DataTable, const UCustomizableObjectNodeTable& TableNode, FMutableGraphGenerationContext& GenerationContext, TArray<uint32>& OutRowIds);

/** Generates a Data Table from the Data Tables referenced in a Script Struct */
UDataTable* GenerateDataTableFromStruct(const UCustomizableObjectNodeTable* TableNode, FMutableGraphGenerationContext& GenerationContext);

/** Generates and populates the "None" row for each column. */
bool GenerateNoneRow(const UCustomizableObjectNodeTable& TableNode, const UEdGraphPin* Pin, const FString& ColumnName, UE::Mutable::Private::Ptr<UE::Mutable::Private::FTable> MutableTable, FMutableGraphGenerationContext& GenerationContext);

/** This method adds the original Data Table(s) of the processed row at the end of the log message when the data table is a Composite Data table. */
void LogRowGenerationMessage(const UCustomizableObjectNodeTable* TableNode, const UDataTable* DataTable, FMutableGraphGenerationContext& GenerationContext, const FString& Message, const FString& RowName);
