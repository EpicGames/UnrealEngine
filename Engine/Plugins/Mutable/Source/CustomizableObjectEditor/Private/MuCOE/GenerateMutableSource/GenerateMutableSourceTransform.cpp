// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateMutableSourceTransform.h"

#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTransformConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTransformParameter.h"
#include "MuT/NodeMatrixConstant.h"
#include "MuT/NodeMatrixParameter.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMatrix> GenerateMutableSourceTransform(const UEdGraphPin* Pin, FMutableGraphGenerationContext& GenerationContext)
{
	check(Pin);

	RETURN_ON_CYCLE(*Pin, GenerationContext)

	CheckNode(*Pin, GenerationContext);

	UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(Pin->GetOwningNode());
	const FGeneratedKey Key(reinterpret_cast<void*>(&GenerateMutableSourceTransform), *Pin, *Node, GenerationContext);
	if (const FGeneratedData* Generated = GenerationContext.Generated.Find(Key))
	{
		return static_cast<UE::Mutable::Private::NodeMatrix*>(Generated->Node.get());
	}

	if (Node->IsNodeOutDatedAndNeedsRefresh())
	{
		Node->SetRefreshNodeWarning();
	}

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMatrix> Result;
	if (const UCustomizableObjectNodeTransformConstant* TypedNodeTransformConstant = Cast<UCustomizableObjectNodeTransformConstant>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMatrixConstant> MatrixNode = new UE::Mutable::Private::NodeMatrixConstant{};
		Result = MatrixNode;
		MatrixNode->Value = FMatrix44f(TypedNodeTransformConstant->Value.ToMatrixWithScale());
	}
	else if (const UCustomizableObjectNodeTransformParameter* TypedNodeTransformParameter = Cast<UCustomizableObjectNodeTransformParameter>(Node))
	{
		UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMatrixParameter> MatrixNode = new UE::Mutable::Private::NodeMatrixParameter();
		Result = MatrixNode;
		
		MatrixNode->Name = TypedNodeTransformParameter->GetParameterName(&GenerationContext.MacroNodesStack);
		MatrixNode->Uid = GenerationContext.GetNodeIdUnique(Node).ToString();
		MatrixNode->DefaultValue = FMatrix44f(TypedNodeTransformParameter->DefaultValue.ToMatrixWithScale());

		GenerationContext.ParameterUIDataMap.Add(TypedNodeTransformParameter->GetParameterName(&GenerationContext.MacroNodesStack), FMutableParameterData(
			TypedNodeTransformParameter->ParamUIMetadata,
			EMutableParameterType::Transform));
	}
	else
	{
		GenerationContext.Log(LOCTEXT("UnimplementedNode", "Node type not implemented yet."), Node);
	}
	
	FGeneratedData CacheData = FGeneratedData(Node, Result);
	GenerationContext.Generated.Add(Key, CacheData);
	GenerationContext.GeneratedNodes.Add(Node);
	
	if (Result)
	{
		Result->SetMessageContext(Node);
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
