// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/GraphTraversal.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExposePin.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSectionVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackApplication.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackDefinition.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshape.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"

#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeReroute.h"
#include "Nodes/CONodeExternalOperation.h"
#include "Nodes/CONodeClipMeshWithMesh.h"
#include "Nodes/CONodeRemoveMesh.h"
#include "Nodes/CONodeTransformWithBone.h"
#include "Nodes/CONodeTransformInMesh.h"
#include "Nodes/CONodeSwitch.h"
#include "UObject/UObjectIterator.h"


TArray<UEdGraphPin*> FollowPinArray(const UEdGraphPin& Pin, bool bIgnoreOrphan, bool* bOutCycleDetected)
{
	MUTABLE_CPUPROFILER_SCOPE(FollowPinArray)
	
	bool bCycleDetected = false;
		
	TArray<UEdGraphPin*> Result;

	// Early out optimization
	if (Pin.LinkedTo.IsEmpty())
	{
		if (bOutCycleDetected)
		{
			*bOutCycleDetected = bCycleDetected;	
		}

		return Result;
	}
	
	TSet<const UEdGraphPin*> Visited;
	Visited.Reserve(32);

	TArray<const UEdGraphPin*> PinsToVisit;
	PinsToVisit.Reserve(32);
	
	PinsToVisit.Add(&Pin);
	while (PinsToVisit.Num())
	{
		const UEdGraphPin& CurrentPin = *PinsToVisit.Pop();

		if (bIgnoreOrphan && IsPinOrphan(CurrentPin))
		{
			continue;
		}

		Visited.FindOrAdd(&CurrentPin, &bCycleDetected);
		if (bCycleDetected)
		{
			continue;
		}

		for (UEdGraphPin* LinkedPin : CurrentPin.LinkedTo)
		{
			if (bIgnoreOrphan && IsPinOrphan(*LinkedPin))
			{
				continue;
			}

			if (const UCustomizableObjectNodeReroute* NodeReroute = Cast<UCustomizableObjectNodeReroute>(LinkedPin->GetOwningNodeUnchecked()))
			{
				PinsToVisit.Add(Pin.Direction == EGPD_Input ? NodeReroute->GetInputPin() : NodeReroute->GetOutputPin());
			}
			else if (const UCustomizableObjectNodeExternalPin* ExternalPinNode = Cast<UCustomizableObjectNodeExternalPin>(LinkedPin->GetOwningNodeUnchecked()))
			{
				check(Pin.Direction == EGPD_Input);
				
				if (const UCustomizableObjectNodeExposePin* LinkedNode = ExternalPinNode->GetNodeExposePin())
				{
					const UEdGraphPin* ExposePin = LinkedNode->InputPin();
					check(ExposePin);
					PinsToVisit.Add(ExposePin);
				}
			}
			else if (const UCustomizableObjectNodeExposePin* ExposePinNode = Cast<UCustomizableObjectNodeExposePin>(LinkedPin->GetOwningNodeUnchecked()))
			{
				check(Pin.Direction == EGPD_Output);

				for (TObjectIterator<UCustomizableObjectNodeExternalPin> It; It; ++It)
				{
					const UCustomizableObjectNodeExternalPin* LinkedNode = *It;
					
					if (LinkedNode &&
						LinkedNode->GetNodeExposePin() == ExposePinNode)
					{
						const UEdGraphPin* ExternalPin = LinkedNode->GetExternalPin();
						check(ExternalPin);
						PinsToVisit.Add(ExternalPin);
					}
				}
			}
			else
			{
				Result.Add(LinkedPin);
			}
		}
	}

	if (bOutCycleDetected)
	{
		*bOutCycleDetected = bCycleDetected;	
	}

	return Result;
}


TArray<UEdGraphPin*> FollowInputPinArray(const UEdGraphPin& Pin, bool* bOutCycleDetected)
{
	check(Pin.Direction == EGPD_Input);
	return FollowPinArray(Pin, true, bOutCycleDetected);
}


UEdGraphPin* FollowInputPin(const UEdGraphPin& Pin, bool* CycleDetected)
{
	TArray<UEdGraphPin*> Result = FollowInputPinArray(Pin, CycleDetected);
	check(Result.Num() <= 1); // Use FollowInputPinArray if the pin can have more than one input.

	if (!Result.IsEmpty())
	{
		return Result[0];
	}
	else
	{
		return nullptr;
	}
}


TArray<UEdGraphPin*> FollowOutputPinArray(const UEdGraphPin& Pin, bool* bOutCycleDetected)
{
	check(Pin.Direction == EGPD_Output);
	return FollowPinArray(Pin, true, bOutCycleDetected);
}


UEdGraphPin* FollowOutputPin(const UEdGraphPin& Pin, bool* CycleDetected)
{
	TArray<UEdGraphPin*> Result = FollowOutputPinArray(Pin, CycleDetected);
	check(Result.Num() <= 1); // Use FollowInputPinArray if the pin can have more than one input.

	if (!Result.IsEmpty())
	{
		return Result[0];
	}
	else
	{
		return nullptr;
	}
}


TArray<UEdGraphPin*> ReverseFollowPinArray(const UEdGraphPin& Pin, bool bIgnoreOrphan, bool* bOutCycleDetected)
{
	bool bCycleDetected = false;
		
	TArray<UEdGraphPin*> Result;
	
	TSet<const UEdGraphPin*> Visited;

	TArray<UEdGraphPin*> PinsToVisit;
	PinsToVisit.Add(const_cast<UEdGraphPin*>(&Pin));
	while (PinsToVisit.Num())
	{
		UEdGraphPin& CurrentPin = *PinsToVisit.Pop();

		if (!bIgnoreOrphan && IsPinOrphan(CurrentPin))
		{
			continue;
		}

		Visited.FindOrAdd(&CurrentPin, &bCycleDetected);
		if (bCycleDetected)
		{
			continue;
		}
		
		if (const UCustomizableObjectNodeExposePin* ExposePinNode = Cast<UCustomizableObjectNodeExposePin>(CurrentPin.GetOwningNodeUnchecked()))
		{
			check(Pin.Direction == EGPD_Input);

			for (TObjectIterator<UCustomizableObjectNodeExternalPin> It; It; ++It)
			{
				const UCustomizableObjectNodeExternalPin* LinkedNode = *It;

				if (IsValid(LinkedNode) &&
					!LinkedNode->IsTemplate() &&
					LinkedNode->GetNodeExposePin() == ExposePinNode)
				{
					const UEdGraphPin* ExternalPin = LinkedNode->GetExternalPin();
					check(ExternalPin);

					for (UEdGraphPin* LinkedPin : ExternalPin->LinkedTo)
					{
						PinsToVisit.Add(LinkedPin);
					}
				}
			}
		}
		else if (const UCustomizableObjectNodeExternalPin* ExternalPinNode = Cast<UCustomizableObjectNodeExternalPin>(CurrentPin.GetOwningNodeUnchecked()))
		{
			check(Pin.Direction == EGPD_Output);
				
			if (const UCustomizableObjectNodeExposePin* LinkedNode = ExternalPinNode->GetNodeExposePin())
			{
				const UEdGraphPin* ExposePin = LinkedNode->InputPin();
				check(ExposePin);

				for (UEdGraphPin* LinkedPin : ExposePin->LinkedTo)
				{
					PinsToVisit.Add(LinkedPin);
				}
			}
		}
		else if (const UCustomizableObjectNodeReroute* NodeReroute = Cast<UCustomizableObjectNodeReroute>(CurrentPin.GetOwningNodeUnchecked()))
		{
			UEdGraphPin* ReroutePin = Pin.Direction == EGPD_Output ? NodeReroute->GetInputPin() : NodeReroute->GetOutputPin();

			for (UEdGraphPin* LinkedPin : ReroutePin->LinkedTo)
			{
				PinsToVisit.Add(LinkedPin);
			}
		}
		else
		{
			if (bIgnoreOrphan || !IsPinOrphan(CurrentPin))
			{
				Result.Add(&CurrentPin);
			}
		}
	}
	
	if (bOutCycleDetected)
	{
		*bOutCycleDetected = bCycleDetected;	
	}
	
	return Result;
}


UCustomizableObjectNodeObject* GetRootNode(const UCustomizableObject* Object)
{
	if (!Object)
	{
		return nullptr;
	}
	
	for (UEdGraphNode* Node : Object->GetPrivate()->GetSource()->Nodes)
	{
		if (UCustomizableObjectNodeObject* NodeObject = Cast<UCustomizableObjectNodeObject>(Node))
		{
			if (NodeObject->bIsBase)
			{
				return NodeObject;
			}
		}
	}

	return nullptr;
}


bool GetParentsUntilRoot(const UCustomizableObject* Object, TArray<UCustomizableObjectNodeObject*>& ArrayNodeObject, TArray<const  UCustomizableObject*>& ArrayCustomizableObject)
{
	UCustomizableObjectNodeObject* Root = GetRootNode(Object);

	bool bSuccess = true;

	if (Root)
	{
		if (!ArrayCustomizableObject.Contains(Object))
		{
			ArrayNodeObject.Add(Root);
			ArrayCustomizableObject.Add(Object);
		}
		else
		{
			// This object has already been visited which means that there is a Cycle between Customizable Objects
			return false;
		}

		if (Root->ParentObject != nullptr)
		{
			bSuccess = GetParentsUntilRoot(Root->ParentObject, ArrayNodeObject, ArrayCustomizableObject);
		}
	}

	return bSuccess;
}


bool HasCandidateAsParent(UCustomizableObjectNodeObject* Node, UCustomizableObject* ParentCandidate)
{
	if (Node->ParentObject == ParentCandidate)
	{
		return true;
	}

	if (Node->ParentObject != nullptr)
	{
		UCustomizableObjectNodeObject* ParentNodeObject = GetRootNode(Node->ParentObject);

		if (ParentNodeObject->ParentObject)
		{
			return HasCandidateAsParent(ParentNodeObject, ParentCandidate);
		}
		else
		{
			return false;
		}
	}

	return false;
}


UCustomizableObject* GetFullGraphRootObject(const UCustomizableObjectNodeObject* Node, TArray<const UObject*>& VisitedObjects)
{
	if (Node->ParentObject != nullptr)
	{
		VisitedObjects.Add(Node->ParentObject);

		UCustomizableObjectNodeObject* Root = GetRootNode(Node->ParentObject);

		if (Root->ParentObject == nullptr)
		{
			return Node->ParentObject;
		}
		else
		{
			if (VisitedObjects.Contains(Root->ParentObject))
			{
				//There is a cycle
				return nullptr;
			}
			else
			{
				return GetFullGraphRootObject(Root, VisitedObjects);
			}
		}
	}

	return nullptr;
}




UCustomizableObject* GraphTraversal::GetObject(const UCustomizableObjectNode& Node)
{
	if (Node.IsInMacro())
	{
		return nullptr;
	}

	return CastChecked<UCustomizableObject>(Node.GetGraph()->GetOuter());
}


UCustomizableObject* GraphTraversal::GetRootObject(UCustomizableObject* ChildObject, TArray<const UObject*>* OutVisitedObjects)
{
	const UCustomizableObject* ConstChildObject = ChildObject;
	return const_cast<UCustomizableObject*>(GetRootObject(ConstChildObject, OutVisitedObjects));
}


const UCustomizableObject* GraphTraversal::GetRootObject(const UCustomizableObject* ChildObject, TArray<const UObject*>* OutVisitedObjects)
{
	// Grab a node to start the search -> Get the root since it should be always present
	UCustomizableObjectNodeObject* ObjectRootNode = GetRootNode(ChildObject);

	if (ObjectRootNode && ObjectRootNode->ParentObject)
	{
		if (!OutVisitedObjects)	
		{
			TArray<const UObject*> VisitedObjects;
			return GetFullGraphRootObject(ObjectRootNode, VisitedObjects);
		}
		else
		{
			return GetFullGraphRootObject(ObjectRootNode, *OutVisitedObjects);
		}
	}

	// No parent object found, return input as the parent of the graph
	// This can also mean the ObjectRootNode does not exist because it has not been opened yet (so no nodes have been generated)
	return ChildObject;
}


void RecursiveVisitNodes(UCustomizableObjectNode& CurrentNode, const TFunction<void(UCustomizableObjectNode&)>& VisitFunction,
	TArray<const UCustomizableObjectNodeMacroInstance*>& MacroContext, TSet<TPair<UCustomizableObjectNode*, TArray<const UCustomizableObjectNodeMacroInstance*>>>& VisitedNodes, const TMultiMap<FGuid, UCustomizableObjectNodeObject*>& ObjectGroupMap)
{
	// Check if we already visited this node
	if (VisitedNodes.Contains({&CurrentNode, MacroContext}))
	{
		return;
	}

	// Add it to the cache of already visited nodes
	VisitedNodes.Add({ &CurrentNode, MacroContext});

	// Visit the current Node
	VisitFunction(CurrentNode);

	// Iterate through all nodes linked to the current node
	for (UEdGraphPin* Pin : CurrentNode.GetAllNonOrphanPins())
	{
		if (Pin->Direction != EGPD_Input)
		{
			continue;
		}

		for (UEdGraphPin* ConnectedPin : FollowInputPinArray(*Pin))
		{
			UEdGraphNode* ConnectedNode = ConnectedPin->GetOwningNode();

			if (UCustomizableObjectNodeObjectGroup* ObjectGroupNode = Cast<UCustomizableObjectNodeObjectGroup>(ConnectedNode))
			{
				// Visit the pins directly connected to the ObjectGroup node such as the child objects and projectors
				RecursiveVisitNodes(*ObjectGroupNode, VisitFunction, MacroContext, VisitedNodes, ObjectGroupMap);

				// Visit the child objects in other COs
				TArray<UCustomizableObjectNodeObject*> ChildObjectNodes;
				ObjectGroupMap.MultiFind(ObjectGroupNode->NodeGuid, ChildObjectNodes);

				for (UCustomizableObjectNode* ChildObjectNode : ChildObjectNodes)
				{
					RecursiveVisitNodes(*ChildObjectNode, VisitFunction, MacroContext, VisitedNodes, ObjectGroupMap);
				}
			}

			else if (UCustomizableObjectNodeMacroInstance* MacroInstanceNode = Cast<UCustomizableObjectNodeMacroInstance>(ConnectedNode))
			{
				// First visit the graph of the Macro
				if (const UEdGraphPin* OutputPin = MacroInstanceNode->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, ConnectedPin->PinName))
				{
					if (UCustomizableObjectNode* OutputNode = Cast<UCustomizableObjectNode>(OutputPin->GetOwningNode()))
					{
						MacroContext.Push(MacroInstanceNode);
						RecursiveVisitNodes(*OutputNode, VisitFunction, MacroContext, VisitedNodes, ObjectGroupMap);
						MacroContext.Pop();
					}
				}

				// Continue visiting from the Macro
				RecursiveVisitNodes(*MacroInstanceNode, VisitFunction, MacroContext, VisitedNodes, ObjectGroupMap);
			}

			else if (UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(ConnectedNode))
			{
				RecursiveVisitNodes(*Node, VisitFunction, MacroContext, VisitedNodes, ObjectGroupMap);
			}
		}
	}
}


void GraphTraversal::VisitNodes(UCustomizableObjectNode& StartNode, const TFunction<void(UCustomizableObjectNode&)>& VisitFunction, const TMultiMap<FGuid, UCustomizableObjectNodeObject*>* ObjectGroupMap, TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext)
{
	// Set to keep tracking of the visited nodes. We use the macro context to re-visit macros that can be instantiated more than once.
	TSet<TPair<UCustomizableObjectNode*, TArray<const UCustomizableObjectNodeMacroInstance*>>> VisitedNodes;
	
	TMultiMap<FGuid, UCustomizableObjectNodeObject*> LocalObjectGroupMap;
	TArray<const UCustomizableObjectNodeMacroInstance*> LocalMacroContext;

	if (!ObjectGroupMap)
	{
		LocalObjectGroupMap = GetNodeGroupObjectNodeMapping(GetObject(StartNode));
		ObjectGroupMap = &LocalObjectGroupMap;
	}

	RecursiveVisitNodes(StartNode, VisitFunction, MacroContext ? *MacroContext : LocalMacroContext, VisitedNodes, *ObjectGroupMap);

	check(MacroContext ? MacroContext->IsEmpty() : LocalMacroContext.IsEmpty());
}


const UEdGraphPin* GraphTraversal::FindIOPinSourceThroughMacroContext(const UEdGraphPin& Pin, TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext)
{
	const UEdGraphPin* ReturnPin = nullptr;
	const UEdGraphNode* Node = Pin.GetOwningNode();
	check(Node);

	if (const UCustomizableObjectNodeMacroInstance* NodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		if (const UEdGraphPin* OutputPin = NodeMacro->GetMacroTunnelPin(Pin.Direction == EGPD_Output ? ECOMacroIOType::COMVT_Output : ECOMacroIOType::COMVT_Input, Pin.PinName))
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*OutputPin))
			{
				TArray<const UCustomizableObjectNodeMacroInstance*> NewMacroContext;

				if (!MacroContext)
				{
					MacroContext = &NewMacroContext;
				}
				else if (MacroContext->Contains(NodeMacro))
				{
					return nullptr;
				}

				MacroContext->Push(NodeMacro);
				ReturnPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, MacroContext);
				MacroContext->Pop();
			}
		}
	}

	else if (const UCustomizableObjectNodeTunnel* NodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		if (MacroContext && MacroContext->Num())
		{
			const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = MacroContext->Pop();

			if (MacroInstanceNode)
			{
				if (const UEdGraphPin* FollowPin = MacroInstanceNode->FindPin(Pin.PinName, NodeTunnel->bIsInputNode ? EEdGraphPinDirection::EGPD_Input : EEdGraphPinDirection::EGPD_Output))
				{
					if (const UEdGraphPin* ConnectedPin = NodeTunnel->bIsInputNode ? FollowInputPin(*FollowPin) : FollowOutputPin(*FollowPin))
					{
						ReturnPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, MacroContext);
					}
				}
			}

			MacroContext->Push(MacroInstanceNode);
		}
	}

	else
	{
		ReturnPin = &Pin;
	}

	return ReturnPin;
}


UCustomizableObjectNodeObject* GraphTraversal::GetFullGraphRootNode(const UCustomizableObject* Object, TArray<const UCustomizableObject*>& VisitedObjects)
{
	if (Object != nullptr)
	{
		VisitedObjects.Add(Object);

		UCustomizableObjectNodeObject* Root = GetRootNode(Object);

		if (Root->ParentObject == nullptr)
		{
			return Root;
		}
		else
		{
			if (VisitedObjects.Contains(Root->ParentObject))
			{
				//There is a cycle
				return nullptr;
			}
			else
			{
				return GraphTraversal::GetFullGraphRootNode(Root->ParentObject, VisitedObjects);
			}
		}
	}

	return nullptr;
}


const UEdGraphPin* FindMeshBaseSource(const UEdGraphPin& Pin, const bool bOnlyLookForStaticMesh, TArray<const UCustomizableObjectNodeMacroInstance*> MacroContext)
{
	check(Pin.Direction == EGPD_Output);
	check(Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Mesh
		||
		Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_MeshSection
		||
		Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Modifier
	);

	const UEdGraphNode* Node = Pin.GetOwningNode();
	check(Node);

	if (Cast<UCustomizableObjectNodeSkeletalMesh>(Node))
	{
		if (!bOnlyLookForStaticMesh)
		{
			return &Pin;
		}
	}

	else if (Cast<UCustomizableObjectNodeStaticMesh>(Node))
	{
		return &Pin;
	}

	else if (const UCustomizableObjectNodeMeshReshape* TypedNodeReshape = Cast<UCustomizableObjectNodeMeshReshape>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeReshape->BaseMeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCustomizableObjectNodeMeshMorph* TypedNodeMorph = Cast<UCustomizableObjectNodeMeshMorph>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMorph->MeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCONodeSwitch* TypedNodeSwitch = CastSwitch(Node, UEdGraphSchema_CustomizableObject::PC_Mesh))
	{
		if (const UEdGraphPin* EnumParameterPin = FollowInputPin(*TypedNodeSwitch->SwitchParameterPinReference.Get()))
		{
			if (const UCustomizableObjectNodeEnumParameter* EnumNode = Cast<UCustomizableObjectNodeEnumParameter>(EnumParameterPin->GetOwningNode()))
			{
				if (const UEdGraphPin* DefaultPin = TypedNodeSwitch->SwitchPins[EnumNode->DefaultIndex].Get())
				{
					if (const UEdGraphPin* ConnectedPin = FollowInputPin(*DefaultPin))
					{
						return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
					}
				}
			}
		}
	}

	else if (const UCustomizableObjectNodeMeshVariation* TypedNodeMeshVar = Cast<UCustomizableObjectNodeMeshVariation>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMeshVar->DefaultPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}

		for (int32 i = 0; i < TypedNodeMeshVar->GetNumVariations(); ++i)
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMeshVar->VariationPin(i)))
			{
				return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
			}
		}
	}

	else if (const UCONodeSkeletalMeshSection* TypedNodeMat = Cast<UCONodeSkeletalMeshSection>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMat->GetMeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCONodeSkeletalMeshSectionVariation* TypedNodeMatVar = Cast<UCONodeSkeletalMeshSectionVariation>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMatVar->DefaultPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCONodeModifierExtendSkeletalMeshSection* TypedNodeExtend = Cast<UCONodeModifierExtendSkeletalMeshSection>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeExtend->AddMeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCustomizableObjectNodeMeshMorphStackApplication* TypedNodeMorphStackApp = Cast<UCustomizableObjectNodeMeshMorphStackApplication>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*TypedNodeMorphStackApp->GetMeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCustomizableObjectNodeMeshReshape* NodeMeshReshape = Cast<UCustomizableObjectNodeMeshReshape>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeMeshReshape->BaseMeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (Cast<UCustomizableObjectNodeTable>(Node))
	{
		if (!bOnlyLookForStaticMesh)
		{
			return &Pin;
		}
	}

	else if (const UCustomizableObjectNodeAnimationPose* NodeMeshPose = Cast<UCustomizableObjectNodeAnimationPose>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeMeshPose->GetInputMeshPin()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (Cast<UCustomizableObjectNodeSkeletalMeshParameter>(Node))
	{
		if (!bOnlyLookForStaticMesh)
		{
			return &Pin;
		}
	}

	else if (const UCustomizableObjectNodeMacroInstance* NodeMacro = Cast<UCustomizableObjectNodeMacroInstance>(Node))
	{
		const UEdGraphPin* OutputPin = NodeMacro->GetMacroTunnelPin(ECOMacroIOType::COMVT_Output, Pin.PinName);

		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*OutputPin))
		{
			MacroContext.Push(NodeMacro);
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCustomizableObjectNodeTunnel* NodeTunnel = Cast<UCustomizableObjectNodeTunnel>(Node))
	{
		const UCustomizableObjectNodeMacroInstance* MacroInstanceNode = MacroContext.Pop();
		check(MacroInstanceNode);

		if (const UEdGraphPin* InputPin = MacroInstanceNode->FindPin(Pin.PinName, EEdGraphPinDirection::EGPD_Input))
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*InputPin))
			{
				return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
			}
		}
	}
	
	else if (const UCONodeClipMeshWithMesh* NodeClipMeshWithMesh = Cast<UCONodeClipMeshWithMesh>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeClipMeshWithMesh->BaseMeshPin.Get()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}
	
	else if (const UCONodeRemoveMesh* NodeMeshRemoveMesh = Cast<UCONodeRemoveMesh>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeMeshRemoveMesh->BaseMeshPin.Get()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (const UCONodeTransformWithBone* NodeTransformWithBone = Cast<UCONodeTransformWithBone>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeTransformWithBone->BaseMeshPin.Get()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}
	
	else if (const UCONodeTransformInMesh* NodeTransformWithMesh = Cast<UCONodeTransformInMesh>(Node))
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*NodeTransformWithMesh->BaseMeshPin.Get()))
		{
			return FindMeshBaseSource(*ConnectedPin, bOnlyLookForStaticMesh, MacroContext);
		}
	}

	else if (Cast<UCONodeExternalOperation>(Node))
	{
		if (!bOnlyLookForStaticMesh)
		{
			return &Pin;
		}
	}

	else
	{
		unimplemented(); // Case missing.
	}
	
	return nullptr;
}


void GetNodeGroupObjectNodeMappingImmersive(UCustomizableObject* Object, IAssetRegistry& AssetRegistry, TSet<UCustomizableObject*>& Visited, TMultiMap<FGuid, UCustomizableObjectNodeObject*>& Mapping, bool bAsync, bool& bPendingLoad)
{
	MUTABLE_CPUPROFILER_SCOPE(GetNodeGroupObjectNodeMappingImmersive);
	
	if (!Object || Visited.Contains(Object))
	{
		return;
	}

	Visited.Add(Object);
	
	const bool bHasGroupNodes = Object->GetPrivate()->GetSource()->Nodes.ContainsByPredicate([](const TObjectPtr<UEdGraphNode>& Node)
	{
		return Node.IsA(UCustomizableObjectNodeObjectGroup::StaticClass());
	});
	
	if (!bHasGroupNodes)
	{
		return;
	}
	
	TArray<FName> ArrayReferenceNames;
	AssetRegistry.GetReferencers(*Object->GetOuter()->GetPathName(), ArrayReferenceNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

	FARFilter Filter;
	Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// when cooking, only search on-disk packages, to ensure deterministic results
	
	for (const FName& ReferenceName : ArrayReferenceNames)
	{
		if (!ReferenceName.ToString().StartsWith(TEXT("/TempAutosave")))
		{
			Filter.PackageNames.Add(ReferenceName);
		}
	}
	
	if (ArrayReferenceNames.IsEmpty())
	{
		return;
	}

	Filter.ClassPaths.Add(UCustomizableObject::StaticClass()->GetClassPathName());
	
	TArray<FAssetData> ArrayAssetData;
	AssetRegistry.GetAssets(Filter, ArrayAssetData);

	for (FAssetData& AssetData : ArrayAssetData)
	{
		UCustomizableObject* ChildObject = nullptr;
		{
			MUTABLE_CPUPROFILER_SCOPE(LoadObject);
			
			if (bAsync)
			{
				if (AssetData.IsAssetLoaded())
				{
					ChildObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(AssetData));
				}
				else
				{
					UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstance();
					
					FSoftObjectPath SoftObjectPath = AssetData.ToSoftObjectPath();
					System->GetPrivate()->StreamableManager->RequestAsyncLoad(SoftObjectPath);
					
					bPendingLoad |= true;
				}
			}
			else
			{
				ChildObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(AssetData));
			}
		}
		
		if (!ChildObject)
		{
			continue;
		}

		if (ChildObject != Object && !ChildObject->HasAnyFlags(RF_Transient))
		{
			if (UCustomizableObjectNodeObject* ChildRoot = GetRootNode(ChildObject))
			{
				if (ChildRoot->ParentObject == Object)
				{
					Mapping.Add(ChildRoot->ParentObjectGroupId, ChildRoot);
				}
			}
		}

		GetNodeGroupObjectNodeMappingImmersive(ChildObject, AssetRegistry, Visited, Mapping, bAsync, bPendingLoad);
	}
}


TMultiMap<FGuid, UCustomizableObjectNodeObject*> GetNodeGroupObjectNodeMapping(UCustomizableObject* Object)
{
	MUTABLE_CPUPROFILER_SCOPE(GetNodeGroupObjectNodeMapping);

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	TSet<UCustomizableObject*> Visited;
	TMultiMap<FGuid, UCustomizableObjectNodeObject*> Mapping;

	bool bPendingLoad = false;
	GetNodeGroupObjectNodeMappingImmersive(Object, AssetRegistry, Visited, Mapping, false, bPendingLoad);
	
	return Mapping;
}


void GraphTraversal::GetAllObjectsInGraph(UCustomizableObject* Object, TSet<UCustomizableObject*>& OutObjects)
{
	if (!Object)
	{
		return;
	}

	// Search the root of the CO's graph
	UCustomizableObject* RootObject = GraphTraversal::GetRootObject(Object);
	TMultiMap<FGuid, UCustomizableObjectNodeObject*> DummyMap;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	
	bool bPendingLoad = false;
	GetNodeGroupObjectNodeMappingImmersive(RootObject, AssetRegistry, OutObjects, DummyMap, false, bPendingLoad);
}


bool GraphTraversal::GetAllObjectsInGraphIncremental(UCustomizableObject* Object, TSet<UCustomizableObject*>& OutObjects)
{
	if (!Object)
	{
		return true;
	}

	// Search the root of the CO's graph
	UCustomizableObject* RootObject = GraphTraversal::GetRootObject(Object);
	TMultiMap<FGuid, UCustomizableObjectNodeObject*> DummyMap;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	bool bPendingLoad = false;
	GetNodeGroupObjectNodeMappingImmersive(RootObject, AssetRegistry, OutObjects, DummyMap, true, bPendingLoad);
	
	return bPendingLoad;
}


namespace GraphTraversal
{
	bool IsRootObject(const UCustomizableObject& Object)
	{
		const TObjectPtr<UEdGraph> Source = Object.GetPrivate()->GetSource();
		if (!Source || !Source->Nodes.Num())
		{
			// Conservative approach.
			return true;
		}

		TArray<UCustomizableObjectNodeObject*> ObjectNodes;
		Source->GetNodesOfClass<UCustomizableObjectNodeObject>(ObjectNodes);

		// Look for the base object node
		const UCustomizableObjectNodeObject* Root = nullptr;
		for (TArray<UCustomizableObjectNodeObject*>::TIterator It(ObjectNodes); It; ++It)
		{
			if ((*It)->bIsBase)
			{
				Root = *It;
			}
		}

		return Root && !Root->ParentObject;
	}
}


void NodePinConnectionListChanged(const TArray<UEdGraphPin*>& Pins)
{
	TMap<UEdGraphNode*, TSet<UEdGraphPin*>> SortedPins;
	for (UEdGraphPin* Pin : Pins) 
	{
		if (UEdGraphNode* Node = Pin->GetOwningNodeUnchecked())
		{
			SortedPins.FindOrAdd(Node).Add(Pin);
		}
	}

	for (TTuple<UEdGraphNode*, TSet<UEdGraphPin*>> Pair : SortedPins)
	{
		for (UEdGraphPin* ConnectedPin : Pair.Value)
		{
			Pair.Key->PinConnectionListChanged(ConnectedPin);
		}
		
		Pair.Key->NodeConnectionListChanged();
	}
}


