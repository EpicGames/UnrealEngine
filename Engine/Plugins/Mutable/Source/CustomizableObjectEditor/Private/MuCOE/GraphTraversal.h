// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"

class UCustomizableObject;
class UCustomizableObjectNodeObject;
class UEdGraph;
class UEdGraphPin;
class UCustomizableObjectNode;
class UCustomizableObjectNodeMacroInstance;


/** Follow the given pin returning its connected pin.
 *
 * - Skips all orphan pins.
 * - Follows External Pin and Reroute nodes.
 *
 * @param Pin Pin to follow.
 * @param bIgnoreOrphan If true, it will not follow orphan pins.
 * @param bOutCycleDetected If provided, it will set to true if a cycle has been found. */
TArray<UEdGraphPin*> FollowPinArray(const UEdGraphPin& Pin, bool bIgnoreOrphan = false, bool* bOutCycleDetected = nullptr);

/** Follow the given input pin returning the output connected pin.
 *
 * - Skips all orphan pins.
 * - Follows External Pin nodes.
 *
 * @param Pin Pin to follow.
 * @param bOutCycleDetected If provided, it will set to true if a cycle has been found.
 * @return Connected pins. */
TArray<UEdGraphPin*> FollowInputPinArray(const UEdGraphPin& Pin, bool* bOutCycleDetected = nullptr);

/** Non-array version of FollowInputPinArray. The pin can only have one connected pin. */
UEdGraphPin* FollowInputPin(const UEdGraphPin& Pin, bool* bOutCycleDetected = nullptr);

/** Follow the given input output returning the input connected pin.
 * 
 * - Skips all orphan pins.
 * - Follows External Pin nodes.
 * - It will only follow External Pin nodes of loaded CO (i.e., Expose Pin nodes of CO which are NOT loaded will not be found)!
 *
 * @param Pin Pin to follow.
 * @param bOutCycleDetected If provided, it will set to true if a cycle has been found.
 * @return Connected pins. */
TArray<UEdGraphPin*> FollowOutputPinArray(const UEdGraphPin& Pin, bool* bOutCycleDetected = nullptr);

/** Non-array version of FollowOutputPinArray. The pin can only have one connected pin. */
UEdGraphPin* FollowOutputPin(const UEdGraphPin& Pin, bool* CycleDetected = nullptr);

/** See FollowPinArray.
 * Given a pin, follow it in reverse (through the owning node instead of the linked pins). */
TArray<UEdGraphPin*> ReverseFollowPinArray(const UEdGraphPin& Pin, bool bIgnoreOrphan = false, bool* bOutCycleDetected = nullptr);

/** Returns the root Object Node of the Customizable Object's graph */
UCustomizableObjectNodeObject* GetRootNode(const UCustomizableObject* Object);

/** Return in ArrayNodeObject the roots nodes in each Customizable Object graph until the whole root node is found (i.e. the one with parent = nullptr)
 * return false if a cycle is found between Customizable Objects */
bool GetParentsUntilRoot(const UCustomizableObject* Object, TArray<UCustomizableObjectNodeObject*>& ArrayNodeObject, TArray<const UCustomizableObject*>& ArrayCustomizableObject);

/** Returns true if the Candidate is parent of the current Customizable Object */
bool HasCandidateAsParent(UCustomizableObjectNodeObject* Node, UCustomizableObject* ParentCandidate);

/** Return the full graph Customizable Object root of the given node.
 * @param Node The CO Node we want to get its root CO
 * @param VisitedObjects List of objects used to record the search path already taken. It is used to determine if a cycle has been found.
 * return The root CO. It can be null if a cycle has been found.
 */
UCustomizableObject* GetFullGraphRootObject(const UCustomizableObjectNodeObject* Node, TArray<const UObject*>& VisitedObjects);

namespace GraphTraversal
{
	/** Return Customizable Object of the given node.
	* Returns null if the Node belongs to a Macro graph
	*/
	UCustomizableObject* GetObject(const UCustomizableObjectNode& Node);

	/** Provided a CO object it provides the root CO it is connected. In other words : it returns the root of the entire
		 * mutable graph.
		 * @param InObject Customizable object whose root CO we are asking for.
		 * @return The CO that is the root of the provided Customizable Object. It can be equal to InObject if the provided
		 * object does not have any parent.	*/
	CUSTOMIZABLEOBJECTEDITOR_API UCustomizableObject* GetRootObject(UCustomizableObject* ChildObject, TArray<const UObject*>* OutVisitedObjects = nullptr);
	CUSTOMIZABLEOBJECTEDITOR_API const UCustomizableObject* GetRootObject(const UCustomizableObject* ChildObject, TArray<const UObject*>* OutVisitedObjects = nullptr);
	
	/** Return the full graph Customizable Object Node root of the node given as parameter */
	UCustomizableObjectNodeObject* GetFullGraphRootNode(const UCustomizableObject* Object, TArray<const UCustomizableObject*>& VisitedObjects);


	/** From the StartNode visit all connected nodes in the hierarchy.
	 *
	 *   @param StartNode Node to start the visit.
	 *   @param VisitFunction Called for each UCustomizableObjectNode node found. Order is not guaranteed. 
	 *   @param ObjectGroupMap Key is the Object Group node id, values are the attached Child Object nodes. Used to prune the traversal.
	 *   @param MacroContext Keeps track of the macro context when going through macros
	  */
	void VisitNodes(UCustomizableObjectNode& StartNode, const TFunction<void(UCustomizableObjectNode&)>& VisitFunction,
		const TMultiMap<FGuid, UCustomizableObjectNodeObject*>* ObjectGroupMap = nullptr, TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr);

	/** Returns a pin linked to the original pin. This function goes through Macro and Tunnel Nodes.
	* @param Pin: Input/Output pin where the search starts.
	* @param MacroContext: Copy of the current macro context.
	*/
	const UEdGraphPin* FindIOPinSourceThroughMacroContext(const UEdGraphPin& Pin, TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr);
}

/** Given an output pin, return the output pin where the mesh is located. */
const UEdGraphPin* FindMeshBaseSource(const UEdGraphPin& Pin, const bool bOnlyLookForStaticMesh, TArray<const UCustomizableObjectNodeMacroInstance*> MacroContext = {});

/** Return the mapping of Group Object Nodes to Child Object Nodes of the given hierarchy.
 * @param Object Child or root Object to start the search from. */
TMultiMap<FGuid, UCustomizableObjectNodeObject*> GetNodeGroupObjectNodeMapping(UCustomizableObject* Object);

namespace GraphTraversal
{
	/** Returns all the Customizable Objects in a graph starting at the root object */
	void GetAllObjectsInGraph(UCustomizableObject* Object, TSet<UCustomizableObject*>& OutObjects);

	/** Returns all the Customizable Objects in a graph starting at the root object.
	 * @return true if the full hierarchy could not be discovered due to UCustomizableObjects still async loading. Async loadings are called automatically. */
	bool GetAllObjectsInGraphIncremental(UCustomizableObject* Object, TSet<UCustomizableObject*>& OutObjects);

    /** Return ture if the given Customizable Object is Root Object (not a Child Object). */
    bool IsRootObject(const UCustomizableObject& Object); 
}

/** For each given pin, call PinConnectionListChanged and NodeConnectionListChanged in the correct order.
 *  Order: For each node, first call all PinConnectionListChanged, then NodeConnectionListChanged. */
void NodePinConnectionListChanged(const TArray<UEdGraphPin*>& Pins);
