// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/MutableMeshPreviewUtils.h"

#include "Animation/Skeleton.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuR/MeshBufferUtils.h"
#include "MuCO/UnrealConversionUtils.h"
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuR/Skeleton.h"
#include "MuR/Mesh.h"
#include "MuR/OpMeshFormat.h"
#include "Engine/SkeletalMesh.h"
#include "MuCO/CustomizableObject.h"
#include "UObject/Package.h"

class UMaterialInterface;


namespace MutableMeshPreviewUtils
{
	namespace
	{
		/** It generates a copy of the provided constant mesh but making sure the buffer structure is the one
		 * unreal and our systems want. This is because some of the meshes we provide to mutable do not all have the
		 * same structure.
		 * @param InMutableMesh Mutable mesh whose buffers should be formatted following the desired buffer structure.
		 * @return An identical mutable mesh to the provided one in terms of data but with the buffer structure that UE
		 * requires
		 */
		TSharedPtr<const UE::Mutable::Private::FMesh> GenerateCompatibleMutableMesh(const UE::Mutable::Private::FMesh* InMutableMesh)
		{
			// 1) Generate a mutable mesh to be the one hosting the buffer format we desire (it will be required for 
			TSharedPtr<UE::Mutable::Private::FMesh> FormattedMutableMesh = MakeShared<UE::Mutable::Private::FMesh>();
			UE::Mutable::Private::FMeshBufferSet& FormattedVertexBuffers = FormattedMutableMesh->GetVertexBuffers();
			UE::Mutable::Private::FMeshBufferSet& FormattedIndexBuffers = FormattedMutableMesh->GetIndexBuffers();
			
			// 2) Generate the buffers for that mesh with the format unreal expects
			{
				// 2.1)		Locate what buffers are present on the mutable constant mesh
				const UE::Mutable::Private::FMeshBufferSet& OriginVertexBuffers = InMutableMesh->GetVertexBuffers();

				// Data extracted from the original buffers to be used later when setting up the formatted mesh
				bool bOriginDoesHaveVertexColorData = false;
				int32 NumbOfTextCoordChannels = 0;
				int32 MaxNumBonesPerVertex = 0;
				int32 BoneIndicesSizeBytes = 0;
				int32 BoneWeightsSizeBytes = 0;

				// Contingency to avoid mismatches between the ideal semantic indices on the provided mutable mesh against the
				// ones that should be .
				int TextureSemanticIndices[UE::Mutable::Private::MeshBufferUtils::MaxTexCordChannelCount] = {-1,-1,-1,-1};
				
				// Vertex buffers
				const int32 MutableMeshBuffersCount = OriginVertexBuffers.GetBufferCount();
				for (int32 BufferIndex = 0; BufferIndex < MutableMeshBuffersCount; BufferIndex++)
				{
					const int32 BufferChannelCount = OriginVertexBuffers.GetBufferChannelCount(BufferIndex);
					for (int32 ChannelIndex = 0; ChannelIndex < BufferChannelCount; ChannelIndex ++)
					{
						UE::Mutable::Private::EMeshBufferSemantic BufferSemantic = UE::Mutable::Private::EMeshBufferSemantic::None;
						UE::Mutable::Private::EMeshBufferFormat BufferFormat = UE::Mutable::Private::EMeshBufferFormat::None;
						int32 BufferComponentCount = 0;
						int32 SemanticIndex = 0;

						// Get the data from mutable that we require from the selected buffer set buffer
						OriginVertexBuffers.GetChannel
						(
							BufferIndex,
							ChannelIndex,
							&(BufferSemantic),
							&(SemanticIndex),
							&(BufferFormat),
							&(BufferComponentCount),
							nullptr
						);

						switch (BufferSemantic)
						{
							case UE::Mutable::Private::EMeshBufferSemantic::Color:
							{
								// Does the original mesh have vert color data?
								bOriginDoesHaveVertexColorData = true;
								break;
							}

							case UE::Mutable::Private::EMeshBufferSemantic::TexCoords:
							{
								// Store the texture semantic index to be later used during the buffer formatting process on  UE::Mutable::Private::MeshFormat
								TextureSemanticIndices[NumbOfTextCoordChannels] = SemanticIndex;
									
								// Store the amount of located texture data channels
								NumbOfTextCoordChannels++;
								break;
							}

							case UE::Mutable::Private::EMeshBufferSemantic::BoneIndices:
							{
								// Store the amount of bones a vertex can be skinned to.
								MaxNumBonesPerVertex = BufferComponentCount;
								BoneIndicesSizeBytes = BufferFormat == UE::Mutable::Private::EMeshBufferFormat::Int16 ? 2 : 1;
								break;
							}

							case UE::Mutable::Private::EMeshBufferSemantic::BoneWeights:
							{
								// Store the amount of bones a vertex can be skinned to.
								BoneWeightsSizeBytes = BufferFormat == UE::Mutable::Private::EMeshBufferFormat::NUInt16 ? 2 : 1;
								break;
							}

							default:
							{
								break;
							}
						}
					}
				}
				
				// 2.2)		Generate the mesh buffer Sets so they follow the expected structure.

				// Setup the buffers to later setup the channels each of them have
				int32 MutableBufferCount = MUTABLE_VERTEXBUFFER_TEXCOORDS + 1;
				{
					// Bone Indices
					{
						// They are mandatory when generating a new skeletal mesh
						MutableBufferCount++;
					}
					
					// (Optional) Vertex colors
					if (bOriginDoesHaveVertexColorData)
					{
						MutableBufferCount++;
					}
				}
				FormattedVertexBuffers.SetBufferCount(MutableBufferCount);

				// Setup the channels of each buffer

				int32 CurrentVertexBuffer = 0;
				{
					using namespace UE::Mutable::Private;

					// Vertex buffer (positions)
					MeshBufferUtils::SetupVertexPositionsBuffer(CurrentVertexBuffer,FormattedVertexBuffers);
					CurrentVertexBuffer++;

					// Tangent buffer
					MeshBufferUtils::SetupTangentBuffer(CurrentVertexBuffer,FormattedVertexBuffers);
					CurrentVertexBuffer++;
				
					// Texture coords buffer
					const bool bHighPrecision = true;
					MeshBufferUtils::SetupTexCoordinatesBuffer(CurrentVertexBuffer,NumbOfTextCoordChannels, bHighPrecision, FormattedVertexBuffers, TextureSemanticIndices);
					CurrentVertexBuffer++;
				
					// Skin buffer
					MeshBufferUtils::SetupSkinBuffer(CurrentVertexBuffer, BoneIndicesSizeBytes, BoneWeightsSizeBytes, MaxNumBonesPerVertex, FormattedVertexBuffers);
					CurrentVertexBuffer++;

					// Color buffer
					if (bOriginDoesHaveVertexColorData)
					{
						MeshBufferUtils::SetupVertexColorBuffer(CurrentVertexBuffer,FormattedVertexBuffers);
						CurrentVertexBuffer++;
					}

					// Index buffer
					MeshBufferUtils::SetupIndexBuffer(FormattedIndexBuffers, EMeshBufferFormat::UInt32);
				}
			}
			
			// At this point a new mutable mesh with no data but with the appropriate structure for UE to be able to convert it to an
			// Skeletal mesh is ready to be used as an argument for the process of reformatting the Mutable Mesh we want to preview

			// 3) Call UE::Mutable::Private::MeshFormat to get the mesh with the wanted buffer structure to have the data of the original mesh
			TSharedPtr<UE::Mutable::Private::FMesh> FormatResult = MakeShared<UE::Mutable::Private::FMesh>();

			bool bOutSuccess = false;
			UE::Mutable::Private::MeshFormat(FormatResult.Get(), InMutableMesh, FormattedMutableMesh.Get(),
				false, true, true, false,
				bOutSuccess);
		
			return FormatResult;
		}

		/** Method designed to scan the vertex buffers of the provided mutable mesh and tell the caller if the buffers are
		 * set up following what UE requires. 
		 * @param InMutableMesh The Mutable Mesh that is wanted to be checked.
		 * @param bLogFindingsToConsole If true the console will display not only if the buffers were found but also if they are
		 * badly located inside the buffer set.
		 * @return True if the buffers are where they should be, false if not correctly placed or missing.
		 */
		bool AreVertexBuffersCorrectlyLocated(const UE::Mutable::Private::FMesh* InMutableMesh, bool bLogFindingsToConsole = false )
		{
			bool bCanBeConverted = true;

			// At the moment requires the presence of all "mandatory" buffers. Only meshes designed to be displayed by unreal
			// will therefore be processable at the moment and not always
			const UE::Mutable::Private::FMeshBufferSet& VertexBuffers = InMutableMesh->GetVertexBuffers();

			int32 Buffer = -1;
			int32 Channel = -1;

			// POSITION BUFFER INDEX CHECK
			VertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::Position, 0, &Buffer, &Channel);
			if (Buffer == -1)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning, "\tVertex Positions Buffer not found.");
				}

				bCanBeConverted = false;
			}
			else if (Buffer != MUTABLE_VERTEXBUFFER_POSITION)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning,
					       "\tBad Vertex Positions buffer position ( at %d instead of %d ).", Buffer, MUTABLE_VERTEXBUFFER_POSITION)
				}

				bCanBeConverted = false;
			}

			// SKIN WEIGHTS CHECK
			VertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::BoneIndices, 0, &Buffer, &Channel);
			if (Buffer == -1)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning, "\tSkinning Buffer not found.");
				}

				bCanBeConverted = false;
			}

			// TANGENT BUFFER INDEX CHECK
			VertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::Tangent, 0, &Buffer, &Channel);
			if (Buffer == -1)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning, "\tTangents Buffer not found.");
				}

				bCanBeConverted = false;
			}
			else if (Buffer != MUTABLE_VERTEXBUFFER_TANGENT)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning,
					       "\tBad Tangent buffer position (  at %d instead of %d ). ", Buffer, MUTABLE_VERTEXBUFFER_TANGENT);
				}

				bCanBeConverted = false;
			}

			// TEX COORDS BUFFER INDEX CHECK
			VertexBuffers.FindChannel(UE::Mutable::Private::EMeshBufferSemantic::TexCoords, 0, &Buffer, &Channel);
			if (Buffer == -1)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning, "\tTexCords Buffer not found.");
				}

				bCanBeConverted = false;
			}
			else if (Buffer != MUTABLE_VERTEXBUFFER_TEXCOORDS)
			{
				if (bLogFindingsToConsole)
				{
					UE_LOGF(LogMutable, Warning,
					       "\tBad TexCoords buffer position ( at %d instead of %d ). ", Buffer, MUTABLE_VERTEXBUFFER_TEXCOORDS);
				}

				bCanBeConverted = false;
			}
			
			return bCanBeConverted;
		}


		/** Provides the OutSkeletalMesh with a new skeleton
		 * @param OutSkeletalMesh - The skeletal mesh whose skeleton we want to generate.
		 * @return True if the operation could be performed successfully, false if not.
		 */
		bool BuildSkeletalMeshSkeletonData(const UE::Mutable::Private::FMesh* InMutableMesh, USkeletalMesh* OutSkeletalMesh)
		{
			if (!InMutableMesh || !OutSkeletalMesh)
			{
				return false;
			}
			
			// The skeleton will only have one bone, and all bon indices will be mapped to it.
			TObjectPtr<USkeleton> Skeleton = NewObject<USkeleton>();

			// Build new RefSkeleton	
			{
				// Scope is important, FReferenceSkeletonModifier will rebuild the reference skeleon on destroy
				FReferenceSkeletonModifier RefSkeletonModifier(Skeleton);
				RefSkeletonModifier.Add(FMeshBoneInfo(FName("Root"), FString("Root"), INDEX_NONE), FTransform::Identity);
			}

			OutSkeletalMesh->SetSkeleton(Skeleton);
			OutSkeletalMesh->SetRefSkeleton(Skeleton->GetReferenceSkeleton());
			OutSkeletalMesh->GetRefBasesInvMatrix().Empty(1);
			OutSkeletalMesh->CalculateInvRefMatrices();

			// Compute imported bounds
			FBoxSphereBounds Bounds;

			const int32 BonePoseCount = InMutableMesh->GetBonePoseCount();
			if (BonePoseCount > 0)
			{
				// Extract bounds from the pose of the mesh
				TArray<FVector> Points;
				Points.Reserve(BonePoseCount);
				for (int32 BoneIndex = 0; BoneIndex < BonePoseCount; ++BoneIndex)
				{
					FTransform3f Transform;
					InMutableMesh->GetBonePoseTransform(BoneIndex, Transform);

					Points.Add(FVector(Transform.GetTranslation()));
				}

				Bounds = FBoxSphereBounds(Points.GetData(), Points.Num());
				Bounds = Bounds.ExpandBy(1.2f);
			}
			else
			{
				// Set bounds even if they're not correct
				Bounds = FBoxSphereBounds(FSphere(FVector(0,0,0), 1000));
			}

			OutSkeletalMesh->SetImportedBounds(Bounds);

			return true;
		}

		/** Prepares the OutSkeletal mesh with the required materials taking in consideration the amount of surfaces found on
		 * the InMutableMesh. As it is working with a mutable mesh, and a mutable mesh only represents one lod, the target lod is 0
		 * @param InMutableMesh - The reference mutable mesh to be reading data from
		 * @param OutSkeletalMesh - The skeletal mesh to be getting it's materials set up.
		 */
		void BuildSkeletalMeshElementData(const UE::Mutable::Private::FMesh* InMutableMesh, USkeletalMesh* OutSkeletalMesh)
		{
			// Set up Unreal's default material
			UMaterialInterface* UnrealMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
			OutSkeletalMesh->GetMaterials().SetNum(1);
			OutSkeletalMesh->GetMaterials()[0] = UnrealMaterial;

			OutSkeletalMesh->GetLODInfo(0)->LODMaterialMap.SetNumZeroed(1);

			// Add RenderSections for each surface in the mesh
			if (InMutableMesh)
			{
				for (int32 SurfaceIndex = 0; SurfaceIndex < InMutableMesh->GetSurfaceCount(); ++SurfaceIndex)
				{
					new(OutSkeletalMesh->GetResourceForRendering()->LODRenderData[0].RenderSections) FSkelMeshRenderSection();
				}
			}
		}


		/** Prepares the skeletal mesh to be able to be rendered. It should provide you with a mesh ready to be rendered
		 * @param InMutableMesh - The mutable mesh to be used as reference
		 * @param InRefSkeletalMesh - A reference skeletal mesh to aid on the preparation of the OutSkeletalMesh
		 * @param InBoneMap - An array with all the bones used by the mesh.
		 * @param OutSkeletalMesh - The skeletal mesh to set up.
		 * @return True if the operation could be performed successfully, false if not.
		 */
		bool BuildSkeletalMeshRenderData(const UE::Mutable::Private::FMesh* InMutableMesh, USkeletalMesh* OutSkeletalMesh)
		{
			OutSkeletalMesh->SetHasVertexColors(false);

			// Find how many bones the BoneMap could have
			const int32 NumBonesInBoneMap = !InMutableMesh->GetBoneMap().IsEmpty() ? InMutableMesh->GetBoneMap().Num() : InMutableMesh->GetBonePoseCount();
			
			// Fill the BoneMap with zeros
			TArray<FName> BoneMap;
			BoneMap.Init(FName("Root"), FMath::Max(NumBonesInBoneMap, 1));

			FSkeletalMeshLODRenderData& LODResource = OutSkeletalMesh->GetResourceForRendering()->LODRenderData[0];
			
			// Load buffer data found on the mutable model onto the out skeletal mesh
			// It includes vertex and index buffers
			const bool bWasSetupSuccessful = UnrealConversionUtils::SetupRenderSections(
				LODResource,
				InMutableMesh,
				OutSkeletalMesh->GetRefSkeleton(),
				BoneMap,
				0);

			if (!bWasSetupSuccessful)
			{
				return false;
			}
			
			UnrealConversionUtils::CopyMutableVertexBuffers(
				LODResource,
				InMutableMesh,
				false); 
			

			// Ensure there's at least one bone in the BoneMap of each render section, the root.
			for (FSkelMeshRenderSection& RenderSection : LODResource.RenderSections)
			{
				if (RenderSection.BoneMap.IsEmpty())
				{
					RenderSection.BoneMap.Add(0);
				}
			}

			// Add root as the only required bone
			LODResource.ActiveBoneIndices.Add(0);
			LODResource.RequiredBones.Add(0);
			
			// SurfaceIDs. Required to copy index buffers with padding
			const int32 NumSurfaces = InMutableMesh->Surfaces.Num();
			TArray<uint32> SurfaceIDs;
			SurfaceIDs.SetNum(NumSurfaces);

			for (int32 SurfaceIndex = 0; SurfaceIndex < NumSurfaces; ++SurfaceIndex)
			{
				SurfaceIDs[SurfaceIndex] = InMutableMesh->GetSurfaceId(SurfaceIndex);
			}

			// Copy index buffers or fail to generate the mesh
			bool bMarkRenderStateDirty = false;
			if (!UnrealConversionUtils::CopyMutableIndexBuffers(LODResource, InMutableMesh, bMarkRenderStateDirty))
			{
				return false;
			}
			
			// Update LOD and streaming data
			LODResource.bIsLODOptional = false;
			LODResource.bStreamedDataInlined = false;
			
			return true;
		}
	}

	/*
	 * Publicly accessible functions of MutableMeshPreviewUtils
	 */

	USkeletalMesh* GenerateSkeletalMeshFromMutableMesh(const UE::Mutable::Private::FMesh* InMutableMesh)
	{
		if (!InMutableMesh)
		{
			return nullptr;
		}

		if (InMutableMesh->GetVertexCount() == 0 || InMutableMesh->GetVertexBuffers().IsDescriptor())
		{
			return nullptr;
		}

		TSharedPtr<const UE::Mutable::Private::FMesh> MeshCopy;

		// Buffer structures checks and corrective operations
		{
			// If the semantic channels for the required buffer data are on not the buffer they are expected to be then
			// and only then format the mesh to follow what our systems and Unreal expect in terms of channel on buffer structure
			if (!AreVertexBuffersCorrectlyLocated(InMutableMesh))
			{
				UE_LOGF(LogMutable, Warning,
				       "Restructuring Mutable Mesh Buffers:  ");
				
				// Since we know the mesh does not have the required buffer structure do format it to match our requirements
				MeshCopy = GenerateCompatibleMutableMesh(InMutableMesh);
				InMutableMesh = MeshCopy.Get();

				// If the conversion was required then make sure all went well
				if (!AreVertexBuffersCorrectlyLocated(InMutableMesh,true))
				{
					// The formatting operation failed, do not proceed
					UE_LOGF(LogMutable,Error, "Restructuring of mutable mesh buffers failed: Aborting Skeletal Mesh generation. ");
					return nullptr;
				}
				
				UE_LOGF(LogMutable, Verbose, "Buffers Restructuring was succesfull.");
			}

			// We should now have a mutable mesh following a buffer structure where the required buffers are there to be
			// later found and used to generate a proper Skeletal Mesh
		}
		
		// Generate a new skeletal mesh to be filled up with the data found on the mutable mesh and the reference skeletal mesh
		USkeletalMesh* GeneratedSkeletalMesh = NewObject<USkeletalMesh>();

		// Build the reference skeleton for the Generated Skeletal mesh
		bool bSuccess = BuildSkeletalMeshSkeletonData(InMutableMesh, GeneratedSkeletalMesh);
		if (!bSuccess)
		{
			return nullptr;
		}
		
		// Prepare the mesh to be filled with rendering data (buffers)
		{
			GeneratedSkeletalMesh->AllocateResourceForRendering();
			GeneratedSkeletalMesh->GetResourceForRendering()->LODRenderData.Add(new FSkeletalMeshLODRenderData());

			FSkeletalMeshLODInfo LastLODInfo{};
			LastLODInfo.BuildSettings.bUseFullPrecisionUVs = true;
			LastLODInfo.bAllowCPUAccess = false;
			GeneratedSkeletalMesh->AddLODInfo(LastLODInfo);
		}

		// Set the material data and initialize the sections that will be used later
		BuildSkeletalMeshElementData(InMutableMesh,GeneratedSkeletalMesh);
		
		// Load all data from the mutable mesh onto the skeletal mesh object
		bSuccess = BuildSkeletalMeshRenderData(InMutableMesh, GeneratedSkeletalMesh);
		if (!bSuccess)
		{
			return nullptr;
		}

		// Prepares the mesh to be rendered
		GeneratedSkeletalMesh->InitResources();
		
		return GeneratedSkeletalMesh;
	}
}
