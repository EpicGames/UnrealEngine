// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Mesh.h"

class USkeletalMesh;

/**
 * Namespace designed to contain a series of methods used for the generation of unreal meshes for previewing purposes.
 * While they can be accessed to do other stuff it is not granted they will produce a complete USkeletalMesh since at the
 * moment it only tries to generate something viewable, but may be missing bone, vertex or/and animation data.
 */
namespace MutableMeshPreviewUtils
{
	/**
	 * Method designed to provide a way of getting a USkeletalMesh based on the data found on a mutable mesh. 
	 * @param InMutableMesh - The mutable mesh used as base for the generation of the USkeletalMesh
	 * @return - A new Skeletal mesh based on the provided data. It may be a nullptr if the conversion fails.
	 */
	CUSTOMIZABLEOBJECTEDITOR_API USkeletalMesh* GenerateSkeletalMeshFromMutableMesh(const UE::Mutable::Private::FMesh*);

}
