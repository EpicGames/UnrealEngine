// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/MutableUtils.h"

#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Engine/StaticMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"


TArray<FVector2f> GetUV(const USkeletalMesh& SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const int32 UVIndex)
{
	TArray<FVector2f> Result;

	const FSkeletalMeshModel* ImportedModel = SkeletalMesh.GetImportedModel();
	const FSkeletalMeshLODModel& LOD = ImportedModel->LODModels[LODIndex];
	const FSkelMeshSection& Section = LOD.Sections[SectionIndex];
	
	TArray<FSoftSkinVertex> Vertices;
	LOD.GetVertices(Vertices);

	TArray<uint32> Indices;
	SkeletalMesh.GetResourceForRendering()->LODRenderData[LODIndex].MultiSizeIndexContainer.GetIndexBuffer(Indices);
	
	int32 IndexIndex = Section.BaseIndex;
	for (uint32 FaceIndex = 0; FaceIndex < Section.NumTriangles; ++FaceIndex, IndexIndex += 3)
	{
		Result.Add(Vertices[Indices[IndexIndex + 0]].UVs[UVIndex]);
		Result.Add(Vertices[Indices[IndexIndex + 1]].UVs[UVIndex]);

		Result.Add(Vertices[Indices[IndexIndex + 1]].UVs[UVIndex]);
		Result.Add(Vertices[Indices[IndexIndex + 2]].UVs[UVIndex]);
		
		Result.Add(Vertices[Indices[IndexIndex + 2]].UVs[UVIndex]);
		Result.Add(Vertices[Indices[IndexIndex + 0]].UVs[UVIndex]);
	}

	return Result;
}


TArray<FVector2f> GetUV(const UStaticMesh& StaticMesh, const int32 LODIndex, const int32 SectionIndex, const int32 UVIndex)
{
	TArray<FVector2f> Result;

	const FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();
	const FStaticMeshLODResources& LOD = RenderData->LODResources[LODIndex];
	const FStaticMeshSection& Section = LOD.Sections[SectionIndex];
	
	const FStaticMeshVertexBuffer& VertexBuffer = LOD.VertexBuffers.StaticMeshVertexBuffer;
	
	FIndexArrayView Indices = LOD.IndexBuffer.GetArrayView();

	int32 IndexIndex = Section.FirstIndex;
	for (uint32 FaceIndex = 0; FaceIndex < Section.NumTriangles; ++FaceIndex, IndexIndex += 3)
	{
		Result.Add(VertexBuffer.GetVertexUV(Indices[IndexIndex + 0], UVIndex));
		Result.Add(VertexBuffer.GetVertexUV(Indices[IndexIndex + 1], UVIndex));

		Result.Add(VertexBuffer.GetVertexUV(Indices[IndexIndex + 1], UVIndex));
		Result.Add(VertexBuffer.GetVertexUV(Indices[IndexIndex + 2], UVIndex));
		
		Result.Add(VertexBuffer.GetVertexUV(Indices[IndexIndex + 2], UVIndex));
		Result.Add(VertexBuffer.GetVertexUV(Indices[IndexIndex + 0], UVIndex));
	}
	
	return Result;
}


bool HasNormalizedBounds(const FVector2f& Point)
{
	return Point.X >= 0.0f && Point.X <= 1.0 && Point.Y >= 0.0f && Point.Y <= 1.0;
}
