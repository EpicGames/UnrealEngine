// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"

class USkeletalMesh;
class UStaticMesh;


/** Returns the mesh UV. */
TArray<FVector2f> GetUV(const USkeletalMesh& SkeletalMesh, const int32 LODIndex, const int32 SectionIndex, const int32 UVIndex);

/** Returns the mesh UV. */
TArray<FVector2f> GetUV(const UStaticMesh& StaticMesh, const int32 LODIndex, const int32 SectionIndex, const int32 UVIndex);

/** Returns if the given point is inside the 0-1 bounds. */
bool HasNormalizedBounds(const FVector2f& Point);
