// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeClipMeshWithMesh.h"

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCONodeClipMeshWithMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	BaseMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Base Mesh"), LOCTEXT("BaseMesh", "Base Mesh"));
	ClipMeshPin =  CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Clip Mesh"), LOCTEXT("ClipMesh", "Clip Mesh"));

	OutputMeshPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh);
}


void UCONodeClipMeshWithMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}
	
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeClipMeshWithMesh, Transform))
	{
		if (TransformChangedDelegate.IsBound())
		{
			TransformChangedDelegate.Broadcast(Transform);
		}
	}
}


FText UCONodeClipMeshWithMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Clip_Mesh_With_Mesh", "Mesh Clip With Mesh");
}


FLinearColor UCONodeClipMeshWithMesh::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


FText UCONodeClipMeshWithMesh::GetTooltipText() const
{
	return LOCTEXT("Clip_Mesh_With_Mesh_Tooltip", "Removes the part of a mesh section that is completely enclosed in a mesh volume.");
}

#undef LOCTEXT_NAMESPACE
