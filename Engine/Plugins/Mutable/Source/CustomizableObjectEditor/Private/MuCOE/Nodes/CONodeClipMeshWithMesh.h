// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeClipMeshWithMesh.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FGuid;

DECLARE_MULTICAST_DELEGATE_OneParam(OnTransformChanged, const FTransform&);

UCLASS()
class UCONodeClipMeshWithMesh : public UCustomizableObjectNode
{
	GENERATED_BODY()
	
public:
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	
	// UCustomizableObjectNode interface
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	
protected:

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	
public:
	
	/** Transform to apply to the clip mesh before clipping. */
	UPROPERTY(EditAnywhere, Category = ClipMesh)
	FTransform Transform = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = ClipMesh)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
	
	UPROPERTY()
	FEdGraphPinReference BaseMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference ClipMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference OutputMeshPin;
	
	/**
	 * Delegate invoked each time the BoundingMeshTransform value gets modified
	 */
	OnTransformChanged TransformChangedDelegate;
};
