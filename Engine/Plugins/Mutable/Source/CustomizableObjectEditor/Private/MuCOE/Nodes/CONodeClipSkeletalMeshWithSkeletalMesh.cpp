// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeClipSkeletalMeshWithSkeletalMesh.h"

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

FText UCONodeClipSkeletalMeshWithSkeletalMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("ClipSkeletalMeshWithSkeletalMesh_NodeTitle", "Skeletal Mesh Clip With Skeletal Mesh");
}


FLinearColor UCONodeClipSkeletalMeshWithSkeletalMesh::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


FText UCONodeClipSkeletalMeshWithSkeletalMesh::GetTooltipText() const
{
	return LOCTEXT("ClipSkeletalMeshWithSkeletalMesh_Tooltip", "Removes the part of a Skeletal Mesh that is completely enclosed in a mesh volume.");
}


void UCONodeClipSkeletalMeshWithSkeletalMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}
	
	// Transform update
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeClipSkeletalMeshWithSkeletalMesh, Transform))
	{
		if (TransformChangedDelegate.IsBound())
		{
			TransformChangedDelegate.Broadcast(Transform);
		}
		
		return;
	}
	
	// Preview Skeletal mesh update
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeClipSkeletalMeshWithSkeletalMesh, PreviewMesh) || 
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeClipSkeletalMeshWithSkeletalMesh, PreviewMeshLOD) || 
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeClipSkeletalMeshWithSkeletalMesh, PreviewMeshSection))
	{
		if (PreviewMeshChangedDelegate.IsBound())
		{
			check (ClipSkeletalMeshPin.Get());
			PreviewMeshChangedDelegate.Broadcast(*this, &Transform, *ClipSkeletalMeshPin.Get());
		}
	}
}


void UCONodeClipSkeletalMeshWithSkeletalMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	BaseSkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Base Skeletal Mesh"), LOCTEXT("ClipSkeletalMeshWithSkeletalMesh_BaseMeshName", "Base Skeletal Mesh"));
	ClipSkeletalMeshPin =  CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Clip Skeletal Mesh"), LOCTEXT("ClipSkeletalMeshWithSkeletalMesh_ClipMeshName", "Clip Skeletal Mesh"));

	OutputSkeletalMeshPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}

#undef LOCTEXT_NAMESPACE

