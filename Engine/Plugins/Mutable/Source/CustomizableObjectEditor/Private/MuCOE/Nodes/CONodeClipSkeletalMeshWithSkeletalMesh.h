// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeClipSkeletalMeshWithSkeletalMesh.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FGuid;

DECLARE_MULTICAST_DELEGATE_OneParam(OnTransformChanged, const FTransform&);
DECLARE_MULTICAST_DELEGATE_ThreeParams(OnPreviewMeshChanged, UCustomizableObjectNode&, FTransform*, const UEdGraphPin&);

UCLASS()
class UCONodeClipSkeletalMeshWithSkeletalMesh : public UCustomizableObjectNode
{
	GENERATED_BODY()
	
public:
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	
	// UCustomizableObjectNode interface
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	
protected:

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	
public:
	
	/** Transform to apply to the clip mesh before clipping. */
	UPROPERTY(EditAnywhere, Category = ClipMesh)
	FTransform Transform = FTransform::Identity;

	/**
	 * Delegate invoked each time the MeshTransform value gets modified
	 */
	OnTransformChanged TransformChangedDelegate;
	
	UPROPERTY(EditAnywhere, Category = ClipMesh)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
	
	UPROPERTY()
	FEdGraphPinReference BaseSkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference ClipSkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference OutputSkeletalMeshPin;
	
	UPROPERTY(EditAnywhere, Category= PreviewMesh)
	TObjectPtr<USkeletalMesh> PreviewMesh;
	
	OnPreviewMeshChanged PreviewMeshChangedDelegate;
	
	/** Determines the LOD to be used when rendering the preview mesh. */
	UPROPERTY(EditAnywhere, Category= PreviewMesh)
	uint32 PreviewMeshLOD = 0;
	
	/** Determines the section to be used when rendering the preview mesh. A value of -1 will make the preview show all mesh sections*/
	UPROPERTY(EditAnywhere, Category= PreviewMesh, meta=(ClampMin=-1))
	int32 PreviewMeshSection = -1;
};