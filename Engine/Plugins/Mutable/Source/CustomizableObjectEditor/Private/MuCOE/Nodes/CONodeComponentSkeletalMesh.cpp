// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeComponentSkeletalMesh.h"

#include "CONodeMaterialConstant.h"
#include "ScopedTransaction.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectSchemaActions.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeComponentSkeletalMesh)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


bool UCONodeComponentSkeletalMeshPinRemapper::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin, const uint32 OldPinIndex, const uint32 NewPinIndex) const
{
	const bool bArePinsEqual =
		Helper_GetPinName(&OldPin) == Helper_GetPinName(&NewPin) &&
		OldPin.PinType == NewPin.PinType &&
		OldPin.Direction == NewPin.Direction;
	
	// If they are equal in regard to pin name, direction, type... check if the actual pin data is the same
	if (bArePinsEqual)
	{
		// Compare the pin data
		const UCONodeComponentSkeletalMeshMaterialPinData* OldPinData = Cast<UCONodeComponentSkeletalMeshMaterialPinData>(Node.GetPinData(OldPin));
		const UCONodeComponentSkeletalMeshMaterialPinData* NewPinData = Cast<UCONodeComponentSkeletalMeshMaterialPinData>(Node.GetPinData(NewPin));
		
		// If both cases have custom Overlay/Override pin data then...
		if (OldPinData && NewPinData)
		{
			// If both have the same PinData and share the same index in their pin array then they are the same pin
			return OldPinIndex == NewPinIndex && *OldPinData == *NewPinData;
		}
		
		// If one or the other PinData is missing then they are different
		if ((!OldPinData && NewPinData) || (OldPinData && !NewPinData))
		{
			// One or the other does not have MaterialPinData, they must be different
			return false;
		}
			
		// None of the pins have special PinData so they are normal pins with the same name, type and direction
		return true;
	}

	return false;
}


void UCONodeComponentSkeletalMeshPinRemapper::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins,
	const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	uint32 OldPinIndex = 0;
	for (UEdGraphPin* OldPin : OldPins)
	{
		bool bFound = false;
 
		uint32 NewPinIndex = 0;
		for (UEdGraphPin* NewPin : NewPins)
		{
			if (Equal(Node, *OldPin, *NewPin, OldPinIndex, NewPinIndex))
			{
				bFound = true;

				PinsToRemap.Add(OldPin, NewPin);
				break;
			}
			
			++NewPinIndex;
		}
		
		if (!bFound && OldPin->LinkedTo.Num())
		{
			PinsToOrphan.Add(OldPin);
		}
		
		++OldPinIndex;
	}
}


void UCONodeComponentSkeletalMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	Super::AllocateDefaultPins(RemapPins);

	SkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
	OverlayMaterialPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, TEXT("Overlay Material"),
		LOCTEXT("OverlayMaterial", "Overlay Material"));
	
	// Regenerate the new pins from the old pins
	MaterialSlotPins.Reset();
	const TArray<UEdGraphPin*> AllPins = GetAllNonOrphanPins();
	for (const UEdGraphPin* OldPin : AllPins)
	{
		check(OldPin)

		if (OldPin->Direction != EEdGraphPinDirection::EGPD_Input)
		{
			continue;
		}

		if (UCONodeComponentSkeletalMeshMaterialPinData* OldPinData = Cast<UCONodeComponentSkeletalMeshMaterialPinData>(GetPinData(*OldPin)))
		{
			UCONodeComponentSkeletalMeshMaterialPinData* PinData = NewObject<UCONodeComponentSkeletalMeshMaterialPinData>(this);
			PinData->TargetMaterialSlotName = OldPinData->TargetMaterialSlotName;

			// The SubCategory could have changed after a node refresh
			const FName OldPinSubCategory = OldPin->PinType.PinSubCategory;

			// Use the same pin data as the old pin since the new one is a copy
			const UEdGraphPin* NewPin = CreateMaterialSlotPin(OldPinSubCategory, PinData);
			MaterialSlotPins.Add(NewPin);
		}
	}
}


FText UCONodeComponentSkeletalMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Component_Mesh", "Skeletal Mesh Component");
}


FEdGraphPinReference UCONodeComponentSkeletalMesh::GetOverlayMaterialAssetPin() const
{
	return OverlayMaterialPin.Get();
}


bool UCONodeComponentSkeletalMesh::IsSingleOutputNode() const
{
	// todo UE-225446 : By limiting the number of connections this node can have we avoid a check failure. However, this method should be
	// removed in the future and the inherent issue with 1:n output connections should be fixed in its place
	return true;
}


bool UCONodeComponentSkeletalMesh::HasPinViewer() const
{
	return true;
}


EAddPinNodeButtonLocation UCONodeComponentSkeletalMesh::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::INPUT;
}


void UCONodeComponentSkeletalMesh::AddPinFromUI()
{
	Super::AddPinFromUI();

	FScopedTransaction Transaction(LOCTEXT("UCONodeComponentMesh_AddPinFromNodeUITransaction", "Add Pin from Node UI"));
	Modify();

	UCONodeComponentSkeletalMeshMaterialPinData* PinData = NewObject<UCONodeComponentSkeletalMeshMaterialPinData>(this);

	// By default create Override subtype pins
	if (const UEdGraphPin* NewPin = CreateMaterialSlotPin(UEdGraphSchema_CustomizableObject::PSC_Material_Override, PinData))
	{
		MaterialSlotPins.Add(NewPin);
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCONodeComponentSkeletalMesh::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	const bool bCanPinBeRemoved = !Pin.LinkedTo.Num() && !Pin.bOrphanedPin;
	if (!bCanPinBeRemoved)
	{
		return false;
	}

	return IsMaterialSlotPin(Pin);
}


bool UCONodeComponentSkeletalMesh::CanRenamePin(const UEdGraphPin& Pin) const
{
	return IsMaterialSlotPin(Pin) || Super::CanRenamePin(Pin);
}


EEditablePinNameBoxVisibilityPolicy UCONodeComponentSkeletalMesh::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	if (IsMaterialSlotPin(Pin))
	{
		return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
	}

	return Super::GetEditablePinNameVisibilityPolicy(Pin);
}


bool UCONodeComponentSkeletalMesh::ShouldPinViewerShowPinEditableName(const UEdGraphPin& Pin) const
{
	if (IsMaterialSlotPin(Pin))
	{
		return true;
	}

	return Super::ShouldPinViewerShowPinEditableName(Pin);
}


void UCONodeComponentSkeletalMesh::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	if (IsMaterialSlotPin(Pin))
	{
		if (UCONodeComponentSkeletalMeshMaterialPinData* PinData = CastChecked<UCONodeComponentSkeletalMeshMaterialPinData>(GetPinData(Pin)))
		{
			const FName NewName = FName{Value.ToString()};
			if (NewName != PinData->TargetMaterialSlotName)
			{
				FScopedTransaction LocalTransaction(LOCTEXT("UCONodeComponentMesh_SetMaterialPinEditableNameTransaction", "Change Material PinName"));
				PinData->Modify();

				PinData->TargetMaterialSlotName = NewName;

				// Tell the listeners that this pin editable name has changed
				PinEditableNameChangedDelegate.Broadcast(Pin, Value);
			}
		}
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}


FText UCONodeComponentSkeletalMesh::GetPinEditableName(const UEdGraphPin& Pin) const
{
	if (IsMaterialSlotPin(Pin))
	{
		if (const UCONodeComponentSkeletalMeshMaterialPinData* PinData = Cast<UCONodeComponentSkeletalMeshMaterialPinData>(GetPinData(Pin)))
		{
			return FText::FromName(PinData->TargetMaterialSlotName);
		}
	}

	return Super::GetPinEditableName(Pin);
}


bool UCONodeComponentSkeletalMesh::IsMaterialSlotPin(const UEdGraphPin& Pin) const
{
	return MaterialSlotPins.Contains(&Pin);
}


UCustomizableObjectNodeRemapPins* UCONodeComponentSkeletalMesh::CreateRemapPinsDefault() const
{
	return NewObject<UCONodeComponentSkeletalMeshPinRemapper>();
}


bool UCONodeComponentSkeletalMesh::CustomRemovePin(UEdGraphPin& Pin)
{
	MaterialSlotPins.Remove(&Pin);
	return Super::CustomRemovePin(Pin);
}


TArray<FName> UCONodeComponentSkeletalMesh::GetPinAllowedSubTypes(const UEdGraphPin& Pin) const
{
	if (IsMaterialSlotPin(Pin))
	{
		const TArray<FName> PinAllowedSubTypes = {
			UEdGraphSchema_CustomizableObject::PSC_Material_Overlay,
			UEdGraphSchema_CustomizableObject::PSC_Material_Override
		};
		return PinAllowedSubTypes;
	}

	return Super::GetPinAllowedSubTypes(Pin);
}


UEdGraphPin* UCONodeComponentSkeletalMesh::CreateMaterialSlotPin(const FName& PinSubCategory, UCONodeComponentSkeletalMeshMaterialPinData* InPinData)
{
	const FName NewPinName =  TEXT("Overlay-or-Override");
	const FText NewPinFriendlyName = FText::FromString(
		UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(PinSubCategory).ToString() + " Material");

	UEdGraphPin* NewMaterialPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, NewPinName, NewPinFriendlyName,
		InPinData);
	NewMaterialPin->PinType.PinSubCategory = PinSubCategory;
	return NewMaterialPin;
}



#undef LOCTEXT_NAMESPACE


