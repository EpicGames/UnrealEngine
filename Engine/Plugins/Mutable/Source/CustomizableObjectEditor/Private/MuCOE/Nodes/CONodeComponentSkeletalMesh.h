// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeComponent.h"

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"
#include "MuR/System.h"

#include "CONodeComponentSkeletalMesh.generated.h"


class UCONodeMutableSkeletalMeshMake;
class UCONodeSkeletalMeshMake;


UCLASS()
class UCONodeComponentSkeletalMeshMaterialPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = NoCategory)
	FName TargetMaterialSlotName;
};



UCLASS()
class UCONodeComponentSkeletalMeshPinRemapper : public UCustomizableObjectNodeRemapPins
{
public:
	GENERATED_BODY()

	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin, const uint32 OldPinInex, const uint32 NewPinIndex) const;
	
	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};


UENUM()
enum class ECustomizableObjectSelectionOverride : uint8
{
	NoOverride = 0 UMETA(DisplayName = "No Override"),
	Disable    = 1 UMETA(DisplayName = "Disable"    ),
	Enable     = 2 UMETA(DisplayName = "Enable"     )
};


// Type replacing the UCustomizableObjectNodeComponentMesh and UCustomizableObjectNodeComponentPassthroughMesh nodes


UCLASS()
class UCONodeComponentSkeletalMesh : public UCustomizableObjectNodeComponent
{
	GENERATED_BODY()

public:
	
	// UEdGraphNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	//UCustomizableObjectNode interface
	virtual bool IsSingleOutputNode() const override;

	virtual bool HasPinViewer() const override;
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;

	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const override;
	virtual bool ShouldPinViewerShowPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	
	// Override / Overlay pin configuration
	virtual TArray<FName> GetPinAllowedSubTypes(const UEdGraphPin& Pin) const override;
	
	// Own interface
	FEdGraphPinReference GetOverlayMaterialAssetPin() const;
	
protected:
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	
private:
	UEdGraphPin* CreateMaterialSlotPin(const FName& PinSubCategory, UCONodeComponentSkeletalMeshMaterialPinData* InPinData);

	bool IsMaterialSlotPin(const UEdGraphPin& Pin) const;

public:
	
	/** All the Skeletal Meshes generated for this component will use the Reference Skeletal Mesh properties
	 * for everything Mutable doesn't create or modify. This includes data like LOD distances, Physics
	 * properties, Bounding Volumes, Base Skeleton, and more.
	 *
	 * The Reference Skeletal Mesh can be used as a placeholder mesh when there are too many actors or in 
	 * situations of stress where the generation of the Skeletal Mesh might take a few seconds to complete. */
	UPROPERTY(EditAnywhere, Category = ComponentMesh)
	TObjectPtr<USkeletalMesh> ReferenceSkeletalMesh;
	
	UPROPERTY()
	FEdGraphPinReference SkeletalMeshPin;

	UPROPERTY()
	FEdGraphPinReference OverlayMaterialPin;
	
	UPROPERTY()
	TArray<FEdGraphPinReference> MaterialSlotPins;
};
