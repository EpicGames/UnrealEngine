// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeExternalOperation.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuR/External/Operation.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeExternalOperation::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UE::Mutable::FExternalOperation* Operation = OperationInstancedStruct.GetPtr<UE::Mutable::FExternalOperation>();
	const FText Name = Operation ? OperationInstancedStruct.GetScriptStruct()->GetDisplayNameText() : CachedOperationName;
	
	if (TitleType == ENodeTitleType::ListView)
	{
		return Name;
	}
	else
	{
		return FText::Format(LOCTEXT("External_NodeTitle", "{0}\nExternal"), Name);
	}
}


FLinearColor UCONodeExternalOperation::GetNodeTitleColor() const
{
	const FName Category = UEdGraphSchema_CustomizableObject::GetPinCategoryName(OutputPin.Get()->PinType.PinCategory);
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(Category);
}


bool UCONodeExternalOperation::IsLoaded() const
{
	return OperationInstancedStruct.IsValid();
}


void UCONodeExternalOperation::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	Super::AllocateDefaultPins(RemapPins);

	const UE::Mutable::FExternalOperation* Operation = OperationInstancedStruct.GetPtr<UE::Mutable::FExternalOperation>();
	if (!Operation)
	{
		return;
	}
	
	// Inputs
	InputPins.Empty();
	
	for (TPair<FText, const UScriptStruct*> Pair : Operation->GetInputs())
	{
		const FName Category = UEdGraphSchema_CustomizableObject::GetPinCategoryName(Pair.Value);

		UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, Category, FName(Pair.Key.ToString()), Pair.Key);
		InputPins.Add(Pin);
	}

	// Output
	{
		const TPair<FText, const UScriptStruct*> Pair = Operation->GetOutput();

		const FName Category = UEdGraphSchema_CustomizableObject::GetPinCategoryName(Pair.Value);	
		OutputPin = CustomCreatePin(EGPD_Output, Category, FName(Pair.Key.ToString()), Pair.Key);
	}

	CachedOperationName = OperationInstancedStruct.GetScriptStruct()->GetDisplayNameText();
}


#undef LOCTEXT_NAMESPACE
