// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeExternalOperation.generated.h"


UCLASS()
class UCONodeExternalOperation : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool IsLoaded() const override;
	
	UPROPERTY()
	FInstancedStruct OperationInstancedStruct;

	UPROPERTY()
	TArray<FEdGraphPinReference> InputPins;

	UPROPERTY()
	FEdGraphPinReference OutputPin;
	
	/** In case of the operation not being loaded, the node uses this cached name so we can at least show the user which operation it was. */
	UPROPERTY()
	FText CachedOperationName;
};
