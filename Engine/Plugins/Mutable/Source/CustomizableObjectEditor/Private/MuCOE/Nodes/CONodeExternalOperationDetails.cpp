// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeExternalOperationDetails.h"

#include "Editor.h"
#include "CONodeExternalOperation.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IStructureDetailsView.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


TSharedRef<IDetailCustomization> FCONodeExternalOperationDetails::MakeInstance()
{
	return MakeShareable(new FCONodeExternalOperationDetails);
}


void FCONodeExternalOperationDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (!DetailsView || !DetailsView->GetSelectedObjects().Num())
	{
		return;
	}

	UCONodeExternalOperation* Node = Cast<UCONodeExternalOperation>(DetailsView->GetSelectedObjects()[0]);
	if (!Node)
	{
		return;
	}

	const UStruct* Struct = Node->OperationInstancedStruct.GetScriptStruct();
	if (!Struct)
	{
		return;
	}

	TSharedRef<FStructOnScope> StructData = MakeShared<FStructOnScope>(Struct, Node->OperationInstancedStruct.GetMutableMemory());

	for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		FName PropertyName = Property->GetFName();
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.AddStructurePropertyData(StructData, PropertyName);
		IDetailPropertyRow& DetailPropertyRow = DetailBuilder.AddPropertyToCategory(PropertyHandle.ToSharedRef());

		TSharedPtr<IPropertyHandle> PropertyRowHandle = DetailPropertyRow.GetPropertyHandle();
		
		PropertyRowHandle->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda([Node]()
		{
			GEditor->BeginTransaction(LOCTEXT("EditProperty", "Edited property"));
			Node->Modify();
		}));
		
		PropertyRowHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([this](const FPropertyChangedEvent& InEvent)
		{
			GEditor->EndTransaction();
		}));

		PropertyRowHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda([Node]()
		{
			GEditor->BeginTransaction(LOCTEXT("EditChildProperty", "Edited child property"))	;
			Node->Modify();
		}));

		PropertyRowHandle->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([this](const FPropertyChangedEvent& InEvent)
		{
			GEditor->EndTransaction();
		}));
	}
}


void FCONodeExternalOperationDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	IDetailCustomization::CustomizeDetails(DetailBuilder);

	DetailBuilderPtr = DetailBuilder;
}


#undef LOCTEXT_NAMESPACE
