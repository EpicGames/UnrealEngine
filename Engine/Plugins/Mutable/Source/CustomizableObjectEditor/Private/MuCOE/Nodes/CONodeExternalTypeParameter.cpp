// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeExternalTypeParameter.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeExternalTypeParameter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const FText Title = Super::GetNodeTitle(TitleType);
	
	if (TitleType == ENodeTitleType::ListView)
	{
		return Title;
	}
	else
	{
		return FText::Format(LOCTEXT("ExternalParameter_NodeTitle", "{0}\nExternal"), Title);
	}
}


FName UCONodeExternalTypeParameter::GetCategory() const
{
	if (DefaultValue.IsValid())
	{
		CachedType = UEdGraphSchema_CustomizableObject::GetPinCategoryName(DefaultValue.GetScriptStruct());
	}
	
	return CachedType;
}


FText UCONodeExternalTypeParameter::GetCategoryFriendlyName() const
{
	if (DefaultValue.IsValid())
	{
		CachedFriendlyName = UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetCategory());
	}

	return CachedFriendlyName;
}


bool UCONodeExternalTypeParameter::IsLoaded() const
{
	return DefaultValue.IsValid();
}


bool UCONodeExternalTypeParameter::IsAffectedByLOD() const
{
	return false;
}


#undef LOCTEXT_NAMESPACE

