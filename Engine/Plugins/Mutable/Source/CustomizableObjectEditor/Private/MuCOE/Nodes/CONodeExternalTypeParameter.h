// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeParameter.h"

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeExternalTypeParameter.generated.h"

class UScriptStruct;


UCLASS()
class UCONodeExternalTypeParameter : public UCustomizableObjectNodeParameter
{
	GENERATED_BODY()

public:
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	// UCustomizableObjectNode interface
	virtual bool IsLoaded() const override;
	virtual bool IsAffectedByLOD() const override;

	// UCustomizableObjectNodeParameter interface
	virtual FName GetCategory() const override;
	virtual FText GetCategoryFriendlyName() const override;

	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (MutableExternalType))
	FInstancedStruct DefaultValue;

	/** In case of the operation not being loaded, the node uses this cached name so we can at least show the user which type it was. */
	UPROPERTY()
	mutable FName CachedType;

	UPROPERTY()
	mutable FText CachedFriendlyName;
};
