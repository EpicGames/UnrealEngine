// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeMaterialBreak.h"

#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeMaterialBreak)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

const FName BreakMaterialMaterialParameterPinName = "Parameter";
const FText BreakMaterialMaterialParameterPinText = LOCTEXT("Parameter", "Parameter");


void UCONodeMaterialBreak::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	MaterialPinRef = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material);
	
	// Get old pins
	TArray<UEdGraphPin*> OldPins = GetAllNonOrphanPins();

	// Regenerate the new pins from the old pin's PinData which may have changed
	for (const UEdGraphPin* Pin : OldPins)
	{
		if (Pin->Direction != EGPD_Output)
		{
			continue;
		}
		
		if (UCONodeMaterialBreakParameterPinData* PinData = Cast<UCONodeMaterialBreakParameterPinData>(GetPinData(*Pin)))
		{
			// Use the same pin data as the old pin since the new one is a copy
			UEdGraphPin* NewPin = CustomCreatePin(EGPD_Output, Pin->PinType.PinCategory, BreakMaterialMaterialParameterPinName, BreakMaterialMaterialParameterPinText, PinData);

			if (PinData->LayerIndex == 0)
			{
				NewPin->PinFriendlyName = FText::Format(LOCTEXT("BackgroundLayerText", "{0} - Background"), FText::FromName(BreakMaterialMaterialParameterPinName));
			}
			else if (PinData->LayerIndex > 0)
			{
				NewPin->PinFriendlyName = FText::Format(LOCTEXT("LayerText", "{0} - Layer {1}"), FText::FromName(BreakMaterialMaterialParameterPinName), FText::FromString(FString::FromInt(PinData->LayerIndex)));
			}
		}
	}
}


void UCONodeMaterialBreakParameterPinData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeMaterialBreakParameterPinData, LayerIndex))
		{
			CastChecked<UCustomizableObjectNode>(GetOuter())->ReconstructNode();
		}
	}
}


FText UCONodeMaterialBreak::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Material_Break", "Material Break");
}


FLinearColor UCONodeMaterialBreak::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Material);
}


FText UCONodeMaterialBreak::GetTooltipText() const
{
	return LOCTEXT("Material_Break_Tooltip", "Allows to get the parameter value from the input material node.");
}


TSharedPtr<IDetailsView> UCONodeMaterialBreak::CustomizePinDetails(const UEdGraphPin& Pin) const
{
	if (UCONodeMaterialBreakParameterPinData* PinData = Cast<UCONodeMaterialBreakParameterPinData>(GetPinData(Pin)))
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;

		const TSharedRef<IDetailsView> SettingsView = EditModule.CreateDetailView(DetailsViewArgs);
		SettingsView->SetObject(PinData);

		return SettingsView;
	}
	else
	{
		return nullptr;
	}
}


bool UCONodeMaterialBreak::HasPinViewer() const
{
	return true;
}


bool UCONodeMaterialBreak::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	const bool bCanPinBeRemoved = !Pin.LinkedTo.Num() && !Pin.bOrphanedPin;
	if (!bCanPinBeRemoved)
	{
		return false;
	}
	
	return Pin.Direction == EGPD_Output;
}


EAddPinNodeButtonLocation UCONodeMaterialBreak::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::OUTPUT;
}


UCustomizableObjectNodeRemapPins* UCONodeMaterialBreak::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByPosition>();
}


TArray<FName> UCONodeMaterialBreak::GetAllowedPinViewerCreationTypes() const
{
	return { 
		UEdGraphSchema_CustomizableObject::PC_Texture, 
		UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough, 
		UEdGraphSchema_CustomizableObject::PC_Color, 
		UEdGraphSchema_CustomizableObject::PC_Float 
	};
}


TArray<FName> UCONodeMaterialBreak::GetPinAllowedTypes(const UEdGraphPin& Pin) const
{
	if (Pin.Direction == EGPD_Output)
	{
		const TArray<FName> PinAllowedTypes = GetAllowedPinViewerCreationTypes();
		check(PinAllowedTypes.Contains(Pin.PinType.PinCategory));
		
		return PinAllowedTypes;
	}
	else
	{	
		return Super::GetPinAllowedTypes(Pin);
	}
}


void UCONodeMaterialBreak::AddPinFromUI()
{
	Super::AddPinFromUI();

	FScopedTransaction Transaction(LOCTEXT("CONodeMaterialBreak_AddPinFromNodeUITransaction", "Add Pin from Node UI"));
	Modify();
	
	const FName NewPinCategory = UEdGraphSchema_CustomizableObject::PC_Float;
	UCONodeMaterialBreakParameterPinData* PinData = NewObject<UCONodeMaterialBreakParameterPinData>(this);
	if (CustomCreatePin(EEdGraphPinDirection::EGPD_Output, NewPinCategory, BreakMaterialMaterialParameterPinName, BreakMaterialMaterialParameterPinText, PinData))
	{
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCONodeMaterialBreak::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.Direction == EGPD_Output;
}


EEditablePinNameBoxVisibilityPolicy UCONodeMaterialBreak::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


void UCONodeMaterialBreak::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (UCONodeMaterialBreakParameterPinData* CastedPinData = Cast<UCONodeMaterialBreakParameterPinData>(PinData))
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCONodeMaterialBreak_SetNamePinEditableNameTransaction", "Change Pin Name"));
		CastedPinData->Modify();
		
		CastedPinData->ParameterName = FName{Value.ToString()};
		PinEditableNameChangedDelegate.Broadcast(Pin, Value);
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}


FText UCONodeMaterialBreak::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (const UCONodeMaterialBreakParameterPinData* CastedPinData = Cast<UCONodeMaterialBreakParameterPinData>(PinData))
	{
		return FText::FromName(CastedPinData->ParameterName);
	}

	return Super::GetPinEditableName(Pin);
}


FName UCONodeMaterialBreak::GetPinParameterName(const UEdGraphPin& Pin) const
{
	return GetPinData<UCONodeMaterialBreakParameterPinData>(Pin).ParameterName;
}


int32 UCONodeMaterialBreak::GetPinParameterLayerIndex(const UEdGraphPin& Pin) const
{
	return GetPinData<UCONodeMaterialBreakParameterPinData>(Pin).LayerIndex;
}


#undef LOCTEXT_NAMESPACE
