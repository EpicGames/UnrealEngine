// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeMaterialBreak.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }
class UCustomizableObjectNodeRemapPins;


UCLASS(MinimalAPI)
class UCONodeMaterialBreakParameterPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:
	
	/** Layer index of this parameter inside a Layered Material.
	-1 if the material does not contain layers.*/
	UPROPERTY(EditAnywhere, Category = NoCategory)
	int32 LayerIndex = -1;
	
	UPROPERTY()
	FName ParameterName;
	
	/**
	 * Handle change in the LayerIndex property
	 * @param PropertyChangedEvent The Property Change Event
	 */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};


UCLASS(MinimalAPI)
class UCONodeMaterialBreak : public UCustomizableObjectNode
{
public:

	GENERATED_BODY()

	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual TSharedPtr<IDetailsView> CustomizePinDetails(const UEdGraphPin& Pin) const override;
	virtual bool HasPinViewer() const override;
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	virtual TArray<FName> GetAllowedPinViewerCreationTypes() const override;
	virtual TArray<FName> GetPinAllowedTypes(const UEdGraphPin& Pin) const override;

	// UCustomizableObjectNode Interface : Editable area of the pin name management
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	
	// Own interface
	FName GetPinParameterName(const UEdGraphPin& Pin) const;
	int32 GetPinParameterLayerIndex(const UEdGraphPin& Pin) const;

	UPROPERTY()
	FEdGraphPinReference MaterialPinRef;
};

#undef UE_API
