// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeMaterialConstant.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeMaterialConstant)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCONodeMaterialConstant::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{ 
	MaterialPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Material);
}


void UCONodeMaterialConstant::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::StoreMaterialPinReferenceFromConstantMaterialNode)
	{
		for (const UEdGraphPin* NodePin : Pins)
		{
			if (NodePin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Material)
			{
				MaterialPin = NodePin;
				break;
			}
		}
	}
}


FText UCONodeMaterialConstant::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Material_Constant", "Material");
}


FLinearColor UCONodeMaterialConstant::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Material);
}


FText UCONodeMaterialConstant::GetTooltipText() const
{
	return LOCTEXT("Material_Constant_Tooltip", "Define an Unreal Engine Material that does not change at runtime.");
}

#undef LOCTEXT_NAMESPACE
