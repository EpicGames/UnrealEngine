// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "CONodeMaterialConstant.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }
class UCustomizableObjectNodeRemapPins;

UCLASS(MinimalAPI)
class UCONodeMaterialConstant : public UCustomizableObjectNode
{
public:

	GENERATED_BODY()

	// UCustomizableObjectNode interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	/** Value of the Node. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TSoftObjectPtr<UMaterialInterface> Material;

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	UPROPERTY()
	FEdGraphPinReference MaterialPin;
};

#undef UE_API
