// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeMaterialModify.h"

#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


const FName MaterialModifyMaterialParameterPinName = "Parameter";
const FText MaterialModifyMaterialParameterPinText = LOCTEXT("Parameter", "Parameter");


void UCONodeMaterialModifyParameterPinData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeMaterialModifyParameterPinData, LayerIndex))
		{
			CastChecked<UCustomizableObjectNode>(GetOuter())->ReconstructNode();
		}
	}
}


FText UCONodeMaterialModify::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("ModifyMaterialNodeTitle","Material Modify");
}


FLinearColor UCONodeMaterialModify::GetNodeTitleColor() const
{
	return GetDefault<UEdGraphSchema_CustomizableObject>()->GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Material);
}


FText UCONodeMaterialModify::GetTooltipText() const
{
	return LOCTEXT("ModifyMaterialNodeTooltip","Modifies a new material instance from the Reference Material.");
}


void UCONodeMaterialModify::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Input Material Pin
	const FName Type = UEdGraphSchema_CustomizableObject::PC_Material;
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(Type);
	const FText PinFriendlyName = UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(Type);
	
	MaterialPinRef = CustomCreatePin(EGPD_Input, Type, PinName, PinFriendlyName);
	
	//Output Material Pin
	CustomCreatePin(EGPD_Output, Type, PinName, PinFriendlyName);

	// Parameter Pins
	// Get old pins
	TArray<UEdGraphPin*> InputPins = GetAllNonOrphanPins();

	// Regenerate the new pins from the old pin's pindata which may have changed
	for (const UEdGraphPin* InputPin : InputPins)
	{
		if (InputPin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			if (UCONodeMaterialModifyParameterPinData* OldPinData = Cast<UCONodeMaterialModifyParameterPinData>(GetPinData(*InputPin)))
			{
				UCONodeMaterialModifyParameterPinData* NewPinData = nullptr;
				
				if (InputPin->PinType.PinCategory ==  UEdGraphSchema_CustomizableObject::PC_Float )
				{
					NewPinData = NewObject<UCONodeMaterialModifyScalarParamPinData>(this);
				}
				else if (InputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture )
				{
					NewPinData = NewObject<UCONodeMaterialModifyTextureParamPinData>(this);
				}
				else if (InputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough )
				{
					NewPinData = NewObject<UCONodeMaterialModifyPassthroughTextureParamPinData>(this);
				}
				else if (InputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Color )
				{
					NewPinData = NewObject<UCONodeMaterialModifyColorParamPinData>(this);
				}
				
				check(NewPinData)
				if (NewPinData)
				{
					UEdGraphPin* NewPin = CustomCreatePin(EGPD_Input, InputPin->PinType.PinCategory, MaterialModifyMaterialParameterPinName, MaterialModifyMaterialParameterPinText,  NewPinData);
					check(NewPin)
				
					if (OldPinData->LayerIndex == 0)
					{
						NewPin->PinFriendlyName = FText::Format(LOCTEXT("BackgroundLayerText","{0} - Background"), FText::FromName(MaterialModifyMaterialParameterPinName));
					}
					else if (OldPinData->LayerIndex > 0)
					{
						NewPin->PinFriendlyName = FText::Format(LOCTEXT("LayerText", "{0} - Layer {1}"), FText::FromName(MaterialModifyMaterialParameterPinName), FText::FromString(FString::FromInt(OldPinData->LayerIndex)));
					}
				}
			}
		}
	}
}


TSharedPtr<IDetailsView> UCONodeMaterialModify::CustomizePinDetails(const UEdGraphPin& Pin) const
{
	if (UCONodeMaterialModifyParameterPinData* PinData = Cast<UCONodeMaterialModifyParameterPinData>(GetPinData(Pin)))
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;

		const TSharedRef<IDetailsView> SettingsView = EditModule.CreateDetailView(DetailsViewArgs);
		SettingsView->SetObject(PinData);

		return SettingsView;
	}
	else
	{
		return nullptr;
	}
}


bool UCONodeMaterialModify::HasPinViewer() const
{
	return true;
}


EAddPinNodeButtonLocation UCONodeMaterialModify::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::INPUT;
}


UCustomizableObjectNodeRemapPins* UCONodeMaterialModify::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByPosition>();
}


TArray<FName> UCONodeMaterialModify::GetAllowedPinViewerCreationTypes() const
{
	return { GetDefault<UEdGraphSchema_CustomizableObject>()->PC_Texture,
		GetDefault<UEdGraphSchema_CustomizableObject>()->PC_Texture_Passthrough,
		GetDefault<UEdGraphSchema_CustomizableObject>()->PC_Color,
		GetDefault<UEdGraphSchema_CustomizableObject>()->PC_Float };
}


TArray<FName> UCONodeMaterialModify::GetPinAllowedTypes(const UEdGraphPin& Pin) const
{
	if (Pin.Direction == EGPD_Input && FEdGraphPinReference{&Pin} != MaterialPinRef)
	{
		const TArray<FName> PinAllowedTypes = GetAllowedPinViewerCreationTypes();
		check(PinAllowedTypes.Contains(Pin.PinType.PinCategory));
		
		return PinAllowedTypes;
	}
	else
	{	
		return Super::GetPinAllowedTypes(Pin);
	}
}


void UCONodeMaterialModify::AddPinFromUI()
{
	Super::AddPinFromUI();

	FScopedTransaction Transaction(LOCTEXT("CONodeMaterialModify_AddPinFromNodeUITransaction", "Add Pin from Node UI"));
	Modify();
	
	UCONodeMaterialModifyScalarParamPinData* ScalarParameterPinData = NewObject<UCONodeMaterialModifyScalarParamPinData>(this);
	if (UEdGraphPin* NewPin = CustomCreatePin(EEdGraphPinDirection::EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float,
		MaterialModifyMaterialParameterPinName, MaterialModifyMaterialParameterPinText, ScalarParameterPinData))
	{
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCONodeMaterialModify::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	return Pin.Direction == EGPD_Input && !Pin.LinkedTo.Num() && !Pin.bOrphanedPin && &Pin != MaterialPinRef.Get();
}


FName UCONodeMaterialModify::GetPinParameterName(const UEdGraphPin& Pin) const
{
	return GetPinData<UCONodeMaterialModifyParameterPinData>(Pin).ParameterName;
}


int32 UCONodeMaterialModify::GetPinParameterLayerIndex(const UEdGraphPin& Pin) const
{
	return GetPinData<UCONodeMaterialModifyParameterPinData>(Pin).LayerIndex;
}


TSoftObjectPtr<UTexture2D> UCONodeMaterialModify::GetTexturePinParameterReferenceTexture(const UEdGraphPin& Pin) const
{
	const UCONodeMaterialModifyTextureParamPinData& TextureParameterPnData = GetPinData<UCONodeMaterialModifyTextureParamPinData>(Pin);
	return TextureParameterPnData.ReferenceTexture;
}


int32 UCONodeMaterialModify::GetImagePinParameterUVIndex(const UEdGraphPin& Pin) const
{
	const UCONodeMaterialModifyTextureParamPinData& TextureParameterPnData = GetPinData<UCONodeMaterialModifyTextureParamPinData>(Pin);
	switch(TextureParameterPnData.UVLayoutMode)
	{
	case EMaterialModifyUVLayoutMode::Ignore:
		return UCONodeSkeletalMeshSectionPinDataImage::UV_LAYOUT_IGNORE;
	case EMaterialModifyUVLayoutMode::Index:
		return TextureParameterPnData.UVLayout;
	default:
		unimplemented();
	}
		
	return UCONodeSkeletalMeshSectionPinDataImage::UV_LAYOUT_IGNORE;
}


void UCONodeMaterialModify::GetMaterialTextureParameterPins(TArray<FEdGraphPinReference>& OutTextureParameterPins) const
{
	TArray<UEdGraphPin*> AllNonOrphanPins = GetAllNonOrphanPins();

	for (UEdGraphPin* NonOrphanPin : AllNonOrphanPins)
	{
		if (NonOrphanPin && 
			NonOrphanPin->Direction == EGPD_Input &&
			NonOrphanPin->PinType.PinCategory == GetDefault<UEdGraphSchema_CustomizableObject>()->PC_Texture)
		{
			check(GetPinData(*NonOrphanPin)->IsA<UCONodeMaterialModifyTextureParamPinData>());
			OutTextureParameterPins.Add(NonOrphanPin);
		}
	}
}


bool UCONodeMaterialModify::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.Direction == EGPD_Input && FEdGraphPinReference{&Pin} != MaterialPinRef;
}


EEditablePinNameBoxVisibilityPolicy UCONodeMaterialModify::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


void UCONodeMaterialModify::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (UCONodeMaterialModifyParameterPinData* CastedPinData = Cast<UCONodeMaterialModifyParameterPinData>(PinData))
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCONodeMaterialModify_SetNamePinEditableNameTransaction", "Change Pin Name"));
		CastedPinData->Modify();
		
		CastedPinData->ParameterName = FName{Value.ToString()};
		PinEditableNameChangedDelegate.Broadcast(Pin, Value);
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}


FText UCONodeMaterialModify::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (const UCONodeMaterialModifyParameterPinData* CastedPinData = Cast<UCONodeMaterialModifyParameterPinData>(PinData))
	{
		return FText::FromName(CastedPinData->ParameterName);
	}

	return Super::GetPinEditableName(Pin);
}


#undef LOCTEXT_NAMESPACE
