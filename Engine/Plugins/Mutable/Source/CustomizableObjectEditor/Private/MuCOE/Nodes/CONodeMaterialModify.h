// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CONodeSkeletalMeshSection.h"
#include "IDetailCustomization.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "CONodeMaterialModify.generated.h"

class UCustomizableObjectNodeRemapPinsByName;


// todo: Remove this enum and use the "EUVLayoutMode" so we support the option of getting the UV layout from the material  
/** Image Pin, UV Layout Mode. */
UENUM()
enum class EMaterialModifyUVLayoutMode
{
	/* Texture should not be transformed by any layout. These textures will not be reduced automatically for LODs. */
	Ignore,
	/** User specified UV Index. */
	Index
};


/** Base class for all Material Parameters. */
UCLASS()
class UCONodeMaterialModifyParameterPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:
	
	/** Layer index of this parameter inside a Layered Material.
	-1 if the material does not contain layers.*/
	UPROPERTY(EditAnywhere, Category = NoCategory)
	int32 LayerIndex = -1;
	
	UPROPERTY()
	FName ParameterName;
	
	/**
	 * Handle change in the LayerIndex property
	 * @param PropertyChangedEvent The Property Change Event
	 */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	

	virtual void Copy(const UCustomizableObjectNodePinData& Other) override
	{
		if (const UCONodeMaterialModifyParameterPinData* PinDataOldPin = Cast<UCONodeMaterialModifyParameterPinData>(&Other))
		{
			LayerIndex = PinDataOldPin->LayerIndex;
			ParameterName = PinDataOldPin->ParameterName;
		}
	}
};


/**
 * PinData object representing pins of the PC_Texture type.
 */
UCLASS()
class UCONodeMaterialModifyTextureParamPinData : public UCONodeMaterialModifyParameterPinData
{
	GENERATED_BODY()
	
public:
	
	virtual void Copy(const UCustomizableObjectNodePinData& Other) override
	{
		Super::Copy(Other);
		
		if (const UCONodeMaterialModifyTextureParamPinData* PinDataOldPin = Cast<UCONodeMaterialModifyTextureParamPinData>(&Other))
		{
            UVLayoutMode = PinDataOldPin->UVLayoutMode;
            UVLayout = PinDataOldPin->UVLayout;
            ReferenceTexture = PinDataOldPin->ReferenceTexture;
		}
	}

	constexpr static int32 UV_LAYOUT_IGNORE = -1;
	
	UPROPERTY(EditAnywhere, Category = NoCategory)
	EMaterialModifyUVLayoutMode UVLayoutMode = EMaterialModifyUVLayoutMode::Ignore;
	
	/** Index of the UV channel that will be used with this image. It is necessary to apply the proper layout transformations to it. */
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (EditCondition = "UVLayoutMode == EMaterialModifyUVLayoutMode::Index", EditConditionHides))
	int32 UVLayout = -2;
	
	/** Reference Texture used to decide the texture properties of the mutable-generated textures
	* connected to this material. If null, it will try to be guessed at compile time from
	* the graph. */
	UPROPERTY(EditAnywhere, Category = NoCategory) // Required to be EditAnywhere for the selector to work.
	TSoftObjectPtr<UTexture2D> ReferenceTexture = nullptr;
};

/**
 * PinData object representing pins of the PC_PassthroughTexture type.
 */
UCLASS()
class UCONodeMaterialModifyPassthroughTextureParamPinData : public UCONodeMaterialModifyParameterPinData
{
	GENERATED_BODY()
};


/**
 * PinData object representing pins of the PC_Float type.
 */
UCLASS()
class UCONodeMaterialModifyScalarParamPinData : public UCONodeMaterialModifyParameterPinData
{
	GENERATED_BODY()
};


/**
 * PinData object representing pins of the PC_Color type.
 */
UCLASS()
class UCONodeMaterialModifyColorParamPinData : public UCONodeMaterialModifyParameterPinData
{
	GENERATED_BODY()
};


UCLASS()
class UCONodeMaterialModify : public UCustomizableObjectNode
{
public:

	GENERATED_BODY()

	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

protected:
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	
public:
	// UCustomizableObjectNode interface
	virtual TSharedPtr<IDetailsView> CustomizePinDetails(const UEdGraphPin& Pin) const override;
	virtual bool HasPinViewer() const override;
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;
	virtual TArray<FName> GetAllowedPinViewerCreationTypes() const override;
	virtual TArray<FName> GetPinAllowedTypes(const UEdGraphPin& Pin) const override;

	// Own Interface : Editable area of the pin name management
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	
	// Own interface
	FName GetPinParameterName(const UEdGraphPin& Pin) const;
	int32 GetPinParameterLayerIndex(const UEdGraphPin& Pin) const;
	TSoftObjectPtr<UTexture2D> GetTexturePinParameterReferenceTexture(const UEdGraphPin& Pin) const;
	int32 GetImagePinParameterUVIndex(const UEdGraphPin& Pin) const;
	void GetMaterialTextureParameterPins(TArray<FEdGraphPinReference>& OutTextureParameterPins) const;
	
	/* Settings used to define the maximum number of blocks the Layouts will have.The number of blocks and the reference textures are used
	 to define the final texture size of runtime generated textures. */
	UPROPERTY(Category="Layout Settings", EditAnywhere, DisplayName="UV Packaging Settings")
	FLayoutSettings UVPackagingSettings[4];
	
	UPROPERTY()
	FEdGraphPinReference MaterialPinRef;
};
