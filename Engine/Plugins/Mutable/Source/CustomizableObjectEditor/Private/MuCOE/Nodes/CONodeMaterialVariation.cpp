// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeMaterialVariation.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeMaterialVariation)

FName UCONodeMaterialVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Material;
}
