// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeModifierEditSkeletalMeshSection.h"

#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeLayoutBlocks.h"
#include "MuCOE/GraphTraversal.h"
#include "Materials/MaterialParameters.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeModifierEditSkeletalMeshSection)

class FCustomizableObjectNodeParentedMaterial;
class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCONodeModifierEditSkeletalMeshSection::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const int32 NumImages = GetNumParameters(EMaterialParameterType::Texture);
	for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
	{
		{
			UCustomizableObjectNodeEditMaterialPinEditImageData* PinEditImageData = NewObject<UCustomizableObjectNodeEditMaterialPinEditImageData>(this);
			PinEditImageData->ImageParamId = GetParameterId(EMaterialParameterType::Texture, ImageIndex);
			
			const FName ImageName = GetParameterName(EMaterialParameterType::Texture, ImageIndex);
			UEdGraphPin* PinImage = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, ImageName, FText::FromName(ImageName), PinEditImageData);
			PinImage->bHidden = true;

			PinsParameterMap.Add(PinEditImageData->ImageParamId, FEdGraphPinReference(PinImage));

			FString PinMaskName = ImageName.ToString() + FString(TEXT(" Mask"));
			UEdGraphPin* PinMask = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, *PinMaskName, FText::FromString(PinMaskName));
			PinMask->bHidden = true;
			
			PinEditImageData->PinMask = FEdGraphPinReference(PinMask);
		}
	}
	
	Super::AllocateDefaultPins(RemapPins);

	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));
}


const UEdGraphPin* UCONodeModifierEditSkeletalMeshSection::GetUsedImageMaskPin(const FNodeMaterialParameterId& ImageId) const
{
	if (const UEdGraphPin* Pin = GetUsedImagePin(ImageId))
	{
		UCustomizableObjectNodeEditMaterialPinEditImageData& PinData = GetPinData<UCustomizableObjectNodeEditMaterialPinEditImageData>(*Pin);
		return PinData.PinMask.Get();
	}

	return nullptr;
}


bool UCONodeModifierEditSkeletalMeshSection::IsSingleOutputNode() const
{
	return true;
}


bool UCONodeModifierEditSkeletalMeshSection::CustomRemovePin(UEdGraphPin& Pin)
{
	for (TMap<FNodeMaterialParameterId, FEdGraphPinReference>::TIterator It = PinsParameterMap.CreateIterator(); It; ++It)
	{
		if (It.Value().Get() == &Pin) // We could improve performance if FEdGraphPinReference exposed the pin id.
		{
			It.RemoveCurrent();
			break;
		}
	}

	return Super::CustomRemovePin(Pin);
}


bool UCONodeModifierEditSkeletalMeshSection::HasPinViewer() const
{
	return true;
}


void UCONodeModifierEditSkeletalMeshSection::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	const auto GetLayouts = [](UCONodeSkeletalMeshSection& MeshSectionNode, TArray<UCustomizableObjectLayout*>& OutLayouts)
		{
			UEdGraphPin* MeshPin = MeshSectionNode.GetMeshPin();
			if (!MeshPin)
			{
				return;
			}

			const UEdGraphPin* ConnectedPin = FollowInputPin(*MeshPin);
			if (!ConnectedPin)
			{
				return;
			}

			const UEdGraphPin* SourceMeshPin = FindMeshBaseSource(*ConnectedPin, false);
			if (!SourceMeshPin)
			{
				return;
			}

			if (const UCustomizableObjectNodeSkeletalMesh* MeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(SourceMeshPin->GetOwningNode()))
			{
				UCustomizableObjectNodeSkeletalMeshPinDataMesh* MeshPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(MeshNode->GetPinData(*SourceMeshPin));
				
				if (!MeshPinData)
				{
					return;
				}

				// The custom version of SkeletalMesh nodes may be up to date if they're in a different CO. 
				OutLayouts = MeshPinData->Layouts;

				if (OutLayouts.IsEmpty())
				{
					// Pre FCustomizableObjectCustomVersion::RemoveNodeLayout code. Get Layouts from NodeLayoutBlocks
					TArray<UEdGraphPin*> NonOrphanPins = MeshNode->GetAllNonOrphanPins();
					for (const UEdGraphPin* Pin : NonOrphanPins)
					{
						if (const UCustomizableObjectNodeSkeletalMeshPinDataLayout* PinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataLayout>(MeshNode->GetPinData(*Pin)))
						{
							if (PinData->GetLODIndex() == MeshPinData->GetLODIndex() &&
								PinData->GetSectionIndex() == MeshPinData->GetSectionIndex())
							{
								if (const UEdGraphPin* SourceLayoutConnectedPin = FollowInputPin(*Pin))
								{
									const UCustomizableObjectNodeLayoutBlocks* LayoutNode = Cast<UCustomizableObjectNodeLayoutBlocks>(SourceLayoutConnectedPin->GetOwningNode());
									if (LayoutNode && LayoutNode->Layout)
									{
										OutLayouts.Add(LayoutNode->Layout);
									}
								}
							}
						}
					}
				}
			}
			else if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(SourceMeshPin->GetOwningNode()))
			{
				OutLayouts = TableNode->GetLayouts(SourceMeshPin);
			}
		};

	// Convert deprecated node index list to the node id list.
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PostLoadToCustomVersion
		&& BlockIds_DEPRECATED.Num() < Blocks_DEPRECATED.Num())
	{
		UCONodeSkeletalMeshSection* ParentMaterialNode = GetCustomizableObjectExternalNode<UCONodeSkeletalMeshSection>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);

		TArray<UCustomizableObjectLayout*> Layouts;
		if (ParentMaterialNode)
		{
			GetLayouts(*ParentMaterialNode, Layouts);
		}

		if (!Layouts.IsValidIndex(ParentLayoutIndex))
		{
			UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierEditSkeletalMeshSection refers to an invalid texture layout index %d. Parent node has %d layouts.", 
				*GetOutermost()->GetName(), ParentLayoutIndex, Layouts.Num());
		}
		else
		{
			UCustomizableObjectLayout* ParentLayout = Layouts[ParentLayoutIndex];

			if (Cast<UCONodeSkeletalMeshSection>(ParentMaterialNode))
			{
				for (int32 IndexIndex = BlockIds_DEPRECATED.Num(); IndexIndex < Blocks_DEPRECATED.Num(); ++IndexIndex)
				{
					int32 BlockIndex = Blocks_DEPRECATED[IndexIndex];
					if (ParentLayout->Blocks.IsValidIndex(BlockIndex) )
					{
						const FGuid Id = ParentLayout->Blocks[BlockIndex].Id;
						if (Id.IsValid())
						{
							BlockIds_DEPRECATED.Add(Id);
						}
						else
						{
							UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierEditSkeletalMeshSection refers to an valid layout block %d but that block doesn't have an id.",
								*GetOutermost()->GetName(), BlockIndex );
						}
					}
					else
					{
						UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierEditSkeletalMeshSection refers to an invalid layout block index %d. Parent node has %d blocks.",
							*GetOutermost()->GetName(), BlockIndex, ParentLayout->Blocks.Num());
					}
				}
			}
		}
	}
	
	// Convert deprecated node id list to absolute rect list.
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UseUVRects)
	{
		// If we are here, it means this node was loaded from a version that didn't have it's own layout.
		check(Layout->Blocks.IsEmpty());

		UCONodeSkeletalMeshSection* ParentMaterialNode = GetCustomizableObjectExternalNode<UCONodeSkeletalMeshSection>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);

		TArray<UCustomizableObjectLayout*> ParentLayouts;
		if (ParentMaterialNode)
		{
			GetLayouts(*ParentMaterialNode, ParentLayouts);
		}

		if (!ParentLayouts.IsValidIndex(ParentLayoutIndex))
		{
			UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierEditSkeletalMeshSection refers to an invalid texture layout index %d. Parent node has %d layouts.",
				*GetOutermost()->GetName(), ParentLayoutIndex, ParentLayouts.Num());
		}
		else
		{
			UCustomizableObjectLayout* ParentLayout = ParentLayouts[ParentLayoutIndex];
			FIntPoint GridSize = ParentLayout->GetGridSize();

			Layout->SetGridSize(GridSize);

			if (Cast<UCONodeSkeletalMeshSection>(ParentMaterialNode))
			{
				for (const FGuid& BlockId : BlockIds_DEPRECATED)
				{
					bool bSkipBlock = false;
					for (const FCustomizableObjectLayoutBlock& ExistingBlock : Layout->Blocks)
					{
						if (ExistingBlock.Id == BlockId)
						{
							bSkipBlock = true;
							UE_LOGF(LogMutable, Log, "[%ls] UCONodeModifierEditSkeletalMeshSection has a duplicated layout block id. One has been ignored during version upgrade.", *GetOutermost()->GetName());
							break;
						}
					}

					if (bSkipBlock)
					{
						continue;
					}

					bool bFoundInParent = false;
					for (const FCustomizableObjectLayoutBlock& ParentBlock : ParentLayout->Blocks)
					{
						if (ParentBlock.Id == BlockId)
						{
							bFoundInParent = true;

							FCustomizableObjectLayoutBlock NewBlock;
							NewBlock = ParentBlock;

							// Clear some unnecessary data.
							NewBlock.bReduceBothAxes = false;
							NewBlock.bReduceByTwo = false;
							NewBlock.Priority = 0;

							Layout->Blocks.Add(NewBlock);
							break;
						}
					}

					if (!bFoundInParent)
					{
						UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierEditSkeletalMeshSection refers to and invalid layout block. It has been ignored during version upgrade.", *GetOutermost()->GetName());
					}
				}
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeMaterial)
	{
		UCONodeSkeletalMeshSection* ParentMaterial = GetCustomizableObjectExternalNode<UCONodeSkeletalMeshSection>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);

		if (ParentMaterial)
		{
			for (const FCustomizableObjectNodeEditMaterialImage& Image : Images_DEPRECATED)
			{
				const UEdGraphPin* ImagePin = FindPin(Image.Name);
				const UEdGraphPin* PinMask = FindPin(Image.Name + " Mask");
				if (!ImagePin || !PinMask)
				{
					continue;
				}
				
				UCustomizableObjectNodeEditMaterialPinEditImageData*  PinEditImageData = NewObject<UCustomizableObjectNodeEditMaterialPinEditImageData>(this);
				PinEditImageData->ImageId_DEPRECATED = FGuid::NewGuid();
				PinEditImageData->PinMask = PinMask;
				
				// Search for the Image Id the Edit pin was referring to.
				const int32 NumImages = ParentMaterial->GetNumParameters(EMaterialParameterType::Texture);
				for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
				{
					if (ParentMaterial->GetParameterName(EMaterialParameterType::Texture, ImageIndex).ToString() == Image.Name)
					{
						PinEditImageData->ImageId_DEPRECATED = ParentMaterial->GetParameterId(EMaterialParameterType::Texture, ImageIndex).ParameterId;
						break;
					}
				}
				
				AddPinData(*ImagePin, *PinEditImageData);
			}
		}
		
		Images_DEPRECATED.Empty();
	}

	// Fill PinsParameter.
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeMaterialPerformanceBug) // || CustomizableObjectCustomVersion < FCustomizableObjectCustomVersion::AutomaticNodeMaterialPerformance
	{
		for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if (const UCustomizableObjectNodeEditMaterialPinEditImageData* PinData = Cast<UCustomizableObjectNodeEditMaterialPinEditImageData>(GetPinData(*Pin)))
			{
				PinsParameter_DEPRECATED.Add(PinData->ImageId_DEPRECATED, FEdGraphPinReference(Pin));
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ConvertEditAndExtendToModifiers)
	{
		// Look for the parent material and set it as the modifier reference material 

		PinsParameterMap = PinsParameterMap_DEPRECATED;
		PinsParameterMap_DEPRECATED.Empty();

		UCustomizableObjectNode* ParentNode = GetCustomizableObjectExternalNode<UCustomizableObjectNode>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);

		if (UCONodeSkeletalMeshSection* MeshSectionNode = Cast<UCONodeSkeletalMeshSection>(ParentNode))
		{
			ReferenceMaterial = MeshSectionNode->GetMaterial();
		}
		else if (UCONodeModifierExtendSkeletalMeshSection* ExtendParentNode = Cast<UCONodeModifierExtendSkeletalMeshSection>(ParentNode))
		{
			ReferenceMaterial = ExtendParentNode->ReferenceMaterial;
		}
		else
		{
			// Conversion failed?
			ensure(false);
			UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierExtendSkeletalMeshSection version upgrade failed.", *GetOutermost()->GetName());
		}

		ReconstructNode();
	}
}


FText UCONodeModifierEditSkeletalMeshSection::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Edit_MeshSection", "Edit Mesh Section");
}


FString UCONodeModifierEditSkeletalMeshSection::GetRefreshMessage() const
{
	return TEXT("Source material has changed, texture channels might have been added, removed or renamed. Please refresh the parent material node to reflect those changes.");
}


FText UCONodeModifierEditSkeletalMeshSection::GetTooltipText() const
{
	return LOCTEXT("Edit_Material_Tooltip", "Modify the texture parameters of an ancestor's material partially or completely.");
}


void UCONodeModifierEditSkeletalMeshSection::SetLayoutIndex(const int32 LayoutIndex)
{
	ParentLayoutIndex = LayoutIndex;
}


#undef LOCTEXT_NAMESPACE

