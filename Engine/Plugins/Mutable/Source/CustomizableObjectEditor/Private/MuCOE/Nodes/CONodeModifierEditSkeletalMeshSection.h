// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditLayoutBlocks.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"

#include "CONodeModifierEditSkeletalMeshSection.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }
struct FCustomizableObjectNodeEditMaterialImage;
struct FNodeMaterialParameterId;

class FArchive;
class FCustomizableObjectNodeParentedMaterial;
class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UObject;


/** Additional data for the Parent Texture Parameter to edit pin. */
UCLASS(MinimalAPI)
class UCustomizableObjectNodeEditMaterialPinEditImageData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	/** Parameter FGuid + Layer Index */
	UPROPERTY()
	FNodeMaterialParameterId ImageParamId;

	UPROPERTY();
	FEdGraphPinReference PinMask;

	UPROPERTY()
	FGuid ImageId_DEPRECATED;
};


UCLASS(MinimalAPI)
class UCONodeModifierEditSkeletalMeshSection : public UCustomizableObjectNodeModifierEditLayoutBlocks
{

	GENERATED_BODY()

public:
	
	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual FString GetRefreshMessage() const override;
	UE_API virtual bool IsSingleOutputNode() const override;
	UE_API virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	UE_API virtual bool HasPinViewer() const override;
	
	// Own interface
	
	/** */
	UE_API virtual void SetLayoutIndex(const int32 LayoutIndex);

	/** Returns the Image mask pin of the given Image that will be edited.
	 *
	 * @returns Always returns a valid pin if EditsImage(const FGuid&) returns true. */
	UE_API const UEdGraphPin* GetUsedImageMaskPin(const FNodeMaterialParameterId& ImageId) const;

private:

	// Deprecated properties
	UPROPERTY()
	TMap<FNodeMaterialParameterId, FEdGraphPinReference> PinsParameterMap_DEPRECATED;

	UPROPERTY()
	TMap<FGuid, FEdGraphPinReference> PinsParameter_DEPRECATED;
	
	UPROPERTY()
	TArray<FCustomizableObjectNodeEditMaterialImage> Images_DEPRECATED;
	
	// Old layout blocks to patch. Now in parent class.
	UPROPERTY()
	TArray<int32> Blocks_DEPRECATED;
};

#undef UE_API
