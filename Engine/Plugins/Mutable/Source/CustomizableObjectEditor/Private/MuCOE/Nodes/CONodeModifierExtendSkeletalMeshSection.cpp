// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"

#include "CONodeSkeletalMeshSection.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialInterface.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeModifierExtendSkeletalMeshSection)

class UCustomizableObjectNode;
class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCONodeModifierExtendSkeletalMeshSection::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ExtendMaterialRemoveImages)
	{
		const UCONodeSkeletalMeshSection* ParentMaterial = GetCustomizableObjectExternalNode<UCONodeSkeletalMeshSection>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);
		if (ParentMaterial)
		{
			for (const FCustomizableObjectNodeExtendMaterialImage& Image : Images_DEPRECATED)
			{
				const UEdGraphPin* ImagePin = FindPin(Image.Name);
				if (!ImagePin)
				{
					continue;
				}

				FGuid ImageId = FGuid::NewGuid();

				// Search for the Image Id the Extend pin was referring to.
				const int32 NumImages = ParentMaterial->GetNumParameters(EMaterialParameterType::Texture);
				for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
				{
					if (ParentMaterial->GetParameterName(EMaterialParameterType::Texture, ImageIndex).ToString() == Image.Name)
					{
						ImageId = ParentMaterial->GetParameterId(EMaterialParameterType::Texture, ImageIndex).ParameterId;
						break;
					}
				}

				PinsParameter_DEPRECATED.Add(ImageId, FEdGraphPinReference(ImagePin));                
			}
		}

		Images_DEPRECATED.Empty();
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ConvertEditAndExtendToModifiers)
	{
		// Look for the parent material and set it as the modifier reference material 
		PinsParameterMap = PinsParameterMap_DEPRECATED;
		PinsParameterMap_DEPRECATED.Empty();

		UCustomizableObjectNode* ParentNode = GetCustomizableObjectExternalNode<UCustomizableObjectNode>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);
		if (ParentNode)
		{
			// Add an autogenerated tag to the legacy parent, so that it can be referenced in this modifier.
			FString NewLegacyTag = MakeNodeAutoTag(ParentNode);
			RequiredTags.Add(NewLegacyTag);

			if (UCONodeSkeletalMeshSection* MeshSectionNode = Cast<UCONodeSkeletalMeshSection>(ParentNode))
			{
				ReferenceMaterial = MeshSectionNode->GetMaterial();
				MeshSectionNode->Tags.AddUnique(NewLegacyTag);
			}
			else if (UCONodeModifierExtendSkeletalMeshSection* ExtendParentNode = Cast<UCONodeModifierExtendSkeletalMeshSection>(ParentNode))
			{
				ReferenceMaterial = ExtendParentNode->ReferenceMaterial;
				ExtendParentNode->Tags.AddUnique(NewLegacyTag);
			}
			else
			{
				// Conversion failed?
				ensure(false);
				UE_LOGF(LogMutable, Warning, "[%ls] UCONodeModifierExtendSkeletalMeshSection version upgrade failed.", *GetOutermost()->GetName());
			}

			// If the tag was added to another CO, keep track of the addition to be able to repeat it in case
			// that CO is not re-saved but this one is.
			UCustomizableObject* ThisNodeObject = GraphTraversal::GetObject(*this);
			if (ParentMaterialObject_DEPRECATED != ThisNodeObject)
			{
				FLegacyTag LegacyTag;
				LegacyTag.ParentObject = ParentMaterialObject_DEPRECATED;
				LegacyTag.ParentNode = ParentMaterialNodeId_DEPRECATED;
				LegacyTag.Tag = NewLegacyTag;
				LegacyBackportsRequiredTags.AddUnique(LegacyTag);
			}
		}
		else
		{
			// No parent, no conversion.
			UE_LOGF(LogMutable, Log, "[%ls] UCONodeModifierExtendSkeletalMeshSection has no parent. It will not be upgraded.", *GetOutermost()->GetName());
		}

		ReconstructNode();
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!EnableTagsPinRef.Get())
		{
			UEdGraphPin* EnableTagsPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Enable Tags"), LOCTEXT("EnableTags", "Enable Tags"));
			EnableTagsPin->PinType.ContainerType = EPinContainerType::Array;
			EnableTagsPin->PinToolTip = "List of Tags that this node will Enable";
			EnableTagsPinRef = EnableTagsPin;
		}
	}
}


void UCONodeModifierExtendSkeletalMeshSection::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	CustomCreatePin(EGPD_Input, Schema->PC_Mesh, TEXT("Add Mesh"), LOCTEXT("AddMesh", "Add Mesh"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);

	// Create enable tags pin
	{
		UEdGraphPin* EnableTagsPin = CustomCreatePin(EGPD_Input, Schema->PC_String, TEXT("Enable Tags"), LOCTEXT("EnableTags", "Enable Tags"));
		EnableTagsPin->PinType.ContainerType = EPinContainerType::Array;
		EnableTagsPin->PinToolTip = "List of Tags that this node will Enable";
		EnableTagsPinRef = EnableTagsPin;
	}

	// Begin texture pins
	{
		const int32 NumImages = GetNumParameters(EMaterialParameterType::Texture);
		for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
		{
			const FName ImageName = GetParameterName(EMaterialParameterType::Texture, ImageIndex);
			UEdGraphPin* PinImage = CustomCreatePin(EGPD_Input, Schema->PC_Texture, ImageName, FText::FromName(ImageName));

			const FNodeMaterialParameterId ImageId = GetParameterId(EMaterialParameterType::Texture, ImageIndex);
				
			PinsParameterMap.Add(ImageId, FEdGraphPinReference(PinImage));
		}
	}
	// End texture pins

	CustomCreatePin(EGPD_Output, Schema->PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));
}


FText UCONodeModifierExtendSkeletalMeshSection::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Extend_MeshSection", "Extend Mesh Section");
}


FString UCONodeModifierExtendSkeletalMeshSection::GetRefreshMessage() const
{
	return TEXT("The Reference Material has changed, texture channels might have been added, removed or renamed. Please refresh to reflect those changes.");
}


UEdGraphPin* UCONodeModifierExtendSkeletalMeshSection::AddMeshPin() const
{
	return FindPin(TEXT("Add Mesh"));
}


UEdGraphPin* UCONodeModifierExtendSkeletalMeshSection::GetEnableTagsPin() const
{
	return EnableTagsPinRef.Get();
}


TArray<UCustomizableObjectLayout*> UCONodeModifierExtendSkeletalMeshSection::GetLayouts() const
{
	TArray<UCustomizableObjectLayout*> Result;

	if (UEdGraphPin* MeshPin = AddMeshPin())
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*MeshPin))
		{
			if (const UEdGraphPin* SourceMeshPin = FindMeshBaseSource(*ConnectedPin, false))
			{
				if (const UCustomizableObjectNodeSkeletalMesh* MeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(SourceMeshPin->GetOwningNode()))
				{
					Result = MeshNode->GetLayouts(*SourceMeshPin);
				}
				else if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(SourceMeshPin->GetOwningNode()))
				{
					Result = TableNode->GetLayouts(SourceMeshPin);
				}
			}
		}
	}

	return Result;
}


FText UCONodeModifierExtendSkeletalMeshSection::GetTooltipText() const
{
	return LOCTEXT("Extend_Material_Tooltip", "Extend a mesh section: add a new mesh fragment, and add its corresponding texture to the modified material texture parameters.");
}


bool UCONodeModifierExtendSkeletalMeshSection::IsSingleOutputNode() const
{
	return true;
}


bool UCONodeModifierExtendSkeletalMeshSection::CustomRemovePin(UEdGraphPin& Pin)
{
	for (TMap<FNodeMaterialParameterId, FEdGraphPinReference>::TIterator It = PinsParameterMap.CreateIterator(); It; ++It)
	{
		if (It.Value().Get() == &Pin) // We could improve performance if FEdGraphPinReference exposed the pin id.
		{
			It.RemoveCurrent();
			break;
		}
	}

	return Super::CustomRemovePin(Pin);
}


TArray<FString> UCONodeModifierExtendSkeletalMeshSection::GetEnableTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext)
{ 
	const UEdGraphPin* TagsPin = GetEnableTagsPin();

	if (!TagsPin)
	{
		return Tags;
	}

	const TArray<UEdGraphPin*> ConnectedPins = FollowInputPinArray(*TagsPin);

	// Getting tags from linked pin
	if (ConnectedPins.Num())
	{
		TArray<FString> OutTags;

		for (const UEdGraphPin* StringPin : ConnectedPins)
		{
			const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*StringPin, MacroContext);

			if (SourceStringPin)
			{
				if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
				{
					OutTags.AddUnique(StringNode->Value);
				}
			}
			
		}

		return OutTags;
	}
	
	return Tags;
}


TArray<FString>* UCONodeModifierExtendSkeletalMeshSection::GetEnableTagsArray()
{
	return &Tags;
}


FString UCONodeModifierExtendSkeletalMeshSection::GetInternalTagDisplayName()
{
	UMaterialInterface* Material = ReferenceMaterial;
	return FString::Printf(TEXT("Extend Mesh [%s]"), Material ? *Material->GetName() : TEXT("no-material"));
}


#undef LOCTEXT_NAMESPACE
