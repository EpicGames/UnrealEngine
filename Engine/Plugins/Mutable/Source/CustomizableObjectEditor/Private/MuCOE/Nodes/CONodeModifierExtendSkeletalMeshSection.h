// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CONodeModifierSkeletalMeshSection.h"
#include "MuCOE/CustomizableObjectLayout.h"

#include "CONodeModifierExtendSkeletalMeshSection.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObject;
class UCustomizableObjectNode;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FCustomizableObjectNodeExtendMaterialImage;
struct FEdGraphPinReference;


UCLASS(MinimalAPI)
class UCONodeModifierExtendSkeletalMeshSection : public UCONodeModifierSkeletalMeshSection
{
	GENERATED_BODY()

public:

	/** Tags enabled when the modifier is applied. Also used to decide what modifiers get applied to the data added by this node.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = EnableTags)
	TArray<FString> Tags;

	/** First LOD where the modifier is applied to. */
	UPROPERTY(EditAnywhere, Category = LOD, meta = (ClampMin = 0, ClampMax = 255))
	uint32 FirstLOD = 0;

public:
	
	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual FString GetRefreshMessage() const override;
	UE_API virtual bool IsSingleOutputNode() const override;
	UE_API virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	UE_API virtual TArray<FString> GetEnableTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) override;
	UE_API virtual TArray<FString>* GetEnableTagsArray() override;
	UE_API virtual FString GetInternalTagDisplayName() override;

	// Own interface
	UE_API UEdGraphPin* AddMeshPin() const;
	UE_API UEdGraphPin* GetEnableTagsPin() const;

	UE_API TArray<UCustomizableObjectLayout*> GetLayouts() const;

private:

	// Deprecated properties

	UPROPERTY()
	TObjectPtr<UCustomizableObject> ParentMaterialObject_DEPRECATED = nullptr;

	UPROPERTY()
	FGuid ParentMaterialNodeId_DEPRECATED;
	

	UPROPERTY()
	TMap<FNodeMaterialParameterId, FEdGraphPinReference> PinsParameterMap_DEPRECATED;

	UPROPERTY()
	TMap<FGuid, FEdGraphPinReference> PinsParameter_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectNodeExtendMaterialImage> Images_DEPRECATED;

	UPROPERTY()
	FEdGraphPinReference EnableTagsPinRef;
};

#undef UE_API
