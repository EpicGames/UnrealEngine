// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeModifierSkeletalMeshMerge.h"

#include "CustomizableObjectNodeExposePin.h"
#include "CustomizableObjectNodeObject.h"
#include "CustomizableObjectNodeObjectChild.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectSchemaActions.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshMake_V2.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "ScopedTransaction.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeModifierSkeletalMeshMerge)


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCONodeModifierMutableSkeletalMeshMerge::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	Super::AllocateDefaultPins(RemapPins);

	ParentSkeletalMeshNamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Skeletal Mesh Name"), LOCTEXT("SkeletalMeshName", "Skeletal Mesh Name"));
	SkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Added"), LOCTEXT("CONodeModifierSkeletalMeshMerge_ToAddSkeletalMeshName", "Added"));
	OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier);
}


bool UCONodeModifierMutableSkeletalMeshMerge::IsAffectedByLOD() const
{
	return false;
}


bool UCONodeModifierMutableSkeletalMeshMerge::IsSingleOutputNode() const
{
	// todo UE-225446 : By limiting the number of connections this node can have we avoid a check failure. However, this method should be
	// removed in the future and the inherent issue with 1:n output connections should be fixed in its place
	return true;
}


void UCONodeModifierMutableSkeletalMeshMerge::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!ParentSkeletalMeshNamePin.Get())
		{
			ParentSkeletalMeshNamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Component Name"), LOCTEXT("ComponentName", "Component Name"));
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeComponentMeshAddToNumLODs)
	{
		constexpr int32 NewNumLODs = 8;

		for (int32 LODIndex = NumLODs_DEPRECATED; LODIndex < NewNumLODs; ++LODIndex)
		{
			FString LODName = FString::Printf(TEXT("LOD %d"), LODIndex);

			UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
			Pin->PinType.ContainerType = EPinContainerType::Array;

			LODPins_DEPRECATED.Add(Pin);
		}

		NumLODs_DEPRECATED = NewNumLODs;

		UEdGraphPin* NamePin = ParentSkeletalMeshNamePin.Get();

		const int32 Index = Pins.IndexOfByPredicate([&](const UEdGraphPin* Pin) { return Pin == NamePin; });
		if (Index != INDEX_NONE)
		{
			Pins.RemoveAt(Index);
			Pins.Add(NamePin);
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ComponentAddSkeletalMeshChangedInputsOrder)
	{
		auto RemappingFunction = [&](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* Remapper) -> void
		{
			ParentSkeletalMeshNamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Component Name"), LOCTEXT("ComponentName", "Component Name"));
			OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component);

			LODPins_DEPRECATED.Empty(NumLODs_DEPRECATED);
			for (int32 LODIndex = 0; LODIndex < NumLODs_DEPRECATED; ++LODIndex)
			{
				FString LODName = FString::Printf(TEXT("LOD %d"), LODIndex);

				UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
				Pin->PinType.ContainerType = EPinContainerType::Array;

				LODPins_DEPRECATED.Add(Pin);
			}
		};

		FixupReconstructPins(CreateRemapPinsByName(), RemappingFunction);
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ComponentAddNodeModifierOutputPin)
	{
		// Fixup the Component name pin so it gets to be the Skeletal Mesh name instead
		check(ParentSkeletalMeshNamePin.Get());
		{
			UEdGraphPin* ParentComponentPinPointer = ParentSkeletalMeshNamePin.Get();
				
			TArray<UEdGraphPin*> LinkedTo = ParentComponentPinPointer->LinkedTo;
			if (LinkedTo.Num())
			{
				check (LinkedTo.Num() == 1);
				
				// remove old pin
				ParentComponentPinPointer->BreakAllPinLinks(true, false);
			}
			
			RemovePin(ParentComponentPinPointer);
			
			// Crete the new string pin and leave it at the first position so it appears before the LOD pins
			ParentComponentPinPointer = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Skeletal Mesh Name"), LOCTEXT("SkeletalMeshName", "Skeletal Mesh Name"));
			Pins.Remove(ParentComponentPinPointer);
			Pins.Insert(ParentComponentPinPointer, 0);
			
			ParentSkeletalMeshNamePin = ParentComponentPinPointer;
			
			if (LinkedTo.Num())
			{
				ParentSkeletalMeshNamePin.Get()->MakeLinkTo(LinkedTo[0]);
			}
		}
		
		// We have changed the output pin from being of the "component" type to being of the "modifer" type. Make the change for the old nodes
		check(OutputPin.Get());
		
		UCustomizableObjectNode* LinkedNode = nullptr;
		{
			// Get the list of connections that the OutputPin (Component type pin) does have
			const TArray<UEdGraphPin*> PinConnections = OutputPin.Get()->LinkedTo;
			if (PinConnections.Num())
			{
				// The output pin can only be connected to one other node, it does not support multiple connections at the same time
				check(PinConnections.Num() == 1);
				
				UEdGraphNode* OwningNode = PinConnections[0]->GetOwningNode();
				LinkedNode = CastChecked<UCustomizableObjectNode>(OwningNode);
			}
		}
		
		// If the Component output pin was connected to something then resolve the change for that node
		if (LinkedNode)
		{
			auto ReplaceOutputPinLambda = [&]() -> void
			{
				UEdGraphPin* OutputPinPointer = OutputPin.Get();
					
				// Remove component type output pin and create a replacement for it
				OutputPinPointer->BreakAllPinLinks(true, false);
				RemovePin(OutputPinPointer);
		
				OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier);
			};
			
			UEdGraph* Graph = GetGraph();
			check(Graph);
			
			// Based on what node the other node is, perform one connection or another
			if (UCustomizableObjectNodeObject* LinkedBaseObjectNode = Cast<UCustomizableObjectNodeObject>(LinkedNode))
			{
				ReplaceOutputPinLambda();
				
				UEdGraphPin* OtherNodeInputPin = LinkedBaseObjectNode->ModifiersPin();
				OutputPin.Get()->MakeLinkTo(OtherNodeInputPin);
				
				Graph->NotifyNodeChanged(LinkedBaseObjectNode);
			}
			else if (UCustomizableObjectNodeObjectChild* LinkedChildObject = Cast<UCustomizableObjectNodeObjectChild>(LinkedNode))
			{
				ReplaceOutputPinLambda();
				
				UEdGraphPin* OtherNodeInputPin = LinkedChildObject->ModifiersPin();
				OutputPin.Get()->MakeLinkTo(OtherNodeInputPin);
				
				Graph->NotifyNodeChanged(LinkedChildObject);
			}
			else
			{
				// Mark the current output pin as orphan to bring attention to it
				OutputPin.Get()->bOrphanedPin = true;
				SetRefreshNodeWarning();
				
				// Report the problem
				const FText Message = FText::Format(LOCTEXT("UCONodeSkeletalMeshAddTo_FailureToUpdateComponentPinMessage", "The node Component pin is connected to a [{0}] node which has no equivalent modifier based alternative. Manual update will be needed."), FText::FromString(LinkedNode->GetName()));
				FCustomizableObjectEditorLogger::CreateLog(Message)
					.Severity(EMessageSeverity::Warning)
					.Context(*this)
					.BaseObject(false)
					.CustomizableObject(GraphTraversal::GetObject(*this))
					.Log();
				
				// Create a new modifier pin (th e
				OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier);
			}
		}
	}

	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ModifierSkeletalMeshMergeUseSkeletalMeshPin)
	{
		// The LOD pins are being replaced by a single Skeletal Mesh input pin fed by a UCONodeSkeletalMeshMake_V2 node.
		check(!SkeletalMeshPin.Get());
		SkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Skeletal Mesh"), LOCTEXT("SkeletalMesh", "Skeletal Mesh"));

		// Reorder the freshly created pin to sit right after the Skeletal Mesh Name pin to match the layout of newly created nodes.
		if (UEdGraphPin* SkeletalMeshPinPtr = SkeletalMeshPin.Get())
		{
			Pins.Remove(SkeletalMeshPinPtr);

			int32 InsertIndex = 0;
			if (UEdGraphPin* NamePinPtr = ParentSkeletalMeshNamePin.Get())
			{
				const int32 NameIndex = Pins.IndexOfByKey(NamePinPtr);
				if (NameIndex != INDEX_NONE)
				{
					InsertIndex = NameIndex + 1;
				}
			}

			Pins.Insert(SkeletalMeshPinPtr, InsertIndex);
		}

		// Detect whether any of the deprecated LOD pins had connections that need to be migrated.
		bool bIsAtLeastOneLODPinConnected = false;
		for (const FEdGraphPinReference& LODPinRef : LODPins_DEPRECATED)
		{
			const UEdGraphPin* LODPin = LODPinRef.Get();
			check(LODPin);
			if (!LODPin->LinkedTo.IsEmpty())
			{
				bIsAtLeastOneLODPinConnected = true;
				break;
			}
		}

		// Only spawn the new make node if there is something to migrate.
		if (bIsAtLeastOneLODPinConnected)
		{
			UEdGraph* Graph = GetGraph();
			check(Graph);

			constexpr float Separation = 400;
			const FVector2D DefaultSpawnPosition = FVector2D(GetNodePosX(), GetNodePosY());

			UCONodeSkeletalMeshMake_V2* MakeMutableMeshNode = Cast<UCONodeSkeletalMeshMake_V2>(
				FCustomizableObjectSchemaAction_NewNode::CreateNode(Graph, nullptr, DefaultSpawnPosition, Cast<UEdGraphNode>(NewObject<UCONodeSkeletalMeshMake_V2>())));

			// Connect the new make node output to this node's new Skeletal Mesh input pin.
			MakeMutableMeshNode->SkeletalMeshPin.Get()->MakeLinkTo(SkeletalMeshPin.Get(), false);

			// Make sure the new make node has enough LOD pins to hold every connection we are about to migrate.
			const int32 NumDeprecatedLODPins = LODPins_DEPRECATED.Num();
			for (int32 LODIndex = MakeMutableMeshNode->LODPins.Num(); LODIndex < NumDeprecatedLODPins; ++LODIndex)
			{
				const FString LODName = FString::Printf(TEXT("LOD %d"), LODIndex);

				UEdGraphPin* LODPin = MakeMutableMeshNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
				LODPin->PinType.ContainerType = EPinContainerType::Array;

				MakeMutableMeshNode->LODPins.Add(LODPin);
			}

			check(MakeMutableMeshNode->LODPins.Num() >= NumDeprecatedLODPins);

			// Transfer every connection from this node's deprecated LOD pins to the new make node.
			for (int32 LODIndex = 0; LODIndex < NumDeprecatedLODPins; ++LODIndex)
			{
				UEdGraphPin* MakeNodeLODPin = MakeMutableMeshNode->LODPins[LODIndex].Get();
				UEdGraphPin* ThisNodeLODPin = LODPins_DEPRECATED[LODIndex].Get();

				for (UEdGraphPin* LinkedToPin : ThisNodeLODPin->LinkedTo)
				{
					MakeNodeLODPin->MakeLinkTo(LinkedToPin, false);
				}

				ThisNodeLODPin->BreakAllPinLinks(false, false);
			}

			// Place this node to the right of the new make node so the wiring stays readable.
			SetNodePosX(DefaultSpawnPosition.X + MakeMutableMeshNode->GetWidth() + Separation);

			Graph->NotifyNodeChanged(MakeMutableMeshNode);
		}

		// Remove the deprecated LOD pins from this node.
		for (const FEdGraphPinReference& LODPinRef : LODPins_DEPRECATED)
		{
			if (UEdGraphPin* LODPin = LODPinRef.Get())
			{
				CustomRemovePin(*LODPin);
			}
		}
		LODPins_DEPRECATED.Reset();

		if (UEdGraph* Graph = GetGraph())
		{
			Graph->NotifyNodeChanged(this);
		}
	}
}


bool UCONodeModifierMutableSkeletalMeshMerge::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCONodeModifierMutableSkeletalMeshMerge::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UEdGraphPin* ParentCompNamePin = GetParentSkeletalMeshNamePin();
	if (ParentCompNamePin && ParentCompNamePin->PinId == Pin.PinId)
	{
		return FText::FromName(GetParentSkeletalMeshName());
	}
	else
	{
		return Super::GetPinEditableName(Pin);
	}
}


void UCONodeModifierMutableSkeletalMeshMerge::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	const UEdGraphPin* ParentCompNamePin = GetParentSkeletalMeshNamePin();
	if (ParentCompNamePin && ParentCompNamePin->PinId == Pin.PinId)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeComponent_SetParentComponentPinEditableNameTransaction", "Change ParentComponentName PinName"));
		Modify();
		
		SetParentSkeletalMeshName(FName(*InValue.ToString()));
		PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
	}
	else
	{
		Super::SetPinEditableName(Pin, InValue);
	}
}


FText UCONodeModifierMutableSkeletalMeshMerge::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("ModifierMutableSkeletalMeshMerge_NodeTitle", "Skeletal Mesh Merge");
}


FLinearColor UCONodeModifierMutableSkeletalMeshMerge::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Modifier);
}


void UCONodeModifierMutableSkeletalMeshMerge::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == GetParentSkeletalMeshNamePin())
	{
		GetGraph()->NotifyGraphChanged();
	}
}


FName UCONodeModifierMutableSkeletalMeshMerge::GetParentSkeletalMeshName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (const UEdGraphPin* ParentNamePin = GetParentSkeletalMeshNamePin())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*ParentNamePin))
		{
			const UEdGraphPin* StringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, MacroContext);

			if (const UCustomizableObjectNodeStaticString* StringNode = StringPin ? Cast<UCustomizableObjectNodeStaticString>(StringPin->GetOwningNode()) : nullptr)
			{
				return FName(StringNode->Value);
			}
		}
	}

	return ParentSkeletalMeshName;
}


void UCONodeModifierMutableSkeletalMeshMerge::SetParentSkeletalMeshName(const FName& InSkeletalMeshName)
{
	ParentSkeletalMeshName = InSkeletalMeshName;
}


UEdGraphPin* UCONodeModifierMutableSkeletalMeshMerge::GetParentSkeletalMeshNamePin() const
{
	return ParentSkeletalMeshNamePin.Get();
}


#undef LOCTEXT_NAMESPACE

