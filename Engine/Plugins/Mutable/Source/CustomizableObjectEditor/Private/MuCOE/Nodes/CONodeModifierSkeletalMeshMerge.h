// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MuCOE/Nodes/CustomizableObjectNode.h"


#include "CONodeModifierSkeletalMeshMerge.generated.h"

class UCustomizableObjectNodeMacroInstance;

UCLASS()
class UCONodeModifierMutableSkeletalMeshMerge : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;

	//UCustomizableObjectNode interface
	virtual bool IsAffectedByLOD() const override;
	virtual bool IsSingleOutputNode() const override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;

protected:
	
	//UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	
public:
	// Own interface
	FName GetParentSkeletalMeshName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;
	void SetParentSkeletalMeshName(const FName& InSkeletalMeshName);
	UEdGraphPin* GetParentSkeletalMeshNamePin() const;

private:
	UPROPERTY()
	FName ParentSkeletalMeshName;

public:
	UPROPERTY()
	int32 NumLODs_DEPRECATED = 1;

	UPROPERTY()
	TArray<FEdGraphPinReference> LODPins_DEPRECATED;

	UPROPERTY()
	FEdGraphPinReference SkeletalMeshPin;

	UPROPERTY()
	FEdGraphPinReference OutputPin;

private:
	UPROPERTY()
	FEdGraphPinReference ParentSkeletalMeshNamePin;	
};

