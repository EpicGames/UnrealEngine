// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeModifierSkeletalMeshSection.h"

#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "Materials/MaterialParameters.h"
#include "MaterialCachedData.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeModifierSkeletalMeshSection)


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"



void UCONodeModifierSkeletalMeshSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeModifierSkeletalMeshSection, ReferenceMaterial))
		{
			UCustomizableObjectNode::ReconstructNode();
		}
	}
}


bool UCONodeModifierSkeletalMeshSection::UsesImage(const FNodeMaterialParameterId& ImageId) const
{
	if (const UEdGraphPin* Pin = GetUsedImagePin(ImageId))
	{
		return FollowInputPin(*Pin) != nullptr;
	}
	else
	{
		return false;
	}
}


const UEdGraphPin* UCONodeModifierSkeletalMeshSection::GetUsedImagePin(const FNodeMaterialParameterId& ImageId) const
{
	if (const FEdGraphPinReference* PinReference = PinsParameterMap.Find(ImageId))
	{
		if (const UEdGraphPin& Pin = *PinReference->Get(); !IsPinOrphan(Pin)) // We always have a valid pin reference. If it is nullptr, it means that something went wrong.
		{
			return &Pin;
		}
	}

	return nullptr;
}


bool UCONodeModifierSkeletalMeshSection::IsNodeOutDatedAndNeedsRefresh()
{
	const bool bOutdated = [&, this]()
		{
			for (const TTuple<FNodeMaterialParameterId, FEdGraphPinReference> Pair : PinsParameterMap)
			{
				const UEdGraphPin& Pin = *Pair.Value.Get();
				if (!IsPinOrphan(Pin))
				{
					bool bHas = UCONodeSkeletalMeshSection::HasParameter(ReferenceMaterial,Pair.Key);
					if (UsesImage(Pair.Key) && !bHas)
					{
						return true;
					}
				}
			}

			return false;
		}();

	// Remove previous compilation warnings
	if (!bOutdated && bHasCompilerMessage)
	{
		RemoveWarnings();
		GetGraph()->NotifyGraphChanged();
	}

	return bOutdated;
}


int32 UCONodeModifierSkeletalMeshSection::GetNumParameters(const EMaterialParameterType Type) const
{
	if (ReferenceMaterial)
	{
		return ReferenceMaterial->GetCachedExpressionData().GetParameterTypeEntry(Type).ParameterInfoSet.Num();
	}
	else
	{
		return 0;
	}
}


FNodeMaterialParameterId UCONodeModifierSkeletalMeshSection::GetParameterId(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	const FMaterialCachedExpressionData& Data = ReferenceMaterial->GetCachedExpressionData();

	if (Data.EditorOnlyData)
	{
		if (Data.EditorOnlyData->EditorEntries[(int32)Type].EditorInfo.Num() != 0)
		{
			const FGuid ParameterId = Data.EditorOnlyData->EditorEntries[(int32)Type].EditorInfo[ParameterIndex].ExpressionGuid;
			const int32 LayerIndex = GetParameterLayerIndex(Type, ParameterIndex);

			return { ParameterId, LayerIndex };
		}
	}

	return FNodeMaterialParameterId();
}


FName UCONodeModifierSkeletalMeshSection::GetParameterName(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	check(ReferenceMaterial);

	const FMaterialCachedParameterEntry& Entry = ReferenceMaterial->GetCachedExpressionData().GetParameterTypeEntry(Type);

	for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
	{
		const int32 IteratorIndex = It.GetId().AsInteger();

		if (IteratorIndex == ParameterIndex)
		{
			return (*It).Name;
		}
	}

	// The parameter should exist
	check(false);

	return FName();
}


int32 UCONodeModifierSkeletalMeshSection::GetParameterLayerIndex(const UMaterialInterface* InMaterial, const EMaterialParameterType Type, const int32 ParameterIndex)
{
	check(InMaterial);

	const FMaterialCachedParameterEntry& Entry = InMaterial->GetCachedExpressionData().GetParameterTypeEntry(Type);

	for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
	{
		const int32 IteratorIndex = It.GetId().AsInteger();

		if (IteratorIndex == ParameterIndex)
		{
			return (*It).Index;
		}
	}

	// The parameter should exist
	check(false);

	return -1;
}


int32 UCONodeModifierSkeletalMeshSection::GetParameterLayerIndex(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	return GetParameterLayerIndex(ReferenceMaterial.Get(), Type, ParameterIndex);
}


FText UCONodeModifierSkeletalMeshSection::GetParameterLayerName(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	check(ReferenceMaterial)

		int32 LayerIndex = GetParameterLayerIndex(Type, ParameterIndex);

	FMaterialLayersFunctions LayersValue;
	ReferenceMaterial->GetMaterialLayers(LayersValue);


	return LayersValue.EditorOnly.LayerNames.IsValidIndex(LayerIndex) ? LayersValue.EditorOnly.LayerNames[LayerIndex] : FText();
}


#undef LOCTEXT_NAMESPACE
