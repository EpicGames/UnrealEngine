// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"

#include "CONodeModifierSkeletalMeshSection.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UObject;


UCLASS(MinimalAPI, abstract)
class UCONodeModifierSkeletalMeshSection : public UCustomizableObjectNodeModifierBase
{

	GENERATED_BODY()

public:

	/** Reference material that defines the structure of the data to be extended.
	* The Sections modified with this modifier are supposed to have the same texture parameters, but don't need to have exactly
	* the ReferenceMaterial set.
	*/
	UPROPERTY(EditAnywhere, Category = Parameters)
	TObjectPtr<UMaterialInterface> ReferenceMaterial = nullptr;

	/** Relates a Parameter id (and layer if is a layered material) to a Pin. Only used to improve performance. */
	UPROPERTY()
	TMap<FNodeMaterialParameterId, FEdGraphPinReference> PinsParameterMap;

public:

	//
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// UCustomizableObjectNode interface
	UE_API virtual bool IsNodeOutDatedAndNeedsRefresh() override;

	// Own interface

	/** */
	UE_API bool UsesImage(const FNodeMaterialParameterId& ImageId) const;

	/** */
	UE_API const UEdGraphPin* GetUsedImagePin(const FNodeMaterialParameterId& ImageId) const;

	/** */
	UE_API int32 GetNumParameters(const EMaterialParameterType Type) const;
	UE_API FNodeMaterialParameterId GetParameterId(const EMaterialParameterType Type, const int32 ParameterIndex) const;
	UE_API FName GetParameterName(const EMaterialParameterType Type, const int32 ParameterIndex) const;
	static UE_API int32 GetParameterLayerIndex(const UMaterialInterface* InMaterial, const EMaterialParameterType Type, const int32 ParameterIndex);
	UE_API int32 GetParameterLayerIndex(const EMaterialParameterType Type, const int32 ParameterIndex) const;
	UE_API FText GetParameterLayerName(const EMaterialParameterType Type, const int32 ParameterIndex) const;
};

#undef UE_API
