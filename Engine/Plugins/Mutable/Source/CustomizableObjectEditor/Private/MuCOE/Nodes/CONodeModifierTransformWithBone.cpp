// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeModifierTransformWithBone.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeModifierTransformWithBone)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeModifierTransformWithBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Transform_Mesh_In_Bone_Hierarchy_Modifier", "Mesh Transform In Bone Hierarchy");
}


FText UCONodeModifierTransformWithBone::GetTooltipText() const
{
	return LOCTEXT("Transform_Mesh_In_Bone_Hierarchy_Modifier_Tooltip", "Applies a transform to the vertices of a mesh that are skinned to the target bone or its child bones");
}


void UCONodeModifierTransformWithBone::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	TransformPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Transform);

	OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier);

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


UEdGraphPin* UCONodeModifierTransformWithBone::GetOutputPin() const
{
	return OutputPin.Get();
}


#undef LOCTEXT_NAMESPACE
