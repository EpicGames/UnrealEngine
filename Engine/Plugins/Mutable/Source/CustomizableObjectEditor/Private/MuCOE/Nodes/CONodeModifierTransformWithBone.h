// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuT/NodeSurfaceModifier.h"

#include "CONodeModifierTransformWithBone.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
struct FPropertyChangedEvent;

UCLASS()
class UCONodeModifierTransformWithBone : public UCustomizableObjectNodeModifierBase
{
	GENERATED_BODY()
	
public:
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	// UCustomizableObjectNodeModifierBase interface
	virtual UEdGraphPin* GetOutputPin() const override;

	/** Root bone used to filter vertices based on skinning weights. Vertices skinned to this
	  * bone or below will be transformed. */
	UPROPERTY(EditAnywhere, Category = Transform)
	FString BoneName;

	/** Vertex influences under this threshold will not be considered when filtering vertices.
	  * Percentage of total skin weight [0.0 , 1.0]. */
	UPROPERTY(EditAnywhere, Category = Transform, meta = (ClampMin = 0.0f, ClampMax = 1.0f))
	float ThresholdFactor = 0.05f;
	
	UPROPERTY()
	FEdGraphPinReference TransformPin;

	UPROPERTY()
	FEdGraphPinReference OutputPin;
};
