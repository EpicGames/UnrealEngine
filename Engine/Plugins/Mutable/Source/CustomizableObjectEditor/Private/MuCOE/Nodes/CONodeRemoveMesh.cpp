// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeRemoveMesh.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeRemoveMesh)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCONodeRemoveMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	BaseMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Base Mesh"), LOCTEXT("BaseMesh", "Base Mesh"));
	RemoveMeshPin =  CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Remove Mesh"), LOCTEXT("RemoveMesh", "Remove Mesh"));
	
	OutputMeshPin =  CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh);
}


FText UCONodeRemoveMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Remove_Mesh", "Mesh Remove");
}


FText UCONodeRemoveMesh::GetTooltipText() const
{
	return LOCTEXT("Remove_Mesh_Tooltip",
	"Removes the faces of a mesh section that are defined only by the vertexes shared by said mesh section and the input mesh. \n"
	"It also removes any vertex and edge that only define deleted faces. \n"
	"If the removed mesh covers all the faces included in one or more layout blocks those blocks get removed, freeing layout space in the final texture.");
}


FLinearColor UCONodeRemoveMesh::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


bool UCONodeRemoveMesh::IsSingleOutputNode() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE
