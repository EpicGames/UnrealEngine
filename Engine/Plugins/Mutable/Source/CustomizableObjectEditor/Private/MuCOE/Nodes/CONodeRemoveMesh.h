// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeRemoveMesh.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;

UCLASS()
class UCONodeRemoveMesh : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	
	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool IsSingleOutputNode() const override;

	UPROPERTY(EditAnywhere, Category = RemoveOptions)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
	
	UPROPERTY()
	FEdGraphPinReference BaseMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference RemoveMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference OutputMeshPin;
};


#undef UE_API
