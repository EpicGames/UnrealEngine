// Copyright Epic Games, Inc. All Rights Reserved.

#include "CONodeSkeletalMeshBreak_V2.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "ScopedTransaction.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeSkeletalMeshBreak_V2::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UCONodeSkeletalMeshBreak_NodeTitle","Skeletal Mesh Break");
}


FText UCONodeSkeletalMeshBreak_V2::GetTooltipText() const
{
	return Super::GetTooltipText();
}


FLinearColor UCONodeSkeletalMeshBreak_V2::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


void UCONodeSkeletalMeshBreak_V2::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Passthrough skm input
	InputPassthroughMesh = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	
	// Regenerate the new pins from the old pin's PinData which may have changed
	OutputMaterialPins.Reset();

	// Get old pins
	TArray<UEdGraphPin*> AllNonOrphanPins = GetAllNonOrphanPins();
	for (const UEdGraphPin* Pin : AllNonOrphanPins)
	{
		if (!IsMaterialOutputPin(*Pin))
		{
			continue;
		}
		
		if (UCONodeSkeletalMeshBreakPinData_V2* PinData = Cast<UCONodeSkeletalMeshBreakPinData_V2>(GetPinData(*Pin)))
		{
			CreateMaterialOutputPin(PinData);
		}
	}
}


FText UCONodeSkeletalMeshBreak_V2::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (const UCONodeSkeletalMeshBreakPinData_V2* CastedPinData = Cast<UCONodeSkeletalMeshBreakPinData_V2>(PinData))
	{
		return FText::FromName(CastedPinData->TargetMaterialSlotName);
	}

	return Super::GetPinEditableName(Pin);
}


void UCONodeSkeletalMeshBreak_V2::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (UCONodeSkeletalMeshBreakPinData_V2* CastedPinData = Cast<UCONodeSkeletalMeshBreakPinData_V2>(PinData))
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCONodeSkeletalMeshBreak_SetNamePinEditableNameTransaction", "Change Pin Name"));
		CastedPinData->Modify();
		
		CastedPinData->TargetMaterialSlotName = FName{Value.ToString()};
		PinEditableNameChangedDelegate.Broadcast(Pin, Value);
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}


bool UCONodeSkeletalMeshBreak_V2::HasPinViewer() const
{
	return true;
}


EAddPinNodeButtonLocation UCONodeSkeletalMeshBreak_V2::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::OUTPUT;
}


EEditablePinNameBoxVisibilityPolicy UCONodeSkeletalMeshBreak_V2::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


void UCONodeSkeletalMeshBreak_V2::AddPinFromUI()
{
	FScopedTransaction Transaction(LOCTEXT("UCONodeSkeletalMeshBreak_AddPinFromNodeUITransaction", "Add Material Output Pin from Node UI"));
	Modify();

	UCONodeSkeletalMeshBreakPinData_V2* PinData = NewObject<UCONodeSkeletalMeshBreakPinData_V2>(this);
	if (CreateMaterialOutputPin(PinData))
	{
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCONodeSkeletalMeshBreak_V2::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	const bool bCanPinBeRemoved = !Pin.LinkedTo.Num() && !Pin.bOrphanedPin;
	if (!bCanPinBeRemoved)
	{
		return false;
	}
	
	return IsMaterialOutputPin(Pin);
}


bool UCONodeSkeletalMeshBreak_V2::CustomRemovePin(UEdGraphPin& Pin)
{
	OutputMaterialPins.Remove(&Pin);
	return Super::CustomRemovePin(Pin);
}


bool UCONodeSkeletalMeshBreak_V2::CanRenamePin(const UEdGraphPin& Pin) const
{
	return IsMaterialOutputPin(Pin);
}


bool UCONodeSkeletalMeshBreak_V2::CreateMaterialOutputPin(UCONodeSkeletalMeshBreakPinData_V2* InPinData)
{
	UEdGraphPin* NewPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Material, TEXT("Material Slot"), LOCTEXT("MaterialSlot", "Material Slot"), InPinData);
	if (NewPin)
	{
		OutputMaterialPins.Add(NewPin);
		return true;
	}
	
	return false;
}


UCustomizableObjectNodeRemapPins* UCONodeSkeletalMeshBreak_V2::CreateRemapPinsDefault() const
{
	return Super::CreateRemapPinsByPosition();
}


bool UCONodeSkeletalMeshBreak_V2::IsMaterialOutputPin(const UEdGraphPin& Pin) const
{
	return Pin.Direction == EEdGraphPinDirection::EGPD_Output && Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Material;
}




#undef LOCTEXT_NAMESPACE
