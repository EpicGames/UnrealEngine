// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeSkeletalMeshBreak_V2.generated.h"

UCLASS(MinimalAPI)
class UCONodeSkeletalMeshBreakPinData_V2 : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = NoCategory)
	FName TargetMaterialSlotName;
};


UCLASS(MinimalAPI)
class UCONodeSkeletalMeshBreak_V2 : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	virtual bool HasPinViewer() const override;
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;
	virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;

	UPROPERTY()
	FEdGraphPinReference InputPassthroughMesh;

	UPROPERTY()
	TArray<FEdGraphPinReference> OutputMaterialPins;
	
private:

	bool IsMaterialOutputPin(const UEdGraphPin& Pin) const;

	bool CreateMaterialOutputPin(UCONodeSkeletalMeshBreakPinData_V2* InPinData);

protected:
	
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;

};

