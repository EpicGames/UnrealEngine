// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshMake_V2.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "ScopedTransaction.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSkeletalMeshMake_V2)


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCONodeSkeletalMeshMake_V2::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Inputs
	const int32 NumLODs = !LODPins.IsEmpty() ? LODPins.Num() : 1;
	LODPins.Empty(NumLODs);
	for (uint16 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		const FString LODName = GetNameForLODPin(LODIndex);
		UEdGraphPin* LODPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
		LODPin->PinType.ContainerType = EPinContainerType::Array;

		LODPins.Add(LODPin);
	}
	
	// Output
	SkeletalMeshPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


UCustomizableObjectNodeRemapPins* UCONodeSkeletalMeshMake_V2::CreateRemapPinsDefault() const
{
	return CreateRemapPinsByPosition();
}


EAddPinNodeButtonLocation UCONodeSkeletalMeshMake_V2::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::INPUT;
}


void UCONodeSkeletalMeshMake_V2::AddPinFromUI()
{
	Super::AddPinFromUI();

	FScopedTransaction Transaction(LOCTEXT("CONodeMutableSkeletalMeshMake_AddPinFromNodeUITransaction", "Add Pin from Node UI"));
	Modify();
	
	const FString LODName = GetNameForLODPin(LODPins.Num());
	UEdGraphPin* LODPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
	LODPin->PinType.ContainerType = EPinContainerType::Array;

	LODPins.Add(LODPin);
	
	GetGraph()->NotifyNodeChanged(this);
}


bool UCONodeSkeletalMeshMake_V2::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	// Disallow the removal of already connected pins (to prevent human error)
	// Only allow the removal of a pin if that pin is one definning a lod there are more than one LODs available (0 LODs is not a valid option)
	return Pin.Direction == EGPD_Input && LODPins.Num() > 1 && !Pin.LinkedTo.Num() && !Pin.bOrphanedPin  && LODPins.Contains(&Pin);
}


bool UCONodeSkeletalMeshMake_V2::CustomRemovePin(UEdGraphPin& Pin)
{
	if (LODPins.Remove(&Pin))
	{
		const bool bWasRemovalSuccessful = Super::CustomRemovePin(Pin);
		check(bWasRemovalSuccessful);
	
		ReconstructNode();
	
		return bWasRemovalSuccessful;
	}
	else
	{
		return Super::CustomRemovePin(Pin);
	}
}


FText UCONodeSkeletalMeshMake_V2::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("CONodeMutableSkeletalMeshMake_NodeTitle", "Skeletal Mesh Make");
}


FLinearColor UCONodeSkeletalMeshMake_V2::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}

#undef LOCTEXT_NAMESPACE
