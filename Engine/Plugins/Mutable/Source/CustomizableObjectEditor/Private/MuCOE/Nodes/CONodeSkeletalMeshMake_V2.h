// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CONodeSkeletalMeshMake_V2.generated.h"



UCLASS()
class UCONodeSkeletalMeshMake_V2 : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// UEdGraphNode
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	
	// UCustomizableObjectNode
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;
	virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	
protected:
	
	// UCustomizableObjectNode
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	
private:
	
	FString GetNameForLODPin(const int32& InIndex) const
	{
		return FString::Printf(TEXT("LOD %d"), InIndex);
	}
	
public:
	
	UPROPERTY()
	TArray<FEdGraphPinReference> LODPins;
	
	UPROPERTY()
	FEdGraphPinReference SkeletalMeshPin;
};
