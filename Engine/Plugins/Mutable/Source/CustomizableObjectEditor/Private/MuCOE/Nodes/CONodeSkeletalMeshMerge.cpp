// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshMerge.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeSkeletalMeshMerge::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("CONodeSkeletalMeshMerge_NodeTitle", "Skeletal Mesh Merge");
}


FLinearColor UCONodeSkeletalMeshMerge::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


void UCONodeSkeletalMeshMerge::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Inputs
	BaseSkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh,  TEXT("Base"), LOCTEXT("CONodeSkeletalMeshMerge_BaseSkeletalMeshName", "Base"));
	ToAddSkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Added"), LOCTEXT("CONodeSkeletalMeshMerge_ToAddSkeletalMeshName", "Added"));

	// Output
	SkeletalMeshOutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


#undef LOCTEXT_NAMESPACE