// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeSkeletalMeshMerge.generated.h"


UCLASS()
class UCONodeSkeletalMeshMerge : public UCustomizableObjectNode
{
public:
	
	GENERATED_BODY()
	
	// UEdGraphNode
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	
protected:
	
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	
public:
	
	UPROPERTY()
	FEdGraphPinReference BaseSkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference ToAddSkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference SkeletalMeshOutputPin;
	
};
