// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshModify.h"

#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

const FName SkeletalMeshModifySlotPinName = "Slot";
const FText SkeletalMeshModifySlotPinText = LOCTEXT("Slot", "Slot");


FText UCONodeSkeletalMeshModify::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("MutableSkeletalMeshModify_NodeTitle","Skeletal Mesh Modify");
}


FLinearColor UCONodeSkeletalMeshModify::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


FText UCONodeSkeletalMeshModify::GetTooltipText() const
{
	return LOCTEXT("MutableSkeletalMeshModify_NodeTooltip","Override the material being used in a given slot of the provided skeletal mesh.");
}


TSharedPtr<IDetailsView> UCONodeSkeletalMeshModify::CustomizePinDetails(const UEdGraphPin& Pin) const
{
	if (UCONodeMutableSkeletalMeshModifySlotPinData* PinData = Cast<UCONodeMutableSkeletalMeshModifySlotPinData>(GetPinData(Pin)))
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;

		const TSharedRef<IDetailsView> SettingsView = EditModule.CreateDetailView(DetailsViewArgs);
		SettingsView->SetObject(PinData);

		return SettingsView;
	}

	return nullptr;
}


bool UCONodeSkeletalMeshModify::HasPinViewer() const
{
	return true;
}


EAddPinNodeButtonLocation UCONodeSkeletalMeshModify::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::INPUT;
}

void UCONodeSkeletalMeshModify::AddPinFromUI()
{
	Super::AddPinFromUI();

	FScopedTransaction Transaction(LOCTEXT("CONodeMutableSkeletalMeshModify_AddPinFromNodeUITransaction", "Add Pin from Node UI"));
	Modify();
	
	UCONodeMutableSkeletalMeshModifySlotPinData* PinData = NewObject<UCONodeMutableSkeletalMeshModifySlotPinData>(this);
	if (CustomCreatePin(EEdGraphPinDirection::EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material,
		SkeletalMeshModifySlotPinName, SkeletalMeshModifySlotPinText, PinData))
	{
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCONodeSkeletalMeshModify::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	return IsSlotPin(Pin) && !Pin.bOrphanedPin && !Pin.LinkedTo.Num();
}


bool UCONodeSkeletalMeshModify::CanPinBeHidden(const UEdGraphPin& Pin) const
{
	return Super::CanPinBeHidden(Pin) && IsSlotPin(Pin);
}


bool UCONodeSkeletalMeshModify::CanRenamePin(const UEdGraphPin& Pin) const
{
	return IsSlotPin(Pin);
}


EEditablePinNameBoxVisibilityPolicy UCONodeSkeletalMeshModify::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


void UCONodeSkeletalMeshModify::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (UCONodeMutableSkeletalMeshModifySlotPinData* CastedPinData = Cast<UCONodeMutableSkeletalMeshModifySlotPinData>(PinData))
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCONodeMutableSkeletalMeshModify_SetNamePinEditableNameTransaction", "Change Pin Name"));
		CastedPinData->Modify();
		
		CastedPinData->SlotName = FName{Value.ToString()};
		PinEditableNameChangedDelegate.Broadcast(Pin, Value);
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}

FText UCONodeSkeletalMeshModify::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (const UCONodeMutableSkeletalMeshModifySlotPinData* CastedPinData = Cast<UCONodeMutableSkeletalMeshModifySlotPinData>(PinData))
	{
		return FText::FromName(CastedPinData->SlotName);
	}

	return Super::GetPinEditableName(Pin);
}


void UCONodeSkeletalMeshModify::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Static inputs
	MutableSkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	
	// Output
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	
	// Runtime Slot added pins
	{
		const TArray<UEdGraphPin*> NonOrphanPins = GetAllNonOrphanPins();

		// Regenerate the new pins from the old pin's pindata which may have changed
		for (const UEdGraphPin* Pin : NonOrphanPins)
		{
			check(Pin);
			if (IsSlotPin(*Pin))
			{
				UCustomizableObjectNodePinData* PinData = GetPinData(*Pin);
				CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, SkeletalMeshModifySlotPinName, SkeletalMeshModifySlotPinText, PinData);
			}
		}
	}
}


UCustomizableObjectNodeRemapPins* UCONodeSkeletalMeshModify::CreateRemapPinsDefault() const
{
	return CreateRemapPinsByPosition();
}


bool UCONodeSkeletalMeshModify::IsSlotPin(const UEdGraphPin& Pin) const
{
	return Pin.PinName == SkeletalMeshModifySlotPinName;
}


FName UCONodeSkeletalMeshModify::GetTargetSlotName(const UEdGraphPin& Pin) const
{
	check(IsSlotPin(Pin));
	
	const UCONodeMutableSkeletalMeshModifySlotPinData* PinData = CastChecked<UCONodeMutableSkeletalMeshModifySlotPinData>(GetPinData(Pin));
	return PinData->SlotName;
}

#undef LOCTEXT_NAMESPACE
