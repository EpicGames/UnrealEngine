// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "CONodeSkeletalMeshModify.generated.h"


/** Base class for all skeletal mesh slot pins. */
UCLASS()
class UCONodeMutableSkeletalMeshModifySlotPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	FName SlotName;
};


UCLASS()
class UCONodeSkeletalMeshModify : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	
	// UCustomizableObjectNode interface
	virtual TSharedPtr<IDetailsView> CustomizePinDetails(const UEdGraphPin& Pin) const override;
	virtual bool HasPinViewer() const override;
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;
	virtual bool CanPinBeHidden(const UEdGraphPin& Pin) const override;

protected:
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;

public:
	// Own Interface : Editable area of the pin name management
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	
	bool IsSlotPin(const UEdGraphPin& Pin) const;
	FName GetTargetSlotName(const UEdGraphPin& Pin) const;

	UPROPERTY()
	FEdGraphPinReference MutableSkeletalMeshPin;
};