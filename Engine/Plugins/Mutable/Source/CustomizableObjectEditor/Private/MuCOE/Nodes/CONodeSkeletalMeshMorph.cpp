// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshMorph.h"

#include "CustomizableObjectNodeStaticString.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeSkeletalMeshMorph::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("CONodeMutableSkeletalMeshMorph_NodeTitle", "Skeletal Mesh Morph");
}


FLinearColor UCONodeSkeletalMeshMorph::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


FText UCONodeSkeletalMeshMorph::GetTooltipText() const
{
	return LOCTEXT("CONodeMutableSkeletalMeshMorph_Tooltip", "Changes the weight of a skeletal mesh morph target.");
}


bool UCONodeSkeletalMeshMorph::CanRenamePin(const UEdGraphPin& Pin) const
{
	return  FEdGraphPinReference(&Pin) == MorphTargetNamePinReference;
}


FText UCONodeSkeletalMeshMorph::GetPinEditableName(const UEdGraphPin& Pin) const
{
	return CanRenamePin(Pin) == true ? FText::FromString(MorphTargetName) : FText();
}


void UCONodeSkeletalMeshMorph::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	if (CanRenamePin(Pin))
	{
		MorphTargetName = Value.ToString();
	}
}


FName UCONodeSkeletalMeshMorph::GetMorphTargetName(FMutableGraphGenerationContext& GenerationContext) const
{
	const UEdGraphPin* InputMorphNamePin = MorphTargetNamePinReference.Get();
	check(InputMorphNamePin);
	if (const UEdGraphPin* ConnectedPin = FollowInputPin(*InputMorphNamePin))
	{
		if (const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, &GenerationContext.MacroNodesStack))
		{
			if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
			{
				// Use the value set in the connected Static string node
				return FName(StringNode->Value);
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MorphStringNodeTypeNotHandled", "Could not extract the target name from the connected string providing node."), this, EMessageSeverity::Error);
				return NAME_None;
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MorphStringNodeFailed", "Could not find a linked String node."), this, EMessageSeverity::Error);
			return NAME_None;
		}
	}
	else
	{
		// If no connection is found to the input name pin then use the value stored locally
		return FName(MorphTargetName);
	}
}


void UCONodeSkeletalMeshMorph::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	MeshPinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	
	const FText MorphFactorPinName = LOCTEXT("CONodeMutableSkeletalMeshMorph_MorphFactorPinName", "Factor");
	MorphFactorPinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, *MorphFactorPinName.ToString(), MorphFactorPinName);
	
	const FText MorphTargetPinName = LOCTEXT("CONodeMutableSkeletalMeshMorph_MorphTargetPinName", "Morph Target Name");
	MorphTargetNamePinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, *MorphTargetPinName.ToString(), MorphTargetPinName);
	
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


#undef LOCTEXT_NAMESPACE

