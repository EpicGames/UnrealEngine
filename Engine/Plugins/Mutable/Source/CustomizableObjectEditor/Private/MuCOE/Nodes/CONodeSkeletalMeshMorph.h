// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeSkeletalMeshMorph.generated.h"

class UCustomizableObjectNodeRemapPins;


UCLASS()
class UCONodeSkeletalMeshMorph : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;

	/**
	 * Based on if something is connected to the input string pin for the Morph target name returns or the value from the connected string node or the value
	 * stored locally (used to store the value set when the target morph pin is not connected)
	 * @param GenerationContext The compilation generation context.
	 * @return The name of the target morph. May come from the locally stored value or the connected string node.
	 */
	FName GetMorphTargetName(FMutableGraphGenerationContext& GenerationContext) const;
	
protected:
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

public:
	
	/***
	 * Input skeletal mesh whose morph you want to interact with
	 */
	UPROPERTY()
	FEdGraphPinReference MeshPinReference;

	/**
	 * The weight value to be applied onto the targetted morph
	 */
	UPROPERTY()
	FEdGraphPinReference MorphFactorPinReference;
	
private:
	
	/**
	 * The name of the morph target present in the provided skeletal mesh you want to interact with.
	 */
	UPROPERTY()
	FString MorphTargetName;

	/**
	 * The name of the morph target you want to control
	 */
	UPROPERTY()
	FEdGraphPinReference MorphTargetNamePinReference;
};
