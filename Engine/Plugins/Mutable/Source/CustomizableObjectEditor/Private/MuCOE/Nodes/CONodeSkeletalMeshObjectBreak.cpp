// Copyright Epic Games, Inc. All Rights Reserved.

#include "CONodeSkeletalMeshObjectBreak.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "ScopedTransaction.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeSkeletalMeshObjectBreak::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("UCONodeSkeletalMeshObjectBreak_NodeTitle","Skeletal Mesh Object Break");
}


FText UCONodeSkeletalMeshObjectBreak::GetTooltipText() const
{
	return Super::GetTooltipText();
}


FLinearColor UCONodeSkeletalMeshObjectBreak::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object);
}


void UCONodeSkeletalMeshObjectBreak::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Passthrough skm input
	InputPassthroughMesh = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object);
	
	// Regenerate the new pins from the old pin's PinData which may have changed
	OutputMaterialPins.Reset();

	// Get old pins
	TArray<UEdGraphPin*> AllNonOrphanPins = GetAllNonOrphanPins();
	for (const UEdGraphPin* Pin : AllNonOrphanPins)
	{
		if (!IsMaterialOutputPin(*Pin))
		{
			continue;
		}
		
		if (UCONodeSkeletalMeshObjectBreakPinData* PinData = Cast<UCONodeSkeletalMeshObjectBreakPinData>(GetPinData(*Pin)))
		{
			CreateMaterialOutputPin(PinData);
		}
	}
}

void UCONodeSkeletalMeshObjectBreak::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SkeletalMeshBreakMaterialPinType)
	{
		InputPassthroughMesh.Get()->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object;
	}
}


FText UCONodeSkeletalMeshObjectBreak::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (const UCONodeSkeletalMeshObjectBreakPinData* CastedPinData = Cast<UCONodeSkeletalMeshObjectBreakPinData>(PinData))
	{
		return FText::FromName(CastedPinData->TargetMaterialSlotName);
	}

	return Super::GetPinEditableName(Pin);
}


void UCONodeSkeletalMeshObjectBreak::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	UCustomizableObjectNodePinData* PinData = GetPinData(Pin);
	if (UCONodeSkeletalMeshObjectBreakPinData* CastedPinData = Cast<UCONodeSkeletalMeshObjectBreakPinData>(PinData))
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCONodeSkeletalMeshBreak_SetNamePinEditableNameTransaction", "Change Pin Name"));
		CastedPinData->Modify();
		
		CastedPinData->TargetMaterialSlotName = FName{Value.ToString()};
		PinEditableNameChangedDelegate.Broadcast(Pin, Value);
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}


bool UCONodeSkeletalMeshObjectBreak::HasPinViewer() const
{
	return true;
}


EAddPinNodeButtonLocation UCONodeSkeletalMeshObjectBreak::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::OUTPUT;
}


EEditablePinNameBoxVisibilityPolicy UCONodeSkeletalMeshObjectBreak::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


void UCONodeSkeletalMeshObjectBreak::AddPinFromUI()
{
	FScopedTransaction Transaction(LOCTEXT("UCONodeSkeletalMeshBreak_AddPinFromNodeUITransaction", "Add Material Output Pin from Node UI"));
	Modify();

	UCONodeSkeletalMeshObjectBreakPinData* PinData = NewObject<UCONodeSkeletalMeshObjectBreakPinData>(this);
	if (CreateMaterialOutputPin(PinData))
	{
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCONodeSkeletalMeshObjectBreak::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	const bool bCanPinBeRemoved = !Pin.LinkedTo.Num() && !Pin.bOrphanedPin;
	if (!bCanPinBeRemoved)
	{
		return false;
	}
	
	return IsMaterialOutputPin(Pin);
}


bool UCONodeSkeletalMeshObjectBreak::CustomRemovePin(UEdGraphPin& Pin)
{
	OutputMaterialPins.Remove(&Pin);
	return Super::CustomRemovePin(Pin);
}


bool UCONodeSkeletalMeshObjectBreak::CanRenamePin(const UEdGraphPin& Pin) const
{
	return IsMaterialOutputPin(Pin);
}


bool UCONodeSkeletalMeshObjectBreak::CreateMaterialOutputPin(UCONodeSkeletalMeshObjectBreakPinData* InPinData)
{
	UEdGraphPin* NewPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Material, TEXT("Material Slot"), LOCTEXT("MaterialSlot", "Material Slot"), InPinData);
	if (NewPin)
	{
		OutputMaterialPins.Add(NewPin);
		return true;
	}
	
	return false;
}


UCustomizableObjectNodeRemapPins* UCONodeSkeletalMeshObjectBreak::CreateRemapPinsDefault() const
{
	return Super::CreateRemapPinsByPosition();
}


bool UCONodeSkeletalMeshObjectBreak::IsMaterialOutputPin(const UEdGraphPin& Pin) const
{
	return Pin.Direction == EEdGraphPinDirection::EGPD_Output && Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Material;
}




#undef LOCTEXT_NAMESPACE
