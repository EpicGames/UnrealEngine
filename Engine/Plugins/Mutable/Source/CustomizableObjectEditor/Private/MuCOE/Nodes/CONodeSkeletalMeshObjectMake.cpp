// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshObjectMake.h"

#include "CONodeSkeletalMeshMake_V2.h"
#include "CustomizableObjectNodeComponent.h"
#include "CustomizableObjectNodeStaticString.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"

#include "ScopedTransaction.h"
#include "MuCO/CustomizableObjectCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSkeletalMeshObjectMake)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

UCONodeSkeletalMeshObjectMake::UCONodeSkeletalMeshObjectMake()
{
	const FString CVarName = TEXT("r.SkeletalMesh.MinLodQualityLevel");
	const FString ScalabilitySectionName = TEXT("ViewDistanceQuality");
	LODSettings.MinQualityLevelLOD.SetQualityLevelCVarForCooking(*CVarName, *ScalabilitySectionName);
}


FText UCONodeSkeletalMeshObjectMake::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Skeletal_Mesh_Make_Node_Title", "Skeletal Mesh Object Make");
}


FLinearColor UCONodeSkeletalMeshObjectMake::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


void UCONodeSkeletalMeshObjectMake::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	MeshNamePin = CustomCreatePin(EEdGraphPinDirection::EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
	SkeletalMeshPin = CustomCreatePin(EEdGraphPinDirection::EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	
	PassthroughSkeletalMeshPin = CustomCreatePin(EEdGraphPinDirection::EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
}


bool UCONodeSkeletalMeshObjectMake::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCONodeSkeletalMeshObjectMake::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UEdGraphPin* ComponentPin = MeshNamePin.Get();
	if (ComponentPin && ComponentPin->PinId == Pin.PinId)
	{
		return FText::FromString(GetSkeletalMeshName());
	}

	return FText::GetEmpty();
}


void UCONodeSkeletalMeshObjectMake::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	const UEdGraphPin* ComponentPin = MeshNamePin.Get();
	if (ComponentPin && ComponentPin->PinId == Pin.PinId)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCONodeSkeletalMeshMake_SetSKMPinEditableNameTransaction", "Change SKMName PinName"));
		Modify();
		
		SkeletalMeshName = InValue.ToString();
		PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
	}
}


void UCONodeSkeletalMeshObjectMake::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::CopySKMComponentNameToMakeSKMNode)
	{
		const UEdGraphPin* OutputPin = PassthroughSkeletalMeshPin.Get();
		check(OutputPin);
		
		// Only proceed if the output pin is connected to a Skeletal Mesh component node
		if (const UEdGraphPin* ConnectedPin = FollowOutputPin(*OutputPin))
		{
			const UEdGraphNode* OwningNode = ConnectedPin->GetOwningNode();
			if (const UCustomizableObjectNodeComponent* SkeletalMeshComponentNode = Cast<UCustomizableObjectNodeComponent>(OwningNode))
			{
				const UEdGraphPin* MeshNamePinPtr = MeshNamePin.Get();
				check(MeshNamePinPtr);
				
				// The name to be used as an override for the skeletal mesh name
				const FString ComponentName = SkeletalMeshComponentNode->GetComponentName(nullptr).ToString();
				
				const UEdGraphPin* ConnectedNamePin = FollowInputPin(*MeshNamePinPtr);

				// Set the name stored in the local property if no node is found to be connected to the string input pin
				if (!ConnectedNamePin)
				{
					SkeletalMeshName = ComponentName;
				}
				// Set the name in the connected node if that node can be resolved to a UCustomizableObjectNodeStaticString
				else
				{
					if (UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(ConnectedNamePin->GetOwningNode()))
					{
						StringNode->Value = ComponentName;
					}
				}
			}
		}
	}	
}


const UCONodeSkeletalMeshMake_V2* UCONodeSkeletalMeshObjectMake::GetConnectedMutableSkeletalMeshMakeNode(
	TArray<const UCustomizableObjectNodeMacroInstance*>& MacroContext) const
{
	if (const UEdGraphPin* MeshPin = SkeletalMeshPin.Get())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*MeshPin))
		{
			const UEdGraphPin* SkeletalMeshMakeOutputPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, &MacroContext);

			if (const UCONodeSkeletalMeshMake_V2* MutableMakeNode = SkeletalMeshMakeOutputPin ? Cast<UCONodeSkeletalMeshMake_V2>(SkeletalMeshMakeOutputPin->GetOwningNode()) : nullptr)
			{
				return MutableMakeNode;
			}
		}
	}
	
	return nullptr;
}


FString UCONodeSkeletalMeshObjectMake::GetSkeletalMeshName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (const UEdGraphPin* NamePin = MeshNamePin.Get())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*NamePin))
		{
			const UEdGraphPin* StringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, MacroContext);
			if (const UCustomizableObjectNodeStaticString* StringNode = StringPin ? Cast<UCustomizableObjectNodeStaticString>(StringPin->GetOwningNode()) : nullptr)
			{
				return StringNode->Value;
			}
		}
	}

	return SkeletalMeshName;
}

#undef LOCTEXT_NAMESPACE
