// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CONodeSkeletalMeshObjectMake.generated.h"

class UCONodeSkeletalMeshMake_V2;

UCLASS()
class UCONodeSkeletalMeshObjectMake : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	UCONodeSkeletalMeshObjectMake();
	
	// UEdGraphNode
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	
	// UCustomizableObjectNode
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue) override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	/**
	 * Get a pointer to the connected UCONodeMutableSkeletalMeshMake node.
	 * @param MacroContext The current macro stack
	 * @return A valid pointer if the node is connected, a null pointer otherwise.
	 */
	const UCONodeSkeletalMeshMake_V2* GetConnectedMutableSkeletalMeshMakeNode(TArray<const UCustomizableObjectNodeMacroInstance*>& MacroContext) const;;
	
	FString GetSkeletalMeshName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;
	
	UPROPERTY(EditAnywhere, Category=SkeletalMesh, meta = (ClampMin=1))
	uint16 NumLODs = 1;
	
	UPROPERTY(EditAnywhere, Category = SkeletalMesh)
	FMutableLODSettings LODSettings;
	
	/** Name to use by the generated skeletal mesh object. */
	UPROPERTY()
	FEdGraphPinReference MeshNamePin;
	
	UPROPERTY()
	FEdGraphPinReference SkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference PassthroughSkeletalMeshPin;
	
private:
	UPROPERTY()
	FString SkeletalMeshName = "Default Name";
};