// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshReshape.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeSkeletalMeshReshape::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("CONodeSkeletalMeshReshape_NodeTitle", "Skeletal Mesh Reshape");
}


FLinearColor UCONodeSkeletalMeshReshape::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


FText UCONodeSkeletalMeshReshape::GetTooltipText() const
{
	return LOCTEXT("CONodeSkeletalMeshReshape_Tooltip", "Reshape a skeletal mesh using a base shape and a target shape.");
}


void UCONodeSkeletalMeshReshape::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output,UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	BasePinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Base Skeletal Mesh"), LOCTEXT("BaseSkeletalMesh", "Base Skeletal Mesh"));
	BaseShapePinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Base Shape"), LOCTEXT("BaseShape", "Base Shape"));
	TargetShapePinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh, TEXT("Target Shape"), LOCTEXT("TargetShape", "Target Shape"));
}


#undef LOCTEXT_NAMESPACE
