// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshape.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeCommon.h"

#include "CONodeSkeletalMeshReshape.generated.h"

class UCustomizableObjectNodeRemapPins;


UCLASS()
class UCONodeSkeletalMeshReshape : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

protected:

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

public:

	/** Input skeletal mesh that will be reshaped. */
	UPROPERTY()
	FEdGraphPinReference BasePinReference;

	/** Input skeletal mesh describing the base shape used to sample the original geometry. */
	UPROPERTY()
	FEdGraphPinReference BaseShapePinReference;

	/** Input skeletal mesh describing the destination shape for the reshape. */
	UPROPERTY()
	FEdGraphPinReference TargetShapePinReference;

	/** Enable the deformation of the vertices of the base mesh. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bReshapeVertices = true;

	/** Enable recompute normals after the reshape operation. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (EditCondition = "bReshapeVertices"))
	bool bRecomputeNormals = false;

	/** Enable laplacian smoothing to the result of the base mesh reshape. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (EditCondition = "bReshapeVertices"))
	bool bApplyLaplacianSmoothing = false;

	/** Enable the deformation of the skeleton of the base mesh. */
	UPROPERTY(EditAnywhere, Category = ReshapePose)
	bool bReshapePose = false;

	/** Enable the deformation of physics volumes of the base mesh. */
	UPROPERTY(EditAnywhere, Category = ReshapePhysics)
	bool bReshapePhysics = false;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FMeshReshapeColorUsage VertexColorUsage;

	/** Bone reshape selection method. */
	UPROPERTY(EditAnywhere, Category = ReshapePose, meta = (EditCondition = "bReshapePose"))
	EBoneDeformSelectionMethod SelectionMethod = EBoneDeformSelectionMethod::ONLY_SELECTED;

	/** Bones that participate in the skeleton reshape. Type the bone name as it appears in the source skeleton. */
	UPROPERTY(EditAnywhere, Category = ReshapePose, meta = (EditCondition = "bReshapePose"))
	TArray<FName> BonesToDeform;

	/** Physics body selection method. */
	UPROPERTY(EditAnywhere, Category = ReshapePhysics, meta = (DisplayName = "Selection Method", EditCondition = "bReshapePhysics"))
	EBoneDeformSelectionMethod PhysicsSelectionMethod = EBoneDeformSelectionMethod::ONLY_SELECTED;

	/** Bones whose physics bodies participate in the physics reshape. Type the bone name as it appears in the source skeleton. */
	UPROPERTY(EditAnywhere, Category = ReshapePhysics, meta = (EditCondition = "bReshapePhysics"))
	TArray<FName> PhysicsBodiesToDeform;
};
