// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"

#include "MaterialCachedData.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "Materials/MaterialInstance.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/SCONodeSkeletalMeshSection.h"
#include "ObjectEditorUtils.h"
#include "PropertyCustomizationHelpers.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSkeletalMeshSection)

class SGraphNode;
class SWidget;
class UCustomizableObjectLayout;
class UObject;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


const TArray<EMaterialParameterType> UCONodeSkeletalMeshSection::ParameterTypes = {
	EMaterialParameterType::Texture,
	EMaterialParameterType::Vector,
	EMaterialParameterType::Scalar };


bool UCustomizableObjectNodeMaterialRemapPinsByName::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const
{
	const UCONodeSkeletalMeshSectionPinDataParameter* PinDataOldPin = Cast<UCONodeSkeletalMeshSectionPinDataParameter>(Node.GetPinData(OldPin));
	const UCONodeSkeletalMeshSectionPinDataParameter* PinDataNewPin = Cast<UCONodeSkeletalMeshSectionPinDataParameter>(Node.GetPinData(NewPin));
	if (PinDataOldPin && PinDataNewPin)
	{
		return PinDataOldPin->MaterialParameterId == PinDataNewPin->MaterialParameterId && (!OldPin.LinkedTo.Num() || OldPin.PinType == NewPin.PinType); // Pin type must match only if it was connected
	}
	else
	{
		return Super::Equal(Node, OldPin, NewPin);
	}
}


void UCustomizableObjectNodeMaterialRemapPinsByName::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	for (UEdGraphPin* OldPin : OldPins)
	{
		bool bFound = false;

		for (UEdGraphPin* NewPin : NewPins)
		{
			if (Equal(Node, *OldPin, *NewPin))
			{
				bFound = true;

				if (UEdGraphPin** Result = PinsToRemap.Find(OldPin))
				{
					if ((*Result)->bOrphanedPin) // The node can have a deprecated and non-deprecated pin that should remap to the same new pin. Prioritize the non-deprecated
					{
						*Result = NewPin;
					}
				}
				else
				{
					PinsToRemap.Add(OldPin, NewPin);
				}
			}
		}

		if (!bFound && (OldPin->LinkedTo.Num() || HasSavedPinData(Node, *OldPin)))
		{
			PinsToOrphan.Add(OldPin);
		}
	}
}


bool UCustomizableObjectNodeMaterialRemapPinsByName::HasSavedPinData(const UCustomizableObjectNode& Node, const UEdGraphPin &Pin) const
{
	if (const UCONodeSkeletalMeshSectionPinDataParameter* PinData = Cast<UCONodeSkeletalMeshSectionPinDataParameter>(Node.GetPinData(Pin)))
	{
		return !PinData->IsDefault();
	}
	else
	{
		return false;
	}
}


bool UCONodeSkeletalMeshSectionPinDataImage::IsDefault() const
{
	const UCONodeSkeletalMeshSectionPinDataImage* Default = Cast<UCONodeSkeletalMeshSectionPinDataImage>(GetClass()->GetDefaultObject(false));

	return PinMode == Default->PinMode &&
		UVLayoutMode == Default->UVLayoutMode &&
		ReferenceTexture == Default->ReferenceTexture;
}


EPinMode UCONodeSkeletalMeshSectionPinDataImage::GetPinMode() const
{
	return PinMode;
}


void UCONodeSkeletalMeshSectionPinDataImage::SetPinMode(const EPinMode InPinMode)
{
	FObjectEditorUtils::SetPropertyValue(this, GET_MEMBER_NAME_CHECKED(UCONodeSkeletalMeshSectionPinDataImage, PinMode), InPinMode);
	NodeMaterial->GetPostImagePinModeChangedDelegate()->Broadcast();
}


void UCONodeSkeletalMeshSectionPinDataImage::Init(UCONodeSkeletalMeshSection& InMeshSectionNode)
{
	NodeMaterial = &InMeshSectionNode;
}


void UCONodeSkeletalMeshSectionPinDataImage::Copy(const UCustomizableObjectNodePinData& Other)
{
	if (const UCONodeSkeletalMeshSectionPinDataImage* PinDataOldPin = Cast<UCONodeSkeletalMeshSectionPinDataImage>(&Other))
	{
		PinMode = PinDataOldPin->PinMode;
		UVLayoutMode = PinDataOldPin->UVLayoutMode;
		UVLayout = PinDataOldPin->UVLayout;
		ReferenceTexture = PinDataOldPin->ReferenceTexture;
	}
}


FName UCONodeSkeletalMeshSection::GetPinName(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	const FString ParameterName = GetParameterName(Type, ParameterIndex).ToString();
	return FName(GetParameterLayerIndex(Type, ParameterIndex) != INDEX_NONE ? GetParameterLayerName(Type, ParameterIndex).ToString() + " - " + ParameterName : ParameterName);
}


int32 UCONodeSkeletalMeshSection::GetExpressionTextureCoordinate(UMaterial* Material, const FGuid& ImageId)
{
	if (const UMaterialExpressionTextureSample* TextureSample = Material->FindExpressionByGUID<UMaterialExpressionTextureSample>(ImageId))
	{
		if (!TextureSample->Coordinates.Expression)
		{
			return TextureSample->ConstCoordinate;
		}
		else if (const UMaterialExpressionTextureCoordinate* TextureCoords = Cast<UMaterialExpressionTextureCoordinate>(TextureSample->Coordinates.Expression))
		{
			return TextureCoords->CoordinateIndex;
		}
	}

	return -1;
}


FName UCONodeSkeletalMeshSection::NodePinModeToImagePinMode(const ENodePinMode NodePinMode)
{
	switch (NodePinMode)
	{
	case ENodePinMode::Mutable:
		return UEdGraphSchema_CustomizableObject::PC_Texture;
	case ENodePinMode::Passthrough:
		return UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough;
	default:
		unimplemented();
		return {};
	}
}


FName UCONodeSkeletalMeshSection::GetImagePinMode(const EPinMode PinMode) const
{
	switch (PinMode)
	{
	case EPinMode::Default:
		return NodePinModeToImagePinMode(TextureParametersMode);
	case EPinMode::Mutable:
		return UEdGraphSchema_CustomizableObject::PC_Texture;
	case EPinMode::Passthrough:
		return UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough;
	default:
		unimplemented();
		return {};
	}
}


FName UCONodeSkeletalMeshSection::GetImagePinMode(const UEdGraphPin& Pin) const
{
	return GetImagePinMode(GetPinData<UCONodeSkeletalMeshSectionPinDataImage>(Pin).GetPinMode());
}


int32 UCONodeSkeletalMeshSection::GetImageUVLayoutFromMaterial(const int32 ImageIndex) const
{
	const FNodeMaterialParameterId ImageId = GetParameterId(EMaterialParameterType::Texture, ImageIndex);

	if (const int32 TextureCoordinate = GetExpressionTextureCoordinate(Material->GetMaterial(), ImageId.ParameterId);
		TextureCoordinate >= 0)
	{
		return TextureCoordinate;
	}

	FMaterialLayersFunctions Layers;
	Material->GetMaterialLayers(Layers);

	TArray<TArray<TObjectPtr<UMaterialFunctionInterface>>*> MaterialFunctionInterfaces;
	MaterialFunctionInterfaces.SetNumUninitialized(2);
	MaterialFunctionInterfaces[0] = &Layers.Layers;
	MaterialFunctionInterfaces[1] = &Layers.Blends;

	for (const TArray<TObjectPtr<UMaterialFunctionInterface>>* MaterialFunctionInterface : MaterialFunctionInterfaces)
	{
		for (const TObjectPtr<UMaterialFunctionInterface>& Layer : *MaterialFunctionInterface)
		{
			if (const int32 TextureCoordinate = GetExpressionTextureCoordinate(Layer->GetPreviewMaterial()->GetMaterial(), ImageId.ParameterId); TextureCoordinate >= 0)
			{
				return TextureCoordinate;
			}	
		}
	}

	return -1;
}


bool UCONodeSkeletalMeshSectionPinDataParameter::IsDefault() const
{
	return true;
}


/** Translates a given EPinMode to FText. */
FText EPinModeToText(const EPinMode PinMode)
{
	switch (PinMode)
	{
	case EPinMode::Default:
		return LOCTEXT("EPinModeDefault", "Node Defined");

	case EPinMode::Mutable:
		return LOCTEXT("EPinModeMutable", "Mutable");

	case EPinMode::Passthrough:
		return LOCTEXT("EPinModePassthrough", "Object");

	default:
		check(false); // Missing case.
		return FText();
	}
}


void UCONodeSkeletalMeshSectionPinDataImage::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeSkeletalMeshSectionPinDataImage, PinMode))
		{
			NodeMaterial->UCustomizableObjectNode::ReconstructNode();
		}
	}
}


bool UCONodeSkeletalMeshSectionPinDataImage::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeSkeletalMeshSectionPinDataImage, PinMode))
	{
		return !NodeMaterial->GetPin(*this)->LinkedTo.Num();
	}
	
	return Super::CanEditChange(InProperty);
}


void UCONodeSkeletalMeshSectionPinDataImage::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::PostLoad();
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeMaterialPinDataImageDetails)
	{
		if (UVLayout == UV_LAYOUT_IGNORE)
		{
			UVLayoutMode = EUVLayoutMode::Ignore;
			UVLayout = 0;
		}
		else if (UVLayout == UV_LAYOUT_DEFAULT)
		{
			UVLayoutMode = EUVLayoutMode::FromMaterial;
			UVLayout = 0;
		}
		else
		{
			UVLayoutMode = EUVLayoutMode::Index;
		}
	}
}


void UCONodeSkeletalMeshSection::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	{
		const FString PinFriendlyName = TEXT("Mesh");
		const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));
		InputMeshPin = CustomCreatePin(EGPD_Input, Schema->PC_Mesh, FName(*PinName), FText::FromString(PinFriendlyName));
	}
	
	{
		const FString PinFriendlyName = UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(Schema->PC_Material).ToString();
		const FString PinName = FString(TEXT("Material_Input_Pin"));
		InputMaterialPin = CustomCreatePin(EGPD_Input, Schema->PC_Material, FName(*PinName), FText::FromString(PinFriendlyName));
		InputMaterialPin.Get()->PinToolTip = "Pin for a Material from a Table Node";
	}

	{
		const FString PinFriendlyName = "Enable Tags";
		const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));
		UEdGraphPin* TagsPin = CustomCreatePin(EGPD_Input, Schema->PC_String, FName(PinName), FText::FromString(PinFriendlyName));
		TagsPin->PinType.ContainerType = EPinContainerType::Array;
		TagsPin->PinToolTip = "List of Tags that this node will Enable";
		EnableTagsPinRef = TagsPin;
	}

	for (const EMaterialParameterType Type : ParameterTypes)
	{
		AllocateDefaultParameterPins(Type);
	}

	{
		const FString PinFriendlyName = TEXT("Mesh Section");
		const FString PinName = PinFriendlyName + FString(TEXT("_Output_Pin"));
		CustomCreatePin(EGPD_Output, Schema->PC_MeshSection, FName(*PinName), FText::FromString(PinFriendlyName));
	}
}


bool UCONodeSkeletalMeshSection::CanPinBeHidden(const UEdGraphPin& Pin) const
{
	return Super::CanPinBeHidden(Pin) &&
		Pin.Direction == EGPD_Input && 
		Pin.PinType.PinCategory != UEdGraphSchema_CustomizableObject::PC_Mesh;
}


bool UCONodeSkeletalMeshSection::HasPinViewer() const
{
	return true;
}


FString UCONodeSkeletalMeshSection::GetInternalTagDisplayName()
{
	return FString::Printf(TEXT("Mesh Section [%s]"), Material ? *Material->GetName() : TEXT("no-material"));
}


UCustomizableObjectNodeRemapPinsByName* UCONodeSkeletalMeshSection::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeMaterialRemapPinsByName>();
}


void UCONodeSkeletalMeshSection::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeMaterial)
	{
		if (Material)
		{
			ConditionalPostLoadReference(*Material); // Make sure the Material has been fully loaded.
		}
		
		for (const FCustomizableObjectNodeMaterialImage& Image : Images_DEPRECATED)
		{
			const FString OldPinName = (Image.LayerIndex == -1 ? Image.Name : Image.PinName) + FString(TEXT("_Input_Image"));
			UEdGraphPin* OldPin = FindPin(OldPinName);
			if (!OldPin) // If we can not find a pin it means that the data was corrupted (old Image array and pins where not synchronized).
			{
				continue;
			}

			UCONodeSkeletalMeshSectionPinDataImage* PinData = NewObject<UCONodeSkeletalMeshSectionPinDataImage>(this);
			PinData->ParameterId_DEPRECATED = FGuid::NewGuid();
			PinData->ReferenceTexture = Image.ReferenceTexture;

			// Find referenced Material Parameter
			const int32 NumParameters = GetNumParameters(EMaterialParameterType::Texture);
			for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
			{
				if (GetParameterName(EMaterialParameterType::Texture, ParameterIndex).ToString() == Image.Name)
				{
					PinData->ParameterId_DEPRECATED = GetParameterId(EMaterialParameterType::Texture, ParameterIndex).ParameterId;

					if (Image.UVLayout == -1)
					{
						PinData->UVLayout = Image.UVLayout;
					}
					else
					{
						const int32 UVLayout = GetImageUVLayoutFromMaterial(ParameterIndex);
						if (UVLayout < 0) // Could not be deduced from the Material
						{
							PinData->UVLayout = Image.UVLayout;						
						}
						else if (UVLayout == Image.UVLayout)
						{
							PinData->UVLayout = UV_LAYOUT_DEFAULT;							
						}
						else
						{
							PinData->UVLayout = UVLayout;
						}					
					}
					
					break;
				}
			}
			
			AddPinData(*OldPin, *PinData);
		}

		for (const FCustomizableObjectNodeMaterialVector& Vector : VectorParams_DEPRECATED)
		{
			const FString OldPinName = (Vector.LayerIndex == -1 ? Vector.Name : Vector.PinName) + FString(TEXT("_Input_Vector"));
			const UEdGraphPin* OldPin = FindPin(OldPinName);
			if (!OldPin) // If we can not find a pin it means that the data was corrupted (old ScalarParams array and pins where not synchronized).
			{
				continue;
			}
			
			UCONodeSkeletalMeshSectionPinDataVector* PinData = NewObject<UCONodeSkeletalMeshSectionPinDataVector>(this);
			PinData->ParameterId_DEPRECATED = FGuid::NewGuid();
			
			// Find referenced Material Parameter
			const int32 NumParameters = GetNumParameters(EMaterialParameterType::Vector);
			for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
			{
				if (GetParameterName(EMaterialParameterType::Vector, ParameterIndex).ToString() == Vector.Name)
				{
					PinData->ParameterId_DEPRECATED = GetParameterId(EMaterialParameterType::Vector, ParameterIndex).ParameterId;
					break;
				}
			}
			
			AddPinData(*OldPin, *PinData);
		}

		for (const FCustomizableObjectNodeMaterialScalar& Scalar : ScalarParams_DEPRECATED)
		{
			const FString OldPinName = (Scalar.LayerIndex == -1 ? Scalar.Name : Scalar.PinName) + FString(TEXT("_Input_Scalar"));
			const UEdGraphPin* OldPin = FindPin(OldPinName);
			if (!OldPin) // If we can not find a pin it means that the data was corrupted (old ScalarParams array and pins where not synchronized).
			{
				continue;
			}
			
			UCONodeSkeletalMeshSectionPinDataScalar* PinData = NewObject<UCONodeSkeletalMeshSectionPinDataScalar>(this);
			PinData->ParameterId_DEPRECATED = FGuid::NewGuid();

			// Find referenced Material Parameter
			const int32 NumParameters = GetNumParameters(EMaterialParameterType::Scalar);
			for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
			{
				if (GetParameterName(EMaterialParameterType::Scalar, ParameterIndex).ToString() == Scalar.Name)
				{
					PinData->ParameterId_DEPRECATED = GetParameterId(EMaterialParameterType::Scalar, ParameterIndex).ParameterId;
					break;
				}
			}
			
			AddPinData(*OldPin, *PinData);
		}

		// Check if there are still pins which where not present in the Images, ScalarParams and ScalarParams arrays.
		for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if ((Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture ||
 				 Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Color ||
				 Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Float) &&
				!GetPinData(*Pin))
			{
				UCONodeSkeletalMeshSectionPinDataParameter* PinData = [&](UObject* Outer) -> UCONodeSkeletalMeshSectionPinDataParameter*
				{
					if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture)
					{
						return NewObject<UCONodeSkeletalMeshSectionPinDataImage>(Outer);	
					}
					else if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Color)
					{
						return NewObject<UCONodeSkeletalMeshSectionPinDataVector>(Outer);
					}
					else if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Float)
					{
						return NewObject<UCONodeSkeletalMeshSectionPinDataScalar>(Outer);	
					}
					else
					{
						check(false); // Parameter type not contemplated.
						return nullptr;
					}
				}(this);
				
				PinData->ParameterId_DEPRECATED = FGuid::NewGuid();
				
				AddPinData(*Pin, *PinData);
			}
		}
		
		Images_DEPRECATED.Empty();
		VectorParams_DEPRECATED.Empty();
		ScalarParams_DEPRECATED.Empty();
		
		Super::ReconstructNode(); // Super required to avoid ambiguous call compilation error.
	}

	// Fill PinsParameter.
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeMaterialPerformanceBug) // || CustomizableObjectCustomVersion < FCustomizableObjectCustomVersion::AutomaticNodeMaterialPerformance
	{
		for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if (const UCONodeSkeletalMeshSectionPinDataParameter* PinData = Cast<UCONodeSkeletalMeshSectionPinDataParameter>(GetPinData(*Pin)))
			{
				PinsParameter_DEPRECATED.Add(PinData->ParameterId_DEPRECATED, FEdGraphPinReference(Pin));
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeMaterialUXImprovements)
	{
		TextureParametersMode = bDefaultPinModeMutable_DEPRECATED ?
			ENodePinMode::Mutable :
			ENodePinMode::Passthrough;
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ExtendMaterialOnlyMutableModeParameters)
	{
		const uint32 NumTextureParameters = GetNumParameters(EMaterialParameterType::Texture);
		for (uint32 ImageIndex = 0; ImageIndex < NumTextureParameters; ++ImageIndex)
		{
			if (const UEdGraphPin* Pin = GetParameterPin(EMaterialParameterType::Texture, ImageIndex))
			{
				UCONodeSkeletalMeshSectionPinDataImage* PinDataImage = Cast<UCONodeSkeletalMeshSectionPinDataImage>(GetPinData(*Pin));
				PinDataImage->NodeMaterial = this;
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ExtendMaterialOnlyMutableModeParametersBug)
	{
		for (const UEdGraphPin* Pin : GetAllPins())
		{
			if (UCONodeSkeletalMeshSectionPinDataImage* const PinDataImage = Cast<UCONodeSkeletalMeshSectionPinDataImage>(GetPinData(*Pin)))
			{
				PinDataImage->NodeMaterial = this;
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeMaterialAddTablePin)
	{
		if (Material)
		{
			ConditionalPostLoadReference(*Material); // Make sure the Material has been fully loaded.
		}
		
		Super::ReconstructNode();
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddedTableMaterialSwitch)
	{
		if (Material)
		{
			ConditionalPostLoadReference(*Material); // Make sure the Material has been fully loaded.
		}
		
		UMaterialInstance* DefaultPinValue = nullptr;

		if (const UEdGraphPin* MaterialAssetPin = GetMaterialAssetPin())
		{
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*MaterialAssetPin))
			{
				if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(ConnectedPin->GetOwningNode()))
				{
					DefaultPinValue = TableNode->GetColumnDefaultAssetByType<UMaterialInstance>(ConnectedPin);
				}
			}
		}

		if (DefaultPinValue && DefaultPinValue->TextureParameterValues.Num())
		{
			const uint32 NumTextureParameters = GetNumParameters(EMaterialParameterType::Texture);
			for (uint32 ImageIndex = 0; ImageIndex < NumTextureParameters; ++ImageIndex)
			{
				if (const UEdGraphPin* ImagePin = GetParameterPin(EMaterialParameterType::Texture, ImageIndex))
				{
					FGuid ParameterId = GetParameterId(EMaterialParameterType::Texture, ImageIndex).ParameterId;

					TArray<FMaterialParameterInfo> TextureParameterInfo;
					TArray<FGuid> TextureGuids;

					// Getting parent's texture infos
					DefaultPinValue->GetMaterial()->GetAllTextureParameterInfo(TextureParameterInfo, TextureGuids);

					int32 TextureIndex = TextureGuids.Find(ParameterId);

					if (TextureIndex == INDEX_NONE)
					{
						continue;
					}

					FName TextureName = TextureParameterInfo[TextureIndex].Name;

					// Checking if the pin's texture has been modified in the material instance
					for (const FTextureParameterValue& Texture : DefaultPinValue->TextureParameterValues)
					{
						if (TextureName == Texture.ParameterInfo.Name)
						{
							if (UCONodeSkeletalMeshSectionPinDataImage* PinDataImage = Cast<UCONodeSkeletalMeshSectionPinDataImage>(GetPinData(*ImagePin)))
							{
								PinDataImage->PinMode = EPinMode::Mutable;
							}
						}
					}
				}
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NewComponentOptions)
	{
		MeshComponentName_DEPRECATED = FName(FString::FromInt(MeshComponentIndex_DEPRECATED));
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeMaterialTypedImagePins)
	{
		const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
		
		for (UEdGraphPin* Pin : GetAllPins())
		{
			if (Pin->PinType.PinCategory != UEdGraphSchema_CustomizableObject::PC_Texture)
			{
				continue;
			}
			
			if (Pin->LinkedTo.Num())
			{
				const UEdGraphPin* LinkedPin = Pin->LinkedTo[0];

				UCONodeSkeletalMeshSectionPinDataImage& PinData = GetPinData<UCONodeSkeletalMeshSectionPinDataImage>(*Pin);

				if (LinkedPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture)
				{
					Pin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_Texture;

					if (PinData.PinMode == EPinMode::Default &&
						TextureParametersMode == ENodePinMode::Passthrough)
					{
						PinData.PinMode = EPinMode::Mutable;
					}
					else if (PinData.PinMode == EPinMode::Passthrough)
					{
						PinData.PinMode = EPinMode::Mutable;
					}
				}
				else if (LinkedPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
				{
					Pin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough;

					if (PinData.PinMode == EPinMode::Default &&
						TextureParametersMode == ENodePinMode::Mutable)
					{
						PinData.PinMode = EPinMode::Passthrough;
					}
					else if (PinData.PinMode == EPinMode::Mutable)
					{
						PinData.PinMode = EPinMode::Passthrough;
					}
				}
			}
			else
			{
				if (!IsImageMutableMode(*Pin))
				{
					Pin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough;
				}				
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixedMultilayerMaterialIds)
	{
		if (Material && Material->GetCachedExpressionData().bHasMaterialLayers)
		{
			ConditionalPostLoadReference(*Material); // Make sure the Material has been fully loaded.
			
			// Needed since we can not get the layer index of repeated parameters
			Super::ReconstructNode();
		}
		else
		{
			for (TMap<FGuid, FEdGraphPinReference>::TIterator It = PinsParameter_DEPRECATED.CreateIterator(); It; ++It)
			{
				PinsParameterMap.Add({ It.Key(), -1 }, It.Value());

				// Move pin data id info to the new struct
				UEdGraphPin* GraphPin = It.Value().Get();
				if (!GraphPin)
				{
					continue;
				}

				UCustomizableObjectNodePinData* GenericPinData = GetPinData(*GraphPin);
				if (!GenericPinData)
				{
					continue;
				}

				if (UCONodeSkeletalMeshSectionPinDataParameter* PinData = Cast<UCONodeSkeletalMeshSectionPinDataParameter>(GenericPinData))
				{
					PinData->MaterialParameterId.LayerIndex = INDEX_NONE;
					PinData->MaterialParameterId.ParameterId = PinData->ParameterId_DEPRECATED;
				}
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MaterialPinsRename)
	{
		UEdGraphPin* MaterialPin = FindPin(TEXT("Material_Output_Pin"), EEdGraphPinDirection::EGPD_Output);
		if (MaterialPin)
		{
			const FString PinFriendlyName = TEXT("Mesh Section");
			const FString PinName = PinFriendlyName + FString(TEXT("_Output_Pin"));
			MaterialPin->PinName = FName(PinName);
			MaterialPin->PinFriendlyName = FText::FromString(PinFriendlyName);
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName)
	{
		const FString ExpectedPinFriendlyName = "Table Material";
		const FString ExpectedPinName = ExpectedPinFriendlyName + FString(TEXT("_Input_Pin"));	
		
		if (UEdGraphPin* FoundPin = FindPin(ExpectedPinName,EGPD_Input))
		{
			const FString TargetPinFriendlyName = "Material";
			const FString TargetPinName = TargetPinFriendlyName + FString(TEXT("_Input_Pin"));
			
			FoundPin->PinFriendlyName = FText::FromString(TargetPinFriendlyName);
			FoundPin->PinName = FName(TargetPinName);
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!EnableTagsPinRef.Get())
		{
			const FString PinFriendlyName = "Enable Tags";
			const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));
			UEdGraphPin* TagsPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, FName(PinName),  FText::FromString(PinFriendlyName));
			TagsPin->PinType.ContainerType = EPinContainerType::Array;
			TagsPin->PinToolTip = "List of Tags that this node will Enable";
			EnableTagsPinRef = TagsPin;
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixMaterialNodeMaterialPinIncorrectLocalization)
	{
		for (UEdGraphPin* Pin : GetAllPins())
		{
			if (Pin)
			{
				if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Material && 
					Pin->Direction == EGPD_Input && Pin->PinName.ToString().EndsWith(TEXT("_Input_Pin")) &&
					Pin->PinName != FString(TEXT("Material_Input_Pin")))
				{
					FString PinName = FString(TEXT("Material_Input_Pin"));
					Pin->PinName = FName(*PinName);
				}
			}
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MeshSectionNodeInputPinsAsReferences)
	{
		if (!InputMeshPin.Get())
		{
			// const FString PinFriendlyName = TEXT("Mesh");
			// const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));
			// InputMeshPin = CustomCreatePin(EGPD_Input, "mesh", FName(*PinName), FText::FromString(PinFriendlyName));
			
			const FString PinFriendlyName = TEXT("Mesh");
			const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));

			if (UEdGraphPin* Pin = FindPin(PinName))
			{
				InputMeshPin = Pin;
			}
			else
			{
				InputMeshPin =  FindPin(PinFriendlyName);
			}
			
			check(InputMeshPin.Get())
		}
	
		if (!InputMaterialPin.Get())
		{
			// const FString PinFriendlyName = FString{TEXT("materialAsset")};
			// const FString PinName = FString(TEXT("Material_Input_Pin"));
			// InputMaterialPin = CustomCreatePin(EGPD_Input, "materialAsset", FName(*PinName), FText::FromString(PinFriendlyName));
			// InputMaterialPin.Get()->PinToolTip = "Pin for a Material from a Table Node";
			
			const FString PinFriendlyName = UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(UEdGraphSchema_CustomizableObject::PC_Material).ToString();
			const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));

			UEdGraphPin* Pin = FindPin(PinName);
			if (Pin)
			{
				InputMaterialPin = Pin;
			}
			else
			{
				// Changing the locale can change the GetPinCategoryFriendlyName() result, leading to failing to find the MaterialAsset pin.
				// This is a provisional fix while incorrect FriendlyName usages are removed in UE-294583
				Pin = FindPin(PinFriendlyName);

				if (Pin)
				{
					InputMaterialPin = Pin;
				}
				else
				{
					InputMaterialPin = FindPin(FString("Material_Input_Pin"));
				}
			}
			
			check(InputMaterialPin.Get())
		}
	}
}


void UCONodeSkeletalMeshSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);
}


void UCONodeSkeletalMeshSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeSkeletalMeshSection, Material))
		{
			Super::ReconstructNode(); // Super required to avoid ambiguous call compilation error.
		}
		else if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeSkeletalMeshSection, TextureParametersMode))
		{
			Super::ReconstructNode();
			PostImagePinModeChangedDelegate.Broadcast();
		}
	}
}


FText UCONodeSkeletalMeshSection::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::ListView || !Material)
	{
		return LOCTEXT("Mesh Section", "Mesh Section");
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MeshSectionName"), FText::FromString(Material->GetName()));

		return FText::Format(LOCTEXT("MeshSection_Title", "{MeshSectionName}\nMesh Section"), Args);
	}
}


FLinearColor UCONodeSkeletalMeshSection::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_MeshSection);
}


UEdGraphPin* UCONodeSkeletalMeshSection::OutputPin() const
{
	FString PinFriendlyName = TEXT("Mesh Section");
	FString PinName = PinFriendlyName + FString(TEXT("_Output_Pin"));

	UEdGraphPin* Pin = FindPin(PinName, EEdGraphPinDirection::EGPD_Output);
	if (!Pin)
	{
		Pin = FindPin(TEXT("Mesh Section"), EEdGraphPinDirection::EGPD_Output);
	}

	// Legacy name
	if (!Pin)
	{
		PinFriendlyName = TEXT("Material");
		PinName = PinFriendlyName + FString(TEXT("_Output_Pin"));

		Pin = FindPin(PinName, EEdGraphPinDirection::EGPD_Output);
		if (!Pin)
		{
			Pin = FindPin(TEXT("Material"), EEdGraphPinDirection::EGPD_Output);
		}		
	}

	return Pin;
}


UEdGraphPin* UCONodeSkeletalMeshSection::GetMeshPin() const
{
	return InputMeshPin.Get();
}


UEdGraphPin* UCONodeSkeletalMeshSection::GetMaterialAssetPin() const
{
	return InputMaterialPin.Get();
}


UEdGraphPin* UCONodeSkeletalMeshSection::GetEnableTagsPin() const
{
	return EnableTagsPinRef.Get();
}


bool UCONodeSkeletalMeshSection::IsImageMutableMode(const int32 ImageIndex) const
{
	if (const UEdGraphPin* Pin = GetParameterPin(EMaterialParameterType::Texture, ImageIndex))
	{
		return IsImageMutableMode(*Pin);
	}
	else
	{
		return NodePinModeToImagePinMode(TextureParametersMode) == UEdGraphSchema_CustomizableObject::PC_Texture;
	}
}


bool UCONodeSkeletalMeshSection::IsImageMutableMode(const UEdGraphPin& Pin) const
{
	return GetImagePinMode(Pin) == UEdGraphSchema_CustomizableObject::PC_Texture; // The ImageMutableMode is stored in the PinData, not in the PinCategory
}


void UCONodeSkeletalMeshSection::SetImagePinMode(UEdGraphPin& Pin, const EPinMode PinMode) const
{
	UCONodeSkeletalMeshSectionPinDataImage& PinData = GetPinData<UCONodeSkeletalMeshSectionPinDataImage>(Pin);

	Pin.PinType.PinCategory = GetImagePinMode(PinMode);	// Change the category so that it will remap correctly when reconstructing.
	
	PinData.SetPinMode(PinMode); // Will trigger a reconstruct.
}


UTexture2D* UCONodeSkeletalMeshSection::GetImageReferenceTexture(const int32 ImageIndex) const
{
	if (const UEdGraphPin* Pin = GetParameterPin(EMaterialParameterType::Texture, ImageIndex))
	{
		return GetPinData<UCONodeSkeletalMeshSectionPinDataImage>(*Pin).ReferenceTexture;
	}
	else
	{
		return nullptr;
	}
}


UTexture2D* UCONodeSkeletalMeshSection::GetImageValue(const int32 ImageIndex) const
{
	FName TextureName = GetParameterName(EMaterialParameterType::Texture, ImageIndex);
	int32 LayerIndex = GetParameterLayerIndex(EMaterialParameterType::Texture, ImageIndex);
	
	FHashedMaterialParameterInfo TextureInfo;
	TextureInfo.Name = FScriptName(TextureName);
	TextureInfo.Index = LayerIndex;
	TextureInfo.Association = LayerIndex == INDEX_NONE ? GlobalParameter : LayerParameter; // TODO: Blend Parameters

	UTexture* Texture = nullptr;
	Material->GetTextureParameterValue(TextureInfo, Texture);

	return Cast<UTexture2D>(Texture);
}


TArray<UCustomizableObjectLayout*> UCONodeSkeletalMeshSection::GetLayouts() const
{
	TArray<UCustomizableObjectLayout*> Result;

	if (UEdGraphPin* MeshPin = GetMeshPin())
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*MeshPin))
		{
			if (const UEdGraphPin* SourceMeshPin = FindMeshBaseSource(*ConnectedPin, false))
			{
				if (const UCustomizableObjectNodeSkeletalMesh* MeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(SourceMeshPin->GetOwningNode()))
				{
					Result = MeshNode->GetLayouts(*SourceMeshPin);
				}
				else if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(SourceMeshPin->GetOwningNode()))
				{
					Result = TableNode->GetLayouts(SourceMeshPin);
				}
			}
		}
	}

	return Result;
}


int32 UCONodeSkeletalMeshSection::GetImageUVLayout(const int32 ImageIndex) const
{
	if (const UEdGraphPin* Pin = GetParameterPin(EMaterialParameterType::Texture, ImageIndex))
	{
		const UCONodeSkeletalMeshSectionPinDataImage& PinData = GetPinData<UCONodeSkeletalMeshSectionPinDataImage>(*Pin);
		switch(PinData.UVLayoutMode)
		{
		case EUVLayoutMode::FromMaterial:
			break;
		case EUVLayoutMode::Ignore:
			return UCONodeSkeletalMeshSectionPinDataImage::UV_LAYOUT_IGNORE;
		case EUVLayoutMode::Index:
			return PinData.UVLayout;
		default:
			unimplemented();
		}
	}

	const int32 UVIndex = GetImageUVLayoutFromMaterial(ImageIndex);
	if (UVIndex == -1)
	{
		const FText ParamName = FText::FromName(GetParameterName(EMaterialParameterType::Texture, ImageIndex));
		const FText Msg = FText::Format(LOCTEXT("UVLayoutMaterialError", "Could not deduce the UV Layout Index in [{0}]. Use [Index] UV Layout Mode or, in the UMaterial, remove any nodes connected to the [UVs] pin in [{0}] node ."), ParamName);
		FCustomizableObjectEditorLogger::CreateLog(Msg)
			.Severity(EMessageSeverity::Warning)
			.Context(*this)
			.CustomizableObject(GraphTraversal::GetObject(*this))
			.Log();
		
		return 0;
	}

	return UVIndex;
}


int32 UCONodeSkeletalMeshSection::GetNumParameters(const EMaterialParameterType Type) const
{
	if (Material)
	{
		return Material->GetCachedExpressionData().GetParameterTypeEntry(Type).ParameterInfoSet.Num();
	}
	else
	{
		return 0;
	}
}


FNodeMaterialParameterId UCONodeSkeletalMeshSection::GetParameterId(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	const FMaterialCachedExpressionData& Data = Material->GetCachedExpressionData();

	if (Data.EditorOnlyData)
	{
		if (Data.EditorOnlyData->EditorEntries[(int32)Type].EditorInfo.Num() != 0)
		{
			const FGuid ParameterId = Data.EditorOnlyData->EditorEntries[(int32)Type].EditorInfo[ParameterIndex].ExpressionGuid;
			const int32 LayerIndex = GetParameterLayerIndex(Type, ParameterIndex);

			return { ParameterId, LayerIndex };
		}
	}
	
	return FNodeMaterialParameterId();
}


FName UCONodeSkeletalMeshSection::GetParameterName(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	check(Material);
	
	const FMaterialCachedParameterEntry& Entry = Material->GetCachedExpressionData().GetParameterTypeEntry(Type);

	for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
	{
		const int32 IteratorIndex = It.GetId().AsInteger();

		if (IteratorIndex == ParameterIndex)
		{
			return (*It).Name;
		}
	}
	
	// The parameter should exist
	check(false);

	return FName();
}


int32 UCONodeSkeletalMeshSection::GetParameterLayerIndex(const UMaterialInterface* InMaterial, const EMaterialParameterType Type, const int32 ParameterIndex)
{
	check(InMaterial);

	const FMaterialCachedParameterEntry& Entry = InMaterial->GetCachedExpressionData().GetParameterTypeEntry(Type);

	for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
	{
		const int32 IteratorIndex = It.GetId().AsInteger();

		if (IteratorIndex == ParameterIndex)
		{
			return (*It).Index;
		}
	}

	// The parameter should exist
	check(false);

	return -1;
}


int32 UCONodeSkeletalMeshSection::GetParameterLayerIndex(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	return GetParameterLayerIndex(Material.Get(), Type, ParameterIndex);
}


FText UCONodeSkeletalMeshSection::GetParameterLayerName(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	check(Material)

	int32 LayerIndex = GetParameterLayerIndex(Type,ParameterIndex);

	FMaterialLayersFunctions LayersValue;
	Material->GetMaterialLayers(LayersValue);


	return LayersValue.EditorOnly.LayerNames.IsValidIndex(LayerIndex) ? LayersValue.EditorOnly.LayerNames[LayerIndex] : FText();
}


bool UCONodeSkeletalMeshSection::HasParameter(const UMaterialInterface* InMaterial, const FNodeMaterialParameterId& ParameterId)
{
	if (!InMaterial)
	{
		return false;
	}

	for (const EMaterialParameterType Type : ParameterTypes)
	{
		const FMaterialCachedExpressionData& Data = InMaterial->GetCachedExpressionData();
		const FMaterialCachedParameterEntry& Entry = Data.GetParameterTypeEntry(Type);

		if (!Data.EditorOnlyData || Data.EditorOnlyData->EditorEntries[(int32)Type].EditorInfo.IsEmpty())
		{
			continue;
		}

		for (TSet<FMaterialParameterInfo>::TConstIterator It(Entry.ParameterInfoSet); It; ++It)
		{
			const int32 IteratorIndex = It.GetId().AsInteger();
			
			const FGuid& ParamGuid = Data.EditorOnlyData->EditorEntries[(int32)Type].EditorInfo[IteratorIndex].ExpressionGuid;
			const int32 LayerIndex = GetParameterLayerIndex(InMaterial, Type, IteratorIndex);
			const FNodeMaterialParameterId ParamId = { ParamGuid, LayerIndex };

			if (ParamId == ParameterId)
			{
				return true;
			}
		}
	}

	return false;
}


bool UCONodeSkeletalMeshSection::HasParameter(const FNodeMaterialParameterId& ParameterId) const
{
	return HasParameter( Material.Get(), ParameterId );
}


UEdGraphPin* UCONodeSkeletalMeshSection::GetParameterPin(const EMaterialParameterType Type, const int32 ParameterIndex) const
{
	const FNodeMaterialParameterId ParameterId = GetParameterId(Type, ParameterIndex);
	
	return GetParameterPin(ParameterId);
}


UEdGraphPin* UCONodeSkeletalMeshSection::GetParameterPin(const FNodeMaterialParameterId& ParameterId) const
{
	if (const FEdGraphPinReference* Result = PinsParameterMap.Find(ParameterId))
	{
		return Result->Get();
	}
	else
	{
		return nullptr;
	}
}


bool UCONodeSkeletalMeshSection::IsNodeOutDatedAndNeedsRefresh()
{
	const bool bOutdated = RealMaterialDataHasChanged();

	// Remove previous compilation warnings
	if (!bOutdated && bHasCompilerMessage)
	{
		RemoveWarnings();
		GetGraph()->NotifyGraphChanged();
	}

	return bOutdated;
}


FString UCONodeSkeletalMeshSection::GetRefreshMessage() const
{
	return "Referenced material has changed, texture channels might have been added, removed or renamed. Please refresh the node material to reflect those changes.";
}


TSharedPtr<IDetailsView> UCONodeSkeletalMeshSection::CustomizePinDetails(const UEdGraphPin& Pin) const
{
	if (UCONodeSkeletalMeshSectionPinDataImage* PinData = Cast<UCONodeSkeletalMeshSectionPinDataImage>(GetPinData(Pin)))
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		
		const TSharedRef<IDetailsView> SettingsView = EditModule.CreateDetailView(DetailsViewArgs);
		SettingsView->SetObject(PinData);
		
		return SettingsView;
	}
	else
	{
		return nullptr;
	}
}


bool UCONodeSkeletalMeshSection::CustomRemovePin(UEdGraphPin& Pin)
{
	for (TMap<FNodeMaterialParameterId, FEdGraphPinReference>::TIterator Iterator = PinsParameterMap.CreateIterator(); Iterator; ++Iterator)
	{
		if (Iterator->Value.Get() == &Pin)
		{
			Iterator.RemoveCurrent();
			break;
		}
	}

	return Super::CustomRemovePin(Pin);
}


void UCONodeSkeletalMeshSection::SetMaterial(UMaterialInterface* InMaterial)
{
	Material = InMaterial;
}


UMaterialInterface* UCONodeSkeletalMeshSection::GetMaterial() const
{
	return Material;	
}


TArray<FString> UCONodeSkeletalMeshSection::GetEnableTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext)
{
	// Getting tags from linked pin
	const UEdGraphPin* EnableTagsPin = GetEnableTagsPin();

	if (!EnableTagsPin)
	{
		return Tags;
	}

	const TArray<UEdGraphPin*> ConnectedPins = FollowInputPinArray(*EnableTagsPin);

	if (ConnectedPins.Num())
	{
		TArray<FString> OutTags;

		for (const UEdGraphPin* StringPin : ConnectedPins)
		{
			const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*StringPin, MacroContext);

			if (SourceStringPin)
			{
				if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
				{
					OutTags.AddUnique(StringNode->Value);
				}
			}
		}

		return OutTags;
	}

	return Tags;
}


TArray<FString>* UCONodeSkeletalMeshSection::GetEnableTagsArray()
{
	return &Tags;
}


bool UCONodeSkeletalMeshSection::RealMaterialDataHasChanged() const
{
	for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
	{
		if (const UCONodeSkeletalMeshSectionPinDataParameter* PinData = Cast<UCONodeSkeletalMeshSectionPinDataParameter>(GetPinData(*Pin)))
		{
			if (!HasParameter(PinData->MaterialParameterId) &&
				(FollowInputPin(*Pin) || !PinData->IsDefault()))
			{
				return true;
			}
		}
	}

	return false;
}


FPostImagePinModeChangedDelegate* UCONodeSkeletalMeshSection::GetPostImagePinModeChangedDelegate()
{
	return &PostImagePinModeChangedDelegate;
}


bool UCONodeSkeletalMeshSection::IsPinRelevant(const UEdGraphPin* Pin) const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
	{	
		return Pin->PinType.PinCategory == Schema->PC_Mesh;
	}
	else if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
	{	
		return Pin->PinType.PinCategory == Schema->PC_MeshSection;
	}
	else
	{
		return false;
	}
}


FText UCONodeSkeletalMeshSection::GetTooltipText() const
{
	return LOCTEXT("MeshSection_Tooltip", "Defines a Customizable Object mesh section.\nIt has a mesh section, a material assigned to it and the runtime modifiable inputs to the material asset parameters.");
}


TSharedPtr<SGraphNode> UCONodeSkeletalMeshSection::CreateVisualWidget()
{
	return SNew(SCONodeSkeletalMeshSection, this);
}


UCONodeSkeletalMeshSectionPinDataParameter* UCONodeSkeletalMeshSection::CreatePinData(const EMaterialParameterType Type, const int32 ParameterIndex)
{
	UCONodeSkeletalMeshSectionPinDataParameter* PinData = nullptr;

	switch (Type)
	{
	case EMaterialParameterType::Texture:
		{
			UCONodeSkeletalMeshSectionPinDataImage* PinDataImage = NewObject<UCONodeSkeletalMeshSectionPinDataImage>(this);
			PinDataImage->Init(*this);
		
			PinData = PinDataImage;
			break;
		}
	case EMaterialParameterType::Vector:
		{
			PinData = NewObject<UCONodeSkeletalMeshSectionPinDataVector>(this);	
			break;
		}

	case EMaterialParameterType::Scalar:
		{
			PinData = NewObject<UCONodeSkeletalMeshSectionPinDataScalar>(this);	
			break;
		}

	default:
		check(false); // Parameter type not contemplated.
	}

	PinData->MaterialParameterId = GetParameterId(Type, ParameterIndex);

	return PinData;
}

void UCONodeSkeletalMeshSection::AllocateDefaultParameterPins(const EMaterialParameterType Type)
{
	const int32 NumParameters = GetNumParameters(Type);
	for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
	{
		UCONodeSkeletalMeshSectionPinDataParameter* PinData = CreatePinData(Type, ParameterIndex);

		const FName PinName = GetPinName(Type, ParameterIndex);

		FName PinCategory;
		switch(Type)
		{
		case EMaterialParameterType::Texture:
			if (IsImageMutableMode(ParameterIndex)) // If a pin exists, we store the PinMode in its PinData.
			{
				PinCategory = UEdGraphSchema_CustomizableObject::PC_Texture;				
			}
			else
			{
				PinCategory = UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough;
			}
			break;

		case EMaterialParameterType::Vector:
			PinCategory = UEdGraphSchema_CustomizableObject::PC_Color;
			break;

		case EMaterialParameterType::Scalar:
			PinCategory = UEdGraphSchema_CustomizableObject::PC_Float;
			break;

		default:
			check(false); // Type not contemplated
		}		
		
		UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, PinCategory, PinName, FText::FromName(PinName), PinData);
		Pin->bHidden = true;

		PinsParameterMap.Add(PinData->MaterialParameterId, FEdGraphPinReference(Pin));
	}
}


void UCONodeSkeletalMeshSection::SetDefaultMaterial()
{
	if (const UEdGraphPin* MeshPin = GetMeshPin();
		!Material && MeshPin)
	{
		if (const UEdGraphPin* LinkedMeshPin = FollowInputPin(*MeshPin))
		{
			UEdGraphNode* LinkedMeshNode = LinkedMeshPin->GetOwningNode();
	
			if (const UCustomizableObjectNodeSkeletalMesh* NodeSkeletalMesh = Cast<UCustomizableObjectNodeSkeletalMesh>(LinkedMeshNode))
			{
				Material = NodeSkeletalMesh->GetMaterialFor(LinkedMeshPin);
				if (Material)
				{
					Super::ReconstructNode(); // Super required to avoid ambiguous call compilation error.
				}
			}
			else if (const UCustomizableObjectNodeStaticMesh* NodeStaticMesh = Cast<UCustomizableObjectNodeStaticMesh>(LinkedMeshNode))
			{
				Material = NodeStaticMesh->GetMaterialFor(LinkedMeshPin);
				if (Material)
				{
					Super::ReconstructNode();
				}
			}
		}
	}
}


void UCONodeSkeletalMeshSection::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	
	if (Pin == GetMeshPin())
	{
			SetDefaultMaterial();
	}
	else if (Cast<UCONodeSkeletalMeshSectionPinDataImage>(GetPinData(*Pin))) // Image pin
	{
		// If necessary, automatically change the Pin Mode. Connected pin can never change its type.
		if (Pin->LinkedTo.Num())
		{
			if (UEdGraphPin* LinkedPin = Pin->LinkedTo[0])
			{
				EPinMode PinMode = EPinMode::Default;
		
				if (LinkedPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture)
				{
					PinMode = EPinMode::Mutable;
				}
				else if (LinkedPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough || LinkedPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Object)
				{
					PinMode = EPinMode::Passthrough;
				}
				else
				{
					unimplemented()
				}
		
				SetImagePinMode(*Pin, PinMode);
			}
		}
	}
}


void UCONodeSkeletalMeshSection::PostPasteNode()
{
	Super::PostPasteNode();
	SetDefaultMaterial();
}


bool UCONodeSkeletalMeshSection::CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (InOwnedInputPin && InOutputPin)
	{
		if ((InOwnedInputPin->PinType.PinCategory == Schema->PC_Texture && (InOutputPin->PinType.PinCategory == Schema->PC_Texture_Passthrough || InOutputPin->PinType.PinCategory == Schema->PC_Texture_Object)) ||
			(InOwnedInputPin->PinType.PinCategory == Schema->PC_Texture_Passthrough && InOutputPin->PinType.PinCategory == Schema->PC_Texture))
		{
			return true;
		}
	}

	return Super::CanConnect(InOwnedInputPin, InOutputPin, bOutIsOtherNodeBlocklisted, bOutArePinsCompatible);
}




#undef LOCTEXT_NAMESPACE
