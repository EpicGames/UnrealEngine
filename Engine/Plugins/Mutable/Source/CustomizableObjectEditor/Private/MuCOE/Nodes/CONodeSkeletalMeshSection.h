// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "EdGraph/EdGraphPin.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"
#include "Containers/StaticArray.h"

#include "CONodeSkeletalMeshSection.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

enum class EMaterialParameterType : uint8;
namespace ENodeTitleType { enum Type : int; }

class FArchive;
class SGraphNode;
class SWidget;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UMaterial;
class UMaterialInterface;
class UObject;
class UTexture2D;
struct FCustomizableObjectNodeMaterialImage;
struct FCustomizableObjectNodeMaterialScalar;
struct FCustomizableObjectNodeMaterialVector;
struct FEdGraphPinReference;
struct FFrame;
struct FPropertyChangedEvent;


DECLARE_MULTICAST_DELEGATE(FPostImagePinModeChangedDelegate)


/** This struct helps us to identify a material parameter using its id and layer index in case of multimaterials
* When a multilayer material has the same material in multiple layer the parameter Id is not enough to identify a parameter
* we also need the its layer index.
*/
USTRUCT()
struct FNodeMaterialParameterId
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ParameterId;

	UPROPERTY()
	int32 LayerIndex = INDEX_NONE;

	bool operator==(const FNodeMaterialParameterId& Other) const = default;
};


inline uint32 GetTypeHash(const FNodeMaterialParameterId& Key)
{
	uint32 Hash = GetTypeHash(Key.ParameterId);
	Hash = HashCombine(Hash, GetTypeHash(Key.LayerIndex));

	return Hash;
}


/** Custom remap pins by name action.
 *
 * Remap pins by Texture Parameter Id. */
UCLASS()
class UCustomizableObjectNodeMaterialRemapPinsByName : public UCustomizableObjectNodeRemapPinsByName
{
	GENERATED_BODY()
public:
	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const override;

	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;

	bool HasSavedPinData(const UCustomizableObjectNode& Node, const UEdGraphPin &Pin) const;
};


/** Base class for all Material Parameters. */
UCLASS(MinimalAPI)
class UCONodeSkeletalMeshSectionPinDataParameter : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	/** Parameter id + layer index */
	UPROPERTY()
	FNodeMaterialParameterId MaterialParameterId;

	/** Texture Parameter Id. */
	UPROPERTY()
	FGuid ParameterId_DEPRECATED;

	/** Returns true if all properties are in its default state. */
	UE_API virtual bool IsDefault() const;
};


/** Node pin mode. All pins set to EPinMode::Default will use this this mode. */
UENUM()
enum class ENodePinMode
{
	Mutable UMETA(ToolTip = "All Material Texture FParameters go through Mutable."),
	Passthrough UMETA(DisplayName = "Object", ToolTip = "All Material Texture FParameters are not modified by Mutable.")
};


/** Image pin, pin mode. */
UENUM()
enum class EPinMode
{
	Default UMETA(DisplayName = "Node Defined", ToolTip = "Use node's \"Default Texture Parameter Mode\"."),
	Mutable UMETA(ToolTip = "The Material Texture FParameters goes through Mutable."),
	Passthrough UMETA(DisplayName = "Object", ToolTip = "The Material Texture FParameters is not modified by Mutable.")
};


/** Image Pin, UV Layout Mode. */
UENUM()
enum class EUVLayoutMode
{
	/** Does not override the UV Index specified in the Material. */
	FromMaterial,
	/* Texture should not be transformed by any layout. These textures will not be reduced automatically for LODs. */
	Ignore,
	/** User specified UV Index. */
	Index
};


/** Enum to FText. */
FText EPinModeToText(EPinMode PinMode);


UCLASS(MinimalAPI)
class UCONodeSkeletalMeshSection : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// UObject interface.
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual bool CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual bool CanPinBeHidden(const UEdGraphPin& Pin) const override;
	UE_API virtual bool HasPinViewer() const override;
	UE_API virtual UCustomizableObjectNodeRemapPinsByName* CreateRemapPinsDefault() const override;
	virtual bool ProvidesCustomPinRelevancyTest() const override { return true; }
	UE_API virtual bool IsPinRelevant(const UEdGraphPin* Pin) const override;
	UE_API virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	UE_API virtual bool IsNodeOutDatedAndNeedsRefresh() override;
	UE_API virtual FString GetRefreshMessage() const override;
	UE_API virtual TSharedPtr<IDetailsView> CustomizePinDetails(const UEdGraphPin& Pin) const override;
	UE_API virtual TArray<FString> GetEnableTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) override;
	UE_API virtual TArray<FString>* GetEnableTagsArray() override;
	UE_API virtual FString GetInternalTagDisplayName() override;
	UE_API virtual TArray<UCustomizableObjectLayout*> GetLayouts() const;
	UE_API virtual UEdGraphPin* OutputPin() const;
	UE_API virtual UMaterialInterface* GetMaterial() const;
	UE_API virtual int32 GetNumParameters(EMaterialParameterType Type) const;
	UE_API virtual FNodeMaterialParameterId GetParameterId(EMaterialParameterType Type, int32 ParameterIndex) const;
	UE_API virtual FName GetParameterName(EMaterialParameterType Type, int32 ParameterIndex) const;
	UE_API virtual int32 GetParameterLayerIndex(EMaterialParameterType Type, int32 ParameterIndex) const;
	UE_API virtual FText GetParameterLayerName(EMaterialParameterType Type, int32 ParameterIndex) const;
	UE_API virtual bool HasParameter(const FNodeMaterialParameterId& ParameterId) const;
	UE_API virtual UEdGraphPin* GetParameterPin(EMaterialParameterType Type, int32 ParameterIndex) const;
	UE_API virtual UEdGraphPin* GetParameterPin(const FNodeMaterialParameterId& ParameterId) const;
	UE_API virtual bool IsImageMutableMode(int32 ImageIndex) const;
	UE_API virtual bool IsImageMutableMode(const UEdGraphPin& Pin) const;
	UE_API virtual UTexture2D* GetImageReferenceTexture(int32 ImageIndex) const;
	UE_API virtual UTexture2D* GetImageValue(int32 ImageIndex) const;
	UE_API virtual int32 GetImageUVLayout(int32 ImageIndex) const;
	UE_API virtual UEdGraphPin* GetMeshPin() const;
	UE_API virtual UEdGraphPin* GetMaterialAssetPin() const;
	UE_API virtual UEdGraphPin* GetEnableTagsPin() const;
	UE_API virtual bool RealMaterialDataHasChanged() const;
	UE_API virtual FPostImagePinModeChangedDelegate* GetPostImagePinModeChangedDelegate();
	
	// Own Interface

	UE_API void SetMaterial(UMaterialInterface* InMaterial);

	static UE_API bool HasParameter(const UMaterialInterface* InMaterial, const FNodeMaterialParameterId& ParameterId);
	static UE_API int32 GetParameterLayerIndex(const UMaterialInterface* InMaterial, EMaterialParameterType Type, int32 ParameterIndex);
	
	/* Max LOD to propagate this section to, when using automatic LODs. */
	UPROPERTY(EditAnywhere, Category = AdvancedSettings)
	int32 MaxLOD = INDEX_NONE;
	
	UPROPERTY(EditAnywhere, Category = Tags)
	TArray<FString> Tags;

private:
	/** Set the Pin Mode of a Texture Parameter Pin. */
	UE_API void SetImagePinMode(UEdGraphPin& Pin, EPinMode PinMode) const;
	
	/** Delegate called when a Texture Parameter Pin Mode changes. */
	FPostImagePinModeChangedDelegate PostImagePinModeChangedDelegate;

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY(EditAnywhere, Category=CustomizableObject, DisplayName = "Default Texture Parameter Mode", Meta = (ToolTip = "All Mateiral Texture FParameters set to \"Node Defined\" will use this mode."))
	ENodePinMode TextureParametersMode = ENodePinMode::Passthrough;

	UPROPERTY()
	int32 MeshComponentIndex_DEPRECATED = 0;

	UPROPERTY()
	FEdGraphPinReference InputMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference InputMaterialPin;

public:
	/** Selects which Mesh component of the Instance this material belongs to */
	UPROPERTY()
	FName MeshComponentName_DEPRECATED;

private:

	/** Last static or skeletal mesh connected. Used to remove the callback once disconnected. */
	TWeakObjectPtr<UCustomizableObjectNode> LastMeshNodeConnected;

	/** List of material parameter types that are actually relevant to mutable. */
	static UE_API const TArray<EMaterialParameterType> ParameterTypes;

	/** Relates a Parameter id (key) (and layer if is a layered material) to a Pin (value). Only used to improve performance.
	  * If a deprecated pin and a non-deprecated pin have the same Parameter id, this the non-deprecated one prevails. */
	UPROPERTY()
	TMap<FNodeMaterialParameterId, FEdGraphPinReference> PinsParameterMap;

	UPROPERTY()
	FEdGraphPinReference EnableTagsPinRef;
	
	/** Create the pin data of the given parameter type. */
	UE_API UCONodeSkeletalMeshSectionPinDataParameter* CreatePinData(EMaterialParameterType Type, int32 ParameterIndex);

	/** Allocate a pin for each parameter of the given type. */
	UE_API void AllocateDefaultParameterPins(EMaterialParameterType Type);

	/** Set the default Material from the connected static or skeletal mesh. */
	UE_API void SetDefaultMaterial();

	/** Format pin name. */
	UE_API FName GetPinName(EMaterialParameterType Type, int32 ParameterIndex) const;

	/** Returns the texture coordinate of the given Material Expression. Returns -1 if not found. */
	static UE_API int32 GetExpressionTextureCoordinate(UMaterial* Material, const FGuid &ImageId);

	/** Return the Pin Category given the node NodePinMode. */
	static UE_API FName NodePinModeToImagePinMode(ENodePinMode NodePinMode);

	/** Return the Pin Category given a PinMode. */
	UE_API FName GetImagePinMode(EPinMode PinMode) const;

	/** Return the Pin Category given a Pin. */
	UE_API FName GetImagePinMode(const UEdGraphPin& Pin) const;

	/** Get the UV Layout Index defined in the Material. */
	UE_API int32 GetImageUVLayoutFromMaterial(int32 ImageIndex) const;
	
public:
	
	/* Settings used to define the maximum number of blocks the Layouts will have.The number of blocks and the reference textures are used
	 to define the final texture size of runtime generated textures. */
	UPROPERTY(Category="Layout Settings", EditAnywhere, DisplayName="UV Packaging Settings")
	FLayoutSettings UVPackagingSettings[4];

private:
	
	// Deprecated properties
	/** Set all pins to Mutable mode. Even so, each pin can override its behavior. */
	UPROPERTY()
	bool bDefaultPinModeMutable_DEPRECATED = false;
	
	UPROPERTY()
	TArray<FCustomizableObjectNodeMaterialImage> Images_DEPRECATED;
	
	UPROPERTY()
	TArray<FCustomizableObjectNodeMaterialVector> VectorParams_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectNodeMaterialScalar> ScalarParams_DEPRECATED;

	UPROPERTY()
	TMap<FGuid, FEdGraphPinReference> PinsParameter_DEPRECATED;
};


/** Additional data for a Material Texture Parameter pin. */
UCLASS(MinimalAPI)
class UCONodeSkeletalMeshSectionPinDataImage : public UCONodeSkeletalMeshSectionPinDataParameter
{
	GENERATED_BODY()

	friend void UCONodeSkeletalMeshSection::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion);
	
public:
	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	// UCustomizableObjectNodePinData interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// NodePinDataParameter interface
	/** Virtual function used to copy pin data when remapping pins. */
	UE_API virtual void Copy(const UCustomizableObjectNodePinData& Other) override;
	
	// UCustomizableObjectNodeMaterialPinParameter interface
	UE_API virtual bool IsDefault() const override;

	// Own interface
	/** Constructor parameters. Should always be called after a NewObject. */
	UE_API void Init(UCONodeSkeletalMeshSection& InMeshSectionNode);

	UE_API EPinMode GetPinMode() const;

	UE_API void SetPinMode(EPinMode InPinMode);
	
private:
	/** Image pin mode. If is not default, overrides the defined node behaviour. */
	UPROPERTY(EditAnywhere, Category = NoCategory, DisplayName = "Texture Parameter Mode")
	EPinMode PinMode = EPinMode::Default;

public:
	constexpr static int32 UV_LAYOUT_IGNORE = -1;

	UPROPERTY(EditAnywhere, Category = NoCategory)
	EUVLayoutMode UVLayoutMode = EUVLayoutMode::FromMaterial;
	
	/** Index of the UV channel that will be used with this image. It is necessary to apply the proper layout transformations to it. */
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (EditCondition = "UVLayoutMode == EUVLayoutMode::Index", EditConditionHides))
	int32 UVLayout = -2;
	
	/** Reference Texture used to decide the texture properties of the mutable-generated textures
	* connected to this material. If null, it will try to be guessed at compile time from
	* the graph. */
	UPROPERTY(EditAnywhere, Category = NoCategory) // Required to be EditAnywhere for the selector to work.
	TObjectPtr<UTexture2D> ReferenceTexture = nullptr;

private:
	UPROPERTY()
	TObjectPtr<UCONodeSkeletalMeshSection> NodeMaterial = nullptr;
};


/** Additional data for a Material Vector Parameter pin. */
UCLASS(MinimalAPI)
class UCONodeSkeletalMeshSectionPinDataVector : public UCONodeSkeletalMeshSectionPinDataParameter
{
	GENERATED_BODY()
};


/** Additional data for a Material Float Parameter pin. */
UCLASS(MinimalAPI)
class UCONodeSkeletalMeshSectionPinDataScalar : public UCONodeSkeletalMeshSectionPinDataParameter
{
	GENERATED_BODY()
};

#undef UE_API
