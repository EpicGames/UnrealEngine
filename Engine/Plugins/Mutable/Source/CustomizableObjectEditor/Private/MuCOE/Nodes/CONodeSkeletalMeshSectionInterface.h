// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeSkeletalMeshSectionInterface.generated.h"

class UEdGraphPin;
class UMaterialInterface;

UINTERFACE(MinimalAPI)
class UCONodeSkeletalMeshSectionInterface : public UInterface
{
	GENERATED_BODY()
};


class ICONodeSkeletalMeshSectionInterface
{
	GENERATED_BODY()

public:
	// Own interface

	/** Returns the Unreal mesh (e.g., USkeletalMesh, UStaticMesh...). */
	virtual TSoftObjectPtr<UMaterialInterface> GetMaterial() const PURE_VIRTUAL(UCustomizableObjectNodeMaterialInterface::GetMaterial, return {};);

	/** Returns the output Mesh pin associated to the given LODIndex and SectionIndex. Override. */
	virtual UEdGraphPin* GetMaterialPin() const PURE_VIRTUAL(UCustomizableObjectNodeMaterialInterface::GetMaterialPin, return {};);
};
