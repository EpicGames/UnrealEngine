// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshSectionSwitch.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSkeletalMeshSectionSwitch)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCONodeSkeletalMeshSectionSwitch::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	PinType = UEdGraphSchema_CustomizableObject::PC_MeshSection;

	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixMaterialPinsRename)
	{
		const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

		// if there are pins that were manually fixed and re-saved after the FCustomizableObjectCustomVersion::MaterialPinsRename
		// conversion, convert them to the new fixed version.

		// In the new fix we only change the friendy name, but the actual name remains "material..."

		// Output
		UEdGraphPin* OldPin = FindPin(TEXT("Mesh Section"),EEdGraphPinDirection::EGPD_Output);
		if (OldPin)
		{
			UEdGraphPin* NewPin = FindPin(TEXT("Material"), EEdGraphPinDirection::EGPD_Output);
			if (!NewPin)
			{
				NewPin = CustomCreatePin(EGPD_Output, Schema->PC_MeshSection, TEXT("Material"), LOCTEXT("Material", "Material"));
			}

			FGuid PinId = NewPin->PinId;
			NewPin->CopyPersistentDataFromOldPin(*OldPin);
			NewPin->PinId = PinId;
			NewPin->bHidden = OldPin->bHidden;
			
			CustomRemovePin(*OldPin);
		}

		// Inputs
		int32 InputCount = -1;
		for (UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				InputCount++;
			}
		}
		
		for (int32 InputIndex=0; InputIndex < InputCount; ++InputIndex)
		{
			OldPin = FindPin(FString::Printf(TEXT("Mesh Section %d "),InputIndex), EEdGraphPinDirection::EGPD_Input);
			if (OldPin)
			{
				FString NewName = FString::Printf(TEXT("Material %d "), InputIndex);
				UEdGraphPin* NewPin = FindPin(NewName, EEdGraphPinDirection::EGPD_Input);
				if (!NewPin)
				{
					NewPin = CustomCreatePin(EEdGraphPinDirection::EGPD_Input, Schema->PC_MeshSection, FName(*NewName), FText::FromString(NewName));
				}

				FGuid PinId = NewPin->PinId;
				NewPin->CopyPersistentDataFromOldPin(*OldPin);
				NewPin->PinId = PinId;
				NewPin->bHidden = OldPin->bHidden;

				CustomRemovePin(*OldPin);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE

