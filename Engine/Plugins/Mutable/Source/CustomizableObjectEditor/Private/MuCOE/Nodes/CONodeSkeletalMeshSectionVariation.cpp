// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshSectionVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSkeletalMeshSectionVariation)


void UCONodeSkeletalMeshSectionVariation::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeVariationSerializationIssue)
	{
		for (const FCustomizableObjectMaterialVariation& OldVariation : Variations_DEPRECATED)
		{
			FCustomizableObjectVariation Variation;
			Variation.Tag = OldVariation.Tag;
			
			VariationsData.Add(Variation);
		}
	}
}


FName UCONodeSkeletalMeshSectionVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_MeshSection;
}


bool UCONodeSkeletalMeshSectionVariation::IsInputPinArray() const
{
	return true;
}


bool UCONodeSkeletalMeshSectionVariation::IsSingleOutputNode() const
{
	return true;
}
