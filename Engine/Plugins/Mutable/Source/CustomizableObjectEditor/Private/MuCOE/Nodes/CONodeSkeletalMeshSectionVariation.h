// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"

#include "CONodeSkeletalMeshSectionVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

struct FCustomizableObjectMaterialVariation;


UENUM(BlueprintType)
enum class ECustomizableObjectNodeMaterialVariationType : uint8
{
	Tag 		UMETA(DisplayName = "Tag"),
	State 		UMETA(DisplayName = "State"),
};


UCLASS(MinimalAPI)
class UCONodeSkeletalMeshSectionVariation : public UCustomizableObjectNodeVariation
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	ECustomizableObjectNodeMaterialVariationType Type = ECustomizableObjectNodeMaterialVariationType::Tag;

private:
	// Deprecated properties
	UPROPERTY()
	TArray<FCustomizableObjectMaterialVariation> Variations_DEPRECATED;

public:
	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual bool IsSingleOutputNode() const override;
	
	// UCustomizableObjectNodeVariation interface
	UE_API virtual FName GetCategory() const override;
	UE_API virtual bool IsInputPinArray() const override;
};

#undef UE_API
