// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshTransformWithBone.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSkeletalMeshTransformWithBone)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeSkeletalMeshTransformWithBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Transform_SkeletalMesh_In_Bone_Hierarchy_Modifier", "Skeletal Mesh Transform In Bone Hierarchy");
}


FText UCONodeSkeletalMeshTransformWithBone::GetTooltipText() const
{
	return LOCTEXT("Transform_SkeletalMesh_In_Bone_Hierarchy_Modifier_Tooltip", "Applies a transform to the vertices of a Skeletal Mesh that are skinned to the target bone or its child bones");
}


FLinearColor UCONodeSkeletalMeshTransformWithBone::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


void UCONodeSkeletalMeshTransformWithBone::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	SkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	TransformPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Transform);

	OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


#undef LOCTEXT_NAMESPACE
