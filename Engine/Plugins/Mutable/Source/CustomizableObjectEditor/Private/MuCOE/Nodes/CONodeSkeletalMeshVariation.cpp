// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSkeletalMeshVariation.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"


FName UCONodeSkeletalMeshVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_SkeletalMesh;
}

