// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"

#include "CONodeSkeletalMeshVariation.generated.h"


UCLASS()
class UCONodeSkeletalMeshVariation : public UCustomizableObjectNodeVariation
{
public:
	GENERATED_BODY()
	
	// UCustomizableObjectNodeVariation interface
	virtual FName GetCategory() const override;
};
