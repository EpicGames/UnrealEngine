// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeSwitch.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuT/NodeScalarEnumParameter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeSwitch)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"



void UCONodeSwitch::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == SwitchParameterPinReference.Get())
	{
		Super::ReconstructNode();
	}

	Super::PinConnectionListChanged(Pin);
}


void UCONodeSwitch::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, GetCategory());

	UEdGraphPin* OldEnumPin = SwitchParameterPinReference.Get();
	SwitchParameterPinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Enum, TEXT("Switch Parameter"), LOCTEXT("SwitchParameter", "Parameter"));

	SwitchPins.Empty();	
	if (OldEnumPin) // May be null the first time the node is created.
	{
		// This will fail if the node is connected to a macro node which will then prevent the generation of the input option pins
		// todo : UE-294591
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*OldEnumPin))
		{
			if (UCustomizableObjectNodeEnumParameter* EnumNode = Cast<UCustomizableObjectNodeEnumParameter>(LinkedPin->GetOwningNode()))
			{
				for (int32 EnumValueIndex = 0; EnumValueIndex < EnumNode->Values.Num(); ++EnumValueIndex)
				{
					const FString OptionName = EnumNode->Values[EnumValueIndex].Name;
					SwitchPins.Add(CustomCreatePin(EGPD_Input, GetCategory(), FName(*OptionName), FText::FromString(OptionName)));
				}
			}
		}
	}
}


FName UCONodeSwitch::GetCategory() const
{
	return PinType;
}


FText UCONodeSwitch::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("Switch_Title", "{0} Switch"), UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetCategory()));
}


FLinearColor UCONodeSwitch::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(GetCategory());
}


FText UCONodeSwitch::GetTooltipText() const
{
	return LOCTEXT("Switch_Tooltip", "Change the resulting value depending on what is currently chosen among a predefined amount of sources.");
}


void UCONodeSwitch::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::BugPinsSwitch)
	{
		SwitchParameterPinReference = FindPin(TEXT("Switch Parameter"));	
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ChangedSwitchNodesInputPinsFriendlyNames)
	{
		const FString CategoryFriendlyName = UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetCategory()).ToString();
		const FString PinPrefix = UEdGraphSchema_CustomizableObject::GetPinCategoryName(GetCategory()).ToString() + " ";

		int32 NumOptions = 0;
		for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if (Pin->GetName().StartsWith(PinPrefix))
			{
				NumOptions++;
			}
		}
		
		for (int32 ElementIndex = 0; ElementIndex < NumOptions; ++ElementIndex)
		{
			UEdGraphPin* ElementPin = FindPin(PinPrefix + FString::FromInt(ElementIndex) + TEXT(" "));

			if (ElementPin->PinFriendlyName.ToString().StartsWith(TEXT("Material")))
			{
				ElementPin->PinFriendlyName = FText::FromString(CategoryFriendlyName + TEXT(" ") + FString::FromInt(ElementIndex));
			}
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PassthroughParameter)
	{
		const FString PinPrefix = UEdGraphSchema_CustomizableObject::GetPinCategoryName(GetCategory()).ToString() + " ";

		int32 NumOptions = 0;
		for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if (Pin->GetName().StartsWith(PinPrefix))
			{
				NumOptions++;
			}
		}
		
		TArray<FString> OptionNames;
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*SwitchParameterPinReference.Get()))
		{
			if (UCustomizableObjectNodeEnumParameter* EnumNode = Cast<UCustomizableObjectNodeEnumParameter>(LinkedPin->GetOwningNode()))
			{
				for (int32 ValueIndex = 0; ValueIndex < EnumNode->Values.Num(); ++ValueIndex)
				{
					OptionNames.Add(EnumNode->Values[ValueIndex].Name);
				}
			}
		}
		
		for (int32 ElementIndex = 0; ElementIndex < NumOptions; ++ElementIndex)
		{
			UEdGraphPin* ElementPin = FindPin(PinPrefix + FString::FromInt(ElementIndex) + TEXT(" "));
			if (OptionNames.IsValidIndex(ElementIndex))
			{
				ElementPin->PinName = FName(OptionNames[ElementIndex]);
				ElementPin->PinFriendlyName = FText::FromName(ElementPin->PinName);
				SwitchPins.Add(ElementPin);
			}
			else
			{
				ElementPin->bOrphanedPin = true;
			}
		}
	}
}


void UCONodeSwitch::PostBackwardsCompatibleFixup()
{
	Super::PostBackwardsCompatibleFixup();
	Super::ReconstructNode();
}


FString UCONodeSwitch::GetOutputPinName() const
{
	return FString();
}


const UCONodeSwitch* CastSwitch(const UEdGraphNode* Node, FName Type)
{
	const UCONodeSwitch* NodeSwitch = Cast<const UCONodeSwitch>(Node);
	return NodeSwitch && NodeSwitch->PinType == Type ? NodeSwitch : nullptr;
}


UCONodeSwitch* CastSwitch(UEdGraphNode* Node, FName Type)
{
	UCONodeSwitch* NodeSwitch = Cast<UCONodeSwitch>(Node);
	return NodeSwitch && NodeSwitch->PinType == Type ? NodeSwitch : nullptr;
}

#undef LOCTEXT_NAMESPACE
