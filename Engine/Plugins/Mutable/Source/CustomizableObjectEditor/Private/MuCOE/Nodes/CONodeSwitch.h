// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeSwitch.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UObject;
struct FFrame;

namespace UE::Mutable::Private
{
	class NodeScalarEnumParameter;
}

UCLASS()
class UCONodeSwitch : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	
	// UCustomizableObjectNode interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual void PostBackwardsCompatibleFixup() override;

	/** Get the output pin category. Override. */
	virtual FName GetCategory() const;

	/** Get the output pin name. Override. */
	virtual FString GetOutputPinName() const;

protected:
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	
private:
	
	/** Last NodeEnumParameter connected. Used to remove the callback once disconnected. */
	TWeakObjectPtr<UCustomizableObjectNode> LastNodeEnumParameterConnected;

public:
	UPROPERTY()
	FEdGraphPinReference SwitchParameterPinReference;
	
	UPROPERTY()
	TArray<FEdGraphPinReference> SwitchPins;

	UPROPERTY()
	FName PinType;
};


const UCONodeSwitch* CastSwitch(const UEdGraphNode* Node, FName Type);
UCONodeSwitch* CastSwitch(UEdGraphNode* Node, FName Type);

#undef UE_API
