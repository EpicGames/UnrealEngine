// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeTransformInMesh.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeTransformInMesh)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeTransformInMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Transform_Mesh_In_Mesh", "Mesh Transform In Mesh");
}


FText UCONodeTransformInMesh::GetTooltipText() const
{
	return LOCTEXT("Transform_Mesh_In_Mesh_Tooltip", "Applies a transform to the vertices of a mesh that are contained within the given bounding mesh");
}


FLinearColor UCONodeTransformInMesh::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


void UCONodeTransformInMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	BaseMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Base Mesh"), LOCTEXT("BaseMesh", "Base Mesh"));
	BoundingMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Bounding Mesh"), LOCTEXT("BoundingMesh", "Bounding Mesh"));
	TransformMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Transform, TEXT("Transform"), LOCTEXT("Transform", "Transform"));
	
	OutputMeshPin =  CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh);
}


void UCONodeTransformInMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}
	
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCONodeTransformInMesh, BoundingMeshTransform))
	{
		if (TransformChangedDelegate.IsBound())
		{
			TransformChangedDelegate.Broadcast(BoundingMeshTransform);
		}
	}
}


bool UCONodeTransformInMesh::IsDeprecated() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE
