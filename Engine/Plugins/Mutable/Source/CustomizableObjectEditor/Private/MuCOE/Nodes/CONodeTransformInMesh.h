// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CONodeTransformInMesh.generated.h"


namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
struct FPropertyChangedEvent;

DECLARE_MULTICAST_DELEGATE_OneParam(OnTransformChanged, const FTransform&);

UCLASS()
class UCONodeTransformInMesh : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsDeprecated() const override;

	/** Transform to apply to the bounding mesh before selecting for vertices to transform. */
	UPROPERTY(EditAnywhere, Category = BoundingMesh)
	FTransform BoundingMeshTransform = FTransform::Identity;
	
	UPROPERTY()
	FEdGraphPinReference BaseMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference BoundingMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference TransformMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference OutputMeshPin;
	
	/**
	 * Delegate invoked each time the BoundingMeshTransform value gets modified
	 */
	OnTransformChanged TransformChangedDelegate;
};
