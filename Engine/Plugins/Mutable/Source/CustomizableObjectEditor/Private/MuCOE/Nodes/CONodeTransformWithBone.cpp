// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CONodeTransformWithBone.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CONodeTransformWithBone)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCONodeTransformWithBone::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Transform_Mesh_In_Bone_Hierarchy", "Mesh Transform In Bone Hierarchy");
}


FText UCONodeTransformWithBone::GetTooltipText() const
{
	return LOCTEXT("Transform_Mesh_In_Bone_Hierarchy_Tooltip", "Applies a transform to the vertices of a mesh that are skinned to the target bone or its child bones");
}


void UCONodeTransformWithBone::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	BaseMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh);
	TransformPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Transform);

	OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh);
}


FLinearColor UCONodeTransformWithBone::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


#undef LOCTEXT_NAMESPACE
