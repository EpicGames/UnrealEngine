// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuT/NodeSurfaceModifier.h"

#include "CONodeTransformWithBone.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
struct FPropertyChangedEvent;

DECLARE_MULTICAST_DELEGATE_OneParam(OnTransformChanged, const FTransform&);

UCLASS()
class UCONodeTransformWithBone : public UCustomizableObjectNode
{
	GENERATED_BODY()
	
public:
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual FLinearColor GetNodeTitleColor() const override;

	/** Root bone used to filter vertices based on skinning weights. Vertices skinned to this
	  * bone or below will be transformed. */
	UPROPERTY(EditAnywhere, Category = Transform)
	FString BoneName;

	/** Vertex influences under this threshold will not be considered when filtering vertices.
	  * Percentage of total skin weight [0.0 .. 1.0]. */
	UPROPERTY(EditAnywhere, Category = Transform, meta = (ClampMin = 0.0f, ClampMax = 1.0f))
	float ThresholdFactor = 0.05f;
	
	UPROPERTY()
	FEdGraphPinReference BaseMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference TransformPin;

	UPROPERTY()
	FEdGraphPinReference OutputPin;
	
	/**
	* Delegate invoked each time the BoundingMeshTransform value gets modified
	*/
	OnTransformChanged TransformChangedDelegate;
};