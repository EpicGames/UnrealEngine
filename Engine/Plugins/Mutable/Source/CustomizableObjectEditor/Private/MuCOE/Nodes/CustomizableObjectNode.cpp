// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "SCustomizableObjectNode.h"
#include "Containers/Queue.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"
#include "Toolkits/ToolkitManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNode)

class IToolkit;
class SWidget;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

/** This is used to generate automatic unique tags for some nodes. */
#define MUTABLE_INTERNAL_TAG_PREFIX		"MutableInternalTag_"


UCustomizableObjectGraph* UCustomizableObjectNode::GetCustomizableObjectGraph() const
{
	return Cast<UCustomizableObjectGraph>(GetOuter());
}


bool UCustomizableObjectNode::IsSingleOutputNode() const
{
	return false;
}


UEdGraphPin* UCustomizableObjectNode::CustomCreatePin(EEdGraphPinDirection Direction, const FName& Category, UCustomizableObjectNodePinData* PinData)
{
	const FName NewPinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(Category);
	const FText NewPinFriendlyName = UEdGraphSchema_CustomizableObject::GetPinFriendlyName(Category);

	return CustomCreatePin(Direction, Category, NewPinName, NewPinFriendlyName, PinData);
}


UEdGraphPin* UCustomizableObjectNode::CustomCreatePin(EEdGraphPinDirection Direction, const FName& Category, const FName& Name, const FText& FriendlyName, UCustomizableObjectNodePinData* PinData)
{
	UEdGraphPin* Pin = CreatePin(Direction, Category, Name);
	
	Pin->PinFriendlyName = FriendlyName;
	
	if (PinData)
	{
		AddPinData(*Pin, *PinData);		
	}

	return Pin;
}


bool UCustomizableObjectNode::ShouldBreakExistingConnections(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	return IsSingleOutputNode();
}


void UCustomizableObjectNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	PostEditChangePropertyDelegate.Broadcast(this, PropertyChangedEvent);
}


TSharedPtr<FCustomizableObjectGraphEditorToolkit> UCustomizableObjectNode::GetGraphEditor() const
{
	const UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(GetCustomizableObjectGraph()->GetOuter());

	TSharedPtr<FCustomizableObjectGraphEditorToolkit> CustomizableObjectEditor;
	if (CustomizableObject)
	{
		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(CustomizableObject);
		if (FoundAssetEditor.IsValid())
		{
			return StaticCastSharedPtr<FCustomizableObjectGraphEditorToolkit>(FoundAssetEditor);
		}
	}

	return TSharedPtr<FCustomizableObjectGraphEditorToolkit>();
}


bool UCustomizableObjectNode::CustomRemovePin(UEdGraphPin& Pin)
{
	PinsDataId.Remove(Pin.PinId);
	
	return RemovePin(&Pin);	
}


bool UCustomizableObjectNode::ShouldAddToContextMenu(FText& OutCategory) const
{
	return false;
}


void UCustomizableObjectNode::SetRefreshNodeWarning()
{
	if (!bHasCompilerMessage && ErrorType < EMessageSeverity::Warning)
	{
		GetGraph()->NotifyGraphChanged();
	
		bHasCompilerMessage = true;
		ErrorType = EMessageSeverity::Warning;
		ErrorMsg = GetRefreshMessage();
	}
}


void UCustomizableObjectNode::RemoveWarnings()
{
	bHasCompilerMessage = false;
	ErrorType = 0;
	ErrorMsg.Empty();
}


void UCustomizableObjectNode::AllocateDefaultPins()
{
	AllocateDefaultPins(nullptr);
}


void UCustomizableObjectNode::ReconstructNode()
{
	ReconstructNode(CreateRemapPinsDefault());
}


void UCustomizableObjectNode::FixupReconstructPins(UCustomizableObjectNodeRemapPins* RemapPinsAction, TFunction<void(UCustomizableObjectNode*, UCustomizableObjectNodeRemapPins*)> FuncAllocateDefaultPins)
{
	Modify();

	// Break any single sided links. All connections must always be present in both nodes
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		TArray<class UEdGraphPin*>& LinkedToRef = Pin->LinkedTo;
		for (int32 LinkIdx=0; LinkIdx < LinkedToRef.Num(); LinkIdx++)
		{
			UEdGraphPin* OtherPin = LinkedToRef[LinkIdx];
			// If we are linked to a pin that its owner doesn't know about, break that link
			if (OtherPin && !OtherPin->GetOwningNode()->Pins.Contains(OtherPin))
			{
				Pin->LinkedTo.Remove(OtherPin);
			}
		}
	}

	RemoveWarnings();

	// Move the existing orphan and non-orphan pins to a saved array
	TArray<UEdGraphPin*> OldPins(Pins); // We can not empty Pins at this point since it will break all FEdGraphPinReference during the reconstruction.

	// Recreate the new pins
	FuncAllocateDefaultPins(this, RemapPinsAction);
	
	// Try to remap orphan and non-orphan pins.
	TArray<UEdGraphPin*> NewPins;
	NewPins.Reset(Pins.Num() - OldPins.Num());
	for (UEdGraphPin* Pin : Pins)
	{
		if (!OldPins.Contains(Pin))
		{
			NewPins.Add(Pin);
		}
	}

	TMap<UEdGraphPin*, UEdGraphPin*> PinsToRemap;
	TArray<UEdGraphPin*> PinsToOrphan;
	RemapPinsAction->RemapPins(*this, OldPins, NewPins, PinsToRemap, PinsToOrphan);

	// Check only.
	for (const TTuple<UEdGraphPin*, UEdGraphPin*>& Pair : PinsToRemap)
	{
		check(NewPins.Contains(Pair.Value)); // Can only remap and old pin to a new pin.
		check(OldPins.Contains(Pair.Key));
	}

	RemapPins(PinsToRemap);
	RemapPinsData(PinsToRemap);
	
	// Check only.
	for (UEdGraphPin* Pin : PinsToOrphan)
	{
		check(OldPins.Contains(Pin)); // Can only orphan old pins.
	}

	bool bOrphanedPin = false;
	FName FirstOldPin;
	for (UEdGraphPin* OldPin : OldPins)
	{
		OldPin->Modify();

		if (PinsToOrphan.Contains(OldPin))
		{
			if (!bOrphanedPin)
			{
				bOrphanedPin = !OldPin->bOrphanedPin;
				if (bOrphanedPin)
				{
					FirstOldPin = OldPin->GetFName();
				}
			}
			OrphanPin(*OldPin);
		
			// Move pin to the end.
			Pins.RemoveSingle(OldPin);
			Pins.Add(OldPin);
		}
		else
		{
			// Remove the old pin
			OldPin->BreakAllPinLinks();
			
			CustomRemovePin(*OldPin);
		}
	}

	if (UEdGraph* Graph = GetCustomizableObjectGraph())
	{
		if (bOrphanedPin)
		{
			FCustomizableObjectEditorLogger::CreateLog(FText::Format(LOCTEXT("OrphanPinsWarningReconstruct", "Failed to remap old pins. First old pin: {0}"), FText::FromName(FirstOldPin)))
			.BaseObject()
			.Severity(EMessageSeverity::Warning)
			.Context(*this)
			.CustomizableObject(GraphTraversal::GetObject(*this))
			.Log();
		}

		Graph->NotifyNodeChanged(this);
	}
	
	PostReconstructNodeDelegate.Broadcast();
}


void UCustomizableObjectNode::DestroyNode()
{
	Super::DestroyNode();
	DestroyNodeDelegate.Broadcast();
}


TSharedPtr<SGraphNode> UCustomizableObjectNode::CreateVisualWidget()
{
	return SNew(SCustomizableObjectNode, this);
}


bool UCustomizableObjectNode::GetCanRenameNode() const
{
	return false;
}


void UCustomizableObjectNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (!FromPin)
	{
		return;
	}

	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	for (UEdGraphPin* Pin : GetAllNonOrphanPins())
	{	
		if (Schema->TryCreateConnection(FromPin, Pin))
		{
			break;
		}		
	}
}


void UCustomizableObjectNode::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();
	NodeConnectionListChangedDelegate.Broadcast();
}


void UCustomizableObjectNode::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	PinConnectionListChangedDelegate.Broadcast(Pin);
}


void UCustomizableObjectNode::PostInitProperties()
{
	Super::PostInitProperties();

	RemoveWarnings();
}


void UCustomizableObjectNode::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	// Fix UE large world coordinates automatic pin conversion. 
	// Now all pins with PinCategory == FName("float") get automatically changed to the new double type
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixBlueprintPinsUseRealNumbers)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			FEdGraphPinType* const PinType = &Pin->PinType;

			if (PinType->PinCategory == TEXT("real") && PinType->PinSubCategory == TEXT("double"))
			{
				PinType->PinCategory = TEXT("float");
				PinType->PinSubCategory = FName();
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeMaterialPerformance)
	{
		for (TTuple<FEdGraphPinReference, UCustomizableObjectNodePinData*> Pair : PinsData_DEPRECATED)
		{
			PinsDataId.Add(Pair.Key.Get()->PinId, Pair.Value);
		}

		PinsData_DEPRECATED.Empty();
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixBlueprintPinsUseRealNumbersAgain)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			FEdGraphPinType* const PinType = &Pin->PinType;

			if (PinType->PinCategory == TEXT("real") && PinType->PinSubCategory == TEXT("double"))
			{
				PinType->PinCategory = TEXT("float");
				PinType->PinSubCategory = FName();
			}
		}
	}
}


void UCustomizableObjectNode::ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPinsMode)
{
	FixupReconstructPins(RemapPinsMode, [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* Action){ Node->AllocateDefaultPins(Action); });
}


bool UCustomizableObjectNode::CanConnect( const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const
{
	bOutIsOtherNodeBlocklisted = false;

	bOutArePinsCompatible = InOwnedInputPin->PinType.PinCategory == InOutputPin->PinType.PinCategory ||
		InOwnedInputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Wildcard ||
		InOutputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Wildcard ||
		(InOutputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Object && InOwnedInputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough) ||
		(InOutputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object && InOwnedInputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
		
	
	return bOutArePinsCompatible;
}


bool UCustomizableObjectNode::IsAffectedByLOD() const
{ 
	return true; 
}


TArray<FString> UCustomizableObjectNode::GetEnableTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext)
{ 
	return TArray<FString>(); 
}


TArray<FString>* UCustomizableObjectNode::GetEnableTagsArray()
{
	return nullptr;
}


FString UCustomizableObjectNode::GetInternalTag() const
{
	return FString::Printf(TEXT("%s%s"), TEXT(MUTABLE_INTERNAL_TAG_PREFIX), *NodeGuid.ToString());
}


FGuid UCustomizableObjectNode::GetInternalTagNodeId(const FString& Tag)
{
	FGuid TempId;

	bool bCorrect = Tag.StartsWith(TEXT(MUTABLE_INTERNAL_TAG_PREFIX));
	if (bCorrect)
	{
		int32 PrefixSize = FString(TEXT(MUTABLE_INTERNAL_TAG_PREFIX)).Len();
		FString IdString = Tag.RightChop(PrefixSize);
		bCorrect = FGuid::Parse(IdString, TempId);
	}

	return bCorrect ? TempId : FGuid();
}


bool UCustomizableObjectNode::IsInternalTag(const FString& Tag)
{
	return GetInternalTagNodeId(Tag).IsValid();
}


bool UCustomizableObjectNode::FindNodeForInternalTag(const FString& Tag, UCustomizableObjectNode*& OutNode, UCustomizableObject*& OutObject)
{
	OutNode = nullptr;
	OutObject = nullptr;

	FGuid NodeId = GetInternalTagNodeId(Tag);
	if (!NodeId.IsValid())
	{
		return false;
	}

	// Scan all potential receivers
	UCustomizableObject* ThisNodeObject = GraphTraversal::GetObject(*this);
	UCustomizableObject* RootObject = GraphTraversal::GetRootObject(ThisNodeObject);

	TSet<UCustomizableObject*> AllCustomizableObject;
	GraphTraversal::GetAllObjectsInGraph(RootObject, AllCustomizableObject);

	for (UCustomizableObject* CustObject : AllCustomizableObject)
	{
		if (CustObject)
		{
			for (const TObjectPtr<UEdGraphNode>& CandidateNode : CustObject->GetPrivate()->GetSource()->Nodes)
			{
				if (CandidateNode->NodeGuid == NodeId)
				{
					OutNode = Cast<UCustomizableObjectNode>(CandidateNode);
					OutObject = CustObject;
					return true;
				}
			}
		}
	}

	return false;
}


FString UCustomizableObjectNode::GetInternalTagDisplayName()
{
	ensure(false);
	return FString();
}


FString UCustomizableObjectNode::GetTagDisplayName(const FString& InTag)
{
	UCustomizableObjectNode* InternalTagNode = nullptr;
	UCustomizableObject* InternalTagObject = nullptr;
	bool bIsInternal = FindNodeForInternalTag(InTag, InternalTagNode, InternalTagObject);
	if (bIsInternal && InternalTagNode)
	{
		return InternalTagNode->GetInternalTagDisplayName();
	}

	return InTag;
}


UCustomizableObjectNodeRemapPins* UCustomizableObjectNode::CreateRemapPinsDefault() const
{
	return CreateRemapPinsByName();
}


UCustomizableObjectNodeRemapPinsByName* UCustomizableObjectNode::CreateRemapPinsByName() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByName>();
}


void UCustomizableObjectNode::RemapPin(UEdGraphPin& NewPin, const UEdGraphPin& OldPin)
{
	FGuid PinId = NewPin.PinId;
	
	NewPin.CopyPersistentDataFromOldPin(OldPin);
	NewPin.PinId = PinId;
	NewPin.bHidden = OldPin.bHidden;	
}


UCustomizableObjectNodeRemapPinsByPosition* UCustomizableObjectNode::CreateRemapPinsByPosition() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByPosition>();
}


void UCustomizableObjectNode::RemapPins(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap)
{
	for (const TTuple<UEdGraphPin*, UEdGraphPin*>& Pair : PinsToRemap)
	{
		RemapPin(*Pair.Value, *Pair.Key);
	}
	
	RemapPinsDelegate.Broadcast(PinsToRemap);
}


void UCustomizableObjectNode::RemapPinsData(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap)
{
	for (const TTuple<UEdGraphPin*, UEdGraphPin*>& Pair : PinsToRemap)
	{
		// Move pin data.
		if (const TObjectPtr<UCustomizableObjectNodePinData>* PinDataOldPin = PinsDataId.Find(Pair.Key->PinId))
		{
			PinsDataId[Pair.Value->PinId]->Copy(**PinDataOldPin);
		}
	}
}


void UCustomizableObjectNode::AddPinData(const UEdGraphPin& Pin, UCustomizableObjectNodePinData& PinData)
{
	check(PinData.GetOuter() != GetTransientPackage())
	PinsDataId.Add(Pin.PinId, &PinData);
}


bool UCustomizableObjectNode::IsExperimental() const
{
	return false;
}


bool UCustomizableObjectNode::IsDeprecated() const
{
	return false;
}


bool UCustomizableObjectNode::IsLoaded() const
{
	return true;
}


TArray<UEdGraphPin*> UCustomizableObjectNode::GetAllOrphanPins() const
{
	TArray<UEdGraphPin*> OrphanPins;

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->bOrphanedPin)
		{
			OrphanPins.Add(Pin);
		}
	}

	return OrphanPins;
}


TArray<UEdGraphPin*> UCustomizableObjectNode::GetAllNonOrphanPins() const
{
	TArray<UEdGraphPin*> NonOrphanPins;
	NonOrphanPins.Reserve(Pins.Num());
	
	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin->bOrphanedPin)
		{
			NonOrphanPins.Add(Pin);
		}
	}

	return NonOrphanPins;
}


UCustomizableObjectNodePinData::UCustomizableObjectNodePinData()
{
	SetFlags(RF_Transactional);
}


bool UCustomizableObjectNodePinData::operator==(const UCustomizableObjectNodePinData& Other) const
{
	return Equals(Other);
}


bool UCustomizableObjectNodePinData::operator!=(const UCustomizableObjectNodePinData& Other) const
{
	return !operator==(Other);
}


bool UCustomizableObjectNodePinData::Equals(const UCustomizableObjectNodePinData& Other) const
{
	return GetClass() == Other.GetClass();	
}


void UCustomizableObjectNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);
}


void UCustomizableObjectNode::PostLoad()
{
	// Do not do work here. Do work at PostBackwardsCompatibleFixup.
	Super::PostLoad();
}


void UCustomizableObjectNode::SetPinHidden(UEdGraphPin& InPin, bool bHidden)
{
	const FText TransactionName = bHidden ? (LOCTEXT("HidePinTransaction", "Hide Pin")) : LOCTEXT("ShowPinTransaction", "Show Pin");
	FScopedTransaction Transaction(TransactionName);
	Modify();
	
	InPin.SafeSetHidden(bHidden && CanPinBeHidden(InPin));
	GetGraph()->NotifyGraphChanged();
}


void UCustomizableObjectNode::SetPinHidden(const TArray<UEdGraphPin*>& InPins, bool bHidden)
{
	const FText TransactionName = bHidden ? (LOCTEXT("HideAllPinsTransaction", "Hide All Pins")) : LOCTEXT("ShowAllPinsTransaction", "Show All Pins");
	FScopedTransaction Transaction(TransactionName);
	Modify();
	
	for (UEdGraphPin* Pin : InPins)
	{
		Pin->SafeSetHidden(bHidden && CanPinBeHidden(*Pin));
	}

	GetGraph()->NotifyGraphChanged();
}


bool UCustomizableObjectNode::CanPinBeHidden(const UEdGraphPin& Pin) const
{
	return !Pin.LinkedTo.Num() && !Pin.bOrphanedPin && HasPinViewer();
}


bool UCustomizableObjectNode::CanRenamePin(const UEdGraphPin& Pin) const
{
	return false;
}


EEditablePinNameBoxVisibilityPolicy UCustomizableObjectNode::GetEditablePinNameVisibilityPolicy (const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::HIDE_IF_LINKED;
}


bool UCustomizableObjectNode::ShouldPinViewerShowPinEditableName(const UEdGraphPin& Pin) const
{
	return CanRenamePin(Pin);
}


FText UCustomizableObjectNode::GetPinEditableName(const UEdGraphPin& Pin) const
{
	return {};
}


void UCustomizableObjectNode::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	
}


FLinearColor UCustomizableObjectNode::GetPinColor(const UEdGraphPin& Pin) const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(Pin.PinType.PinCategory);
}


bool UCustomizableObjectNode::HasPinViewer() const
{
	return false;
}


EAddPinNodeButtonLocation UCustomizableObjectNode::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::NONE;
}


void UCustomizableObjectNode::AddPinFromUI()
{
	// You are not expected to call this method if not part of the "Add pin" in-node button operation. Use "CustomAddPin()" method instead.
	ensure(GetAddPinButtonNodeSide() != EAddPinNodeButtonLocation::NONE);
}


bool UCustomizableObjectNode::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	return false;
}


TSharedPtr<IDetailsView> UCustomizableObjectNode::CustomizePinDetails(const UEdGraphPin& Pin) const
{
	return nullptr;
}


UEdGraphPin* UCustomizableObjectNode::GetPin(const UCustomizableObjectNodePinData& PinData)
{
	for (const TTuple<FGuid, TObjectPtr<UCustomizableObjectNodePinData>>& Pair : PinsDataId)
	{
		if (Pair.Value == &PinData)
		{
			return FindPinById(Pair.Key);
		}
	}
	
	return nullptr;
}


UCustomizableObjectNodePinData* UCustomizableObjectNode::GetPinData(const UEdGraphPin& Pin) const
{
	if (TObjectPtr<UCustomizableObjectNodePinData> const* Result = PinsDataId.Find(Pin.PinId))
	{
		return *Result;	
	}
	else
	{
		return nullptr;
	}
}


bool UCustomizableObjectNode::IsNodeSupportedInMacros() const
{
	return true;
}


bool UCustomizableObjectNode::IsInMacro() const
{
	if (const UCustomizableObjectGraph* Graph = Cast<UCustomizableObjectGraph>(GetGraph()))
	{
		return Graph->IsMacro();
	}
	
	return false;
}


TArray<FName> UCustomizableObjectNode::GetAllowedPinViewerCreationTypes() const
{
	return UEdGraphSchema_CustomizableObject::SupportedMacroPinTypes;
}


TArray<FName> UCustomizableObjectNode::GetPinAllowedTypes(const UEdGraphPin& Pin) const
{
	return {};
}

TArray<FName> UCustomizableObjectNode::GetPinAllowedSubTypes(const UEdGraphPin& Pin) const
{
	return {};
}


bool UCustomizableObjectNode::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	if (const UCustomizableObjectGraph* CurrentGraph = Cast<UCustomizableObjectGraph>(TargetGraph))
	{
		if (CurrentGraph->IsMacro())
		{
			return IsNodeSupportedInMacros();
		}

		return true;
	}

	return false;
}


#undef LOCTEXT_NAMESPACE
