// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeAnimationPose.h"

#include "Animation/AnimationPoseData.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/PoseAsset.h"
#include "BonePose.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeAnimationPose)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeAnimationPose::UCustomizableObjectNodeAnimationPose() : Super()
	, PoseAsset(nullptr)
{
}


void UCustomizableObjectNodeAnimationPose::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh);
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_PoseAsset);
	CustomCreatePin(EGPD_Output,  UEdGraphSchema_CustomizableObject::PC_Mesh);
}

void UCustomizableObjectNodeAnimationPose::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* InputMesh = FindPin(TEXT("Input Mesh"), EGPD_Input))
		{
			InputMesh->PinName = TEXT("Mesh");
			InputMesh->PinFriendlyName = LOCTEXT("Mesh_Pin_Category", "Mesh");
		}

		if (UEdGraphPin* OutputMesh = FindPin(TEXT("Table Pose"), EGPD_Input))
		{
			OutputMesh->PinName = TEXT("PoseAsset");
			OutputMesh->PinFriendlyName = LOCTEXT("Pose_Pin_Category", "PoseAsset");
		}
		
		if (UEdGraphPin* OutputMesh = FindPin(TEXT("Output Mesh"), EGPD_Output))
		{
			OutputMesh->PinName = TEXT("Mesh");
			OutputMesh->PinFriendlyName = LOCTEXT("Mesh_Pin_Category", "Mesh");
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeAnimationPose::GetInputMeshPin() const
{
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_Mesh);
	UEdGraphPin* Pin = FindPin(PinName, EGPD_Input);
	return Pin;
}


UEdGraphPin* UCustomizableObjectNodeAnimationPose::GetTablePosePin() const
{
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_PoseAsset);
	UEdGraphPin* Pin = FindPin(PinName, EGPD_Input);
	return Pin;
}


FText UCustomizableObjectNodeAnimationPose::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (PoseAsset)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("SkeletalMeshName"), FText::FromString(PoseAsset->GetName()));

		return FText::Format(LOCTEXT("AnimationPose_Title", "{SkeletalMeshName}\nAnimation Pose"), Args);
	}
	else
	{
		return LOCTEXT("PoseMesh", "Mesh Pose");
	}
}


FLinearColor UCustomizableObjectNodeAnimationPose::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


void UCustomizableObjectNodeAnimationPose::StaticRetrievePoseInformation(UPoseAsset* PoseAsset, USkeletalMesh* RefSkeletalMesh, TArray<FName>& OutArrayBoneName, TArray<FTransform>& OutArrayTransform)
{
	if (PoseAsset && RefSkeletalMesh)
	{
		// Need this for the FCompactPose below
		FMemMark Mark(FMemStack::Get());

		UDebugSkelMeshComponent* SkeletalMeshComponent = NewObject<UDebugSkelMeshComponent>();
		SkeletalMeshComponent->SetSkeletalMesh(RefSkeletalMesh);
		SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationBlueprint);
		SkeletalMeshComponent->AllocateTransformData();
		SkeletalMeshComponent->SetAnimation(PoseAsset);
		SkeletalMeshComponent->RefreshBoneTransforms();
		SkeletalMeshComponent->InitAnim(false);

		// PoseSkeleton might be different than the RefSkeletalMesh's skeleton, use it as reference to extract the pose.
		USkeleton* PoseSkeleton = PoseAsset->GetSkeleton();
		const FReferenceSkeleton& PoseRefSkeleton = PoseSkeleton->GetReferenceSkeleton();

		// Use all bones from the pose's skeleton as RequiredBones
		TArray<FBoneIndexType> RequiredBones;
		RequiredBones.SetNumUninitialized(PoseRefSkeleton.GetRawBoneNum());

		for (uint16 BoneIndex = 0; BoneIndex < PoseRefSkeleton.GetRawBoneNum(); ++BoneIndex)
		{
			RequiredBones[BoneIndex] = BoneIndex;
		}

		FBoneContainer BoneContainer;
		BoneContainer.InitializeTo(RequiredBones, SkeletalMeshComponent->GetCurveFilterSettings(), *PoseSkeleton);

		// Needs a FMemMark declared before in the stack context so that the memory allocated by the FCompactPose is freed correctly
		FCompactPose OutPose;
		OutPose.SetBoneContainer(&BoneContainer);

		FBlendedCurve OutCurve;
		UE::Anim::FStackAttributeContainer OutAttributes;
		FAnimationPoseData OutAnimData(OutPose, OutCurve, OutAttributes);
		PoseAsset->GetBaseAnimationPose(OutAnimData);
		OutPose = OutAnimData.GetPose();
		OutCurve = OutAnimData.GetCurve();

		OutCurve.CopyFrom(SkeletalMeshComponent->GetAnimCurves());

		// Assuming one single pose, with a weight set to 1.0
		FAnimExtractContext ExtractionContext;
		ExtractionContext.bExtractRootMotion = false;
		ExtractionContext.CurrentTime = 0.0f;
		
		const TArray<FName>& PoseNames = PoseAsset->GetPoseFNames();
		ExtractionContext.PoseCurves.Add(FPoseCurve(0, PoseNames[0], 1.0f));
		FAnimationPoseData SecondOutAnimData(OutPose, OutCurve, OutAttributes);
		PoseAsset->GetAnimationPose(SecondOutAnimData, ExtractionContext);
		OutPose = OutAnimData.GetPose();
		OutCurve = OutAnimData.GetCurve();

		const TArray<FTransform, FAnimStackAllocator>& ArrayPoseBoneTransform = OutPose.GetBones();

		// Extract final pose by combining the OutPose with the RefSkeleton pose (for bones missing in the PoseRefSkeleton)
		const FReferenceSkeleton& RefSkeleton = RefSkeletalMesh->GetRefSkeleton();
		for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

			FTransform CumulativePoseTransform;

			int32 ParentIndex = BoneIndex;
			while (ParentIndex >= 0)
			{
				FName ParentName = RefSkeleton.GetBoneName(ParentIndex);
				if (int32 PoseBoneIndex = PoseRefSkeleton.FindBoneIndex(ParentName); PoseBoneIndex >= 0)
				{
					CumulativePoseTransform *= ArrayPoseBoneTransform[PoseBoneIndex];
				}
				else
				{
					// Accumulate the transform using the RefSkeleton's transform if the pose doesn't have the bone
					CumulativePoseTransform *= RefSkeleton.GetRefBonePose()[ParentIndex];
				}
			
				ParentIndex = RefSkeleton.GetParentIndex(ParentIndex);
			}

			const FTransform BoneToComponentTransform = SkeletalMeshComponent->GetEditableComponentSpaceTransforms()[BoneIndex];
			FTransform TransformToAdd = BoneToComponentTransform.Inverse() * CumulativePoseTransform;

			OutArrayBoneName.Add(BoneName);
			OutArrayTransform.Add(TransformToAdd);
		}
	}
}


#undef LOCTEXT_NAMESPACE
