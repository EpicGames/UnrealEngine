// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeAnimationPose.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FString;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
class UPoseAsset;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeAnimationPose : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeAnimationPose();

	UPROPERTY(EditAnywhere, Category=NoCategory)
	TObjectPtr<UPoseAsset> PoseAsset;

	// TODO Remove. Only used because poses are the diff from a Reference Skeleton. Once the pose is absolute remove this.
	UPROPERTY(EditAnywhere, Category=NoCategory)
	TObjectPtr<USkeletalMesh> ReferenceSkeletalMesh;
	
	// Begin EdGraphNode interface
	UE_API UEdGraphPin* GetInputMeshPin() const;
	UE_API UEdGraphPin* GetTablePosePin() const;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	// End EdGraphNode interface

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	// Retrieve the pose information from the PoseAsset
	static UE_API void StaticRetrievePoseInformation(UPoseAsset* PoseAsset, class USkeletalMesh* RefSkeletalMesh, TArray<FName>& OutArrayBoneName, TArray<FTransform>& OutArrayTransform);
};

#undef UE_API
