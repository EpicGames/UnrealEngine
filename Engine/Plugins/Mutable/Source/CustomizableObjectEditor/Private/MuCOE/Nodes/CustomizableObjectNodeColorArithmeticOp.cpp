// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeColorArithmeticOp.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeColorArithmeticOp)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeColorArithmeticOp::UCustomizableObjectNodeColorArithmeticOp()
	: Super()
{
	Operation = EColorArithmeticOperation::E_Add;
}


void UCustomizableObjectNodeColorArithmeticOp::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	//UProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	//if ( PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("NumLODs") )
	{
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeColorArithmeticOp::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Color, TEXT("A"), LOCTEXT("A", "A"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Color,  TEXT("B"), LOCTEXT("B", "B"));
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Color, TEXT("Result"), LOCTEXT("Result", "Result"));
}


FText UCustomizableObjectNodeColorArithmeticOp::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::ListView)
	{
		return LOCTEXT("Color_Arithmetic_Operation", "Color Arithmetic Operation");
	}

	const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/CustomizableObjectEditor.EColorArithmeticOperation"), EFindObjectFlags::ExactClass);

	if (!EnumPtr)
	{
		return FText::FromString(FString("Color Operation"));
	}

	const int32 index = EnumPtr->GetIndexByValue((int32)Operation);
	return EnumPtr->GetDisplayNameTextByIndex(index);
}


FLinearColor UCustomizableObjectNodeColorArithmeticOp::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Color);
}


FText UCustomizableObjectNodeColorArithmeticOp::GetTooltipText() const
{
	return LOCTEXT("Color_Arithmetic_Tooltip", "Perform an arithmetic operation between two colors on a per-component basis.");
}


#undef LOCTEXT_NAMESPACE

