// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeColorArithmeticOp.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UENUM(BlueprintType)
enum class EColorArithmeticOperation : uint8
{
	E_Add 	UMETA(DisplayName = "+"),
	E_Sub 	UMETA(DisplayName = "-"),
	E_Mul	UMETA(DisplayName = "x"),
	E_Div	UMETA(DisplayName = "/")
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeColorArithmeticOp : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeColorArithmeticOp();

	// UObject interface.
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;
	bool ShouldOverridePinNames() const override { return true; }

	// UCustomizableObjectNode interface
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	// First Operand
	UEdGraphPin* XPin() const
	{
		return FindPin(TEXT("A"));
	}

	// Second Operand
	UEdGraphPin* YPin() const
	{
		return FindPin(TEXT("B"));
	}

	UPROPERTY(EditAnywhere, Category = ColorArithmeticOperation)
	EColorArithmeticOperation Operation;

	// UCustomizableObjectNode interface
	virtual bool IsAffectedByLOD() const { return false; }

};

#undef UE_API
