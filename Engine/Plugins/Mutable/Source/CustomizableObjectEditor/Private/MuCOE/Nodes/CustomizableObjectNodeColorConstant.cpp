// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeColorConstant.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeColorConstant)

class UCustomizableObjectNodeRemapPins;
struct FGeometry;
struct FPropertyChangedEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeColorConstant::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Color);
}


FText UCustomizableObjectNodeColorConstant::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Color_Constant", "Color");
}


FLinearColor UCustomizableObjectNodeColorConstant::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Color);
}


FText UCustomizableObjectNodeColorConstant::GetTooltipText() const
{
	return LOCTEXT("Color_Constant_Tooltip", "Define a constant color value.");
}


void UCustomizableObjectNodeColorConstant::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Color");
			Pin->PinFriendlyName = LOCTEXT("Color_Pin_Category", "Color");;
		}
	}
}


TSharedPtr<SGraphNode> UCustomizableObjectNodeColorConstant::CreateVisualWidget()
{
	return SNew(SGraphNodeColorConstant, this);
}


// SGraph Node Color Constant 

void SGraphNodeColorConstant::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	NodeColorConstant = Cast< UCustomizableObjectNodeColorConstant >(InGraphNode);

	WidgetStyle = FCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("SpinBox");

	SCustomizableObjectNode::Construct({}, InGraphNode);
}


void SGraphNodeColorConstant::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	// Collapsing arrow of the title area
	DefaultTitleAreaWidget->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(FMargin(5))
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SGraphNodeColorConstant::OnExpressionPreviewChanged)
		.IsChecked(IsExpressionPreviewChecked())
		.Cursor(EMouseCursor::Default)
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(GetExpressionPreviewArrow())
			]
		]
	];
}


void SGraphNodeColorConstant::OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState)
{
	NodeColorConstant->bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	UpdateGraphNode();
}


ECheckBoxState SGraphNodeColorConstant::IsExpressionPreviewChecked() const
{
	return NodeColorConstant->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


const FSlateBrush* SGraphNodeColorConstant::GetExpressionPreviewArrow() const
{
	return FCustomizableObjectEditorStyle::Get().GetBrush(NodeColorConstant->bCollapsed ? TEXT("Nodes.ArrowDown") : TEXT("Nodes.ArrowUp"));
}


void SGraphNodeColorConstant::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	LeftNodeBox->AddSlot()
	[
		SNew(SVerticalBox)
		.Visibility(NodeColorConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f,0.0f,0.0f,5.0f)
		[
			SNew(SColorBlock)
			.Color_Lambda([&]() { return NodeColorConstant->Value; })
			.ShowBackgroundForAlpha(true)
			//.IgnoreAlpha(false)
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.Size(FVector2D(50.0f,50.0f))
			.OnMouseButtonDown(this, &SGraphNodeColorConstant::OnColorPreviewClicked)
		]
	];
	
	MainBox->AddSlot()
	[
		SAssignNew(ColorEditor,SVerticalBox)
		.Visibility(EVisibility::Collapsed)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 0.0f, 10.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(NodeColorConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RedText", "R:"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSpinBox<float>)
				.MinDesiredWidth(100.0f)
				.Value_Lambda([this]() {return NodeColorConstant->Value.R; })
				.OnValueChanged(this, &SGraphNodeColorConstant::OnSpinBoxValueChanged,ColorChannel::RED)
				.OnValueCommitted(this, &SGraphNodeColorConstant::OnSpinBoxValueCommitted, ColorChannel::RED)
				.Style(&WidgetStyle)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Delta(0.01f)
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f, 10.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(NodeColorConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GreenText", "G:"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSpinBox<float>)
				.MinDesiredWidth(100.0f)
				.Value_Lambda([this]() {return NodeColorConstant->Value.G; })
				.OnValueChanged(this, &SGraphNodeColorConstant::OnSpinBoxValueChanged, ColorChannel::GREEN)
				.OnValueCommitted(this, &SGraphNodeColorConstant::OnSpinBoxValueCommitted, ColorChannel::GREEN)
				.Style(&WidgetStyle)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Delta(0.01f)
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f, 10.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(NodeColorConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BlueText", "B:"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSpinBox<float>)
				.MinDesiredWidth(100.0f)
				.Value_Lambda([this]() {return NodeColorConstant->Value.B; })
				.OnValueChanged(this, &SGraphNodeColorConstant::OnSpinBoxValueChanged, ColorChannel::BLUE)
				.OnValueCommitted(this, &SGraphNodeColorConstant::OnSpinBoxValueCommitted, ColorChannel::BLUE)
				.Style(&WidgetStyle)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Delta(0.01f)
			]
		]
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f, 10.0f, 10.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(NodeColorConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AlphaText", "A:"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSpinBox<float>)
				.MinDesiredWidth(100.0f)
				.Value_Lambda([this]() {return NodeColorConstant->Value.A; })
				.OnValueChanged(this, &SGraphNodeColorConstant::OnSpinBoxValueChanged, ColorChannel::ALPHA)
				.OnValueCommitted(this, &SGraphNodeColorConstant::OnSpinBoxValueCommitted, ColorChannel::ALPHA)
				.Style(&WidgetStyle)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Delta(0.01f)
			]
		]
	];
}


void SGraphNodeColorConstant::OnSpinBoxValueChanged(float Value, ColorChannel channel)
{
	switch (channel)
	{
	case ColorChannel::RED:
		NodeColorConstant->Value.R = Value;
		break;
	case ColorChannel::GREEN:
		NodeColorConstant->Value.G = Value;
		break;
	case ColorChannel::BLUE:
		NodeColorConstant->Value.B = Value;
		break;
	case ColorChannel::ALPHA:
		NodeColorConstant->Value.A = Value;
		break;
	default:
		break;
	}

	NodeColorConstant->GetCustomizableObjectGraph()->GetOuter()->MarkPackageDirty();
}


void SGraphNodeColorConstant::OnSpinBoxValueCommitted(float Value, ETextCommit::Type, ColorChannel channel)
{
	switch (channel)
	{
	case ColorChannel::RED:
		NodeColorConstant->Value.R = Value;
		break;
	case ColorChannel::GREEN:
		NodeColorConstant->Value.G = Value;
		break;
	case ColorChannel::BLUE:
		NodeColorConstant->Value.B = Value;
		break;
	case ColorChannel::ALPHA:
		NodeColorConstant->Value.A = Value;
		break;
	default:
		break;
	}

	NodeColorConstant->GetCustomizableObjectGraph()->GetOuter()->MarkPackageDirty();
}


FReply SGraphNodeColorConstant::OnColorPreviewClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (ColorEditor->GetVisibility() == EVisibility::Collapsed)
		{
			ColorEditor->SetVisibility(EVisibility::Visible);
		}
		else
		{
			ColorEditor->SetVisibility(EVisibility::Collapsed);
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
