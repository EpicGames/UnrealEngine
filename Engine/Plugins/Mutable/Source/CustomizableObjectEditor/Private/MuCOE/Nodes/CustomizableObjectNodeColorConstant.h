// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CustomizableObjectNodeColorConstant.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class SOverlay;
class SVerticalBox;
class UCustomizableObjectNodeColorConstant;
class UCustomizableObjectNodeRemapPins;
class UObject;
struct FGeometry;
struct FPointerEvent;
struct FPropertyChangedEvent;
struct FSlateBrush;


enum class ColorChannel
{
	RED,
	GREEN,
	BLUE,
	ALPHA
};

// Class create color inputs in the NodeColorConstant
class SGraphNodeColorConstant : public SCustomizableObjectNode
{
public:

	SLATE_BEGIN_ARGS(SGraphNodeColorConstant) {}
	SLATE_END_ARGS();

	SGraphNodeColorConstant() : SCustomizableObjectNode() {}

	// Builds the SGraphNodeFloatConstant when needed
	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// Overriden functions to build the SGraphNode widgets
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	// Callbacks for the widget
	void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsExpressionPreviewChecked() const;
	const FSlateBrush* GetExpressionPreviewArrow() const;

private:

	// Callback for the OnValueChanged of the SpinBox
	void OnSpinBoxValueChanged(float Value, ColorChannel channel);

	// Callback for the OnValueCommited of the SpinBox
	void OnSpinBoxValueCommitted(float Value, ETextCommit::Type, ColorChannel channel);

	// Callback for the OnClicked of the ColorBox
	FReply OnColorPreviewClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

private:

	// Pointer to the NodeFloatconstant that owns this SGraphNode
	UCustomizableObjectNodeColorConstant* NodeColorConstant = nullptr;

	// Style for the SpinBox
	FSpinBoxStyle WidgetStyle;

	// Pointer to store the color editors to manage the visibility
	TSharedPtr<SVerticalBox> ColorEditor;
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeColorConstant : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	// End EdGraphNode interface

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool IsAffectedByLOD() const override { return false; }
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Creates the SGraph Node widget for the Color Editor
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	UPROPERTY(EditAnywhere, Category=CustomizableObject, meta = (DontUpdateWhileEditing))
	FLinearColor Value = FLinearColor(1,1,1,1);

	// Determines if the Node is collapsed or not
	bool bCollapsed = true;

};

#undef UE_API
