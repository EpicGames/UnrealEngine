// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeColorFromFloats.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeColorFromFloats)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeColorFromFloats::UCustomizableObjectNodeColorFromFloats()
	: Super()
{

}


void UCustomizableObjectNodeColorFromFloats::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeColorFromFloats::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("R"), LOCTEXT("R", "R"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("G"), LOCTEXT("G", "G"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("B"), LOCTEXT("B", "B"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("A"), LOCTEXT("A", "A"));
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Color, TEXT("Color"), LOCTEXT("Color", "Color"));
}


FText UCustomizableObjectNodeColorFromFloats::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Color_From_Floats", "Color From Floats");
}


FLinearColor UCustomizableObjectNodeColorFromFloats::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Color);
}


FText UCustomizableObjectNodeColorFromFloats::GetTooltipText() const
{
	return LOCTEXT("Color_From_Floats_Tooltip", "Defines a color from its four RGBA components.");
}


#undef LOCTEXT_NAMESPACE

