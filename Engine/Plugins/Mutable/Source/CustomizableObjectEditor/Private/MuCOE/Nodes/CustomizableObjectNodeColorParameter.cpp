// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeColorParameter.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeColorParameter)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeColorParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Color");
			Pin->PinFriendlyName = LOCTEXT("Color_Pin_Category", "Color");;
		}
	}
}


FName UCustomizableObjectNodeColorParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Color;
}

#undef LOCTEXT_NAMESPACE

