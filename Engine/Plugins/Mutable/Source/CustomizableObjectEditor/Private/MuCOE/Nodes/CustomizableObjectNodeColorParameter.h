// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "CustomizableObjectNodeColorParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeColorParameter : public UCustomizableObjectNodeParameter
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=CustomizableObject, meta = (DontUpdateWhileEditing))
	FLinearColor DefaultValue = FLinearColor(1, 1, 1, 1);

	// Begin EdGraphNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;
};

#undef UE_API
