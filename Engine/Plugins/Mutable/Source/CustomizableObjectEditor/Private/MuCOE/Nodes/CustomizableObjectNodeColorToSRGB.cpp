// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeColorToSRGB.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeColorToSRGB)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeColorToSRGB::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	InputPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Color, TEXT("Linear"), LOCTEXT("Linear", "Linear"));
	OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Color, TEXT("sRGB"), LOCTEXT("sRGB", "sRGB"));
}


FText UCustomizableObjectNodeColorToSRGB::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("LinearColor_To_sRGB", "Color Linear To sRGB");
}


FLinearColor UCustomizableObjectNodeColorToSRGB::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Color);
}


FText UCustomizableObjectNodeColorToSRGB::GetTooltipText() const
{
	return LOCTEXT("Color_TO_SRGB_Tooltip", "Converts a linear color to sRGB.");
}


UEdGraphPin* UCustomizableObjectNodeColorToSRGB::GetInputPin() const
{
	return InputPin.Get();
}


UEdGraphPin* UCustomizableObjectNodeColorToSRGB::GetOutputPin() const
{
	return OutputPin.Get();
}


#undef LOCTEXT_NAMESPACE

