// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeColorToSRGB.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;

UCLASS(MinimalAPI)
class UCustomizableObjectNodeColorToSRGB : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;
	bool ShouldOverridePinNames() const override { return true; }

	// UCustomizableObjectNode interface
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	virtual bool IsAffectedByLOD() const { return false; }

	// Own interface
	UEdGraphPin* GetInputPin() const;
	UEdGraphPin* GetOutputPin() const;

private:

	UPROPERTY()
	FEdGraphPinReference InputPin;

	UPROPERTY()
	FEdGraphPinReference OutputPin;
};

#undef UE_API
