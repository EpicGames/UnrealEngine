// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"

#include "CustomizableObjectNodeColorVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

struct FCustomizableObjectColorVariation;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeColorVariation : public UCustomizableObjectNodeVariation
{
public:
	GENERATED_BODY()

	// Deprecated properties
	UPROPERTY()
	TArray<FCustomizableObjectColorVariation> Variations_DEPRECATED;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	// UCustomizableObjectNodeVariation interface
	UE_API virtual FName GetCategory() const override;
};

#undef UE_API
