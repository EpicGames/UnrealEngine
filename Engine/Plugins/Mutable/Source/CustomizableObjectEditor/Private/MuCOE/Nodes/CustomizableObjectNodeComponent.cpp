// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeComponent.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeComponent)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCustomizableObjectNodeComponent::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"), LOCTEXT("Component", "Component"));
	ComponentNamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
}


FLinearColor UCustomizableObjectNodeComponent::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Component);
}


bool UCustomizableObjectNodeComponent::IsAffectedByLOD() const
{ 
	return false; 
}


void UCustomizableObjectNodeComponent::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!ComponentNamePin.Get())
		{
			ComponentNamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
		}
	}
}


bool UCustomizableObjectNodeComponent::ShouldPinViewerShowPinEditableName(const UEdGraphPin& Pin) const
{
	// Prevent the PinViewer from showing the internal PinEditableName if the pin has been connected (the name is externally driven by another node)
	const UEdGraphPin* ComponentPin = GetComponentNamePin();
	if (ComponentPin && ComponentPin->PinId == Pin.PinId)
	{
		return ComponentPin->LinkedTo.IsEmpty() && Super::ShouldPinViewerShowPinEditableName(Pin);
	}
	
	return Super::ShouldPinViewerShowPinEditableName(Pin);
}


bool UCustomizableObjectNodeComponent::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCustomizableObjectNodeComponent::GetPinEditableName(const UEdGraphPin& Pin) const
{
	const UEdGraphPin* ComponentPin = GetComponentNamePin();
	if (ComponentPin && ComponentPin->PinId == Pin.PinId)
	{
		return FText::FromName(GetComponentName());
	}

	return FText::GetEmpty();
}


void UCustomizableObjectNodeComponent::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	const UEdGraphPin* ComponentPin = GetComponentNamePin();
	if (ComponentPin && ComponentPin->PinId == Pin.PinId)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeComponent_SetComponentPinEditableNameTransaction", "Change ComponentName PinName"));
		Modify();
		
		SetComponentName(FName(*InValue.ToString()));
		PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
	}
	else
	{
		Super::SetPinEditableName(Pin, InValue);
	}
}


void UCustomizableObjectNodeComponent::OnRenameNode(const FString& NewName)
{
	if (!NewName.IsEmpty())
	{
		SetComponentName(FName(*NewName));
	}
}


void UCustomizableObjectNodeComponent::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == GetComponentNamePin())
	{
		GetGraph()->NotifyGraphChanged();
	}
}


FName UCustomizableObjectNodeComponent::GetComponentName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (const UEdGraphPin* NamePin = GetComponentNamePin())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*NamePin))
		{
			const UEdGraphPin* StringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, MacroContext);

			if (const UCustomizableObjectNodeStaticString* StringNode = StringPin ? Cast<UCustomizableObjectNodeStaticString>(StringPin->GetOwningNode()) : nullptr)
			{
				return FName(StringNode->Value);
			}
		}
	}

	return ComponentName;
}


void UCustomizableObjectNodeComponent::SetComponentName(const FName& InComponentName)
{
	ComponentName = InComponentName;
}


UEdGraphPin* UCustomizableObjectNodeComponent::GetComponentNamePin() const
{
	return ComponentNamePin.Get();
}

#undef LOCTEXT_NAMESPACE
