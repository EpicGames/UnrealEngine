// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeComponent.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UCustomizableObjectNodeMacroInstance;
class UEdGraphPin;


UCLASS(MinimalAPI, abstract)
class UCustomizableObjectNodeComponent : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;

	// UCustomizableObjectNode interface
	UE_API virtual bool IsAffectedByLOD() const override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual bool ShouldPinViewerShowPinEditableName(const UEdGraphPin& Pin) const override;
	UE_API virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	UE_API virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	UE_API virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;

	//Own Interface
	UE_API FName GetComponentName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;
	UE_API void SetComponentName(const FName& InComponentName);
	UE_API UEdGraphPin* GetComponentNamePin() const;

protected:
	UPROPERTY()
	FName ComponentName = "Default name"; // TODO GMT Should be a FString due to FNames not being case sensitive at runtime

public:
	UPROPERTY()
	FEdGraphPinReference OutputPin;

	UPROPERTY()
	FEdGraphPinReference ComponentNamePin;
};

#undef UE_API
