// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeComponentMesh.h"

#include "CONodeComponentSkeletalMesh.h"
#include "CONodeMaterialConstant.h"
#include "CONodeSkeletalMeshMake_V2.h"
#include "CONodeSkeletalMeshObjectMake.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectSchemaActions.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeComponentMesh)


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"



UCustomizableObjectNodeComponentMesh::UCustomizableObjectNodeComponentMesh()
{
	const FString CVarName = TEXT("r.SkeletalMesh.MinLodQualityLevel");
	const FString ScalabilitySectionName = TEXT("ViewDistanceQuality");
	LODSettings_DEPRECATED.MinQualityLevelLOD.SetQualityLevelCVarForCooking(*CVarName, *ScalabilitySectionName);
}


void UCustomizableObjectNodeComponentMesh::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ComponentsArray)
	{
		UCustomizableObject* Object = GraphTraversal::GetObject(*this);

		if (FMutableMeshComponentData* Result = Object->GetPrivate()->MutableMeshComponents_DEPRECATED.FindByPredicate([&](const FMutableMeshComponentData& ComponentData)
		{
			return ComponentData.Name == ComponentName;
		}))
		{
			ReferenceSkeletalMesh = Result->ReferenceSkeletalMesh;
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::StoreOverlayMaterialPinAsReference)
	{
		const FString PinFriendlyName = TEXT("Overlay Material");
		const FString PinName = PinFriendlyName + FString(TEXT("_Input_Pin"));

		OverlayMaterialPin = FindPin(PinName);
		if (!OverlayMaterialPin.Get())
		{
			OverlayMaterialPin = FindPin(PinFriendlyName);
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::RemovedOverlayMaterialUpropertyFromSkeletalMeshComponentNode)
	{
		UEdGraphPin* OverlayMaterialPinPtr = OverlayMaterialPin.Get();
		// Only add a new material constant node if a node is not already been used to determine the overlay material
		if (OverlayMaterialPinPtr && !OverlayMaterialPinPtr->LinkedTo.IsEmpty())
		{
			return;
		}
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!OverlayMaterial_DEPRECATED.IsNull())
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			UEdGraph* Graph = GetGraph();
			check(Graph)
			
			// Set node to the left of this node
			const FVector2D MaterialConstantNodePos = FVector2D(GetNodePosX() - 200, GetNodePosY());
			UCONodeMaterialConstant* MaterialConstantNode = Cast<UCONodeMaterialConstant>(FCustomizableObjectSchemaAction_NewNode::CreateNode(Graph, nullptr, MaterialConstantNodePos, Cast<UEdGraphNode>(NewObject<UCONodeMaterialConstant>())));
			
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			MaterialConstantNode->Material = OverlayMaterial_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// Connect the node with this one
			UEdGraphPin* MaterialConstantNodeMaterialPin = MaterialConstantNode->MaterialPin.Get();
			check(MaterialConstantNodeMaterialPin);
			
			MaterialConstantNodeMaterialPin->MakeLinkTo(OverlayMaterialPinPtr, false);
			Graph->NotifyNodeChanged(MaterialConstantNode);
			Graph->NotifyNodeChanged(this);
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SkeletalMeshMakeNodeDeprecations)
	{
		// This node LOD arrays no longer have a use for us as those are defined by the connected node to the Skeletal mesh pin
		// That pin does not yet exist

		// First of all ensure we have the new Skeletal Mesh Input pin by creating it
		check(!SkeletalMeshPin.Get());
		SkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
		
		// Spawn a new CONodeSkeletalMeshMake
		UEdGraph* Graph = GetGraph();
		check(Graph);
		
		constexpr float Separation = 400;
		const FVector2D DefaultSpawnPosition = FVector2D(GetNodePosX(), GetNodePosY());
		
		// Intermediate node between the Mutable Skm Make node and the component node (this)
		// Will always exist as now it is the node that holds the data that the component node was holding until now
		UCONodeSkeletalMeshObjectMake* MakeMeshNode = nullptr;
		{
			MakeMeshNode = Cast<UCONodeSkeletalMeshObjectMake>(FCustomizableObjectSchemaAction_NewNode::CreateNode(Graph, nullptr, DefaultSpawnPosition, Cast<UEdGraphNode>(NewObject<UCONodeSkeletalMeshObjectMake>())));
			MakeMeshNode->LODSettings = LODSettings_DEPRECATED;
			MakeMeshNode->NumLODs = NumLODs_DEPRECATED;
			MakeMeshNode->PassthroughSkeletalMeshPin.Get()->MakeLinkTo(SkeletalMeshPin.Get(), false);
		}
		
		// Move this to a side and leave the new node the position it has
		SetNodePosX( DefaultSpawnPosition.X + MakeMeshNode->GetWidth() + Separation);
		
		// First array is the LODIndex and the second one the pins connected to said LOD Index
		bool bIsAtLeastOneLODPinConnected = false;
		for (FEdGraphPinReference& LODArrayPin : LODPins_DEPRECATED)
		{
			check(LODArrayPin.Get());
			
			// The LOD Pin is connected to a series of sections. Keep them as they will later be connected to the new node (CONOdeSkeletalMeshMake)
			const TArray<UEdGraphPin*> LinkedToPins = LODArrayPin.Get()->LinkedTo;
			if (!LinkedToPins.IsEmpty())
			{
		 		bIsAtLeastOneLODPinConnected = true;
		 		break;
			}
		}
		
		// If at leat one of the LODs do have a connected pin then generate the mutable make node
		if (bIsAtLeastOneLODPinConnected)
		{
			// Set node to the left of this node
			// const FVector2D MakeMutableMeshNodePos = FVector2D(DefaultSpawnPosition.X - 400, GetNodePosY());
			UCONodeSkeletalMeshMake_V2* MakeMutableMeshNode = Cast<UCONodeSkeletalMeshMake_V2>(FCustomizableObjectSchemaAction_NewNode::CreateNode(Graph, nullptr, DefaultSpawnPosition, Cast<UEdGraphNode>(NewObject<UCONodeSkeletalMeshMake_V2>())));
			
			// Connect both nodes
			MakeMutableMeshNode->SkeletalMeshPin.Get()->MakeLinkTo(MakeMeshNode->SkeletalMeshPin.Get(), false);
			
			// Connect to this new node all the connections set in the current LODArrayPin and remove them from the current pin
			const uint16 DefaultAmountOfLodPins = MakeMutableMeshNode->LODPins.Num(); 
			for (uint16 LODIndex = DefaultAmountOfLodPins; LODIndex < NumLODs_DEPRECATED; ++LODIndex)
			{
				const FString LODName = FString::Printf(TEXT("LOD %d"), LODIndex);
				
				UEdGraphPin* LODPin = MakeMutableMeshNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
				LODPin->PinType.ContainerType = EPinContainerType::Array;
				
				MakeMutableMeshNode->LODPins.Add(LODPin);
			}

			check(MakeMutableMeshNode->LODPins.Num() == LODPins_DEPRECATED.Num());
			
			// At this point all lod pins have been generated so it should be ok to link them with the old data from this node
			for (uint16 LODIndex = 0; LODIndex < NumLODs_DEPRECATED; ++LODIndex)
			{
				UEdGraphPin* MakeNodeLODPin = MakeMutableMeshNode->LODPins[LODIndex].Get();
				UEdGraphPin* ThisNodeLODPin = LODPins_DEPRECATED[LODIndex].Get();

				// Se the sections for the make node and then reset the said connections from this node
				for (UEdGraphPin* LinkedToPin : ThisNodeLODPin->LinkedTo)
				{
					MakeNodeLODPin->MakeLinkTo(LinkedToPin, false);
				}
				
				ThisNodeLODPin->BreakAllPinLinks(false, false);
			}

			Graph->NotifyNodeChanged(MakeMutableMeshNode);
			
			// Move this to a side and leave the new node the position it has
			SetNodePosX(DefaultSpawnPosition.X + Separation * 2);
			MakeMeshNode->SetNodePosX(DefaultSpawnPosition.X + Separation);
		}
		
		// Now remove the deprecated LODPins from this node
		for (FEdGraphPinReference LODPinRef : LODPins_DEPRECATED)
		{
			UEdGraphPin* LodPin = LODPinRef.Get();
			check(LodPin);
			CustomRemovePin(*LodPin);
		}
		
		Graph->NotifyNodeChanged(MakeMeshNode);
		Graph->NotifyNodeChanged(this);
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MergedSkeletalMeshComponentNodes)
	{
		// AddMissingOverlayPin
		{
			auto PinAllocationLambdaExpr = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* RemapPins) -> void
			{
				UCustomizableObjectNodeComponentMesh* CastedNode = CastChecked<UCustomizableObjectNodeComponentMesh>(Node);
				
				// From parent
				CastedNode->OutputPin = CastedNode->CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"), LOCTEXT("Component", "Component"));
				CastedNode->ComponentNamePin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
			
				// From this
				CastedNode->OverlayMaterialPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, TEXT("Overlay Material_Input_Pin"), LOCTEXT("OverlayMaterial", "Overlay Material"));
				CastedNode->OverlayMaterialPin.Get()->PinToolTip = "Pin for an Overlay Material from a Table Node"; 
				
				CastedNode->SkeletalMeshPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
			};
			
			FixupReconstructPins(CreateRemapPinsByName(), PinAllocationLambdaExpr);
		}

	
		// Create the node that will replace this current one 
		
		// Create the new node 
		const TObjectPtr<UCONodeComponentSkeletalMesh> NewComponentNode = 
			Cast<UCONodeComponentSkeletalMesh>(
				FCustomizableObjectSchemaAction_NewNode::CreateNode(
					GetGraph(), nullptr, FVector2D(GetNodePosX(), GetNodePosY()), Cast<UEdGraphNode>(NewObject<UCONodeComponentSkeletalMesh>())));
	
		// Fill it with our data
		NewComponentNode->ReferenceSkeletalMesh = ReferenceSkeletalMesh;
		NewComponentNode->SetComponentName(GetComponentName());
		
		// Create the new node with our data so we are sure it is in a valid state for it to get our connections populated
		{
			auto PinAllocationLambdaExpr = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* RemapPins) -> void
        	{
        		UCONodeComponentSkeletalMesh* CastedNode = CastChecked<UCONodeComponentSkeletalMesh>(Node);
        	
        		CastedNode->OutputPin = CastedNode->CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"),
        			LOCTEXT("Component", "Component"));
        		CastedNode->ComponentNamePin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"),
        			LOCTEXT("Name", "Name"));
        	
        		CastedNode->SkeletalMeshPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
        		CastedNode->OverlayMaterialPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material,
        			TEXT("Overlay Material"), LOCTEXT("OverlayMaterial", "Overlay Material"));
        	};
        	
        	NewComponentNode->FixupReconstructPins(CreateRemapPinsByName(), PinAllocationLambdaExpr);
		}
	
		auto RemapLinks = [](UEdGraphPin* FromPin, UEdGraphPin* ToPin)->void
		{
			check(FromPin);
			check(ToPin);
		
			if (FromPin->LinkedTo.IsValidIndex(0))
			{
				check(FromPin->LinkedTo.Num() == 1);
				ToPin->MakeLinkTo(FromPin->LinkedTo[0], false);
			
				FromPin->BreakAllPinLinks(true, false);
			}
		};
		
		// Output Pin
		RemapLinks( OutputPin.Get(), NewComponentNode->OutputPin.Get());
		
		// Name pin
		RemapLinks( ComponentNamePin.Get(), NewComponentNode->ComponentNamePin.Get());
		
		// Skeletal Mesh
		RemapLinks( SkeletalMeshPin.Get(), NewComponentNode->SkeletalMeshPin.Get());
		
		// Overlay Material
		RemapLinks( OverlayMaterialPin.Get(), NewComponentNode->OverlayMaterialPin.Get());
		
		// Now remove this node as is no longer of use
		GetGraph()->RemoveNode(this, true, false);
	}
}


#undef LOCTEXT_NAMESPACE
