// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CONodeSkeletalMeshObjectMake.h"
#include "CustomizableObjectNodeComponent.h"
#include "MuR/System.h"

#include "CustomizableObjectNodeComponentMesh.generated.h"

class UMaterialInterface;


UCLASS()
class UCustomizableObjectNodeComponentMesh : public UCustomizableObjectNodeComponent
{
	GENERATED_BODY()

public:
	
	UCustomizableObjectNodeComponentMesh();
	
	// UObject interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	/** All the Skeletal Meshes generated for this component will use the Reference Skeletal Mesh properties
	 * for everything Mutable doesn't create or modify. This includes data like LOD distances, Physics
	 * properties, Bounding Volumes, Base Skeleton, and more.
	 *
	 * The Reference Skeletal Mesh can be used as a placeholder mesh when there are too many actors or in 
	 * situations of stress where the generation of the Skeletal Mesh might take a few seconds to complete. */
	UPROPERTY(EditAnywhere, Category = ComponentMesh)
	TObjectPtr<USkeletalMesh> ReferenceSkeletalMesh;

	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> OverlayMaterial_DEPRECATED;

	UPROPERTY()
	FMutableLODSettings LODSettings_DEPRECATED;

	UPROPERTY()
	int32 NumLODs_DEPRECATED = 1;

	UPROPERTY()
	FEdGraphPinReference SkeletalMeshPin;
	
	UPROPERTY()
	TArray<FEdGraphPinReference> LODPins_DEPRECATED;

	UPROPERTY()
	FEdGraphPinReference OverlayMaterialPin;
};
