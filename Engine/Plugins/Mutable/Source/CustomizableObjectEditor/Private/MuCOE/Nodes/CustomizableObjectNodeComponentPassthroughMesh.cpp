// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeComponentPassthroughMesh.h"

#include "CONodeMaterialBreak.h"
#include "CONodeComponentSkeletalMesh.h"
#include "CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "ScopedTransaction.h"
#include "MuCOE/CustomizableObjectSchemaActions.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeComponentPassthroughMesh)

class UObject;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeComponentPassthroughMesh::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PinSkeletalMesh)
	{
		TArray<UEdGraphPin*> OldPins = Pins;
		for (UEdGraphPin* Pin : OldPins)
		{
			if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_MeshSection)
			{
				CustomRemovePin(*Pin);
			}
		}

		if (!SkeletalMeshPin.Get())
		{
			SkeletalMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PassthroughSkeletalMeshComponentOverrideSubtypeFix)
	{
		// Set the subcategory of the pins that do require one defined. We use the "Override" one as that is the behaviour the pins had before the option to make them able to be set as "overlay" was added.
		for (FEdGraphPinReference MaterialPinReference : MaterialSlotPins)
		{
			UEdGraphPin* MaterialPin = MaterialPinReference.Get();
			if (!MaterialPin)
			{
				checkNoEntry();
				continue;
			}
			
			MaterialPin->PinType.PinSubCategory = UEdGraphSchema_CustomizableObject::PSC_Material_Override;
			MaterialPin->PinFriendlyName = FText::FromString("Override Material");
			MaterialPin->PinName = FName("Override Material");
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PassthroughSkeletalMeshComponentComponentMaterialOverride)
	{
		// Add an overlay material pin after the PassthroughSkeletalMeshPin so it gets to be on top of the Material Override/Overlay pins.
		
		const UEdGraphPin* PassthroughSkeletalMeshPin = SkeletalMeshPin.Get();
		const int32 PassthroughMeshPinIndex = Pins.IndexOfByKey(PassthroughSkeletalMeshPin);
		
		check(PassthroughMeshPinIndex != INDEX_NONE);
		
		UEdGraphPin* NewMaterialOverlayPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, "Overlay Material" , FText::FromString("Overlay Material"));
		Pins.Remove(NewMaterialOverlayPin);
		Pins.Insert(NewMaterialOverlayPin, PassthroughMeshPinIndex + 1);
		
		OverlayMaterialPin = NewMaterialOverlayPin;
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixOverrideAndOverlayMaterialPinNames)
	{
		// Some nodes may show repeated Overlay Material pins. Remove the one not being referenced if found
		{
			const UEdGraphPin* OverlayMaterialPinPointer = OverlayMaterialPin.Get();
			check(OverlayMaterialPinPointer);
		
			// In some cases we may have more than one Material Overlay pin, in that case remove the one that is not being referenced by us
			const TArray<UEdGraphPin*> AllPins = GetAllNonOrphanPins();
			for (UEdGraphPin* ScannedPin : AllPins)
			{
				if (ScannedPin->Direction != OverlayMaterialPinPointer->Direction)
				{
					continue;
				}
			
				if (ScannedPin->PinType.PinCategory != OverlayMaterialPinPointer->PinType.PinCategory)
				{
					continue;
				}
			
				// Ignore the Material slot pins
				if (MaterialSlotPins.Contains(ScannedPin))
				{
					continue;
				}
			
				// Only remove it if it is not the one currently being referenced
				if (OverlayMaterialPin.Get() != ScannedPin)
				{
					CustomRemovePin(*ScannedPin);
				}
			}
		}
		
		// At this point, if more than one "Overlay Pin" was part of the node only the one being in use (referenced) will be keept
		
		auto PinAllocationLambdaExpr = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* RemapPins) -> void
		{
			UCustomizableObjectNodeComponentPassthroughMesh* CastedNode = CastChecked<UCustomizableObjectNodeComponentPassthroughMesh>(Node);
			
			CastedNode->OutputPin = CastedNode->CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"), LOCTEXT("Component", "Component"));
			CastedNode->ComponentNamePin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
			CastedNode->SkeletalMeshPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
			CastedNode->OverlayMaterialPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, "Overlay Material" , FText::FromString("Overlay Material"), nullptr);

			// Get old pins
			const TArray<UEdGraphPin*> AllPins = CastedNode->GetAllNonOrphanPins();

			// Regenerate the new pins from the old pins
			CastedNode->MaterialSlotPins.Reset();
			for (const UEdGraphPin* OldPin : AllPins)
			{
				check(OldPin)
	
				if (OldPin->Direction != EEdGraphPinDirection::EGPD_Input)
				{
					continue;
				}
	
				if (UCONodeComponentSkeletalMeshMaterialPinData* OldPinData = Cast<UCONodeComponentSkeletalMeshMaterialPinData>(CastedNode->GetPinData(*OldPin)))
				{
					UCONodeComponentSkeletalMeshMaterialPinData* PinData = NewObject<UCONodeComponentSkeletalMeshMaterialPinData>(CastedNode);
					PinData->TargetMaterialSlotName = OldPinData->TargetMaterialSlotName;
		
					// The SubCategory could have changed after a node refresh
					const FName OldPinSubCategory = OldPin->PinType.PinSubCategory;
		
					// Use the same pin data as the old pin since the new one is a copy
					const FName NewPinName =  TEXT("Overlay-or-Override");
					const FText NewPinFriendlyName =  FText::FromString( UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(OldPinSubCategory).ToString() + " Material");

					UEdGraphPin* NewMaterialPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, NewPinName, NewPinFriendlyName, OldPinData);
					NewMaterialPin->PinType.PinSubCategory = OldPinSubCategory;

					CastedNode->MaterialSlotPins.Add(NewMaterialPin);
				}
			}
		};
		
		FixupReconstructPins(NewObject<UCustomizableObjectNodeRemapPinsByPosition>(), PinAllocationLambdaExpr);
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MergedSkeletalMeshComponentNodes)
	{
		// Create the new node 
		const TObjectPtr<UCONodeComponentSkeletalMesh> NewComponentNode = 
			Cast<UCONodeComponentSkeletalMesh>(
				FCustomizableObjectSchemaAction_NewNode::CreateNode(
					GetGraph(), nullptr, FVector2D(GetNodePosX(), GetNodePosY()), Cast<UEdGraphNode>(NewObject<UCONodeComponentSkeletalMesh>())));
	
		// Fill it with our data
		NewComponentNode->SetComponentName(GetComponentName());

		// Create the new node with our data so we are sure it is in a valid state for it to get our connections populated
		{
			auto PinAllocationLambdaExpr = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* RemapPins) -> void
			{
				UCONodeComponentSkeletalMesh* CastedNode = CastChecked<UCONodeComponentSkeletalMesh>(Node);
        
				CastedNode->OutputPin = CastedNode->CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"),
					LOCTEXT("Component", "Component"));
				CastedNode->ComponentNamePin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"),
					LOCTEXT("Name", "Name"));
        
				CastedNode->SkeletalMeshPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
				CastedNode->OverlayMaterialPin = CastedNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material,
					TEXT("Overlay Material"), LOCTEXT("OverlayMaterial", "Overlay Material"));
			};
        	
			NewComponentNode->FixupReconstructPins(CreateRemapPinsByName(), PinAllocationLambdaExpr);
		}
		
		// Reference Skeletal Mesh
		{
			// If no reference skeletal mesh is set try to resolve it by accessing the reference skeletal mesh of the connected Skeletal Mesh Parameter node
			const UEdGraphPin* SkeletalMeshPinPointer = SkeletalMeshPin.Get();
			check(SkeletalMeshPinPointer);
	
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*SkeletalMeshPinPointer))
			{
				const UEdGraphNode* OtherNode = ConnectedPin->GetOwningNode();
				if (const UCustomizableObjectNodeSkeletalMeshParameter* ConnectedParameterNode = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(OtherNode))
				{
					NewComponentNode->ReferenceSkeletalMesh = ConnectedParameterNode->ReferenceValue.LoadSynchronous();
				}
			}
		}
		
		// Once the basic structure of the new node is set now add the material slot pins with their data 
		auto RemapLinks = [](UEdGraphPin* FromPin, UEdGraphPin* ToPin)->void
		{
			check(FromPin);
			check(ToPin);
	
			if (FromPin->LinkedTo.IsValidIndex(0))
			{
				check(FromPin->LinkedTo.Num() == 1);
				ToPin->MakeLinkTo(FromPin->LinkedTo[0], false);
		
				FromPin->BreakAllPinLinks(true, false);
			}
		};
		
		
		// Create new material slot pin nodes for the other node based on the nodes we have
		{
			for (FEdGraphPinReference& OurMaterialSlotPin : MaterialSlotPins)
			{
				const UEdGraphPin* OurMaterialSlotPinPointer = OurMaterialSlotPin.Get();
				check(OurMaterialSlotPinPointer);

				// Add this Material slot pin data to the new node alongside a new material pin
				{
					const FName PinSubCategory = OurMaterialSlotPinPointer->PinType.PinSubCategory;
					const FName NewPinName = TEXT("Overlay-or-Override");
					const FText NewPinFriendlyName = FText::FromString(
						UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(PinSubCategory).ToString() + " Material");

					UCONodeComponentSkeletalMeshMaterialPinData* OurMaterialSlotPinData = CastChecked<UCONodeComponentSkeletalMeshMaterialPinData>(GetPinData(*OurMaterialSlotPinPointer));
					UEdGraphPin* NewMaterialPin = NewComponentNode->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Material, NewPinName, NewPinFriendlyName,
						OurMaterialSlotPinData);
					NewMaterialPin->PinType.PinSubCategory = PinSubCategory;
					
					NewComponentNode->MaterialSlotPins.Add(NewMaterialPin);
				}
			}
			
			// Now proceed to remap the connections
			const uint32 MaterialSLotPinsCount = MaterialSlotPins.Num();
			for (uint32 MaterialSlotIndex = 0; MaterialSlotIndex < MaterialSLotPinsCount; ++MaterialSlotIndex)
			{
				UEdGraphPin* OurMaterialSlotPin = MaterialSlotPins[MaterialSlotIndex].Get();
				UEdGraphPin* NewNodeMaterialSlotPin = NewComponentNode->MaterialSlotPins[MaterialSlotIndex].Get();

				RemapLinks(OurMaterialSlotPin, NewNodeMaterialSlotPin);
			}
		}
		
		// Output Pin
		{
			UEdGraphPin* OutputPinPointer = OutputPin.Get();
			UEdGraphPin* NewNodeOutputPinPointer = NewComponentNode->OutputPin.Get();
			
			check(OutputPinPointer);
			check(NewNodeOutputPinPointer);

			const int32 LinkedPinCount = OutputPinPointer->LinkedTo.Num();
			for	(int32 LinkedPinIndex = 0; LinkedPinIndex < LinkedPinCount; LinkedPinIndex++)
			{
				UEdGraphPin* LinkedToPin =  OutputPinPointer->LinkedTo[LinkedPinIndex];
				NewNodeOutputPinPointer->MakeLinkTo(LinkedToPin, false);
				OutputPinPointer->BreakLinkTo(LinkedToPin, false);
			}
		}
		
		// Name pin
		RemapLinks( ComponentNamePin.Get(), NewComponentNode->ComponentNamePin.Get());
		
		// Skeletal Mesh
		RemapLinks( SkeletalMeshPin.Get(), NewComponentNode->SkeletalMeshPin.Get());
		
		// Overlay Material
		RemapLinks( OverlayMaterialPin.Get(), NewComponentNode->OverlayMaterialPin.Get());
		
		// Now remove this node as is no longer of use
		GetGraph()->RemoveNode(this, true, false);
	}
}


#undef LOCTEXT_NAMESPACE

