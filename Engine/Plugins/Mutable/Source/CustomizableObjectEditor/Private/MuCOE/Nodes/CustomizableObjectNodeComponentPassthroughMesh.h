// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeComponent.h"

#include "CustomizableObjectNodeComponentPassthroughMesh.generated.h"

class UCONodeComponentSkeletalMeshMaterialPinData;

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class FAssetThumbnail;
class FAssetThumbnailPool;
class FSkeletalMeshModel;
class ISinglePropertyView;
class UAnimInstance;
class UCustomizableObjectLayout;
class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeRemapPinsByName;
class UMaterialInterface;
class UObject;
class USkeletalMesh;
class UTexture2D;
struct FPropertyChangedEvent;
struct FSkeletalMaterial;
class UEdGraphPin;


UCLASS()
class UCustomizableObjectNodeComponentPassthroughMesh : public UCustomizableObjectNodeComponent
{
public:
	GENERATED_BODY()
	
	// UObject interface.
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	UPROPERTY()
	FEdGraphPinReference SkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference OverlayMaterialPin;

	UPROPERTY()
	TArray<FEdGraphPinReference> MaterialSlotPins;
};
