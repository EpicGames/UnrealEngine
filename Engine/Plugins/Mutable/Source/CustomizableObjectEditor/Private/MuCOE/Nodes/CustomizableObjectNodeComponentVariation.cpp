// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeComponentVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeComponentVariation)



FName UCustomizableObjectNodeComponentVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Component;
}


bool UCustomizableObjectNodeComponentVariation::IsInputPinArray() const
{
	return true;
}


bool UCustomizableObjectNodeComponentVariation::IsSingleOutputNode() const
{
	return true;
}
