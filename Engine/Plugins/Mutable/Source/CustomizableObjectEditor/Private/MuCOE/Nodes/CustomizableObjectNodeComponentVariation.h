// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"

#include "CustomizableObjectNodeComponentVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

struct FCustomizableObjectComponentVariation;


UENUM(BlueprintType)
enum class ECustomizableObjectNodeComponentVariationType : uint8
{
	Tag 		UMETA(DisplayName = "Tag"),
	State 		UMETA(DisplayName = "State"),
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeComponentVariation : public UCustomizableObjectNodeVariation
{
public:
	GENERATED_BODY()

	//UPROPERTY(EditAnywhere, Category = CustomizableObject)
	//ECustomizableObjectNodeComponentVariationType Type = ECustomizableObjectNodeComponentVariationType::Tag;

public:
	// UCustomizableObjectNode interface
	UE_API virtual bool IsSingleOutputNode() const override;
	
	// UCustomizableObjectNodeVariation interface
	UE_API virtual FName GetCategory() const override;
	UE_API virtual bool IsInputPinArray() const override;
};

#undef UE_API
