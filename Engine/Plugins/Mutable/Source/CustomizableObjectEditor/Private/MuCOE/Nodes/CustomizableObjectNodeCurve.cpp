// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeCurve.h"

#include "Curves/CurveBase.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeCurve)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeCurve::UCustomizableObjectNodeCurve()
	: Super()
{

}


void UCustomizableObjectNodeCurve::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("CurveAsset"))
	{
		ReconstructNode();
	}
}


void UCustomizableObjectNodeCurve::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PostLoadToCustomVersion
		&& CurveAsset)
	{
		TArray<FRichCurveEditInfo> Curves = CurveAsset->GetCurves();

		TArray<UEdGraphPin*> OutputPins;
		for (int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
		{
			if (Pins[PinIndex]->Direction == EGPD_Output)
			{
				OutputPins.Add(Pins[PinIndex]);
			}
		}
		
		for (int32 i = 0; i < Curves.Num(); ++i)
		{
			if (UEdGraphPin* CurvePin = FindPin(FString::Printf(TEXT("Curve %d"), i)))
			{
				FString NewPinName = *(FString("Curve ") + Curves[i].CurveName.ToString());
				CurvePin->PinName = FName(NewPinName);
				CurvePin->Modify(true);
			}
			else if (OutputPins.IsValidIndex(i) && OutputPins[i] && Helper_GetPinName(OutputPins[i]) != (FString("Curve ") + Curves[i].CurveName.ToString()))
			{
				CurvePin = OutputPins[i];
				FString NewPinName = *(FString("Curve ") + Curves[i].CurveName.ToString());
				CurvePin->PinName = FName(NewPinName);
				CurvePin->Modify(true);
			}
		}
	}
}


void UCustomizableObjectNodeCurve::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	FString RemapIndex = Helper_GetPinName(InputPin());

	CustomCreatePin(EGPD_Input, Schema->PC_Float, TEXT("Input"), LOCTEXT("Input", "Input"));
	
	if (CurveAsset)
	{
		TArray<FRichCurveEditInfo> Curves = CurveAsset->GetCurves();

		for (int32 i = 0; i < Curves.Num(); ++i)
		{
			FString PinName = *(FString("Curve ") + Curves[i].CurveName.ToString());
			CustomCreatePin(EGPD_Output, Schema->PC_Float, FName(*PinName), FText::FromString(PinName));
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeCurve::InputPin() const
{
	return FindPin(TEXT("Input"));
}


UEdGraphPin* UCustomizableObjectNodeCurve::CurvePins(int32 Index) const
{
	if (CurveAsset && Index >= 0)
	{
		TArray<FRichCurveEditInfo> Curves = CurveAsset->GetCurves();

		if(Index < Curves.Num())
		{
			return FindPin(FString("Curve ") + Curves[Index].CurveName.ToString());
		}
	}
	return nullptr;
}


int32 UCustomizableObjectNodeCurve::GetNumCurvePins() const
{
	int32 NumCurvePins = 0;

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->GetName().StartsWith(TEXT("Curve ")))
		{
			NumCurvePins++;
		}
	}

	return NumCurvePins;
}


FText UCustomizableObjectNodeCurve::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Curve_Node", "Float Sample Curve");
}


FLinearColor UCustomizableObjectNodeCurve::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Float);
}


bool UCustomizableObjectNodeCurve::IsPinRelevant(const UEdGraphPin* Pin) const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	return Pin->PinType.PinCategory == Schema->PC_Float;
}


FText UCustomizableObjectNodeCurve::GetTooltipText() const
{
	return LOCTEXT("Curve_Tooltip", "Get the values from curve asset channels at a sample point.");
}


UCustomizableObjectNodeRemapPins* UCustomizableObjectNodeCurve::CreateRemapPinsDefault() const
{
	return CreateRemapPinsByPosition();
}


#undef LOCTEXT_NAMESPACE

