// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeCurve.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCurveBase;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeCurve : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeCurve();

	// UObject interface.
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	UE_API UEdGraphPin* InputPin() const;
	UE_API UEdGraphPin* CurvePins(int32 Index) const;
	UE_API int32 GetNumCurvePins() const;

	UPROPERTY(EditAnywhere, Category = Curve)
	TObjectPtr<UCurveBase> CurveAsset;

	// UCustomizableObjectNode interface
	virtual bool IsAffectedByLOD() const override { return false; }

	bool ProvidesCustomPinRelevancyTest() const override { return true; }
	UE_API bool IsPinRelevant(const UEdGraphPin* Pin) const override;

	/** Override the default behaivour of remap pins. Use remap pins by position by default. */
	UE_API UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
};

#undef UE_API
