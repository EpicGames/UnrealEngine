// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/PinViewer/SPinViewer.h"

class IDetailLayoutBuilder;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeDetails);
}


void FCustomizableObjectNodeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		if (UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(DetailsView->GetSelectedObjects()[0]))
		{
			if (!Node->HasPinViewer())
			{
				return;
			}
			
			IDetailCategoryBuilder& PinViewerCategoryBuilder = DetailBuilder.EditCategory("PinViewer", LOCTEXT("PinViewer", "Pins"), ECategoryPriority::Uncommon);
			PinViewerCategoryBuilder.AddCustomRow(LOCTEXT("PinViewerDetailsCategory", "PinViewer")).ShouldAutoExpand(true)
			[
				SNew(SPinViewer).Node(Node)
			];
		}
	}
}


// Only implemented to silence -Woverloaded-virtual warning.
void FCustomizableObjectNodeDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	IDetailCustomization::CustomizeDetails(DetailBuilder);

	DetailBuilderPtr = DetailBuilder;
}


void FCustomizableObjectNodeDetails::RefreshDetails()
{
	TSharedPtr<IDetailLayoutBuilder> DetailBuilder = DetailBuilderPtr.Pin();
	if (DetailBuilder.IsValid())
	{
		DetailBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
