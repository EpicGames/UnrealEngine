// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FReply;
class IDetailLayoutBuilder;
class UCustomizableObjectNode;


/** Base of all UCustomizableObjectNode details. */
class FCustomizableObjectNodeDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

protected:

	/** Force the refreshing of this details view. Only use if you are sure this will not pose a performance issue. */
	void RefreshDetails();
	
private:
	
	// Pointer to the DetailBuilder.
	TWeakPtr<IDetailLayoutBuilder> DetailBuilderPtr;
};
