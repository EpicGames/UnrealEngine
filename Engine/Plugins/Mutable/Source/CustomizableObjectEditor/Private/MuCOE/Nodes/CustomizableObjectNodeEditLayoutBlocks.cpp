// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditLayoutBlocks.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeModifierEditLayoutBlocks::UCustomizableObjectNodeModifierEditLayoutBlocks()
	: Super()
{
	Layout = CreateDefaultSubobject<UCustomizableObjectLayout>(FName("CustomizableObjectLayout"));
	Layout->Blocks.Empty();
}

#undef LOCTEXT_NAMESPACE

