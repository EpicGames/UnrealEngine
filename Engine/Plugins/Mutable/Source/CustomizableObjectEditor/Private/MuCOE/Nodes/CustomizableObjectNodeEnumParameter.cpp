// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeEnumParameter)


class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeEnumParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if ( PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("Values") )
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeEnumParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Enum");
			Pin->PinFriendlyName = LOCTEXT("Enum_Pin_Category", "Enum");
		}
	}
}


FText UCustomizableObjectNodeEnumParameter::GetTooltipText() const
{
	return LOCTEXT("Enum_Parameter_Tooltip",
		"Exposes and defines a parameter offering multiple choices to modify the Customizable Object.\nAlso defines a default one among them. \nIt's abstract, does not define what type those options refer to.");
}


FName UCustomizableObjectNodeEnumParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Enum;
}

#undef LOCTEXT_NAMESPACE

