// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "CustomizableObjectNodeEnumParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UObject;
struct FPropertyChangedEvent;


USTRUCT()
struct FCustomizableObjectNodeEnumValue
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	FString Name;

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "Parameter UI Metadata"))
	FMutableParamUIMetadata ParamUIMetadata;
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeEnumParameter : public UCustomizableObjectNodeParameter
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	int32 DefaultIndex = 0;

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TArray<FCustomizableObjectNodeEnumValue> Values;


	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Begin EdGraphNode interface
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;

};

#undef UE_API
