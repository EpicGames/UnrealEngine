// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeExposePin.h"

#include "ObjectEditorUtils.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeExposePin)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeExposePin::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetName() == GET_MEMBER_NAME_STRING_CHECKED(UCustomizableObjectNodeExposePin, Name))
	{
		OnNameChangedDelegate.Broadcast();
	}
}


void UCustomizableObjectNodeExposePin::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	UEdGraphPin* InputPin = CustomCreatePin(EGPD_Input, PinType);
	const bool bIsArrayPinCategory = PinType == UEdGraphSchema_CustomizableObject::PC_GroupProjector;
	if (bIsArrayPinCategory)
	{
		InputPin->PinType.ContainerType = EPinContainerType::Array;
	}
}


bool UCustomizableObjectNodeExposePin::IsNodeSupportedInMacros() const
{
	return false;
}


FText UCustomizableObjectNodeExposePin::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeTitle;
	FFormatNamedArguments Args;
	Args.Add(TEXT("NodeName"), FText::FromString(Name));
	Args.Add(TEXT("PinType"), UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(PinType));

	if (TitleType == ENodeTitleType::ListView)
	{
		NodeTitle = LOCTEXT("Expose_Pin_Title_ListView", "{PinType} Export");
	}
	else if (TitleType == ENodeTitleType::EditableTitle)
	{
		NodeTitle = LOCTEXT("Expose_Pin_Title_Edit", "{NodeName}");
	}
	else
	{
		NodeTitle = LOCTEXT("Expose_Pin_Title", "{NodeName}\n{PinType} Export");
	}

	return FText::Format(NodeTitle, Args);
}


FLinearColor UCustomizableObjectNodeExposePin::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(PinType);
}


FText UCustomizableObjectNodeExposePin::GetTooltipText() const
{
	return LOCTEXT("Expose_Pin_Tooltip", "Exposes a value to the rest of its Customizable Object hierarchy.");
}


void UCustomizableObjectNodeExposePin::OnRenameNode(const FString& NewName)
{
	if (!NewName.IsEmpty())
	{
		FObjectEditorUtils::SetPropertyValue(this, GET_MEMBER_NAME_STRING_CHECKED(UCustomizableObjectNodeExposePin, Name), NewName);
	}
}


void UCustomizableObjectNodeExposePin::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	// Rename the Pin named "Object" with the friendly name that represents its category : "Color", "Transform" , "Enum"...
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Object")))
		{
			// Set the name of the pin based on the type of pin it is (extracted from UEdGraphSchema_CustomizableObject::GetPinCategoryName and
			// UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName to keep this fixup immutable)
			if (PinType == UEdGraphSchema_CustomizableObject::PC_Object)
			{
				Pin->PinName = TEXT("Object");
				Pin->PinFriendlyName = LOCTEXT("Object_Pin_Category", "Object");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Component)
			{
				Pin->PinName = TEXT("Component");
				Pin->PinFriendlyName = LOCTEXT("Component_Pin_Category", "Component");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_MeshSection)
			{
				Pin->PinName = TEXT("Material");
				Pin->PinFriendlyName = LOCTEXT("MeshSection_Pin_Category", "Mesh Section");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Modifier)
			{
				Pin->PinName = TEXT("Modifier");
				Pin->PinFriendlyName = LOCTEXT("Modifier_Pin_Category", "Modifier");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Mesh)
			{
				Pin->PinName = TEXT("Mesh");
				Pin->PinFriendlyName = LOCTEXT("Mesh_Pin_Category", "Mesh");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Texture)
			{
				Pin->PinName = TEXT("Texture");
				Pin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
			{
				Pin->PinName = TEXT("PassThrough Texture");
				Pin->PinFriendlyName = LOCTEXT("PassThrough_Image_Pin_Category", "PassThrough Texture");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Projector)
			{
				Pin->PinName = TEXT("Projector");
				Pin->PinFriendlyName = LOCTEXT("Projector_Pin_Category", "Projector");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_GroupProjector)
			{
				Pin->PinName = TEXT("Group Projector");
				Pin->PinFriendlyName = LOCTEXT("Group_Projector_Pin_Category", "Group Projector");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Color)
			{
				Pin->PinName = TEXT("Color");
				Pin->PinFriendlyName = LOCTEXT("Color_Pin_Category", "Color");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Float)
			{
				Pin->PinName = TEXT("Float");
				Pin->PinFriendlyName = LOCTEXT("Float_Pin_Category", "Float");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Bool)
			{
				Pin->PinName = TEXT("Bool");
				Pin->PinFriendlyName = LOCTEXT("Bool_Pin_Category", "Bool");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Enum)
			{
				Pin->PinName = TEXT("Enum");
				Pin->PinFriendlyName = LOCTEXT("Enum_Pin_Category", "Enum");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Stack)
			{
				Pin->PinName = TEXT("Stack");
				Pin->PinFriendlyName = LOCTEXT("Stack_Pin_Category", "Stack");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Material)
			{
				Pin->PinName = TEXT("Material");
				Pin->PinFriendlyName = LOCTEXT("Material_Asset_Pin_Category", "Material");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Wildcard)
			{
				Pin->PinName = TEXT("Wildcard");
				Pin->PinFriendlyName = LOCTEXT("Wildcard_Pin_Category", "Wildcard");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_PoseAsset)
			{
				Pin->PinName = TEXT("PoseAsset");
				Pin->PinFriendlyName = LOCTEXT("Pose_Pin_Category", "PoseAsset");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Transform)
			{
				Pin->PinName = TEXT("Transform");
				Pin->PinFriendlyName = LOCTEXT("Transform_Pin_Category", "Transform");
			}
			else
			{
				bool bPinTypeWasFound = false;
				for (const FRegisteredCustomizableObjectPinType& RegisteredCustomizableObjectPin : ICustomizableObjectModule::Get().GetExtendedPinTypes())
				{
					if (RegisteredCustomizableObjectPin.PinType.Name == PinType)
					{
						Pin->PinName = RegisteredCustomizableObjectPin.PinType.Name;
						Pin->PinFriendlyName = RegisteredCustomizableObjectPin.PinType.DisplayName;
						bPinTypeWasFound = true;
						break;
					}
				}

				// Need to fail gracefully here in case a plugin that was active when this graph was
				// created is no longer loaded.
				if (!bPinTypeWasFound)
				{
					Pin->PinName = TEXT("Unknown");
					Pin->PinFriendlyName =  LOCTEXT("Unknown_Pin_Category", "Unknown");
				}
			}
		}
	}
}


bool UCustomizableObjectNodeExposePin::CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const
{
	// Check the pin types do match
	bOutArePinsCompatible = Super::CanConnect(InOwnedInputPin, InOutputPin, bOutIsOtherNodeBlocklisted, bOutArePinsCompatible);

	// Check the type of the other node to make sure it is not one we do not want to allow the connection with
	bOutIsOtherNodeBlocklisted = Cast<UCustomizableObjectNodeExternalPin>(InOutputPin->GetOwningNode()) != nullptr;

	return bOutArePinsCompatible && !bOutIsOtherNodeBlocklisted;
}


FString UCustomizableObjectNodeExposePin::GetNodeName() const
{
	return Name;
}


#undef LOCTEXT_NAMESPACE
