// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeExposePin.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


DECLARE_MULTICAST_DELEGATE(FOnNameChangedDelegate);


/** Export Node. */
UCLASS()
class UCustomizableObjectNodeExposePin : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual bool GetCanRenameNode() const override { return true; }
	virtual void OnRenameNode(const FString& NewName) override;

	// UCustomizableObjectNode interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual bool CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const override;
	virtual bool IsNodeSupportedInMacros() const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Return the Expose Pin Node expose pin name. */
	FString GetNodeName() const;
	
protected:
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

public:
	
	// This is actually PinCategory
	UPROPERTY()
	FName PinType;
	
	UEdGraphPin* InputPin() const
	{
		const FName InputPinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(PinType);
		return FindPin(InputPinName);
	}

	/** Will be broadcast when the UPROPERTY Name changes. */
	FOnNameChangedDelegate OnNameChangedDelegate;
	
private:
	UPROPERTY(Category = CustomizableObject, EditAnywhere)
	FString Name = "Default Name";
};

#undef UE_API
