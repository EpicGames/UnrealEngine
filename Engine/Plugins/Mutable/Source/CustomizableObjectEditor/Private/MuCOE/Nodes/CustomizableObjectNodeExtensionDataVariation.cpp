// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeExtensionDataVariation.h"

#include "MuCOE/ExtensionDataCompilerInterface.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuT/NodeExtensionDataVariation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeExtensionDataVariation)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCustomizableObjectNodeExtensionDataVariation::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (FProperty* PropertyThatChanged = InPropertyChangedEvent.Property)
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(InPropertyChangedEvent);
}

FLinearColor UCustomizableObjectNodeExtensionDataVariation::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(GetCategory());
}

bool UCustomizableObjectNodeExtensionDataVariation::IsAffectedByLOD() const
{
	return false;
}

bool UCustomizableObjectNodeExtensionDataVariation::ShouldAddToContextMenu(FText& OutCategory) const
{
	OutCategory = FText::FromName(GetCategory());
	return true;
}

void UCustomizableObjectNodeExtensionDataVariation::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* InRemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	CustomCreatePin(EGPD_Output, GetCategory(), GetOutputPinName(), FText::FromName(GetDefaultPinName()));

	for (int32 VariationIndex = Variations.Num() - 1; VariationIndex >= 0; --VariationIndex)
	{
		FText FriendlyName = FText::Format(LOCTEXT("VariationPinFriendlyName", "Variation {0} [{1}]"), { VariationIndex, FText::FromString(Variations[VariationIndex].Tag) });
		CustomCreatePin(EGPD_Input, GetCategory(), GetVariationPinName(VariationIndex), FriendlyName);
	}

	CustomCreatePin(EGPD_Input, GetCategory(), GetDefaultPinName(), FText::FromName(GetDefaultPinName()));
}

UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> UCustomizableObjectNodeExtensionDataVariation::GenerateMutableNode(FExtensionDataCompilerInterface& InCompilerInterface) const
{
	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionDataVariation> VariationNode = new UE::Mutable::Private::NodeExtensionDataVariation;

	if (const UEdGraphPin* ConnectedPin = FollowInputPin(*GetDefaultPin()))
	{
		if (const ICustomizableObjectExtensionNode* ExtensionNode = Cast<ICustomizableObjectExtensionNode>(ConnectedPin->GetOwningNode()))
		{
			if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> DefaultValueExtensionData = ExtensionNode->GenerateMutableNode(InCompilerInterface))
			{
				VariationNode->SetDefaultValue(DefaultValueExtensionData);
			}
			else
			{
				InCompilerInterface.CompilerLog(LOCTEXT("ExtensionDataVariationFailed", "Extension Data Variation failed"), this);
			}
		}
	}

	VariationNode->SetVariationCount(Variations.Num());
	for (int32 VariationIndex = 0; VariationIndex < Variations.Num(); ++VariationIndex)
	{
		if (const UEdGraphPin* VariationPin = GetVariationPin(VariationIndex))
		{
			VariationNode->SetVariationTag(VariationIndex, Variations[VariationIndex].Tag);
			if (const UEdGraphPin* ConnectedPin = FollowInputPin(*VariationPin))
			{
				if (const ICustomizableObjectExtensionNode* ExtensionNode = Cast<ICustomizableObjectExtensionNode>(ConnectedPin->GetOwningNode()))
				{
					if (UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> VariationExtensionData = ExtensionNode->GenerateMutableNode(InCompilerInterface))
					{
						VariationNode->SetVariationValue(VariationIndex, VariationExtensionData);
					}
				}
			}
		}
	}

	InCompilerInterface.AddGeneratedNode(this);

	return VariationNode;
}

FName UCustomizableObjectNodeExtensionDataVariation::GetDefaultPinName() const
{
	return TEXT("Default");
}

FName UCustomizableObjectNodeExtensionDataVariation::GetVariationPinName(int32 InIndex) const
{
	return FName{ *FString::Format(TEXT("Variation {0}"), { InIndex }) };
}

UEdGraphPin* UCustomizableObjectNodeExtensionDataVariation::GetDefaultPin() const
{
	return FindPin(GetDefaultPinName());
}

UEdGraphPin* UCustomizableObjectNodeExtensionDataVariation::GetVariationPin(int32 InIndex) const
{
	return FindPin(GetVariationPinName(InIndex));
}

int32 UCustomizableObjectNodeExtensionDataVariation::GetNumVariations() const
{
	return Variations.Num();
}

#undef LOCTEXT_NAMESPACE
