// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/ICustomizableObjectExtensionNode.h"

#include "CustomizableObjectNodeExtensionDataVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

USTRUCT()
struct FCustomizableObjectExtensionDataVariation
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "CustomizableObject")
	FString Tag;
};

UCLASS(MinimalAPI, abstract)
class UCustomizableObjectNodeExtensionDataVariation
	: public UCustomizableObjectNode
	, public ICustomizableObjectExtensionNode
{
	GENERATED_BODY()

public:

	//~Begin UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~End UObject interface

	//~Begin UEdGraphNode interface
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	//~End UEdGraphNode interface

	//~Begin UCustomizableObjectNode interface
	UE_API virtual bool IsAffectedByLOD() const override;
	UE_API virtual bool ShouldAddToContextMenu(FText& OutCategory) const override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* InRemapPins) override;
	//~End UCustomizableObjectNode interface

	//~Begin ICustomizableObjectExtensionNode interface
	UE_API virtual UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> GenerateMutableNode(FExtensionDataCompilerInterface& InCompilerInterface) const override;
	//~End ICustomizableObjectExtensionNode interface

	virtual FName GetCategory() const PURE_VIRTUAL(GetCategory(), return FName();)
	virtual FName GetOutputPinName() const PURE_VIRTUAL(GetOutputPinName(), return FName();)

	UE_API FName GetDefaultPinName() const;
	UE_API FName GetVariationPinName(int32 InIndex) const;
	UE_API class UEdGraphPin* GetDefaultPin() const;
	UE_API class UEdGraphPin* GetVariationPin(int32 InIndex) const;

	UE_API int32 GetNumVariations() const;

public:

	UPROPERTY(EditAnywhere, Category = "CustomizableObject")
	TArray<FCustomizableObjectExtensionDataVariation> Variations;

};

#undef UE_API
