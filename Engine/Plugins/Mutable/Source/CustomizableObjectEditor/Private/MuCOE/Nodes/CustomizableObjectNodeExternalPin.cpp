// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExposePin.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeExternalPin)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeExternalPin::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);
}


void UCustomizableObjectNodeExternalPin::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	// Get the pin type from the actual pin.
	if (GetLinkerCustomVersion(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::BeforeCustomVersionWasAdded)
	{
		if (PinType.IsNone())
		{
			PinType = Pins[0]->PinType.PinCategory;
		}
	}

	// All pins named "Object" will be updated to use the friendly name of the pin category. 
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Object")))
		{
			// Set the name of the pin based on the type of pin it is (extracted from UEdGraphSchema_CustomizableObject::GetPinCategoryName and
			// UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName to keep this fixup immutable)
			if (PinType == UEdGraphSchema_CustomizableObject::PC_Object)
			{
				Pin->PinName = TEXT("Object");
				Pin->PinFriendlyName = LOCTEXT("Object_Pin_Category", "Object");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Component)
			{
				Pin->PinName = TEXT("Component");
				Pin->PinFriendlyName = LOCTEXT("Component_Pin_Category", "Component");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_MeshSection)
			{
				Pin->PinName = TEXT("Material");
				Pin->PinFriendlyName = LOCTEXT("MeshSection_Pin_Category", "Mesh Section");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Modifier)
			{
				Pin->PinName = TEXT("Modifier");
				Pin->PinFriendlyName = LOCTEXT("Modifier_Pin_Category", "Modifier");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Mesh)
			{
				Pin->PinName = TEXT("Mesh");
				Pin->PinFriendlyName = LOCTEXT("Mesh_Pin_Category", "Mesh");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Texture)
			{
				Pin->PinName = TEXT("Texture");
				Pin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
			{
				Pin->PinName = TEXT("PassThrough Texture");
				Pin->PinFriendlyName = LOCTEXT("PassThrough_Image_Pin_Category", "PassThrough Texture");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Projector)
			{
				Pin->PinName = TEXT("Projector");
				Pin->PinFriendlyName = LOCTEXT("Projector_Pin_Category", "Projector");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_GroupProjector)
			{
				Pin->PinName = TEXT("Group Projector");
				Pin->PinFriendlyName = LOCTEXT("Group_Projector_Pin_Category", "Group Projector");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Color)
			{
				Pin->PinName = TEXT("Color");
				Pin->PinFriendlyName = LOCTEXT("Color_Pin_Category", "Color");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Float)
			{
				Pin->PinName = TEXT("Float");
				Pin->PinFriendlyName = LOCTEXT("Float_Pin_Category", "Float");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Bool)
			{
				Pin->PinName = TEXT("Bool");
				Pin->PinFriendlyName = LOCTEXT("Bool_Pin_Category", "Bool");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Enum)
			{
				Pin->PinName = TEXT("Enum");
				Pin->PinFriendlyName = LOCTEXT("Enum_Pin_Category", "Enum");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Stack)
			{
				Pin->PinName = TEXT("Stack");
				Pin->PinFriendlyName = LOCTEXT("Stack_Pin_Category", "Stack");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Material)
			{
				Pin->PinName = TEXT("Material");
				Pin->PinFriendlyName = LOCTEXT("Material_Asset_Pin_Category", "Material");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Wildcard)
			{
				Pin->PinName = TEXT("Wildcard");
				Pin->PinFriendlyName = LOCTEXT("Wildcard_Pin_Category", "Wildcard");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_PoseAsset)
			{
				Pin->PinName = TEXT("PoseAsset");
				Pin->PinFriendlyName = LOCTEXT("Pose_Pin_Category", "PoseAsset");
			}
			else if (PinType == UEdGraphSchema_CustomizableObject::PC_Transform)
			{
				Pin->PinName = TEXT("Transform");
				Pin->PinFriendlyName = LOCTEXT("Transform_Pin_Category", "Transform");
			}
			else
			{
				bool bPinTypeWasFound = false;
				for (const FRegisteredCustomizableObjectPinType& RegisteredCustomizableObjectPin : ICustomizableObjectModule::Get().GetExtendedPinTypes())
				{
					if (RegisteredCustomizableObjectPin.PinType.Name == PinType)
					{
						Pin->PinName = RegisteredCustomizableObjectPin.PinType.Name;
						Pin->PinFriendlyName = RegisteredCustomizableObjectPin.PinType.DisplayName;
						bPinTypeWasFound = true;
						break;
					}
				}

				// Need to fail gracefully here in case a plugin that was active when this graph was
				// created is no longer loaded.
				if (!bPinTypeWasFound)
				{
					Pin->PinName = TEXT("Unknown");
					Pin->PinFriendlyName =  LOCTEXT("Unknown_Pin_Category", "Unknown");
				}
			}
		}
	}
}


void UCustomizableObjectNodeExternalPin::PostBackwardsCompatibleFixup()
{
	Super::PostBackwardsCompatibleFixup();

	if (UCustomizableObjectNodeExposePin* NodeExposePin = GetNodeExposePin())
	{
		OnNameChangedDelegateHandle = NodeExposePin->OnNameChangedDelegate.AddUObject(this, &Super::ReconstructNode);
		DestroyNodeDelegateHandle = NodeExposePin->DestroyNodeDelegate.AddUObject(this, &Super::ReconstructNode);
	}

	// Reconstruct the node since the NodeExposePin pin name may have changed while not loaded.
	Super::ReconstructNode();
}


void UCustomizableObjectNodeExternalPin::SetExternalObjectNodeId(FGuid Guid)
{
	PrePropagateConnectionChanged();

	if (UCustomizableObjectNodeExposePin* NodeExposePin = GetNodeExposePin())
	{
		NodeExposePin->OnNameChangedDelegate.Remove(OnNameChangedDelegateHandle);
		NodeExposePin->DestroyNodeDelegate.Remove(DestroyNodeDelegateHandle);
	}

	ExternalObjectNodeId = Guid;

	if (UCustomizableObjectNodeExposePin* NodeExposePin = GetNodeExposePin())
	{
		OnNameChangedDelegateHandle = NodeExposePin->OnNameChangedDelegate.AddUObject(this, &Super::ReconstructNode);
		DestroyNodeDelegateHandle = NodeExposePin->DestroyNodeDelegate.AddUObject(this, &Super::ReconstructNode);
	}
	
	Super::ReconstructNode();

	PropagateConnectionChanged();
}


UEdGraphPin* UCustomizableObjectNodeExternalPin::GetExternalPin() const
{
	return Pins[0];
}


UCustomizableObjectNodeExposePin* UCustomizableObjectNodeExternalPin::GetNodeExposePin() const
{
	return GetCustomizableObjectExternalNode<UCustomizableObjectNodeExposePin>(ExternalObject, ExternalObjectNodeId);
}


void UCustomizableObjectNodeExternalPin::PrePropagateConnectionChanged()
{
	PropagatePreviousPin = ReverseFollowPinArray(*GetExternalPin());
}


void UCustomizableObjectNodeExternalPin::PropagateConnectionChanged()
{
	// Propagate new left.
	PropagatePreviousPin.Append(ReverseFollowPinArray(*GetExternalPin())); // Notify old connections and new connections.
	NodePinConnectionListChanged(PropagatePreviousPin); // This function avoids double notifications.
	
	// Propagate right.
	NodePinConnectionListChanged(FollowOutputPinArray(*GetExternalPin()));
}


void UCustomizableObjectNodeExternalPin::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	UEdGraphPin* OutputPin = CustomCreatePin(EGPD_Output, PinType);
	const bool bIsArrayPinCategory = PinType == UEdGraphSchema_CustomizableObject::PC_GroupProjector;
	if (bIsArrayPinCategory)
	{
		OutputPin->PinType.ContainerType = EPinContainerType::Array;
	}

}


FText UCustomizableObjectNodeExternalPin::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const FText PinTypeName = UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(PinType);

	if (TitleType == ENodeTitleType::ListView || !ExternalObject)
	{
		return FText::Format(LOCTEXT("External_Pin_Title", "{0} Import"), PinTypeName);
	}
	else
	{
		if (const UCustomizableObjectNodeExposePin* ExportNode = GetNodeExposePin())
		{
			return FText::Format(LOCTEXT("External_Pin_ExportNode_Title", "{0}\n{1} Import"), FText::FromString(*ExportNode->GetNodeName()), PinTypeName);
		}
		else
		{
			return FText::Format(LOCTEXT("External_Pin_ExternalObject_Title", "{0}\n{1} Import"), FText::FromString(ExternalObject->GetName()), PinTypeName);
		}
	}
}


FLinearColor UCustomizableObjectNodeExternalPin::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(PinType);
}


FText UCustomizableObjectNodeExternalPin::GetTooltipText() const
{
	return LOCTEXT("Import_Pin_Tooltip", "Make use of a value defined elsewhere in this Customizable Object hierarchy.");
}

bool UCustomizableObjectNodeExternalPin::CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const
{
	// Chech the pin types do match
	bOutArePinsCompatible = Super::CanConnect(InOwnedInputPin, InOutputPin, bOutIsOtherNodeBlocklisted, bOutArePinsCompatible);

	// Check the type of the other node to make sure it is not one we do not want to allow the connection with
	bOutIsOtherNodeBlocklisted = Cast<UCustomizableObjectNodeExposePin>(InOutputPin->GetOwningNode()) != nullptr;

	return bOutArePinsCompatible && !bOutIsOtherNodeBlocklisted;
}


void UCustomizableObjectNodeExternalPin::ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPinsMode)
{
	Super::ReconstructNode(RemapPinsMode);

	if (!ExternalObject)
	{
		ExternalObject = Cast<UCustomizableObject>(GetOutermostObject());
		ExternalObjectNodeId = FGuid();
	}
}


void UCustomizableObjectNodeExternalPin::BeginPostDuplicate(bool bDuplicateForPIE)
{
	Super::BeginPostDuplicate(bDuplicateForPIE);

	if (ExternalObjectNodeId.IsValid())
	{
		if (UCustomizableObjectGraph* CEdGraph = Cast<UCustomizableObjectGraph>(GetGraph()))
		{
			ExternalObjectNodeId = CEdGraph->RequestNotificationForNodeIdChange(ExternalObjectNodeId, NodeGuid);
		}
	}
}


void UCustomizableObjectNodeExternalPin::UpdateReferencedNodeId(const FGuid& NewGuid)
{
	ExternalObjectNodeId = NewGuid;
}


void UCustomizableObjectNodeExternalPin::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
	
	if (PropertyAboutToChange)
	{
		if (PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeExternalPin, ExternalObject))
		{
			PrePropagateConnectionChanged();
		}
	}
}


void UCustomizableObjectNodeExternalPin::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeExternalPin, ExternalObject))
		{
			PropagateConnectionChanged();
		}
	}
}


UCustomizableObjectNodeRemapPins* UCustomizableObjectNodeExternalPin::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByPosition>();
}


#undef LOCTEXT_NAMESPACE
