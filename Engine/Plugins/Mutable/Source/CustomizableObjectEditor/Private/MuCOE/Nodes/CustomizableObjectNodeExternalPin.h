// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeExternalPin.generated.h"

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObject;
class UCustomizableObjectNodeExposePin;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;


/** Import Node. */
UCLASS()
class UCustomizableObjectNodeExternalPin : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// UObject interface.
	virtual void Serialize(FArchive& Ar) override;
	
	// EdGraphNode interface 
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	
	// UCustomizableObjectNode interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual void PostBackwardsCompatibleFixup() override;
	virtual void BeginPostDuplicate(bool bDuplicateForPIE) override;
	virtual bool CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const override;
	virtual void ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPinsMode) override;
	virtual void UpdateReferencedNodeId(const FGuid& NewGuid) override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	// Own interface
	
	/** Set the linked Node Expose Pin node guid. */
	void SetExternalObjectNodeId(FGuid Guid);

	/** Return the external pin. Can return nullptr. */
	UEdGraphPin* GetExternalPin() const;

	/** Return the linked Expose Pin node. Return nullptr if not set. */
	UCustomizableObjectNodeExposePin* GetNodeExposePin() const;

protected:
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	
private:
	void PrePropagateConnectionChanged();
	void PropagateConnectionChanged();

public:

	// This is actually PinCategory
	UPROPERTY()
	FName PinType;

	/** External Customizable Object which the linked Node Expose Pin belong to. */
	UPROPERTY()
	TObjectPtr<UCustomizableObject> ExternalObject;

private:
	
	/** Linked Node Expose Pin node guid. */
	UPROPERTY()
	FGuid ExternalObjectNodeId;
	
	FDelegateHandle OnNameChangedDelegateHandle;
	FDelegateHandle DestroyNodeDelegateHandle;

	/** Connected pins (pins connected to the Export Node pin) before changing the import/export implicit connection. */
	TArray<UEdGraphPin*> PropagatePreviousPin;
};
