// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeExternalPinDetails.h"

#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExposePin.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExternalPin.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/LoadUtils.h"
#include "Widgets/Input/STextComboBox.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeExternalPinDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeExternalPinDetails );
}

void FCustomizableObjectNodeExternalPinDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	Node = nullptr;
	
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeExternalPin>(DetailsView->GetSelectedObjects()[0].Get());
	}

	IDetailCategoryBuilder& StatesCategory = DetailBuilder.EditCategory("States");
	IDetailCategoryBuilder& ExternalCategory = DetailBuilder.EditCategory("ExternalObject");
	
	if (Node)
	{
		DetailBuilder.HideProperty("ExternalObjectNodeId");
		DetailBuilder.HideProperty("ExternalObject");
		GroupNodeComboBoxOptions.Empty();

		ExternalCategory.AddCustomRow(LOCTEXT("FCustomizableObjectNodeExternalPinDetails", "Blocks"))
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UCustomizableObject::StaticClass())
			.OnObjectChanged(this, &FCustomizableObjectNodeExternalPinDetails::ParentObjectSelectionChanged)
			.ObjectPath(Node->ExternalObject->GetPathName())
			.ForceVolatile(true)
		];

		if (Node->ExternalObject)
		{
			const UCustomizableObjectNodeExposePin* SelectedNodeExposePin = Node->GetNodeExposePin();

			TArray<UCustomizableObjectNodeExposePin*> GroupNodes;
			Node->ExternalObject->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeExposePin>(GroupNodes);

			TSharedPtr<FString> ItemToSelect;

			for (const UCustomizableObjectNodeExposePin* GroupNode : GroupNodes)
			{
				if (Node->PinType == GroupNode->PinType)
				{
					GroupNodeComboBoxOptions.Add(MakeShareable(new FString(GroupNode->GetNodeName())));

					if (SelectedNodeExposePin == GroupNode)
					{
						ItemToSelect = GroupNodeComboBoxOptions.Last();
					}
				}
			}

			GroupNodeComboBoxOptions.Sort(CompareNames);

			ExternalCategory.AddCustomRow(LOCTEXT("FCustomizableObjectNodeExternalPinDetails", "Blocks"))
			[
				SNew(SBorder)
					.BorderImage(UE_MUTABLE_GET_BRUSH("NoBorder"))
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					[
						SNew(STextComboBox)
						.OptionsSource(&GroupNodeComboBoxOptions)
						.InitiallySelectedItem(ItemToSelect)
						.OnSelectionChanged(this, &FCustomizableObjectNodeExternalPinDetails::OnGroupNodeComboBoxSelectionChanged)
					]
			];
		}
	}
	else
	{
		StatesCategory.AddCustomRow( LOCTEXT("Node","Node") )
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "Node not found", "Node not found" ) )
		];
	}
}


void FCustomizableObjectNodeExternalPinDetails::ParentObjectSelectionChanged(const FAssetData & AssetData)
{
	const FScopedTransaction Transaction(LOCTEXT("ChangedImportPinCustomizableObjectTransaction", "Changed Import Pin Customizable Object"));
	Node->Modify();
	Node->ExternalObject = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(AssetData));
	Node->SetExternalObjectNodeId(FGuid());

	RefreshDetails();
}


void FCustomizableObjectNodeExternalPinDetails::OnGroupNodeComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Node->ExternalObject)
	{
		TArray<UCustomizableObjectNodeExposePin*> GroupNodes;
		Node->ExternalObject->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeExposePin>(GroupNodes);

		for (UCustomizableObjectNodeExposePin* NodeExposePin: GroupNodes)
		{
			if (Node->PinType == NodeExposePin->PinType &&
				*Selection == NodeExposePin->GetNodeName()) // Type required since the Name is not enough to identify a Expose Pin node.
			{
				const FScopedTransaction Transaction(LOCTEXT("ChangedImportPinNodeTransaction", "Changed Import Pin Node"));
				Node->Modify();
				Node->SetExternalObjectNodeId(NodeExposePin->NodeGuid);
				break;
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
