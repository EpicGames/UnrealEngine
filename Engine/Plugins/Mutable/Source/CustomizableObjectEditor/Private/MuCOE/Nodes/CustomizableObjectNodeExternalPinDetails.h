// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeDetails.h"
#include "IDetailCustomization.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class UCustomizableObjectNodeExternalPin;
struct FAssetData;


class FCustomizableObjectNodeExternalPinDetails : public FCustomizableObjectNodeDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	UCustomizableObjectNodeExternalPin* Node = nullptr;
	
	TArray<TSharedPtr<FString>> GroupNodeComboBoxOptions;

	void ParentObjectSelectionChanged(const FAssetData& AssetData);

	void OnGroupNodeComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
};
