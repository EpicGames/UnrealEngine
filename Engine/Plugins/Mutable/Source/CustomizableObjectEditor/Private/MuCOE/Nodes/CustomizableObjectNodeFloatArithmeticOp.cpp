// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeFloatArithmeticOp.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeFloatArithmeticOp)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeFloatArithmeticOp::UCustomizableObjectNodeFloatArithmeticOp()
	: Super()
{
	Operation = EFloatArithmeticOperation::E_Add;
}


void UCustomizableObjectNodeFloatArithmeticOp::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("A"), LOCTEXT("A", "A"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("B"), LOCTEXT("B", "B"));
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Result"), LOCTEXT("Result", "Result"));
}


FText UCustomizableObjectNodeFloatArithmeticOp::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::ListView)
	{
		return LOCTEXT("Float_Arithmetic_Operation", "Float Arithmetic Operation");
	}

	const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/CustomizableObjectEditor.EFloatArithmeticOperation"), EFindObjectFlags::ExactClass);

	if (!EnumPtr)
	{
		return FText::FromString(FString("Float Operation"));
	}

	const int32 index = EnumPtr->GetIndexByValue((int32)Operation);
	return EnumPtr->GetDisplayNameTextByIndex(index);
}


FLinearColor UCustomizableObjectNodeFloatArithmeticOp::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Float);
}


FText UCustomizableObjectNodeFloatArithmeticOp::GetTooltipText() const
{
	return LOCTEXT("Float_Arithmetic_Tooltip", "Perform an arithmetic operation between two floats.");
}


#undef LOCTEXT_NAMESPACE

