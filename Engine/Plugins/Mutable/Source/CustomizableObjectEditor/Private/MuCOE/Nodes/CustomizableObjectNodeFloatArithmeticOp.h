// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeFloatArithmeticOp.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UENUM(BlueprintType)
enum class EFloatArithmeticOperation : uint8
{
	E_Add 	UMETA(DisplayName = "+"),
	E_Sub 	UMETA(DisplayName = "-"),
	E_Mul	UMETA(DisplayName = "x"),
	E_Div	UMETA(DisplayName = "/")
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeFloatArithmeticOp : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeFloatArithmeticOp();

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	virtual bool ShouldOverridePinNames() const override { return true; }

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	// First Operand
	UEdGraphPin* XPin() const
	{
		return FindPin(TEXT("A"));
	}

	// Second Operand
	UEdGraphPin* YPin() const
	{
		return FindPin(TEXT("B"));
	}

	UPROPERTY(EditAnywhere, Category = FloatArithmeticOperation)
	EFloatArithmeticOperation Operation;

	// UCustomizableObjectNode interface
	virtual bool IsAffectedByLOD() const override { return false; }

};

#undef UE_API
