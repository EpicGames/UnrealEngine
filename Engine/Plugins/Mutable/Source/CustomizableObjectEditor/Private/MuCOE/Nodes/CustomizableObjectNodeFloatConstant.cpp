// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeFloatConstant.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeFloatConstant)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeFloatConstant::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Float);
}


FText UCustomizableObjectNodeFloatConstant::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Float_Constant", "Float");
}


FLinearColor UCustomizableObjectNodeFloatConstant::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Float);
}


FText UCustomizableObjectNodeFloatConstant::GetTooltipText() const
{
	return LOCTEXT("Float_Constant_Tooltip", "Define a rational number that does not change at runtime.");
}


void UCustomizableObjectNodeFloatConstant::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Float");
			Pin->PinFriendlyName = LOCTEXT("Float_Pin_Category", "Float");
		}
	}
}

TSharedPtr<SGraphNode> UCustomizableObjectNodeFloatConstant::CreateVisualWidget()
{
	return SNew(SGraphNodeFloatConstant, this);
}


// SGraphNode Float Constant

void SGraphNodeFloatConstant::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	NodeFloatConstant = Cast< UCustomizableObjectNodeFloatConstant >(InGraphNode);

	WidgetStyle = FCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("SpinBox");

	SCustomizableObjectNode::Construct({}, InGraphNode);
}


void SGraphNodeFloatConstant::UpdateGraphNode()
{
	SGraphNode::UpdateGraphNode();
}


void SGraphNodeFloatConstant::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	// Collapsing arrow of the title area
	DefaultTitleAreaWidget->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(FMargin(5))
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SGraphNodeFloatConstant::OnExpressionPreviewChanged)
		.IsChecked(IsExpressionPreviewChecked())
		.Cursor(EMouseCursor::Default)
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(GetExpressionPreviewArrow())
			]
		]
	];
}


void SGraphNodeFloatConstant::OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState)
{
	NodeFloatConstant->bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	UpdateGraphNode();
}


ECheckBoxState SGraphNodeFloatConstant::IsExpressionPreviewChecked() const
{
	return NodeFloatConstant->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


const FSlateBrush* SGraphNodeFloatConstant::GetExpressionPreviewArrow() const
{
	return FCustomizableObjectEditorStyle::Get().GetBrush(NodeFloatConstant->bCollapsed ? TEXT("Nodes.ArrowDown") : TEXT("Nodes.ArrowUp"));
}


void SGraphNodeFloatConstant::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	MainBox->AddSlot()
	.AutoHeight()
	.Padding(10.0f, 0.0f, 10.0f, 10.0f)
	[
		SNew(SHorizontalBox)
		.Visibility(NodeFloatConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FloatValueText", "Float Value:"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f,0.0f,0.0f,0.0f)
		[
			SNew(SSpinBox<float>)
			.MinDesiredWidth(60.0f)
			.Value_Lambda([this]() {return NodeFloatConstant->Value; })
			.OnValueChanged(this,&SGraphNodeFloatConstant::OnSpinBoxValueChanged)
			.OnValueCommitted(this,&SGraphNodeFloatConstant::OnSpinBoxValueCommitted)
			.Style(&WidgetStyle)
			.Delta(0.001f)
		]
	];
}


void SGraphNodeFloatConstant::OnSpinBoxValueChanged(float Value)
{
	NodeFloatConstant->Value = Value;
	NodeFloatConstant->GetCustomizableObjectGraph()->GetOuter()->MarkPackageDirty();
}


void SGraphNodeFloatConstant::OnSpinBoxValueCommitted(float Value, ETextCommit::Type)
{
	NodeFloatConstant->Value = Value;
	NodeFloatConstant->GetCustomizableObjectGraph()->GetOuter()->MarkPackageDirty();
}

#undef LOCTEXT_NAMESPACE
