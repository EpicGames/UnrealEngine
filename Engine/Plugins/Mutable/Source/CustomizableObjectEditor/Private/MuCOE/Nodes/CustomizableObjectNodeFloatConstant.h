// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CustomizableObjectNodeFloatConstant.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class SOverlay;
class SVerticalBox;
class UCustomizableObjectNodeFloatConstant;
class UCustomizableObjectNodeRemapPins;
class UObject;
struct FPropertyChangedEvent;
struct FSlateBrush;

// Class create an input float in the NodeFloatConstant
class SGraphNodeFloatConstant : public SCustomizableObjectNode
{
public:

	SLATE_BEGIN_ARGS(SGraphNodeFloatConstant) {}

	SLATE_END_ARGS();

	SGraphNodeFloatConstant() : SCustomizableObjectNode() {};

	// Builds the SGraphNodeFloatConstant when needed
	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// Calls the needed functions to build the SGraphNode widgets
	void UpdateGraphNode();

	// Overriden functions to build the SGraphNode widgets
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	// Callbacks for the widget
	void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsExpressionPreviewChecked() const;
	const FSlateBrush* GetExpressionPreviewArrow() const;

private:

	// Callback for the OnValueChanged of the SpinBox
	void OnSpinBoxValueChanged(float Value);

	// Callback for the OnValueCommited of the SpinBox
	void OnSpinBoxValueCommitted(float Value, ETextCommit::Type);

	// Pointer to the NodeFloatconstant that owns this SGraphNode
	UCustomizableObjectNodeFloatConstant* NodeFloatConstant;

	// Style for the SpinBox
	FSpinBoxStyle WidgetStyle;

};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeFloatConstant : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool IsAffectedByLOD() const override { return false; }
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Creates the SGraph Node widget for the thumbnail
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (DontUpdateWhileEditing))
	float Value = 1.0f;

	// Determines if the Node is collapsed or not
	bool bCollapsed = true;
};

#undef UE_API
