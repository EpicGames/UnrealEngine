// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeFloatParameter.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeFloatParameter)


class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeFloatParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if ( PropertyThatChanged && (PropertyThatChanged->GetName() == TEXT("DescriptionImage") || PropertyThatChanged->GetName() == TEXT("Name")) )
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeFloatParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::RemovedParameterDecorations)
	{
		ReconstructNode();
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Float");
			Pin->PinFriendlyName = LOCTEXT("Float_Pin_Category", "Float");
		}
	}
}


FName UCustomizableObjectNodeFloatParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Float;
}


#undef LOCTEXT_NAMESPACE
