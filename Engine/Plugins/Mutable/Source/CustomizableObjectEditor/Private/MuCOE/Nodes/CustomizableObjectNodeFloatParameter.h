// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "CustomizableObjectNodeFloatParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FEdGraphPinReference;
struct FPropertyChangedEvent;


USTRUCT()
struct FCustomizableObjectNodeFloatDescription
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	FString Name;
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeFloatParameter : public UCustomizableObjectNodeParameter
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	float DefaultValue = 1.0f;
	
	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;

};

#undef UE_API
