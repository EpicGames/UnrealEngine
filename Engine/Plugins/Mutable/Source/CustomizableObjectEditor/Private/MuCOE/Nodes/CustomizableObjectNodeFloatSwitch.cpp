// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeFloatSwitch.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeFloatSwitch)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeFloatSwitch::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	PinType = UEdGraphSchema_CustomizableObject::PC_Float;
	
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

}


#undef LOCTEXT_NAMESPACE

