// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeFloatVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeFloatVariation)


void UCustomizableObjectNodeFloatVariation::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeVariationSerializationIssue)
	{
		for (const FCustomizableObjectFloatVariation& OldVariation : Variations_DEPRECATED)
		{
			FCustomizableObjectVariation Variation;
			Variation.Tag = OldVariation.Tag;
			
			VariationsData.Add(Variation);
		}
	}
}


FName UCustomizableObjectNodeFloatVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Float;
}
