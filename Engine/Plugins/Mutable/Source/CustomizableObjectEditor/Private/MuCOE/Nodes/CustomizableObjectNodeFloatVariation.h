// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"

#include "CustomizableObjectNodeFloatVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

struct FCustomizableObjectFloatVariation;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeFloatVariation : public UCustomizableObjectNodeVariation
{
public:
	GENERATED_BODY()
	
	// Deprecated properties
	UPROPERTY()
	TArray<FCustomizableObjectFloatVariation> Variations_DEPRECATED;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	// UCustomizableObjectNodeVariation interface
	UE_API virtual FName GetCategory() const override;
};

#undef UE_API
