// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeGroupProjectorParameter)


class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

FText UCustomizableObjectNodeGroupProjectorParameter::GetTooltipText() const
{
	return LOCTEXT("Group_Projector_Parameter_Tooltip", "Projects one or many textures to all children in the group it's connected to. It modifies only the materials that define a specific material asset texture parameter.");
}


void UCustomizableObjectNodeGroupProjectorParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::GroupProjectorPinTypeAdded)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_GroupProjector;
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::GroupProjectorImagePinRemoved)
	{
		ReconstructNode();
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		// Looking for the projector pin as it was renamed in the same compatibility fix but of the parent
		if (UEdGraphPin* Pin = FindPin(TEXT("Projector")))
		{
			Pin->PinName = TEXT("Group Projector");
			Pin->PinFriendlyName = LOCTEXT("Group_Projector_Pin_Category", "Group Projector");
		}
	}
}


FName UCustomizableObjectNodeGroupProjectorParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_GroupProjector;
}


UEdGraphPin& UCustomizableObjectNodeGroupProjectorParameter::OutputPin() const
{
	const FName Type = UEdGraphSchema_CustomizableObject::PC_GroupProjector;
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(Type);
	return *FindPin(PinName);
}


TArray<FGroupProjectorParameterImage> UCustomizableObjectNodeGroupProjectorParameter::GetOptionTexturesFromTable() const
{
	TArray<FGroupProjectorParameterImage> ArrayResult;

	if (OptionTexturesDataTable == nullptr)
	{
		return ArrayResult;
	}

	TArray<FName> ArrayRowName = OptionTexturesDataTable->GetRowNames();

	FProperty* PropertyTexturePath = OptionTexturesDataTable->FindTableProperty(DataTableTextureColumnName);

	if (PropertyTexturePath == nullptr)
	{
		UE_LOGF(LogMutable, Warning, "WARNING: No column found with texture path information to load projection textures");
		return ArrayResult;
	}

	int32 NameIndex = 0;

	for (TMap<FName, uint8*>::TConstIterator RowIt = OptionTexturesDataTable->GetRowMap().CreateConstIterator(); RowIt; ++RowIt)
	{
		uint8* RowData = RowIt.Value();
		FString PropertyValue(TEXT(""));
		PropertyTexturePath->ExportText_InContainer(0, PropertyValue, RowData, RowData, nullptr, PPF_None);
		UTexture2D* Texture = UE::Mutable::Private::LoadObject<UTexture2D>(nullptr, *PropertyValue);

		if (Texture == nullptr)
		{
			UE_LOGF(LogMutable, Warning, "WARNING: Unable to load texture %ls", *PropertyValue);
		}
		else
		{
			FGroupProjectorParameterImage GroupProjectorParameterImage;
			GroupProjectorParameterImage.OptionName = ArrayRowName[NameIndex].ToString();
			GroupProjectorParameterImage.OptionTexture = Texture;
			ArrayResult.Add(GroupProjectorParameterImage);
		}

		NameIndex++;
	}

	return ArrayResult;
}


TArray<FGroupProjectorParameterImage> UCustomizableObjectNodeGroupProjectorParameter::GetFinalOptionTexturesNoRepeat() const
{
	TArray<FGroupProjectorParameterImage> ArrayDataTable = GetOptionTexturesFromTable();

	for (int32 i = 0; i < OptionTextures.Num(); ++i)
	{
		bool AlreadyAdded = false;
		for (int32 j = 0; j < ArrayDataTable.Num(); ++j)
		{
			if (OptionTextures[i].OptionName == ArrayDataTable[j].OptionName)
			{
				AlreadyAdded = true;
				break;
			}
		}

		if (!AlreadyAdded)
		{
			ArrayDataTable.Add(OptionTextures[i]);
		}
	}

	return ArrayDataTable;
}

#undef LOCTEXT_NAMESPACE
