// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeProjectorParameter.h"

#include "CustomizableObjectNodeGroupProjectorParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UDataTable;
class UObject;
class UPoseAsset;
class UTexture2D;

USTRUCT()
struct FGroupProjectorParameterImage
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	FString OptionName;

	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TObjectPtr<UTexture2D> OptionTexture = nullptr;
};


USTRUCT()
struct FGroupProjectorParameterPose
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	FString PoseName;

	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TObjectPtr<UPoseAsset> OptionPose = nullptr;
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeGroupProjectorParameter : public UCustomizableObjectNodeProjectorParameter
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = ProjectorGroup, Meta = (ToolTip = "Only used when connected to a Group node. Specifies which material channel in the Group node's child material nodes will be connected to the projection."))
	FString MaterialChannelNameToConnect;

	/** Reference Texture used to decide the texture properties of the mutable-generated textures
	* connected to this material. If null, it will try to be guessed at compile time from
	* the graph. */
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TObjectPtr<UTexture2D> ReferenceTexture = nullptr;

	// TODO Remove. Only used because poses are the diff from a Reference Skeleton. Once the pose is absolute remove this.
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TObjectPtr<USkeletalMesh> ReferenceSkeletalMesh = nullptr;
	
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	int32 ProjectionTextureSize = 512;

	UPROPERTY(EditAnywhere, Category = ProjectorGroup, Meta = (ToolTip = "Specifies at which LOD level the projection texture will not be used and possibly save memory. A negative value means they will never be dropped."))
	int32 DropProjectionTextureAtLOD = -1;
	
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TArray<FGroupProjectorParameterImage> OptionTextures;

	/** Poses will only affect component 0 of the CO */ // TODO UE-206803
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TArray<FGroupProjectorParameterPose> OptionPoses;

	/** Name of the column in the Option Images Data Table with the additional option images, (UTexture2D assets). */
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	FName DataTableTextureColumnName;

	/** Table where additional option images besides Option Images are read. The elements in this table have priority
	* over elements from Option Images in case of duplicity. Use the "Data Table Texture Column Name" property to specify
	* the name of the column where textures are read in the table. */
	UPROPERTY(EditAnywhere, Category = ProjectorGroup)
	TObjectPtr<UDataTable> OptionTexturesDataTable = nullptr;

	UPROPERTY(EditAnywhere, Category = ProjectorGroup, Meta = (ClampMin = "0"))
	int32 UVLayout = 0;

	// EdGraphNode interface.
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface.
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;

	// Own interface
	UE_API UEdGraphPin& OutputPin() const;

	/** Return array with the sticker name and UTexture2D for projection */
	UE_API TArray<FGroupProjectorParameterImage> GetOptionTexturesFromTable() const;

	/** Returns the final option images without repeated elements in the option names,
	* the data table has preference over elements in the OptionTextures array */
	UE_API TArray<FGroupProjectorParameterImage> GetFinalOptionTexturesNoRepeat() const;
};

#undef UE_API
