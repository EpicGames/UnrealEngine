// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeLayoutBlocks.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeLayoutBlocks)

class UCustomizableObjectNodeRemapPins;
class UObject;
struct FPropertyChangedEvent;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeLayoutBlocks::UCustomizableObjectNodeLayoutBlocks()
	: Super()
{
	Layout = CreateDefaultSubobject<UCustomizableObjectLayout>(FName("CustomizableObjectLayout"));
}


void UCustomizableObjectNodeLayoutBlocks::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::RemoveNodeLayout)
	{
		DestroyNode();
	}
}


void UCustomizableObjectNodeLayoutBlocks::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);
	
	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::LayoutClassAdded)
	{
		Layout->SetGridSize(GridSize_DEPRECATED);
		Layout->SetMaxGridSize(MaxGridSize_DEPRECATED);
		Layout->Blocks = Blocks_DEPRECATED;
		Layout->PackingStrategy = PackingStrategy_DEPRECATED;

		if (Layout->GetGridSize() == FIntPoint::ZeroValue)
		{
			Layout->SetGridSize(FIntPoint(4));
		}

		if(Layout->GetMaxGridSize() == FIntPoint::ZeroValue)
		{
			Layout->SetMaxGridSize(FIntPoint(4));
		}
	}
}

#undef LOCTEXT_NAMESPACE

