// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeLayoutBlocks.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UCustomizableObjectLayout;
enum class ECustomizableObjectTextureLayoutPackingStrategy : uint8;
namespace ENodeTitleType { enum Type : int; }
struct FCustomizableObjectLayoutBlock;

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeLayoutBlocks : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeLayoutBlocks();

	UPROPERTY()
	FIntPoint GridSize_DEPRECATED;

	/** Used with the fixed layout strategy. */
	UPROPERTY()
	FIntPoint MaxGridSize_DEPRECATED;

	UPROPERTY()
	TArray<FCustomizableObjectLayoutBlock> Blocks_DEPRECATED;

	UPROPERTY()
	ECustomizableObjectTextureLayoutPackingStrategy PackingStrategy_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UCustomizableObjectLayout> Layout = nullptr;

	// EdGraphNode interface
	UE_API virtual void Serialize(FArchive& Ar) override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
};

#undef UE_API
