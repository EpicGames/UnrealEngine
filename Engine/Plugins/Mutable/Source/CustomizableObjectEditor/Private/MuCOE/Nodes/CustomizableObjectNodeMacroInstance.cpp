// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"

#include "Editor.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibraryEditor.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMacroInstance)

#define LOCTEXT_NAMESPACE "CustomizableObjectNodeMacroInstance"


bool UCustomizableObjectNodeMacroInstanceRemapPins::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const
{
	const UCustomizableObjectNodeMacroInstancePinData* PinDataOldPin = Cast<UCustomizableObjectNodeMacroInstancePinData>(Node.GetPinData(OldPin));
	const UCustomizableObjectNodeMacroInstancePinData* PinDataNewPin = Cast<UCustomizableObjectNodeMacroInstancePinData>(Node.GetPinData(NewPin));

	return PinDataOldPin->VariableId == PinDataNewPin->VariableId && OldPin.PinType.PinCategory == NewPin.PinType.PinCategory;
}


void UCustomizableObjectNodeMacroInstanceRemapPins::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	for (UEdGraphPin* OldPin : OldPins)
	{
		bool bFound = false;

		for (UEdGraphPin* NewPin : NewPins)
		{
			if (Equal(Node, *OldPin, *NewPin))
			{
				bFound = true;

				PinsToRemap.Add(OldPin, NewPin);
				break;
			}
		}

		if (!bFound && OldPin->LinkedTo.Num())
		{
			PinsToOrphan.Add(OldPin);
		}
	}
}


UCustomizableObjectNodeMacroInstanceRemapPins* UCustomizableObjectNodeMacroInstance::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeMacroInstanceRemapPins>();
}


FText UCustomizableObjectNodeMacroInstance::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::ListView || !ParentMacro)
	{
		return LOCTEXT("MacroInstanceNodeTitle", "Macro Instance");
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MacroInstanceName"), FText::FromName(ParentMacro->Name));

		return FText::Format(LOCTEXT("MeshSection_Title", "{MacroInstanceName}\nMacro Instance"), Args);
	}
}


FText UCustomizableObjectNodeMacroInstance::GetTooltipText() const
{
	if (ParentMacroLibrary && ParentMacro)
	{
		FString Tooltip = ParentMacro->Name.ToString() + ":\n" + ParentMacro->Description;
		return FText::FromString(Tooltip);
	}

	return LOCTEXT("MacroInstanceNodeTooltip", "Macro Instance");
}


FLinearColor UCustomizableObjectNodeMacroInstance::GetNodeTitleColor() const
{
	return FLinearColor(0.15f, 0.15f, 0.15f);
}


bool UCustomizableObjectNodeMacroInstance::CanUserDeleteNode() const
{
	return true;
}


UObject* UCustomizableObjectNodeMacroInstance::GetJumpTargetForDoubleClick() const
{
	if (ParentMacroLibrary && ParentMacro)
	{
		return ParentMacroLibrary;
	}

	return nullptr;
}


bool UCustomizableObjectNodeMacroInstance::CanJumpToDefinition() const
{
	return ParentMacroLibrary && ParentMacro;
}


void UCustomizableObjectNodeMacroInstance::JumpToDefinition() const
{
	// Open the editor for the Macro library
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(GetJumpTargetForDoubleClick());

	// Find the editor we just opened
	IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ParentMacroLibrary, false);

	// Set the Parent Macro to edit
	if (FCustomizableObjectMacroLibraryEditor* Editor = static_cast<FCustomizableObjectMacroLibraryEditor*>(AssetEditor))
	{
		Editor->SetSelectedMacro(ParentMacro, true);
	}
}


void UCustomizableObjectNodeMacroInstance::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	if (ParentMacroLibrary && ParentMacro)
	{
		for (const UCustomizableObjectMacroInputOutput* Variable : ParentMacro->InputOutputs)
		{
			EEdGraphPinDirection PinDirection = Variable->Type == ECOMacroIOType::COMVT_Input ? EEdGraphPinDirection::EGPD_Input : EEdGraphPinDirection::EGPD_Output;
			UCustomizableObjectNodeMacroInstancePinData* PinData = NewObject<UCustomizableObjectNodeMacroInstancePinData>(this);
			PinData->VariableId = Variable->UniqueId;

			const FName PinName = Variable->Name;
			CustomCreatePin(PinDirection, Variable->PinCategoryType, PinName, FText::FromName(PinName), PinData);
		}
	}
}


void UCustomizableObjectNodeMacroInstance::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeMacroInstance, ParentMacroLibrary))
	{
		ParentMacro = nullptr;
		ReconstructNode();
	}
}


UEdGraphPin* UCustomizableObjectNodeMacroInstance::GetMacroTunnelPin(ECOMacroIOType IONodeType, const FName& PinName) const
{
	UEdGraphPin* MacroIOPin = nullptr;

	if (ParentMacroLibrary && ParentMacro)
	{
		// Input Node contains output pins and output Node contains input pins!
		EEdGraphPinDirection PinDirection = IONodeType == ECOMacroIOType::COMVT_Input ? EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input;
		const UCustomizableObjectNodeTunnel* OutputNode = ParentMacro->GetIONode(IONodeType);
		check(OutputNode);

		MacroIOPin = OutputNode->FindPin(PinName, PinDirection);
	}

	return (MacroIOPin && !MacroIOPin->bOrphanedPin) ? MacroIOPin : nullptr;
}

#undef LOCTEXT_NAMESPACE
