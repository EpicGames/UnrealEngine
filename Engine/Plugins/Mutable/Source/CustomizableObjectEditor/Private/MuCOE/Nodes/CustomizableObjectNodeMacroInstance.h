// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"

#include "CustomizableObjectNodeMacroInstance.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


/** Base class for all Macro Instance Pins. */
UCLASS(MinimalAPI)
class UCustomizableObjectNodeMacroInstancePinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	/** Id of the variable associated to a Macro Instance node pin */
	UPROPERTY()
	FGuid VariableId;

};


UCLASS()
class UCustomizableObjectNodeMacroInstanceRemapPins : public UCustomizableObjectNodeRemapPinsByName
{
	GENERATED_BODY()
public:

	// Specific method to decide when two pins are equal
	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const override;

	// Method to use in the RemapPins step of the node reconstruction process
	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};



UCLASS(MinimalAPI)
class UCustomizableObjectNodeMacroInstance : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	/** Macro Library where the macro to instantiate belongs. */
	UPROPERTY(EditAnywhere, Category = MacroInstance)
	TObjectPtr<UCustomizableObjectMacroLibrary> ParentMacroLibrary;

	/** Macro that represent that instantiates the node. */
	UPROPERTY(EditAnywhere, Category = MacroInstance)
	TObjectPtr<UCustomizableObjectMacro> ParentMacro;

public:

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual bool CanJumpToDefinition() const override;
	UE_API virtual void JumpToDefinition() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual UCustomizableObjectNodeMacroInstanceRemapPins* CreateRemapPinsDefault() const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Node MacroInstance Interface
	/** Given a Macro pin, this function returns its I/O tunnel pin. */
	UE_API UEdGraphPin* GetMacroTunnelPin(ECOMacroIOType IONodeType, const FName& PinName) const;
};

#undef UE_API
