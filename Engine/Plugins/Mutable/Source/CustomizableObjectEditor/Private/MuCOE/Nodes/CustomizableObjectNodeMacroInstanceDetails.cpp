// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstanceDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "Misc/Attribute.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectNodeMacroInstanceDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeMacroInstanceDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeMacroInstanceDetails);
}


void FCustomizableObjectNodeMacroInstanceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeMacroInstance>(DetailsView->GetSelectedObjects()[0].Get());
	}

	IDetailCategoryBuilder& MacroInstanceCategory = DetailBuilder.EditCategory("MacroInstance");

	if (!Node)
	{
		MacroInstanceCategory.AddCustomRow(FText::FromString("MacroInstaceDetailsError"))
		[
			SNew(STextBlock).Text(LOCTEXT("MacroInstaceDetailsErrorMessage", "Error: Node not found."))
		];

		return;
	}

	TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_STRING_CHECKED(UCustomizableObjectNodeMacroInstance, ParentMacro));

	SelectedSource = GenerateComboboxSource();

	// Macro selector widget
	DetailBuilder.EditDefaultProperty(PropertyHandle)->CustomWidget()
	.IsValueEnabled(TAttribute<bool>::CreateSPLambda(this, [this]() { return Node->ParentMacroLibrary != nullptr; }))
	.NameContent()
	[
		SNew(STextBlock).Text(LOCTEXT("ParentMacroText", "Parent Macro"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SComboBox<TSharedPtr<FMacroSelectorItem>>)
		.OptionsSource(&ComboboxSource)
		.OnComboBoxOpening_Lambda([this]() {GenerateComboboxSource(); })
		.InitiallySelectedItem(SelectedSource)
		.OnGenerateWidget_Lambda([](TSharedPtr<FMacroSelectorItem> Item)
		{
			return SNew(STextBlock).Text(FText::FromName(Item->Macro->Name));
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<FMacroSelectorItem> Item, ESelectInfo::Type)
		{
			if (Item.IsValid())
			{
				SelectedSource = Item;
				Node->ParentMacro = Item->Macro;
				Node->ReconstructNode();
			}
		})
		.Content()
		[
			SNew(STextBlock).MinDesiredWidth(200)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FCustomizableObjectNodeMacroInstanceDetails::GetSelectedMacroName)
		]
	]
	.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeMacroInstanceDetails::ResetSelectedParentMacro)));
}


TSharedPtr<FMacroSelectorItem> FCustomizableObjectNodeMacroInstanceDetails::GenerateComboboxSource()
{
	TSharedPtr<FMacroSelectorItem> CurrentSelected = nullptr;
	ComboboxSource.Empty();

	if (Node && Node->ParentMacroLibrary)
	{
		UEdGraph* NodeGraph = Node->GetGraph();
		check(NodeGraph);

		for (UCustomizableObjectMacro* Macro : Node->ParentMacroLibrary->Macros)
		{
			// Do not allow to instantiate a macro in its own graph
			if (Macro && Macro->Graph != NodeGraph)
			{
				TSharedPtr<FMacroSelectorItem> MacroSelector = MakeShareable(new FMacroSelectorItem());
				MacroSelector->Macro = Macro;

				ComboboxSource.Add(MacroSelector);

				if (Node->ParentMacro && Node->ParentMacro == Macro)
				{
					CurrentSelected = MacroSelector;
				}
			}
		}
	}

	return CurrentSelected;
}


FText FCustomizableObjectNodeMacroInstanceDetails::GetSelectedMacroName() const
{
	FText Text = FText::FromString("- Nothing Selected -");

	if (Node->ParentMacro)
	{
		Text = FText::FromName(Node->ParentMacro->Name);
	}

	return Text;
}


void FCustomizableObjectNodeMacroInstanceDetails::ResetSelectedParentMacro()
{
	if (Node)
	{
		Node->ParentMacro = nullptr;
		Node->ReconstructNode();
	}
}


#undef LOCTEXT_NAMESPACE