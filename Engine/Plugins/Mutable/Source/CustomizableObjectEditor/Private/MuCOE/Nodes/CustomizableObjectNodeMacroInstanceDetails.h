// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeDetails.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"

class UCustomizableObjectMacro;

/** Data container for the macro selector combobox */
struct FMacroSelectorItem
{
	TObjectPtr<UCustomizableObjectMacro> Macro;
};


/** Details View of UCustomizableObjectNodeMacroInstance */
class FCustomizableObjectNodeMacroInstanceDetails : public FCustomizableObjectNodeDetails
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance();

	// ILayoutDetails interface
	/** Do not use. Add details customization in the other CustomizeDetails signature. */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/** Generates the soruce of the combobox used to select a Macro. */
	TSharedPtr<FMacroSelectorItem> GenerateComboboxSource();

private:

	FText GetSelectedMacroName() const;

	/** Set the selected macro property to nullptr */
	void ResetSelectedParentMacro();

private:

	/** Pointer to the Macro Instance Node. */
	TObjectPtr<UCustomizableObjectNodeMacroInstance> Node;

	/** Combobox Source array. */
	TArray<TSharedPtr<FMacroSelectorItem>> ComboboxSource;

	/** Current selected Macro in the Combobox. */
	TSharedPtr<FMacroSelectorItem> SelectedSource;

};
