// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableObjectNodeMaterialParameter.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMaterialParameter)


FName UCustomizableObjectNodeMaterialParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Material;
}


TSoftObjectPtr<UMaterialInterface> UCustomizableObjectNodeMaterialParameter::GetMaterial() const
{
	unimplemented(); // TODO Does not make sense
	return nullptr;
}


UEdGraphPin* UCustomizableObjectNodeMaterialParameter::GetMaterialPin() const
{
	unimplemented(); // TODO Does not make sense
	return nullptr;
}
