// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSectionInterface.h"

#include "CustomizableObjectNodeMaterialParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UMaterialInterface;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeMaterialParameter : public UCustomizableObjectNodeParameter, public ICONodeSkeletalMeshSectionInterface
{
public:
	GENERATED_BODY()
	
	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;

	// ICONodeSkeletalMeshSectionInterface interface
	UE_API virtual TSoftObjectPtr<UMaterialInterface> GetMaterial() const override;
	UE_API virtual UEdGraphPin* GetMaterialPin() const override;
	
	/** Default value of the parameter. */
	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TSoftObjectPtr<UMaterialInterface> DefaultValue;

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TSoftObjectPtr<UMaterialInterface> ReferenceValue;
};

#undef UE_API
