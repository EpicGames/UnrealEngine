// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipDeform.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeModifierClipDeform::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	
	CustomCreatePin(EGPD_Input, Schema->PC_Mesh, TEXT("Clip Shape"), LOCTEXT("ClipShape", "Clip Shape"));

	CustomCreatePin(EGPD_Output, Schema->PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


void UCustomizableObjectNodeModifierClipDeform::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UnifyRequiredTags)
	{
		RequiredTags = Tags_DEPRECATED;
		Tags_DEPRECATED.Empty();
	}
}


bool UCustomizableObjectNodeModifierClipDeform::IsExperimental() const
{
	return true;
}


UEdGraphPin* UCustomizableObjectNodeModifierClipDeform::ClipShapePin() const
{
	return FindPin(TEXT("Clip Shape"), EGPD_Input);
}


FText UCustomizableObjectNodeModifierClipDeform::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Clip_Deform_Mesh", "Clip Deform Mesh");
}


FText UCustomizableObjectNodeModifierClipDeform::GetTooltipText() const
{
	return LOCTEXT("Clip_Deform_Tooltip", "Defines a clip with mesh deformation based on a shape mesh and blend weights.");

}
#undef LOCTEXT_NAMESPACE
