// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "Engine/SkeletalMesh.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeModifierClipMorph::UCustomizableObjectNodeModifierClipMorph()
	: Super()
{
	StartOffset = FVector::ZeroVector;
	bLocalStartOffset = true;
	B = 0.f;
	Radius = 8.f;
	Radius2 = 4.f;
	RotationAngle = 0.f;
	Exponent = 1.f;

	Origin = FVector::ZeroVector;
	Normal = -FVector::UpVector;
	MaxEffectRadius = -1.f;
}


FVector UCustomizableObjectNodeModifierClipMorph::GetOriginWithOffset() const
{
	FVector NewOrigin;

	if (bLocalStartOffset)
	{
		FVector XAxis, YAxis, ZAxis;
		FindLocalAxes(XAxis, YAxis, ZAxis);

		NewOrigin = Origin + StartOffset.X * XAxis + StartOffset.Y * YAxis + StartOffset.Z * ZAxis;
	}
	else
	{
		NewOrigin = Origin + StartOffset;
	}

	return NewOrigin;
}

void UCustomizableObjectNodeModifierClipMorph::FindLocalAxes(FVector& XAxis, FVector& YAxis, FVector& ZAxis) const
{
	YAxis = FVector(0.f, 1.f, 0.f);

	if (FMath::Abs(FVector::DotProduct(Normal, YAxis)) > 0.95f)
	{
		YAxis = FVector(0.f, 0.f, 1.f);
	}

	XAxis = FVector::CrossProduct(Normal, YAxis);
	XAxis = XAxis.RotateAngleAxis(RotationAngle, Normal);
	YAxis = FVector::CrossProduct(Normal, XAxis);
	ZAxis = Normal;

	XAxis.Normalize();
	YAxis.Normalize();
}


void UCustomizableObjectNodeModifierClipMorph::ChangeStartOffsetTransform()
{
	// Local Offset
	FVector XAxis, YAxis, ZAxis;
	FindLocalAxes(XAxis, YAxis, ZAxis);

	if (bLocalStartOffset)
	{
		StartOffset = FVector(FVector::DotProduct(StartOffset, XAxis), FVector::DotProduct(StartOffset, YAxis),
							   FVector::DotProduct(StartOffset, ZAxis));
	}
	else
	{
		StartOffset = StartOffset.X * XAxis + StartOffset.Y * YAxis + StartOffset.Z * ZAxis;
	}
}


void UCustomizableObjectNodeModifierClipMorph::InvertNormals()
{
	if (bLocalStartOffset)
	{
		StartOffset.Z *= -1.0f;
		StartOffset.X *= -1.0f;
	}

	Normal *= -1.0f;
}


void UCustomizableObjectNodeModifierClipMorph::UpdateOriginAndNormal()
{
	// Get all the bone names of the skeletal mesh
	if (TStrongObjectPtr<USkeletalMesh> SkeletalMesh = GetReferenceSkeletalMesh())
	{
		for (int32 BoneIndex = 0; BoneIndex < SkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++BoneIndex)
		{
			const FName RefSkeletonBoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(BoneIndex);
			
			// Is the bone set in our property found in the array of bones of the reference skeletal mesh skeleton?
			if (RefSkeletonBoneName == BoneName)
			{
				FVector Location = FVector::ZeroVector;

				const TArray<FTransform>& ReferenceSkeletonBoneArray = SkeletalMesh->GetRefSkeleton().GetRefBonePose();
				int32 ParentIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(BoneName);

				FVector ChildLocation = FVector::ForwardVector;

				for (int32 i = 0; i < SkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++i)
				{
					if (SkeletalMesh->GetRefSkeleton().GetParentIndex(i) == ParentIndex)
					{
						ChildLocation = ReferenceSkeletonBoneArray[i].TransformPosition(FVector::ZeroVector);
						break;
					}
				}
			
				while (ParentIndex >= 0)
				{
					Location = ReferenceSkeletonBoneArray[ParentIndex].TransformPosition(Location);
					ChildLocation = ReferenceSkeletonBoneArray[ParentIndex].TransformPosition(ChildLocation);
					ParentIndex = SkeletalMesh->GetRefSkeleton().GetParentIndex(ParentIndex);
				}

				const FVector Direction = (ChildLocation - Location).GetSafeNormal();

				// Update the properties of the node
				Origin = Location;
				Normal = Direction;

				// Invert the normal if declared by the property
				if (bInvertNormal)
				{
					InvertNormals();
				}
				
				break;
			}
		}
	}
}


TStrongObjectPtr<USkeletalMesh> UCustomizableObjectNodeModifierClipMorph::GetReferenceSkeletalMesh() const
{
	TStrongObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	if (ReferenceSkeletonComponent.IsNone())
	{
		return SkeletalMesh;
	}
	
	if (UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(GetCustomizableObjectGraph()->GetOuter()))
	{
		const FName ReferenceComponentName = ReferenceSkeletonComponent;
		SkeletalMesh = TStrongObjectPtr(CustomizableObject->GetComponentMeshReferenceSkeletalMesh(ReferenceComponentName));
	}

	return SkeletalMesh;
}


UEdGraphPin* UCustomizableObjectNodeModifierClipMorph::GetOutputPin() const
{
	return FindPin(TEXT("Modifier"));
}


void UCustomizableObjectNodeModifierClipMorph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	TSharedPtr<FCustomizableObjectEditor> Editor = StaticCastSharedPtr<FCustomizableObjectEditor>(GetGraphEditor());

	// Hide the gizmo in cases where incomplete data is to be found in the node
	if (Editor)
	{
		// if no name for the bone has been provided hide the gizmo
		if (BoneName.IsNone())
		{
			Editor->HideGizmoClipMorph(false);
			return;
		}

		// Hide gizmo if the component is empty
		if (ReferenceSkeletonComponent.IsNone())
		{
			Editor->HideGizmoClipMorph(false);
			return;
		}
		
		// Hide gizmo if the component name can not resolve to a skeletal mesh
		if (const TStrongObjectPtr<USkeletalMesh> SkeletalMesh = GetReferenceSkeletalMesh(); !SkeletalMesh)
		{
			Editor->HideGizmoClipMorph(false);
			return;
		}
	}

	// Apply the changes provoked by the change of one of the properties of the node
	{
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();
		
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierClipMorph, bLocalStartOffset))
		{
			ChangeStartOffsetTransform();
		}
		// Handle a change in the name of the bone being used
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierClipMorph, BoneName))
		{
			UpdateOriginAndNormal();
		}
		// Handle the inversion of the normals
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierClipMorph, bInvertNormal))
		{
			InvertNormals();
		}
	}
	
	
	// Update the viewport with the data we changed when updating the properties of the node
	if (Editor)
	{
		Editor->ShowGizmoClipMorph(*this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeModifierClipMorph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PostLoadToCustomVersion &&
		bOldOffset_DEPRECATED &&
		bLocalStartOffset)
	{
		// Previous Offset
		FVector Tangent, Binormal;
		Origin.FindBestAxisVectors(Tangent, Binormal);
		FVector OldOffset = StartOffset.X * Tangent + StartOffset.Y * Binormal + StartOffset.Z * Normal;

		// Local Offset
		FVector XAxis, YAxis, ZAxis;
		FindLocalAxes(XAxis, YAxis, ZAxis);

		StartOffset = FVector(FVector::DotProduct(OldOffset, XAxis), FVector::DotProduct(OldOffset, YAxis),
							  FVector::DotProduct(OldOffset, ZAxis));
	}
}


void UCustomizableObjectNodeModifierClipMorph::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	
	CustomCreatePin(EGPD_Output, Schema->PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


FText UCustomizableObjectNodeModifierClipMorph::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("ClipMeshWithPlaneAndMorph", "Clip Mesh With Plane and Morph");
}


void UCustomizableObjectNodeModifierClipMorph::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	if (Pin == GetOutputPin())
	{
		TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = GetGraphEditor();

		if (Editor.IsValid())
		{
			Editor->UpdateGraphNodeProperties();
		}
	}
}


void UCustomizableObjectNodeModifierClipMorph::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UnifyRequiredTags)
	{
		RequiredTags = Tags_DEPRECATED;
		Tags_DEPRECATED.Empty();
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SnapToBoneComponentIndexToName)
	{
		ReferenceSkeletonComponent = FName(FString::FromInt(ReferenceSkeletonIndex_DEPRECATED));
	}
}


FText UCustomizableObjectNodeModifierClipMorph::GetTooltipText() const
{
	return LOCTEXT("Clip_Mesh_Morph_Tooltip", "Defines a cutting plane on a bone to cut tagged Materials that go past it, while morphing the mesh after the cut to blend in more naturally.\nIt only cuts and morphs mesh that receives some influence of that bone or other descendant bones.");
}


#undef LOCTEXT_NAMESPACE
