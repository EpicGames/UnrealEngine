// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithMesh.h"

#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/GraphTraversal.h"

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeModifierClipWithMesh::UCustomizableObjectNodeModifierClipWithMesh()
{
	Transform = FTransform::Identity;
}


void UCustomizableObjectNodeModifierClipWithMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	CustomCreatePin(EGPD_Input, Schema->PC_Mesh, TEXT("Clip Mesh"), LOCTEXT("ClipMesh", "Clip Mesh"));

	CustomCreatePin(EGPD_Output, Schema->PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


void UCustomizableObjectNodeModifierClipWithMesh::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UnifyRequiredTags)
	{
		RequiredTags = Tags_DEPRECATED;
		Tags_DEPRECATED.Empty();
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ModifierClipWithMeshCleanup)
	{
		for ( const FGuid& MeshSectionNodeID : ArrayMaterialNodeToClipWithID_DEPRECATED )
		{
			// Look for the parent material and set it as the modifier reference material 

			if (UCustomizableObjectNode* ParentNode = GetCustomizableObjectExternalNode<UCustomizableObjectNode>(CustomizableObjectToClipWith_DEPRECATED.Get(), MeshSectionNodeID))
			{
				// Add an autogenerated tag to the legacy parent, so that it can be referenced in this modifier.
				FString NewLegacyTag = MakeNodeAutoTag(ParentNode);
				RequiredTags.Add(NewLegacyTag);

				if (TArray<FString>* NodeEnableTags = ParentNode->GetEnableTagsArray())
				{
					NodeEnableTags->AddUnique(NewLegacyTag);
				}
				else
				{
					// Conversion failed?
					ensure(false);
					UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierClipWithMesh version upgrade failed.", *GetOutermost()->GetName());
				}

				// If the tag was added to another CO, keep track of the addition to be able to repeat it in case
				// that CO is not re-saved but this one is.
				UCustomizableObject* ThisNodeObject = GraphTraversal::GetObject(*this);
				if (CustomizableObjectToClipWith_DEPRECATED != ThisNodeObject)
				{
					FLegacyTag LegacyTag;
					LegacyTag.ParentObject = CustomizableObjectToClipWith_DEPRECATED;
					LegacyTag.ParentNode = MeshSectionNodeID;
					LegacyTag.Tag = NewLegacyTag;
					LegacyBackportsRequiredTags.AddUnique(LegacyTag);
				}
			}
			else
			{
				UE_LOGF(LogMutable, Log, "[%ls] UCustomizableObjectNodeModifierClipWithMesh has no parent. It will not be upgraded.", *GetOutermost()->GetName());
			}
		}

		ArrayMaterialNodeToClipWithID_DEPRECATED.Empty();
	}
}


void UCustomizableObjectNodeModifierClipWithMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}
	
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierClipWithMesh, Transform))
	{
		if (TransformChangedDelegate.IsBound())
		{
			TransformChangedDelegate.Broadcast(Transform);
		}
	}
}


FText UCustomizableObjectNodeModifierClipWithMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Clip_Mesh_With_Mesh_Modifier", "Clip Mesh With Mesh");
}


UEdGraphPin* UCustomizableObjectNodeModifierClipWithMesh::GetOutputPin() const
{
	return FindPin(TEXT("Modifier"));
}


UEdGraphPin* UCustomizableObjectNodeModifierClipWithMesh::GetClipMeshPin() const
{
	return FindPin(TEXT("Clip Mesh"));
}


FText UCustomizableObjectNodeModifierClipWithMesh::GetTooltipText() const
{
	return LOCTEXT("Clip_Mesh_With_Mesh_Modifier_Tooltip", "Removes the part of a mesh section that is completely enclosed in a mesh volume.\nIt only removes the faces that fall completely inside the cutting volume, along with the vertices and edges that define only faces that are deleted.");
}

#undef LOCTEXT_NAMESPACE
