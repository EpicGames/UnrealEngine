// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeMeshInterface.generated.h"

class UCustomizableObjectLayout;
class UEdGraphPin;
class UObject;
class UTexture2D;


UINTERFACE(MinimalAPI)
class UCustomizableObjectNodeMeshInterface : public UInterface
{
	GENERATED_BODY()
};


class ICustomizableObjectNodeMeshInterface
{
public:
	GENERATED_BODY()

	// Own interface
	/** Returns the multiple Layouts of a given Mesh pin. A pin can have multiple Layouts since it can have multiple UVs. Override. */
	virtual TArray<UCustomizableObjectLayout*> GetLayouts(const UEdGraphPin& OutPin) const PURE_VIRTUAL(UCustomizableObjectNodeMesh::GetLayouts, return {};);

	virtual UTexture2D* FindTextureForPin(const UEdGraphPin* Pin) const PURE_VIRTUAL(UCustomizableObjectNodeMesh::FindTextureForPin, return {};);

	/** Returns the Unreal mesh (e.g., USkeletalMesh, UStaticMesh...). */
	virtual TSoftObjectPtr<UStreamableRenderAsset> GetMesh() const PURE_VIRTUAL(UCustomizableObjectNodeMesh::GetMesh, return {};);

	/** Returns the output Mesh pin associated to the given LODIndex and SectionIndex. Override. */
	virtual UEdGraphPin* GetMeshPin(int32 LOD, int32 SectionIndex) const PURE_VIRTUAL(UCustomizableObjectNodeMesh::GetMeshPin, return {};);

	/** Given a pin, return the Section and Layout index.
	 *
	 * Will always return a valid result. The result can be valid but out of sync with respect the Unreal mesh asset.
	 * In other words, the pin still represents a LOD 3 but the asset no longer has this third LOD.
	 * 
	 * @param Pin In. Node's owning pin.
	 * @param LODIndex Out.
	 * @param SectionIndex Out. */
	virtual void GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const PURE_VIRTUAL(UCustomizableObjectNodeMesh::GetPinSection, );
};

