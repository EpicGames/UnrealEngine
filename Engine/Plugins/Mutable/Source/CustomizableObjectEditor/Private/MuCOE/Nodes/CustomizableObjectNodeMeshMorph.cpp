// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorph.h"

#include "CustomizableObjectNodeStaticString.h"
#include "Engine/SkeletalMesh.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeCommon.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshape.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMeshMorph)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeMeshMorph::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Mesh"), LOCTEXT("Mesh", "Mesh"));

	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Mesh"), LOCTEXT("Mesh", "Mesh"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Factor"), LOCTEXT("Factor", "Factor"));
	MorphTargetNamePinRef = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Morph Target Name"), LOCTEXT("MorphTargetName", "Morph Target Name"));
}


FText UCustomizableObjectNodeMeshMorph::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Mesh_Morph", "Mesh Morph");
}


FLinearColor UCustomizableObjectNodeMeshMorph::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


UCustomizableObjectNodeSkeletalMesh* UCustomizableObjectNodeMeshMorph::GetSourceSkeletalMesh() const
{
	if (const UEdGraphPin* Pin = MeshPin())
	{
		if (const UEdGraphPin* ConnectedPin = FollowInputPin(*Pin))
		{
			UEdGraphNode* InMeshNode = ConnectedPin->GetOwningNode();
			if (UCustomizableObjectNodeSkeletalMesh* SkeletalMeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(InMeshNode))
			{
				return SkeletalMeshNode;
			}
			else if (UCustomizableObjectNodeMeshMorph* MorphNode = Cast<UCustomizableObjectNodeMeshMorph>(InMeshNode))
			{
				return MorphNode->GetSourceSkeletalMesh();
			}
		}
	}

	return nullptr;
}


bool UCustomizableObjectNodeMeshMorph::IsNodeOutDatedAndNeedsRefresh()
{
	bool Result = false;

	const UCustomizableObjectNodeSkeletalMesh* SkeletalMeshNode = GetSourceSkeletalMesh();

	if (SkeletalMeshNode && SkeletalMeshNode->SkeletalMesh)
	{
		bool bOutdated = true;
		for (int m = 0; m < SkeletalMeshNode->SkeletalMesh->GetMorphTargets().Num(); ++m)
		{
			FString MorphName = SkeletalMeshNode->SkeletalMesh->GetMorphTargets()[m]->GetName();
			if (MorphTargetName == MorphName)
			{
				bOutdated = false;
				break;
			}
		}
		Result = bOutdated;
	}

	// Remove previous compilation warnings
	if (!Result && bHasCompilerMessage)
	{
		RemoveWarnings();
		GetGraph()->NotifyGraphChanged();
	}

	return Result;
}


FString UCustomizableObjectNodeMeshMorph::GetRefreshMessage() const
{
    return "Morph Target not found in the SkeletalMesh. Please Refresh Node and select a valid morph option.";
}


void UCustomizableObjectNodeMeshMorph::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	if (Pin->Direction == EGPD_Input && Pin->PinName == "Mesh")
	{
		TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = GetGraphEditor();

		if (Editor.IsValid())
		{
			Editor->UpdateGraphNodeProperties();
		}
	}
}


FText UCustomizableObjectNodeMeshMorph::GetTooltipText() const
{
	return LOCTEXT("Mesh_Morph_Tooltip", "Changes the weight of a mesh morph target.");
}


UEdGraphPin* UCustomizableObjectNodeMeshMorph::MorphTargetNamePin() const
{
	return MorphTargetNamePinRef.Get();
}


void UCustomizableObjectNodeMeshMorph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::DeformSkeletonOptionsAdded)
	{
		if (bDeformAllBones_DEPRECATED)
		{
			SelectionMethod_DEPRECATED = EBoneDeformSelectionMethod::ALL_BUT_SELECTED;
			BonesToDeform_DEPRECATED.Empty();
		}
	}
}


bool UCustomizableObjectNodeMeshMorph::CanRenamePin(const UEdGraphPin& Pin) const
{
	return  &Pin == MorphTargetNamePin();
}


FText UCustomizableObjectNodeMeshMorph::GetPinEditableName(const UEdGraphPin& Pin) const
{
	return CanRenamePin(Pin) == true ? FText::FromString(MorphTargetName) : FText();
}


void UCustomizableObjectNodeMeshMorph::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	if (CanRenamePin(Pin))
	{
		MorphTargetName = Value.ToString();
	}
}

FName UCustomizableObjectNodeMeshMorph::GetMorphTargetName(FMutableGraphGenerationContext& GenerationContext) const
{
	const UEdGraphPin* InputMorphNamePin = MorphTargetNamePin();
	check(InputMorphNamePin);
	if (const UEdGraphPin* ConnectedPin = FollowInputPin(*InputMorphNamePin))
	{
		if (const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedPin, &GenerationContext.MacroNodesStack))
		{
			if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
			{
				// Use the value set in the connected Static string node
				return FName(StringNode->Value);
			}
			else
			{
				GenerationContext.Log(LOCTEXT("MorphStringNodeTypeNotHandled", "Could not extract the target name from the connected string providing node."), this, EMessageSeverity::Error);
				return NAME_None;
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("MorphStringNodeFailed", "Could not find a linked String node."), this, EMessageSeverity::Error);
			return NAME_None;
		}
	}
	else
	{
		// If no connection is found to the input name pin then use the value stored locally
		return FName(MorphTargetName);
	}
}


void UCustomizableObjectNodeMeshMorph::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!MorphTargetNamePinRef.Get())
		{
			MorphTargetNamePinRef = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Morph Target Name"), LOCTEXT("MorphTargetName", "Morph Target Name"));
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MorphReshapeMigratedToReshapeNode)
	{
		if (!bReshapeSkeleton_DEPRECATED && !bReshapePhysicsVolumes_DEPRECATED)
		{
			return;
		}

		UEdGraph* OwningGraph = GetGraph();
		UEdGraphPin* InputMeshPin = MeshPin();
		UEdGraphPin* OutputMeshPin = FindPin(TEXT("Mesh"), EGPD_Output);
		if (!OwningGraph || !InputMeshPin || !OutputMeshPin || InputMeshPin->LinkedTo.Num() == 0)
		{
			return;
		}

		UEdGraphPin* BaseSourcePin = InputMeshPin->LinkedTo[0];

		// Snapshot downstream consumers of the morph output before mutating any link state.
		TArray<UEdGraphPin*> DownstreamPins = OutputMeshPin->LinkedTo;

		UCustomizableObjectNodeMeshReshape* NewReshape = NewObject<UCustomizableObjectNodeMeshReshape>(OwningGraph);
		NewReshape->SetFlags(RF_Transactional);
		OwningGraph->AddNode(NewReshape, /*bUserAction=*/false, /*bSelectNewNode=*/false);
		NewReshape->CreateNewGuid();
		NewReshape->PostPlacedNewNode();
		NewReshape->BeginConstruct();
		NewReshape->PostBackwardsCompatibleFixup();
		NewReshape->ReconstructNode();

		// bReshapeVertices=false is what makes this equivalent to the old ASTOpMeshMorphReshape:
		// the morphed vertices flow through the Reshape node untouched while the skeleton/physics
		// get re-bound (Base Shape = original) and re-applied (Target Shape = morphed).
		NewReshape->bReshapeVertices = false;
		NewReshape->bRecomputeNormals = false;
		NewReshape->bApplyLaplacianSmoothing = false;
		NewReshape->bReshapePose = bReshapeSkeleton_DEPRECATED;
		NewReshape->bReshapePhysics = bReshapePhysicsVolumes_DEPRECATED;
		NewReshape->SelectionMethod = SelectionMethod_DEPRECATED;
		NewReshape->PhysicsSelectionMethod = PhysicsSelectionMethod_DEPRECATED;
		NewReshape->BonesToDeform_DEPRECATED = BonesToDeform_DEPRECATED;
		NewReshape->PhysicsBodiesToDeform_DEPRECATED = PhysicsBodiesToDeform_DEPRECATED;

		NewReshape->NodePosX = NodePosX + 320;
		NewReshape->NodePosY = NodePosY;

		UEdGraphPin* ReshapeBaseMesh = NewReshape->BaseMeshPin();
		UEdGraphPin* ReshapeBaseShape = NewReshape->BaseShapePin();
		UEdGraphPin* ReshapeTargetShape = NewReshape->TargetShapePin();
		UEdGraphPin* ReshapeOutput = NewReshape->FindPin(TEXT("Mesh"), EGPD_Output);
		if (!ReshapeBaseMesh || !ReshapeBaseShape || !ReshapeTargetShape || !ReshapeOutput)
		{
			return;
		}

		for (UEdGraphPin* Consumer : DownstreamPins)
		{
			OutputMeshPin->BreakLinkTo(Consumer);
		}

		OutputMeshPin->MakeLinkTo(ReshapeBaseMesh);
		OutputMeshPin->MakeLinkTo(ReshapeTargetShape);
		BaseSourcePin->MakeLinkTo(ReshapeBaseShape);

		for (UEdGraphPin* Consumer : DownstreamPins)
		{
			ReshapeOutput->MakeLinkTo(Consumer);
		}
	}
}



#undef LOCTEXT_NAMESPACE
