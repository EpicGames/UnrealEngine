// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeCommon.h"

#include "CustomizableObjectNodeMeshMorph.generated.h"

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FMutableGraphGenerationContext;
struct FMeshReshapeBoneReference;
enum class EBoneDeformSelectionMethod : uint8;


UCLASS()
class UCustomizableObjectNodeMeshMorph : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual bool IsNodeOutDatedAndNeedsRefresh() override;
	virtual FString GetRefreshMessage() const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;

protected:
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

public:
	
	// Own interface
	class UCustomizableObjectNodeSkeletalMesh* GetSourceSkeletalMesh() const;

	UEdGraphPin* MeshPin() const
	{
		return FindPin(TEXT("Mesh"), EGPD_Input);
	}

	UEdGraphPin* FactorPin() const
	{
		return FindPin(TEXT("Factor"));
	}

	UEdGraphPin* MorphTargetNamePin() const;

	virtual void Serialize(FArchive& Ar) override;
	
	FName GetMorphTargetName(FMutableGraphGenerationContext& GenerationContext) const;

private:
	UPROPERTY()
	FString MorphTargetName;

public:

	UPROPERTY()
	bool bReshapeSkeleton_DEPRECATED = false;

	UPROPERTY()
	bool bReshapePhysicsVolumes_DEPRECATED = false;

	UPROPERTY()
	EBoneDeformSelectionMethod SelectionMethod_DEPRECATED = EBoneDeformSelectionMethod::ONLY_SELECTED;

	UPROPERTY()
	EBoneDeformSelectionMethod PhysicsSelectionMethod_DEPRECATED = EBoneDeformSelectionMethod::ONLY_SELECTED;

	UPROPERTY()
	TArray<FMeshReshapeBoneReference> BonesToDeform_DEPRECATED;

	UPROPERTY()
	TArray<FMeshReshapeBoneReference> PhysicsBodiesToDeform_DEPRECATED;

	UPROPERTY()
	bool bDeformAllBones_DEPRECATED = false;

	UPROPERTY()
	bool bDeformAllPhysicsBodies_DEPRECATED = false;

private:

	UPROPERTY()
	FEdGraphPinReference MorphTargetNamePinRef;

};

#undef UE_API
