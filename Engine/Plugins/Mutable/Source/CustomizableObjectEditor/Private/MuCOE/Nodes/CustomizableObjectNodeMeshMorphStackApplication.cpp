// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackApplication.h"

#include "Engine/SkeletalMesh.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMeshMorphStackApplication)

class UCustomizableObjectNodeRemapPins;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeMeshMorphStackApplication::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Input pins
	
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh);
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Stack);
	
	// Output pins
	
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh);
}


void UCustomizableObjectNodeMeshMorphStackApplication::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* InMeshPin = FindPin(TEXT("InMesh"), EGPD_Input))
		{
			InMeshPin->PinName = TEXT("Mesh");
			InMeshPin->PinFriendlyName = LOCTEXT("Mesh_Pin_Category", "Mesh");
		}

		if (UEdGraphPin* InStackPin = FindPin(TEXT("Stack"), EGPD_Input))
		{
			InStackPin->PinFriendlyName = LOCTEXT("Stack_Pin_Category", "Stack");
		}
		
		if (UEdGraphPin* OutMeshPin = FindPin(TEXT("Result Mesh"), EGPD_Output))
		{
			OutMeshPin->PinName = TEXT("Mesh");
			OutMeshPin->PinFriendlyName = LOCTEXT("Mesh_Pin_Category", "Mesh");
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MorphStackCategoryName)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinName == TEXT("Stack"))
			{
				Pin->PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_Stack);
			}
		}
	}
}


FText UCustomizableObjectNodeMeshMorphStackApplication::GetNodeTitle(ENodeTitleType::Type TittleType)const
{
	return LOCTEXT("Mesh_Morph_Stack_Application", "Mesh Morph Stack Application");
}


FLinearColor UCustomizableObjectNodeMeshMorphStackApplication::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


FText UCustomizableObjectNodeMeshMorphStackApplication::GetTooltipText() const
{
	return LOCTEXT("Morph_Stack_Application_Tooltip","Applies a morph stack to a mesh");
}


UEdGraphPin* UCustomizableObjectNodeMeshMorphStackApplication::GetMeshPin() const
{
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_Mesh);
	return FindPin(PinName, EGPD_Input);
}


UEdGraphPin* UCustomizableObjectNodeMeshMorphStackApplication::GetStackPin() const
{
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_Stack);
	return FindPin(PinName, EGPD_Input);
}

TArray<FString> UCustomizableObjectNodeMeshMorphStackApplication::GetMorphList() const
{
	const UEdGraphPin* MeshPin = GetMeshPin();
	if (!MeshPin)
	{
		return {};
	}

	const UEdGraphPin* OutputMeshPin = FollowInputPin(*MeshPin);
	if (!OutputMeshPin)
	{
		return {};
	}

	TArray<FString> MorphNames;

	const UEdGraphPin* MeshNodePin = FindMeshBaseSource(*OutputMeshPin, false);

	if (MeshNodePin && MeshNodePin->GetOwningNode())
	{
		USkeletalMesh* SkeletalMesh = nullptr;

		if (const ICustomizableObjectNodeMeshInterface* MeshNode = Cast<ICustomizableObjectNodeMeshInterface>(MeshNodePin->GetOwningNode()))
		{
			SkeletalMesh = Cast<USkeletalMesh>(UE::Mutable::Private::LoadObject(MeshNode->GetMesh()));
		}
		else if (const UCustomizableObjectNodeTable* TableNode = Cast< UCustomizableObjectNodeTable >(MeshNodePin->GetOwningNode()))
		{
			SkeletalMesh = TableNode->GetColumnDefaultAssetByType<USkeletalMesh>(MeshNodePin);
		}
		else
		{
			unimplemented();
		}

		if (SkeletalMesh)
		{
			for (int32 i = 0; i < SkeletalMesh->GetMorphTargets().Num(); ++i)
			{
				MorphNames.Add(SkeletalMesh->GetMorphTargets()[i]->GetName());
			}
		}
	}

	return MorphNames;
}


#undef LOCTEXT_NAMESPACE
