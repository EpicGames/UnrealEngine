// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeMeshMorphStackApplication.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeMeshMorphStackApplication : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TittleType)const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	// Own interface
	/** Fills the list with all the morphs */
	UE_API TArray<FString> GetMorphList() const;
	
	/** Returns the mesh pin. */
	UE_API UEdGraphPin* GetMeshPin() const;

	/** Returns the stack pin. */
	UE_API UEdGraphPin* GetStackPin() const;
};

#undef UE_API
