// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshMorphStackDefinition.h"

#include "CONodeComponentSkeletalMesh.h"
#include "ScopedTransaction.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMeshMorphStackDefinition)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeMeshMorphStackDefinition::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MorphStackDefinition_RemovedMorphAutoPopulate)
	{
		{
			// Remove the MeshPin as we are no longer going to use it
			if (UEdGraphPin* MeshPin = FindPin(TEXT("Mesh")))
			{		
				// At this point the pin should be safe to remove
				Super::CustomRemovePin(*MeshPin);
			}
		
			auto RemappingFunctionToFixMorphStackPinName = [&](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* Remapper) -> void
			{
				// Change the structure of the data in the Morph target pins so they no longer hold the name of the morph on the pinname but on its pindata
				check(MorphTargetPinReferences.IsEmpty());
				for (int32 MorphNameIndex = 0; MorphNameIndex < MorphNames_DEPRECATED.Num(); ++MorphNameIndex)
				{
					UCONodeMeshMorphStackDefinitionPinData* MorphTargetPinData = NewObject<UCONodeMeshMorphStackDefinitionPinData>(this);
					MorphTargetPinData->TargetMorphName = *MorphNames_DEPRECATED[MorphNameIndex];

					const FString PinName = FString("Target");
					UEdGraphPin* MorphPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, FName(*PinName), FText::FromString(PinName), MorphTargetPinData);
					MorphTargetPinReferences.Add(MorphPin);
				}
			
				// Correct the name of the Mesh Morph pin from "Stack" to "Morph Stack" 
				CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Stack);
			};
			
			FixupReconstructPins(CreateRemapPinsByPosition(), RemappingFunctionToFixMorphStackPinName);
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MorphStackDefinition_RemoveOrphanMeshPin)
	{
		if (UEdGraphPin* MeshPin = FindPin(TEXT("Mesh")))
		{		
			// At this point the pin should be safe to remove
			Super::CustomRemovePin(*MeshPin);
		}
	}
}


void UCustomizableObjectNodeMeshMorphStackDefinition::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Input pins
	MorphTargetPinReferences.Empty();
	TArray<UEdGraphPin*> NonOrphanPins = GetAllNonOrphanPins();
	for (const UEdGraphPin* NonOrphanPin : NonOrphanPins)
	{
		if (!NonOrphanPin)
		{
			continue;
		}
		
		if (!IsMorphTargetPin(*NonOrphanPin))
		{
			continue;
		}
		
		// Retrieve the old pin data
		const UCONodeMeshMorphStackDefinitionPinData* OldMorphTargetPinData = CastChecked<UCONodeMeshMorphStackDefinitionPinData>(GetPinData(*NonOrphanPin));
		
		// Create a new one based on the old one
		UCONodeMeshMorphStackDefinitionPinData* NewMorphTargetPinData = NewObject<UCONodeMeshMorphStackDefinitionPinData>(this);
		NewMorphTargetPinData->TargetMorphName = OldMorphTargetPinData->TargetMorphName;
			
		// Add a new pin
		UEdGraphPin* NewPin = CreateMorphTargetPin(*NewMorphTargetPinData);
		check(NewPin);
		MorphTargetPinReferences.Add(NewPin);
	}

	// Output pins
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Stack);
}


UCustomizableObjectNodeRemapPins* UCustomizableObjectNodeMeshMorphStackDefinition::CreateRemapPinsDefault() const
{
	return CreateRemapPinsByPosition();
}


bool UCustomizableObjectNodeMeshMorphStackDefinition::IsMorphTargetPin(const UEdGraphPin& Pin) const
{
	return Pin.Direction == EGPD_Input && Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Float;
}


UEdGraphPin* UCustomizableObjectNodeMeshMorphStackDefinition::CreateMorphTargetPin(UCONodeMeshMorphStackDefinitionPinData& InPinData)
{
	const FString PinName = FString("Target");
	UEdGraphPin* MorphPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, FName(*PinName), FText::FromString(PinName), &InPinData);
	return MorphPin;
}


FText UCustomizableObjectNodeMeshMorphStackDefinition::GetNodeTitle(ENodeTitleType::Type TittleType)const
{
	return LOCTEXT("Mesh_Morph_Stack_Definition", "Morph Stack Definition");
}


FLinearColor UCustomizableObjectNodeMeshMorphStackDefinition::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Stack);
}


FText UCustomizableObjectNodeMeshMorphStackDefinition::GetTooltipText() const
{
	return LOCTEXT("Morph_Stack_Definition_Tooltip","Allows to stack morphs");
}


FText UCustomizableObjectNodeMeshMorphStackDefinition::GetPinEditableName(const UEdGraphPin& Pin) const
{
	if (ensure(IsMorphTargetPin(Pin)))
	{
		if (const UCONodeMeshMorphStackDefinitionPinData* TypedMorphTargetPinData = Cast<UCONodeMeshMorphStackDefinitionPinData>(GetPinData(Pin)))
		{
			return FText::FromName(TypedMorphTargetPinData->TargetMorphName);
		}
	}
	
	return Super::GetPinEditableName(Pin);
}


void UCustomizableObjectNodeMeshMorphStackDefinition::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	if (ensure(IsMorphTargetPin(Pin)))
	{
		FScopedTransaction LocalTransaction(LOCTEXT("NodeMeshMorphStackDefinition_SetNamePinEditableNameTransaction", "Change Pin Name"));
		
		UCONodeMeshMorphStackDefinitionPinData* TypedMorphTargetPinData = CastChecked<UCONodeMeshMorphStackDefinitionPinData>(GetPinData(Pin));
		TypedMorphTargetPinData->Modify();
		
		TypedMorphTargetPinData->TargetMorphName = FName{Value.ToString()};
		PinEditableNameChangedDelegate.Broadcast(Pin, Value);
	}
	else
	{
		Super::SetPinEditableName(Pin, Value);
	}
}


EEditablePinNameBoxVisibilityPolicy UCustomizableObjectNodeMeshMorphStackDefinition::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


EAddPinNodeButtonLocation UCustomizableObjectNodeMeshMorphStackDefinition::GetAddPinButtonNodeSide() const
{
	return EAddPinNodeButtonLocation::INPUT;
}


void UCustomizableObjectNodeMeshMorphStackDefinition::AddPinFromUI()
{
	Super::AddPinFromUI();
	
	FScopedTransaction Transaction(LOCTEXT("NodeMeshMorphStackDefinition_AddPinFromNodeUITransaction", "Add Pin from Node UI"));
	Modify();
	
	// Create a new one based on the old one
	UCONodeMeshMorphStackDefinitionPinData* NewMorphTargetPinData = NewObject<UCONodeMeshMorphStackDefinitionPinData>(this);
			
	// Add a new pin
	if (const UEdGraphPin* NewPin = CreateMorphTargetPin(*NewMorphTargetPinData))
	{
		MorphTargetPinReferences.Add(NewPin);
		GetGraph()->NotifyNodeChanged(this);
	}
}


bool UCustomizableObjectNodeMeshMorphStackDefinition::CanPinBeRemoved(const UEdGraphPin& Pin) const
{
	return IsMorphTargetPin(Pin);
}


bool UCustomizableObjectNodeMeshMorphStackDefinition::CustomRemovePin(UEdGraphPin& Pin)
{
	if (IsMorphTargetPin(Pin))
	{
		MorphTargetPinReferences.Remove(FEdGraphPinReference(&Pin));
	}

	return Super::CustomRemovePin(Pin);
}


bool UCustomizableObjectNodeMeshMorphStackDefinition::CanRenamePin(const UEdGraphPin& Pin) const
{
	return IsMorphTargetPin(Pin);
}


FName UCustomizableObjectNodeMeshMorphStackDefinition::GetMorphTargetName(const UEdGraphPin& Pin) const
{
	if (ensure(IsMorphTargetPin(Pin)))
	{
		const UCONodeMeshMorphStackDefinitionPinData* TypedMorphTargetPinData = CastChecked<UCONodeMeshMorphStackDefinitionPinData>(GetPinData(Pin));
		return TypedMorphTargetPinData->TargetMorphName;
	}
	
	return NAME_None;
}


#undef LOCTEXT_NAMESPACE
