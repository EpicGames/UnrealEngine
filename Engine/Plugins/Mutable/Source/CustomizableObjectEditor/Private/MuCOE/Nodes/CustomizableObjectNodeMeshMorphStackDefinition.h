// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPins.h"

#include "CustomizableObjectNodeMeshMorphStackDefinition.generated.h"


class UEdGraphPin;
class UObject;


UCLASS()
class UCONodeMeshMorphStackDefinitionPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = NoCategory)
	FName TargetMorphName = NAME_None;
};



UCLASS()
class UCustomizableObjectNodeMeshMorphStackDefinition : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TittleType)const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const override;
	virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const override;
	virtual void AddPinFromUI() override;
	virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const override;
	virtual bool CustomRemovePin(UEdGraphPin& Pin) override;
	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;

	FName GetMorphTargetName(const UEdGraphPin& Pin) const;
	
protected:
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;

private:
	
	bool IsMorphTargetPin(const UEdGraphPin& Pin) const;
	
	UEdGraphPin* CreateMorphTargetPin(UCONodeMeshMorphStackDefinitionPinData& InPinData);

public:

	UPROPERTY()
	TArray<FEdGraphPinReference> MorphTargetPinReferences;
	
private:
	/** List with all the morphs of the linked skeletal mesh. */
	UPROPERTY()
	TArray<FString> MorphNames_DEPRECATED;
};

