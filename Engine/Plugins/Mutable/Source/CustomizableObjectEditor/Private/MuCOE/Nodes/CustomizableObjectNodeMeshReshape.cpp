// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshape.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMeshReshape)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"



void UCustomizableObjectNodeMeshReshape::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output,UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Mesh"), LOCTEXT("Mesh", "Mesh"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Base Mesh"), LOCTEXT("BaseMesh", "Base Mesh"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Base Shape"), LOCTEXT("BaseShape", "Base Shape"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Target Shape"), LOCTEXT("TargetShape", "Target Shape"));
}


FText UCustomizableObjectNodeMeshReshape::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Mesh_Reshape", "Mesh Reshape");
}


FLinearColor UCustomizableObjectNodeMeshReshape::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Mesh);
}


FText UCustomizableObjectNodeMeshReshape::GetTooltipText() const
{
	return LOCTEXT("Mesh_Reshape_Tooltip", "Apply a mesh reshape on a mesh.");
}


FString UCustomizableObjectNodeMeshReshape::GetRefreshMessage() const
{
	return "One or more bones selected to deform do not exist in the current reference mesh skeleton!";
}


void UCustomizableObjectNodeMeshReshape::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::DeformSkeletonOptionsAdded)
	{
		if (bDeformAllBones_DEPRECATED)
		{
			SelectionMethod = EBoneDeformSelectionMethod::ALL_BUT_SELECTED;
			BonesToDeform_V2.Empty();
		}
	}

	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MeshReshapeVertexColorUsageSelection)
	{
		if (bEnableRigidParts_DEPRECATED)
		{
			VertexColorUsage.R = EMeshReshapeVertexColorChannelUsage::RigidClusterId;
			VertexColorUsage.G = EMeshReshapeVertexColorChannelUsage::RigidClusterId;
			VertexColorUsage.B = EMeshReshapeVertexColorChannelUsage::RigidClusterId;
			VertexColorUsage.A = EMeshReshapeVertexColorChannelUsage::RigidClusterId;
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MeshReshapeBonesToDeformAsFName)
	{
		BonesToDeform_V2.Reserve(BonesToDeform_DEPRECATED.Num());
		for (const FMeshReshapeBoneReference& Old : BonesToDeform_DEPRECATED)
		{
			BonesToDeform_V2.Add(Old.BoneName);
		}
		BonesToDeform_DEPRECATED.Empty();

		PhysicsBodiesToDeform_V2.Reserve(PhysicsBodiesToDeform_DEPRECATED.Num());
		for (const FMeshReshapeBoneReference& Old : PhysicsBodiesToDeform_DEPRECATED)
		{
			PhysicsBodiesToDeform_V2.Add(Old.BoneName);
		}
		PhysicsBodiesToDeform_DEPRECATED.Empty();
	}
}


#undef LOCTEXT_NAMESPACE
