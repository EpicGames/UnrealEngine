// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeMeshReshape.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FMeshReshapeBoneReference;
enum class EBoneDeformSelectionMethod : uint8;


UENUM()
enum class EMeshReshapeVertexColorChannelUsage
{
	None = 0,
	RigidClusterId = 1,
	MaskWeight = 2
};

USTRUCT()
struct FMeshReshapeColorUsage
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	EMeshReshapeVertexColorChannelUsage R = EMeshReshapeVertexColorChannelUsage::None;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	EMeshReshapeVertexColorChannelUsage G = EMeshReshapeVertexColorChannelUsage::None;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	EMeshReshapeVertexColorChannelUsage B = EMeshReshapeVertexColorChannelUsage::None;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	EMeshReshapeVertexColorChannelUsage A = EMeshReshapeVertexColorChannelUsage::None;
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeMeshReshape : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API FString GetRefreshMessage() const override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	inline UEdGraphPin* BaseMeshPin() const
	{
		return FindPin(TEXT("Base Mesh"), EGPD_Input);
	}

	inline UEdGraphPin* BaseShapePin() const
	{
		return FindPin(TEXT("Base Shape"), EGPD_Input);
	}

	inline UEdGraphPin* TargetShapePin() const
	{
		return FindPin(TEXT("Target Shape"), EGPD_Input);
	}
	
	/** Enable the deformation of the vertices of the base mesh. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bReshapeVertices = true;

	/** Enable recompute normals after the reshape operation. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (EditCondition = "bReshapeVertices"))
	bool bRecomputeNormals = false;

	/** Enable laplacian smoothing to the result of the base mesh reshape. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (EditCondition = "bReshapeVertices"))
	bool bApplyLaplacianSmoothing = false;

	/** Enable the deformation of the skeleton of the base mesh. */
	UPROPERTY(EditAnywhere, Category = ReshapePose)
	bool bReshapePose = false;

	/** Enable the deformation of physics volumes of the base mesh */
    UPROPERTY(EditAnywhere, Category = ReshapePhysics)
    bool bReshapePhysics = false;
	
	UPROPERTY()
	bool bEnableRigidParts_DEPRECATED = false;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FMeshReshapeColorUsage VertexColorUsage;

	/** Bone Reshape Selection Method */
	UPROPERTY(EditAnywhere, Category = ReshapePose, meta = (EditCondition = "bReshapePose"))
	EBoneDeformSelectionMethod SelectionMethod;

	/** Array with selected bones that will be deformed */
	UPROPERTY(EditAnywhere, Category = ReshapePose, DisplayName = "Bones To Deform", meta = (EditCondition = "bReshapePose"))
	TArray<FName> BonesToDeform_V2;

	UPROPERTY()
	TArray<FMeshReshapeBoneReference> BonesToDeform_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = ReshapePhysics, meta = (DisplayName = "Selection Method", EditCondition = "bReshapePhysics"))
	EBoneDeformSelectionMethod PhysicsSelectionMethod;

	/** Array with bones with physics bodies that will be deformed */
	UPROPERTY(EditAnywhere, Category = ReshapePhysics, DisplayName = "Physics Bodies To Deform", meta = (EditCondition = "bReshapePhysics"))
	TArray<FName> PhysicsBodiesToDeform_V2;

	UPROPERTY()
	TArray<FMeshReshapeBoneReference> PhysicsBodiesToDeform_DEPRECATED;

	UPROPERTY()
	bool bDeformAllBones_DEPRECATED = false;

	UPROPERTY()
	bool bDeformAllPhysicsBodies_DEPRECATED = false;
};

#undef UE_API
