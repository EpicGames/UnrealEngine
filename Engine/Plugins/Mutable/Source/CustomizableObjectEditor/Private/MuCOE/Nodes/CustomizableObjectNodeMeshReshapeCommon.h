// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CustomizableObjectNodeMeshReshapeCommon.generated.h"

UENUM()
enum class EBoneDeformSelectionMethod : uint8
{
	// Only selected bones will be deform
	ONLY_SELECTED = 0 UMETA(DisplayName = "Only Selected"),

	// All bones will be deform except the selected ones
	ALL_BUT_SELECTED = 1 UMETA(DisplayName = "All But Selected")
};


USTRUCT()
struct FMeshReshapeBoneReference
{
	GENERATED_USTRUCT_BODY()

	/** Name of the bone that will be deformed */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FName BoneName;
};
