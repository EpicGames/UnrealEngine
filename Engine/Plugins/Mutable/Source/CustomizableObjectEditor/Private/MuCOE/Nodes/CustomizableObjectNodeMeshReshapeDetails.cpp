// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshReshapeDetails.h"


class IDetailLayoutBuilder;

#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeMeshReshapeDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeMeshReshapeDetails);
}


void FCustomizableObjectNodeMeshReshapeDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
}


void FCustomizableObjectNodeMeshReshapeDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
}


#undef LOCTEXT_NAMESPACE
