// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;


// TODO: This is no longer used, remove.

class FCustomizableObjectNodeMeshReshapeDetails : public IDetailCustomization
{
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it 
	static TSharedRef<IDetailCustomization> MakeInstance();

	// ILayoutDetails interface
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

private:


private:

};
