// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshSectionDetails.h"

#include "CONodeMaterialModify.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/STextComboBox.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/Attribute.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeMeshSectionDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeMeshSectionDetails );
}


void FCustomizableObjectNodeMeshSectionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCONodeSkeletalMeshSection>( DetailsView->GetSelectedObjects()[0].Get() );
	}

	if (!Node)
	{
		return;
	}

	// Move tags to enable higher.
	IDetailCategoryBuilder& TagsCategory = DetailBuilder.EditCategory("Tags");
	TagsCategory.SetSortOrder(-5000);

	// Add the required tags widget
	{
		TagsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCONodeSkeletalMeshSection, Tags), UCONodeSkeletalMeshSection::StaticClass());
		DetailBuilder.HideProperty(TagsPropertyHandle);

		TagsPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeMeshSectionDetails::OnEnableTagsPropertyChanged));
		TagsPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeMeshSectionDetails::OnEnableTagsPropertyChanged));

		TagsCategory.AddCustomRow(FText::FromString(TEXT("Enable Tags")))
		.PropertyHandleList({ TagsPropertyHandle })
		.NameContent()
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.Padding(FMargin(0, 4.0f, 0, 4.0f))
				[
					SNew(STextBlock)
						.Text(LOCTEXT("MeshSectionDetails_Tags", "Enable Tags"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.IsEnabled(this, &FCustomizableObjectNodeMeshSectionDetails::IsTagsPropertyWidgetEnabled)
						.ToolTipText(this, &FCustomizableObjectNodeMeshSectionDetails::TagsPropertyWidgetTooltip)
				]
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SAssignNew(this->TagListWidget, SMutableTagListWidget)
				.Node(Node)
				.TagArray(&Node->Tags)
				.AllowInternalTags(false)
				.EmptyListText(LOCTEXT("MeshSectionDetails_NoTags", "No tags enabled by this mesh section."))
				.OnTagListChanged(this, &FCustomizableObjectNodeMeshSectionDetails::OnEnableTagsPropertyChanged)
				.IsEnabled(this, &FCustomizableObjectNodeMeshSectionDetails::IsTagsPropertyWidgetEnabled)
				.ToolTipText(this, &FCustomizableObjectNodeMeshSectionDetails::TagsPropertyWidgetTooltip)
		];
	}
}


void FCustomizableObjectNodeMeshSectionDetails::OnEnableTagsPropertyChanged()
{
	// This seems necessary to detect the "Reset to default" actions.
	TagListWidget->RefreshOptions();
	Node->Modify();
}


bool FCustomizableObjectNodeMeshSectionDetails::IsTagsPropertyWidgetEnabled() const
{
	const UEdGraphPin* EnableTagsPin = Node->GetEnableTagsPin();

	// Disabled if there is a string node linked to the "Enable Tags" pin.
	return !(EnableTagsPin && FollowInputPinArray(*EnableTagsPin).Num());
}


FText FCustomizableObjectNodeMeshSectionDetails::TagsPropertyWidgetTooltip() const
{
	const UEdGraphPin* EnableTagsPin = Node->GetEnableTagsPin();

	if (EnableTagsPin && FollowInputPinArray(*EnableTagsPin).Num())
	{
		return LOCTEXT("EnableTagsWidgetTooltip_Ignored","Disabled. When there are string nodes linked to the Enable Tags pin, the tag list is ignored.");
	}
	else
	{
		return LOCTEXT("EnableTagsWidgetTooltip", "List of Tags that this node will enable.");
	}
}


#undef LOCTEXT_NAMESPACE
