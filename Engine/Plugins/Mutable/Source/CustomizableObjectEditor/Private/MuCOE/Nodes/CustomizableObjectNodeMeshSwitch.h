// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CONodeSwitch.h"

#include "CustomizableObjectNodeMeshSwitch.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


// DEPRECATED
UCLASS(NotPlaceable, HideDropdown)
class UCustomizableObjectNodeMeshSwitch : public UCONodeSwitch
{
public:
	GENERATED_BODY()

	// UCustomizableObjectNodeSwitch interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
};

#undef UE_API
