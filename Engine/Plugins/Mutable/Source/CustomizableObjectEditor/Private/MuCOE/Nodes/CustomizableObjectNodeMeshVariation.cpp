// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeMeshVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeMeshVariation)


void UCustomizableObjectNodeMeshVariation::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeVariationSerializationIssue)
	{
		for (const FCustomizableObjectMeshVariation& OldVariation : Variations_DEPRECATED)
		{
			FCustomizableObjectVariation Variation;
			Variation.Tag = OldVariation.Tag;
			
			VariationsData.Add(Variation);
		}
	}
}


FName UCustomizableObjectNodeMeshVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Mesh;
}
