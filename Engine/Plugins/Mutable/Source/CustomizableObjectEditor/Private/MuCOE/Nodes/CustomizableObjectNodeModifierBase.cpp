// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuT/NodeSurfaceModifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierBase)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FLinearColor UCustomizableObjectNodeModifierBase::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Modifier);
}


UEdGraphPin* UCustomizableObjectNodeModifierBase::GetOutputPin() const
{
	return FindPin(TEXT("Modifier"));
}


UEdGraphPin* UCustomizableObjectNodeModifierBase::RequiredTagsPin() const
{
	return RequiredTagsPinRef.Get();
}


TArray<FString> UCustomizableObjectNodeModifierBase::GetNodeRequiredTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	const UEdGraphPin* ReqTagsPin = RequiredTagsPin();

	if (!ReqTagsPin)
	{
		return RequiredTags;
	}

	const TArray<UEdGraphPin*> ConnectedPins = FollowInputPinArray(*ReqTagsPin);

	if (ConnectedPins.Num())
	{
		TArray<FString> OutTags;

		for (const UEdGraphPin* StringPin : ConnectedPins)
		{
			const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*StringPin, MacroContext);

			if (SourceStringPin)
			{
				if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
				{
					OutTags.Add(StringNode->Value);
				}
			}
		}

		return OutTags;
	}

	return RequiredTags;
}


bool UCustomizableObjectNodeModifierBase::IsApplicableTo(UCustomizableObjectNode* Candidate)
{
	if (!Candidate)
	{
		return false;
	}

	const FString InternalTag = Candidate->GetInternalTag();
	ensure(!InternalTag.IsEmpty());			// We expect this string to never be empty
	
	TArray<FString> EnabledTags = Candidate->GetEnableTags();
	EnabledTags.Add(InternalTag);
	
	if (EnabledTags.Num())
	{
		switch ( MultipleTagPolicy )
		{
		case EMutableMultipleTagPolicy::OnlyOneRequired:
		{
			for (const FString& RequiredTag : RequiredTags)
			{
				if (InternalTag==RequiredTag || EnabledTags.Contains(RequiredTag))
				{
					return true;
				}
			}
			break;
		}

		case EMutableMultipleTagPolicy::AllRequired:
		{
			for (const FString& RequiredTag : RequiredTags)
			{
				if (InternalTag != RequiredTag && !EnabledTags.Contains(RequiredTag))
				{
					return false;
				}
			}
			return true;
		}

		default:
			// Not implemented
			check(false);
		}
	}

	return false;
}


void UCustomizableObjectNodeModifierBase::GetPossiblyModifiedNodes(TArray<UCustomizableObjectNode*>& CandidateNodes)
{
	// Scan all potential receivers
	UCustomizableObject* ThisNodeObject = GraphTraversal::GetObject(*this);
	UCustomizableObject* RootObject = GraphTraversal::GetRootObject(ThisNodeObject);

	TSet<UCustomizableObject*> AllCustomizableObject;
	GraphTraversal::GetAllObjectsInGraph(RootObject, AllCustomizableObject);

	for (const UCustomizableObject* CustObject : AllCustomizableObject)
	{
		if (CustObject)
		{
			for (const TObjectPtr<UEdGraphNode>& CandidateNode : CustObject->GetPrivate()->GetSource()->Nodes)
			{
				UCustomizableObjectNode* Typed = Cast<UCustomizableObjectNode>(CandidateNode);
				if (IsApplicableTo(Typed))
				{
					CandidateNodes.Add(Typed);
				}
			}
		}
	}
}

void UCustomizableObjectNodeModifierBase::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	// Remove "Material" pin and add "Modifier", fix the other side of the connections, if possible
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddModifierPin)
	{
		// Replace the output pin
		UEdGraphPin* OldPin = FindPin(TEXT("Material"));
		if (OldPin)
		{
			UEdGraphPin* NewPin = FindPin(TEXT("Modifier")); 
			if (!NewPin)
			{
				NewPin = CustomCreatePin(EGPD_Output, Schema->PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));
			}

			FGuid PinId = NewPin->PinId;
			NewPin->CopyPersistentDataFromOldPin(*OldPin);
			NewPin->PinId = PinId;
			NewPin->bHidden = OldPin->bHidden;

			CustomRemovePin(*OldPin);

			// Reconnect it to the correct output of the target node
			TArray<UEdGraphPin*> LinksToRemove;
			for (int32 LinkIndex = 0; LinkIndex < NewPin->LinkedTo.Num(); ++LinkIndex)
			{
				UEdGraphPin* LinkedToPin = NewPin->LinkedTo[LinkIndex];

				if (!LinkedToPin)
				{
					continue;
				}

				UEdGraphNode* ToNode = LinkedToPin->GetOwningNode();
				if (!ToNode)
				{
					continue;
				}

				if (UCustomizableObjectNodeObject* ToObjectNode = Cast<UCustomizableObjectNodeObject>(ToNode))
				{
					UEdGraphPin* ToModifierPin = ToObjectNode->ModifiersPin();
					if (ToModifierPin == LinkedToPin)
					{
						// It is already correctly connected
						continue;
					}

					if (ToModifierPin)
					{
						LinksToRemove.Add(LinkedToPin);
						NewPin->MakeLinkTo(ToModifierPin);
					}
				}
			}

			// Remove reconnected links
			for (UEdGraphPin* LinkToRemove : LinksToRemove)
			{
				NewPin->BreakLinkTo(LinkToRemove);
			}
		}
	}

	// Fix the "Modifier" pin connection
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixModifierPin)
	{
		// Replace the output pin
		UEdGraphPin* ModifierPin = FindPin(TEXT("Modifier"));
		if (ModifierPin)
		{
			// Reconnect it to the correct output of the target node
			TArray<UEdGraphPin*> LinksToRemove;
			for (int32 LinkIndex = 0; LinkIndex < ModifierPin->LinkedTo.Num(); ++LinkIndex)
			{
				UEdGraphPin* LinkedToPin = ModifierPin->LinkedTo[LinkIndex];

				if (!LinkedToPin)
				{
					continue;
				}

				UEdGraphNode* ToNode = LinkedToPin->GetOwningNode();
				if (!ToNode)
				{
					continue;
				}

				if (UCustomizableObjectNodeObject* ToObjectNode = Cast<UCustomizableObjectNodeObject>(ToNode))
				{
					UEdGraphPin* ToModifierPin = ToObjectNode->ModifiersPin();
					if (ToModifierPin == LinkedToPin)
					{
						// It is already correctly connected
						continue;
					}

					if (ToModifierPin)
					{
						LinksToRemove.Add(LinkedToPin);
						ModifierPin->MakeLinkTo(ToModifierPin);
					}
					else
					{
						ensure(false);
					}
				}
				else
				{
					// The modifier is connected to a node for which automatic upgrade support is not implemented. This will be warned in PostBackwardsCompatibleFixup
				}
			}

			// Remove reconnected links
			for (UEdGraphPin* LinkToRemove : LinksToRemove)
			{
				ModifierPin->BreakLinkTo(LinkToRemove);
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!RequiredTagsPinRef.Get())
		{
			UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Target Tags"), LOCTEXT("TargetTags", "Target Tags"));
			Pin->PinType.ContainerType = EPinContainerType::Array;
			RequiredTagsPinRef = Pin;
		}
	}
}

void UCustomizableObjectNodeModifierBase::PostBackwardsCompatibleFixup()
{
	Super::PostBackwardsCompatibleFixup();

	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	// Check for old legacy connections that need manual update
	{
		UEdGraphPin* NewPin = FindPin(TEXT("Modifier"));
		if (NewPin)
		{
			for (int32 LinkIndex = 0; LinkIndex < NewPin->LinkedTo.Num(); ++LinkIndex)
			{
				UEdGraphPin* LinkedToPin = NewPin->LinkedTo[LinkIndex];

				if (!LinkedToPin)
				{
					continue;
				}

				UEdGraphNode* ToNode = LinkedToPin->GetOwningNode();
				if (!ToNode)
				{
					continue;
				}

				if (LinkedToPin->PinType.PinCategory != Schema->PC_Modifier)
				{
					// The modifier is connected to a node for which automatic upgrade support is not implemented
					FString Msg = FString::Printf(TEXT("A modifier node has a legacy connection to a node [%s] without automatic upgrade support. Manual update is probably needed."), *ToNode->GetName());
					FCustomizableObjectEditorLogger::CreateLog(FText::FromString(Msg))
						.Severity(EMessageSeverity::Warning)
						.Context(*this)
						.BaseObject(true)
						.CustomizableObject(GraphTraversal::GetObject(*this))
						.Log();
				}
			}
		}
	}

	// Apply backwards compatibility auto-generated tags to external objects.
	for (const FLegacyTag& Tag : LegacyBackportsRequiredTags)
	{
		// If we still have it in this node
		if (RequiredTags.Contains(Tag.Tag))
		{
			UCustomizableObjectNode* ParentNode = GetCustomizableObjectExternalNode<UCustomizableObjectNode>(Tag.ParentObject.Get(), Tag.ParentNode);
			if (ParentNode)
			{
				TArray<FString>* NodeEnableTags = ParentNode->GetEnableTagsArray();
				if (NodeEnableTags)
				{
					NodeEnableTags->AddUnique(Tag.Tag);
				}
			}
		}
	}
}


FString UCustomizableObjectNodeModifierBase::MakeNodeAutoTag( UEdGraphNode* Node )
{
	const FString PackageName = Node->GetOutermost()->GetPathName();
	const FString ShortName = FPackageName::GetShortName(PackageName);
	FString NewLegacyTag = FString::Printf(TEXT("%s_%s"), *ShortName, *Node->NodeGuid.ToString());
	return NewLegacyTag;
}


void UCustomizableObjectNodeModifierBase::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Target Tags"), LOCTEXT("TargetTags", "Target Tags"));
	Pin->PinType.ContainerType = EPinContainerType::Array;
	RequiredTagsPinRef = Pin;
}


#undef LOCTEXT_NAMESPACE
