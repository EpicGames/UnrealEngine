// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuT/NodeSurfaceModifier.h"

#include "CustomizableObjectNodeModifierBase.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

enum class EMutableMultipleTagPolicy:uint8;


USTRUCT()
struct FLegacyTag
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftObjectPtr<UCustomizableObject> ParentObject;

	UPROPERTY()
	FGuid ParentNode;

	UPROPERTY()
	FString Tag;

	bool operator==(const FLegacyTag&) const = default;
};


UCLASS(MinimalAPI, abstract)
class UCustomizableObjectNodeModifierBase : public UCustomizableObjectNode
{

	GENERATED_BODY()

public:

	/** Materials in all other objects that activate this tags will be affected by this modifier. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifier)
	TArray<FString> RequiredTags;

	/** Policy to use when multiple Target Mesh Section tags are set. */
	UPROPERTY(EditAnywhere, Category = Modifier)
	EMutableMultipleTagPolicy MultipleTagPolicy = EMutableMultipleTagPolicy::OnlyOneRequired;

public:

	// EdGraphNode interface
	UE_API virtual FLinearColor GetNodeTitleColor() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void PostBackwardsCompatibleFixup() override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	// Own interface
	UE_API virtual UEdGraphPin* GetOutputPin() const;
	UE_API virtual UEdGraphPin* RequiredTagsPin() const;
	UE_API TArray<FString> GetNodeRequiredTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;

	/** Return true if the tags and policy in this node would make this node modify the given parameter node. */
	UE_API bool IsApplicableTo(UCustomizableObjectNode* Candidate);

	/** Get the list of all nodes of this object that may be affected by this modifier. This is an approximate
	* query: it includes all nodes that may be affected by this modifier but it may include nodes that will not 
	* be modified by this modifier because of other conditions.
	*/
	UE_API void GetPossiblyModifiedNodes(TArray<UCustomizableObjectNode*>&);

	/** Generate a tag that is unique and stable for the given node. */
	static UE_API FString MakeNodeAutoTag(UEdGraphNode*);

protected:

	/** This is a list of tags autogenerated due to backwards compatibility upgrade. 
	* It must be removed when we remove the related backwards compatibility code.
	*/
	UPROPERTY()
	TArray<FLegacyTag> LegacyBackportsRequiredTags;

private:

	UPROPERTY()
	FEdGraphPinReference RequiredTagsPinRef;

};

#undef UE_API
