// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBaseDetails.h"

#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/Attribute.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierBaseDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeModifierBaseDetails );
}


void FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeModifierBase>( DetailsView->GetSelectedObjects()[0].Get() );
	}

	if (!Node)
	{
		return;
	}
	
	// Move modifier conditions to the top.
	IDetailCategoryBuilder& ModifierCategory = DetailBuilder.EditCategory("Modifier");
	ModifierCategory.SetSortOrder(-10000);

	// Add the required tags widget
	RequiredTagsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierBase, RequiredTags), UCustomizableObjectNodeModifierBase::StaticClass());
	DetailBuilder.HideProperty(RequiredTagsPropertyHandle);

	RequiredTagsPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged));
	RequiredTagsPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged));

	TagsPolicyPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierBase, MultipleTagPolicy), UCustomizableObjectNodeModifierBase::StaticClass());
	TagsPolicyPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged));

	ModifierCategory.AddCustomRow(FText::FromString(TEXT("Target Tags")))
		.PropertyHandleList({ RequiredTagsPropertyHandle })
		.NameContent()
			.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.Padding(FMargin(0,4.0f,0,4.0f))
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ModifierDetails_RequiredTags", "Target Tags"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.IsEnabled(this, &FCustomizableObjectNodeModifierBaseDetails::IsTagsPropertyWidgetEnabled)
						.ToolTipText(this, &FCustomizableObjectNodeModifierBaseDetails::TagsPropertyWidgetTooltip)
				]
		]
		.ValueContent()
			.HAlign(HAlign_Fill)
		[
			SAssignNew(this->TagListWidget, SMutableTagListWidget)
				.Node(Node)
				.TagArray(&Node->RequiredTags)
				.EmptyListText(LOCTEXT("ModifierDetails_NoRequiredTagsWarning", "Warning: There are no required tags, so this modifier will not do anything."))
				.OnTagListChanged( this, &FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged )
				.IsEnabled(this, &FCustomizableObjectNodeModifierBaseDetails::IsTagsPropertyWidgetEnabled)
				.ToolTipText(this, &FCustomizableObjectNodeModifierBaseDetails::TagsPropertyWidgetTooltip)
		];
}


void FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged()
{
	// This seems necessary to detect the "Reset to default" actions.
	TagListWidget->RefreshOptions();
	Node->Modify();
}


bool FCustomizableObjectNodeModifierBaseDetails::IsTagsPropertyWidgetEnabled() const
{
	const UEdGraphPin* ReqTagsPin = Node->RequiredTagsPin();

	// Disabled if there is a string node linked to the "Target Tags" pin.
	return !(ReqTagsPin && FollowInputPinArray(*ReqTagsPin).Num());
}


FText FCustomizableObjectNodeModifierBaseDetails::TagsPropertyWidgetTooltip() const
{
	const UEdGraphPin* ReqTagsPin = Node->RequiredTagsPin();

	if (ReqTagsPin && FollowInputPinArray(*ReqTagsPin).Num())
	{
		return LOCTEXT("RequiredTagsWidgetTooltip_Ignored", "Disabled. When there are string nodes linked to the Target Tags pin, the tag list is ignored.");
	}
	else
	{
		return LOCTEXT("RequiredTagsWidgetTooltip", "Collection of tags to be used to decide which Mesh Sections will get the action of this Modifier.");
	}
}

#undef LOCTEXT_NAMESPACE
