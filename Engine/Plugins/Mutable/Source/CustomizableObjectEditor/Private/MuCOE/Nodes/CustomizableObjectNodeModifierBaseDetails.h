// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeDetails.h"
#include "MuCOE/SMutableTagListWidget.h"
#include "MuCOE/SMutableSearchComboBox.h"
#include "IDetailCustomization.h"
#include "Widgets/Views/SListView.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UCustomizableObjectNode;
class UCustomizableObjectNodeObject;

/** */
class FCustomizableObjectNodeModifierBaseDetails : public FCustomizableObjectNodeDetails
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it 
	static TSharedRef<IDetailCustomization> MakeInstance();

	// ILayoutDetails interface
	void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

protected:

	/** Called when a required tags or policy property has changed, potentially changing the modified nodes. */
	virtual void OnRequiredTagsPropertyChanged();

private:

	/** Disables/Enables the tags property widget. */
	bool IsTagsPropertyWidgetEnabled() const;

	/** Set the widget tooltip. */
	FText TagsPropertyWidgetTooltip() const;

private:

	class UCustomizableObjectNodeModifierBase* Node = nullptr;

	/** */
	TSharedPtr<IPropertyHandle> RequiredTagsPropertyHandle;
	TSharedPtr<IPropertyHandle> TagsPolicyPropertyHandle;

	/** */
	TSharedPtr<SMutableTagListWidget> TagListWidget;

};
