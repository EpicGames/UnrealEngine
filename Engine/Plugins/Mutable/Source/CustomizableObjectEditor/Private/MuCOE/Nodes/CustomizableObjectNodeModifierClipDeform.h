// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuT/NodeSurfaceModifier.h"

#include "CustomizableObjectNodeModifierClipDeform.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;

UENUM()
enum class EShapeBindingMethod : uint32
{
	ClosestProject = 0,
	ClosestToSurface = 1,
	NormalProject = 2
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeModifierClipDeform : public UCustomizableObjectNodeModifierBase
{

	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<FString> Tags_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = MeshClipDeform)
	EShapeBindingMethod BindingMethod;

	UPROPERTY(EditAnywhere, Category = RemoveOptions)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

public:

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual bool IsExperimental() const override;

	// Own interface

	/** Input pins. */
	UE_API UEdGraphPin* ClipShapePin() const;
};

#undef UE_API
