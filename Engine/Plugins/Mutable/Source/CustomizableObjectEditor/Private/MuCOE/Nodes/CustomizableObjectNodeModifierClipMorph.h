// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuR/Types.h"

#include "CustomizableObjectNodeModifierClipMorph.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;

UCLASS(MinimalAPI)
class UCustomizableObjectNodeModifierClipMorph : public UCustomizableObjectNodeModifierBase
{
public:
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> Tags_DEPRECATED;

	uint32 ReferenceSkeletonIndex_DEPRECATED = 0;

	UPROPERTY(EditAnywhere, Category = MeshToClipAndMorph)
	FName ReferenceSkeletonComponent;

	UPROPERTY(EditAnywhere, Category = MeshToClipAndMorph)
	FName BoneName;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName = "Invert normal direction", ToolTip = "Flag to invert the normal direction"))
	bool bInvertNormal;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName="Morph Start Offset", ToolTip="Offset from the origin of the selected bone to the actual start of the morph."))
	FVector StartOffset;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName = "Local Start Offset", ToolTip = "Toggles between a local or global start offset."))
	bool bLocalStartOffset;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName="Morph length", ToolTip="The length from the morph start to the clip plane."))
	float B;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName="Ellipse Radius 1", ToolTip="First radius of the ellipse that the mesh is morphed into."))
	float Radius;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName = "Ellipse Radius 2", ToolTip = "Second radius of the ellipse that the mesh is morphed into."))
	float Radius2;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName = "Ellipse Rotation", ToolTip = "Ellipse Rotation in degrees around the bone axis."))
	float RotationAngle;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName = "Morph Curve Control", ToolTip = "Controls the morph curve shape. A value of 1 is linear, less than 1 is concave and greater than 1 convex."))
	float Exponent;


	UPROPERTY()
	FVector Origin;

	UPROPERTY()
	FVector Normal;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters, meta = (DisplayName = "Max Effect Radius", ToolTip = "The maximum distance from the origin of the widget where vertices will be affected. If negative, there will be no limit."))
	float MaxEffectRadius;

	UPROPERTY(EditAnywhere, Category = MeshClipParameters)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

private:

	UPROPERTY()
	bool bOldOffset_DEPRECATED;

public:
	
	UE_API UCustomizableObjectNodeModifierClipMorph();

	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Own interface
	UE_API UEdGraphPin* GetOutputPin() const;

	UE_API FVector GetOriginWithOffset() const;
	UE_API void FindLocalAxes(FVector& Right, FVector& Up, FVector& Forward) const;

	/**
	 * Change StartOffset from World to Local of the other way around
	 */
	UE_API void ChangeStartOffsetTransform();

	/**
	 * Invert the normals of the Clipping plane
	 */
	UE_API void InvertNormals();
	
	/**
	 * Update the origin and normal based on the BoneName property
	 */
	UE_API void UpdateOriginAndNormal();

	/**
	 * Get the reference skeletal mesh based on the ReferenceSkeletonComponent property
	 * @return The skeletal mesh component used as reference for the ReferenceSkeletonComponent property.
	 * Null if the ReferenceSkeletonComponent is not valid
	 */
	UE_API TStrongObjectPtr<USkeletalMesh> GetReferenceSkeletalMesh() const;
};

#undef UE_API
