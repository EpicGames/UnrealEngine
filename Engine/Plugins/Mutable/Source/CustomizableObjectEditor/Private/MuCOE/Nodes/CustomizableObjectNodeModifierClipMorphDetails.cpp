// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorphDetails.h"

#include "DetailLayoutBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "IDetailsView.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "Widgets/Input/STextComboBox.h"

class UCustomizableObjectNodeObject;


#define LOCTEXT_NAMESPACE "MeshClipMorphDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierClipMorphDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeModifierClipMorphDetails);
}


void FCustomizableObjectNodeModifierClipMorphDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails(DetailBuilder);

	Node = nullptr;
	DetailBuilderPtr = &DetailBuilder;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = TStrongObjectPtr(Cast<UCustomizableObjectNodeModifierClipMorph>(DetailsView->GetSelectedObjects()[0].Get()));
	}

	// Invalid node : Early out
	if (!Node)
	{
		IDetailCategoryBuilder& BlocksCategory = DetailBuilder.EditCategory("MeshToClipAndMorph");
		BlocksCategory.AddCustomRow(LOCTEXT("Node", "Node"))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ClipMorphDetails_InvertNormal_NodeNotFound", "Node not found"))
		];
		return;
	}

	// Handle changes in the string provided to the "Component Name" property
	const TSharedPtr<IPropertyHandle> ReferenceSkeletonIndexPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierClipMorph, ReferenceSkeletonComponent));
	const FSimpleDelegate OnReferenceSkeletonComponentChangedDelegate = 
			FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierClipMorphDetails::OnReferenceSkeletonComponentChanged);
	ReferenceSkeletonIndexPropertyHandle->SetOnPropertyValueChanged(OnReferenceSkeletonComponentChangedDelegate);
	
	// Handle cases where the name of the bone get changed but not due to our custom UI (for example on reset)
	const TSharedRef<IPropertyHandle> BoneProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierClipMorph, BoneName));
	const FSimpleDelegate OnBoneNamePropertyResetHandle =
		FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierClipMorphDetails::OnBoneNameReset);
	BoneProperty->SetOnPropertyResetToDefault(OnBoneNamePropertyResetHandle);
	
	// Pre-fill internal objects based on the current context
	CacheSkeletalMesh();
	if (SkeletalMesh)
	{
		CacheSkeletalMeshBoneNames();
	}
	
	// Override the display of the BoneProperty so it uses a dropdown instead of a simple string entry box. the node willhandle the update of
	// the property
	DetailBuilder.EditDefaultProperty(BoneProperty)
	->CustomWidget()
	.Visibility(TAttribute<EVisibility>(this, &FCustomizableObjectNodeModifierClipMorphDetails::ShouldBoneDropdownBeVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("ClipMorphDetails_BoneName", "Bone Name"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SAssignNew(BoneSelectionComboBox,STextComboBox)
		.OptionsSource(&BoneComboOptions)
		.InitiallySelectedItem(InitiallySelectedBone)
		.OnSelectionChanged(this, &FCustomizableObjectNodeModifierClipMorphDetails::OnBoneComboBoxSelectionChanged, BoneProperty)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}


void FCustomizableObjectNodeModifierClipMorphDetails::CacheSkeletalMesh()
{
	if (!Node)
	{
		return;
	}
	
	SkeletalMesh = Node->GetReferenceSkeletalMesh();
}


void FCustomizableObjectNodeModifierClipMorphDetails::CacheSkeletalMeshBoneNames()
{
	BoneComboOptions.Empty();
	InitiallySelectedBone.Reset();
	
	if (BoneSelectionComboBox)
	{
		// Ensure the combo box displayed element is reset when changing the data being pointed at by "InitiallySelectedBone"
		BoneSelectionComboBox->ClearSelection();
	}

	if (SkeletalMesh)
	{
		for (int32 i = 0; i < SkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++i)
		{
			const FName BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(i);
			BoneComboOptions.Add(MakeShareable(new FString(BoneName.ToString())));
	
			if (BoneName == Node->BoneName)
			{
				InitiallySelectedBone = BoneComboOptions.Last();
			}
		}
	
		BoneComboOptions.Sort(CompareNames);
	}
}


void FCustomizableObjectNodeModifierClipMorphDetails::OnBoneNameReset()
{
	if (BoneSelectionComboBox)
	{
		// Ensure the combo box displayed element is reset when changing the data being pointed at by "InitiallySelectedBone"
		BoneSelectionComboBox->ClearSelection();
	}
}


void FCustomizableObjectNodeModifierClipMorphDetails::OnBoneComboBoxSelectionChanged(TSharedPtr<FString> InSelectedBoneName, ESelectInfo::Type InSelectInfo, TSharedRef<IPropertyHandle> BoneProperty)
{
	// Update the property of the node
	if (InSelectedBoneName.IsValid())
	{
		BoneProperty->SetValue(*InSelectedBoneName);
	}
	else
	{
		BoneProperty->ResetToDefault();
	}
}


void FCustomizableObjectNodeModifierClipMorphDetails::OnReferenceSkeletonComponentChanged()
{
	CacheSkeletalMesh();
	CacheSkeletalMeshBoneNames();
}


EVisibility FCustomizableObjectNodeModifierClipMorphDetails::ShouldBoneDropdownBeVisible() const
{
	if (BoneComboOptions.Num() > 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


#undef LOCTEXT_NAMESPACE
