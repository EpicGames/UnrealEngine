// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeModifierBaseDetails.h"
#include "IDetailCustomization.h"

enum class ECheckBoxState : uint8;
namespace ESelectInfo { enum Type : int; }

class STextComboBox;
class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class USkeletalMesh;
class UCustomizableObjectNodeModifierClipMorph;

class FCustomizableObjectNodeModifierClipMorphDetails : public FCustomizableObjectNodeModifierBaseDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	/** Store the Skeletal Mesh of the current node for later access. */
	void CacheSkeletalMesh();
	
	/** If a skeletal mesh has been cached this method will cache the name of the bones part of the skeleton of the cached skeletal mesh*/
	void CacheSkeletalMeshBoneNames();

	/** Method invoked each time the BoneName property gets reset from the UI */
	void OnBoneNameReset();

	/**
	 * Method invoked each time the selected boneName in the exposed combobox is changed by the user
	 * @param InSelectedBoneName The new selected bone name
	 * @param InSelectInfo The type of selection (combobox selection)
	 * @param BoneProperty The property to be updated with the string found in InSelection
	 */
	void OnBoneComboBoxSelectionChanged(TSharedPtr<FString> InSelectedBoneName, ESelectInfo::Type InSelectInfo, TSharedRef<IPropertyHandle> BoneProperty);

	/** Method invoked each time the component name property gets its value updated */
	void OnReferenceSkeletonComponentChanged();

	/**
	 * Determines if the bone dropdown should be visible or not
	 * @return The visibility state to use for the bone name dropdown.
	 */
	EVisibility ShouldBoneDropdownBeVisible() const;
	
	/** The node whose data we are representing and updating */
	TStrongObjectPtr<UCustomizableObjectNodeModifierClipMorph> Node = nullptr;
	
	/** The combo box that we use to display a list of bone names for the current Skeletal mesh */
	TSharedPtr<STextComboBox> BoneSelectionComboBox = nullptr;
	
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

	/** The name of the bone we have initially selected */
	TSharedPtr<FString> InitiallySelectedBone;
	
	/** The name of all bones we could select for the skeletal mesh skeleton of the current component */
	TArray<TSharedPtr<FString>> BoneComboOptions;

	/** The skeletal mesh of the component the node is modifying. It will be used to get an array of bones. */
	TStrongObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;
};
