// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"

#include "CustomizableObjectNodeModifierClipWithMesh.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FGuid;

DECLARE_MULTICAST_DELEGATE_OneParam(OnTransformChanged, const FTransform&);

UCLASS(MinimalAPI)
class UCustomizableObjectNodeModifierClipWithMesh : public UCustomizableObjectNodeModifierBase
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<FString> Tags_DEPRECATED;

	//!< If assigned, then a material inside this CO will be clipped by this node.
    //!< If several materials with the same name, all are considered (to cover all LOD levels)
    UPROPERTY()
	TObjectPtr<UCustomizableObject> CustomizableObjectToClipWith_DEPRECATED = nullptr;

    //!< Array with the Guids of the nodes with the same material inside the CustomizableObjectToClipWith CO (if any is assigned)
    UPROPERTY()
    TArray<FGuid> ArrayMaterialNodeToClipWithID_DEPRECATED;

	/** Transform to apply to the clip mesh before clipping. */
	UPROPERTY(EditAnywhere, Category = ClipMesh)
	FTransform Transform;

	UPROPERTY(EditAnywhere, Category = ClipMesh)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

	/**
	 * Delegate invoked each time the BoundingMeshTransform value gets modified
	 */
	OnTransformChanged TransformChangedDelegate;
	
public:

	UE_API UCustomizableObjectNodeModifierClipWithMesh();

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// Own interface
	UE_API virtual UEdGraphPin* GetOutputPin() const override;

	UE_API UEdGraphPin* GetClipMeshPin() const;

};

#undef UE_API
