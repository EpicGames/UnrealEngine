// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithUVMask.h"

#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCO/CustomizableObjectCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierClipWithUVMask)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeModifierClipWithUVMask::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	CustomCreatePin(EGPD_Input, Schema->PC_Texture, TEXT("Clip Mask"), LOCTEXT("ClipMask", "Clip Mask"));
	CustomCreatePin(EGPD_Output, Schema->PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


FText UCustomizableObjectNodeModifierClipWithUVMask::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Clip_With_UV_Mask", "Clip With UV Mask");
}


void UCustomizableObjectNodeModifierClipWithUVMask::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = GetGraphEditor();

	if (Editor.IsValid())
	{
		Editor->UpdateGraphNodeProperties();
	}
}


void UCustomizableObjectNodeModifierClipWithUVMask::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UnifyRequiredTags)
	{
		RequiredTags = Tags_DEPRECATED;
		Tags_DEPRECATED.Empty();
	}
}


UEdGraphPin* UCustomizableObjectNodeModifierClipWithUVMask::ClipMaskPin() const
{
	return FindPin(TEXT("Clip Mask"));
}


FText UCustomizableObjectNodeModifierClipWithUVMask::GetTooltipText() const
{
	return LOCTEXT("Clip_Mask_Tooltip", "Removes the part of a material that has a UV layout inside a mask defined with a texture.\nIt only removes the faces that fall completely inside the mask, along with the vertices and edges that define only faces that are deleted.");
}

#undef LOCTEXT_NAMESPACE
