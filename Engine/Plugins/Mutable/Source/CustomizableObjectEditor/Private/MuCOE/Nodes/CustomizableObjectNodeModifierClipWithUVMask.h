// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuT/NodeSurfaceModifier.h"

#include "CustomizableObjectNodeModifierClipWithUVMask.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;

UCLASS(MinimalAPI)
class UCustomizableObjectNodeModifierClipWithUVMask : public UCustomizableObjectNodeModifierBase
{
	GENERATED_BODY()

public:

	/** Materials in all other objects that activate this tags will be clipped with this UV mask. */
	UPROPERTY()
	TArray<FString> Tags_DEPRECATED;

	/** UV channel index that will be used to get the UVs to apply the clipping mask to. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ClipOptions)
	int32 UVChannelForMask = 0;

	UPROPERTY(EditAnywhere, Category = ClipOptions)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

public:

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;


	// Own interface

	/** Access to input pins. */
	UE_API UEdGraphPin* ClipMaskPin() const;
};

#undef UE_API
