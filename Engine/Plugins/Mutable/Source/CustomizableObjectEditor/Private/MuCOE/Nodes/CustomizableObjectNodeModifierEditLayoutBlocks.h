// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditMeshSectionBase.h"
#include "MuCOE/CustomizableObjectLayout.h"

#include "CustomizableObjectNodeModifierEditLayoutBlocks.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


UCLASS(MinimalAPI, Abstract)
class UCustomizableObjectNodeModifierEditLayoutBlocks : public UCustomizableObjectNodeModifierEditMeshSectionBase
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeModifierEditLayoutBlocks();

public:

	/** Deprecated data from the time this node referred to other layout node blocks. Info is now in the Layout attribute. */
	UPROPERTY()
	TArray<FGuid> BlockIds_DEPRECATED;

	/** Layout defining the blocks to edit. It is a standalone layout, unrelated to whatever is connected
	 * to the UV channels of the mesh being edited.
	 * Some properties in the UCustomizableObjectLayout, like shrinking strategy or priorities will not be relevant.
	 */
	UPROPERTY()
	TObjectPtr<UCustomizableObjectLayout> Layout = nullptr;

};

#undef UE_API
