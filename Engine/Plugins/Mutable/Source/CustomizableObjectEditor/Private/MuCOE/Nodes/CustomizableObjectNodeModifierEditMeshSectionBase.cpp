// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditMeshSectionBase.h"

#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectPin.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuR/MutableTrace.h"
#include "Materials/MaterialParameters.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierEditMeshSectionBase)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeModifierEditMeshSectionBase::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ConvertEditAndExtendToModifiers)
	{
		// Look for the parent material and set it as the modifier reference material 
		UCustomizableObjectNode* ParentNode = GetCustomizableObjectExternalNode<UCustomizableObjectNode>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);

		if (ParentNode)
		{
			// Add an autogenerated tag to the legacy parent, so that it can be referenced in this modifier.
			FString NewLegacyTag = MakeNodeAutoTag(ParentNode);
			RequiredTags.Add(NewLegacyTag);

			TArray<FString>* NodeEnableTags = ParentNode->GetEnableTagsArray();
			if (NodeEnableTags)
			{
				NodeEnableTags->AddUnique(NewLegacyTag);
			}
			else
			{
				// Conversion failed?
				ensure(false);
				UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierEditMeshSectionBase version upgrade failed.", *GetOutermost()->GetName());
			}

			// If the tag was added to another CO, keep track of the addition to be able to repeat it in case
			// that CO is not re-saved but this one is.
			UCustomizableObject* ThisNodeObject = GraphTraversal::GetObject(*this);
			if (ParentMaterialObject_DEPRECATED != ThisNodeObject)
			{
				FLegacyTag LegacyTag;
				LegacyTag.ParentObject = ParentMaterialObject_DEPRECATED;
				LegacyTag.ParentNode = ParentMaterialNodeId_DEPRECATED;
				LegacyTag.Tag = NewLegacyTag;
				LegacyBackportsRequiredTags.AddUnique(LegacyTag);
			}
		}
		else
		{
			UE_LOGF(LogMutable, Log, "[%ls] UCustomizableObjectNodeModifierEditMeshSectionBase has no parent. It will not be upgraded.", *GetOutermost()->GetName());
		}
	}
}


UCustomizableObjectLayout* UCustomizableObjectNodeModifierEditMeshSectionBase::GetPossibleParentLayout()
{
	MUTABLE_CPUPROFILER_SCOPE(GetPossibleParentLayout);

	// Try to find the parent layout, because we want to show its UVs in the widget
	UCustomizableObjectLayout* ParentLayout = nullptr;

	TArray<UCustomizableObjectNode*> CandidateNodes;
	GetPossiblyModifiedNodes(CandidateNodes);

	// For now just show the first one
	for (UCustomizableObjectNode* Candidate : CandidateNodes)
	{
		TArray<UCustomizableObjectLayout*> AllLayouts;

		if (UCONodeSkeletalMeshSection* MeshSectionNode = Cast<UCONodeSkeletalMeshSection>(Candidate))
		{
			AllLayouts = MeshSectionNode->GetLayouts();
		}

		else if (UCONodeModifierExtendSkeletalMeshSection* ExtendNode = Cast<UCONodeModifierExtendSkeletalMeshSection>(Candidate))
		{
			AllLayouts = ExtendNode->GetLayouts();
		}

		if (AllLayouts.IsValidIndex(ParentLayoutIndex))
		{
			ParentLayout = AllLayouts[ParentLayoutIndex];
		}

		if (ParentLayout)
		{
			break;
		}
	}

	return ParentLayout;
}


#undef LOCTEXT_NAMESPACE

