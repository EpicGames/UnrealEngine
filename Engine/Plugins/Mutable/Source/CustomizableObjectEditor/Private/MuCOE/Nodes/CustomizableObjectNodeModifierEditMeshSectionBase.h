// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CONodeModifierSkeletalMeshSection.h"

#include "CustomizableObjectNodeModifierEditMeshSectionBase.generated.h"

class UCustomizableObject;
class UEdGraphPin;


UCLASS(Abstract)
class UCustomizableObjectNodeModifierEditMeshSectionBase : public UCONodeModifierSkeletalMeshSection
{
	GENERATED_BODY()
	
public:

	/** Layout index used when the modifier is based on UVs, or UV blocks. */
	UPROPERTY()
	int ParentLayoutIndex = 0;

	/** Deprecated properties from the time these nodes had hard parent node references.
	* The data is now used to upgrade to newer versions, but will be removed in the future. 
	*/
	UPROPERTY()
	TObjectPtr<UCustomizableObject> ParentMaterialObject_DEPRECATED = nullptr;

	UPROPERTY()
	FGuid ParentMaterialNodeId_DEPRECATED;
	
	// UCustomizableObjectNode interface
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Own interface

	/** Scan all the related objects and find a candidate layout where this modifier can be applied. */
	UCustomizableObjectLayout* GetPossibleParentLayout();
	
};
