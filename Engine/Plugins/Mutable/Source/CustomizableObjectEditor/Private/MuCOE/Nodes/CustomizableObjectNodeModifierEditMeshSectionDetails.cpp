// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditMeshSectionDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailsView.h"
#include "MuCOE/Nodes/CONodeModifierEditSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/SCustomizableObjectLayoutEditor.h"
#include "MuCOE/PinViewer/SPinViewer.h"
#include "MuR/MutableTrace.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/STextComboBox.h"
#include "UObject/Package.h"

class FString;

#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierEditMeshSectionDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeModifierEditMeshSectionDetails);
}


void FCustomizableObjectNodeModifierEditMeshSectionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails(DetailBuilder);

	Node = nullptr;
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCONodeModifierEditSkeletalMeshSection>(DetailsView->GetSelectedObjects()[0].Get());
	}

	
	IDetailCategoryBuilder& LayoutCategory = DetailBuilder.EditCategory("Layout Editor");

	if (!Node)
	{
		LayoutCategory.AddCustomRow(LOCTEXT("BlocksDetails_NodeNotFound", "NodeNotFound"))
			[
				SNew(STextBlock)
					.Text(LOCTEXT("Node not found", "Node not found"))
			];
		return;
	}

	// UV Channel combo (for now hardcoded to a maximum of 4)
	{
		UVChannelOptions.Empty();

		TSharedPtr<FString> CurrentUVChannel;
		for (int32 Index = 0; Index < 4; ++Index)
		{
			UVChannelOptions.Add(MakeShareable(new FString(FString::Printf(TEXT("%d"), Index))));

			if (Node->ParentLayoutIndex == Index)
			{
				CurrentUVChannel = UVChannelOptions.Last();
			}
		}

		IDetailGroup& LayoutOptionsGroup = LayoutCategory.AddGroup(TEXT("EditLayoutGroup"), LOCTEXT("EditMeshSection", "Edit Layout"), false, true);
		LayoutOptionsGroup.HeaderRow()
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("UVChannel", "Edit UV Channel"))
					.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(STextComboBox)
					.InitiallySelectedItem(CurrentUVChannel)
					.OptionsSource(&UVChannelOptions)
					.OnSelectionChanged(this, &FCustomizableObjectNodeModifierEditMeshSectionDetails::OnUVChannelChanged)
					.Font(DetailBuilder.GetDetailFont())
			];
	}

	// Mesh sections with the editable layouts
	TArray<FLayoutEditorMeshSection> MeshSectionsAndLayouts;

	// Add Dummy section with the layout to edit.
	FLayoutEditorMeshSection& MeshSection = MeshSectionsAndLayouts.AddDefaulted_GetRef();
	MeshSection.MeshName = MakeShareable(new FString("NameNone"));
	MeshSection.Layouts.Add(Node->Layout);

	LayoutBlocksEditor = SNew(SCustomizableObjectLayoutEditor)
		.Node(Node)
		.MeshSections(MeshSectionsAndLayouts)
		.OnPreUpdateLayoutDelegate(this, &FCustomizableObjectNodeModifierEditMeshSectionDetails::OnPreUpdateLayout);

	FCustomizableObjectLayoutEditorDetailsBuilder LayoutEditorBuilder;
	LayoutEditorBuilder.LayoutEditor = LayoutBlocksEditor;
	LayoutEditorBuilder.bShowGridSize = true;

	LayoutEditorBuilder.CustomizeDetails(DetailBuilder);

	LayoutBlocksEditor->UpdateLayout(Node->Layout);
}


void FCustomizableObjectNodeModifierEditMeshSectionDetails::OnRequiredTagsPropertyChanged()
{
	FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged();

	if (ensure(LayoutBlocksEditor))
	{
		LayoutBlocksEditor->UpdateLayout(Node->Layout);
	}
}


void FCustomizableObjectNodeModifierEditMeshSectionDetails::OnPreUpdateLayout()
{
	MUTABLE_CPUPROFILER_SCOPE(FCustomizableObjectNodeModifierEditMeshSectionDetails_UpdateLayout);

	if (ensure(LayoutBlocksEditor))
	{
		// Try to find the parent layout, because we want to show its UVs in the widget
		UCustomizableObjectLayout* ParentLayout = Node->GetPossibleParentLayout();
		LayoutBlocksEditor->SetUVsOverride(ParentLayout);
	}
}


void FCustomizableObjectNodeModifierEditMeshSectionDetails::OnUVChannelChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (!Node->Layout || !LayoutBlocksEditor)
	{
		return;
	}

	int32 Index = UVChannelOptions.Find(NewSelection);

	if (Node->ParentLayoutIndex != Index)
	{
		Node->ParentLayoutIndex = Index;

		Node->Modify();

		LayoutBlocksEditor->UpdateLayout(Node->Layout);
	}
}

#undef LOCTEXT_NAMESPACE
