// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CustomizableObjectNodeModifierBaseDetails.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"

class FString;
class IDetailLayoutBuilder;
class UCONodeModifierEditSkeletalMeshSection;
class SCustomizableObjectLayoutEditor;


class FCustomizableObjectNodeModifierEditMeshSectionDetails : public FCustomizableObjectNodeModifierBaseDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/** FCustomizableObjectNodeModifierBaseDetails interface. */
	virtual void OnRequiredTagsPropertyChanged() override;

private:
	UCONodeModifierEditSkeletalMeshSection* Node;

	// Layout block editor widget
	TSharedPtr<SCustomizableObjectLayoutEditor> LayoutBlocksEditor;

	/** List of available layout grid sizes. */
	TArray<TSharedPtr<FString>> UVChannelOptions;

	/** Layout Options Callbacks */
	void OnUVChannelChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	/** Reset the layout in the widget to force a refresh. */
	void OnPreUpdateLayout();
};
