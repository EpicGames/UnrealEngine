// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierExtendMeshSectionDetails.h"

#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/Attribute.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierExtendMeshSectionDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeModifierExtendMeshSectionDetails );
}


void FCustomizableObjectNodeModifierExtendMeshSectionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCONodeModifierExtendSkeletalMeshSection>( DetailsView->GetSelectedObjects()[0].Get() );
	}

	if (!Node)
	{
		return;
	}

	// Move tags to enable higher.
	IDetailCategoryBuilder& TagsCategory = DetailBuilder.EditCategory("EnableTags");
	TagsCategory.SetSortOrder(-5000);

	// Add the required tags widget
	{
		EnableTagsPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCONodeModifierExtendSkeletalMeshSection, Tags), UCONodeModifierExtendSkeletalMeshSection::StaticClass());
		DetailBuilder.HideProperty(EnableTagsPropertyHandle);

		EnableTagsPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::OnRequiredTagsPropertyChanged));
		EnableTagsPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::OnRequiredTagsPropertyChanged));

		TagsCategory.AddCustomRow(FText::FromString(TEXT("Enable Tags")))
		.PropertyHandleList({ EnableTagsPropertyHandle })
		.NameContent()
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.Padding(FMargin(0, 4.0f, 0, 4.0f))
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ExtendMeshSectionDetails_Tags", "Tags enabled for extended data"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.IsEnabled(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::IsTagsPropertyWidgetEnabled)
						.ToolTipText(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::TagsPropertyWidgetTooltip)
				]
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SAssignNew(this->EnableTagListWidget, SMutableTagListWidget)
				.Node(Node)
				.TagArray(&Node->Tags)
				.AllowInternalTags(false)
				.EmptyListText(LOCTEXT("ExtendMeshSectionDetails_NoTags", "No tags enabled by this extended mesh section."))
				.OnTagListChanged(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::OnEnableTagsPropertyChanged)
				.IsEnabled(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::IsTagsPropertyWidgetEnabled)
				.ToolTipText(this, &FCustomizableObjectNodeModifierExtendMeshSectionDetails::TagsPropertyWidgetTooltip)
		];
	}
}


void FCustomizableObjectNodeModifierExtendMeshSectionDetails::OnEnableTagsPropertyChanged()
{
	// This seems necessary to detect the "Reset to default" actions.
	EnableTagListWidget->RefreshOptions();
	Node->Modify();
}


bool FCustomizableObjectNodeModifierExtendMeshSectionDetails::IsTagsPropertyWidgetEnabled() const
{
	const UEdGraphPin* EnableTagsPin = Node->GetEnableTagsPin();

	// Disabled if there is a string node linked to the "Enable Tags" pin.
	return !(EnableTagsPin && FollowInputPinArray(*EnableTagsPin).Num());
}


FText FCustomizableObjectNodeModifierExtendMeshSectionDetails::TagsPropertyWidgetTooltip() const
{
	const UEdGraphPin* EnableTagsPin = Node->GetEnableTagsPin();

	if (EnableTagsPin && FollowInputPinArray(*EnableTagsPin).Num())
	{
		return LOCTEXT("EnableTagsWidgetTooltip_Ignored", "Disabled. When there are string nodes linked to the Enable Tags pin, the tag list is ignored.");
	}
	else
	{
		return LOCTEXT("EnableTagsWidgetTooltip", "List of Tags that this node will enable.");
	}
}


#undef LOCTEXT_NAMESPACE
