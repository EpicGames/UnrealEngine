// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSection.h"

#include "CustomizableObjectNodeStaticString.h"
#include "Engine/SkeletalMesh.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierMorphMeshSection)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeModifierMorphMeshSection::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));
	FactorPinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Factor"), LOCTEXT("Factor", "Factor"));

	MorphTargetNamePinRef = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String,  TEXT("Morph Target Name"), LOCTEXT("MorphTargetName", "Morph Target Name"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


FText UCustomizableObjectNodeModifierMorphMeshSection::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Morph_MeshSection", "Morph Mesh Section");
}


FString UCustomizableObjectNodeModifierMorphMeshSection::GetRefreshMessage() const
{
	return "Morph Target not found in the SkeletalMesh. Please Refresh Node and select a valid morph option.";
}


FText UCustomizableObjectNodeModifierMorphMeshSection::GetTooltipText() const
{
	return LOCTEXT("Morph_Material_Tooltip", "Fully activate one morph of a parent's material.");
}


bool UCustomizableObjectNodeModifierMorphMeshSection::IsSingleOutputNode() const
{
	return true;
}


bool UCustomizableObjectNodeModifierMorphMeshSection::IsMorphTargetNamePin(const UEdGraphPin& Pin) const
{
	return FEdGraphPinReference(&Pin) == MorphTargetNamePinRef;
}


void UCustomizableObjectNodeModifierMorphMeshSection::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!MorphTargetNamePinRef.Get())
		{
			MorphTargetNamePinRef = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Morph Target Name"), LOCTEXT("MorphTargetName", "Morph Target Name"));
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ModifierMorphMeshSection_CacheFactorPin)
	{
		FactorPinReference = FindPin(TEXT("Factor"));
		check(FactorPinReference.Get());
	}
}


bool UCustomizableObjectNodeModifierMorphMeshSection::CanRenamePin(const UEdGraphPin& Pin) const
{
	return IsMorphTargetNamePin(Pin);
}


FText UCustomizableObjectNodeModifierMorphMeshSection::GetPinEditableName(const UEdGraphPin& Pin) const
{
	if (IsMorphTargetNamePin(Pin))
	{
		return FText::FromString(MorphTargetName);
	}

	return Super::GetPinEditableName(Pin);
}


void UCustomizableObjectNodeModifierMorphMeshSection::SetPinEditableName(const UEdGraphPin& Pin, const FText& Value)
{
	if (IsMorphTargetNamePin(Pin))
	{
		MorphTargetName = Value.ToString();
	}
}


FString UCustomizableObjectNodeModifierMorphMeshSection::GetMorphTargetName(FMutableGraphGenerationContext& GenerationContext) const
{
	FString OutputMorphTargetName = MorphTargetName;

	const UEdGraphPin* MorphTargetNamePinPtr = MorphTargetNamePinRef.Get();
	check(MorphTargetNamePinPtr);
	if (const UEdGraphPin* ConnectedStringPin = FollowInputPin(*MorphTargetNamePinPtr))
	{
		if (const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedStringPin, &GenerationContext.MacroNodesStack))
		{
			if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
			{
				OutputMorphTargetName = StringNode->Value;
			}
		}
		else
		{
			GenerationContext.Log(LOCTEXT("NodeModifierMorphMeshSection_FailedToFindLinkedName", "Could not find a linked String node."), this);
		}
	}
	
	return OutputMorphTargetName;
}


#undef LOCTEXT_NAMESPACE
