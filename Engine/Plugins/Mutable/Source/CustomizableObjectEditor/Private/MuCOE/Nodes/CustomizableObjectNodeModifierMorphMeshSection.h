// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/GenerateMutableSource/GenerateMutableSource.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditMeshSectionBase.h"

#include "CustomizableObjectNodeModifierMorphMeshSection.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;


UCLASS()
class UCustomizableObjectNodeModifierMorphMeshSection: public UCustomizableObjectNodeModifierEditMeshSectionBase
{
	GENERATED_BODY()
	
public:
	
	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual FString GetRefreshMessage() const override;
	virtual bool IsSingleOutputNode() const override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	
	FString GetMorphTargetName(FMutableGraphGenerationContext& GenerationContext) const;

private:

	bool IsMorphTargetNamePin(const UEdGraphPin& Pin) const;

public:

	UPROPERTY()
	FEdGraphPinReference FactorPinReference;

private:
	
	UPROPERTY()
	FString MorphTargetName;
	
	UPROPERTY()
	FEdGraphPinReference MorphTargetNamePinRef;
};
