// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSectionDetails.h"

#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierMorphMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "MuCO/LoadUtils.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierMorphMeshSectionDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeModifierMorphMeshSectionDetails );
}


void FCustomizableObjectNodeModifierMorphMeshSectionDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails(DetailBuilder);
	
	// This property is not relevant for this node
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UCONodeModifierSkeletalMeshSection, ReferenceMaterial), UCONodeModifierSkeletalMeshSection::StaticClass());
}

#undef LOCTEXT_NAMESPACE
