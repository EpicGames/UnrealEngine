// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeModifierBaseDetails.h"
#include "MuCOE/SMutableSearchComboBox.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UCustomizableObjectNodeObject;


class FCustomizableObjectNodeModifierMorphMeshSectionDetails : public FCustomizableObjectNodeModifierBaseDetails
{
public:
	// Makes a new instance of this detail layout class for a specific detail view requesting it 
	static TSharedRef<IDetailCustomization> MakeInstance();

	// ILayoutDetails interface
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
};
