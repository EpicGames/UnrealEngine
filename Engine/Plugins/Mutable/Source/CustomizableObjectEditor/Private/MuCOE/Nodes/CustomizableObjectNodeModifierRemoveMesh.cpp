// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMesh.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCO/CustomizableObjectCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierRemoveMesh)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeModifierRemoveMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	CustomCreatePin(EGPD_Input, Schema->PC_Mesh, TEXT("Remove Mesh"), LOCTEXT("RemoveMesh", "Remove Mesh"));
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier);

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


FText UCustomizableObjectNodeModifierRemoveMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Remove_Mesh_Modifier", "Remove Mesh");
}


void UCustomizableObjectNodeModifierRemoveMesh::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	if (Pin == GetOutputPin())
	{
		TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = GetGraphEditor();

		if (Editor.IsValid())
		{
			Editor->UpdateGraphNodeProperties();
		}
	}
}

FText UCustomizableObjectNodeModifierRemoveMesh::GetTooltipText() const
{
	return LOCTEXT("Remove_Mesh_Modifier_Tooltip",
	"Removes the faces of a mesh section that are defined only by the vertexes shared by said mesh section and the input mesh. \n"
	"It also removes any vertex and edge that only define deleted faces. \n"
	"If the removed mesh covers all the faces included in one or more layout blocks those blocks get removed, freeing layout space in the final texture.");
}

bool UCustomizableObjectNodeModifierRemoveMesh::IsSingleOutputNode() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE
