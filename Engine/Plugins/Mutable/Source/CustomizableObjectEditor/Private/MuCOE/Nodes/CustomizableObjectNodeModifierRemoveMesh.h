// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditMeshSectionBase.h"

#include "CustomizableObjectNodeModifierRemoveMesh.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UCustomizableObjectNodeModifierRemoveMesh : public UCustomizableObjectNodeModifierEditMeshSectionBase
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = RemoveOptions)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

public:

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin * Pin);

	UEdGraphPin* RemoveMeshPin() const
	{
		return FindPin(TEXT("Remove Mesh"));
	}

	UE_API bool IsSingleOutputNode() const override;
};

#undef UE_API
