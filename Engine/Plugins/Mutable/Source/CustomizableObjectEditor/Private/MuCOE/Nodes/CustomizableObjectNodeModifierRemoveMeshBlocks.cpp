// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocks.h"

#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeLayoutBlocks.h"
#include "MuCOE/GraphTraversal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierRemoveMeshBlocks)

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeModifierRemoveMeshBlocks::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	// Convert deprecated node index list to the node id list.
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PostLoadToCustomVersion
		&& BlockIds_DEPRECATED.Num() < Blocks_DEPRECATED.Num())
	{
		UCONodeSkeletalMeshSection* ParentMaterialNode = GetCustomizableObjectExternalNode<UCONodeSkeletalMeshSection>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);
		if (ParentMaterialNode)
		{
			TArray<UCustomizableObjectLayout*> Layouts = ParentMaterialNode->GetLayouts();

			if (!Layouts.IsValidIndex(ParentLayoutIndex))
			{
				UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks refers to an invalid texture layout index %d. Parent node has %d layouts.",
					*GetOutermost()->GetName(), ParentLayoutIndex, Layouts.Num());
			}
			else if (UCONodeSkeletalMeshSection* ParentMaterial = Cast<UCONodeSkeletalMeshSection>(ParentMaterialNode))
			{
				UCustomizableObjectLayout* ParentLayout = Layouts[ParentLayoutIndex];

				for (int IndexIndex = BlockIds_DEPRECATED.Num(); IndexIndex < Blocks_DEPRECATED.Num(); ++IndexIndex)
				{
					const int BlockIndex = Blocks_DEPRECATED[IndexIndex];
					if (!ParentLayout->Blocks.IsValidIndex(BlockIndex))
					{
						UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks refers to an invalid layout block index %d. Parent node has %d blocks.",
							*GetOutermost()->GetName(), BlockIndex, ParentLayout->Blocks.Num());

						continue;
					}
					
					const FGuid Id = ParentLayout->Blocks[BlockIndex].Id;
					if (!Id.IsValid())
					{
						UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks refers to an valid layout block %d but that block doesn't have an id.",
							*GetOutermost()->GetName(), BlockIndex);

						continue;
					}

					BlockIds_DEPRECATED.Add(Id);
				}
			}
		}
		else
		{
			UE_LOGF(LogMutable, Log, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks has no parent. It will not be upgraded.", *GetOutermost()->GetName());
		}
	}

	// Convert deprecated node id list to absolute rect list.
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UseUVRects)
	{
		// If we are here, it means this node was loaded from a version that didn't have it's own layout.
		check(Layout->Blocks.IsEmpty());

		UCONodeSkeletalMeshSection* ParentMaterialNode = GetCustomizableObjectExternalNode<UCONodeSkeletalMeshSection>(ParentMaterialObject_DEPRECATED.Get(), ParentMaterialNodeId_DEPRECATED);

		TArray<UCustomizableObjectLayout*> ParentLayouts;

		if (!ParentMaterialNode)
		{
			// Conversion failed?
			ensure(false);
			UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks version upgrade failed.", *GetOutermost()->GetName());
		}
		else
		{
			[&]()
				{
					UEdGraphPin* MeshPin = ParentMaterialNode->GetMeshPin();
					if (!MeshPin)
					{
						return;
					}

					const UEdGraphPin* ConnectedPin = FollowInputPin(*MeshPin);
					if (!ConnectedPin)
					{
						return;
					}

					const UEdGraphPin* SourceMeshPin = FindMeshBaseSource(*ConnectedPin, false);
					if (!SourceMeshPin)
					{
						return;
					}

					if (const UCustomizableObjectNodeSkeletalMesh* MeshNode = Cast<UCustomizableObjectNodeSkeletalMesh>(SourceMeshPin->GetOwningNode()))
					{
						UCustomizableObjectNodeSkeletalMeshPinDataMesh* MeshPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(MeshNode->GetPinData(*SourceMeshPin));

						if (!MeshPinData)
						{
							return;
						}

						// The custom version of SkeletalMesh nodes may be up to date if it belongs to a different CO. 
						ParentLayouts = MeshPinData->Layouts;

						if (ParentLayouts.IsEmpty())
						{
							// Pre FCustomizableObjectCustomVersion::RemoveNodeLayout code. Get Layouts from NodeLayoutBlocks
							for (const UEdGraphPin* Pin : MeshNode->GetAllNonOrphanPins())
							{
								if (const UCustomizableObjectNodeSkeletalMeshPinDataLayout* PinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataLayout>(MeshNode->GetPinData(*Pin)))
								{
									if (PinData->GetLODIndex() == MeshPinData->GetLODIndex() &&
										PinData->GetSectionIndex() == MeshPinData->GetSectionIndex())
									{
										if (const UEdGraphPin* SourceLayoutConnectedPin = FollowInputPin(*Pin))
										{
											const UCustomizableObjectNodeLayoutBlocks* LayoutNode = Cast<UCustomizableObjectNodeLayoutBlocks>(SourceLayoutConnectedPin->GetOwningNode());
											if (LayoutNode && LayoutNode->Layout)
											{
												ParentLayouts.Add(LayoutNode->Layout);
											}
										}
									}
								}
							}
						}
					}
					else if (const UCustomizableObjectNodeTable* TableNode = Cast<UCustomizableObjectNodeTable>(SourceMeshPin->GetOwningNode()))
					{
						ParentLayouts = TableNode->GetLayouts(SourceMeshPin);
					}
				}(); // invoke lambda GetLayouts();
		}

		if (!ParentLayouts.IsValidIndex(ParentLayoutIndex))
		{
			UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks refers to an invalid texture layout index %d. Parent node has %d layouts.",
				*GetOutermost()->GetName(), ParentLayoutIndex, ParentLayouts.Num());
		}
		else
		{
			UCustomizableObjectLayout* ParentLayout = ParentLayouts[ParentLayoutIndex];
			FIntPoint GridSize = ParentLayout->GetGridSize();

			Layout->SetGridSize(GridSize);

			if (Cast<UCONodeSkeletalMeshSection>(ParentMaterialNode))
			{
				for (const FGuid& BlockId : BlockIds_DEPRECATED)
				{
					bool bSkipBlock = false;
					for (const FCustomizableObjectLayoutBlock& ExistingBlock : Layout->Blocks)
					{
						if (ExistingBlock.Id == BlockId)
						{
							bSkipBlock = true;
							UE_LOGF(LogMutable, Log, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks has a duplicated layout block id. One has been ignored during version upgrade.", *GetOutermost()->GetName());
							break;
						}
					}

					if (bSkipBlock)
					{
						continue;
					}

					bool bFoundInParent = false;
					for (const FCustomizableObjectLayoutBlock& ParentBlock : ParentLayout->Blocks)
					{
						if (ParentBlock.Id == BlockId)
						{
							bFoundInParent = true;

							FCustomizableObjectLayoutBlock NewBlock;
							NewBlock = ParentBlock;

							// Clear some unnecessary data.
							NewBlock.bReduceBothAxes = false;
							NewBlock.bReduceByTwo = false;
							NewBlock.Priority = 0;

							Layout->Blocks.Add(NewBlock);
							break;
						}
					}

					if (!bFoundInParent)
					{
						UE_LOGF(LogMutable, Warning, "[%ls] UCustomizableObjectNodeModifierRemoveMeshBlocks refers to and invalid layout block. It has been ignored during version upgrade.", *GetOutermost()->GetName());
					}
				}
			}
		}
	}
}


void UCustomizableObjectNodeModifierRemoveMeshBlocks::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	Super::AllocateDefaultPins(RemapPins);
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier);
}


FText UCustomizableObjectNodeModifierRemoveMeshBlocks::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Remove_Mesh_Blocks", "Remove Mesh Blocks");
}


FText UCustomizableObjectNodeModifierRemoveMeshBlocks::GetTooltipText() const
{
	return LOCTEXT("Remove_Mesh_Blocks_Tooltip", "Remove all the geometry in the chosen layout blocks from a material.");
}


bool UCustomizableObjectNodeModifierRemoveMeshBlocks::IsSingleOutputNode() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
