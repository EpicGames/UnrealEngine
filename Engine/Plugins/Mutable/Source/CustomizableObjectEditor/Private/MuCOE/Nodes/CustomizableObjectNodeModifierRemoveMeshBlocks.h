// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierEditLayoutBlocks.h"

#include "CustomizableObjectNodeModifierRemoveMeshBlocks.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;

UCLASS(MinimalAPI)
class UCustomizableObjectNodeModifierRemoveMeshBlocks : public UCustomizableObjectNodeModifierEditLayoutBlocks
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<int32> Blocks_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = RemoveOptions)
	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

public:

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual bool IsSingleOutputNode() const override;
};

#undef UE_API
