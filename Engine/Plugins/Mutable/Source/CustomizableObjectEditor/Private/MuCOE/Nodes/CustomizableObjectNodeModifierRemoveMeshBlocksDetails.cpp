// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocksDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailsView.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshBlocks.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/SCustomizableObjectLayoutEditor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/STextComboBox.h"
#include "UObject/Package.h"

class FString;


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeModifierRemoveMeshBlocksDetails );
}


void FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails(DetailBuilder);

	Node = nullptr;
	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeModifierRemoveMeshBlocks>(DetailsView->GetSelectedObjects()[0].Get());
	}

	// This property is not relevant for this node
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UCONodeModifierSkeletalMeshSection, ReferenceMaterial), UCONodeModifierSkeletalMeshSection::StaticClass());

	IDetailCategoryBuilder& LayoutCategory = DetailBuilder.EditCategory("Layout Editor");
	LayoutCategory.SetSortOrder(10000);

	if (!Node)
	{
		LayoutCategory.AddCustomRow(LOCTEXT("BlocksDetails_NodeNotFound", "NodeNotFound"))
			[
				SNew(STextBlock)
					.Text(LOCTEXT("Node not found", "Node not found"))
			];
		return;
	}

	// UV Channel combo (for now hardcoded to a maximum of 4)
	{
		int32 MaxGridSize = 128;
		UVChannelOptions.Empty();

		TSharedPtr<FString> CurrentUVChannel;
		for (int32 Index = 0; Index<4; ++Index)
		{
			UVChannelOptions.Add(MakeShareable(new FString(FString::Printf(TEXT("%d"), Index))));

			if (Node->ParentLayoutIndex == Index)
			{
				CurrentUVChannel = UVChannelOptions.Last();
			}
		}

		IDetailGroup& LayoutOptionsGroup = LayoutCategory.AddGroup(TEXT("LayoutOptionsGroup"), LOCTEXT("LayoutGroup", "Edit Layout"), false, true);
		LayoutOptionsGroup.HeaderRow()
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("UVChannel", "Edit UV Channel"))
					.Font(DetailBuilder.GetDetailFont())
			]
			.ValueContent()
			[
				SNew(STextComboBox)
					.InitiallySelectedItem(CurrentUVChannel)
					.OptionsSource(&UVChannelOptions)
					.OnSelectionChanged(this, &FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::OnUVChannelChanged)
					.Font(DetailBuilder.GetDetailFont())
			];
	}

	TArray<FLayoutEditorMeshSection> MeshLayoutsAndLayouts;

	FLayoutEditorMeshSection& MeshSection = MeshLayoutsAndLayouts.AddDefaulted_GetRef();
	MeshSection.MeshName = MakeShareable(new FString("NameNone"));
	MeshSection.Layouts.Add(Node->Layout);

	LayoutBlocksEditor = SNew(SCustomizableObjectLayoutEditor)
		.Node(Node)
		.MeshSections(MeshLayoutsAndLayouts)
		.OnPreUpdateLayoutDelegate(this, &FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::OnPreUpdateLayout);

	FCustomizableObjectLayoutEditorDetailsBuilder LayoutEditorBuilder;
	LayoutEditorBuilder.LayoutEditor = LayoutBlocksEditor;
	LayoutEditorBuilder.bShowGridSize = true;

	LayoutEditorBuilder.CustomizeDetails(DetailBuilder);

	LayoutBlocksEditor->UpdateLayout(Node->Layout);
}


void FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::OnRequiredTagsPropertyChanged()
{
	FCustomizableObjectNodeModifierBaseDetails::OnRequiredTagsPropertyChanged();

	if (ensure(LayoutBlocksEditor))
	{
		LayoutBlocksEditor->UpdateLayout(Node->Layout);
	}
}


void FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::OnPreUpdateLayout()
{
	if (ensure(LayoutBlocksEditor))
	{
		// Try to find the parent layout, because we want to show its UVs in the widget
		UCustomizableObjectLayout* ParentLayout = Node->GetPossibleParentLayout();
		LayoutBlocksEditor->SetUVsOverride(ParentLayout);
	}
}


void FCustomizableObjectNodeModifierRemoveMeshBlocksDetails::OnUVChannelChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (!Node->Layout)
	{
		return;
	}

	int32 Index = UVChannelOptions.Find(NewSelection);

	if (Node->ParentLayoutIndex != Index)
	{
		Node->ParentLayoutIndex = Index;
		Node->MarkPackageDirty();

		LayoutBlocksEditor->UpdateLayout(Node->Layout);
	}
}


#undef LOCTEXT_NAMESPACE
