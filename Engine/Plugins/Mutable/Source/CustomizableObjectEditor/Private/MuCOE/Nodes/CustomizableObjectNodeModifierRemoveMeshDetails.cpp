// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMeshDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailsView.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierRemoveMesh.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/STextComboBox.h"
#include "UObject/Package.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeModifierRemoveMeshDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeModifierRemoveMeshDetails );
}


void FCustomizableObjectNodeModifierRemoveMeshDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeModifierBaseDetails::CustomizeDetails(DetailBuilder);

	// This property is not relevant for this node
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UCONodeModifierSkeletalMeshSection, ReferenceMaterial), UCONodeModifierSkeletalMeshSection::StaticClass());

}

#undef LOCTEXT_NAMESPACE
