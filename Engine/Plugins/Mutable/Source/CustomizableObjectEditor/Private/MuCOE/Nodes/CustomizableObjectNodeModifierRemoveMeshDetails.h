// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeModifierBaseDetails.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

class FCustomizableObjectNodeModifierRemoveMeshDetails : public FCustomizableObjectNodeModifierBaseDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

};
