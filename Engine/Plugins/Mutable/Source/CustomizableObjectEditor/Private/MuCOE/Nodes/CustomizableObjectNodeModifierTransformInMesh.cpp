// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeModifierTransformInMesh.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeModifierTransformInMesh)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FText UCustomizableObjectNodeModifierTransformInMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Modifier_Transform_Mesh_In_Mesh", "Mesh Transform In Mesh");
}


FText UCustomizableObjectNodeModifierTransformInMesh::GetTooltipText() const
{
	return LOCTEXT("Modifier_Transform_Mesh_In_Mesh_Tooltip", "Applies a transform to the vertices of a mesh that are contained within the given bounding mesh");
}


void UCustomizableObjectNodeModifierTransformInMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	BoundingMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Mesh, TEXT("Bounding Mesh"), LOCTEXT("BoundingMesh", "Bounding Mesh"));
	TransformMeshPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Transform, TEXT("Transform"), LOCTEXT("Transform", "Transform"));
	
	OutputPin =  CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Modifier, TEXT("Modifier"), LOCTEXT("Modifier", "Modifier"));

	//Create Node Modifier Common Pins
	Super::AllocateDefaultPins(RemapPins);
}


void UCustomizableObjectNodeModifierTransformInMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;
	}
	
	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeModifierTransformInMesh, BoundingMeshTransform))
	{
		if (TransformChangedDelegate.IsBound())
		{
			TransformChangedDelegate.Broadcast(BoundingMeshTransform);
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeModifierTransformInMesh::GetOutputPin() const
{
	return OutputPin.Get();
}


bool UCustomizableObjectNodeModifierTransformInMesh::IsDeprecated() const
{
	return true;
}


void UCustomizableObjectNodeModifierTransformInMesh::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeModifierTransformInMeshStorePinsAsReferences)
	{
		const FString OutputPinName = TEXT("Modifier");
		const FString BoundingMeshPinName = TEXT("Bounding Mesh");
		const FString TransformPinName = TEXT("Transform");

		for (UEdGraphPin* Pin : Pins)
		{
			const FString PinName = Pin->GetName();
			const EEdGraphPinDirection PinDirection = Pin->Direction;
			
			if (PinDirection == EEdGraphPinDirection::EGPD_Input)
			{
				if (PinName == BoundingMeshPinName)
				{
					BoundingMeshPin = Pin;
				}
				else if (PinName == TransformPinName)
				{
					TransformMeshPin = Pin;
				}
			}
			else 
			{
				if (PinName == OutputPinName)
				{
					OutputPin = Pin;
				}
			}
		}
		
		check(BoundingMeshPin.Get());
		check(TransformMeshPin.Get());
		check(OutputPin.Get());
	}
}

#undef LOCTEXT_NAMESPACE
