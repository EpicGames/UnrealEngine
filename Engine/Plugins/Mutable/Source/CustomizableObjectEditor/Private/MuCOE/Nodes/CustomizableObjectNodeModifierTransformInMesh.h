// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuT/NodeSurfaceModifier.h"

#include "CustomizableObjectNodeModifierTransformInMesh.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;

DECLARE_MULTICAST_DELEGATE_OneParam(OnTransformChanged, const FTransform&);

UCLASS()
class UCustomizableObjectNodeModifierTransformInMesh : public UCustomizableObjectNodeModifierBase
{
	GENERATED_BODY()

public:
	
	// EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;

	// UCustomizableObjectNodeModifierBase interface
	virtual UEdGraphPin* GetOutputPin() const override;
	virtual bool IsDeprecated() const override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	/** Transform to apply to the bounding mesh before selecting for vertices to transform. */
	UPROPERTY(EditAnywhere, Category = BoundingMesh)
	FTransform BoundingMeshTransform = FTransform::Identity;
	
	UPROPERTY()
	FEdGraphPinReference BoundingMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference TransformMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference OutputPin;
	
	/**
	 * Delegate invoked each time the BoundingMeshTransform value gets modified
	 */
	OnTransformChanged TransformChangedDelegate;
};
