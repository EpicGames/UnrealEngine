// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"

#include "CustomizableObjectNodeStaticString.h"
#include "CONodeSwitch.h"
#include "MuCO/ICustomizableObjectModule.h"
#include "MuCO/CustomizableObjectExtension.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentMesh.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSectionVariation.h"
#include "Misc/UObjectToken.h"
#include "Logging/MessageLog.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/Nodes/CONodeModifierSkeletalMeshMerge.h"
#include "Settings/EditorStyleSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeObject)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


const FName UCustomizableObjectNodeObject::ChildrenPinName(TEXT("Children"));
const FName UCustomizableObjectNodeObject::ComponentsPinName(TEXT("Components"));
const FName UCustomizableObjectNodeObject::ModifiersPinName(TEXT("Modifiers"));
const FName UCustomizableObjectNodeObject::OutputPinName(TEXT("Object"));


UCustomizableObjectNodeObject::UCustomizableObjectNodeObject()
	: Super()
{
	bIsBase = true;
	ObjectName = "Unnamed Object";
	Identifier = FGuid::NewGuid();
}


void UCustomizableObjectNodeObject::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::StateTextureCompressionStrategyEnum)
	{
		for (FCustomizableObjectState& State : States)
		{
			if (State.TextureCompressionStrategy == ETextureCompressionStrategy::None
				&&
				State.bDontCompressRuntimeTextures_DEPRECATED)
			{
				State.bDontCompressRuntimeTextures_DEPRECATED = false;
				State.TextureCompressionStrategy = ETextureCompressionStrategy::DontCompressRuntime;
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::RegenerateNodeObjectsIds)
	{
		// This will regenerate all the Node Object Guids to finally remove the duplicated Guids warning.
		// It is safe to do this here as Node Object do not use its node guid to link themeselves to other nodes.
		CreateNewGuid();

		// This change may make cooks to become undeterministic, if the object GUID is finally used (it is a "toggle group" option).
		UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(GetCustomizableObjectGraph()->GetOuter());
		if (CustomizableObject)
		{
			FMessageLog("Mutable").Message(EMessageSeverity::Info)
				->AddToken(FTextToken::Create(LOCTEXT("Indeterministic Warning", "The object was saved with an old version and it may generate indeterministic packages. Resave it to fix the problem.")))
				->AddToken(FUObjectToken::Create(CustomizableObject));
		}
	}

	// Update state never-stream flag from deprecated enum
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::CustomizableObjectStateHasSeparateNeverStreamFlag)
	{
		for (FCustomizableObjectState& s : States)
		{
			s.bDisableTextureStreaming = s.TextureCompressionStrategy != ETextureCompressionStrategy::None;
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::StateUIMetadata)
	{
		for (FCustomizableObjectState& State : States)
		{
			State.UIMetadata.ObjectFriendlyName = State.StateUIMetadata_DEPRECATED.ObjectFriendlyName;
			State.UIMetadata.UISectionName = State.StateUIMetadata_DEPRECATED.UISectionName;
			State.UIMetadata.UIOrder = State.StateUIMetadata_DEPRECATED.UIOrder;
			State.UIMetadata.UIThumbnail = State.StateUIMetadata_DEPRECATED.UIThumbnail;
			State.UIMetadata.ExtraInformation = State.StateUIMetadata_DEPRECATED.ExtraInformation;
			State.UIMetadata.ExtraAssets = State.StateUIMetadata_DEPRECATED.ExtraAssets;
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NewComponentOptions)
	{
		// Like we did in the CO components, we use the index of the component as the name of the component
		for (int32 ComponentIndex = 0; ComponentIndex < ComponentSettings_DEPRECATED.Num(); ++ComponentIndex)
		{
			ComponentSettings_DEPRECATED[ComponentIndex].ComponentName = FString::FromInt(ComponentIndex);
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MovedCompatibilityFromPostBackwardsCompatibleFixup)
	{
		// Fix up ComponentSettings. Only root nodes
		if (ComponentSettings_DEPRECATED.IsEmpty() && bIsBase && !ParentObject)
		{
			FComponentSettings ComponentSettingsTemplate;

			if (UCustomizableObject* CurrentObject = Cast<UCustomizableObject>(GetOutermostObject()))
			{
				ComponentSettings_DEPRECATED.Init(ComponentSettingsTemplate, CurrentObject->GetPrivate()->MutableMeshComponents_DEPRECATED.Num());
			}
		}
	}

	// Add the "Modifiers" pin
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddModifierPin)
	{
		const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

		UEdGraphPin* OldPin = FindPin(TEXT("Modifiers"));
		if (!OldPin)
		{
			UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, Schema->PC_Modifier, ModifiersPinName, LOCTEXT("Modifiers", "Modifiers"));
			Pin->PinType.ContainerType = EPinContainerType::Array;
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeComponentMesh)
	{
		auto NodeComponentMeshBaseAllocateDefaultPins = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins*)
		{
			UCustomizableObjectNodeComponentMesh* NodeComponentMesh = CastChecked<UCustomizableObjectNodeComponentMesh>(Node);
				
			NodeComponentMesh->LODPins_DEPRECATED.Empty(NodeComponentMesh->NumLODs_DEPRECATED);
			for (int32 NodeComponentLODIndex = 0; NodeComponentLODIndex < NodeComponentMesh->NumLODs_DEPRECATED; ++NodeComponentLODIndex)
			{
				FString LODName = FString::Printf(TEXT("LOD %d"), NodeComponentLODIndex);

				UEdGraphPin* Pin = Node->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
				Pin->PinType.ContainerType = EPinContainerType::Array;
				NodeComponentMesh->LODPins_DEPRECATED.Add(Pin);
			}
						
			NodeComponentMesh->OutputPin = Node->CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"), LOCTEXT("Component", "Component"));
		};
		
		auto NodeAddToComponentMeshBaseAllocateDefaultPins = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins*)
		{
			UCONodeModifierMutableSkeletalMeshMerge* NodeAddSkeletalMesh = CastChecked<UCONodeModifierMutableSkeletalMeshMerge>(Node);

			NodeAddSkeletalMesh->LODPins_DEPRECATED.Empty(NodeAddSkeletalMesh->NumLODs_DEPRECATED);
			for (int32 NodeComponentLODIndex = 0; NodeComponentLODIndex < NodeAddSkeletalMesh->NumLODs_DEPRECATED; ++NodeComponentLODIndex)
			{
				FString LODName = FString::Printf(TEXT("LOD %d"), NodeComponentLODIndex);

				UEdGraphPin* Pin = Node->CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_MeshSection, FName(*LODName), FText::FromString(LODName));
				Pin->PinType.ContainerType = EPinContainerType::Array;
				NodeAddSkeletalMesh->LODPins_DEPRECATED.Add(Pin);
			}

			NodeAddSkeletalMesh->OutputPin = Node->CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Component, TEXT("Component"), LOCTEXT("Component", "Component"));
		};

		auto NodeObjectAllocateDefaultPins = [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins*)
		{
			UCustomizableObjectNodeObject* NodeObject = CastChecked<UCustomizableObjectNodeObject>(Node);

			const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

			UEdGraphPin* ComponentPin = NodeObject->CustomCreatePin(EGPD_Input, Schema->PC_Component, ComponentsPinName, LOCTEXT("Components", "Components"));
			ComponentPin->PinType.ContainerType = EPinContainerType::Array;

			UEdGraphPin* ModifierPin = NodeObject->CustomCreatePin(EGPD_Input, Schema->PC_Modifier, ModifiersPinName, LOCTEXT("Modifiers", "Modifiers"));
			ModifierPin->PinType.ContainerType = EPinContainerType::Array;

			UEdGraphPin* ChildrenPin = NodeObject->CustomCreatePin(EGPD_Input, Schema->PC_Object, ChildrenPinName, LOCTEXT("Children", "Children"));
			ChildrenPin->PinType.ContainerType = EPinContainerType::Array;

			for (const FRegisteredObjectNodeInputPin& Pin : ICustomizableObjectModule::Get().GetAdditionalObjectNodePins())
			{
				// Use the global pin name here to prevent extensions using the same pin names from
				// interfering with each other.
				//
				// This also prevents extension pins from clashing with the built-in pins from this node,
				// such as "Object".
				UEdGraphPin* GraphPin = NodeObject->CustomCreatePin(EGPD_Input, Pin.InputPin.PinType, Pin.GlobalPinName, Pin.InputPin.DisplayName);
				if (Pin.InputPin.bIsArray)
				{
					GraphPin->PinType.ContainerType = EPinContainerType::Array;
				}
			}

			UEdGraphPin* OutputPin = NodeObject->CustomCreatePin(EGPD_Output, Schema->PC_Object, OutputPinName, LOCTEXT("Output", "Output"));

			if (NodeObject->bIsBase)
			{
				OutputPin->bHidden = true;
			}
		};

		auto CreateNewNode = [](class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, class UEdGraphNode* InNodeTemplate)
		{
			// UE Code from FSchemaAction_NewNode::CreateNode(...). Overlap calculations performed before AutowireNewNode(...).

			constexpr int32 NodeDistance = 60;
			
			// Duplicate template node to create new node
			UEdGraphNode* ResultNode = DuplicateObject<UEdGraphNode>(InNodeTemplate, ParentGraph);

			ResultNode->SetFlags(RF_Transactional);

			ParentGraph->AddNode(ResultNode, true);

			ResultNode->CreateNewGuid();
			ResultNode->PostPlacedNewNode();
			if (UCustomizableObjectNode* TypedResultNode = Cast<UCustomizableObjectNode>(ResultNode))
			{
				TypedResultNode->BeginConstruct();
				TypedResultNode->PostBackwardsCompatibleFixup();
			}
			ResultNode->ReconstructNode(); // Mutable node lifecycle always starts at ReconstructNode.

			// For input pins, new node will generally overlap node being dragged off
			// Work out if we want to visually push away from connected node
			int32 XLocation = Location.X;
			if (FromPin && FromPin->Direction == EGPD_Input)
			{
				UEdGraphNode* PinNode = FromPin->GetOwningNode();
				const float XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

				if (XDelta < NodeDistance)
				{
					// Set location to edge of current node minus the max move distance
					// to force node to push off from connect node enough to give selection handle
					XLocation = PinNode->NodePosX - NodeDistance;
				}
			}

			ResultNode->AutowireNewNode(FromPin);

			ResultNode->NodePosX = XLocation;
			ResultNode->NodePosY = Location.Y;
			ResultNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

			return ResultNode;
		};
		
		const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
		
		bool bMoved = false;

		int32 NodesCreated = 0;

		if (!ComponentsPin()) // Some old nodes do not have the component pin.
		{
			UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, Schema->PC_Component, ComponentsPinName, LOCTEXT("Components", "Components"));
			Pin->PinType.ContainerType = EPinContainerType::Array;
		}
		
		if (!ParentObject && // Is a root object
			bIsBase)
		{
			if (!bMoved)
			{
				bMoved = true;
				NodePosX += 400; // Move it a bit to make space for the new component nodes.
			}

			for (const FMutableMeshComponentData& MeshComponent : GraphTraversal::GetObject(*this)->GetPrivate()->MutableMeshComponents_DEPRECATED)
			{
				UCustomizableObjectNodeComponentMesh* NewNode = NewObject<UCustomizableObjectNodeComponentMesh>(this);
				UEdGraphNode* Node = CreateNewNode(GetGraph(), ComponentsPin(), FVector2D(NodePosX - 300.0,  NodePosY + 200.0 * NodesCreated), NewNode);
				UCustomizableObjectNodeComponentMesh* NodeComponentMesh = CastChecked<UCustomizableObjectNodeComponentMesh>(Node);

				++NodesCreated;
				
				NodeComponentMesh->SetComponentName(MeshComponent.Name);
				NodeComponentMesh->NumLODs_DEPRECATED = NumLODs_DEPRECATED;

				NodeComponentMesh->FixupReconstructPins(CreateRemapPinsByName(), NodeComponentMeshBaseAllocateDefaultPins);
				
				ComponentsPin()->MakeLinkTo(NodeComponentMesh->OutputPin.Get());
			}
		}

		TMap<FName, UCONodeModifierMutableSkeletalMeshMerge*> ExistingNodeComponentMeshAddTo;
		
		// Create all NodeComponentMeshAddTo.
		for (int32 LODIndex = 0; LODIndex < NumLODs_DEPRECATED; ++LODIndex)
		{
			UEdGraphPin* OldLODPin = FindPin(FString::Printf(TEXT("%s%d "), TEXT("LOD "), LODIndex));
			if (!OldLODPin)
			{
				continue;
			}
			
			TArray<UEdGraphPin*> CopyLinkedPins = OldLODPin->LinkedTo;
			for (UEdGraphPin* LinkedPin : CopyLinkedPins) // Import/Exports/Reroute not supported.
			{
				UEdGraphNode* LinkedNode = LinkedPin->GetOwningNode();

				auto CreateNodeComponent = [&, this](const FName& MeshComponentName)
				{
					if (UCONodeModifierMutableSkeletalMeshMerge** Result = ExistingNodeComponentMeshAddTo.Find(MeshComponentName))
					{
						return *Result;
					}
					
					if (!bMoved)
					{
						bMoved = true;
						NodePosX += 400; // Move it a bit to make space for the new component nodes.
					}
					
					UCONodeModifierMutableSkeletalMeshMerge* NewNode = NewObject<UCONodeModifierMutableSkeletalMeshMerge>(this);
					UEdGraphNode* Node = CreateNewNode(GetGraph(), ComponentsPin(), FVector2D(NodePosX - 300.0,  NodePosY + 200.0 * NodesCreated), NewNode);
					UCONodeModifierMutableSkeletalMeshMerge* NodeComponentMeshAddTo = CastChecked<UCONodeModifierMutableSkeletalMeshMerge>(Node);

					++NodesCreated;

					NodeComponentMeshAddTo->SetParentSkeletalMeshName(MeshComponentName);
					NodeComponentMeshAddTo->NumLODs_DEPRECATED = NumLODs_DEPRECATED;

					// Create LOD pins.
					NodeComponentMeshAddTo->FixupReconstructPins(CreateRemapPinsByName(), NodeAddToComponentMeshBaseAllocateDefaultPins);

					ComponentsPin()->MakeLinkTo(NodeComponentMeshAddTo->OutputPin.Get());

					ExistingNodeComponentMeshAddTo.Add(MeshComponentName, NodeComponentMeshAddTo);
					
					return NodeComponentMeshAddTo;
				};

				
				UCONodeModifierMutableSkeletalMeshMerge* NodeComponentMeshAddTo = nullptr;
				bool bFixNode = false;
				
				if (UCONodeSkeletalMeshSection* NodeMaterial = Cast<UCONodeSkeletalMeshSection>(LinkedNode))
				{
					bFixNode = true;

					NodeComponentMeshAddTo = CreateNodeComponent(NodeMaterial->MeshComponentName_DEPRECATED);
				}
				else if (const UCONodeSwitch* NodeMaterialSwitch = CastSwitch(LinkedNode, UEdGraphSchema_CustomizableObject::PC_MeshSection))
				{
					bFixNode = true;

					[&]() // Lambda to ease the control flow.
					{
						if (!NodeMaterialSwitch->SwitchPins.Num()) // We should at least have a component to know were to connect this material. If not, not supported.
						{
							return;
						}

						FName ComponentName;
						bool bFirst = true;
						for (int32 ElementIndex = 0; ElementIndex < NodeMaterialSwitch->SwitchPins.Num(); ++ElementIndex)
						{
							if (UEdGraphPin* ConnectedPin = FollowInputPin(*NodeMaterialSwitch->SwitchPins[ElementIndex].Get()))
							{
								if (UCONodeSkeletalMeshSection* FirstNodeMaterialBase = Cast<UCONodeSkeletalMeshSection>(ConnectedPin->GetOwningNode()))
								{
									if (bFirst)
									{
										bFirst = false;
										ComponentName = FirstNodeMaterialBase->MeshComponentName_DEPRECATED;
									}
									else
									{
										if (ComponentName != FirstNodeMaterialBase->MeshComponentName_DEPRECATED) // All components must match. If not, not supported.
										{
											return;
										}
									}
								}
							}
						}

						NodeComponentMeshAddTo = CreateNodeComponent(ComponentName);
					}();
				}
				else if (UCONodeSkeletalMeshSectionVariation* NodeMaterialVariation = Cast<UCONodeSkeletalMeshSectionVariation>(LinkedNode))
				{
					bFixNode = true;

					[&]() // Lambda to ease the control flow.
					{
						if (!NodeMaterialVariation->GetNumVariations()) // We should at least have a component to know were to connect this material. If not, not supported.
						{
							return;
						}

						FName ComponentName;
						bool bFirst = true;

						TArray<UEdGraphPin*> ConnectedDefaultPins = FollowInputPinArray(*NodeMaterialVariation->DefaultPin(), nullptr);
						if (ConnectedDefaultPins.Num() > 0)
						{
							UEdGraphPin* FirstDefaultPin = ConnectedDefaultPins[0];
							if (UCONodeSkeletalMeshSection* FirstNodeMaterialBase = Cast<UCONodeSkeletalMeshSection>(FirstDefaultPin->GetOwningNode()))
							{
								ComponentName = FirstNodeMaterialBase->MeshComponentName_DEPRECATED;

							}
						}
						
						for (int32 ElementIndex = 0; ElementIndex < NodeMaterialVariation->GetNumVariations(); ++ElementIndex)
						{
							UEdGraphPin* VariationPin = NodeMaterialVariation->VariationPin(ElementIndex);
							if (VariationPin)
							{
								TArray<UEdGraphPin*> ConnectedPins = FollowInputPinArray(*VariationPin, nullptr);
								if (ConnectedPins.Num() > 0)
								{
									UEdGraphPin* ConnectedPin = ConnectedPins[0];
									if (UCONodeSkeletalMeshSection* FirstNodeMaterialBase = Cast<UCONodeSkeletalMeshSection>(ConnectedPin->GetOwningNode()))
									{
										if (bFirst)
										{
											bFirst = false;
											ComponentName = FirstNodeMaterialBase->MeshComponentName_DEPRECATED;
										}
										else
										{
											if (ComponentName != FirstNodeMaterialBase->MeshComponentName_DEPRECATED) // All components must match. If not, not supported.
											{
												return;
											}
										}
									}
								}
							}
						}

						if (!bFirst)
						{
							NodeComponentMeshAddTo = CreateNodeComponent(ComponentName);
						}
					}();
				}

				if (bFixNode && !NodeComponentMeshAddTo)
				{
					FString Msg = FString::Printf(TEXT("A Object node has a legacy connection to a node [%s] without automatic upgrade support. Manual update is probably needed."), *LinkedNode->GetName());
					FCustomizableObjectEditorLogger::CreateLog(FText::FromString(Msg))
						.Severity(EMessageSeverity::Warning)
						.Context(*this)
						.BaseObject(true)
						.CustomizableObject(GraphTraversal::GetObject(*this))
						.Log();
				}

				if (NodeComponentMeshAddTo)
				{
					LinkedPin->MakeLinkTo(NodeComponentMeshAddTo->LODPins_DEPRECATED[LODIndex].Get());
					LinkedPin->BreakLinkTo(OldLODPin);
				}
			}
		}
		
		FixupReconstructPins(CreateRemapPinsByName(), NodeObjectAllocateDefaultPins);
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MergeNodeComponents)
	{
		if (UEdGraphPin* PinComponents = ComponentsPin())
		{
			// Find all Mesh Component nodes.
			TMap<FName, UCustomizableObjectNodeComponentMesh*> NodeComponentMeshes;
			for (UEdGraphPin* Pin : PinComponents->LinkedTo)
			{
				if (UCustomizableObjectNodeComponentMesh* NodeComponentMesh = Cast<UCustomizableObjectNodeComponentMesh>(Pin->GetOwningNode()))
				{
					NodeComponentMeshes.Emplace(NodeComponentMesh->GetComponentName(), NodeComponentMesh);
				}
			}

			TArray<UCONodeModifierMutableSkeletalMeshMerge*> NodesToRemove;

			// Find all Add To Mesh Component nodes.
			for (UEdGraphPin* Pin : PinComponents->LinkedTo)
			{
				if (UCONodeModifierMutableSkeletalMeshMerge* NodeComponentMeshAddTo = Cast<UCONodeModifierMutableSkeletalMeshMerge>(Pin->GetOwningNode()))
				{
					UCustomizableObjectNodeComponentMesh** Result = NodeComponentMeshes.Find(NodeComponentMeshAddTo->GetParentSkeletalMeshName());
					if (!Result)
					{
						continue;
					}

					UCustomizableObjectNodeComponentMesh* NodeComponentMesh = *Result;
					if (NodeComponentMesh->NumLODs_DEPRECATED != NodeComponentMeshAddTo->NumLODs_DEPRECATED)
					{
						continue;
					}

					for (int32 LODIndex = 0; LODIndex < NodeComponentMesh->NumLODs_DEPRECATED; ++LODIndex)
					{
						for (UEdGraphPin* LinkedPin : NodeComponentMeshAddTo->LODPins_DEPRECATED[LODIndex].Get()->LinkedTo)
						{
							LinkedPin->MakeLinkTo(NodeComponentMesh->LODPins_DEPRECATED[LODIndex].Get());
						}
					}

					NodesToRemove.Add(NodeComponentMeshAddTo);
				}
			}

			for (UCONodeModifierMutableSkeletalMeshMerge* Node : NodesToRemove)
			{
				GetGraph()->RemoveNode(Node);
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::RealTimeMorphTargetOverrideDataStructureRework)
	{
		for (FRealTimeMorphSelectionOverride& Override : RealTimeMorphSelectionOverrides)
		{
			for (int32 SkeletalMeshIndex = 0; SkeletalMeshIndex < Override.SkeletalMeshesNames_DEPRECATED.Num(); ++SkeletalMeshIndex)
			{
				// Only add SkeletalMesh override if the morph selection was NoOverride and the the skeletalmesh had override. 
				if (Override.SelectionOverride == ECustomizableObjectSelectionOverride::NoOverride &&
					Override.Override_DEPRECATED[SkeletalMeshIndex] != ECustomizableObjectSelectionOverride::NoOverride)
				{
					FSkeletalMeshMorphTargetOverride& SkeletalMeshOverride = Override.SkeletalMeshes.AddDefaulted_GetRef();
					SkeletalMeshOverride.SkeletalMeshName = Override.SkeletalMeshesNames_DEPRECATED[SkeletalMeshIndex];
					SkeletalMeshOverride.SelectionOverride = Override.Override_DEPRECATED[SkeletalMeshIndex];
				}
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!NamePin.Get())
		{
			NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
		}
	}
}


void UCustomizableObjectNodeObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!Identifier.IsValid())
	{
		Identifier = FGuid::NewGuid();
	}

	// Update the cached flag in the main object
	UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>( GetCustomizableObjectGraph()->GetOuter() );
	if (CustomizableObject)
	{
		CustomizableObject->GetPrivate()->SetIsChildObject(ParentObject != nullptr);
	}

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("NumLODs"))
	{
		NumLODs_DEPRECATED = FMath::Clamp(NumLODs_DEPRECATED, 1, 64);

		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeObject::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
	
	UEdGraphPin* ComponentsPin = CustomCreatePin(EGPD_Input, Schema->PC_Component, ComponentsPinName, LOCTEXT("Components", "Components"));
	ComponentsPin->PinType.ContainerType = EPinContainerType::Array;
	
	UEdGraphPin* ModifiersPin = CustomCreatePin(EGPD_Input, Schema->PC_Modifier, ModifiersPinName, LOCTEXT("Modifiers", "Modifiers"));
	ModifiersPin->PinType.ContainerType = EPinContainerType::Array;

	UEdGraphPin* ChildrenPin = CustomCreatePin(EGPD_Input, Schema->PC_Object, ChildrenPinName, LOCTEXT("Children", "Children"));
	ChildrenPin->PinType.ContainerType = EPinContainerType::Array;
	
	for (const FRegisteredObjectNodeInputPin& Pin : ICustomizableObjectModule::Get().GetAdditionalObjectNodePins())
	{
		// Use the global pin name here to prevent extensions using the same pin names from
		// interfering with each other.
		//
		// This also prevents extension pins from clashing with the built-in pins from this node,
		// such as "Object".
		UEdGraphPin* GraphPin = CustomCreatePin(EGPD_Input, Pin.InputPin.PinType, Pin.GlobalPinName, Pin.InputPin.DisplayName);
		if (Pin.InputPin.bIsArray)
		{
			GraphPin->PinType.ContainerType = EPinContainerType::Array;

		}
	}

	UEdGraphPin* OutputPin = CustomCreatePin(EGPD_Output, Schema->PC_Object, OutputPinName, LOCTEXT("Output", "Output"));

	if (bIsBase)
	{
		OutputPin->bHidden = true;
	}
}


bool UCustomizableObjectNodeObject::IsNodeSupportedInMacros() const
{
	return false;
}


bool UCustomizableObjectNodeObject::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCustomizableObjectNodeObject::GetPinEditableName(const UEdGraphPin& Pin) const
{
	if (NamePin.Get()->PinId == Pin.PinId)
	{
		return FText::FromString(GetObjectName());
	}
	else
	{
		return Super::GetPinEditableName(Pin);
	}
}


void UCustomizableObjectNodeObject::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	if (NamePin.Get()->PinId == Pin.PinId)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeObject_SetNamePinEditableNameTransaction", "Change NamePin PinName"));
		Modify();
		
		SetObjectName(InValue.ToString());
		PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
	}
	else
	{
		Super::SetPinEditableName(Pin, InValue);
	}
}


FText UCustomizableObjectNodeObject::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (bIsBase)
	{
		return LOCTEXT("Base_Object", "Base Object");
	}
	else
	{
		return LOCTEXT("Base_Object_Deprecated", "Base Object (Deprecated)");
	}
}


FLinearColor UCustomizableObjectNodeObject::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Object);
}


void UCustomizableObjectNodeObject::PrepareForCopying()
{
	FText Msg(LOCTEXT("Cannot copy object node","There can only be one Customizable Object Node Object element per graph") );
	FMessageLog MessageLog("Mutable");
	MessageLog.Notify(Msg, EMessageSeverity::Info, true);
}


bool UCustomizableObjectNodeObject::CanUserDeleteNode() const
{
	return !bIsBase;
}


bool UCustomizableObjectNodeObject::CanDuplicateNode() const
{
	return !bIsBase;
}


void UCustomizableObjectNodeObject::PostBackwardsCompatibleFixup()
{
	Super::PostBackwardsCompatibleFixup();

	// Reconstruct in case any extension pins have changed
	ReconstructNode();
}


void UCustomizableObjectNodeObject::PostPasteNode()
{
	Super::PostPasteNode();

	Identifier = FGuid::NewGuid();
}

void UCustomizableObjectNodeObject::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	Identifier = FGuid::NewGuid();
}


void UCustomizableObjectNodeObject::SetParentObject(UCustomizableObject* CustomizableParentObject)
{
	if (TSharedPtr<FCustomizableObjectEditor> Editor = StaticCastSharedPtr<FCustomizableObjectEditor>(GetGraphEditor()))
	{
		if (CustomizableParentObject != Editor->GetCustomizableObject())
		{
			ParentObject = CustomizableParentObject;

			// Update the cached flag in the main object
			UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(GetCustomizableObjectGraph()->GetOuter());
			if (CustomizableObject)
			{
				CustomizableObject->GetPrivate()->SetIsChildObject(ParentObject != nullptr);
			}
		}
	}
}


FText UCustomizableObjectNodeObject::GetTooltipText() const
{
	return LOCTEXT("Base_Object_Tooltip",
	"As root object: Defines a customizable object root, its basic properties and its relationship with descendant Customizable Objects.\n\nAs a child object: Defines a Customizable Object children outside of the parent asset, to ease organization of medium and large\nCustomizable Objects. (Functionally equivalent to the Child Object Node.)");
}


bool UCustomizableObjectNodeObject::IsSingleOutputNode() const
{
	return true;
}


void UCustomizableObjectNodeObject::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == NamePin.Get())
	{
		GetGraph()->NotifyGraphChanged();
	}
}


FString UCustomizableObjectNodeObject::GetObjectName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (const UEdGraphPin* Pin = NamePin.Get())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*Pin))
		{
			const UEdGraphPin* StringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, MacroContext);

			if (const UCustomizableObjectNodeStaticString* StringNode = StringPin ? Cast<UCustomizableObjectNodeStaticString>(StringPin->GetOwningNode()) : nullptr)
			{
				return StringNode->Value;
			}
		}
	}

	return ObjectName;
}


void UCustomizableObjectNodeObject::SetObjectName(const FString& Name)
{
	ObjectName = Name;
}


#undef LOCTEXT_NAMESPACE
