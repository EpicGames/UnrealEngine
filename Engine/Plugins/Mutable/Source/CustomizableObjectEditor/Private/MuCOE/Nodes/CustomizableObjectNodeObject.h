// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CONodeComponentSkeletalMesh.h"
#include "CustomizableObjectNodeComponentMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuR/System.h"

#include "CustomizableObjectNodeObject.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObject;
class UCONodeSkeletalMeshSection;
class UCustomizableObjectNodeRemapPins;
class UObject;
struct FPropertyChangedEvent;
struct FSoftObjectPath;
struct FComponentSettings;


USTRUCT()
struct FCustomizableObjectState
{
	GENERATED_USTRUCT_BODY()

	friend UCustomizableObjectNodeObject;
	
	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	FString Name;

	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (ShowParameterOptions))
	TArray<FString> RuntimeParameters;

	/** Special treatment of texture compression for this state. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	ETextureCompressionStrategy TextureCompressionStrategy = ETextureCompressionStrategy::None;

	/** If this is enabled, Mesh streaming won't be used for this state, and all LODs will be generated when an instance is first updated. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (EditCondition="!bLiveUpdateMode"))
	bool bDisableMeshStreaming = false;

	/** If this is enabled, texture streaming won't be used for this state, and full images will be generated when an instance is first updated. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (EditCondition="!bLiveUpdateMode"))
	bool bDisableTextureStreaming = false;

	/** LiveUpdateMode will reuse instance temp. data between updates and speed up update times, but spend much more memory. Good for customization screens, not for actual gameplay modes. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bLiveUpdateMode = false;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bBuildOnlyFirstLOD = false;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TMap<FString, FString> ForcedParameterValues;

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "State UI Metadata"))
	FMutableStateUIMetadata UIMetadata;

private:
	// Deprecated
	UPROPERTY()
	FMutableParamUIMetadata StateUIMetadata_DEPRECATED;
	
	/** This is now TextureCompressionStrategy.  */
	UPROPERTY()
	bool bDontCompressRuntimeTextures_DEPRECATED = false;
};

USTRUCT()
struct FSkeletalMeshMorphTargetOverride
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = RealTimeMorphTargets)
	FName SkeletalMeshName;

	UPROPERTY(EditAnywhere, Category = RealTimeMorphTargets)
	ECustomizableObjectSelectionOverride SelectionOverride = ECustomizableObjectSelectionOverride::NoOverride;
};

USTRUCT()
struct FRealTimeMorphSelectionOverride
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = RealTimeMorphTargets)
	FName MorphName;

	UPROPERTY(EditAnywhere, Category = RealTimeMorphTargets)
	ECustomizableObjectSelectionOverride SelectionOverride = ECustomizableObjectSelectionOverride::NoOverride;

	UPROPERTY(EditAnywhere, Category = RealTimeMorphTargets, meta=(TitleProperty=SkeletalMeshName))
	TArray<FSkeletalMeshMorphTargetOverride> SkeletalMeshes;

    UPROPERTY()
    TArray<FName> SkeletalMeshesNames_DEPRECATED;

    UPROPERTY()
    TArray<ECustomizableObjectSelectionOverride> Override_DEPRECATED;
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeObject : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeObject();

protected:

	UPROPERTY()
	FString ObjectName;

public:

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "Parameter UI Metadata"))
	FMutableParamUIMetadata ParamUIMetadata;

	UPROPERTY()
	int32 NumLODs_DEPRECATED = 1;

	UPROPERTY()
	int32 NumMeshComponents_DEPRECATED = 1;

	UPROPERTY(EditAnywhere, Category=CustomizableObject ,meta = (TitleProperty = "Name"))
	TArray<FCustomizableObjectState> States;

	UPROPERTY(EditAnywhere, Category = AttachedToExternalObject)
	TObjectPtr<UCustomizableObject> ParentObject;

	UPROPERTY(EditAnywhere, Category = AttachedToExternalObject)
	FGuid ParentObjectGroupId;

	UPROPERTY()
	FGuid Identifier;

	UPROPERTY(EditAnywhere, Category = RealTimeMorphTargets, meta=(TitleProperty=MorphName, EditCondition="ParentObject==nullptr"))
	TArray<FRealTimeMorphSelectionOverride> RealTimeMorphSelectionOverrides;

	UPROPERTY()
	TArray<FComponentSettings> ComponentSettings_DEPRECATED;

private:
	UPROPERTY()
	FEdGraphPinReference NamePin;

public:
	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void PrepareForCopying() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void PostBackwardsCompatibleFixup() override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual bool IsNodeSupportedInMacros() const override;
	UE_API virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	UE_API virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	UE_API virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	
	// Own Interface
	UE_API void SetParentObject(UCustomizableObject* CustomizableParentObject);
	UE_API void SetObjectName(const FString& Name);
	UE_API virtual FString GetObjectName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;

	// Own interface
	UPROPERTY()
	bool bIsBase;
	
	UEdGraphPin* ComponentsPin() const
	{
		return FindPin(ComponentsPinName);
	}

	UEdGraphPin* ModifiersPin() const
	{
		return FindPin(ModifiersPinName);
	}

	UEdGraphPin* ChildrenPin() const
	{
		return FindPin(ChildrenPinName);
	}

	UEdGraphPin* OutputPin() const
	{
		return FindPin(OutputPinName);
	}

	
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual bool CanDuplicateNode() const override;
	
	UE_API bool IsSingleOutputNode() const override;
	
	// Array filled in the Details of the node to store all the parameter names of a CO graph (full tree)
	TArray<FString> ParameterNames;

private:
	static UE_API const FName ChildrenPinName;
	static UE_API const FName ComponentsPinName;
	static UE_API const FName ModifiersPinName;
	static UE_API const FName OutputPinName;

};

#undef UE_API
