// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeObjectChild.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeObjectChild)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeObjectChild::UCustomizableObjectNodeObjectChild()
	: Super()
{
	bIsBase = false;
}


FText UCustomizableObjectNodeObjectChild::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Child_Object_Title_List", "Child Object");
}


void UCustomizableObjectNodeObjectChild::PrepareForCopying()
{
	// Overriden to hide parent's class error message
}


bool UCustomizableObjectNodeObjectChild::CanUserDeleteNode() const
{
	return true;
}


bool UCustomizableObjectNodeObjectChild::CanDuplicateNode() const
{
	return true;
}


FText UCustomizableObjectNodeObjectChild::GetTooltipText() const
{
	return LOCTEXT("Child_Object_Tooltip",
		"Defines a customizable object children in the same asset as its parent, to ease the addition of small Customizable Objects directly into\ntheir parents asset. Functionally equivalent to the Base Object Node when it has a parent defined. It can be a children of the root\nobject or of any children, allowing arbitrary nesting of objects. Defines materials that can be added to its parent, modify it, remove\nparts of it or change any of its parameters. Also defines properties for others to use or modify.");
}


bool UCustomizableObjectNodeObjectChild::IsNodeSupportedInMacros() const
{
	return true;
}



#undef LOCTEXT_NAMESPACE
