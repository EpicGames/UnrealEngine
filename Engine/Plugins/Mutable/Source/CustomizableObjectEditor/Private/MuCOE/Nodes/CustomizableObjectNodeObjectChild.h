// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"

#include "CustomizableObjectNodeObjectChild.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeObjectChild : public UCustomizableObjectNodeObject
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeObjectChild();
	
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PrepareForCopying() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual bool CanDuplicateNode() const override;
	
	// UCustomizableObjectNode interface
	UE_API virtual bool IsNodeSupportedInMacros() const override;
};

#undef UE_API
