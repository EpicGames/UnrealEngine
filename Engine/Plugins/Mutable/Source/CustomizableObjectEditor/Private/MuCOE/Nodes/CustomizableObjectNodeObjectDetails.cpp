// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeObjectDetails.h"

#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeEnumParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeGroupProjectorParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/SCustomizableObjectNodeObjectRTMorphTargetOverride.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IPropertyTypeCustomization> FCustomizableObjectStateParameterSelector::MakeInstance()
{
	return MakeShared<FCustomizableObjectStateParameterSelector>();
}


void FCustomizableObjectStateParameterSelector::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	PropertyHandle = InPropertyHandle;

	if (OuterObjects.Num())
	{
		BaseObjectNode = Cast<UCustomizableObjectNodeObject>(OuterObjects[0]);
		
		if (!BaseObjectNode.IsValid())
		{
			return;
		}
	}

	FString SelectedParameterName;

	InPropertyHandle->GetValue(SelectedParameterName);
	GenerateParameterOptions(SelectedParameterName);

	InHeaderRow
	.NameContent().HAlign(EHorizontalAlignment::HAlign_Fill).VAlign(EVerticalAlignment::VAlign_Center)
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent().MinDesiredWidth(300.0f)
	[
		SNew(SBorder)
		.BorderBackgroundColor(FLinearColor::Transparent)
		[
			SNew(SSearchableComboBox)
			.InitiallySelectedItem(SelectedParameter)
			.OptionsSource(&ParameterOptions)
			.OnSelectionChanged(this, &FCustomizableObjectStateParameterSelector::OnParameterNameSelectionChanged)
			.OnGenerateWidget(this, &FCustomizableObjectStateParameterSelector::OnGenerateStateParameterSelectorComboBox)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FCustomizableObjectStateParameterSelector::GetSelectedParameterName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	]
	.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectStateParameterSelector::ResetSelectedParameterButtonClicked)));
}


void FCustomizableObjectStateParameterSelector::OnParameterNameSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		FString Value = Selection == ParameterOptions[0] ? FString() : *Selection;
		PropertyHandle->SetValue(Value);
		SelectedParameter = Selection;
	}
}


FText FCustomizableObjectStateParameterSelector::GetSelectedParameterName() const
{
	if (SelectedParameter.IsValid())
	{
		return FText::FromString(*SelectedParameter);
	}

	return FText();
}


void FCustomizableObjectStateParameterSelector::GenerateParameterOptions(const FString& SelectedValue)
{
	ParameterOptions.Empty();
	ParameterOptions.Add(MakeShareable(new FString("- Nothing Selected -")));

	SelectedParameter = ParameterOptions.Last();

	for (const FString& ParameterName : BaseObjectNode->ParameterNames)
	{
		ParameterOptions.Add(MakeShareable(new FString(ParameterName)));

		if (ParameterName == SelectedValue)
		{
			SelectedParameter = ParameterOptions.Last();
		}
	}

	// we should always have something selected
	check(SelectedParameter.IsValid());
}


TSharedRef<SWidget> FCustomizableObjectStateParameterSelector::OnGenerateStateParameterSelectorComboBox(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(FText::FromString(*InItem.Get())).Font(IDetailLayoutBuilder::GetDetailFont());
}


void FCustomizableObjectStateParameterSelector::ResetSelectedParameterButtonClicked()
{
	check(ParameterOptions.Num() > 0);

	PropertyHandle->SetValue(FString());
	SelectedParameter = ParameterOptions[0];
}



// Details -------------------------------------------------------------------------------

TSharedRef<IDetailCustomization> FCustomizableObjectNodeObjectDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeObjectDetails );
}


void FCustomizableObjectNodeObjectDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	BaseObjectNode = nullptr;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		BaseObjectNode = Cast<UCustomizableObjectNodeObject>( DetailsView->GetSelectedObjects()[0].Get() );
	}

	IDetailCategoryBuilder& StatesCategory = DetailBuilder.EditCategory("States");
	IDetailCategoryBuilder& ExternalCategory = DetailBuilder.EditCategory("AttachedToExternalObject");
	
	if (BaseObjectNode.IsValid())
	{
		// Properties
		TSharedRef<IPropertyHandle> StatesProperty = DetailBuilder.GetProperty("States");
		//TODO(Max UE-215837)
		//TSharedRef<IPropertyHandle> ComponentsProperty = DetailBuilder.GetProperty("Components");
		TSharedRef<IPropertyHandle> ParentObjectProperty = DetailBuilder.GetProperty("ParentObject");
		TSharedRef<IPropertyHandle> LODsProperty = DetailBuilder.GetProperty("NumLODs");
		
		// Callbacks
		StatesProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeObjectDetails::OnStatesPropertyChanged));

		// Hidden Properties
		DetailBuilder.HideProperty("ParentObjectGroupId");
		DetailBuilder.HideProperty("ParentObject");

		// Hide States and RealTimeMorphSelectionOverrides when this node is a child object (not a base, or has a parent).
		if (!BaseObjectNode->bIsBase || BaseObjectNode->ParentObject)
		{
			DetailBuilder.HideProperty(StatesProperty);
			DetailBuilder.HideProperty("RealTimeMorphSelectionOverrides");
		}

		GroupNodeComboOptions.Empty();

		if (BaseObjectNode->bIsBase)
		{
			FillParameterNamesArray();

			ExternalCategory.AddCustomRow(LOCTEXT("FCustomizableObjectNodeObjectDetails", "Blocks"))
			[
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UCustomizableObject::StaticClass())
				.OnObjectChanged(this, &FCustomizableObjectNodeObjectDetails::ParentObjectSelectionChanged)
				.ObjectPath(BaseObjectNode->ParentObject->GetPathName())
				.ForceVolatile(true)
			];

			if (BaseObjectNode->ParentObject)
			{
				TArray<UCustomizableObjectNodeObjectGroup*> GroupNodes;
				BaseObjectNode->ParentObject->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeObjectGroup>(GroupNodes);

				TSharedPtr<FString> ItemToSelect;

				for (UCustomizableObjectNodeObjectGroup* GroupNode : GroupNodes)
				{
					GroupNodeComboOptions.Add(MakeShareable(new FString(GroupNode->GetGroupName())));

					if (BaseObjectNode->ParentObjectGroupId == GroupNode->NodeGuid)
					{
						ItemToSelect = GroupNodeComboOptions.Last();
					}
				}

				if (!BaseObjectNode->ParentObjectGroupId.IsValid() && ParentComboOptions.Num() > 0)
				{
					ItemToSelect = GroupNodeComboOptions.Last();
				}

                GroupNodeComboOptions.Sort(CompareNames);

				TSharedRef<IPropertyHandle> ParentProperty = DetailBuilder.GetProperty("ParentObjectGroupId");

				ExternalCategory.AddCustomRow(LOCTEXT("FCustomizableObjectNodeObjectDetails", "Blocks"))
				[
					SNew(SProperty, ParentProperty)
					.ShouldDisplayName(false)
					.CustomWidget()
					[
						SNew(SBorder)
						.BorderImage(UE_MUTABLE_GET_BRUSH("NoBorder"))
						.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
						[
							SNew(STextComboBox)
							.OptionsSource(&GroupNodeComboOptions)
							.InitiallySelectedItem(ItemToSelect)
							.OnSelectionChanged(this, &FCustomizableObjectNodeObjectDetails::OnGroupNodeComboBoxSelectionChanged, ParentProperty)
						]
					]
				];
				
				DetailBuilder.HideProperty(StatesProperty);
			}
		}
		else
		{
			DetailBuilder.HideProperty("NumMeshComponents");
			DetailBuilder.HideProperty("ObjectName");
		}

		
		/*
		StatesCategory.AddWidget()
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.Padding( 2.0f )
			[
				SNew( SFilterableDetail, NSLocEdL( "States", "States" ), &StatesCategory )
				[
					SAssignNew(StatesTree, STreeView<TSharedPtr< FStateDetailsNode > >)
						//.Visibility(EVisibility::Collapsed)
						.SelectionMode(ESelectionMode::Single)
						.TreeItemsSource( &RootTreeItems )
						// Called to child items for any given parent item
						.OnGetChildren( this, &FCustomizableObjectDetails::OnGetChildrenForStateTree )
						// Generates the actual widget for a tree item
						.OnGenerateRow( this, &FCustomizableObjectDetails::OnGenerateRowForStateTree ) 

						// Generates the right click menu.
						//.OnContextMenuOpening(this, &SClassViewer::BuildMenuWidget)

						// Find out when the user selects something in the tree
						//.OnSelectionChanged( this, &SClassViewer::OnClassViewerSelectionChanged )

						// Allow for some spacing between items with a larger item height.
						.ItemHeight(20.0f)

						.HeaderRow
						(
							SNew(SHeaderRow)
							.Visibility(EVisibility::Collapsed)
							+ SHeaderRow::Column(TEXT("State"))
							.DefaultLabel(LocEdL("CustomizableObjectDetails","State","State"))
						)
				]
			]
		];
		*/
	}
	else
	{
		StatesCategory.AddCustomRow( LOCTEXT("Node","Node") )
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "Node not found", "Node not found" ) )
		];
	}
}


void FCustomizableObjectNodeObjectDetails::ParentObjectSelectionChanged(const FAssetData & AssetData)
{
	if (BaseObjectNode.IsValid())
	{
		UCustomizableObject* Parent = Cast<UCustomizableObject>(UE::Mutable::Private::LoadObject(AssetData));
		BaseObjectNode->SetParentObject(Parent);

		// If set the parent to nullt, invalidate also the reference GUID
		if (!Parent)
		{
			BaseObjectNode->ParentObjectGroupId.Invalidate();
		}
	}

	RefreshDetails();
}


void FCustomizableObjectNodeObjectDetails::OnGroupNodeComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> ParentProperty)
{
	if (Selection.IsValid() && BaseObjectNode->ParentObject != nullptr)
	{
		TArray<UCustomizableObjectNodeObjectGroup*> GroupNodes;
		BaseObjectNode->ParentObject->GetPrivate()->GetSource()->GetNodesOfClass<UCustomizableObjectNodeObjectGroup>(GroupNodes);

		for (UCustomizableObjectNodeObjectGroup* GroupNode : GroupNodes)
		{
			if (*Selection == GroupNode->GetGroupName())
			{
				const FScopedTransaction Transaction(LOCTEXT("ChangedAttachedToExternalObjectTransaction", "Changed Attached to External Object"));
				BaseObjectNode->Modify();
				BaseObjectNode->ParentObjectGroupId = GroupNode->NodeGuid;
			}
		}
	}
}


void FCustomizableObjectNodeObjectDetails::OnStatesPropertyChanged()
{
	RefreshDetails();
}


void FCustomizableObjectNodeObjectDetails::FillParameterNamesArray()
{
	BaseObjectNode->ParameterNames.Empty();

	UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(BaseObjectNode->GetOutermostObject());

	if (!CustomizableObject)
	{
		return;
	}

	// Get full graph root customizable object
	UCustomizableObject* RootObjet = GraphTraversal::GetRootObject(CustomizableObject);

	// Full tree graph of customizable objects
	TSet<UCustomizableObject*> CustomObjectTree;

	// Get and load the whole tree of customizable object
	GraphTraversal::GetAllObjectsInGraph(RootObjet, CustomObjectTree);

	// Array to store all the ids of group nodes of type toggle
	TArray<FGuid> ToggleGroupObjectIds;

	// Array to store all the Child Object Nodes
	TArray<const UCustomizableObjectNodeObject*> AllObjectNodes;

	//TODO(Max) Use the Visit function of the GraphTraversal to take into account macro parameters?

	for (const UCustomizableObject* Object : CustomObjectTree)
	{
		// Cheking Private to avoid a static analysis waring
		if (!Object || !Object->GetPrivate() || !Object->GetPrivate()->GetSource())
		{
			continue;
		}

		UEdGraph* Source = Object->GetPrivate()->GetSource();

		// All type of parameter nodes
		TArray<UCustomizableObjectNodeColorParameter*> ColorParameterNodes;
		Source->GetNodesOfClass(ColorParameterNodes);

		for (const UCustomizableObjectNodeColorParameter* ColorParameterNode : ColorParameterNodes)
		{
			if (ColorParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(ColorParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeFloatParameter*> FloatParameterNodes;
		Source->GetNodesOfClass(FloatParameterNodes);

		for (const UCustomizableObjectNodeFloatParameter* FloatParameterNode : FloatParameterNodes)
		{
			if (FloatParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(FloatParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeEnumParameter*> EnumParameterNodes;
		Source->GetNodesOfClass(EnumParameterNodes);

		for (const UCustomizableObjectNodeEnumParameter* EnumParameterNode : EnumParameterNodes)
		{
			if (EnumParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(EnumParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeGroupProjectorParameter*>GroupProjectorParameterNodes;
		Source->GetNodesOfClass(GroupProjectorParameterNodes);

		for (const UCustomizableObjectNodeGroupProjectorParameter* GroupProjectorParameterNode : GroupProjectorParameterNodes)
		{
			if (GroupProjectorParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(GroupProjectorParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeProjectorParameter*> ProjectorParameterNodes;
		Source->GetNodesOfClass(ProjectorParameterNodes);

		for (const UCustomizableObjectNodeProjectorParameter* ProjectorParameterNode : ProjectorParameterNodes)
		{
			if (ProjectorParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(ProjectorParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeTextureParameter*> TextureParameterNodes;
		Source->GetNodesOfClass(TextureParameterNodes);

		for (const UCustomizableObjectNodeTextureParameter* TextureParameterNode : TextureParameterNodes)
		{
			if (TextureParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(TextureParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeSkeletalMeshParameter*> MeshParameterNodes;
		Source->GetNodesOfClass(MeshParameterNodes);

		for (const UCustomizableObjectNodeSkeletalMeshParameter* MeshParameterNode : MeshParameterNodes)
		{
			if (MeshParameterNode)
			{
				BaseObjectNode->ParameterNames.Add(MeshParameterNode->GetParameterName());
			}
		}

		TArray<UCustomizableObjectNodeTable*> TableNodes;
		Source->GetNodesOfClass(TableNodes);

		for (const UCustomizableObjectNodeTable* TableNode : TableNodes)
		{
			if (TableNode)
			{
				BaseObjectNode->ParameterNames.Add(TableNode->ParameterName);
			}
		}


		TArray<UCustomizableObjectNodeObjectGroup*> GroupNodes;
		Source->GetNodesOfClass(GroupNodes);

		for (const UCustomizableObjectNodeObjectGroup* GroupNode : GroupNodes)
		{
			if (GroupNode)
			{
				if (GroupNode->GroupType == ECustomizableObjectGroupType::COGT_TOGGLE)
				{
					ToggleGroupObjectIds.Add(GroupNode->NodeGuid);
				}
				else
				{
					BaseObjectNode->ParameterNames.Add(GroupNode->GetGroupName());
				}
			}
		}

		TArray<UCustomizableObjectNodeObject*> ObjectNodes;
		Source->GetNodesOfClass(ObjectNodes);

		for (const UCustomizableObjectNodeObject* ObjectNode : ObjectNodes)
		{
			if (ObjectNode)
			{
				AllObjectNodes.Add(ObjectNode);
			}
		}
	}

	// Now that we know all the group objects of type toggle, we process all the object nodes that can generate a parameter
	for (const UCustomizableObjectNodeObject* ObjectNode : AllObjectNodes)
	{
		FGuid ParentObjectGroupId = FGuid();

		if (ObjectNode->bIsBase)
		{
			ParentObjectGroupId = ObjectNode->ParentObjectGroupId;
		}
		else
		{
			if (const UEdGraphPin* ObjectPin = ObjectNode->OutputPin())
			{
				if (const UEdGraphPin* GroupPin = FollowOutputPin(*ObjectPin))
				{
					if (const UCustomizableObjectNodeObjectGroup* GroupNode = Cast<UCustomizableObjectNodeObjectGroup>(GroupPin->GetOwningNode()))
					{
						ParentObjectGroupId = GroupNode->NodeGuid;
					}
				}
			}
		}

		if (ToggleGroupObjectIds.Contains(ParentObjectGroupId))
		{
			BaseObjectNode->ParameterNames.Add(ObjectNode->GetObjectName());
		}
	}
}

#undef LOCTEXT_NAMESPACE
