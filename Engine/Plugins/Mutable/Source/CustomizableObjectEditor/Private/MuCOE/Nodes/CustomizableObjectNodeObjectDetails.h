// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeDetails.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class SWidget;
class UCustomizableObject;
class UCustomizableObjectNodeObject;
struct FAssetData;
struct FPropertyChangedEvent;

/**	Class created to identify FStrings with metadata ShowParameterOptions.
	This allows us to change the default FString property widget for a ComboBox in the
	CustomizableObjectNodeObject details. */
class FStatePropertyTypeIdentifier : public IPropertyTypeIdentifier 
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
	{
		// Metadata to use a combobox widged instead of a string widge
		return InPropertyHandle.HasMetaData(TEXT("ShowParameterOptions"));
	}
};


/** Custom Widget for the property RuntimeParameters of the States(FCustomizableObjectState) */
class FCustomizableObjectStateParameterSelector : public IPropertyTypeCustomization
{
public:

	FCustomizableObjectStateParameterSelector() {};

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization overrides
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override
	{}

private:

	/** Generates the option of the combobox */
	void GenerateParameterOptions(const FString& SelectedValue);

	/** Generates the Text widget of the combobox */
	TSharedRef<SWidget> OnGenerateStateParameterSelectorComboBox(TSharedPtr<FString> InItem) const;

	/** Callback of OnSelectionChanged of the combobox */
	void OnParameterNameSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Returns the name of the current selected parameter */
	FText GetSelectedParameterName() const;

	/** Callback of the function OverrideResetToDefault */
	void ResetSelectedParameterButtonClicked();

private:

	/** Array with all the possible parameter names */
	TArray<TSharedPtr<FString>> ParameterOptions;

	/** Weak pointer to the Customizable Object that contains this property */
	TWeakObjectPtr<UCustomizableObjectNodeObject> BaseObjectNode;

	/** Pointer to the current selected parameter */
	TSharedPtr<FString> SelectedParameter;

	/** Runtime Parameter Name property of a State */
	TSharedPtr<IPropertyHandle> PropertyHandle;
};


class FCustomizableObjectNodeObjectDetails : public FCustomizableObjectNodeDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	void ParentObjectSelectionChanged(const FAssetData& AssetData);

private:

	TWeakObjectPtr<UCustomizableObjectNodeObject> BaseObjectNode;
	TArray< TSharedPtr<FString> > GroupNodeComboOptions;

	TArray< TSharedPtr<FString> > ParentComboOptions;
	TArray< class UCONodeSkeletalMeshSection* > ParentOptionNode;

	void OnGroupNodeComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, TSharedRef<IPropertyHandle> ParentProperty);
	
	// Refreshes the Details View when the states variable has been updated
	void OnStatesPropertyChanged();
	
	void FillParameterNamesArray();
};
