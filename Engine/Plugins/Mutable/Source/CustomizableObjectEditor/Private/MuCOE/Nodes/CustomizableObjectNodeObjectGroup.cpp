// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeObjectGroup)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeObjectGroup::UCustomizableObjectNodeObjectGroup()
	: Super()
{
	GroupName = "Unnamed Group";
}

void UCustomizableObjectNodeObjectGroup::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	UEdGraphPin* groupPin = GroupProjectorsPin();
	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::GroupProjectorPinTypeAdded
		&& groupPin
		&& groupPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Projector)
	{
		groupPin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_GroupProjector;
	}

	LastGroupName = GroupName;
}


void UCustomizableObjectNodeObjectGroup::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeObjectGroup, GroupName))
	{
		for (UEdGraphPin* LinkedPin : FollowOutputPinArray(*GroupPin()))
		{
			UCustomizableObjectNode* Root = CastChecked<UCustomizableObjectNode>(LinkedPin->GetOwningNode());

			if (UCustomizableObjectNodeObject* CurrentRootNode = Cast<UCustomizableObjectNodeObject>(Root))
			{
				if (CurrentRootNode->ParentObject)
				{
					TArray<const UCustomizableObject*> VisitedObjects;
					CurrentRootNode = GraphTraversal::GetFullGraphRootNode(CurrentRootNode->ParentObject, VisitedObjects);
				}
				
				if (CurrentRootNode && !CurrentRootNode->ParentObject)
				{
					for (FCustomizableObjectState& State : CurrentRootNode->States)
					{
						for (int32 p = 0; p < State.RuntimeParameters.Num(); ++p)
						{
							if (State.RuntimeParameters[p].Equals(LastGroupName))
							{
								State.RuntimeParameters[p]= GroupName;
							}
						}
						if (State.ForcedParameterValues.Contains(GroupName))
						{
							// Forced parameter already contains the NEW name of the parameter. We currently allow inconsistencies while working before compile time, so no warning needed here. When this changes to ID based instead of String based, this would become a warning worthy check (TODO: MTBL-1071).
						}
						else if (State.ForcedParameterValues.Contains(LastGroupName))
						{
							FString LastForcedValue = State.ForcedParameterValues.FindAndRemoveChecked(LastGroupName);
							State.ForcedParameterValues.Emplace(GroupName, LastForcedValue);
						}
					}
				}
			}
		}
	}
	LastGroupName = GroupName;
}


FText UCustomizableObjectNodeObjectGroup::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Object_Group", "Object Group");;
}


FLinearColor UCustomizableObjectNodeObjectGroup::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Object);
}


FText UCustomizableObjectNodeObjectGroup::GetTooltipText() const
{
	return LOCTEXT("Grpup_Object_Tooltip",
	"Define one or multiple parameters that are a collection of Customizable Objects that share a mutual relationship: they either are\nexclusive from each other, at most one of them can be active, or at least one of them has to be, or any combination of them can be\nenabled, or they define materials that will always be shown together.");
}


void UCustomizableObjectNodeObjectGroup::OnRenameNode(const FString& NewName)
{
	if (!NewName.IsEmpty())
	{
		GroupName = NewName;
	}
}


void UCustomizableObjectNodeObjectGroup::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == NamePin.Get())
	{
		GetGraph()->NotifyGraphChanged();
	}
}


void UCustomizableObjectNodeObjectGroup::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));

	UEdGraphPin* ObjectsPin = CustomCreatePin(EGPD_Input, Schema->PC_Object, TEXT("Objects"), LOCTEXT("Objects", "Objects"));
	ObjectsPin->PinType.ContainerType = EPinContainerType::Array;
	
	UEdGraphPin* ProjectorsPin = CustomCreatePin(EGPD_Input, Schema->PC_GroupProjector, TEXT("Projectors"), LOCTEXT("Projectors", "Projectors"));
	ProjectorsPin->PinType.ContainerType = EPinContainerType::Array;
	
	CustomCreatePin(EGPD_Output, Schema->PC_Object, TEXT("Group"), LOCTEXT("Group", "Group"));
}


bool UCustomizableObjectNodeObjectGroup::IsSingleOutputNode() const
{
	return true;
}


void UCustomizableObjectNodeObjectGroup::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!NamePin.Get())
		{
			NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
		}
	}
}


bool UCustomizableObjectNodeObjectGroup::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCustomizableObjectNodeObjectGroup::GetPinEditableName(const UEdGraphPin& Pin) const
{
	if (NamePin.Get()->PinId == Pin.PinId)
	{
		return FText::FromString(GetGroupName());
	}
	else
	{
		return Super::GetPinEditableName(Pin);
	}
}


void UCustomizableObjectNodeObjectGroup::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	if (NamePin.Get()->PinId == Pin.PinId)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeObjectGroup_SetGroupNamePinEditableNameTransaction", "Change Group Name PinName"));
		Modify();
		
		SetGroupName(InValue.ToString());
		PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
	}
	else
	{
		Super::SetPinEditableName(Pin, InValue);
	}
}


UEdGraphPin* UCustomizableObjectNodeObjectGroup::ObjectsPin() const
{
	return FindPin(TEXT("Objects"));
}


UEdGraphPin* UCustomizableObjectNodeObjectGroup::GroupProjectorsPin() const
{
	return FindPin(TEXT("Projectors"));
}


UEdGraphPin* UCustomizableObjectNodeObjectGroup::GroupPin() const
{
	return FindPin(TEXT("Group"));
}


FString UCustomizableObjectNodeObjectGroup::GetGroupName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (NamePin.Get())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*NamePin.Get()))
		{
			const UEdGraphPin* StringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, MacroContext);

			if (const UCustomizableObjectNodeStaticString* StringNode = StringPin ? Cast<UCustomizableObjectNodeStaticString>(StringPin->GetOwningNode()) : nullptr)
			{
				return StringNode->Value;
			}
		}
	}

	return GroupName;
}


void UCustomizableObjectNodeObjectGroup::SetGroupName(const FString& Name)
{
	GroupName = Name;
}

#undef LOCTEXT_NAMESPACE
