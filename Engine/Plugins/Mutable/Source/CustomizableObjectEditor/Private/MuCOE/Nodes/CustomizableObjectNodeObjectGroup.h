// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeObjectGroup.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

enum class ECustomizableObjectGroupType : uint8;
namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeObjectGroup : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeObjectGroup();

	UPROPERTY(EditAnywhere, Category=GroupInfo, meta = (DisplayName = "Type"))
	ECustomizableObjectGroupType GroupType;
	
	UPROPERTY(EditAnywhere, Category=GroupInfo)
	FString DefaultValue;

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "Parameter UI Metadata"))
	FMutableParamUIMetadata ParamUIMetadata;

	// The pose defined in meshes deriving from this node will inherit this bone pose priority. Decides the bone pose to keep when two bones
	// that have the same bone usage flags (E.g. EBoneUsageFlags::Skinning). The one with higher priority will be kept.
	UPROPERTY(EditAnywhere, Category = MeshPose)
	int8 BonePosePriority = 0;

	// The sockets defined in meshes deriving from this node will inherit this socket priority. When in the generated merged mesh there
	// are clashes with socket names, the one with higher priority will be kept and the other discarded.
	UPROPERTY(EditAnywhere, Category = MeshSockets)
	int8 SocketPriority = 0;
	
	UPROPERTY()
	FEdGraphPinReference NamePin;

	// UObject interface.
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual bool IsSingleOutputNode() const override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	UE_API virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	UE_API virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;

	// Own interface
	UE_API UEdGraphPin* ObjectsPin() const;
	UE_API UEdGraphPin* GroupProjectorsPin() const;
	UE_API UEdGraphPin* GroupPin() const;
	UE_API FString GetGroupName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;
	UE_API void SetGroupName(const FString& Name);

private:

	UPROPERTY(EditAnywhere, Category = GroupInfo, meta = (DisplayName = "Name"))
	FString GroupName;

	FString LastGroupName;

};

#undef UE_API
