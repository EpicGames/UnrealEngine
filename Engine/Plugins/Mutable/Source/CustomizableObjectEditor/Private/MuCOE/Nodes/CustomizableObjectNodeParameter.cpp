// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMacroInstance.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeParameter)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeParameter::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const FName Type = GetCategory();
	const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(Type);
	const FText PinFriendlyName = GetCategoryFriendlyName();
	
	UEdGraphPin* ValuePin = CustomCreatePin(EGPD_Output, Type, PinName, PinFriendlyName);
	ValuePin->PinFriendlyName = PinFriendlyName;
	ValuePin->bDefaultValueIsIgnored = true;

	NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
}


bool UCustomizableObjectNodeParameter::IsAffectedByLOD() const
{
	return false;
}


void UCustomizableObjectNodeParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!NamePin.Get())
		{
			NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
		}
	}
}


bool UCustomizableObjectNodeParameter::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCustomizableObjectNodeParameter::GetPinEditableName(const UEdGraphPin& Pin) const
{
	if (NamePin.Get()->PinId == Pin.PinId)
	{
		return FText::FromString(GetParameterName());
	}
	else
	{
		return Super::GetPinEditableName(Pin);
	}
}


void UCustomizableObjectNodeParameter::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	if (NamePin.Get()->PinId == Pin.PinId)
	{
		FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeParameter_SetParameterName", "Change ParameterName PinName"));
		Modify();
		
		SetParameterName(InValue.ToString());
		PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
	}
	else
	{
		Super::SetPinEditableName(Pin, InValue);
	}
}


FText UCustomizableObjectNodeParameter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("ParameterTitle_ListView", "{0} Parameter"), GetCategoryFriendlyName());
}


FLinearColor UCustomizableObjectNodeParameter::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(GetCategory());
}


FText UCustomizableObjectNodeParameter::GetTooltipText() const
{
	return FText::Format(LOCTEXT("Parameter_Tooltip", "Expose a runtime modifiable {0} parameter from the Customizable Object."), FText::FromName(GetCategory()));
}


void UCustomizableObjectNodeParameter::OnRenameNode(const FString& NewName)
{
	if (!NewName.IsEmpty())
	{
		ParameterName = NewName;
	}
}


void UCustomizableObjectNodeParameter::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == NamePin.Get())
	{
		GetGraph()->NotifyGraphChanged();
	}
}


FText UCustomizableObjectNodeParameter::GetCategoryFriendlyName() const
{
	return UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetCategory());
}


FString UCustomizableObjectNodeParameter::GetParameterName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (NamePin.Get())
	{
		if (const UEdGraphPin* LinkedPin = FollowInputPin(*NamePin.Get()))
		{
			const UEdGraphPin* StringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*LinkedPin, MacroContext);

			if (const UCustomizableObjectNodeStaticString* StringNode = StringPin ? Cast<UCustomizableObjectNodeStaticString>(StringPin->GetOwningNode()) : nullptr)
			{
				return StringNode->Value;
			}
		}
	}

	return ParameterName;
}


void UCustomizableObjectNodeParameter::SetParameterName(const FString& Name)
{
	ParameterName = Name;
}

#undef LOCTEXT_NAMESPACE

