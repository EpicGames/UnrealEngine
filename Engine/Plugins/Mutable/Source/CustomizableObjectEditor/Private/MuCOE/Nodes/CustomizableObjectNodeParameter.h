// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UObject;


UCLASS(MinimalAPI, Abstract)
class UCustomizableObjectNodeParameter : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "Parameter UI Metadata"))
	FMutableParamUIMetadata ParamUIMetadata;

	UPROPERTY()
	FEdGraphPinReference NamePin;

private:

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString ParameterName = "Default Name";

public:

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual bool IsAffectedByLOD() const override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	UE_API virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	UE_API virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;

	// Own interface
	/** Return the pin category of this node. */
	virtual FName GetCategory() const PURE_VIRTUAL(UCustomizableObjectNodeParameter::GetCategory, return {}; );
	virtual FText GetCategoryFriendlyName() const;
	UE_API virtual FString GetParameterName(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;
	UE_API virtual void SetParameterName(const FString& Name);
};

#undef UE_API
