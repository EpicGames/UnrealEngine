// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeParameterDetails.h"

#include "CustomizableObjectNodeParameter.h"
#include "DetailLayoutBuilder.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeParameterDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeParameterDetails);
}

void FCustomizableObjectNodeParameterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);
	
	DetailBuilder.HideProperty("ParameterName", UCustomizableObjectNodeParameter::StaticClass());
}