// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodePassThroughTextureVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodePassThroughTextureVariation)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


// UCustomizableObjectNodeVariation interface
FName UCustomizableObjectNodePassThroughTextureVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough;
}


void UCustomizableObjectNodePassThroughTextureVariation::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Texture")))
		{
			Pin->PinName = TEXT("PassThrough Texture");
			Pin->PinFriendlyName = LOCTEXT("PassThrough_Image_Pin_Category", "PassThrough Texture");
		}
	}
}


#undef LOCTEXT_NAMESPACE
