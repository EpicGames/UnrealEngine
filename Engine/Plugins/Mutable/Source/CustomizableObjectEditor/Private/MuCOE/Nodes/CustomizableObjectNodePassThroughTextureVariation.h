// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"

#include "CustomizableObjectNodePassThroughTextureVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


UCLASS(MinimalAPI)
class UCustomizableObjectNodePassThroughTextureVariation : public UCustomizableObjectNodeTextureVariation
{
public:
	GENERATED_BODY()

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	// UCustomizableObjectNodeVariation interface
	UE_API virtual FName GetCategory() const override;
};

#undef UE_API
