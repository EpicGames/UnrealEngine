// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeProjectorConstant.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeProjectorConstant)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeProjectorConstant::UCustomizableObjectNodeProjectorConstant()
	: Super()
	, ProjectionAngle(360.0f)
	, BoneComboBoxLocation(FVector::ZeroVector)
	, BoneComboBoxForwardDirection(FVector::ZeroVector)
	, BoneComboBoxUpDirection(FVector::ZeroVector)
{

}


void UCustomizableObjectNodeProjectorConstant::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == "ProjectionAngle")
	{
		Value.Angle = FMath::DegreesToRadians(ProjectionAngle);
	}
	else if (PropertyName == "ProjectorBone")
	{
		Value.Position = (FVector3f)BoneComboBoxLocation;
		Value.Direction = (FVector3f)BoneComboBoxForwardDirection;
		Value.Up = (FVector3f)BoneComboBoxUpDirection;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeProjectorConstant::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Projector);
}


void UCustomizableObjectNodeProjectorConstant::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ProjectorNodesDefaultValueFix)
	{
		Value.ProjectionType = ProjectionType_DEPRECATED;
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SnapToBoneComponentIndexToName)
	{
		ReferenceSkeletonComponent = FName(FString::FromInt(ReferenceSkeletonIndex_DEPRECATED));
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Projector");
			Pin->PinFriendlyName = LOCTEXT("Projector_Pin_Category", "Projector");
		}
	}
}


ECustomizableObjectProjectorType UCustomizableObjectNodeProjectorConstant::GetProjectorType() const
{
	return Value.ProjectionType;
}


FVector UCustomizableObjectNodeProjectorConstant::GetProjectorPosition() const
{
	return static_cast<FVector>(Value.Position);
}


void UCustomizableObjectNodeProjectorConstant::SetProjectorPosition(const FVector& Position)
{
	Value.Position = static_cast<FVector3f>(Position);
}


FVector UCustomizableObjectNodeProjectorConstant::GetProjectorDirection() const
{
	return static_cast<FVector>(Value.Direction);
}


void UCustomizableObjectNodeProjectorConstant::SetProjectorDirection(const FVector& Direction)
{
	Value.Direction = static_cast<FVector3f>(Direction);
}


FVector UCustomizableObjectNodeProjectorConstant::GetProjectorUp() const
{
	return static_cast<FVector>(Value.Up);
}


void UCustomizableObjectNodeProjectorConstant::SetProjectorUp(const FVector& Up)
{
	Value.Up = static_cast<FVector3f>(Up);
}


FVector UCustomizableObjectNodeProjectorConstant::GetProjectorScale() const
{
	return static_cast<FVector>(Value.Scale);
}


void UCustomizableObjectNodeProjectorConstant::SetProjectorScale(const FVector& Scale)
{
	Value.Scale = static_cast<FVector3f>(Scale);
}


float UCustomizableObjectNodeProjectorConstant::GetProjectorAngle() const
{
	return ProjectionAngle;
}


void UCustomizableObjectNodeProjectorConstant::SetProjectorAngle(float Angle)
{
	ProjectionAngle = Angle;
}


FText UCustomizableObjectNodeProjectorConstant::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Projector_Constant", "Projector");
}


FLinearColor UCustomizableObjectNodeProjectorConstant::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Projector);
}


FText UCustomizableObjectNodeProjectorConstant::GetTooltipText() const
{
		return LOCTEXT("Projector_Constant_Tooltip",
			"Defines a constant projector.It can't move, scale or rotate at runtime. The texture that is projected can still be changed, depending on the configuration of the other inputs of the texture project node that is connected to the projector constant.");
}


#undef LOCTEXT_NAMESPACE
