// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeProjectorConstant.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeProjectorConstant : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeProjectorConstant();

	/**  */
	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta = (ShowOnlyInnerProperties))
	FCustomizableObjectProjector Value;

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "Parameter UI Metadata"))
	FMutableParamUIMetadata ParamUIMetadata;

	UPROPERTY(EditAnywhere, Category = CustomizableObject, Meta = (DisplayName = "Projection Angle (degrees)"))
	float ProjectionAngle;

	UPROPERTY()
	uint32 ReferenceSkeletonIndex_DEPRECATED = 0;

	UPROPERTY(EditAnywhere, Category = ProjectorSnapToBone)
	FName ReferenceSkeletonComponent;

	UPROPERTY(EditAnywhere, Category = ProjectorSnapToBone)
	FName ProjectorBone;

	/** Temporary variable where to put the location information for bone combo box selection changes (in FCustomizableObjectNodeProjectorParameterDetails)*/
	FVector BoneComboBoxLocation;

	/** Temporary variable where to put the direction information for bone combo box selection changes (in FCustomizableObjectNodeProjectorParameterDetails)*/
	FVector BoneComboBoxForwardDirection;

	/** Temporary variable where to put the direction information for bone combo box selection changes (in FCustomizableObjectNodeProjectorParameterDetails)*/
	FVector BoneComboBoxUpDirection;

	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Own interface

	UE_API ECustomizableObjectProjectorType GetProjectorType() const;

	UE_API FVector GetProjectorPosition() const;

	UE_API void SetProjectorPosition(const FVector& Position);

	UE_API FVector GetProjectorDirection() const;

	UE_API void SetProjectorDirection(const FVector& Direction);

	UE_API FVector GetProjectorUp() const;

	UE_API void SetProjectorUp(const FVector& Up);

	UE_API FVector GetProjectorScale() const;

	UE_API void SetProjectorScale(const FVector& Scale);

	UE_API float GetProjectorAngle() const;

	UE_API void SetProjectorAngle(float Angle);

private:

	UPROPERTY()
	ECustomizableObjectProjectorType ProjectionType_DEPRECATED;
};

#undef UE_API
