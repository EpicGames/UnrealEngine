// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeProjectorParameter.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeProjectorParameter)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeProjectorParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == "ProjectionAngle")
	{
		DefaultValue.Angle = FMath::DegreesToRadians(ProjectionAngle);
	}
	else if (PropertyName == "ProjectorBone")
	{
		DefaultValue.Position = (FVector3f)BoneComboBoxLocation;
		DefaultValue.Direction = (FVector3f)BoneComboBoxForwardDirection;
		DefaultValue.Up = (FVector3f)BoneComboBoxUpDirection;
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeProjectorParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ProjectorNodesDefaultValueFix)
	{
		DefaultValue.ProjectionType = ProjectionType_DEPRECATED;
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SnapToBoneComponentIndexToName)
	{
		ReferenceSkeletonComponent = FName(FString::FromInt(ReferenceSkeletonIndex_DEPRECATED));
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Projector");
			Pin->PinFriendlyName = LOCTEXT("Projector_Pin_Category", "Projector");
		}
	}
}


FName UCustomizableObjectNodeProjectorParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Projector;
}


ECustomizableObjectProjectorType UCustomizableObjectNodeProjectorParameter::GetProjectorType() const
{
	return DefaultValue.ProjectionType;
}


FVector UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultPosition() const
{
	return static_cast<FVector>(DefaultValue.Position);
}


void UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultPosition(const FVector& Position)
{
	DefaultValue.Position = static_cast<FVector3f>(Position);
}


FVector UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultDirection() const
{
	return static_cast<FVector>(DefaultValue.Direction);
}


void UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultDirection(const FVector& Direction)
{
	DefaultValue.Direction = static_cast<FVector3f>(Direction);
}


FVector UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultUp() const
{
	return static_cast<FVector>(DefaultValue.Up);
}


void UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultUp(const FVector& Up)
{
	DefaultValue.Up = static_cast<FVector3f>(Up);
}


FVector UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultScale() const
{
	return static_cast<FVector>(DefaultValue.Scale);
}


void UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultScale(const FVector& Scale)
{
	DefaultValue.Scale = static_cast<FVector3f>(Scale);
}


float UCustomizableObjectNodeProjectorParameter::GetProjectorDefaultAngle() const
{
	return ProjectionAngle;
}


void UCustomizableObjectNodeProjectorParameter::SetProjectorDefaultAngle(float Angle)
{
	ProjectionAngle = Angle;
}


#undef LOCTEXT_NAMESPACE
