// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "CustomizableObjectNodeProjectorParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeProjectorParameter : public UCustomizableObjectNodeParameter
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=CustomizableObject, meta = (ShowOnlyInnerProperties))
	FCustomizableObjectProjector DefaultValue;

	UPROPERTY(EditAnywhere, Category = CustomizableObject, Meta = (DisplayName = "Projection Angle (degrees)"))
	float ProjectionAngle = 360.0f;

	UPROPERTY()
	uint32 ReferenceSkeletonIndex_DEPRECATED = 0;

	UPROPERTY(EditAnywhere, Category = ProjectorSnapToBone)
	FName ReferenceSkeletonComponent;

	UPROPERTY(EditAnywhere, Category = ProjectorSnapToBone)
	FName ProjectorBone;

	/** Temporary variable where to put the location information for bone combo box selection changes (in FCustomizableObjectNodeProjectorParameterDetails)*/
	FVector BoneComboBoxLocation = FVector::ZeroVector;

	/** Temporary variable where to put the direction information for bone combo box selection changes (in FCustomizableObjectNodeProjectorParameterDetails)*/
	FVector BoneComboBoxForwardDirection = FVector::ZeroVector;

	/** Temporary variable where to put the direction information for bone combo box selection changes (in FCustomizableObjectNodeProjectorParameterDetails)*/
	FVector BoneComboBoxUpDirection = FVector::ZeroVector;

	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;

	// Own interface

	UE_API ECustomizableObjectProjectorType GetProjectorType() const;

	UE_API FVector GetProjectorDefaultPosition() const;
	
	UE_API void SetProjectorDefaultPosition(const FVector& Position);

	UE_API FVector GetProjectorDefaultDirection() const;

	UE_API void SetProjectorDefaultDirection(const FVector& Direction);

	UE_API FVector GetProjectorDefaultUp() const;

	UE_API void SetProjectorDefaultUp(const FVector& Up);

	UE_API FVector GetProjectorDefaultScale() const;

	UE_API void SetProjectorDefaultScale(const FVector& Scale);

	UE_API float GetProjectorDefaultAngle() const;

	UE_API void SetProjectorDefaultAngle(float Angle);

private:

	UPROPERTY()
	ECustomizableObjectProjectorType ProjectionType_DEPRECATED;
};

#undef UE_API
