// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeProjectorParameterDetails.h"

#include "DetailLayoutBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "ScopedTransaction.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectGraphEditorToolkit.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorConstant.h"
#include "MuCOE/Nodes/CustomizableObjectNodeProjectorParameter.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeProjectorParameterDetails::MakeInstance()
{
	return MakeShareable( new FCustomizableObjectNodeProjectorParameterDetails );
}


void FCustomizableObjectNodeProjectorParameterDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	FCustomizableObjectNodeParameterDetails::CustomizeDetails(DetailBuilder);

	// Set the order in which the categories will be displayed. This is required because we are editing some of them and that makes them change order
	DetailBuilder.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
	{
		(*CategoryMap.FindChecked(FName("Clipboard"))).SetSortOrder(0);
		(*CategoryMap.FindChecked(FName("CustomizableObject"))).SetSortOrder(1);
		(*CategoryMap.FindChecked(FName("ProjectorSnapToBone"))).SetSortOrder(2);
		(*CategoryMap.FindChecked(FName("UI"))).SetSortOrder(3);
	});

	NodeConstant = nullptr;
	NodeParameter = nullptr;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		if (DetailsView->GetSelectedObjects()[0].Get()->IsA(UCustomizableObjectNodeProjectorConstant::StaticClass()))
		{
			NodeConstant = Cast<UCustomizableObjectNodeProjectorConstant>(DetailsView->GetSelectedObjects()[0].Get());
		}
		else if (DetailsView->GetSelectedObjects()[0].Get()->IsA(UCustomizableObjectNodeProjectorParameter::StaticClass()))
		{
			NodeParameter = Cast<UCustomizableObjectNodeProjectorParameter>(DetailsView->GetSelectedObjects()[0].Get());
		}
	}

	
	IDetailCategoryBuilder& BlocksCategory = DetailBuilder.EditCategory( "Clipboard" );
	
	if ((NodeConstant != nullptr) || (NodeParameter != nullptr))
	{
		BlocksCategory.AddCustomRow( LOCTEXT("FCustomizableObjectNodeProjectorParameterDetails", "Projector Data") )
		[
			SNew( SHorizontalBox )
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ToolTipText( LOCTEXT( "Copy_Projector", "Copy projector location to clipboard.") )
				.OnClicked( this, &FCustomizableObjectNodeProjectorParameterDetails::OnProjectorCopyPressed )
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Copy","Copy"))
				]
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ToolTipText( LOCTEXT( "Paste_Projector", "Paste projector location from clipboard.") )
				.OnClicked( this, &FCustomizableObjectNodeProjectorParameterDetails::OnProjectorPastePressed )
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Paste","Paste"))
				]
			]
		];
	}
	else
	{
		BlocksCategory.AddCustomRow( LOCTEXT("Node", "Node") )
		[
			SNew( STextBlock )
			.Text( LOCTEXT( "Node not found", "Node not found" ) )
		];
	}

	TSharedPtr<IPropertyHandle> ReferenceSkeletonIndexPropertyHandle = DetailBuilder.GetProperty("ReferenceSkeletonComponent");
	const FSimpleDelegate OnReferenceSkeletonComponentChangedDelegate =
		FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeProjectorParameterDetails::OnReferenceSkeletonComponentChanged);
	ReferenceSkeletonIndexPropertyHandle->SetOnPropertyValueChanged(OnReferenceSkeletonComponentChangedDelegate);
	
	CaptureContextData();
	CacheSkeletalMeshBoneNames();

	// Add them to the parent combo box
	TSharedRef<IPropertyHandle> BoneProperty = DetailBuilder.GetProperty("ProjectorBone");

	DetailBuilder.EditDefaultProperty(BoneProperty)
	->CustomWidget()
	.Visibility(TAttribute<EVisibility>(this, &FCustomizableObjectNodeProjectorParameterDetails::ShouldBoneDropdownBeVisible))
	.NameContent()
	[
		SNew(STextBlock)
		.Text( LOCTEXT("FCustomizableObjectNodeRemoveMeshDetails", "Projector Bone"))
		.Font(IDetailLayoutBuilder::GetDetailFont())

	].ValueContent()
	[
		SAssignNew(BoneSelectionComboBox, STextComboBox)
		.OptionsSource(&BoneComboOptions)
		.InitiallySelectedItem(BoneToSelect)
		.OnSelectionChanged(this, &FCustomizableObjectNodeProjectorParameterDetails::OnBoneComboBoxSelectionChanged, BoneProperty)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
	
}


FReply FCustomizableObjectNodeProjectorParameterDetails::OnProjectorCopyPressed() const
{
	FString ExportedText;

	if (NodeParameter)
	{
		UScriptStruct* Struct = NodeParameter->DefaultValue.StaticStruct();
        Struct->ExportText(ExportedText, &NodeParameter->DefaultValue, nullptr, nullptr, (PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited | PPF_IncludeTransient), nullptr);
	}
	else if(NodeConstant)
	{
		UScriptStruct* Struct = NodeConstant->Value.StaticStruct();
        Struct->ExportText(ExportedText, &NodeConstant->Value, nullptr, nullptr, (PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited | PPF_IncludeTransient), nullptr);
	}

	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	return FReply::Handled();
}


FReply FCustomizableObjectNodeProjectorParameterDetails::OnProjectorPastePressed() const
{
	FScopedTransaction Transaction(LOCTEXT("PasteTransform", "Paste Transform"));
	
	FString ClipText;

	FPlatformApplicationMisc::ClipboardPaste(ClipText);

	TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = nullptr;

	if (NodeParameter)
	{
		NodeParameter->Modify();
	
		UScriptStruct* Struct = NodeParameter->DefaultValue.StaticStruct();
		Struct->ImportText(*ClipText, &NodeParameter->DefaultValue, nullptr, 0, GLog, GetPathNameSafe(Struct));
		Editor = NodeParameter->GetGraphEditor();
	}
	else if (NodeConstant)
	{
		NodeConstant->Modify();
		
		UScriptStruct* Struct = NodeConstant->Value.StaticStruct();
		Struct->ImportText(*ClipText, &NodeConstant->Value, nullptr, 0, GLog, GetPathNameSafe(Struct));
		Editor = NodeConstant->GetGraphEditor();
	}

	if (Editor.IsValid())
	{
		Editor->UpdateGraphNodeProperties();
	}

	return FReply::Handled();
}


void FCustomizableObjectNodeProjectorParameterDetails::CaptureContextData()
{
	UCustomizableObject* CustomizableObject = nullptr;
	FName ReferenceComponent;

	if (NodeConstant != nullptr)
	{
		CustomizableObject = Cast<UCustomizableObject>(NodeConstant->GetCustomizableObjectGraph()->GetOuter());
		ProjectorBoneName = NodeConstant->ProjectorBone;
		ReferenceComponent = NodeConstant->ReferenceSkeletonComponent;
	}
	else if (NodeParameter != nullptr)
	{
		CustomizableObject = Cast<UCustomizableObject>(NodeParameter->GetCustomizableObjectGraph()->GetOuter());
		ProjectorBoneName = NodeParameter->ProjectorBone;
		ReferenceComponent = NodeParameter->ReferenceSkeletonComponent;
	}

	SkeletalMesh = nullptr;
	if (CustomizableObject)
	{
		SkeletalMesh = CustomizableObject->GetComponentMeshReferenceSkeletalMesh(ReferenceComponent);
	}
}


void FCustomizableObjectNodeProjectorParameterDetails::CacheSkeletalMeshBoneNames()
{
	BoneComboOptions.Empty();
	
	BoneToSelect.Reset();
	if (BoneSelectionComboBox)
	{
		// Ensure the combo box displayed element is reset when changing the data being pointed at by "BoneToSelect"
		BoneSelectionComboBox->ClearSelection();
	}
	
	if (SkeletalMesh)
	{
		for (int32 i = 0; i < SkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++i)
		{
			FName BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(i);
			BoneComboOptions.Add(MakeShareable(new FString(BoneName.ToString())));
	
			if (BoneName == ProjectorBoneName)
			{
				BoneToSelect = BoneComboOptions.Last();
			}
		}
	
		BoneComboOptions.Sort(CompareNames);
	}
}


void FCustomizableObjectNodeProjectorParameterDetails::OnBoneComboBoxSelectionChanged(TSharedPtr<FString> InSelection, ESelectInfo::Type InSelectInfo, TSharedRef<IPropertyHandle> InBoneProperty)
{
	for (int OptionIndex = 0; OptionIndex < BoneComboOptions.Num(); ++OptionIndex)
	{
		if (BoneComboOptions[OptionIndex] == InSelection)
		{
			FVector Location = FVector::ZeroVector;
			FVector Direction = FVector::ForwardVector;

			if (SkeletalMesh)
			{
				const TArray<FTransform>& BoneArray = SkeletalMesh->GetRefSkeleton().GetRefBonePose();
				int32 ParentIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(FName(**InSelection));

				FVector ChildLocation = FVector::ForwardVector;

				for (int32 i = 0; i < SkeletalMesh->GetRefSkeleton().GetRawBoneNum(); ++i)
				{
					if (SkeletalMesh->GetRefSkeleton().GetParentIndex(i) == ParentIndex)
					{
						ChildLocation = BoneArray[i].TransformPosition(FVector::ZeroVector);
						break;
					}
				}

				while (ParentIndex >= 0)
				{
					Location = BoneArray[ParentIndex].TransformPosition(Location);
					ChildLocation = BoneArray[ParentIndex].TransformPosition(ChildLocation);
					ParentIndex = SkeletalMesh->GetRefSkeleton().GetParentIndex(ParentIndex);
				}

				Direction = (ChildLocation - Location).GetSafeNormal();

				if ((Location != FVector::ZeroVector) && (Direction != FVector::ZeroVector))
				{
					FVector UpTemp = (FVector::DotProduct(Direction, FVector(0.0f, 0.0f, 1.0f)) > 0.99f) ? FVector(0.1f, 0.1f, 1.0f).GetSafeNormal() : FVector(0.0f, 0.0f, 1.0f);
					FVector Right = FVector::CrossProduct(UpTemp, Direction);
					FVector UpFinal = FVector::CrossProduct(Direction, Right);

					if (NodeConstant != nullptr)
					{
						NodeConstant->BoneComboBoxLocation = Location;
						NodeConstant->BoneComboBoxForwardDirection = Direction;
						NodeConstant->BoneComboBoxUpDirection = UpFinal.GetSafeNormal();
					}
					else if (NodeParameter != nullptr)
					{
						NodeParameter->BoneComboBoxLocation = Location;
						NodeParameter->BoneComboBoxForwardDirection = Direction;
						NodeParameter->BoneComboBoxUpDirection = UpFinal.GetSafeNormal();
					}
				}
			}

			InBoneProperty->SetValue(*BoneComboOptions[OptionIndex].Get());

			return;
		}
	}
}


void FCustomizableObjectNodeProjectorParameterDetails::OnReferenceSkeletonComponentChanged()
{
	if (NodeConstant != nullptr)
	{
		NodeConstant->ProjectorBone = FName();
	}
	else if (NodeParameter != nullptr)
	{
		NodeParameter->ProjectorBone = FName();
	}
	
	CaptureContextData();
	CacheSkeletalMeshBoneNames();
}


EVisibility FCustomizableObjectNodeProjectorParameterDetails::ShouldBoneDropdownBeVisible() const
{
	if (BoneComboOptions.Num() > 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}



#undef LOCTEXT_NAMESPACE
