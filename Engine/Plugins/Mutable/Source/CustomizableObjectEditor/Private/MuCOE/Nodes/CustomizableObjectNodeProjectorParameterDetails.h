// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeParameterDetails.h"

class FReply;
namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
struct EVisibility;


class FCustomizableObjectNodeProjectorParameterDetails : public FCustomizableObjectNodeParameterDetails
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;


private:

	/** The combo box that we use to display a list of bone names for the current Skeletal mesh */
	TSharedPtr<class STextComboBox> BoneSelectionComboBox = nullptr;
	
	FName ProjectorBoneName;
	TSharedPtr<FString> BoneToSelect;
	TArray< TSharedPtr<FString> > BoneComboOptions;

	class UCustomizableObjectNodeProjectorConstant* NodeConstant = nullptr;
	class UCustomizableObjectNodeProjectorParameter* NodeParameter = nullptr;

	FReply OnProjectorCopyPressed() const;
	FReply OnProjectorPastePressed() const;

	class USkeletalMesh* SkeletalMesh = nullptr;

	/** Cache the Skeletal Mesh of the component being pointed at and the Projector Parameter name of the node whose descriptor this class edits */
	void CaptureContextData();

	/** Cache the names of the bones present in the cached Skeletal Mesh */
	void CacheSkeletalMeshBoneNames();
	
	void OnBoneComboBoxSelectionChanged(TSharedPtr<FString> InSelection, ESelectInfo::Type InSelectInfo, TSharedRef<IPropertyHandle> InBoneProperty);
	void OnReferenceSkeletonComponentChanged();
	EVisibility ShouldBoneDropdownBeVisible() const;
};
