// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeReroute.h"

#include "SCustomizableObjectNodeReroute.h"
#include "SGraphNodeKnot.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeReroute)

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeReroute::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Wildcard, TEXT("InputPin"));
	CreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Wildcard, TEXT("OutputPin"));
}


void UCustomizableObjectNodeReroute::ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPinsMode)
{
	Super::ReconstructNode(RemapPinsMode);
	PropagatePinType();
}


bool UCustomizableObjectNodeReroute::IsSingleOutputNode() const
{
	return false;
}


FText UCustomizableObjectNodeReroute::GetTooltipText() const
{
	return LOCTEXT("KnotTooltip", "Reroute Node (reroutes wires)");
}


FText UCustomizableObjectNodeReroute::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(NodeComment);
	}
	else if (TitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("KnotListTitle", "Add Reroute...");
	}
	else
	{
		return LOCTEXT("KnotTitle", "Reroute");
	}
}


bool UCustomizableObjectNodeReroute::ShouldOverridePinNames() const
{
	return true;
}


FText UCustomizableObjectNodeReroute::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	return FText::GetEmpty(); // Keep the pin size tiny
}


void UCustomizableObjectNodeReroute::OnRenameNode(const FString& NewName)
{
	NodeComment = NewName;
}


bool UCustomizableObjectNodeReroute::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	OutInputPinIndex = 0;
	OutOutputPinIndex = 1;
	return true;
}


TSharedPtr<SGraphNode> UCustomizableObjectNodeReroute::CreateVisualWidget()
{
	return SNew(SCustomizableObjectNodeReroute, this);
}


void UCustomizableObjectNodeReroute::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();
	
	PropagatePinType();
}


UCustomizableObjectNodeRemapPins* UCustomizableObjectNodeReroute::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByPosition>();
}


void UCustomizableObjectNodeReroute::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixedNoPinsInRerouteNodes)
	{
		// Case where the Pins array is empty when it should not be due to an inconmplete undo operation
		if (Pins.IsEmpty())
		{
			// Extracted from UCustomizableObjectNodeReroute::AllocateDefaultPins
			
			UEdGraphPin* MyInputPin = CreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Wildcard, TEXT("InputPin"));
			MyInputPin->bDefaultValueIsIgnored = true;

			CreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Wildcard, TEXT("OutputPin"));
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeReroute::GetInputPin() const
{
	return Pins[0];
}


UEdGraphPin* UCustomizableObjectNodeReroute::GetOutputPin() const
{
	return Pins[1];
}


void UCustomizableObjectNodeReroute::PropagatePinType()
{
	UEdGraphPin* MyInputPin  = GetInputPin();
	UEdGraphPin* MyOutputPin = GetOutputPin();

	for (const UEdGraphPin* Inputs : MyInputPin->LinkedTo)
	{
		if (Inputs->PinType.PinCategory != UEdGraphSchema_CustomizableObject::PC_Wildcard)
		{
			PropagatePinTypeFromDirection(true);
			return;
		}
	}

	for (const UEdGraphPin* Outputs : MyOutputPin->LinkedTo)
	{
		if (Outputs->PinType.PinCategory != UEdGraphSchema_CustomizableObject::PC_Wildcard)
		{
			PropagatePinTypeFromDirection(false);
			return;
		}
	}

	// if all inputs/outputs are wildcards, still favor the inputs first (propagate array/reference/etc. state)
	if (MyInputPin->LinkedTo.Num() > 0)
	{
		// If we can't mirror from output type, we should at least get the type information from the input connection chain
		PropagatePinTypeFromDirection(true);
	}
	else if (MyOutputPin->LinkedTo.Num() > 0)
	{
		// Try to mirror from output first to make sure we get appropriate member references
		PropagatePinTypeFromDirection(false);
	}
	else
	{
		// Revert to wildcard
		MyInputPin->BreakAllPinLinks();
		MyInputPin->PinType.ResetToDefaults();
		MyInputPin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_Wildcard;

		MyOutputPin->BreakAllPinLinks();
		MyOutputPin->PinType.ResetToDefaults();
		MyOutputPin->PinType.PinCategory = UEdGraphSchema_CustomizableObject::PC_Wildcard;
	}
}


void UCustomizableObjectNodeReroute::PropagatePinTypeFromDirection(bool bFromInput)
{
	if (bRecursionGuard)
	{
		return;
	}
	
	// Set the type of the pin based on the source connection, and then percolate
	// that type information up until we no longer reach another Reroute node
	UEdGraphPin* MySourcePin = bFromInput ? GetInputPin() : GetOutputPin();
	UEdGraphPin* MyDestinationPin = bFromInput ? GetOutputPin() : GetInputPin();

	TGuardValue<bool> RecursionGuard(bRecursionGuard, true);

	// Make sure any source knot pins compute their type, this will try to call back
	// into this function but the recursion guard will stop it
	for (const UEdGraphPin* InPin : MySourcePin->LinkedTo)
	{
		if (InPin)
		{
			if (UCustomizableObjectNodeReroute * KnotNode = Cast<UCustomizableObjectNodeReroute>(InPin->GetOwningNode()))
			{
				KnotNode->PropagatePinTypeFromDirection(bFromInput);
			}
		}
	}

	if (const UEdGraphPin* TypeSource = MySourcePin->LinkedTo.Num() ? MySourcePin->LinkedTo[0] : nullptr)
	{
		MySourcePin->PinType = TypeSource->PinType;
		MySourcePin->PinType.ContainerType = EPinContainerType::None;
		
		MyDestinationPin->PinType = TypeSource->PinType;
		MyDestinationPin->PinType.ContainerType = EPinContainerType::None;

		for (const UEdGraphPin* LinkPin : MyDestinationPin->LinkedTo)
		{
			// Order of reconstruction can be such that nulls haven't been cleared out of the destination node's list yet so
			// must protect against null here
			if (LinkPin)
			{
				UEdGraphNode* OwningNode = LinkPin->GetOwningNode();

				// Notify any pins in the destination direction
				if (UCustomizableObjectNodeReroute* KnotNode = Cast<UCustomizableObjectNodeReroute>(OwningNode))
				{
					KnotNode->PropagatePinTypeFromDirection(bFromInput);
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
