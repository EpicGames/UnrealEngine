// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeReroute.generated.h"


/** Code based on UK2Node_Knot. */
UCLASS()
class UCustomizableObjectNodeReroute : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override;
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	virtual void NodeConnectionListChanged() override;
	virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual void ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPinsMode) override;
	virtual bool IsSingleOutputNode() const override;

	// Own interface
	UEdGraphPin* GetInputPin() const;

	UEdGraphPin* GetOutputPin() const;
	
private:
	/** Code based on UK2Node_Knot::PropagatePinType. */
	void PropagatePinType();

	/** Code based on UK2Node_Knot::PropagatePinTypeFromDirection. */
	void PropagatePinTypeFromDirection(bool bFromInput);
	
	/** Code based on UK2Node_Knot. Recursion guard boolean to prevent PropagatePinType from hanging if there is a cycle of reroute nodes. */
	bool bRecursionGuard = false;
};