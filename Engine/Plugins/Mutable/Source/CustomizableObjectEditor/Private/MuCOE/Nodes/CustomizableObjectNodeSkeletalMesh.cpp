// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"

#include "Engine/SkeletalMesh.h"
#include "AssetThumbnail.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ISinglePropertyView.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeLayoutBlocks.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByNameDefaultPin.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Input/SCheckBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeSkeletalMesh)

class UCustomizableObjectNodeRemapPinsByName;
class UObject;
struct FSlateBrush;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"



void UCustomizableObjectNodeSkeletalMesh::LoadObjects()
{
	if (USkeletalMesh* LoadedMesh = UE::Mutable::Private::LoadObject(SkeletalMesh))
	{
		ConditionalPostLoadReference(*LoadedMesh);

		for (const FSkeletalMaterial& SkeletalMaterial : LoadedMesh->GetMaterials())
		{
			if (UMaterialInterface* Material = SkeletalMaterial.MaterialInterface)
			{
				ConditionalPostLoadReference(*Material);
			}
		}
	}
}


void UCustomizableObjectNodeSkeletalMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("SkeletalMesh"))
	{
		ReconstructNode();
	}
}


void UCustomizableObjectNodeSkeletalMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Pass information to the remap pins action context
	if (UCustomizableObjectNodeRemapPinsByNameDefaultPin* RemapPinsCustom = Cast<UCustomizableObjectNodeRemapPinsByNameDefaultPin>(RemapPins))
	{
		RemapPinsCustom->DefaultPin = DefaultPin.Get();
	}
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	// Force the loading of the skeletal mesh so we can later access the internal WeakObjectPtr (may be null if no SKM is set)
	USkeletalMesh* LoadedMesh = UE::Mutable::Private::LoadObject(SkeletalMesh);
	
	if (!LoadedMesh)
	{
		UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(this);
		PinData->Init(-1, -1, -1);
		
		DefaultPin = CustomCreatePin(EGPD_Output, Schema->PC_Mesh, TEXT("Mesh"), LOCTEXT("Mesh", "Mesh"), PinData);
		return;
	}
	else
	{
		DefaultPin = {};
	}
	
	if (const FSkeletalMeshModel* ImportedModel = LoadedMesh->GetImportedModel())
	{
		const int32 NumLODs = LoadedMesh->GetLODNum();
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			const int32 NumSections = ImportedModel->LODModels[LODIndex].Sections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				// Ignore disabled sections.
				if (ImportedModel->LODModels[LODIndex].Sections[SectionIndex].bDisabled)
				{
					continue;
				}

				UMaterialInterface* MaterialInterface = GetMaterialInterfaceFor(LODIndex, SectionIndex);
				
				FString Section = FString::Printf(TEXT("Section %i"), SectionIndex);
				if (MaterialInterface)
				{
					Section.Append(FString::Printf(TEXT(" : %s"), *MaterialInterface->GetName()));
				}

				// Mesh
				{
					UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(this);
					PinData->Init(LODIndex, SectionIndex, ImportedModel->LODModels[LODIndex].NumTexCoords);
					
					const FString MeshPinName = FString::Printf(TEXT("LOD %i - Section %i - Mesh"), LODIndex, SectionIndex);
					
					FText FriendlyName = FText::FromString(FString::Printf(TEXT("LOD %i - %s"), LODIndex, *Section));
					CustomCreatePin(EGPD_Output, Schema->PC_Mesh, FName(*MeshPinName), FriendlyName, PinData);
				}
				
				
				// Images
				if (MaterialInterface)
				{
					const UMaterial* Material = MaterialInterface->GetMaterial();

					TArray<FMaterialParameterInfo> ImageInfos;
					TArray<FGuid> ImageIds;
					Material->GetAllTextureParameterInfo(ImageInfos, ImageIds);

					check(ImageInfos.Num() == ImageIds.Num())
					for (int32 Index = 0; Index < ImageInfos.Num(); ++Index)
					{
						FGuid& ImageId = ImageIds[Index];
						
						UCustomizableObjectNodeSkeletalMeshPinDataImage* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataImage>(this);
						PinData->Init(LODIndex, SectionIndex, ImageId);
						
						const FMaterialParameterInfo& ImageInfo = ImageInfos[Index];
						const FString ImageNameStr = *ImageInfo.Name.ToString();
						const FString TexturePinName = FString::Printf(TEXT("LOD %i - Section %i - Texture Parameter %s"), LODIndex, SectionIndex, *ImageNameStr);

						FText FriendlyName = FText::FromString(FString::Printf(TEXT("LOD %i - %s - %s"), LODIndex, *Section, *ImageNameStr));
						UEdGraphPin* Pin = CustomCreatePin(EGPD_Output, Schema->PC_Texture, FName(*TexturePinName), FriendlyName, PinData);
						Pin->bHidden = true;
					}
				}
			}
		}
	}
}


UCustomizableObjectNodeRemapPins* UCustomizableObjectNodeSkeletalMesh::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeSkeletalMeshRemapPinsBySection>();
}


bool UCustomizableObjectNodeSkeletalMesh::HasPinViewer() const
{
	return true;
}


FText UCustomizableObjectNodeSkeletalMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (!SkeletalMesh.IsNull())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MeshName"), FText::FromString(SkeletalMesh.GetAssetName()));

		return FText::Format(LOCTEXT("SkeletalMesh_Title", "{MeshName}\nSkeletal Mesh"), Args);
	}
	else
	{
		return LOCTEXT("Skeletal_Mesh", "Skeletal Mesh");
	}
}


FLinearColor UCustomizableObjectNodeSkeletalMesh::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


UTexture2D* UCustomizableObjectNodeSkeletalMesh::FindTextureForPin(const UEdGraphPin* Pin) const
{
	if (!Pin)
	{
		return nullptr;
	}

	if (const UCustomizableObjectNodeSkeletalMeshPinDataImage* PinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataImage>(GetPinData(*Pin)))
	{
		if (const UMaterialInterface* MaterialInterface = GetMaterialInterfaceFor(PinData->GetLODIndex(), PinData->GetSectionIndex()))
		{
			const UMaterialInterface* Material = GetMaterialFor(Pin);
			
			TArray<FGuid> ParameterIds;
			TArray<FMaterialParameterInfo> ParameterInfo;
			Material->GetAllParameterInfoOfType(EMaterialParameterType::Texture, ParameterInfo, ParameterIds);

			check(ParameterIds.Num() == ParameterInfo.Num())
			for (int32 Index = 0; Index < ParameterIds.Num(); ++Index)
			{
				if (ParameterIds[Index] == PinData->GetTextureParameterId())
				{
					UTexture* Texture = nullptr;
					MaterialInterface->GetTextureParameterValue(ParameterInfo[Index].Name, Texture);

					return Cast<UTexture2D>(Texture);
				}
			}
		}
	}
	
	return nullptr;
}


TArray<UCustomizableObjectLayout*> UCustomizableObjectNodeSkeletalMesh::GetLayouts(const UEdGraphPin& MeshPin) const
{
	const UCustomizableObjectNodeSkeletalMeshPinDataMesh* MeshPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(GetPinData(MeshPin));
	check(MeshPinData); // Not a mesh pin

	return MeshPinData->Layouts;
}


TSoftObjectPtr<UStreamableRenderAsset> UCustomizableObjectNodeSkeletalMesh::GetMesh() const
{
	return SkeletalMesh;
}


UEdGraphPin* UCustomizableObjectNodeSkeletalMesh::GetMeshPin(const int32 LODIndex, const int32 SectionIndex) const
{
	for (UEdGraphPin* Pin : GetAllNonOrphanPins())
	{
		if (const UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(GetPinData(*Pin)))
		{
			if (PinData->GetLODIndex() == LODIndex &&
				PinData->GetSectionIndex() == SectionIndex)
			{
				return Pin;
			}
		}
	}

	return nullptr;
}


void UCustomizableObjectNodeSkeletalMesh::GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const
{
	if (const UCustomizableObjectNodeSkeletalMeshPinDataSection* PinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataSection>(GetPinData(Pin)))
	{
		OutLODIndex = PinData->GetLODIndex();
		OutSectionIndex = PinData->GetSectionIndex();
	}
	else
	{
		OutLODIndex = -1;
		OutSectionIndex = -1;
	}
}


UMaterialInterface* UCustomizableObjectNodeSkeletalMesh::GetMaterialFor(const UEdGraphPin* Pin) const
{
	if (FSkeletalMaterial* SkeletalMaterial = GetSkeletalMaterialFor(*Pin))
	{
		return SkeletalMaterial->MaterialInterface;
	}

	return nullptr;
}


FSkeletalMaterial* UCustomizableObjectNodeSkeletalMesh::GetSkeletalMaterialFor(const UEdGraphPin& Pin) const
{
	int32 LODIndex;
	int32 SectionIndex;
	GetPinSection(Pin, LODIndex, SectionIndex);

	return GetSkeletalMaterialFor(LODIndex, SectionIndex);
}


int32 UCustomizableObjectNodeSkeletalMesh::GetSkeletalMaterialIndexFor(const UEdGraphPin& Pin) const
{
	int32 LODIndex;
	int32 SectionIndex;
	GetPinSection(Pin, LODIndex, SectionIndex);

	return GetSkeletalMaterialIndexFor(LODIndex, SectionIndex);
}

const FSkelMeshSection* UCustomizableObjectNodeSkeletalMesh::GetSkeletalMeshSectionFor(const UEdGraphPin& Pin) const
{
	int32 LODIndex;
	int32 SectionIndex;
	GetPinSection(Pin, LODIndex, SectionIndex);
	
	return GetSkeletalMeshSectionFor(LODIndex, SectionIndex);
}

bool UCustomizableObjectNodeSkeletalMesh::IsPinRelevant(const UEdGraphPin* Pin) const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	
	if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
	{	
		return Pin->PinType.PinCategory == Schema->PC_Mesh;
	}

	return false;
}


bool UCustomizableObjectNodeSkeletalMesh::IsNodeOutDatedAndNeedsRefresh()
{
	const bool bOutdated = [&]()
	{
		if (!SkeletalMesh) // TODO UE-312665: SkeletalMesh->IsNull();
		{
			return false;
		}

		const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		if (!ImportedModel)
		{
			return false;
		}
	
		for (const UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			auto Connected = [](const UEdGraphPin& Pin)
			{
				return Pin.Direction == EGPD_Input ? FollowInputPin(Pin) != nullptr : !FollowOutputPinArray(Pin).IsEmpty();	
			};
			
			auto OutdatedSectionPinData = [&](const UCustomizableObjectNodeSkeletalMeshPinDataSection& PinData) -> bool
			{
				return !ImportedModel->LODModels.IsValidIndex(PinData.GetLODIndex()) ||
					!ImportedModel->LODModels[PinData.GetLODIndex()].Sections.IsValidIndex(PinData.GetSectionIndex()) ||
					ImportedModel->LODModels[PinData.GetLODIndex()].Sections[PinData.GetSectionIndex()].bDisabled;
			};
		
			if (const UCustomizableObjectNodeSkeletalMeshPinDataLayout* LayoutPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataLayout>(GetPinData(*Pin)))
			{
				if (Connected(*Pin) &&
					(OutdatedSectionPinData(*LayoutPinData) ||
					LayoutPinData->GetUVIndex() < 0 || LayoutPinData->GetUVIndex() >= static_cast<int32>(ImportedModel->LODModels[LayoutPinData->GetLODIndex()].NumTexCoords)))
				{
					return true;
				}
			}
			else if (const UCustomizableObjectNodeSkeletalMeshPinDataMesh* MeshPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(GetPinData(*Pin)))
			{
				if (Connected(*Pin) &&
					OutdatedSectionPinData(*MeshPinData))
				{
					return true;
				}
			}
			else if (const UCustomizableObjectNodeSkeletalMeshPinDataImage* ImagePinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataImage>(GetPinData(*Pin)))
			{				
				const UMaterialInterface* MaterialInterface = GetMaterialInterfaceFor(ImagePinData->GetLODIndex(), ImagePinData->GetSectionIndex());
				if (!MaterialInterface) // If we had an Image pin for sure we had a MaterialInstance.
				{
					return true;
				}

				TArray<FGuid> ParameterIds;
				TArray<FMaterialParameterInfo> ParameterInfo;
				MaterialInterface->GetAllParameterInfoOfType(EMaterialParameterType::Texture, ParameterInfo, ParameterIds);
				
				UTexture* Texture = nullptr;
				if (Connected(*Pin) &&
					(OutdatedSectionPinData(*ImagePinData) ||
					!ParameterIds.Contains(ImagePinData->GetTextureParameterId()))) // Check that the Texture Parameter still exists.
				{
					return true; 
				}
			}
		}

		return false;
	}();
	
	// Remove previous compilation warnings
	if (!bOutdated && bHasCompilerMessage)
	{
		RemoveWarnings();
		GetGraph()->NotifyGraphChanged();
	}

    return bOutdated;
}


FString UCustomizableObjectNodeSkeletalMesh::GetRefreshMessage() const
{
    return "Node data outdated. Please refresh node.";
}


FText UCustomizableObjectNodeSkeletalMesh::GetTooltipText() const
{
	return LOCTEXT("Skeletal_Mesh_Tooltip", "Get access to the sections (also known as material slots) of a skeletal mesh and to each of the sections texture parameters.");
}


void UCustomizableObjectNodeSkeletalMesh::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PostLoadToCustomVersion)
	{
		LoadObjects();
		
		for (FCustomizableObjectNodeSkeletalMeshLOD& LOD : LODs_DEPRECATED)
		{
			for(FCustomizableObjectNodeSkeletalMeshMaterial& Material : LOD.Materials)
			{
				if (Material.MeshPin_DEPRECATED && !Material.MeshPinRef.Get())
				{
					UEdGraphPin* AuxPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(Material.MeshPin_DEPRECATED);
					Material.MeshPinRef.SetPin(AuxPin);
				}

				if (!Material.LayoutPinsRef.Num())
				{
					if (Material.LayoutPins_DEPRECATED.Num())
					{
						for (UEdGraphPin_Deprecated* LayoutPin : Material.LayoutPins_DEPRECATED)
						{
							UEdGraphPin* AuxPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(LayoutPin);
							FEdGraphPinReference AuxEdGraphPinReference(AuxPin);
							Material.LayoutPinsRef.Add(AuxEdGraphPinReference);
						}
					}
					else
					{
						FString MaterialLayoutName = Material.Name + " Layout";
						for (UEdGraphPin* Pin : GetAllNonOrphanPins())
						{
							if (Pin
								&& Pin->Direction == EEdGraphPinDirection::EGPD_Input
								&& (MaterialLayoutName == Helper_GetPinName(Pin)
									|| MaterialLayoutName == Pin->PinFriendlyName.ToString()))
							{
								FEdGraphPinReference AuxEdGraphPinReference(Pin);
								Material.LayoutPinsRef.Add(AuxEdGraphPinReference);
								break;
							}
						}
					}
				}

				if (!Material.ImagePinsRef.Num())
				{
					for (UEdGraphPin_Deprecated* ImagePin : Material.ImagePins_DEPRECATED)
					{
						UEdGraphPin* AuxPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(ImagePin);
						FEdGraphPinReference AuxEdGraphPinReference(AuxPin);
						Material.ImagePinsRef.Add(AuxEdGraphPinReference);
					}
				}
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::ConvertAnimationSlotToFName)
	{
		if (AnimBlueprintSlotName.IsNone() && AnimBlueprintSlot_DEPRECATED != -1)
		{
			AnimBlueprintSlotName = FName(FString::FromInt(AnimBlueprintSlot_DEPRECATED));
			AnimBlueprintSlot_DEPRECATED = -1; // Unnecessary, just in case anyone tried to use it later in this method.
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeSkeletalMesh)
	{
		LoadObjects();
		
		for (int32 LODIndex = 0; LODIndex < LODs_DEPRECATED.Num(); ++LODIndex)
		{
			const FCustomizableObjectNodeSkeletalMeshLOD& LOD = LODs_DEPRECATED[LODIndex];
			
			for (int32 SectionIndex = 0; SectionIndex < LOD.Materials.Num(); ++SectionIndex)
			{
				const FCustomizableObjectNodeSkeletalMeshMaterial& Section = LOD.Materials[SectionIndex];

				{
					UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(this);
					PinData->Init(LODIndex, SectionIndex, -1);
				
					AddPinData(*Section.MeshPinRef.Get(), *PinData);
				}

				if (SkeletalMesh)
				{
					if (const FSkeletalMaterial* SkeletalMaterial = GetSkeletalMaterialFor(LODIndex, SectionIndex))
					{
						if (SkeletalMaterial && SkeletalMaterial->MaterialInterface)
						{
							TArray<FGuid> ParameterIds;
							TArray<FMaterialParameterInfo> ParameterInfo;
							SkeletalMaterial->MaterialInterface->GetAllParameterInfoOfType(EMaterialParameterType::Texture, ParameterInfo, ParameterIds);
							check(ParameterIds.Num() == ParameterInfo.Num());

							for (int32 ImageIndex = 0; ImageIndex < Section.ImagePinsRef.Num(); ++ImageIndex)
							{
								const UEdGraphPin* ImagePin = Section.ImagePinsRef[ImageIndex].Get();

								FGuid TextureParameterId;

								for (int32 Index = 0; Index < ParameterIds.Num(); ++Index)
								{
									if (ParameterInfo[Index].Name == ImagePin->PinFriendlyName.ToString())
									{
										TextureParameterId = ParameterIds[Index];
										break;
									}
								}

								UCustomizableObjectNodeSkeletalMeshPinDataImage* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataImage>(this);
								PinData->Init(LODIndex, SectionIndex, TextureParameterId);

								AddPinData(*ImagePin, *PinData);
							}
						}
					}
				}

				for (int32 LayoutIndex = 0; LayoutIndex < Section.LayoutPinsRef.Num(); ++LayoutIndex)
				{
					UCustomizableObjectNodeSkeletalMeshPinDataLayout* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataLayout>(this);
					PinData->Init(LODIndex, SectionIndex, LayoutIndex);
				
					AddPinData(*Section.LayoutPinsRef[LayoutIndex].Get(), *PinData);
				}
			}
		}

		ReconstructNode();
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeSkeletalMeshPinDataOuter)
	{
		LoadObjects();
		
		ReconstructNode(); // Pins did not have Pin Data. Reconstruct them.
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AutomaticNodeSkeletalMeshPinDataUProperty)
	{
		LoadObjects();
		
		ReconstructNode(CreateRemapPinsByName()); // Correct pins but incorrect Pin Data. Reconstruct and remap pins only by name, no Pin Data.
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::IgnoreDisabledSections)
	{
		LoadObjects();
		
		ReconstructNode(); // Pins representing disabled sections could be present. 
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SkeletalMeshNodeDefaultPinWithoutPinData)
	{
		if (const UEdGraphPin* Pin = DefaultPin.Get())
		{
			UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(this);
			PinData->Init(-1, -1, -1);

			AddPinData(*Pin, *PinData);
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MoveLayoutToNodeSkeletalMesh)
	{
		LoadObjects();
		
		if (const FSkeletalMeshModel* ImportedModel = SkeletalMesh ? SkeletalMesh->GetImportedModel() : nullptr)
		{
			TArray<UEdGraphPin*> PinsToDelete;

			TArray<UEdGraphPin*> NonOrphanPins = GetAllNonOrphanPins();
			for (UEdGraphPin* Pin : NonOrphanPins)
			{
				UCustomizableObjectNodeSkeletalMeshPinDataMesh* MeshPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(GetPinData(*Pin));
				if (!MeshPinData)
				{
					continue;
				}

				const int32 LODIndex = MeshPinData->GetLODIndex();
				const int32 SectionIndex = MeshPinData->GetSectionIndex();
				if (!ImportedModel->LODModels.IsValidIndex(LODIndex))
				{
					continue;
				}

				const int32 NumTexCoords = ImportedModel->LODModels[LODIndex].NumTexCoords;
				MeshPinData->Layouts.SetNum(NumTexCoords);

				for (int32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
				{
					MeshPinData->Layouts[UVIndex] = NewObject<UCustomizableObjectLayout>(this);
					MeshPinData->Layouts[UVIndex]->SetLayout(LODIndex, SectionIndex, UVIndex);
					MeshPinData->Layouts[UVIndex]->SetIgnoreWarningsLOD(0);
				}

				for (UEdGraphPin* OtherPin : NonOrphanPins)
				{
					UCustomizableObjectNodeSkeletalMeshPinDataLayout* LayoutPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataLayout>(GetPinData(*OtherPin));
					if (!LayoutPinData ||
						LayoutPinData->GetLODIndex() != LODIndex ||
						LayoutPinData->GetSectionIndex() != SectionIndex ||
						!MeshPinData->Layouts.IsValidIndex(LayoutPinData->GetUVIndex()))
					{
						continue;
					}

					if (const UEdGraphPin* ConnectedPin = FollowInputPin(*OtherPin))
					{
						const UCustomizableObjectNodeLayoutBlocks* LayoutNode = Cast<UCustomizableObjectNodeLayoutBlocks>(ConnectedPin->GetOwningNode());
						if (LayoutNode && LayoutNode->Layout)
						{
							UCustomizableObjectLayout* Layout = MeshPinData->Layouts[LayoutPinData->GetUVIndex()];
							Layout->Blocks = LayoutNode->Layout->Blocks;
							Layout->SetGridSize(LayoutNode->Layout->GetGridSize());
							Layout->SetMaxGridSize(LayoutNode->Layout->GetMaxGridSize());
							Layout->SetIgnoreVertexLayoutWarnings(LayoutNode->Layout->GetIgnoreVertexLayoutWarnings());
							Layout->SetIgnoreWarningsLOD(LayoutNode->Layout->GetFirstLODToIgnoreWarnings());
							Layout->PackingStrategy = LayoutNode->Layout->PackingStrategy;
							Layout->AutomaticBlocksStrategy = LayoutNode->Layout->AutomaticBlocksStrategy;
							Layout->AutomaticBlocksMergeStrategy = LayoutNode->Layout->AutomaticBlocksMergeStrategy;
							Layout->BlockReductionMethod = LayoutNode->Layout->BlockReductionMethod;
						}
					}

					PinsToDelete.Add(OtherPin);
				}
			}

			for (UEdGraphPin* Pin : PinsToDelete)
			{
				CustomRemovePin(*Pin);
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixAutomaticBlocksStrategyLegacyNodes)
	{
		for (UEdGraphPin* Pin : GetAllPins())
		{
			if (UCustomizableObjectNodeSkeletalMeshPinDataMesh* MeshPinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(GetPinData(*Pin)))
			{
				for (UCustomizableObjectLayout* Layout : MeshPinData->Layouts)
				{
					if (Layout)
					{
						Layout->AutomaticBlocksStrategy = ECustomizableObjectLayoutAutomaticBlocksStrategy::Ignore;
					}
				}
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PassthroughParameter)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinName == TEXT("Skeletal Mesh"))
			{
				Pin->PinName = TEXT("Mesh");
			}
		}
	}
}


TSharedPtr<SGraphNode> UCustomizableObjectNodeSkeletalMesh::CreateVisualWidget()
{
	TSharedPtr<SGraphNodeSkeletalMesh> GraphNode;
	SAssignNew(GraphNode, SGraphNodeSkeletalMesh,this);

	GraphNodeSkeletalMesh = GraphNode;

	return GraphNode;
}


UMaterialInterface* UCustomizableObjectNodeSkeletalMesh::GetMaterialInterfaceFor(const int32 LODIndex, const int32 SectionIndex) const
{
	if (FSkeletalMaterial* SkeletalMaterial = GetSkeletalMaterialFor(LODIndex, SectionIndex))
	{
		return SkeletalMaterial->MaterialInterface;
	}

	return nullptr;
}


FSkeletalMaterial* UCustomizableObjectNodeSkeletalMesh::GetSkeletalMaterialFor(const int32 LODIndex, const int32 SectionIndex) const
{
	if (SkeletalMesh.IsNull())
	{
		return nullptr;
	}

	// TODO UE-312665: Cache data and remove sync load
	USkeletalMesh* LoadedMesh = UE::Mutable::Private::LoadObject(SkeletalMesh);
	if (!LoadedMesh)
	{
		return nullptr;
	}

	const int32 SkeletalMeshMaterialIndex = GetSkeletalMaterialIndexFor(LODIndex, SectionIndex);
	if (LoadedMesh->GetMaterials().IsValidIndex(SkeletalMeshMaterialIndex))
	{
		return &LoadedMesh->GetMaterials()[SkeletalMeshMaterialIndex];
	}
	
	return nullptr;
}

const FSkelMeshSection* UCustomizableObjectNodeSkeletalMesh::GetSkeletalMeshSectionFor(const int32 LODIndex, const int32 SectionIndex) const
{	
	if (SkeletalMesh.IsNull())
	{
		return nullptr;
	}
	
	// TODO UE-312665: Cache data and remove sync load
	USkeletalMesh* LoadedMesh = UE::Mutable::Private::LoadObject(SkeletalMesh);
	if (!LoadedMesh)
	{
		return nullptr;
	}

	const FSkeletalMeshModel* ImportedModel = LoadedMesh->GetImportedModel();
	if (!ImportedModel)
	{
		return nullptr;
	}

	if (!ImportedModel->LODModels.IsValidIndex(LODIndex))
	{
		return nullptr;
	}

	const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		return nullptr;
	}

	return &LODModel.Sections[SectionIndex];
}

int32 UCustomizableObjectNodeSkeletalMesh::GetSkeletalMaterialIndexFor(const int32 LODIndex, const int32 SectionIndex) const
{
	if (SkeletalMesh.IsNull())
	{
		return INDEX_NONE;
	}

	// TODO UE-312665: Cache data and remove sync load
	USkeletalMesh* LoadedMesh = UE::Mutable::Private::LoadObject(SkeletalMesh);
	if (!LoadedMesh)
	{
		return INDEX_NONE;
	}

	// We assume that LODIndex and MaterialIndex are valid for the imported model
	int32 SkeletalMeshMaterialIndex = INDEX_NONE;

	// Check if we have lod info map to get the correct material index
	if (const FSkeletalMeshLODInfo* LodInfo = LoadedMesh->GetLODInfo(LODIndex))
	{
		if (LodInfo->LODMaterialMap.IsValidIndex(SectionIndex))
		{
			SkeletalMeshMaterialIndex = LodInfo->LODMaterialMap[SectionIndex];
		}
	}

	// Only deduce index when the explicit mapping is not found or there is no remap
	if (SkeletalMeshMaterialIndex == INDEX_NONE)
	{
		FSkeletalMeshModel* ImportedModel = LoadedMesh->GetImportedModel();
		if (ImportedModel && ImportedModel->LODModels.IsValidIndex(LODIndex) && ImportedModel->LODModels[LODIndex].Sections.IsValidIndex(SectionIndex))
		{
			SkeletalMeshMaterialIndex = ImportedModel->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex;
		}
	}

	return SkeletalMeshMaterialIndex;
}


//SGraphNode --------------------------------------------

void SGraphNodeSkeletalMesh::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	NodeSkeletalMesh = Cast< UCustomizableObjectNodeSkeletalMesh >(InGraphNode);

	WidgetSize = 128.0f;
	ThumbnailSize = 128;

	TSharedPtr<FCustomizableObjectEditor> Editor = StaticCastSharedPtr< FCustomizableObjectEditor >(NodeSkeletalMesh->GetGraphEditor());

	// Thumbnail
	AssetThumbnailPool = MakeShareable(new FAssetThumbnailPool(32));

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);

	FAssetData AssetData;
	AssetRegistryModule.TryGetAssetByObjectPath(NodeSkeletalMesh->SkeletalMesh.ToSoftObjectPath(), AssetData);
	AssetThumbnail = MakeShareable(new FAssetThumbnail(AssetData, ThumbnailSize, ThumbnailSize, AssetThumbnailPool));

	// Selector
	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FSinglePropertyParams SingleDetails;
	SingleDetails.NamePlacement = EPropertyNamePlacement::Hidden;
	SingleDetails.NotifyHook = Editor.Get();
	SingleDetails.bHideAssetThumbnail = true;

	SkeletalMeshSelector = PropPlugin.CreateSingleProperty(NodeSkeletalMesh, "SkeletalMesh", SingleDetails);

	SCustomizableObjectNode::Construct({}, InGraphNode);
}


void SGraphNodeSkeletalMesh::UpdateGraphNode()
{
	SGraphNode::UpdateGraphNode();
}


void SGraphNodeSkeletalMesh::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	DefaultTitleAreaWidget->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(5))
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SGraphNodeSkeletalMesh::OnExpressionPreviewChanged)
			.IsChecked(IsExpressionPreviewChecked())
			.Cursor(EMouseCursor::Default)
			.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(GetExpressionPreviewArrow())
				]
			]
	];
}


void SGraphNodeSkeletalMesh::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	LeftNodeBox->AddSlot()
		.AutoHeight()
		.MaxHeight(WidgetSize)
		.Padding(10.0f,10.0f,0.0f,0.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(ExpressionPreviewVisibility())

			+SHorizontalBox::Slot()
			.MaxWidth(WidgetSize)
			.Padding(5.0f,5.0f,5.0f,5.0f)
			[
				AssetThumbnail->MakeThumbnailWidget()
			]
		];

	if (SkeletalMeshSelector.IsValid())
	{
		LeftNodeBox->AddSlot()
		.AutoHeight()
		.Padding(10.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(ExpressionPreviewVisibility())

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f,0.0f, 5.0f, 5.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SkeletalMeshSelector.ToSharedRef()
			]
		];
	}
}


void SGraphNodeSkeletalMesh::OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState)
{
	NodeSkeletalMesh->bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	UpdateGraphNode();
}


ECheckBoxState SGraphNodeSkeletalMesh::IsExpressionPreviewChecked() const
{
	return NodeSkeletalMesh->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


const FSlateBrush* SGraphNodeSkeletalMesh::GetExpressionPreviewArrow() const
{
	return FCustomizableObjectEditorStyle::Get().GetBrush(NodeSkeletalMesh->bCollapsed ? TEXT("Nodes.ArrowDown") : TEXT("Nodes.ArrowUp"));
}


EVisibility SGraphNodeSkeletalMesh::ExpressionPreviewVisibility() const
{
	return NodeSkeletalMesh->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible;
}


bool UCustomizableObjectNodeSkeletalMeshRemapPinsBySection::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const
{
	const UCustomizableObjectNodeSkeletalMeshPinDataSection* PinDataOldPin = Cast<UCustomizableObjectNodeSkeletalMeshPinDataSection>(Node.GetPinData(OldPin));
	const UCustomizableObjectNodeSkeletalMeshPinDataSection* PinDataNewPin = Cast<UCustomizableObjectNodeSkeletalMeshPinDataSection>(Node.GetPinData(NewPin));
	if (PinDataOldPin && PinDataNewPin)
	{
		return *PinDataOldPin == *PinDataNewPin;
	}
	else
	{
		return Super::Equal(Node, OldPin, NewPin);	
	}
}


void UCustomizableObjectNodeSkeletalMeshPinDataSection::Init(int32 InLODIndex, int32 InSectionIndex)
{
	LODIndex = InLODIndex;
	SectionIndex = InSectionIndex;
}


int32 UCustomizableObjectNodeSkeletalMeshPinDataSection::GetLODIndex() const
{
	return LODIndex;
}


int32 UCustomizableObjectNodeSkeletalMeshPinDataSection::GetSectionIndex() const
{
	return SectionIndex;
}


bool UCustomizableObjectNodeSkeletalMeshPinDataSection::Equals(const UCustomizableObjectNodePinData& Other) const
{
	if (GetClass() != Other.GetClass())
	{
		return false;	
	}

	const UCustomizableObjectNodeSkeletalMeshPinDataSection& OtherTyped = static_cast<const UCustomizableObjectNodeSkeletalMeshPinDataSection&>(Other);
    if (LODIndex != OtherTyped.LODIndex ||
    	SectionIndex != OtherTyped.SectionIndex)
    {
        return false;	            
    }
	
    return Super::Equals(Other);	
}


void UCustomizableObjectNodeSkeletalMeshPinDataMesh::Copy(const UCustomizableObjectNodePinData& Other)
{
	if (const UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinDataOldPin = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(&Other))
	{
		for (UCustomizableObjectLayout* OldLayout : PinDataOldPin->Layouts)
		{
			if (!OldLayout)
			{
				continue;
			}

			const int32 UVChannel = OldLayout->GetUVChannel();
			if (Layouts.IsValidIndex(UVChannel))
			{
				Layouts[UVChannel] = OldLayout;
			}
		}
	}
}

void UCustomizableObjectNodeSkeletalMeshPinDataMesh::Init(int32 InLODIndex, int32 InSectionIndex, int32 NumTexCoords)
{
	Super::Init(InLODIndex, InSectionIndex);

	if (NumTexCoords > 0)
	{
		UObject* Outer = GetOuter();

		Layouts.SetNum(NumTexCoords);

		for (int32 Index = 0; Index < NumTexCoords; ++Index)
		{
			Layouts[Index] = NewObject<UCustomizableObjectLayout>(Outer);
			Layouts[Index]->SetLayout(InLODIndex, InSectionIndex, Index);
		}
	}
}


void UCustomizableObjectNodeSkeletalMeshPinDataImage::Init(int32 InLODIndex, int32 InSectionIndex, FGuid InTextureParameterId)
{
	Super::Init(InLODIndex, InSectionIndex);

	TextureParameterId = InTextureParameterId;
}


FGuid UCustomizableObjectNodeSkeletalMeshPinDataImage::GetTextureParameterId() const
{
	return TextureParameterId;
}


bool UCustomizableObjectNodeSkeletalMeshPinDataImage::Equals(const UCustomizableObjectNodePinData& Other) const
{
	if (GetClass() != Other.GetClass())
	{
		return false;	
	}

	const UCustomizableObjectNodeSkeletalMeshPinDataImage& OtherTyped = static_cast<const UCustomizableObjectNodeSkeletalMeshPinDataImage&>(Other);
    if (TextureParameterId != OtherTyped.TextureParameterId)
    {
        return false;
    }
	
    return Super::Equals(Other);	
}


void UCustomizableObjectNodeSkeletalMeshPinDataLayout::Init(int32 InLODIndex, int32 InSectionIndex, int32 InUVIndex)
{
	Super::Init(InLODIndex, InSectionIndex);

	UVIndex = InUVIndex;
}


int32 UCustomizableObjectNodeSkeletalMeshPinDataLayout::GetUVIndex() const
{
	return UVIndex;
}


bool UCustomizableObjectNodeSkeletalMeshPinDataLayout::Equals(const UCustomizableObjectNodePinData& Other) const
{
	if (GetClass() != Other.GetClass())
	{
		return false;	
	}

	const UCustomizableObjectNodeSkeletalMeshPinDataLayout& OtherTyped = static_cast<const UCustomizableObjectNodeSkeletalMeshPinDataLayout&>(Other);
    if (UVIndex != OtherTyped.UVIndex)
    {
        return false;	            
    }
	
    return Super::Equals(Other);
}


#undef LOCTEXT_NAMESPACE
