// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/CustomizableObjectEditor_Deprecated.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByNameDefaultPin.h"

#include "CustomizableObjectNodeSkeletalMesh.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class FAssetThumbnail;
class FAssetThumbnailPool;
class ISinglePropertyView;
class SOverlay;
class SVerticalBox;
class UAnimInstance;
class UCustomizableObjectLayout;
class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeRemapPinsByName;
class UMaterialInterface;
class UObject;
class USkeletalMesh;
class UTexture2D;
struct FPropertyChangedEvent;
struct FSkeletalMaterial;
struct FSkelMeshSection;
struct FSlateBrush;


// Class to render the Skeletal Mesh thumbnail of a CustomizableObjectNodeSkeletalMesh
class SGraphNodeSkeletalMesh : public SCustomizableObjectNode
{
public:

	SLATE_BEGIN_ARGS(SGraphNodeSkeletalMesh) {}
	SLATE_END_ARGS();

	SGraphNodeSkeletalMesh() : SCustomizableObjectNode() {};

	// Builds the SGraphNodeSkeletalMesh when needed
	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// Calls the needed functions to build the SGraphNode widgets
	void UpdateGraphNode();

	// Overriden functions to build the SGraphNode widgets
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	// Callbacks for the widget
	void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsExpressionPreviewChecked() const;
	const FSlateBrush* GetExpressionPreviewArrow() const;
	EVisibility ExpressionPreviewVisibility() const;

	// Single property that only draws the combo box widget of the skeletal mesh
	TSharedPtr<ISinglePropertyView> SkeletalMeshSelector;

	// Pointer to the NodeSkeletalMesh that owns this SGraphNode
	class UCustomizableObjectNodeSkeletalMesh* NodeSkeletalMesh;

private:
	
	// Classes needed to get and render the thumbnail of the Skeletal Mesh
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	//This parameter defines the size of the thumbnail widget inside the Node
	float WidgetSize;

	// This parameter defines the resolution of the thumbnail
	uint32 ThumbnailSize;

};


/** Remap pins by pin PinData. */
UCLASS()
class UCustomizableObjectNodeSkeletalMeshRemapPinsBySection : public UCustomizableObjectNodeRemapPinsByNameDefaultPin
{
	GENERATED_BODY()
public:
	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const override;
};


/** PinData of a pin that belongs to a Skeletal Mesh Section. */
UCLASS()
class UCustomizableObjectNodeSkeletalMeshPinDataSection : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:
	void Init(int32 InLODIndex, int32 InSectionIndex);

	int32 GetLODIndex() const;

	int32 GetSectionIndex() const;

protected:
	virtual bool Equals(const UCustomizableObjectNodePinData& Other) const override;

private:
	UPROPERTY()
	int32 LODIndex = -1;

	UPROPERTY()
	int32 SectionIndex = -1;
};


/** PinData of a Mesh pin. */
UCLASS()
class UCustomizableObjectNodeSkeletalMeshPinDataMesh : public UCustomizableObjectNodeSkeletalMeshPinDataSection
{
	GENERATED_BODY()

public:
	
	// NodePinDataParameter interface
	/** Virtual function used to copy pin data when remapping pins. */
	virtual void Copy(const UCustomizableObjectNodePinData& Other) override;

	void Init(int32 InLODIndex, int32 InSectionIndex, int32 NumTexCoords);

	/** Layouts related to this Mesh pin */
	UPROPERTY()
	TArray<TObjectPtr<UCustomizableObjectLayout>> Layouts;
};


/** PinData of a Image pin. */
UCLASS()
class UCustomizableObjectNodeSkeletalMeshPinDataImage : public UCustomizableObjectNodeSkeletalMeshPinDataSection
{
	GENERATED_BODY()

public:
	void Init(int32 InLODIndex, int32 InSectionIndex, FGuid InTextureParameterId);

	FGuid GetTextureParameterId() const;

protected:
	virtual bool Equals(const UCustomizableObjectNodePinData& Other) const override;

private:
	UPROPERTY()
	FGuid TextureParameterId;
};


/** PinData of a Layout pin. */
UCLASS()
class UCustomizableObjectNodeSkeletalMeshPinDataLayout : public UCustomizableObjectNodeSkeletalMeshPinDataSection
{
	GENERATED_BODY()

public:
	void Init(int32 InLODIndex, int32 InSectionIndex, int32 InUVIndex);

	int32 GetUVIndex() const;

protected:
	virtual bool Equals(const UCustomizableObjectNodePinData& Other) const override;

private:
	UPROPERTY()
	int32 UVIndex = -1;
};



UCLASS(MinimalAPI)
class UCustomizableObjectNodeSkeletalMesh : public UCustomizableObjectNode, public ICustomizableObjectNodeMeshInterface
{
public:
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TSoftObjectPtr<USkeletalMesh> SkeletalMesh;

	/** Default pin when there is no mesh. */
	UPROPERTY()
	FEdGraphPinReference DefaultPin;
	
	/** Morphs */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TArray<FString> UsedRealTimeMorphTargetNames;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
    bool bUseAllRealTimeMorphs = false; 
	
	/** The anim instance that will be gathered by a Generated instance if it contains this skeletal mesh part, 
		it will be grouped by component and AnimBlueprintSlot (the next UProperty). */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TSoftClassPtr<UAnimInstance> AnimInstance;

	UPROPERTY()
	int32 AnimBlueprintSlot_DEPRECATED;
	
	/** The anim slot associated with the AnimInstance */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FName AnimBlueprintSlotName;

	/** Animation tags that will be gathered by a Generated instance if it contains this skeletal mesh part,
		it will not be grouped by component or AnimBlueprintSlot */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FGameplayTagContainer AnimationGameplayTags;

	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const override;
	UE_API virtual bool HasPinViewer() const override;
	
	// UCustomizableObjectNodeMesh interface
	UE_API virtual UTexture2D* FindTextureForPin(const UEdGraphPin* Pin) const override;
	UE_API virtual TArray<UCustomizableObjectLayout*> GetLayouts(const UEdGraphPin& MeshPin) const override;
	UE_API virtual TSoftObjectPtr<UStreamableRenderAsset> GetMesh() const override;
	UE_API virtual UEdGraphPin* GetMeshPin(int32 LOD, int32 SectionIndex) const override;
	UE_API virtual void GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const override;

	// Own interface
	
	/** Returns the material associated to the given output pin. */
	UE_API UMaterialInterface* GetMaterialFor(const UEdGraphPin* Pin) const;
	UE_API FSkeletalMaterial* GetSkeletalMaterialFor(const UEdGraphPin& Pin) const;
	UE_API int32 GetSkeletalMaterialIndexFor(const UEdGraphPin& Pin) const;

	UE_API const FSkelMeshSection* GetSkeletalMeshSectionFor(const UEdGraphPin& Pin) const;


	virtual bool ProvidesCustomPinRelevancyTest() const override { return true; }
	UE_API virtual bool IsPinRelevant(const UEdGraphPin* Pin) const override;

	UE_API virtual bool IsNodeOutDatedAndNeedsRefresh() override;
	UE_API virtual FString GetRefreshMessage() const override;

	// Creates the SGraph Node widget for the thumbnail
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	// Determines if the Node is collapsed or not
	bool bCollapsed = true;

	// Pointer to the SGraphNode Skeletal Mesh
	TWeakPtr< SGraphNodeSkeletalMesh > GraphNodeSkeletalMesh;

private:
	UE_API UMaterialInterface* GetMaterialInterfaceFor(const int32 LODIndex, const int32 MaterialIndex) const;
	UE_API FSkeletalMaterial* GetSkeletalMaterialFor(const int32 LODIndex, const int32 SectionIndex) const;
	UE_API int32 GetSkeletalMaterialIndexFor(const int32 LODIndex, const int32 SectionIndex) const;
	
	UE_API const FSkelMeshSection* GetSkeletalMeshSectionFor(const int32 LODIndex, const int32 SectionIndex) const;

	UE_API void LoadObjects();
	
	// Deprecated
	
	UPROPERTY()
	TArray<FCustomizableObjectNodeSkeletalMeshLOD> LODs_DEPRECATED;
};

#undef UE_API
