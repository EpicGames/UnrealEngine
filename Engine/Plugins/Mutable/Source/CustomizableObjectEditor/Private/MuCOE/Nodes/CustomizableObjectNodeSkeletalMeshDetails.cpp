// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "MuCOE/PinViewer/SPinViewer.h"
#include "MuCOE/SCustomizableObjectNodeSkeletalMeshRTMorphSelector.h"
#include "MuCOE/SCustomizableObjectLayoutEditor.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectNodeMaterialDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeSkeletalMeshDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeSkeletalMeshDetails);
}


void FCustomizableObjectNodeSkeletalMeshDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();

    if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeSkeletalMesh>(DetailsView->GetSelectedObjects()[0].Get());
	}

    if (!Node)
    {
        return;
    }

    DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeSkeletalMesh, UsedRealTimeMorphTargetNames));
    DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeSkeletalMesh, bUseAllRealTimeMorphs));

    // Needed to draw the CO information before the Material Layer information
    IDetailCategoryBuilder& CustomizableObject = DetailBuilder.EditCategory("CustomizableObject");

    // Cretaing new categories to show the material layers
    IDetailCategoryBuilder& MorphsCategory = DetailBuilder.EditCategory("RealTimeMorphTargets");

	TSharedPtr<SCustomizableObjectNodeSkeletalMeshRTMorphSelector> MorphSelector;
    MorphsCategory.AddCustomRow(LOCTEXT("MaterialLayerCategory", "RealTimeMorphTargets"))
    [
        SAssignNew(MorphSelector, SCustomizableObjectNodeSkeletalMeshRTMorphSelector).Node(Node)
    ];

	TSharedRef<IPropertyHandle> SkeletalMeshProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeSkeletalMesh, SkeletalMesh));
	SkeletalMeshProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(MorphSelector.Get(), &SCustomizableObjectNodeSkeletalMeshRTMorphSelector::UpdateWidget));

	TArray<FLayoutEditorMeshSection> MeshSectionsAndLayouts;
	GenerateMeshSectionOptions(MeshSectionsAndLayouts);

	TSharedPtr<SCustomizableObjectLayoutEditor> LayoutBlocksEditor = SNew(SCustomizableObjectLayoutEditor)
		.Node(Node)
		.MeshSections(MeshSectionsAndLayouts);

	FCustomizableObjectLayoutEditorDetailsBuilder LayoutEditorBuilder;
	LayoutEditorBuilder.LayoutEditor = LayoutBlocksEditor;
	LayoutEditorBuilder.bShowLayoutSelector = true;
	LayoutEditorBuilder.bShowPackagingStrategy = true;
	LayoutEditorBuilder.bShowAutomaticGenerationSettings = true;
	LayoutEditorBuilder.bShowGridSize = true;
	LayoutEditorBuilder.bShowMaxGridSize = true;
	LayoutEditorBuilder.bShowReductionMethods = true;
	LayoutEditorBuilder.bShowWarningSettings = true;

	LayoutEditorBuilder.CustomizeDetails(DetailBuilder);

	LayoutBlocksEditor->UpdateLayout(nullptr);
}


void FCustomizableObjectNodeSkeletalMeshDetails::GenerateMeshSectionOptions(TArray<FLayoutEditorMeshSection>& OutMeshSections)
{
	OutMeshSections.Empty();

	if (!Node)
	{
		return;
	}

	for (const UEdGraphPin* Pin : Node->GetAllNonOrphanPins())
	{
		if (const UCustomizableObjectNodeSkeletalMeshPinDataMesh* PinData = Cast<UCustomizableObjectNodeSkeletalMeshPinDataMesh>(Node->GetPinData(*Pin)))
		{
			FLayoutEditorMeshSection& MeshSection = OutMeshSections.AddDefaulted_GetRef();
			MeshSection.MeshName = MakeShareable(new FString(Pin->PinFriendlyName.ToString()));

			for (UCustomizableObjectLayout* Layout : PinData->Layouts)
			{
				MeshSection.Layouts.Add(Layout);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
