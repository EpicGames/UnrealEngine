// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeSkeletalMeshParameter)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCustomizableObjectNodeSkeletalMeshParameter::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
	
	MutableSkeletalMeshPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
	PassthroughPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object);
	
	USkeletalMesh* SkeletalMesh = ReferenceValue.LoadSynchronous();

	if (!SkeletalMesh)
	{
		return;
	}
	
	if (const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel())
	{
		for (int32 LODIndex = 0; LODIndex < ImportedModel->LODModels.Num(); LODIndex++)
		{
			const int32 NumSections = ImportedModel->LODModels[LODIndex].Sections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				// Ignore disabled sections.
				if (ImportedModel->LODModels[LODIndex].Sections[SectionIndex].bDisabled)
				{
					continue;
				}

				UMaterialInterface* MaterialInterface = GetMaterialInterfaceFor( SectionIndex);

				const FName MaterialName = MaterialInterface ? MaterialInterface->GetFName() : NAME_None;

				// Mesh
				{
					UCustomizableObjectNodeSkeletalMeshParameterPinDataSection* PinData = NewObject<UCustomizableObjectNodeSkeletalMeshParameterPinDataSection>(this);
					PinData->Init(LODIndex, SectionIndex, ImportedModel->LODModels[LODIndex].NumTexCoords);

					const FString MeshPinName = FString::Printf(TEXT("LOD %i - Section %i: %s"), LODIndex, SectionIndex, *MaterialName.ToString());
					const FText MeshPinFriendlyName = FText::Format(LOCTEXT("SkeletalMeshParameterMeshPin", "LOD {0} - Section {1}: {2}"), LODIndex, SectionIndex, FText::FromString(*MaterialName.ToString()));
					
					UEdGraphPin* Pin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh, FName(*MeshPinName), MeshPinFriendlyName, PinData);
					Pin->bHidden = true;
				}
			}			
		}
	}
}


void UCustomizableObjectNodeSkeletalMeshParameter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeSkeletalMeshParameter, ReferenceValue))
	{
		ReconstructNode();
	}
}


void UCustomizableObjectNodeSkeletalMeshParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (!NamePin.Get())
		{
			NamePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Name"), LOCTEXT("Name", "Name"));
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PinSkeletalMesh)
	{
		PassthroughPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough);
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PassthroughParameter)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinType.PinCategory ==  UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Passthrough)
			{
				Pin->PinType.PinCategory =  UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object;
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SkeletalMeshParameterMutableSkeletalMeshPin)
	{
		MutableSkeletalMeshPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
		
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object)
			{
				Pin->PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object);
				Pin->PinFriendlyName = UEdGraphSchema_CustomizableObject::GetPinFriendlyName(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh_Object);
			}
		}
	}
}


bool UCustomizableObjectNodeSkeletalMeshParameter::HasPinViewer() const
{
	return true;
}


bool UCustomizableObjectNodeSkeletalMeshParameter::IsAffectedByLOD() const
{
	return true;
}


FName UCustomizableObjectNodeSkeletalMeshParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Mesh;
}


FText UCustomizableObjectNodeSkeletalMeshParameter::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Skeletal_Mesh_Parameter", "Skeletal Mesh Parameter");
}


FLinearColor UCustomizableObjectNodeSkeletalMeshParameter::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_SkeletalMesh);
}


FText UCustomizableObjectNodeSkeletalMeshParameter::GetTooltipText() const
{
	return LOCTEXT("Mesh_Parameter_Tooltip", "Expose a runtime modifiable Mesh parameter from the Customizable Object.");
}


void UCustomizableObjectNodeSkeletalMeshParameter::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (Pin == NamePin.Get())
	{
		GetGraph()->NotifyGraphChanged();
	}
}


UTexture2D* UCustomizableObjectNodeSkeletalMeshParameter::FindTextureForPin(const UEdGraphPin* Pin) const
{
	return nullptr;
}


TArray<UCustomizableObjectLayout*> UCustomizableObjectNodeSkeletalMeshParameter::GetLayouts(const UEdGraphPin& MeshPin) const
{
	const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection& MeshPinData = GetPinData<UCustomizableObjectNodeSkeletalMeshParameterPinDataSection>(MeshPin);
	return MeshPinData.Layouts;
}


TSoftObjectPtr<UStreamableRenderAsset> UCustomizableObjectNodeSkeletalMeshParameter::GetMesh() const
{
	return ReferenceValue;
}


UEdGraphPin* UCustomizableObjectNodeSkeletalMeshParameter::GetMeshPin(const int32 LODIndex, const int32 SectionIndex) const
{
	for (UEdGraphPin* Pin : GetAllNonOrphanPins())
	{
		if (const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection* PinData = Cast<UCustomizableObjectNodeSkeletalMeshParameterPinDataSection>(GetPinData(*Pin)))
		{
			if (PinData->GetLODIndex() == LODIndex &&
				PinData->GetSectionIndex() == SectionIndex)
			{
				return Pin;
			}
		}
	}

	return nullptr;
}


void UCustomizableObjectNodeSkeletalMeshParameter::GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const
{
	OutLODIndex = -1;
	OutSectionIndex = -1;

	if (const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection* PinData = Cast<UCustomizableObjectNodeSkeletalMeshParameterPinDataSection>(GetPinData(Pin)))
	{
		OutLODIndex = PinData->GetLODIndex();
		OutSectionIndex = PinData->GetSectionIndex();
	}
}


void UCustomizableObjectNodeSkeletalMeshParameter::GetPinSection(const UEdGraphPin& Pin, int32& OutSectionIndex) const
{
	if (const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection* PinData = Cast<UCustomizableObjectNodeSkeletalMeshParameterPinDataSection>(GetPinData(Pin)))
	{
		OutSectionIndex = PinData->GetSectionIndex();
		return;
	}

	OutSectionIndex = -1;
}


UMaterialInterface* UCustomizableObjectNodeSkeletalMeshParameter::GetMaterialInterfaceFor(const int32 SectionIndex) const
{
	if (FSkeletalMaterial* SkeletalMaterial = GetSkeletalMaterialFor(SectionIndex))
	{
		return SkeletalMaterial->MaterialInterface;
	}

	return nullptr;
}


FSkeletalMaterial* UCustomizableObjectNodeSkeletalMeshParameter::GetSkeletalMaterialFor(const int32 SectionIndex) const
{
	USkeletalMesh* SkeletalMesh = ReferenceValue.LoadSynchronous();

	if (!SkeletalMesh)
	{
		return nullptr;
	}

	const int32 SkeletalMeshMaterialIndex = GetSkeletalMaterialIndexFor(SectionIndex);
	if (SkeletalMesh->GetMaterials().IsValidIndex(SkeletalMeshMaterialIndex))
	{
		return &SkeletalMesh->GetMaterials()[SkeletalMeshMaterialIndex];
	}

	return nullptr;
}

int32 UCustomizableObjectNodeSkeletalMeshParameter::GetSkeletalMaterialIndexFor(const int32 SectionIndex) const
{
	USkeletalMesh* SkeletalMesh = ReferenceValue.LoadSynchronous();

	if (!SkeletalMesh)
	{
		return INDEX_NONE;
	}

	int32 LODIndex = 0;

	// We assume that LODIndex and MaterialIndex are valid for the imported model
	int32 SkeletalMeshMaterialIndex = INDEX_NONE;

	// Check if we have lod info map to get the correct material index
	if (const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LODIndex))
	{
		if (LodInfo->LODMaterialMap.IsValidIndex(SectionIndex))
		{
			SkeletalMeshMaterialIndex = LodInfo->LODMaterialMap[SectionIndex];
		}
	}

	// Only deduce index when the explicit mapping is not found or there is no remap
	if (SkeletalMeshMaterialIndex == INDEX_NONE)
	{
		FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		if (ImportedModel && ImportedModel->LODModels.IsValidIndex(LODIndex) && ImportedModel->LODModels[LODIndex].Sections.IsValidIndex(SectionIndex))
		{
			SkeletalMeshMaterialIndex = ImportedModel->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex;
		}
	}

	return SkeletalMeshMaterialIndex;
}


int32 UCustomizableObjectNodeSkeletalMeshParameter::GetSkeletalMaterialIndexFor(const UEdGraphPin& Pin) const
{
	int32 SectionIndex;
	GetPinSection(Pin, SectionIndex);

	return GetSkeletalMaterialIndexFor(SectionIndex);
}


void UCustomizableObjectNodeSkeletalMeshParameterPinDataSection::Init(int32 InLODIndex, int32 InSectionIndex, int32 NumTexCoords)
{
	LODIndex = InLODIndex;
	SectionIndex = InSectionIndex;

	if (NumTexCoords > 0)
	{
		UObject* Outer = GetOuter();

		Layouts.SetNum(NumTexCoords);

		for (int32 Index = 0; Index < NumTexCoords; ++Index)
		{
			Layouts[Index] = NewObject<UCustomizableObjectLayout>(Outer);
			Layouts[Index]->SetLayout(LODIndex, InSectionIndex, Index);
		}
	}
}


int32 UCustomizableObjectNodeSkeletalMeshParameterPinDataSection::GetLODIndex() const
{
	return LODIndex;
}


int32 UCustomizableObjectNodeSkeletalMeshParameterPinDataSection::GetSectionIndex() const
{
	return SectionIndex;
}


bool UCustomizableObjectNodeSkeletalMeshParameterPinDataSection::Equals(const UCustomizableObjectNodePinData& Other) const
{
	if (GetClass() != Other.GetClass())
	{
		return false;
	}

	const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection& OtherTyped = static_cast<const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection&>(Other);
	if (SectionIndex != OtherTyped.SectionIndex)
	{
		return false;
	}

	return Super::Equals(Other);
}


#undef LOCTEXT_NAMESPACE

