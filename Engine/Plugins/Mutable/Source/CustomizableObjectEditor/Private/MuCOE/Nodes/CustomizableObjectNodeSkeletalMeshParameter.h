// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeParameter.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"

#include "CustomizableObjectNodeSkeletalMeshParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectLayout;
class UMaterialInterface;
struct FSkeletalMaterial;


/** PinData of a pin that belongs to a Skeletal Mesh Section. */
UCLASS()
class UCustomizableObjectNodeSkeletalMeshParameterPinDataSection : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:
	void Init(int32 LODIndex, int32 InSectionIndex, int32 NumTexCoords);

	int32 GetLODIndex() const;
	
	int32 GetSectionIndex() const;

	/** Layouts related to this Mesh pin */
	UPROPERTY()
	TArray<TObjectPtr<UCustomizableObjectLayout>> Layouts;

protected:

	virtual bool Equals(const UCustomizableObjectNodePinData& Other) const override;

private:
	UPROPERTY()
	int32 LODIndex = -1;
	
	UPROPERTY()
	int32 SectionIndex = -1;
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeSkeletalMeshParameter : public UCustomizableObjectNodeParameter, public ICustomizableObjectNodeMeshInterface
{
	GENERATED_BODY()
	
public:
	// UObject interface
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual bool HasPinViewer() const override;
	UE_API virtual bool IsAffectedByLOD() const override;

	// UCustomizableObjectNodeParameter interface
	virtual FName GetCategory() const override;
	
	// ICustomizableObjectNodeMeshInterface interface
	UE_API virtual UTexture2D* FindTextureForPin(const UEdGraphPin* Pin) const override;
	UE_API virtual TArray<UCustomizableObjectLayout*> GetLayouts(const UEdGraphPin& MeshPin) const override;
	UE_API virtual TSoftObjectPtr<UStreamableRenderAsset> GetMesh() const override;
	UE_API virtual UEdGraphPin* GetMeshPin(int32 LOD, int32 SectionIndex) const override;
	UE_API virtual void GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const override;

	// Own interface
	UE_API void GetPinSection(const UEdGraphPin& Pin, int32& OutSectionIndex) const;
	UE_API UMaterialInterface* GetMaterialInterfaceFor(const int32 SectionIndex) const;
	UE_API FSkeletalMaterial* GetSkeletalMaterialFor(const int32 SectionIndex) const;
	UE_API int32 GetSkeletalMaterialIndexFor(const UEdGraphPin& Pin) const;
	UE_API int32 GetSkeletalMaterialIndexFor(const int32 SectionIndex) const;

	/** Default value of the parameter. */
	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TSoftObjectPtr<USkeletalMesh> DefaultValue;

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TSoftObjectPtr<USkeletalMesh> ReferenceValue;

	UPROPERTY()
	FEdGraphPinReference MutableSkeletalMeshPin;
	
	UPROPERTY()
	FEdGraphPinReference PassthroughPin;
};

#undef UE_API
