// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameterDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMeshParameter.h"
#include "MuCOE/PinViewer/SPinViewer.h"
#include "MuCOE/SCustomizableObjectLayoutEditor.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectNodeMaterialDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeSkeletalMeshParameterDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeSkeletalMeshParameterDetails);
}


void FCustomizableObjectNodeSkeletalMeshParameterDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FCustomizableObjectNodeParameterDetails::CustomizeDetails(DetailBuilder);

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();

    if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeSkeletalMeshParameter>(DetailsView->GetSelectedObjects()[0].Get());
	}

    if (!Node)
    {
        return;
    }

    IDetailCategoryBuilder& CustomizableObject = DetailBuilder.EditCategory("CustomizableObject");

	TSharedRef<IPropertyHandle> DefaultValueProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeSkeletalMeshParameter, ReferenceValue));

	TArray<FLayoutEditorMeshSection> MeshSectionsAndLayouts;
	GenerateMeshSectionOptions(MeshSectionsAndLayouts);

	TSharedPtr<SCustomizableObjectLayoutEditor> LayoutBlocksEditor = SNew(SCustomizableObjectLayoutEditor)
		.Node(Node)
		.MeshSections(MeshSectionsAndLayouts);

	FCustomizableObjectLayoutEditorDetailsBuilder LayoutEditorBuilder;
	LayoutEditorBuilder.LayoutEditor = LayoutBlocksEditor;
	LayoutEditorBuilder.bShowLayoutSelector = true;
	LayoutEditorBuilder.bShowPackagingStrategy = true;
	LayoutEditorBuilder.bShowAutomaticGenerationSettings = true;
	LayoutEditorBuilder.bShowGridSize = true;
	LayoutEditorBuilder.bShowMaxGridSize = true;
	LayoutEditorBuilder.bShowReductionMethods = true;
	LayoutEditorBuilder.bShowWarningSettings = true;

	LayoutEditorBuilder.CustomizeDetails(DetailBuilder);

	LayoutBlocksEditor->UpdateLayout(nullptr);
}


void FCustomizableObjectNodeSkeletalMeshParameterDetails::GenerateMeshSectionOptions(TArray<FLayoutEditorMeshSection>& OutMeshSections)
{
	OutMeshSections.Empty();

	if (!Node)
	{
		return;
	}

	for (const UEdGraphPin* Pin : Node->GetAllNonOrphanPins())
	{
		if (const UCustomizableObjectNodeSkeletalMeshParameterPinDataSection* PinData = Cast<UCustomizableObjectNodeSkeletalMeshParameterPinDataSection>(Node->GetPinData(*Pin)))
		{
			FLayoutEditorMeshSection& MeshSection = OutMeshSections.AddDefaulted_GetRef();
			MeshSection.MeshName = MakeShareable(new FString(Pin->PinFriendlyName.ToString()));

			for (UCustomizableObjectLayout* Layout : PinData->Layouts)
			{
				MeshSection.Layouts.Add(Layout);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
