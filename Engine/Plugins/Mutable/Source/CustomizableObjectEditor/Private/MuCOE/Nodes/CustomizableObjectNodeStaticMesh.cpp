// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeStaticMesh.h"

#include "AssetThumbnail.h"
#include "ISinglePropertyView.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeLayoutBlocks.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByNameDefaultPin.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Input/SCheckBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeStaticMesh)

class UCustomizableObjectNodeRemapPinsByName;
class UObject;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


/** Default node pin configuration pin name (node does not have an static mesh). */
static const TCHAR* STATIC_MESH_PIN_NAME = TEXT("Static Mesh");


void UCustomizableObjectNodeStaticMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && PropertyThatChanged->GetName() == GET_MEMBER_NAME_STRING_CHECKED(UCustomizableObjectNodeStaticMesh, StaticMesh))
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeStaticMesh::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PostLoadToCustomVersion)
	{
		for (FCustomizableObjectNodeStaticMeshLOD& LOD : LODs)
		{
			for (FCustomizableObjectNodeStaticMeshMaterial& Material : LOD.Materials)
			{
				if (Material.MeshPin_DEPRECATED && !Material.MeshPinRef.Get())
				{
					UEdGraphPin* AuxPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(Material.MeshPin_DEPRECATED);
					Material.MeshPinRef.SetPin(AuxPin);
				}

				if (Material.LayoutPin_DEPRECATED && !Material.LayoutPinRef.Get())
				{
					UEdGraphPin* AuxPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(Material.LayoutPin_DEPRECATED);
					Material.LayoutPinRef.SetPin(AuxPin);
				}

				if (Material.ImagePins_DEPRECATED.Num() && !Material.ImagePinsRef.Num())
				{
					for (UEdGraphPin_Deprecated* DeprecatedPin : Material.ImagePins_DEPRECATED)
					{
						UEdGraphPin* AuxPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(DeprecatedPin);
						FEdGraphPinReference AuxEdGraphPinReference(AuxPin);
						Material.ImagePinsRef.Add(AuxEdGraphPinReference);
					}
				}
			}
		}
	}
}


void UCustomizableObjectNodeStaticMesh::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Pass information to the remap pins action context
	if (UCustomizableObjectNodeRemapPinsByNameDefaultPin* RemapPinsCustom = Cast<UCustomizableObjectNodeRemapPinsByNameDefaultPin>(RemapPins))
	{
		RemapPinsCustom->DefaultPin = DefaultPin.Get();
	}
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (UE::Mutable::Private::LoadObject(StaticMesh))
	{
		const int32 NumLODs = StaticMesh->GetRenderData()->LODResources.Num();

		LODs.Reset(NumLODs);
		LODs.SetNum(NumLODs);
		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			const int32 NumMaterials = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections.Num();

			LODs[LODIndex].Materials.Reset(NumMaterials);
			LODs[LODIndex].Materials.SetNum(NumMaterials);
			for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
			{
				UMaterialInterface* MaterialInterface = 0;
				int32 StaticMeshMaterialIndex = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[MaterialIndex].MaterialIndex;
				if (StaticMeshMaterialIndex >= 0 && StaticMeshMaterialIndex < StaticMesh->GetStaticMaterials().Num())
				{
					MaterialInterface = StaticMesh->GetStaticMaterials()[StaticMeshMaterialIndex].MaterialInterface;
				}

				FString MaterialName = TEXT("Unnamed Material");
				if (MaterialInterface)
				{
					MaterialName = MaterialInterface->GetName();
				}
				LODs[LODIndex].Materials[MaterialIndex].Name = MaterialName;
				
				// Mesh
				const FString MeshName = MaterialName + TEXT(" Mesh");
				UEdGraphPin* Pin = CustomCreatePin(EGPD_Output, Schema->PC_Mesh, FName(*MeshName), FText::FromString(MeshName));
				LODs[LODIndex].Materials[MaterialIndex].MeshPinRef.SetPin(Pin);

				// Images
				if (MaterialInterface)
				{
					UMaterial* Material = MaterialInterface->GetMaterial();

					TArray<FMaterialParameterInfo> ImageNames;
					TArray<FGuid> ImageIds;
					Material->GetAllTextureParameterInfo(ImageNames, ImageIds);
					for (int32 ImageIndex = 0; ImageIndex < ImageNames.Num(); ++ImageIndex)
					{
						FString ImageName = ImageNames[ImageIndex].Name.ToString();
						Pin = CustomCreatePin(EGPD_Output, Schema->PC_Texture, *ImageName, FText::FromString(ImageName));
						LODs[LODIndex].Materials[MaterialIndex].ImagePinsRef.Add(Pin);
					}
				}
			}
		}

		DefaultPin = FEdGraphPinReference();
	}
	else
	{
		DefaultPin = CustomCreatePin(EGPD_Output, Schema->PC_Mesh, FName(STATIC_MESH_PIN_NAME), FText::FromString(STATIC_MESH_PIN_NAME));
	}
}


FText UCustomizableObjectNodeStaticMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (UE::Mutable::Private::LoadObject(StaticMesh))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("MeshName"), FText::FromString(StaticMesh->GetName()));

		return FText::Format(LOCTEXT("StaticMesh_Title", "{MeshName}\nStatic Mesh"), Args);
	}
	else
	{
		return LOCTEXT("Static_Mesh", "Static Mesh");
	}
}


FLinearColor UCustomizableObjectNodeStaticMesh::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_StaticMesh);
}


UTexture2D* UCustomizableObjectNodeStaticMesh::FindTextureForPin(const UEdGraphPin* Pin) const
{
	if (!UE::Mutable::Private::LoadObject(StaticMesh))
	{
		return 0;
	}

	for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
	{
		for (int32 MaterialIndex = 0; MaterialIndex<LODs[LODIndex].Materials.Num(); ++MaterialIndex)
		{
			UMaterialInterface* MaterialInterface = 0;
			int32 StaticMeshMaterialIndex = -1;
			if (StaticMesh->GetRenderData()->LODResources.Num() > LODIndex && StaticMesh->GetRenderData()->LODResources[LODIndex].Sections.Num()>MaterialIndex)
			{
				StaticMeshMaterialIndex = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[MaterialIndex].MaterialIndex;
			}

			if (StaticMeshMaterialIndex >= 0 && StaticMeshMaterialIndex < StaticMesh->GetStaticMaterials().Num() )
			{
				MaterialInterface = StaticMesh->GetStaticMaterials()[ StaticMeshMaterialIndex ].MaterialInterface;
			}

			if (MaterialInterface)
			{
				for (int32 ImageIndex = 0; ImageIndex < LODs[LODIndex].Materials[MaterialIndex].ImagePinsRef.Num(); ++ImageIndex)
				{
					if (LODs[LODIndex].Materials[MaterialIndex].ImagePinsRef[ImageIndex].Get() == Pin)
					{
						UTexture* Texture=nullptr;
						MaterialInterface->GetTextureParameterValue( Pin?Pin->PinName:FName(), Texture);
						return Cast<UTexture2D>(Texture);
					}
				}
			}
		}
	}

	return 0;
}


TArray<UCustomizableObjectLayout*> UCustomizableObjectNodeStaticMesh::GetLayouts(const UEdGraphPin& OutPin) const
{
	TArray<UCustomizableObjectLayout*> Result;
	return Result;
}


TSoftObjectPtr<UStreamableRenderAsset> UCustomizableObjectNodeStaticMesh::GetMesh() const
{
	return StaticMesh;
}


UEdGraphPin* UCustomizableObjectNodeStaticMesh::GetMeshPin(const int32 LODIndex, const int32 SectionIndex) const
{
	if (LODIndex < LODs.Num())
	{
		if (const FCustomizableObjectNodeStaticMeshLOD& LOD = LODs[LODIndex];
			SectionIndex < LOD.Materials.Num())
		{
			return LOD.Materials[SectionIndex].MeshPinRef.Get();
		}
	}

	return nullptr;
}


bool UCustomizableObjectNodeStaticMesh::IsPinRelevant(const UEdGraphPin* Pin) const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
	{	
		return Pin->PinType.PinCategory == Schema->PC_Mesh;
	}

	return false;
}


UCustomizableObjectNodeRemapPinsByName* UCustomizableObjectNodeStaticMesh::CreateRemapPinsByName() const
{
	return NewObject<UCustomizableObjectNodeRemapPinsByNameDefaultPin>();
}


bool UCustomizableObjectNodeStaticMesh::HasPinViewer() const
{
	return true;
}


bool UCustomizableObjectNodeStaticMesh::IsNodeOutDatedAndNeedsRefresh()
{
	bool Result = false;

	if (UE::Mutable::Private::LoadObject(StaticMesh))
	{
		const int32 LODCount = StaticMesh->GetRenderData()->LODResources.Num();
		if (LODs.Num() != LODCount)
		{
			Result = true;
		}

		for (int32 LODIndex = 0; !Result && LODIndex < LODs.Num(); ++LODIndex)
		{
			const int32 NumMaterials = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
			if (LODs[LODIndex].Materials.Num() != NumMaterials)
			{
				Result = true;
			}

			for (int32 MaterialIndex = 0; !Result && MaterialIndex < NumMaterials; ++MaterialIndex)
			{
				UMaterialInterface* MaterialInterface = nullptr;
				int32 SkeletalMeshMaterialIndex = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[MaterialIndex].MaterialIndex;
				if (SkeletalMeshMaterialIndex >= 0 && SkeletalMeshMaterialIndex < StaticMesh->GetStaticMaterials().Num())
				{
					MaterialInterface = StaticMesh->GetStaticMaterials()[SkeletalMeshMaterialIndex].MaterialInterface;
				}

				if (MaterialInterface)
				{
					/*if (LODs[LODIndex].Materials[MaterialIndex].Name != MaterialInterface->GetName())
					{
						Result = true;
						break;
					}*/
					UMaterial* Material = MaterialInterface->GetMaterial();

					TArray<FMaterialParameterInfo> ImageInfos;
					TArray<FGuid> ImageIds;
					Material->GetAllTextureParameterInfo(ImageInfos, ImageIds);

					if (LODs[LODIndex].Materials[MaterialIndex].ImagePinsRef.Num() != ImageInfos.Num())
					{
						Result = true;
						break;
					}

					for (int32 ImageIndex = 0; ImageIndex < ImageInfos.Num(); ++ImageIndex)
					{
						const FString ImagePinName = LODs[LODIndex].Materials[MaterialIndex].ImagePinsRef[ImageIndex].Get()->GetName();
						if (!ImagePinName.Contains(ImageInfos[ImageIndex].Name.ToString()))
						{
							Result = true;
							break;
						}
					}
				}
			}
		}
	}

	// Remove previous compilation warnings
	if (!Result && bHasCompilerMessage)
	{
		RemoveWarnings();
		GetGraph()->NotifyGraphChanged();
	}

	return Result;
}


FString UCustomizableObjectNodeStaticMesh::GetRefreshMessage() const
{
	return "Node data outdated. Please refresh node.";
}


void UCustomizableObjectNodeStaticMesh::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixStaticMeshNodeLODMaterialDataInvalidation2)
	{
		if (!UE::Mutable::Private::LoadObject(StaticMesh))
		{
			return;
		}

		// Ensure the remapping is actually needed, if not skip the remapping
		if (DefaultPin.Get() == nullptr)
		{
			bool bInvalidMeshPinReferenceFound = false;
			for (const FCustomizableObjectNodeStaticMeshLOD& LODData : LODs)
			{
				// There is no need to remap the pins if they are already correct.
				if (LODData.Materials.IsValidIndex(0) && LODData.Materials[0].MeshPinRef.Get() == nullptr)
				{
					bInvalidMeshPinReferenceFound = true;
					break;
				}
			}

			// If the mesh pin references are correct there is no need to fix them.
			if (!bInvalidMeshPinReferenceFound)
			{
				return;
			}
		}
		
		// The pin references in the LODs array are wrong so they need to be regenerated and remapped
		
		// Remap the pins using the old pin names but DO NOT auto-set one as default
		FixupReconstructPins(NewObject<UCustomizableObjectNodeRemapPinsByNameDefaultPin>(), [&](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* Action)
		{
			// Generate the pins again so we get non default pins and valid info into the LODs array
				
			if (StaticMesh->HasValidRenderData(false, INDEX_NONE))
			{
				// Make the first mesh pin the default one if the DefaultPin was already set (no LOD pins were present before)
				if (UCustomizableObjectNodeRemapPinsByNameDefaultPin* RemapPinsCustom = Cast<UCustomizableObjectNodeRemapPinsByNameDefaultPin>(Action))
				{
					RemapPinsCustom->DefaultPin = DefaultPin.Get();
				}
				
				const int32 NumLODs = StaticMesh->GetRenderData()->LODResources.Num();
				LODs.Reset(NumLODs);
				LODs.SetNum(NumLODs);
				for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
				{
					const int32 NumMaterials = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections.Num();

					LODs[LODIndex].Materials.Reset(NumMaterials);
					LODs[LODIndex].Materials.SetNum(NumMaterials);
					for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
					{
						UMaterialInterface* MaterialInterface = nullptr;
						const int32 StaticMeshMaterialIndex = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[MaterialIndex].MaterialIndex;
						if (StaticMeshMaterialIndex >= 0 && StaticMeshMaterialIndex < StaticMesh->GetStaticMaterials().Num())
						{
							MaterialInterface = StaticMesh->GetStaticMaterials()[StaticMeshMaterialIndex].MaterialInterface;
						}

						FString MaterialName = TEXT("Unnamed Material");
						if (MaterialInterface)
						{
							MaterialName = MaterialInterface->GetName();
						}
						LODs[LODIndex].Materials[MaterialIndex].Name = MaterialName;
						
						// Mesh
						const FString MeshName = MaterialName + TEXT(" Mesh");
						UEdGraphPin* Pin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Mesh, FName(*MeshName), FText::FromString(MeshName));
						LODs[LODIndex].Materials[MaterialIndex].MeshPinRef.SetPin(Pin);

						// Images
						if (MaterialInterface)
						{
							if (const UMaterial* Material = MaterialInterface->GetMaterial())
							{
								TArray<FMaterialParameterInfo> ImageNames;
								TArray<FGuid> ImageIds;
								
								Material->GetAllTextureParameterInfo(ImageNames, ImageIds);
								for (int32 ImageIndex = 0; ImageIndex < ImageNames.Num(); ++ImageIndex)
								{
									FString ImageName = ImageNames[ImageIndex].Name.ToString();
									Pin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture, *ImageName, FText::FromString(ImageName));
									LODs[LODIndex].Materials[MaterialIndex].ImagePinsRef.Add(Pin);
								}
							}	
						}
					}
				}
			}
		});
	}
}



FText UCustomizableObjectNodeStaticMesh::GetTooltipText() const
{
	return LOCTEXT("Static_Mesh_Tooltip", "Static Mesh");
}


void UCustomizableObjectNodeStaticMesh::GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const
{
	OutLODIndex = INDEX_NONE;
	OutSectionIndex = INDEX_NONE;
	
	if (UE::Mutable::Private::LoadObject(StaticMesh))
	{
		for (OutLODIndex = 0; OutLODIndex < LODs.Num(); ++OutLODIndex)
		{
			const FCustomizableObjectNodeStaticMeshLOD& LOD = LODs[OutLODIndex];

			for (OutSectionIndex = 0; OutSectionIndex < LOD.Materials.Num(); ++OutSectionIndex)
			{
				const FCustomizableObjectNodeStaticMeshMaterial& Material = LOD.Materials[OutSectionIndex];

				// Is a Layout pin?
				if (Material.LayoutPinRef.Get() == &Pin)
				{
					return;
				}

				// Is a Mesh pin?
				if (Material.MeshPinRef.Get() == &Pin)
				{
					return;
				}
			
				// Is an Image pin?
				for (int32 ImageIndex = 0; ImageIndex < Material.ImagePinsRef.Num(); ++ImageIndex)
				{
					if (Material.ImagePinsRef[ImageIndex].Get() == &Pin)
					{
						return;
					}
				}
			}
		}
	
		check(false); // Pin not found. Probably a new pin has been added which is not contemplated in this function. 
	}
	
}


UMaterialInterface* UCustomizableObjectNodeStaticMesh::GetMaterialFor(const UEdGraphPin* Pin) const
{
	if (UE::Mutable::Private::LoadObject(StaticMesh))
	{
		check(Pin);
		int32 LODIndex;
		int32 SectionIndex;
		GetPinSection(*Pin, LODIndex, SectionIndex);

		const int32 MaterialIndex = StaticMesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].MaterialIndex;

		if (const TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
			StaticMaterials.IsValidIndex(MaterialIndex))
		{
			return StaticMaterials[MaterialIndex].MaterialInterface;
		}
	}
	
	return nullptr;
}


TSharedPtr<SGraphNode> UCustomizableObjectNodeStaticMesh::CreateVisualWidget()
{
	TSharedPtr<SGraphNodeStaticMesh> GraphNode;
	SAssignNew(GraphNode, SGraphNodeStaticMesh, this);

	GraphNodeStaticMesh = GraphNode;

	return GraphNode;
}


void SGraphNodeStaticMesh::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	NodeStaticMesh = Cast< UCustomizableObjectNodeStaticMesh >(InGraphNode);
	UE::Mutable::Private::LoadObject(NodeStaticMesh->StaticMesh);
	
	WidgetSize = 128.0f;
	ThumbnailSize = 128;

	TSharedPtr<FCustomizableObjectEditor> Editor = StaticCastSharedPtr< FCustomizableObjectEditor >(NodeStaticMesh->GetGraphEditor());

	// Thumbnail
	AssetThumbnailPool = MakeShareable(new FAssetThumbnailPool(32));
	AssetThumbnail = MakeShareable(new FAssetThumbnail(NodeStaticMesh->StaticMesh.Get(), ThumbnailSize, ThumbnailSize, AssetThumbnailPool));

	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FSinglePropertyParams SingleDetails;
	SingleDetails.NamePlacement = EPropertyNamePlacement::Hidden;
	SingleDetails.NotifyHook = Editor.Get();
	SingleDetails.bHideAssetThumbnail = true;

	StaticMeshSelector = PropPlugin.CreateSingleProperty(NodeStaticMesh, GET_MEMBER_NAME_STRING_CHECKED(UCustomizableObjectNodeStaticMesh, StaticMesh), SingleDetails);	

	SCustomizableObjectNode::Construct({}, InGraphNode);
}


void SGraphNodeStaticMesh::UpdateGraphNode()
{
	SGraphNode::UpdateGraphNode();
}


void SGraphNodeStaticMesh::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	DefaultTitleAreaWidget->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(5))
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SGraphNodeStaticMesh::OnExpressionPreviewChanged)
		.IsChecked(IsExpressionPreviewChecked())
		.Cursor(EMouseCursor::Default)
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.Image(GetExpressionPreviewArrow())
		]
		]
		];
}


void SGraphNodeStaticMesh::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	LeftNodeBox->AddSlot()
		.AutoHeight()
		.MaxHeight(WidgetSize)
		.Padding(10.0f, 10.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			.Visibility(ExpressionPreviewVisibility())

		+ SHorizontalBox::Slot()
		.MaxWidth(WidgetSize)
		.Padding(5.0f, 5.0f, 5.0f, 5.0f)
		[
			AssetThumbnail->MakeThumbnailWidget()
		]
		];

	if (StaticMeshSelector.IsValid())
	{
		LeftNodeBox->AddSlot()
			.AutoHeight()
			.Padding(10.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				.Visibility(ExpressionPreviewVisibility())

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 0.0f, 5.0f, 5.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				StaticMeshSelector.ToSharedRef()
			]
			];
	}
}


void SGraphNodeStaticMesh::OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState)
{
	NodeStaticMesh->bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	UpdateGraphNode();
}


ECheckBoxState SGraphNodeStaticMesh::IsExpressionPreviewChecked() const
{
	return NodeStaticMesh->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


const FSlateBrush* SGraphNodeStaticMesh::GetExpressionPreviewArrow() const
{
	return FCustomizableObjectEditorStyle::Get().GetBrush(NodeStaticMesh->bCollapsed ? TEXT("Nodes.ArrowDown") : TEXT("Nodes.ArrowUp"));
}


EVisibility SGraphNodeStaticMesh::ExpressionPreviewVisibility() const
{
	return NodeStaticMesh->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible;
}


#undef LOCTEXT_NAMESPACE
