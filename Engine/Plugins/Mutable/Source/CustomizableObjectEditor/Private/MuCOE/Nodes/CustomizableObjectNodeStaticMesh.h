// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeMeshInterface.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CustomizableObjectNodeStaticMesh.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class FAssetThumbnail;
class FAssetThumbnailPool;
class SOverlay;
class SVerticalBox;
class UCustomizableObjectLayout;
class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeRemapPinsByName;
class UMaterialInterface;
class UObject;
class UStaticMesh;
class UTexture2D;
struct FPropertyChangedEvent;
struct FSlateBrush;


// Class to render the Static Mesh thumbnail of a CustomizableObjectNodeStaticMesh
class SGraphNodeStaticMesh : public SCustomizableObjectNode
{
public:

	SLATE_BEGIN_ARGS(SGraphNodeStaticMesh) {}
	SLATE_END_ARGS();

	SGraphNodeStaticMesh() : SCustomizableObjectNode() {};

	// Builds the SGraphNodeStaticMesh when needed
	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// Calls the needed functions to build the SGraphNode widgets
	void UpdateGraphNode();

	// Overriden functions to build the SGraphNode widgets
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	// Callbacks for the widget
	void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsExpressionPreviewChecked() const;
	const FSlateBrush* GetExpressionPreviewArrow() const;
	EVisibility ExpressionPreviewVisibility() const;

public:

	// Single property that only draws the combo box widget of the static mesh
	TSharedPtr<class ISinglePropertyView> StaticMeshSelector;

	// Pointer to the NodeStaticMesh that owns this SGraphNode
	class UCustomizableObjectNodeStaticMesh* NodeStaticMesh = nullptr;

private:

	// Classes needed to get and render the thumbnail of the Static Mesh
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	//This parameter defines the size of the thumbnail widget inside the Node
	float WidgetSize;

	// This parameter defines the resolution of the thumbnail
	uint32 ThumbnailSize;

};


USTRUCT()
struct FCustomizableObjectNodeStaticMeshMaterial
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	TObjectPtr<UEdGraphPin_Deprecated> MeshPin_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UEdGraphPin_Deprecated> LayoutPin_DEPRECATED;

	UPROPERTY()
	TArray< TObjectPtr<UEdGraphPin_Deprecated>> ImagePins_DEPRECATED;

	UPROPERTY()
	FEdGraphPinReference MeshPinRef;

	UPROPERTY()
	FEdGraphPinReference LayoutPinRef;

	UPROPERTY()
	TArray<FEdGraphPinReference> ImagePinsRef;
};


USTRUCT()
struct FCustomizableObjectNodeStaticMeshLOD
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FCustomizableObjectNodeStaticMeshMaterial> Materials;
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeStaticMesh : public UCustomizableObjectNode, public ICustomizableObjectNodeMeshInterface
{ 
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TSoftObjectPtr<UStaticMesh> StaticMesh;

	/** Images */
	UPROPERTY()
	TArray<FCustomizableObjectNodeStaticMeshLOD> LODs;

	/** Default pin when there is no mesh. */
	UPROPERTY()
	FEdGraphPinReference DefaultPin;
	
	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void Serialize(FArchive& Ar) override;

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool ProvidesCustomPinRelevancyTest() const override { return true; }
	UE_API virtual bool IsPinRelevant(const UEdGraphPin* Pin) const override;
	UE_API virtual UCustomizableObjectNodeRemapPinsByName* CreateRemapPinsByName() const override;
	UE_API virtual bool HasPinViewer() const override;
	UE_API virtual bool IsNodeOutDatedAndNeedsRefresh() override;
	UE_API virtual FString GetRefreshMessage() const override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// UCustomizableObjectNodeMesh interface
	UE_API virtual UTexture2D* FindTextureForPin(const UEdGraphPin* Pin) const override;
	UE_API virtual TArray<UCustomizableObjectLayout*> GetLayouts(const UEdGraphPin& OutPin) const override;
	UE_API virtual TSoftObjectPtr<UStreamableRenderAsset> GetMesh() const override;
	UE_API virtual UEdGraphPin* GetMeshPin(int32 LOD, int32 SectionIndex) const override;
	UE_API virtual void GetPinSection(const UEdGraphPin& Pin, int32& OutLODIndex, int32& OutSectionIndex) const override;

	/** Returns the material associated to the given output pin. */
	UE_API UMaterialInterface* GetMaterialFor(const UEdGraphPin* Pin) const;

	// Creates the SGraph Node widget for the thumbnail
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	// Determines if the Node is collapsed or not
	bool bCollapsed = true;

	// Pointer to the SGraphNodeStaticMesh
	TWeakPtr< SGraphNodeStaticMesh > GraphNodeStaticMesh;

};

#undef UE_API
