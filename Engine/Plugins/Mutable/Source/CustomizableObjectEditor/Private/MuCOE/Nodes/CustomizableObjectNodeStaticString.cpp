// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"
#include "ScopedTransaction.h"
#include "MuCO/CustomizableObjectCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeStaticString)

#define LOCTEXT_NAMESPACE "CustomizableObjectNodeStringConstant"


FText UCustomizableObjectNodeStaticString::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("StaticStringNodeTitle", "Static String");
}


FText UCustomizableObjectNodeStaticString::GetTooltipText() const
{
	return LOCTEXT("StaticStringNodeTooltip", "Static String Node");
}


FLinearColor UCustomizableObjectNodeStaticString::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_String);
}


void UCustomizableObjectNodeStaticString::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_String, TEXT("Value"), LOCTEXT("Value", "Value"));
}


bool UCustomizableObjectNodeStaticString::CanRenamePin(const UEdGraphPin& Pin) const
{
	return true;
}


FText UCustomizableObjectNodeStaticString::GetPinEditableName(const UEdGraphPin& Pin) const
{
	return FText::FromString(Value);
}


void UCustomizableObjectNodeStaticString::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeStaticString_SetStringPinEditableNameTransaction", "Change Literal String PinName"));
	Modify();
	
	Value = InValue.ToString();
	PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
}


void UCustomizableObjectNodeStaticString::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeStaticStringRemoveHiddenInputPin)
	{
		if (UEdGraphPin* OldInputPin = FindPin(TEXT("String")))
		{
			CustomRemovePin(*OldInputPin);
		}
	}
}


EEditablePinNameBoxVisibilityPolicy UCustomizableObjectNodeStaticString::GetEditablePinNameVisibilityPolicy(const UEdGraphPin& Pin) const
{
	return EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE;
}


#undef LOCTEXT_NAMESPACE
