// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"

#include "Animation/PoseAsset.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2DArray.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectEditorLogger.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/MutableUtils.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Animation/AnimInstance.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "UObject/LinkerLoad.h"
#include "PropertyEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTable)

class ICustomizableObjectEditor;
class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


bool UCustomizableObjectNodeTableRemapPins::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const
{
	const UCustomizableObjectNodeTableObjectPinData& OldPinData = Node.GetPinData<UCustomizableObjectNodeTableObjectPinData>(OldPin);
	const UCustomizableObjectNodeTableObjectPinData& NewPinData = Node.GetPinData<UCustomizableObjectNodeTableObjectPinData>(NewPin);

	if (OldPin.Direction != NewPin.Direction)
	{
		return false;
	}

	// If both pins have valid IDs, check if its id has changed instead of checking its name
	if (OldPinData.StructColumnId.IsValid() && NewPinData.StructColumnId.IsValid())
	{
		if (OldPinData.StructColumnId != NewPinData.StructColumnId)
		{
			return false;
		}
	}
	else
	{
		// If one of these two option fails, pins are different
		if (OldPinData.ColumnPropertyName != NewPinData.ColumnPropertyName)
		{
			return false;
		}
	}

	// In this case pin type may have changed but we consider them the same type
	if (OldPin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture ||
		OldPin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
	{
		return true;
	}

	// Non image pins must remain the same pin type
	if (OldPin.PinType != NewPin.PinType)
	{
		return false;
	}

	if (OldPin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Mesh)
	{
		const UCustomizableObjectNodeTableMeshPinData& OldMeshPinData = Node.GetPinData<UCustomizableObjectNodeTableMeshPinData>(OldPin);
		const UCustomizableObjectNodeTableMeshPinData& NewMeshPinData = Node.GetPinData<UCustomizableObjectNodeTableMeshPinData>(NewPin);

		// LOD and Section must remain the same
		return 	OldMeshPinData.LOD == NewMeshPinData.LOD &&
			OldMeshPinData.Material == NewMeshPinData.Material;
	}

	return true;
}


void UCustomizableObjectNodeTableRemapPins::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	for (UEdGraphPin* OldPin : OldPins)
	{
		bool bFound = false;

		for (UEdGraphPin* NewPin : NewPins)
		{
			if (Equal(Node, *OldPin, *NewPin))
			{
				bFound = true;

				PinsToRemap.Add(OldPin, NewPin);
				break;
			}
		}

		if (!bFound && OldPin->LinkedTo.Num())
		{
			PinsToOrphan.Add(OldPin);
		}
	}
}


// Table Node -------

const TArray<FFieldClass*> UCustomizableObjectNodeTable::SupportedFilterTypes = {
		FFloatProperty::StaticClass(),
		FDoubleProperty::StaticClass(),
		FBoolProperty::StaticClass(),
		FNameProperty::StaticClass(),
		FStrProperty::StaticClass(),
		FTextProperty::StaticClass(),
		FIntProperty::StaticClass() };


void UCustomizableObjectNodeTable::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddedTableNodesTextureMode)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture)
			{
				UCustomizableObjectNodeTableObjectPinData* OldPinData = Cast<UCustomizableObjectNodeTableObjectPinData>(GetPinData(*(Pin)));
				UCustomizableObjectNodeTableImagePinData* NewPinData = NewObject<UCustomizableObjectNodeTableImagePinData>(this);

				if (OldPinData && NewPinData)
				{
					NewPinData->ColumnName_DEPRECATED = OldPinData->ColumnName_DEPRECATED;
					NewPinData->ImageMode = ETableTextureType::MUTABLE_TEXTURE; // Old pin type by default
				}

				AddPinData(*Pin, *NewPinData);
			}
		}
	}
	
	// Adding StructColumnID
	if (CustomizableObjectCustomVersion < FCustomizableObjectCustomVersion::AddedColumnIdDataToTableNodePins)
	{
		const UScriptStruct* TableStruct = GetTableNodeStruct();

		if (TableStruct)
		{
			for (UEdGraphPin* Pin : Pins)
			{
				UCustomizableObjectNodeTableObjectPinData* PinData = Cast<UCustomizableObjectNodeTableObjectPinData>(GetPinData(*(Pin)));

				// Adding pindata to float and colors
				if (!PinData)
				{
					PinData = NewObject<UCustomizableObjectNodeTableObjectPinData>(this);
					PinData->ColumnName_DEPRECATED = Pin->PinFriendlyName.ToString();

					AddPinData(*Pin, *PinData);
				}

				FProperty* ColumnProperty = FindTableProperty(FName(*PinData->ColumnName_DEPRECATED));

				if (!ColumnProperty)
				{
					continue;
				}

				FGuid ColumnPropertyId = FStructureEditorUtils::GetGuidForProperty(ColumnProperty);
				PinData->StructColumnId = ColumnPropertyId;

				// Store anim columns in the node instead of the pin
				if (UCustomizableObjectNodeTableMeshPinData* MeshPinData = Cast<UCustomizableObjectNodeTableMeshPinData>(GetPinData(*(Pin))))
				{
					if (!MeshPinData->AnimInstanceColumnName_DEPRECATED.IsEmpty())
					{
						FTableNodeColumnData NewColumnData;
						NewColumnData.AnimInstanceColumnName = MeshPinData->AnimInstanceColumnName_DEPRECATED;
						NewColumnData.AnimSlotColumnName = MeshPinData->AnimSlotColumnName_DEPRECATED;
						NewColumnData.AnimTagColumnName = MeshPinData->AnimTagColumnName_DEPRECATED;

						ColumnDataMap_DEPRECATED.Add(ColumnPropertyId, NewColumnData);
					}
				}
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeTablePinViewer)
	{
		for (UEdGraphPin* Pin : GetAllPins())
		{
			if (UCustomizableObjectNodeTableImagePinData* ImagePinData = Cast<UCustomizableObjectNodeTableImagePinData>(GetPinData(*Pin)))
			{
				ImagePinData->NodeTable = this;
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixAutomaticBlocksStrategyLegacyNodes)
	{
		for (UEdGraphPin* Pin : GetAllPins())
		{
			if (UCustomizableObjectNodeTableMeshPinData* MeshPinData = Cast<UCustomizableObjectNodeTableMeshPinData>(GetPinData(*Pin)))
			{
				for (UCustomizableObjectLayout* Layout : MeshPinData->Layouts)
				{
					if (Layout)
					{
						Layout->AutomaticBlocksStrategy = ECustomizableObjectLayoutAutomaticBlocksStrategy::Ignore;
					}
				}
			}
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SetDisplayNamePropertyAsPinNameOfTableNodes)
	{
		TObjectPtr<UScriptStruct> LoadedStructure = UE::Mutable::Private::LoadObject(Structure);
		if (LoadedStructure)
		{
			ConditionalPostLoadReference(*LoadedStructure);
		}

		TObjectPtr<UDataTable> LoadedTable = UE::Mutable::Private::LoadObject(Table);
		if (LoadedTable)
		{
			ConditionalPostLoadReference(*LoadedTable);
		}
		
		// We asume that until now we only supported UserDefinedStructs (Structure assets generated trhough the Editor)
		const UUserDefinedStruct* UserStruct = Cast<UUserDefinedStruct>(GetTableNodeStruct());

		if (UserStruct)
		{
			for (UEdGraphPin* Pin : Pins)
			{
				if (!Pin)
				{
					continue;
				}

				UCustomizableObjectNodeTableObjectPinData* PinData = Cast<UCustomizableObjectNodeTableObjectPinData>(GetPinData(*Pin));

				if (!PinData)
				{
					continue;
				}

				if (const FProperty* Property = FStructureEditorUtils::GetPropertyByGuid(UserStruct, PinData->StructColumnId))
				{
					// Using the standard naming of the property
					PinData->ColumnPropertyName = Property->GetAuthoredName();
					PinData->ColumnDisplayName = Property->GetDisplayNameText().ToString();

					if (ColumnDataMap_DEPRECATED.Contains(PinData->StructColumnId))
					{
						const FTableNodeColumnData* ColumnData = ColumnDataMap_DEPRECATED.Find(PinData->StructColumnId);
						PinColumnDataMap.Add(PinData->ColumnDisplayName, *ColumnData);

						// Note: We do not need to convert the content of the map because the display name of an editor structure is the same than its property name
					}
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "BackwardCompatibleFixUp Problem: Property doesn't exist");
				}
			}
		}
	}
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::TableNoneOptionsMovedToUnrealCode)
	{
		// set this option to true for already generated pins with the none option.
		bUseMaterialColor = bAddNoneOption;
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddedTableNodeCompilationFilters)
	{
		if (bDisableCheckedRows_DEPRECATED)
		{
			TObjectPtr<UScriptStruct> LoadedStructure = UE::Mutable::Private::LoadObject(Structure);
			if (LoadedStructure)
			{
				ConditionalPostLoadReference(*LoadedStructure);
			}

			TObjectPtr<UDataTable> LoadedTable = UE::Mutable::Private::LoadObject(Table);
			if (LoadedTable)
			{
				ConditionalPostLoadReference(*LoadedTable);
			}

			const UScriptStruct* TableStruct = GetTableNodeStruct();
			FBoolProperty* BoolProperty = nullptr;

			for (TFieldIterator<FProperty> PropertyIt(TableStruct); PropertyIt; ++PropertyIt)
			{
				BoolProperty = CastField<FBoolProperty>(*PropertyIt);

				if (BoolProperty)
				{
					CompilationFilterColumn_DEPRECATED = FName(*BoolProperty->GetDisplayNameText().ToString());
					CompilationFilters_DEPRECATED.Add("false");

					// There should be only one Bool column
					break;
				}
			}

		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::TableNodeCompilationFilterArray)
	{
		if (!CompilationFilterColumn_DEPRECATED.IsNone())
		{
			CompilationFilterOptions.Add({ CompilationFilterColumn_DEPRECATED, CompilationFilters_DEPRECATED, FilterOperationType_DEPRECATED });
		}
	}

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::SetDisplayNamePropertyAsPinNameOfTableNodes2)
	{
		BackwardsCompatibleFixup(FCustomizableObjectCustomVersion::SetDisplayNamePropertyAsPinNameOfTableNodes);
	}
}


void UCustomizableObjectNodeTable::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	TObjectPtr<UDataTable> LoadedTable = UE::Mutable::Private::LoadObject(Table);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTable, Table) &&
		LoadedTable)
	{
		LoadedTable->OnDataTableChanged().Remove(OnTableChangedDelegateHandle);
	}
}


FText UCustomizableObjectNodeTable::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("ParamName"), FText::FromString(ParameterName));

	if (TitleType == ENodeTitleType::ListView)
	{
		return LOCTEXT("Mutable_Table_Title", "Table");
	}
	else if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::Format(LOCTEXT("TableNode_EditableTitle", "{ParamName}"), Args);
	}
	else
	{
		if (TObjectPtr<UDataTable> LoadedTable = UE::Mutable::Private::LoadObject(Table))
		{
			Args.Add(TEXT("TableName"), FText::FromString(LoadedTable->GetName()));
			return FText::Format(LOCTEXT("TableNode_Title_DataTable", "{ParamName}\n{TableName} - Data Table"), Args);
		}
		else if (TObjectPtr<UScriptStruct> LoadedStructure = UE::Mutable::Private::LoadObject(Structure))
		{
			Args.Add(TEXT("StructureName"), FText::FromString(LoadedStructure->GetName()));
			return FText::Format(LOCTEXT("TableNode_Title_ScriptedStruct", "{ParamName}\n{StructureName} - Script Struct"), Args);
		}
	}

	return LOCTEXT("Mutable_Table", "Table");
}


FLinearColor UCustomizableObjectNodeTable::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Object);
}


FText UCustomizableObjectNodeTable::GetTooltipText() const
{
	return LOCTEXT("Node_Table_Tooltip", "Represents all the columns of Data Table asset.");
}


void UCustomizableObjectNodeTable::OnRenameNode(const FString& NewName)
{
	if (!NewName.IsEmpty())
	{
		ParameterName = NewName;
	}
}


void UCustomizableObjectNodeTable::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;

	if (PropertyThatChanged)
	{
		if (PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTable, Table) || PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTable, Structure))
		{
			ParamUIMetadataColumn = NAME_None;
			VersionColumn = NAME_None;
			ReconstructNode();
		}
		else if (PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTable, TableDataGatheringMode))
		{
			if (TObjectPtr<UDataTable> LoadedTable = UE::Mutable::Private::LoadObject(Table))
			{
				LoadedTable->OnDataTableChanged().Remove(OnTableChangedDelegateHandle);
			}

			Table = nullptr;
			Structure = nullptr;

			FilterPaths.Empty();

			ReconstructNode();
		}
	}
}


void UCustomizableObjectNodeTable::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Reset of the column data map
	TMap<FString, FTableNodeColumnData> AuxOldColumnData = PinColumnDataMap;
	PinColumnDataMap.Reset();

	// Getting Struct Pointer
	const UScriptStruct* TableStruct = GetTableNodeStruct();

	if (!TableStruct)
	{
		return;
	}

	NumProperties = GetColumnTitles().Num();

	// Getting Default Struct Values
	// A Script Struct always has at leaset one property
	TArray<int8> DefaultDataArray;
	DefaultDataArray.SetNumZeroed(TableStruct->GetStructureSize());
	TableStruct->InitializeStruct(DefaultDataArray.GetData());

	static TArray<TObjectPtr<UClass>> SupportedSoftObjectTypes = {
		USkeletalMesh::StaticClass(),
		UStaticMesh::StaticClass(),
		UTexture2D::StaticClass(),
		UTexture::StaticClass(),
		UMaterialInterface::StaticClass(),
		UPoseAsset::StaticClass()
	};

	TArray<UEdGraphPin*> OldPins(Pins);
	TSet<FString> PinNames;
	bool bHasRepeatedNames = false;
	
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

		UEdGraphPin* OutPin = nullptr;
		FString PinName = ColumnProperty->GetDisplayNameText().ToString();
		FString PropertyName = ColumnProperty->GetAuthoredName();
		FGuid PropertyID = FStructureEditorUtils::GetGuidForProperty(ColumnProperty);

		// Checking if there are repeated pin names (Not supported).
		if (PinNames.Contains(PinName))
		{
			bHasRepeatedNames = true;
			continue;
		}

		PinNames.Add(PinName);

		if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
		{
			// Only process object properties that might have pointers to objects of any of the "SupportedSoftObjectTypes"
			const bool bPotentiallySupportedObject = SoftObjectProperty->PropertyClass && SupportedSoftObjectTypes.ContainsByPredicate([SoftObjectProperty](const TObjectPtr<UClass>& Type)
			{
				return SoftObjectProperty->PropertyClass->IsChildOf(Type) || Type->IsChildOf(SoftObjectProperty->PropertyClass);
			});

			if (bPotentiallySupportedObject)
			{
				UObject* Object = nullptr;

				// Getting default UObject
				uint8* CellData = SoftObjectProperty->ContainerPtrToValuePtr<uint8>(DefaultDataArray.GetData());

				if (CellData)
				{
					Object = UE::Mutable::Private::LoadObject(SoftObjectProperty->GetPropertyValue(CellData));
				}

				if (Object)
				{
					if (Object->IsA(USkeletalMesh::StaticClass()) || Object->IsA(UStaticMesh::StaticClass()))
					{
						if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
						{
							const int NumLODs = SkeletalMesh->GetLODNum();

							for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
							{
								int32 NumMaterials = SkeletalMesh->GetImportedModel()->LODModels[LODIndex].Sections.Num();

								for (int32 MatIndex = 0; MatIndex < NumMaterials; ++MatIndex)
								{
									FString TableMeshPinName = GenerateSkeletalMeshMutableColumName(PinName, LODIndex, MatIndex);

									// Pin Data
									UCustomizableObjectNodeTableMeshPinData* PinData = NewObject<UCustomizableObjectNodeTableMeshPinData>(this);
									PinData->ColumnPropertyName = PropertyName;
									PinData->ColumnDisplayName = PinName;
									PinData->StructColumnId = PropertyID;
 
									// Mesh Data
									PinData->LOD = LODIndex;
									PinData->Material = MatIndex;

									// Create a new pin for each lod and mesh section
									UEdGraphPin* MeshPin = CustomCreatePin(EGPD_Output, Schema->PC_Mesh, FName(*TableMeshPinName), FText::FromString(TableMeshPinName), PinData);
									MeshPin->SafeSetHidden(false);
									MeshPin->PinToolTip = TableMeshPinName;

									FSkeletalMeshModel* importModel = SkeletalMesh->GetImportedModel();
									if (importModel && importModel->LODModels.Num() > LODIndex)
									{
										const uint32 NumberOfUVLayouts = importModel->LODModels[LODIndex].NumTexCoords;

										for (uint32 LayoutIndex = 0; LayoutIndex < NumberOfUVLayouts; ++LayoutIndex)
										{
											UCustomizableObjectLayout* Layout = NewObject<UCustomizableObjectLayout>(this);
											FString LayoutName = TableMeshPinName;

											if (NumberOfUVLayouts > 1)
											{
												LayoutName += FString::Printf(TEXT(" UV_%d"), LayoutIndex);
											}

											if (Layout)
											{
												PinData->Layouts.Add(Layout);

												Layout->SetLayout(LODIndex, MatIndex, LayoutIndex);
												Layout->SetLayoutName(LayoutName);
											}
										}
									}
								}
							}
						}

						else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
						{
							if (StaticMesh->GetRenderData()->LODResources.Num())
							{
								int32 NumMaterials = StaticMesh->GetRenderData()->LODResources[0].Sections.Num();

								for (int32 MatIndex = 0; MatIndex < NumMaterials; ++MatIndex)
								{
									FString TableMeshPinName = GenerateStaticMeshMutableColumName(PinName, MatIndex);

									UCustomizableObjectNodeTableMeshPinData* PinData = NewObject<UCustomizableObjectNodeTableMeshPinData>(this);
									PinData->ColumnPropertyName = PropertyName;
									PinData->ColumnDisplayName = PinName;
									PinData->StructColumnId = PropertyID;

									// Mesh Data
									PinData->LOD = 0;
									PinData->Material = MatIndex;

									// Create a new pin for each lod and mesh section
									UEdGraphPin* MeshPin = CustomCreatePin(EGPD_Output, Schema->PC_Mesh, FName(*TableMeshPinName), FText::FromString(TableMeshPinName), PinData);
									MeshPin->SafeSetHidden(false);
									MeshPin->PinToolTip = TableMeshPinName;
								}
							}
						}

						if (FTableNodeColumnData* ColumnData = AuxOldColumnData.Find(PinName))
						{
							PinColumnDataMap.Add(PinName, *ColumnData);
						}
					}

					else if (Object->IsA(UTexture2D::StaticClass()))
					{
						UCustomizableObjectNodeTableImagePinData* PinData = NewObject<UCustomizableObjectNodeTableImagePinData>(this);
						PinData->ColumnPropertyName = PropertyName;
						PinData->ColumnDisplayName = PinName;
						PinData->StructColumnId = PropertyID;

						// Texture Data
						PinData->bIsNotTexture2D = false;
						PinData->NodeTable = this;

						FName PinCategory = PinData->ImageMode == ETableTextureType::PASSTHROUGH_TEXTURE ? Schema->PC_Texture_Passthrough : Schema->PC_Texture;

						for (UEdGraphPin* OldPin : OldPins)
						{
							// Checking if this column already exist
							if (UCustomizableObjectNodeTableImagePinData* OldPinData = Cast<UCustomizableObjectNodeTableImagePinData>(GetPinData(*(OldPin))))
							{
								if (OldPinData->ColumnPropertyName == PropertyName)
								{
									PinCategory = OldPinData->ImageMode == ETableTextureType::PASSTHROUGH_TEXTURE ? Schema->PC_Texture_Passthrough : Schema->PC_Texture;
									break;
								}
							}
						}

						OutPin = CustomCreatePin(EGPD_Output, PinCategory, FName(*PinName), FText::FromString(PinName), PinData);
					}

					else if (Object->IsA(UTexture::StaticClass()))
					{
						UCustomizableObjectNodeTableImagePinData* PinData = NewObject<UCustomizableObjectNodeTableImagePinData>(this);
						PinData->ColumnPropertyName = PropertyName;
						PinData->ColumnDisplayName = PinName;
						PinData->StructColumnId = PropertyID;

						// Texture Data
						PinData->ImageMode = ETableTextureType::PASSTHROUGH_TEXTURE;
						PinData->bIsNotTexture2D = true;
						PinData->NodeTable = this;

						OutPin = CustomCreatePin(EGPD_Output, Schema->PC_Texture_Passthrough, FName(*PinName), FText::FromString(PinName), PinData);
					}

					else if (Object->IsA(UMaterialInterface::StaticClass()))
					{
						UCustomizableObjectNodeTableObjectPinData* PinData = NewObject<UCustomizableObjectNodeTableObjectPinData>(this);
						PinData->ColumnPropertyName = PropertyName;
						PinData->ColumnDisplayName = PinName;
						PinData->StructColumnId = PropertyID;

						OutPin = CustomCreatePin(EGPD_Output, Schema->PC_Material, FName(*PinName), FText::FromString(PinName), PinData);
					}

					else if (Object->IsA(UPoseAsset::StaticClass()))
					{
						UCustomizableObjectNodeTableObjectPinData* PinData = NewObject<UCustomizableObjectNodeTableObjectPinData>(this);
						PinData->ColumnPropertyName = PropertyName;
						PinData->ColumnDisplayName = PinName;
						PinData->StructColumnId = PropertyID;

						OutPin = CustomCreatePin(EGPD_Output, Schema->PC_PoseAsset, FName(*PinName), FText::FromString(PinName), PinData);
					}
				}
				else
				{
					const FText Text = FText::FromString(FString::Printf(TEXT("Could not find a Default Value in Structure member [%s]"), *PinName));

					FCustomizableObjectEditorLogger::CreateLog(Text)
						.Category(ELoggerCategory::General)
						.Severity(EMessageSeverity::Warning)
						.Context(*this)
						.CustomizableObject(GraphTraversal::GetObject(*this))
						.Log();
				}
			}
		}

		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ColumnProperty))
		{
			if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
			{
				UCustomizableObjectNodeTableObjectPinData* PinData = NewObject<UCustomizableObjectNodeTableObjectPinData>(this);
				PinData->ColumnPropertyName = PropertyName;
				PinData->ColumnDisplayName = PinName;
				PinData->StructColumnId = PropertyID;

				OutPin = CustomCreatePin(EGPD_Output, Schema->PC_Color, FName(*PinName), FText::FromString(PinName), PinData);
			}
		}

		else if (const FNumericProperty* NumFloatProperty = CastField<FFloatProperty>(ColumnProperty))
		{
			UCustomizableObjectNodeTableObjectPinData* PinData = NewObject<UCustomizableObjectNodeTableObjectPinData>(this);
			PinData->ColumnPropertyName = PropertyName;
			PinData->ColumnDisplayName = PinName;
			PinData->StructColumnId = PropertyID;

			OutPin = CustomCreatePin(EGPD_Output, Schema->PC_Float, FName(*PinName), FText::FromString(PinName), PinData);
		}
		
		else if (const FNumericProperty* NumDoubleProperty = CastField<FDoubleProperty>(ColumnProperty))
		{
			UCustomizableObjectNodeTableObjectPinData* PinData = NewObject<UCustomizableObjectNodeTableObjectPinData>(this);
			PinData->ColumnPropertyName = PropertyName;
			PinData->ColumnDisplayName = PinName;
			PinData->StructColumnId = PropertyID;

			OutPin = CustomCreatePin(EGPD_Output, Schema->PC_Float, FName(*PinName), FText::FromString(PinName), PinData);
		}

		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ColumnProperty))
		{
			const FText Text = FText::FromString(FString::Printf(TEXT("Asset format not supported in Structure member [%s]. All assets should be Soft References."), *PinName));

			FCustomizableObjectEditorLogger::CreateLog(Text)
				.Category(ELoggerCategory::General)
				.Severity(EMessageSeverity::Warning)
				.Context(*this)
				.CustomizableObject(GraphTraversal::GetObject(*this))
				.Log();
		}

		if (OutPin)
		{
			OutPin->PinToolTip = PinName;
			OutPin->SafeSetHidden(false);
		}
	}

	TableStruct->DestroyStruct(DefaultDataArray.GetData());

	// Repeated names log
	if (bHasRepeatedNames)
	{
		FCustomizableObjectEditorLogger::CreateLog(FText::Format(LOCTEXT("ContainsRepeatedNamesWarning", "Table Node '{0}' contains 2 or more columns with repeated display names."
			"The pins of these columns will not be available"), FText::FromString(ParameterName)))
			.BaseObject()
			.Severity(EMessageSeverity::Warning)
			.Context(*this)
			.CustomizableObject(GraphTraversal::GetObject(*this))
			.Log();
	}
}


bool UCustomizableObjectNodeTable::IsNodeOutDatedAndNeedsRefresh()
{
	// Getting Struct Pointer
	const UScriptStruct* TableStruct = GetTableNodeStruct();

	if (!TableStruct)
	{
		return Pins.Num() > 0;
	}

	if (NumProperties != GetColumnTitles().Num())
	{
		return true;
	}

	// Getting Default Struct Values
	// A Script Struct always has at leaset one property
	TArray<int8> DefaultDataArray;
	DefaultDataArray.SetNumZeroed(TableStruct->GetStructureSize());
	TableStruct->InitializeStruct(DefaultDataArray.GetData());

	int32  NumPins = 0;

	bool bNeedsUpdate = false;

	for (TFieldIterator<FProperty> It(TableStruct); It && !bNeedsUpdate; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (ColumnProperty)
		{
			const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
			const FString& PinName = ColumnProperty->GetDisplayNameText().ToString();

			if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
			{
				UObject* Object = nullptr;

				// Getting default UObject
				uint8* CellData = SoftObjectProperty->ContainerPtrToValuePtr<uint8>(DefaultDataArray.GetData());

				if (CellData)
				{
					Object = UE::Mutable::Private::LoadObject(SoftObjectProperty->GetPropertyValue(CellData));
				}

				if (!Object)
				{
					continue;
				}

				if (Object->IsA(USkeletalMesh::StaticClass()) || Object->IsA(UStaticMesh::StaticClass()))
				{
					if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Object))
					{
						const int NumLODs = SkeletalMesh->GetLODNum();

						for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
						{
							int32 NumMaterials = SkeletalMesh->GetImportedModel()->LODModels[LODIndex].Sections.Num();

							for (int32 MatIndex = 0; MatIndex < NumMaterials; ++MatIndex)
							{
								FString MeshPinName = GenerateSkeletalMeshMutableColumName(PinName, LODIndex, MatIndex);

								if (CheckPinUpdated(MeshPinName, Schema->PC_Mesh))
								{
									bNeedsUpdate = true;
								}

								NumPins++;
							}
						}
					}
					else if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
					{
						if (StaticMesh->GetRenderData()->LODResources.Num())
						{
							int32 NumMaterials = StaticMesh->GetRenderData()->LODResources[0].Sections.Num();

							for (int32 MatIndex = 0; MatIndex < NumMaterials; ++MatIndex)
							{
								FString MeshPinName = GenerateStaticMeshMutableColumName(PinName, MatIndex);

								if (CheckPinUpdated(MeshPinName, Schema->PC_Mesh))
								{
									bNeedsUpdate = true;
								}

								NumPins++;
							}
						}
					}
				}
				else if (Object->IsA(UTexture2D::StaticClass()))
				{
					if (CheckPinUpdated(PinName, Schema->PC_Texture) && CheckPinUpdated(PinName, Schema->PC_Texture_Passthrough))
					{
						bNeedsUpdate = true;
					}

					NumPins++;
				}
				else if (Object->IsA(UTexture2DArray::StaticClass()))
				{
					if (CheckPinUpdated(PinName, Schema->PC_Texture_Passthrough))
					{
						bNeedsUpdate = true;
					}

					NumPins++;
				}
				else if (Object->IsA(UMaterialInterface::StaticClass()))
				{
					if (CheckPinUpdated(PinName, Schema->PC_Material))
					{
						bNeedsUpdate = true;
					}

					NumPins++;
				}
				else if (Object->IsA(UPoseAsset::StaticClass()))
				{
					if (CheckPinUpdated(PinName, Schema->PC_PoseAsset))
					{
						bNeedsUpdate = true;
					}

					NumPins++;
				}
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ColumnProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
				{
					if (CheckPinUpdated(PinName, Schema->PC_Color))
					{
						bNeedsUpdate = true;
					}

					NumPins++;
				}
			}
			else if (const FFloatProperty* FloatNumProperty = CastField<FFloatProperty>(ColumnProperty))
			{
				if (CheckPinUpdated(PinName, Schema->PC_Float))
				{
					bNeedsUpdate = true;
				}

				NumPins++;
			}
			else if (const FDoubleProperty* DoubleNumProperty = CastField<FDoubleProperty>(ColumnProperty))
			{
				if (CheckPinUpdated(PinName, Schema->PC_Float))
				{
					bNeedsUpdate = true;
				}

				NumPins++;
			}
		}
	}

	TableStruct->DestroyStruct(DefaultDataArray.GetData());

	if (Pins.Num() != NumPins)
	{
		bNeedsUpdate = true;
	}

	return bNeedsUpdate;
}


FString UCustomizableObjectNodeTable::GetRefreshMessage() const
{
	return "Node data outdated. Please refresh node.";
}

// TODO(MTBL-1652): Move this to the Copy method of the NodePinData class which is more PinType specific (and the right plece to do this)
void UCustomizableObjectNodeTable::RemapPinsData(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap)
{
	Super::RemapPinsData(PinsToRemap);
	
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	for (const TTuple<UEdGraphPin*, UEdGraphPin*>& Pair : PinsToRemap)
	{
		if (Pair.Key->PinType.PinCategory == Schema->PC_Mesh)
		{
			UCustomizableObjectNodeTableMeshPinData* PinDataOldPin = Cast<UCustomizableObjectNodeTableMeshPinData>(GetPinData(*(Pair.Key)));
			UCustomizableObjectNodeTableMeshPinData* PinDataNewPin = Cast<UCustomizableObjectNodeTableMeshPinData>(GetPinData(*(Pair.Value)));

			const UScriptStruct* ScriptStruct = GetTableNodeStruct();

			if (PinDataOldPin && PinDataNewPin && ScriptStruct && GetPinMeshType(Pair.Value) == ETableMeshPinType::SKELETAL_MESH)
			{
				// Keeping information added in layout editor if the layout is the same
				for (TObjectPtr<UCustomizableObjectLayout>& NewLayout : PinDataNewPin->Layouts)
				{
					for (TObjectPtr<UCustomizableObjectLayout>& OldLayout : PinDataOldPin->Layouts)
					{
						if (NewLayout->GetLayoutName() == OldLayout->GetLayoutName())
						{
							NewLayout->Blocks = OldLayout->Blocks;
							NewLayout->SetGridSize(OldLayout->GetGridSize());
							NewLayout->SetMaxGridSize(OldLayout->GetMaxGridSize());
							NewLayout->PackingStrategy = OldLayout->PackingStrategy;
							NewLayout->AutomaticBlocksStrategy = OldLayout->AutomaticBlocksStrategy;
							NewLayout->AutomaticBlocksMergeStrategy = OldLayout->AutomaticBlocksMergeStrategy;
							NewLayout->BlockReductionMethod = OldLayout->BlockReductionMethod;

							break;
						}
					}
				}
			}
		}
		else if (Pair.Key->PinType.PinCategory == Schema->PC_Texture || Pair.Key->PinType.PinCategory == Schema->PC_Texture_Passthrough)
		{
			UCustomizableObjectNodeTableImagePinData* PinDataOldPin = Cast<UCustomizableObjectNodeTableImagePinData>(GetPinData(*(Pair.Key)));
			UCustomizableObjectNodeTableImagePinData* PinDataNewPin = Cast<UCustomizableObjectNodeTableImagePinData>(GetPinData(*(Pair.Value)));

			if (PinDataOldPin && PinDataNewPin)
			{
				PinDataNewPin->ImageMode = PinDataOldPin->ImageMode;
				PinDataNewPin->bIsNotTexture2D = PinDataOldPin->bIsNotTexture2D;
			}
		}
	}
}


bool UCustomizableObjectNodeTable::ProvidesCustomPinRelevancyTest() const
{
	return true;
}


bool UCustomizableObjectNodeTable::IsPinRelevant(const UEdGraphPin* Pin) const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	
	return Pin->Direction == EGPD_Input &&
		(Pin->PinType.PinCategory == Schema->PC_Material ||
		Pin->PinType.PinCategory == Schema->PC_Texture ||
		Pin->PinType.PinCategory == Schema->PC_Texture_Passthrough ||
		Pin->PinType.PinCategory == Schema->PC_Texture_Object ||
		Pin->PinType.PinCategory == Schema->PC_Color ||
		Pin->PinType.PinCategory == Schema->PC_Float ||
		Pin->PinType.PinCategory == Schema->PC_Mesh);
}


UTexture2D* UCustomizableObjectNodeTable::FindReferenceTextureParameter(const UEdGraphPin* Pin, FString ParameterImageName) const
{
	UMaterialInterface* Material = GetColumnDefaultAssetByType<UMaterialInterface>(Pin);

	if (Material)
	{
		UTexture* OutTexture = nullptr;
		bool bFound = Material->GetTextureParameterValue(FName(*ParameterImageName), OutTexture);

		if (bFound)
		{
			if (UTexture2D* Texture = Cast<UTexture2D>(OutTexture))
			{
				return Texture;
			}
		}
	}

	return nullptr;
}


UStreamableRenderAsset* UCustomizableObjectNodeTable::GetDefaultMeshForLayout(const UCustomizableObjectLayout* InLayout) const
{
	for (const UEdGraphPin* Pin : Pins)
	{
		if (const UCustomizableObjectNodeTableMeshPinData* MeshPinData = Cast<UCustomizableObjectNodeTableMeshPinData >(GetPinData(*Pin)))
		{
			for (const UCustomizableObjectLayout* Layout : MeshPinData->Layouts)
			{
				if (Layout == InLayout)
				{
					return GetColumnDefaultAssetByType<UStreamableRenderAsset>(MeshPinData->ColumnPropertyName);
				}
			}
		}
	}

	return nullptr;
}


TArray<UCustomizableObjectLayout*> UCustomizableObjectNodeTable::GetLayouts(const UEdGraphPin* Pin) const
{
	TArray<UCustomizableObjectLayout*> Result;

	const UCustomizableObjectNodeTableMeshPinData* PinData = Cast<UCustomizableObjectNodeTableMeshPinData >(GetPinData(*Pin));

	if (PinData)
	{
		for (int32 LayoutIndex = 0; LayoutIndex < PinData->Layouts.Num(); ++LayoutIndex)
		{
			Result.Add(PinData->Layouts[LayoutIndex]);
		}
	}
	
	return Result;
}


FString UCustomizableObjectNodeTable::GetPinColumnName(const UEdGraphPin* Pin) const
{
	const UCustomizableObjectNodeTableObjectPinData* PinData = Cast<UCustomizableObjectNodeTableObjectPinData>(GetPinData(*Pin));

	if (PinData)
	{
		return PinData->ColumnDisplayName;
	}

	return FString();
}


void UCustomizableObjectNodeTable::GetPinLODAndSection(const UEdGraphPin* Pin, int32& LODIndex, int32& SectionIndex) const
{
	if (const UCustomizableObjectNodeTableMeshPinData* PinData = Cast<UCustomizableObjectNodeTableMeshPinData >(GetPinData(*Pin)))
	{
		LODIndex = PinData->LOD;
		SectionIndex = PinData->Material;
	}
	else
	{
		LODIndex = -1;
		SectionIndex = -1;
	}
}


void UCustomizableObjectNodeTable::GetAnimationColumns(const FString& ColumnDisplayName, FString& AnimBPColumnName, FString& AnimSlotColumnName, FString& AnimTagColumnName) const
{
	if (PinColumnDataMap.Contains(ColumnDisplayName))
	{
		const FTableNodeColumnData& ColumnData = PinColumnDataMap[ColumnDisplayName];

		AnimBPColumnName = ColumnData.AnimInstanceColumnName;
		AnimSlotColumnName = ColumnData.AnimSlotColumnName;
		AnimTagColumnName = ColumnData.AnimTagColumnName;
	}
}


bool UCustomizableObjectNodeTable::CheckPinUpdated(const FString& PinName, const FName& PinType) const
{
	if (UEdGraphPin* Pin = FindPin(PinName))
	{
		if (Pin->PinType.PinCategory != PinType)
		{
			return true;
		}
	}
	else
	{
		return true;
	}

	return false;
}


FSoftObjectPtr UCustomizableObjectNodeTable::GetSkeletalMeshAt(const UEdGraphPin* Pin, const UDataTable* DataTable, const FName& RowName) const
{
	if (!DataTable || !DataTable->GetRowStruct() || !Pin || !DataTable->GetRowNames().Contains(RowName))
	{
		return {};
	}
	
	const UScriptStruct* TableStruct = DataTable->GetRowStruct();

	// Here we are using the DataTable function because we are only calling GetSkeletalMeshAt() at Generation Time
	// If we want to use it elswhere, we can use our FindTablreProperty method
	FProperty* ColumnProperty = FindPinProperty(*Pin);

	if (!ColumnProperty)
	{
		return {};
	}

	if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
	{
		if (uint8* RowData = DataTable->FindRowUnchecked(RowName))
		{
			if (uint8* CellData = ColumnProperty->ContainerPtrToValuePtr<uint8>(RowData, 0))
			{
				return SoftObjectProperty->GetPropertyValue(CellData);
			}
		}
	}

	return {};
}


TSoftClassPtr<UAnimInstance> UCustomizableObjectNodeTable::GetAnimInstanceAt(const UEdGraphPin* Pin, const UDataTable* DataTable, const FName& RowName) const
{
	if (!DataTable || !DataTable->GetRowStruct() || !Pin || !DataTable->GetRowNames().Contains(RowName))
	{
		return TSoftClassPtr<UAnimInstance>();
	}

	const UCustomizableObjectNodeTableObjectPinData* PinData = Cast<UCustomizableObjectNodeTableObjectPinData>(GetPinData(*Pin));

	if (!PinData)
	{
		return TSoftClassPtr<UAnimInstance>();
	}

	if (const FTableNodeColumnData* ColumnData = PinColumnDataMap.Find(PinData->ColumnDisplayName))
	{
		FString AnimColumn = ColumnData->AnimInstanceColumnName;

		// Here we are using the DataTable function because we are only calling GetAnimInstanceAt() at Generation Time
		// If we want to use it elswhere, we can use our FindTablreProperty method
		FProperty* ColumnProperty = FindColumnProperty(FName(*AnimColumn));

		if (!ColumnProperty)
		{
			return TSoftClassPtr<UAnimInstance>();
		}

		if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(ColumnProperty))
		{
			if (uint8* RowData = DataTable->FindRowUnchecked(RowName))
			{
				if (uint8* CellData = ColumnProperty->ContainerPtrToValuePtr<uint8>(RowData, 0))
				{
					TSoftClassPtr<UAnimInstance> AnimInstance(SoftClassProperty->GetPropertyValue(CellData).ToSoftObjectPath());

					if (!AnimInstance.IsNull())
					{
						return AnimInstance;
					}
				}
			}
		}
	}

	return TSoftClassPtr<UAnimInstance>();
}


void UCustomizableObjectNodeTableImagePinData::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddedAnyTextureTypeToPassThroughTextures)
	{
		if (bIsArrayTexture_DEPRECATED)
		{
			bIsNotTexture2D = true;
			bIsArrayTexture_DEPRECATED = false;
		}
	}
}


void UCustomizableObjectNodeTableImagePinData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const FProperty* PropertyThatChanged = PropertyChangedEvent.Property)
	{
		if (PropertyThatChanged->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTableImagePinData, ImageMode))
		{
			NodeTable->ReconstructNode();
		}
	}
}


bool UCustomizableObjectNodeTableImagePinData::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTableImagePinData, ImageMode))
	{
		return !NodeTable->GetPin(*this)->LinkedTo.Num() && !bIsNotTexture2D;
	}
	
	return Super::CanEditChange(InProperty);
}


UCustomizableObjectNodeTableRemapPins* UCustomizableObjectNodeTable::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeTableRemapPins>();
}


bool UCustomizableObjectNodeTable::HasPinViewer() const
{
	return true;
}


TSharedPtr<IDetailsView> UCustomizableObjectNodeTable::CustomizePinDetails(const UEdGraphPin& Pin) const
{
	if (UCustomizableObjectNodeTableImagePinData* PinData = Cast<UCustomizableObjectNodeTableImagePinData>(GetPinData(Pin)))
	{
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		
		const TSharedRef<IDetailsView> SettingsView = EditModule.CreateDetailView(DetailsViewArgs);
		SettingsView->SetObject(PinData);
		
		return SettingsView;
	}
	else
	{
		return nullptr;
	}
}


ETableTextureType UCustomizableObjectNodeTable::GetColumnImageMode(const FString& ColumnPropertyName) const
{
	for (const UEdGraphPin* Pin : Pins)
	{
		if (const UCustomizableObjectNodeTableImagePinData* PinData = Cast<UCustomizableObjectNodeTableImagePinData>(GetPinData(*(Pin))))
		{
			if (PinData->ColumnPropertyName == ColumnPropertyName)
			{
				return PinData->ImageMode;
			}
		}
	}

	unimplemented();
	return {};
}


ETableMeshPinType UCustomizableObjectNodeTable::GetPinMeshType(const UEdGraphPin* Pin) const
{
	if ( Pin && Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Mesh)
	{
		FProperty* Property = FindPinProperty(*Pin);

		if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
		{
			if (SoftObjectProperty->PropertyClass->IsChildOf(USkeletalMesh::StaticClass()))
			{
				return ETableMeshPinType::SKELETAL_MESH;
			}
			else if (SoftObjectProperty->PropertyClass->IsChildOf(UStaticMesh::StaticClass()))
			{
				return ETableMeshPinType::STATIC_MESH;
			}
		}
	}

	return ETableMeshPinType::NONE;
}


FString UCustomizableObjectNodeTable::GenerateSkeletalMeshMutableColumName(const FString& PinName, int32 LODIndex, int32 MaterialIndex) const
{
	return PinName + FString::Printf(TEXT(" LOD_%d "), LODIndex) + FString::Printf(TEXT("Mat_%d"), MaterialIndex);
}


FString UCustomizableObjectNodeTable::GenerateStaticMeshMutableColumName(const FString& PinName, int32 MaterialIndex) const
{
	return PinName + FString::Printf(TEXT(" Mat_%d"), MaterialIndex);
}


const UScriptStruct* UCustomizableObjectNodeTable::GetTableNodeStruct() const
{
	const UScriptStruct* TableStruct = nullptr;

	if (TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry)
	{
		if (TObjectPtr<UScriptStruct> LoadedStructure = UE::Mutable::Private::LoadObject(Structure))
		{
			TableStruct = LoadedStructure;
		}
	}
	else
	{
		if (TObjectPtr<UDataTable> LoadedTable = UE::Mutable::Private::LoadObject(Table))
		{
			TableStruct = LoadedTable->GetRowStruct();
		}
	}

	return TableStruct;
}


TArray<FString> UCustomizableObjectNodeTable::GetColumnTitles() const
{
	TArray<FString> Result;
	Result.Add(TEXT("Name"));

	const UScriptStruct* TableStruct = GetTableNodeStruct();

	if (TableStruct)
	{
		for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
		{
			FProperty* Prop = *It;
			check(Prop != nullptr);
			const FString DisplayName = DataTableUtils::GetPropertyExportName(Prop);
			Result.Add(DisplayName);
		}
	}

	return Result;
}


FProperty* UCustomizableObjectNodeTable::FindTableProperty(const FName& PropertyName) const
{
	const UScriptStruct* TableStruct = GetTableNodeStruct();
	
	if (!TableStruct)
	{
		return nullptr;
	}

	FProperty* Property = nullptr;

	Property = TableStruct->FindPropertyByName(PropertyName);
	if (Property == nullptr && TableStruct->IsA<UUserDefinedStruct>())
	{
		const FString PropertyNameStr = PropertyName.ToString();

		for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
		{
			if (PropertyNameStr == TableStruct->GetAuthoredNameForField(*It))
			{
				Property = *It;
				break;
			}
		}
	}

	if (!DataTableUtils::IsSupportedTableProperty(Property))
	{
		Property = nullptr;
	}

	return Property;
}


FProperty* UCustomizableObjectNodeTable::FindPinProperty(const UEdGraphPin& Pin) const
{
	if (UCustomizableObjectNodeTableObjectPinData* PinData = Cast<UCustomizableObjectNodeTableObjectPinData>(GetPinData(Pin)))
	{
		return FindTableProperty(FName(*PinData->ColumnPropertyName));
	}

	return nullptr;
}


FProperty* UCustomizableObjectNodeTable::FindColumnProperty(const FName& ColumnDisplayName) const
{
	const UScriptStruct* TableStruct = GetTableNodeStruct();

	if (!TableStruct)
	{
		return nullptr;
	}

	FProperty* Property = nullptr;

	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		if (ColumnDisplayName == FName(*It->GetDisplayNameText().ToString()))
		{
			Property = *It;
			break;
		}
	}

	if (!DataTableUtils::IsSupportedTableProperty(Property))
	{
		Property = nullptr;
	}

	return Property;
}


TArray<FAssetData> UCustomizableObjectNodeTable::GetParentTables() const
{
	TArray<FAssetData> DataTableAssets;
	if (TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.GetRegistry();

		FARFilter Filter;
		Filter.ClassPaths.Add(FTopLevelAssetPath(UDataTable::StaticClass()));
		Filter.bRecursivePaths = true;
		Filter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// Only query on-disk assets while cooking - ensures results are deterministic

		if (TObjectPtr<UScriptStruct> LoadedStructure = UE::Mutable::Private::LoadObject(Structure))
		{
			TArray<FName> ReferencedTables;
			AssetRegistry.Get()->GetReferencers(LoadedStructure.GetPackage().GetFName(), ReferencedTables, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::NoRequirements);

			for (const FName& ReferencedTable : ReferencedTables)
			{
				Filter.PackageNames.Add(ReferencedTable);
			}
		}

		for (int32 PathIndex = 0; PathIndex < FilterPaths.Num(); ++PathIndex)
		{
			Filter.PackagePaths.Add(FilterPaths[PathIndex]);
		}

		if (!Filter.IsEmpty())
		{
			AssetRegistry.Get()->GetAssets(Filter, DataTableAssets);
		}
	}

	return DataTableAssets;
}

const FSkelMeshSection* UCustomizableObjectNodeTable::GetDefaultSkeletalMeshSectionFor(const UEdGraphPin& MeshPin) const
{
	USkeletalMesh* SkeletalMesh = GetColumnDefaultAssetByType<USkeletalMesh>(&MeshPin);

	if (!SkeletalMesh)
	{
		return nullptr;
	}
	const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();

	if (!ImportedModel)
	{
		return nullptr;
	}

	int32 LODIndex;
	int32 SectionIndex;
	GetPinLODAndSection(&MeshPin, LODIndex, SectionIndex);
	
	if (!ImportedModel->LODModels.IsValidIndex(LODIndex))
	{
		return nullptr;
	}

	const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
	{
		return nullptr;
	}

	return &LODModel.Sections[SectionIndex];
}

FSkeletalMaterial* UCustomizableObjectNodeTable::GetDefaultSkeletalMaterialFor(const UEdGraphPin& MeshPin) const
{
	USkeletalMesh* SkeletalMesh = GetColumnDefaultAssetByType<USkeletalMesh>(&MeshPin);

	if (!SkeletalMesh)
	{
		return nullptr;
	}

	const int32 SkeletalMeshMaterialIndex = GetDefaultSkeletalMaterialIndexFor(MeshPin);
	if (SkeletalMesh->GetMaterials().IsValidIndex(SkeletalMeshMaterialIndex))
	{
		return &SkeletalMesh->GetMaterials()[SkeletalMeshMaterialIndex];
	}

	return nullptr;
}


int32 UCustomizableObjectNodeTable::GetDefaultSkeletalMaterialIndexFor(const UEdGraphPin& MeshPin) const
{
	USkeletalMesh* SkeletalMesh = GetColumnDefaultAssetByType<USkeletalMesh>(&MeshPin);

	if (!SkeletalMesh)
	{
		return INDEX_NONE;
	}

	int32 LODIndex;
	int32 SectionIndex;
	GetPinLODAndSection(&MeshPin, LODIndex, SectionIndex);

	// We assume that LODIndex and MaterialIndex are valid for the imported model
	int32 SkeletalMeshMaterialIndex = INDEX_NONE;

	// Check if we have LOD info map to get the correct material index
	if (const FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LODIndex))
	{
		if (LodInfo->LODMaterialMap.IsValidIndex(SectionIndex))
		{
			SkeletalMeshMaterialIndex = LodInfo->LODMaterialMap[SectionIndex];
		}
	}

	// Only deduce index when the explicit mapping is not found or there is no remap
	if (SkeletalMeshMaterialIndex == INDEX_NONE)
	{
		const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();

		if (ImportedModel && ImportedModel->LODModels.IsValidIndex(LODIndex) && ImportedModel->LODModels[LODIndex].Sections.IsValidIndex(SectionIndex))
		{
			SkeletalMeshMaterialIndex = ImportedModel->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex;
		}
	}

	return SkeletalMeshMaterialIndex;
}


uint8* UCustomizableObjectNodeTable::GetCellData(const FName& RowName, const UDataTable& DataTable, const FProperty& ColumnProperty)
{
	// Get Row Data
	uint8* RowData = DataTable.FindRowUnchecked(RowName);

	if (RowData)
	{
		return ColumnProperty.ContainerPtrToValuePtr<uint8>(RowData, 0);
	}

	return nullptr;
}


TArray<FName> UCustomizableObjectNodeTable::GetEnabledRows(const UDataTable& DataTable) const
{
	TArray<FName> RowNames;
	const UScriptStruct* TableStruct = DataTable.GetRowStruct();

	if (!TableStruct)
	{
		return TArray<FName>();
	}

	TArray<FName> TableRowNames = DataTable.GetRowNames();

	// Sort them to avoid cooked data indeterminism problems. Rows may come from different tables and their loading order
	// is not defined.
	TableRowNames.Sort([](const FName& A, const FName& B) { return A.ToString() < B.ToString(); });

	// Check if there is any filter condition
	if (CompilationFilterOptions.IsEmpty())
	{
		return TableRowNames;
	}

	RowNames.Reserve(TableRowNames.Num());

	for (int32 FilterIndex = 0; FilterIndex < CompilationFilterOptions.Num(); ++FilterIndex)
	{
		FProperty* ColumnProperty = FindColumnProperty(CompilationFilterOptions[FilterIndex].FilterColumn);

		if (!ColumnProperty)
		{
			continue;
		}

		if (UCustomizableObjectNodeTable::SupportedFilterTypes.Contains(ColumnProperty->GetClass()))
		{
			for (int32 RowIndex = 0; RowIndex < TableRowNames.Num(); ++RowIndex)
			{
				if (uint8* CellData = UCustomizableObjectNodeTable::GetCellData(TableRowNames[RowIndex], DataTable, *ColumnProperty))
				{
					// By getting values as Texts we enable the support to FText columns
					FString Value = DataTableUtils::GetPropertyValueAsTextDirect(ColumnProperty, CellData).ToString();

					if (CompilationFilterOptions[FilterIndex].Filters.Contains(Value))
					{
						RowNames.Add(TableRowNames[RowIndex]);
					}
				}
			}
		}

		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ColumnProperty))
		{
			FProperty* InnerProperty = ArrayProperty->Inner;
			check(InnerProperty);

			if (UCustomizableObjectNodeTable::SupportedFilterTypes.Contains(InnerProperty->GetClass()))
			{
				for (int32 RowIndex = 0; RowIndex < TableRowNames.Num(); ++RowIndex)
				{
					if (uint8* CellData = UCustomizableObjectNodeTable::GetCellData(TableRowNames[RowIndex], DataTable, *ArrayProperty))
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, CellData);
						int32 NumMatchingFilters = 0;

						for (int32 NameIndex = 0; NameIndex < ArrayHelper.Num(); ++NameIndex)
						{
							// By getting values as Texts we enable the support to FText columns
							FString Value = DataTableUtils::GetPropertyValueAsTextDirect(InnerProperty, ArrayHelper.GetRawPtr(NameIndex)).ToString();

							if (CompilationFilterOptions[FilterIndex].Filters.Contains(Value))
							{
								if (CompilationFilterOptions[FilterIndex].OperationType == ETableCompilationFilterOperationType::TCFOT_OR)
								{
									RowNames.Add(TableRowNames[RowIndex]);
									break;
								}
								else
								{
									NumMatchingFilters++;

									if (NumMatchingFilters == CompilationFilterOptions[FilterIndex].Filters.Num())
									{
										RowNames.Add(TableRowNames[RowIndex]);
										break;
									}
								}
							}
						}
					}
				}
			}
		}

		// We will iterate through mutliple filters
		TableRowNames = RowNames;
		RowNames.Reset();
	}

	return TableRowNames;
}


#undef LOCTEXT_NAMESPACE
