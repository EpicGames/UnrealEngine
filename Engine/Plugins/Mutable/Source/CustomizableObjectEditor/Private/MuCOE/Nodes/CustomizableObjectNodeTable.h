// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "Engine/Texture2DArray.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"

#include "CustomizableObjectNodeTable.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectLayout;
class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeRemapPinsByName;
class UCustomizableObjectNodeTable;
class UEdGraphPin;
class UObject;
class USkeletalMesh;
class UTexture2D;
class UTexture2DArray;
class UAnimInstance;
struct FGuid;
struct FSkeletalMaterial;
struct FSkelMeshSection;

/** Enum class for the different types of image pins */
UENUM()
enum class ETableTextureType : uint8
{
	PASSTHROUGH_TEXTURE = 0 UMETA(DisplayName = "Object"),
	MUTABLE_TEXTURE = 1 UMETA(DisplayName = "Mutable")
};


/** Enum class for the different types of pin meshes */
enum class ETableMeshPinType : uint8
{
	NONE = 0,
	SKELETAL_MESH = 1,
	STATIC_MESH = 2
};


/** Enum to decide where the data comes from: Struct or Data Table*/
UENUM()
enum class ETableDataGatheringSource : uint8
{
	/** Gathers the information from a data table */
	ETDGM_DataTable = 0 UMETA(DisplayName = "Data Table"),

	/** When compiling the CO, it uses the asset registry to gather and generate a data table. It uses all the data tables found in the specified paths that are references of the selected structure.*/
	ETDGM_AssetRegistry = 1 UMETA(DisplayName = " Struct + Asset Registry")
};

UENUM()
enum class ETableCompilationFilterOperationType : uint8
{
	/** At least one of the filters should be in the Data Table row */
	TCFOT_OR = 0 UMETA(DisplayName = "OR"),

	/** All filters should be in the Data Table row */
	TCFOT_AND = 1 UMETA(DisplayName = "AND")
};


USTRUCT()
struct FTableNodeColumnData
{
	GENERATED_USTRUCT_BODY()

	/** Anim Instance Column name related to this Mesh pin */
	UPROPERTY()
	FString AnimInstanceColumnName = "";

	/** Anim Slot Column name related to this Mesh pin */
	UPROPERTY()
	FString AnimSlotColumnName = "";

	/** Anim Tag Column name related to this Mesh pin */
	UPROPERTY()
	FString AnimTagColumnName = "";
};


/** Base class for all Table Pins. */
UCLASS(MinimalAPI)
class UCustomizableObjectNodeTableObjectPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	/** Id of the property associated to a struct column */
	UPROPERTY()
	FGuid StructColumnId;

	/** Name of the data table column property related to the pin */
	UPROPERTY()
	FString ColumnName_DEPRECATED = "";

	/** Unique Name of the struct property related to the pin. Used to data purposes: search struct porperty, set mutable column name... */
	UPROPERTY()
	FString ColumnPropertyName = "";

	/** Name of the data table column related to the pin. Used for UI purposes.*/
	UPROPERTY()
	FString ColumnDisplayName = "";
};


/** Additional data for Image pins. */
UCLASS(MinimalAPI)
class UCustomizableObjectNodeTableImagePinData : public UCustomizableObjectNodeTableObjectPinData
{
	GENERATED_BODY()

public:
	// UObject interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;

	UPROPERTY(EditAnywhere, Category = NoCategory,  DisplayName = "Texture Parameter Mode")
	ETableTextureType ImageMode = ETableTextureType::MUTABLE_TEXTURE;

	UPROPERTY()
	TObjectPtr<UCustomizableObjectNodeTable> NodeTable = nullptr;

private:
	// Replaced by the more general bIsNot2DTexture
	UPROPERTY()
	bool bIsArrayTexture_DEPRECATED = false;

public:
	UPROPERTY()
	bool bIsNotTexture2D = false;
};


/** Additional data for Mesh pins. */
UCLASS(MinimalAPI)
class UCustomizableObjectNodeTableMeshPinData : public UCustomizableObjectNodeTableObjectPinData
{
	GENERATED_BODY()

public:

	/** Anim Instance Column name related to this Mesh pin */
	UPROPERTY()
	FString AnimInstanceColumnName_DEPRECATED = "";

	/** Anim Slot Column name related to this Mesh pin */
	UPROPERTY()
	FString AnimSlotColumnName_DEPRECATED = "";

	/** Anim Tag Column name related to this Mesh pin */
	UPROPERTY()
	FString AnimTagColumnName_DEPRECATED = "";

	/** LOD of the mesh related to this Mesh pin */
	UPROPERTY()
	int32 LOD = 0;

	/** Section Index (Surface Index) of the mesh related to this Mesh pin */
	UPROPERTY()
	int32 Material = 0;

	/** Layouts related to this Mesh pin */
	UPROPERTY()
	TArray< TObjectPtr<UCustomizableObjectLayout> > Layouts;
};


UCLASS()
class UCustomizableObjectNodeTableRemapPins : public UCustomizableObjectNodeRemapPinsByName
{
	GENERATED_BODY()
public:

	// Specific method to decide when two pins are equal
	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const override;

	// Method to use in the RemapPins step of the node reconstruction process
	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};


USTRUCT()
struct FTableNodeCompilationFilter
{
	GENERATED_USTRUCT_BODY()

	/** Column of the Data Table that contains the compilation filters of each row */
	UPROPERTY(EditAnywhere, Category = CompilationRestrictions)
	FName FilterColumn;

	/** Compilation filters required for this Table node represented in a string
		Supported types: Names, Strings, int, bools (true, false). */
	UPROPERTY(EditAnywhere, Category = CompilationRestrictions)
	TArray<FString> Filters;

	/** Determines how Filters interact with the values of the Compilation Filter Column.
		Check the tooltip of each operation for more information. */
	UPROPERTY(EditAnywhere, Category = CompilationRestrictions)
	ETableCompilationFilterOperationType OperationType = ETableCompilationFilterOperationType::TCFOT_OR;

	bool operator==(const FTableNodeCompilationFilter&) const = default;
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTable : public UCustomizableObjectNode
{
public:
	
	GENERATED_BODY()

	// Name of the property parameter
	UPROPERTY(EditAnywhere, Category = TableProperties)
	FString ParameterName = "Default Name";

	/** If true, adds a "None" parameter option */
	UPROPERTY(EditAnywhere, Category = TableProperties)
	bool bAddNoneOption;

	/** Source where table gathers the data */
	UPROPERTY(EditAnywhere, Category = TableProperties)
	ETableDataGatheringSource TableDataGatheringMode = ETableDataGatheringSource::ETDGM_DataTable;

	// Pointer to the Data Table Asset represented in this node
	UPROPERTY(EditAnywhere, Category = TableProperties, meta = (DontUpdateWhileEditing, EditCondition = "TableDataGatheringMode == ETableDataGatheringSource::ETDGM_DataTable", EditConditionHides))
	TSoftObjectPtr<UDataTable> Table = nullptr;

	// Pointer to the Struct Asset represented in this node
	UPROPERTY(EditAnywhere, Category = TableProperties, meta = (DontUpdateWhileEditing, EditCondition = "TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry", EditConditionHides))
	TSoftObjectPtr<UScriptStruct> Structure = nullptr;

	UPROPERTY(EditAnywhere, Category = TableProperties, meta = (EditCondition = "TableDataGatheringMode == ETableDataGatheringSource::ETDGM_AssetRegistry", EditConditionHides))
	TArray<FName> FilterPaths;
	
	/** Name of the column that contains the Version options. */
	UPROPERTY(EditAnywhere, Category = CompilationRestrictions)
	FName VersionColumn;

	/** Name of the row that will be used as default value. */
	UPROPERTY(EditAnywhere, Category = TableProperties)
	FName DefaultRowName;

	UPROPERTY(EditAnywhere, Category = UI, meta = (DisplayName = "Parameter UI Metadata"))
	FMutableParamUIMetadata ParamUIMetadata;

	/** Name of the column that contains the MutableUIMetadata of the row options. */
	UPROPERTY(EditAnywhere, Category = UI)
	FName ParamUIMetadataColumn;
	
	/** Name of the column that contains the asset to use its thumbnails as option thumbnails. */
	UPROPERTY(EditAnywhere, Category = UI)
	FName ThumbnailColumn;
	
	UPROPERTY(EditAnywhere, Category = UI, meta = (Tooltip = "Given a row, add all tags found in GameplayTag columns to its Parameter UI Metadata"))
	bool bGatherTags = true;

	// TODO Remove. Only used because poses are the diff from a Reference Skeleton. Once the pose is absolute remove this.
	UPROPERTY(EditAnywhere, Category = TableProperties)
	TObjectPtr<USkeletalMesh> PoseReferenceMesh;
	
	/** Map to relate a Structure Column with its Data */
	UPROPERTY()
	TMap<FGuid, FTableNodeColumnData> ColumnDataMap_DEPRECATED;

	/** Map to relate a Structure Column with its Data 
	* Key: ColumnPropertyName variable of the PinData	
	*/
	UPROPERTY()
	TMap<FString, FTableNodeColumnData> PinColumnDataMap;

	/** If true, the "None" colors will use the color of the material parameter.
		If false, the "None" colors will be black.
		Note: if this option is true and the colors are used to generate textures, "None" option colors will be set to black.*/
	UPROPERTY(EditAnywhere, Category = TableProperties, meta = (DisplayAfter = "bAddNoneOption", EditCondition = "bAddNoneOption==true", EditConditionHides))
	bool bUseMaterialColor = false;

	UPROPERTY(EditAnywhere, Category = CompilationRestrictions)
	TArray<FTableNodeCompilationFilter> CompilationFilterOptions;

	// Array with all the classes supported to be used as row filters
	static UE_API const TArray<FFieldClass*> SupportedFilterTypes;

public:

	// UObject interface
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	// EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	virtual bool GetCanRenameNode() const override { return true; }
	
	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API bool IsNodeOutDatedAndNeedsRefresh() override;
	UE_API FString GetRefreshMessage() const override;
	UE_API virtual bool ProvidesCustomPinRelevancyTest() const override;
	UE_API virtual bool IsPinRelevant(const UEdGraphPin* Pin) const override;
	UE_API UCustomizableObjectNodeTableRemapPins* CreateRemapPinsDefault() const;
	UE_API virtual bool HasPinViewer() const override;
	UE_API virtual TSharedPtr<IDetailsView> CustomizePinDetails(const UEdGraphPin& Pin) const override;
	
	/*** Allows to perform work when remapping the pin data. */
	UE_API virtual void RemapPinsData(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap) override;
	
	// Own interface	
	// Returns the reference Texture parameter from a Material
	UE_API UTexture2D* FindReferenceTextureParameter(const UEdGraphPin* Pin, FString ParameterImageName) const;

	// Methods to get the UVs of the skeletal mesh 
	UE_API UStreamableRenderAsset* GetDefaultMeshForLayout(const UCustomizableObjectLayout* Layout) const;

	// Methods to provide the Layouts to the Layout block editors
	UE_API TArray<UCustomizableObjectLayout*> GetLayouts(const UEdGraphPin* Pin) const;

	// Returns the name of the table column related to a pin
	UE_API FString GetPinColumnName(const UEdGraphPin* Pin) const;

	// Returns the LOD of the mesh associated to the input pin
	UE_API void GetPinLODAndSection(const UEdGraphPin* Pin, int32& LODIndex, int32& SectionIndex) const;

	// Get the anim blueprint and anim slot columns related to a mesh
	UE_API void GetAnimationColumns(const FString& ColumnPropertyName, FString& AnimBPColumnName, FString& AnimSlotColumnName, FString& AnimTagColumnName) const;

	template<class T> 
	T* GetColumnDefaultAssetByType(FString ColumnPropertyName) const
	{
		T* ObjectToReturn = nullptr;
		const UScriptStruct* TableStruct = GetTableNodeStruct();

		if (TableStruct)
		{
			// Getting Default Struct Values
			// A Script Struct always has at leaset one property
			TArray<int8> DeafaultDataArray;
			DeafaultDataArray.SetNumZeroed(TableStruct->GetStructureSize());
			TableStruct->InitializeStruct(DeafaultDataArray.GetData());

			FProperty* Property = FindTableProperty(FName(*ColumnPropertyName));

			if (Property)
			{
				if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
				{
					UObject* Object = nullptr;
					
					// Getting default UObject
					uint8* CellData = SoftObjectProperty->ContainerPtrToValuePtr<uint8>(DeafaultDataArray.GetData());

					if (CellData)
					{
						Object = UE::Mutable::Private::LoadObject(SoftObjectProperty->GetPropertyValue(CellData));
					}

					if (Object)
					{
						if (Object->IsA(T::StaticClass()))
						{
							ObjectToReturn = Cast <T>(Object);
						}
					}
				}
			}

			// Cleaning Default Structure Pointer
			TableStruct->DestroyStruct(DeafaultDataArray.GetData());
		}

		return ObjectToReturn;
	}

	template<class T>
	T* GetColumnDefaultAssetByType(const UEdGraphPin* Pin) const
	{
		if (Pin)
		{
			if (const UCustomizableObjectNodeTableObjectPinData* PinData = Cast<UCustomizableObjectNodeTableObjectPinData >(GetPinData(*Pin)))
			{
				return GetColumnDefaultAssetByType<T>(PinData->ColumnPropertyName);
			}
		}

		return nullptr;
	}

	// Generation Mutable Source Methods
	// We should do this in a template!
	UE_API FSoftObjectPtr GetSkeletalMeshAt(const UEdGraphPin* Pin, const UDataTable* DataTable, const FName& RowName) const;
	UE_API TSoftClassPtr<UAnimInstance> GetAnimInstanceAt(const UEdGraphPin* Pin, const UDataTable* DataTable, const FName& RowName) const;
	
	// Returns the image mode of the column
	UE_API ETableTextureType GetColumnImageMode(const FString& ColumnPropertyName) const;

	// Returns the mesh type of the Pin
	UE_API ETableMeshPinType GetPinMeshType(const UEdGraphPin* Pin) const;

	// Functions to generate the names of a mutable table's column
	UE_API FString GenerateSkeletalMeshMutableColumName(const FString& PinName, int32 LODIndex, int32 MaterialIndex) const;
	UE_API FString GenerateStaticMeshMutableColumName(const FString& PinName, int32 MaterialIndex) const;

	/** Returns the struct pointer used to gather data */
	UE_API const UScriptStruct* GetTableNodeStruct() const;

	// Methods from UDataTable but modified to support ScriptStructs
	/** Get an array of all the column titles, using the friendly display name from the property */
	UE_API TArray<FString> GetColumnTitles() const;
	
	/** Returns the property using its name. */
	UE_API FProperty* FindTableProperty(const FName& PropertyName) const;

	/** Returns the property linked to a pin */
	UE_API FProperty* FindPinProperty(const UEdGraphPin& Pin) const;

	/** Returns the property linked to a column display name. Useful for properties that do not generate a pin. */
	UE_API FProperty* FindColumnProperty(const FName& ColumnDisplayName) const;

	/** Return the list of UDataTable that will be used to compose the final UDataTable. */
	UE_API TArray<FAssetData> GetParentTables() const;

	/** Returns the skeletal material associated to the given skeletal mesh output pin. */
	UE_API FSkeletalMaterial* GetDefaultSkeletalMaterialFor(const UEdGraphPin& MeshPin) const;

	/** Returns the index of the skeletal material associated to the given skeletal mesh output pin. */
	UE_API int32 GetDefaultSkeletalMaterialIndexFor(const UEdGraphPin& MeshPin) const;

	/** Returns the SkeletalMesh section associated to the given skeletal mesh output pin. */
	UE_API const FSkelMeshSection* GetDefaultSkeletalMeshSectionFor(const UEdGraphPin& MeshPin) const;

	/** */
	UE_API TArray<FName> GetEnabledRows(const UDataTable& DataTable) const;

	/** */
	static UE_API uint8* GetCellData(const FName& RowName, const UDataTable& DataTable, const FProperty& ColumnProperty);

private:
	/** Number of properties to know when the node needs an update */
	UPROPERTY()
	int32 NumProperties;
	
	FDelegateHandle OnTableChangedDelegateHandle;

	// Checks if a pin already exists and if it has the same type as before the node refresh
	UE_API bool CheckPinUpdated(const FString& PinName, const FName& PinType) const;

	/** If there is a bool column in the table, checked rows will not be compiled */
	UPROPERTY()
	bool bDisableCheckedRows_DEPRECATED = true;

	/** Column of the Data Table that contains the compilation filters of each row */
	UPROPERTY()
	FName CompilationFilterColumn_DEPRECATED;

	/** Compilation filters required for this Table node represented in a string
		Supported types: Names, Strings, int, bools (true, false). */
	UPROPERTY()
	TArray<FString> CompilationFilters_DEPRECATED;

	/** Determines how Filters interact with the values of the Compilation Filter Column.
		Check the tooltip of each operation for more information. */
	UPROPERTY()
	ETableCompilationFilterOperationType FilterOperationType_DEPRECATED = ETableCompilationFilterOperationType::TCFOT_OR;

};

#undef UE_API
