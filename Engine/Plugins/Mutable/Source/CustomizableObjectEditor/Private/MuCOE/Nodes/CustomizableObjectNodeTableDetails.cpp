// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTableDetails.h"

#include "Animation/AnimInstance.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GameplayTagContainer.h"
#include "IDetailGroup.h"
#include "IDetailsView.h"
#include "Layout/Visibility.h"
#include "MuCOE/CustomizableObjectLayout.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "MuCOE/SCustomizableObjectLayoutEditor.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Styling/SlateColor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IDetailCustomization> FCustomizableObjectNodeTableDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectNodeTableDetails);
}


void FCustomizableObjectNodeTableDetails::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	IDetailCustomization::CustomizeDetails(DetailBuilder);

	Node = 0;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder->GetDetailsViewSharedPtr();

	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = Cast<UCustomizableObjectNodeTable>(DetailsView->GetSelectedObjects()[0].Get());
	}

	if (Node.IsValid())
	{
		IDetailCategoryBuilder& CustomizableObjectCategory = DetailBuilder->EditCategory("TableProperties");
		IDetailCategoryBuilder& CompilationRestrictionsCategory = DetailBuilder->EditCategory("CompilationRestrictions");
		DetailBuilder->HideProperty("VersionColumn");
		DetailBuilder->HideProperty("CompilationFilterColumn");
		IDetailCategoryBuilder& UICategory = DetailBuilder->EditCategory("UI");
		DetailBuilder->HideProperty("ParamUIMetadataColumn");
		DetailBuilder->HideProperty("ThumbnailColumn");
		IDetailCategoryBuilder& AnimationCategory = DetailBuilder->EditCategory("AnimationProperties");

		// Attaching the Posrecontruct delegate to force a refresh of the details
		Node->PostReconstructNodeDelegate.AddSP(this, &FCustomizableObjectNodeTableDetails::OnNodePinValueChanged);

		GenerateMeshColumnComboBoxOptions();
		TSharedPtr<FString> CurrentMutableMetadataColumn = GenerateMutableMetaDataColumnComboBoxOptions();
		TSharedPtr<FString> CurrentVersionColumn = GenerateVersionColumnComboBoxOptions();
		TSharedPtr<FString> CurrentThumbnailColumn = GenerateThumbnailColumnComboBoxOptions();

		CustomizableObjectCategory.AddProperty("ParameterName");
		TSharedRef<IPropertyHandle> AddNoneOptionProperty = DetailBuilder->GetProperty("bAddNoneOption");
		TSharedRef<IPropertyHandle> UseMaterialColorProperty = DetailBuilder->GetProperty("bUseMaterialColor");

		IDetailGroup& AddNoneGroup = CustomizableObjectCategory.AddGroup(TEXT("TableNode_NoneOptionGroup"), LOCTEXT("TableNode_NoneOptionGroup", "Add None Option"), false, true);
		AddNoneGroup.HeaderProperty(AddNoneOptionProperty);
		AddNoneGroup.AddPropertyRow(UseMaterialColorProperty);

		CompilationRestrictionsCategory.AddCustomRow(LOCTEXT("VersionColumn_Selector", "VersionColumn"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("VersionColumn_SelectorText", "Version Column"))
					.ToolTipText(LOCTEXT("VersionColumn_SelectorTooltip", "Select the column that contains the version of each row."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(VersionColumnsComboBox, STextComboBox)
					.InitiallySelectedItem(CurrentVersionColumn)
					.OptionsSource(&VersionColumnsOptionNames)
					.OnComboBoxOpening(this, &FCustomizableObjectNodeTableDetails::OnOpenVersionColumnComboBox)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnVersionColumnComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(this, &FCustomizableObjectNodeTableDetails::GetVersionColumnComboBoxTextColor, &VersionColumnsOptionNames)
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnVersionColumnComboBoxSelectionReset)));

		UICategory.AddCustomRow(LOCTEXT("MutableUIMetadataColumn_Selector", "MutableUIMetadataColumn"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("MutableUIMetadataColumn_SelectorText", "Options UI Metadata Column"))
					.ToolTipText(LOCTEXT("MutableUIMetadataColumn_SelectorTooltip", "Select a column that contains a Parameter UI Metadata for each Parameter Option (table row)."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(MutableMetaDataComboBox, STextComboBox)
					.InitiallySelectedItem(CurrentMutableMetadataColumn)
					.OptionsSource(&MutableMetaDataColumnsOptionNames)
					.OnComboBoxOpening(this, &FCustomizableObjectNodeTableDetails::OnOpenMutableMetadataComboBox)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnMutableMetaDataColumnComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(this, &FCustomizableObjectNodeTableDetails::GetComboBoxTextColor, &MutableMetaDataColumnsOptionNames, Node->ParamUIMetadataColumn)
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnMutableMetaDataColumnComboBoxSelectionReset)));

		UICategory.AddCustomRow(LOCTEXT("ThumbnailColumn_Selector", "ThumbnailColumn"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("ThumbnailColumn_SelectorText", "Options Thumbnail Column"))
					.ToolTipText(LOCTEXT("ThumbnailColumn_SelectorTooltip", "Select a column that contains the assets to use its thumbnails as Option thumbnails."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(ThumbnailComboBox, STextComboBox)
					.InitiallySelectedItem(CurrentThumbnailColumn)
					.OptionsSource(&ThumbnailColumnOptionNames)
					.OnComboBoxOpening(this, &FCustomizableObjectNodeTableDetails::OnOpenThumbnailComboBox)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnThumbnailColumnComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(this, &FCustomizableObjectNodeTableDetails::GetComboBoxTextColor, &MutableMetaDataColumnsOptionNames, Node->ThumbnailColumn)
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnThumbnailColumnComboBoxSelectionReset)));



		// Anim Category -----------------------------------

		// Mesh Column Selector
		AnimationCategory.AddCustomRow(LOCTEXT("AnimationProperties", "Animation Properties"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("AnimMeshColumnText", "Mesh Column: "))
					.ToolTipText(LOCTEXT("AnimMeshColumnTooltip", "Select a mesh column from the Data Table to edit its animation options (Applied to all LODs)."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SAssignNew(AnimMeshColumnComboBox, STextComboBox)
					.OptionsSource(&AnimMeshColumnOptionNames)
					.InitiallySelectedItem(AnimMeshColumnOptionNames[0])
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnAnimMeshColumnComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnAnimMeshCustomRowResetButtonClicked)));


		// AnimBP Column Selector
		AnimationCategory.AddCustomRow(LOCTEXT("AnimationProperties", "Animation Properties"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("AnimBPText", "Animation Blueprint Column: "))
					.ToolTipText(LOCTEXT("AnimBlueprintColumnTooltip", "Select an animation blueprint column from the Data Table that will be applied to the mesh selected"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SAssignNew(AnimComboBox, STextComboBox)
					.OptionsSource(&AnimOptionNames)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnAnimInstanceComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnAnimCustomRowResetButtonClicked, EAnimColumnType::EACT_BluePrintColumn)))
			.Visibility(TAttribute<EVisibility>(this, &FCustomizableObjectNodeTableDetails::AnimWidgetsVisibility));


		// AnimSlot Column Selector
		AnimationCategory.AddCustomRow(LOCTEXT("AnimationProperties", "Animation Properties"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("AnimSlotText", "Animation Slot Column: "))
					.ToolTipText(LOCTEXT("AnimSlotColumnTooltip", "Select an animation slot column from the Data Table that will set to the slot value of the animation blueprint"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SAssignNew(AnimSlotComboBox, STextComboBox)
					.OptionsSource(&AnimSlotOptionNames)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnAnimSlotComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnAnimCustomRowResetButtonClicked, EAnimColumnType::EACT_SlotColumn)))
			.Visibility(TAttribute<EVisibility>(this, &FCustomizableObjectNodeTableDetails::AnimWidgetsVisibility));


		// AnimTags Column Selector
		AnimationCategory.AddCustomRow(LOCTEXT("AnimationProperties", "Animation Properties"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("AnimTagsText", "Animation Tags Column: "))
					.ToolTipText(LOCTEXT("AnimTagColumnTooltip", "Select an animation tag column from the Data Table that will set to the animation tags of the animation blueprint"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SAssignNew(AnimTagsComboBox, STextComboBox)
					.OptionsSource(&AnimTagsOptionNames)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableDetails::OnAnimTagsComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectNodeTableDetails::OnAnimCustomRowResetButtonClicked, EAnimColumnType::EACT_TagsColumn)))
			.Visibility(TAttribute<EVisibility>(this, &FCustomizableObjectNodeTableDetails::AnimWidgetsVisibility));

		// Array of MeshSections and their editable layouts
		TArray<FLayoutEditorMeshSection> MeshSectionsAndLayouts;
		GenerateMeshSectionOptions(MeshSectionsAndLayouts);

		TSharedPtr<SCustomizableObjectLayoutEditor> LayoutBlocksEditor = SNew(SCustomizableObjectLayoutEditor)
			.Node(Node.Get())
			.MeshSections(MeshSectionsAndLayouts);

		FCustomizableObjectLayoutEditorDetailsBuilder LayoutEditorBuilder;
		LayoutEditorBuilder.LayoutEditor = LayoutBlocksEditor;
		LayoutEditorBuilder.bShowLayoutSelector = true;
		LayoutEditorBuilder.bShowPackagingStrategy = true;
		LayoutEditorBuilder.bShowAutomaticGenerationSettings = true;
		LayoutEditorBuilder.bShowGridSize = true;
		LayoutEditorBuilder.bShowMaxGridSize = true;
		LayoutEditorBuilder.bShowReductionMethods = true;
		LayoutEditorBuilder.bShowWarningSettings = true;

		LayoutEditorBuilder.CustomizeDetails(*DetailBuilder.Get());

		LayoutBlocksEditor->UpdateLayout(nullptr);
	}
}


void FCustomizableObjectNodeTableDetails::GenerateMeshColumnComboBoxOptions()
{
	AnimMeshColumnOptionNames.Empty();

	// Add first element to clear selection
	AnimMeshColumnOptionNames.Add(MakeShareable(new FString("- Nothing Selected -")));

	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();

	if (!TableStruct)
	{
		return;
	}

	// Get mesh columns only
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
		{
			if (SoftObjectProperty->PropertyClass->IsChildOf(USkeletalMesh::StaticClass())
				|| SoftObjectProperty->PropertyClass->IsChildOf(UStaticMesh::StaticClass()))
			{
				FString MeshColumnName = ColumnProperty->GetDisplayNameText().ToString();
				AnimMeshColumnOptionNames.Add(MakeShareable(new FString(MeshColumnName)));
			}
		}
	}
}


void FCustomizableObjectNodeTableDetails::GenerateMeshSectionOptions(TArray<FLayoutEditorMeshSection>& OutMeshSections)
{
	// Add first element to clear selection
	OutMeshSections.Empty();

	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();

	if (!TableStruct)
	{
		return;
	}

	// Get mesh columns only
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
		{
			if (SoftObjectProperty->PropertyClass->IsChildOf(USkeletalMesh::StaticClass())
				|| SoftObjectProperty->PropertyClass->IsChildOf(UStaticMesh::StaticClass()))
			{
				FString MeshColumnName = ColumnProperty->GetAuthoredName();

				for (const UEdGraphPin* Pin : Node->GetAllNonOrphanPins())
				{
					const UCustomizableObjectNodeTableMeshPinData* PinData = Cast<UCustomizableObjectNodeTableMeshPinData >(Node->GetPinData(*Pin));

					if (!PinData || PinData->ColumnPropertyName != MeshColumnName || Node->GetPinMeshType(Pin) != ETableMeshPinType::SKELETAL_MESH)
					{
						continue;
					}

					if (PinData && PinData->ColumnPropertyName == MeshColumnName)
					{
						FLayoutEditorMeshSection& MeshSection = OutMeshSections.AddDefaulted_GetRef();
						MeshSection.MeshName = MakeShareable(new FString(Pin->PinName.ToString()));

						for (UCustomizableObjectLayout* Layout : PinData->Layouts)
						{
							MeshSection.Layouts.Add(Layout);
						}
					}
				}
			}
		}
	}
}


void FCustomizableObjectNodeTableDetails::OnNodePinValueChanged()
{
	RefreshDetails();
}


// Anim Category --------------------------------------------------------------------------------

void FCustomizableObjectNodeTableDetails::GenerateAnimInstanceComboBoxOptions()
{
	// Options Reset
	AnimOptionNames.Empty();
	AnimSlotOptionNames.Empty();
	AnimTagsOptionNames.Empty();

	// Selection Reset
	AnimComboBox->ClearSelection();
	AnimSlotComboBox->ClearSelection();
	AnimTagsComboBox->ClearSelection();

	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();

	if (!TableStruct || !AnimMeshColumnComboBox.IsValid())
	{
		return;
	}

	FString ColumnName = *AnimMeshColumnComboBox->GetSelectedItem();
	FTableNodeColumnData* MeshColumnData = Node->PinColumnDataMap.Find(ColumnName);

	// Fill in name option arrays and set the selected item if any
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		if (FProperty* ColumnProperty = *It)
		{
			if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(ColumnProperty))
			{
				if (SoftClassProperty->MetaClass->IsChildOf(UAnimInstance::StaticClass()))
				{
					TSharedPtr<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
					AnimOptionNames.Add(Option);

					if (MeshColumnData && MeshColumnData->AnimInstanceColumnName == *Option)
					{
						AnimComboBox->SetSelectedItem(Option);
					}
				}
			}

			else if (CastField<FIntProperty>(ColumnProperty) || CastField<FNameProperty>(ColumnProperty))
			{
				TSharedPtr<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
				AnimSlotOptionNames.Add(Option);

				if (MeshColumnData && MeshColumnData->AnimSlotColumnName == *Option)
				{
					AnimSlotComboBox->SetSelectedItem(Option);
				}
			}

			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ColumnProperty))
			{
				if (StructProperty->Struct == TBaseStructure<FGameplayTagContainer>::Get())
				{
					TSharedPtr<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
					AnimTagsOptionNames.Add(Option);

					if (MeshColumnData && MeshColumnData->AnimTagColumnName == *Option)
					{
						AnimTagsComboBox->SetSelectedItem(Option);
					}
				}
			}
		}
	}
}


EVisibility FCustomizableObjectNodeTableDetails::AnimWidgetsVisibility() const
{
	if (AnimMeshColumnComboBox.IsValid() && AnimMeshColumnComboBox->GetSelectedItem() != AnimMeshColumnOptionNames[0])
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}


void FCustomizableObjectNodeTableDetails::OnAnimMeshColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		GenerateAnimInstanceComboBoxOptions();
	}
}


void FCustomizableObjectNodeTableDetails::OnAnimInstanceComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	bool bIsMeshSelectionValid =  AnimMeshColumnComboBox->GetSelectedItem() != AnimMeshColumnOptionNames[0] && AnimMeshColumnComboBox->GetSelectedItem().IsValid();

	if (bIsMeshSelectionValid && Selection.IsValid() && SelectInfo != ESelectInfo::Direct)
	{
		FString ColumnName = *AnimMeshColumnComboBox->GetSelectedItem();
		FTableNodeColumnData* MeshColumnData = Node->PinColumnDataMap.Find(ColumnName);
		
		if (MeshColumnData)
		{
			MeshColumnData->AnimInstanceColumnName = *Selection;
		}
		else if(!ColumnName.IsEmpty())
		{
			FTableNodeColumnData NewMeshColumnData;
			NewMeshColumnData.AnimInstanceColumnName = *Selection;

			Node->PinColumnDataMap.Add(ColumnName, NewMeshColumnData);
		}

		Node->MarkPackageDirty();
	}
}


void FCustomizableObjectNodeTableDetails::OnAnimSlotComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	bool bIsMeshSelectionValid = AnimMeshColumnComboBox->GetSelectedItem() != AnimMeshColumnOptionNames[0] && AnimMeshColumnComboBox->GetSelectedItem().IsValid();

	if (bIsMeshSelectionValid && Selection.IsValid() && SelectInfo != ESelectInfo::Direct)
	{
		FString ColumnName = *AnimMeshColumnComboBox->GetSelectedItem();
		FTableNodeColumnData* MeshColumnData = Node->PinColumnDataMap.Find(ColumnName);

		if (MeshColumnData)
		{
			MeshColumnData->AnimSlotColumnName = *Selection;
		}
		else if (!ColumnName.IsEmpty())
		{
			FTableNodeColumnData NewMeshColumnData;
			NewMeshColumnData.AnimSlotColumnName = *Selection;

			Node->PinColumnDataMap.Add(ColumnName, NewMeshColumnData);
		}

		Node->MarkPackageDirty();
	}
}


void FCustomizableObjectNodeTableDetails::OnAnimTagsComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	bool bIsMeshSelectionValid = AnimMeshColumnComboBox->GetSelectedItem() != AnimMeshColumnOptionNames[0] && AnimMeshColumnComboBox->GetSelectedItem().IsValid();

	if (bIsMeshSelectionValid && Selection.IsValid() && SelectInfo != ESelectInfo::Direct)
	{
		FString ColumnName = *AnimMeshColumnComboBox->GetSelectedItem();
		FTableNodeColumnData* MeshColumnData = Node->PinColumnDataMap.Find(ColumnName);

		if (MeshColumnData)
		{
			MeshColumnData->AnimTagColumnName = *Selection;
		}
		else if (!ColumnName.IsEmpty())
		{
			FTableNodeColumnData NewMeshColumnData;
			NewMeshColumnData.AnimTagColumnName = *Selection;

			Node->PinColumnDataMap.Add(ColumnName, NewMeshColumnData);
		}

		Node->MarkPackageDirty();
	}
}


void FCustomizableObjectNodeTableDetails::OnAnimMeshCustomRowResetButtonClicked()
{
	if (AnimMeshColumnOptionNames.Num())
	{
		AnimMeshColumnComboBox->SetSelectedItem(AnimMeshColumnOptionNames[0]);
	}
}


void FCustomizableObjectNodeTableDetails::OnAnimCustomRowResetButtonClicked(EAnimColumnType ColumnType)
{
	if (!AnimMeshColumnComboBox->GetSelectedItem().IsValid())
	{
		return;
	}

	FString ColumnName = *AnimMeshColumnComboBox->GetSelectedItem();
	FTableNodeColumnData* MeshColumnData = Node->PinColumnDataMap.Find(ColumnName);

	if (!MeshColumnData)
	{
		return;
	}

	switch (ColumnType)
	{
	case EAnimColumnType::EACT_BluePrintColumn:
	{
		MeshColumnData->AnimInstanceColumnName.Reset();
		AnimComboBox->ClearSelection();

		break;
	}
	case EAnimColumnType::EACT_SlotColumn:
	{
		MeshColumnData->AnimSlotColumnName.Reset();
		AnimSlotComboBox->ClearSelection();

		break;
	}
	case EAnimColumnType::EACT_TagsColumn:
	{
		MeshColumnData->AnimTagColumnName.Reset();
		AnimTagsComboBox->ClearSelection();

		break;
	}
	default:
		break;
	}

	Node->MarkPackageDirty();
}


// Metadata Category --------------------------------------------------------------------------------

TSharedPtr<FString> FCustomizableObjectNodeTableDetails::GenerateMutableMetaDataColumnComboBoxOptions()
{
	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();
	TSharedPtr<FString> CurrentSelection;
	MutableMetaDataColumnsOptionNames.Reset();

	if (!TableStruct)
	{
		return CurrentSelection;
	}

	// Iterating struct Options
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(ColumnProperty))
		{
			if (StructProperty->Struct == FMutableParamUIMetadata::StaticStruct())
			{
				TSharedPtr<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
				MutableMetaDataColumnsOptionNames.Add(Option);

				if (*Option == Node->ParamUIMetadataColumn)
				{
					CurrentSelection = MutableMetaDataColumnsOptionNames.Last();
				}
			}
		}
	}

	if (!Node->ParamUIMetadataColumn.IsNone() && !CurrentSelection)
	{
		MutableMetaDataColumnsOptionNames.Add(MakeShareable(new FString(Node->ParamUIMetadataColumn.ToString())));
		CurrentSelection = MutableMetaDataColumnsOptionNames.Last();
	}

	return CurrentSelection;
}


void FCustomizableObjectNodeTableDetails::OnOpenMutableMetadataComboBox()
{
	TSharedPtr<FString> CurrentSelection = GenerateMutableMetaDataColumnComboBoxOptions();

	if (MutableMetaDataComboBox.IsValid())
	{
		MutableMetaDataComboBox->ClearSelection();
		MutableMetaDataComboBox->RefreshOptions();
		MutableMetaDataComboBox->SetSelectedItem(CurrentSelection);
	}
}


void FCustomizableObjectNodeTableDetails::OnMutableMetaDataColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection && Node->ParamUIMetadataColumn != FName(*Selection) 
		&& (SelectInfo == ESelectInfo::OnKeyPress || SelectInfo == ESelectInfo::OnMouseClick))
	{
		Node->ParamUIMetadataColumn = FName(*Selection);
		Node->MarkPackageDirty();
	}
}


FSlateColor FCustomizableObjectNodeTableDetails::GetComboBoxTextColor(TArray<TSharedPtr<FString>>* CurrentOptions, const FName ColumnName) const
{	
	if (Node->FindColumnProperty(ColumnName) || ColumnName.IsNone())
	{
		return FSlateColor::UseForeground();
	}

	// Table Struct null or does not contain the selected property anymore
	return FSlateColor(FLinearColor(0.9f, 0.05f, 0.05f, 1.0f));
}


void FCustomizableObjectNodeTableDetails::OnMutableMetaDataColumnComboBoxSelectionReset()
{
	Node->ParamUIMetadataColumn = NAME_None;

	if (MutableMetaDataComboBox.IsValid())
	{
		GenerateMutableMetaDataColumnComboBoxOptions();
		MutableMetaDataComboBox->ClearSelection();
		MutableMetaDataComboBox->RefreshOptions();
	}
}


TSharedPtr<FString> FCustomizableObjectNodeTableDetails::GenerateThumbnailColumnComboBoxOptions()
{
	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();
	TSharedPtr<FString> CurrentSelection;
	ThumbnailColumnOptionNames.Reset();

	if (!TableStruct)
	{
		return CurrentSelection;
	}

	// Iterating struct Options
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		if (const FSoftObjectProperty* ObjectProperty = CastField<FSoftObjectProperty>(ColumnProperty))
		{
			TSharedPtr<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
			ThumbnailColumnOptionNames.Add(Option);

			if (*Option == Node->ThumbnailColumn)
			{
				CurrentSelection = ThumbnailColumnOptionNames.Last();
			}
		}
	}

	if (!Node->ThumbnailColumn.IsNone() && !CurrentSelection)
	{
		ThumbnailColumnOptionNames.Add(MakeShareable(new FString(Node->ThumbnailColumn.ToString())));
		CurrentSelection = ThumbnailColumnOptionNames.Last();
	}

	return CurrentSelection;
}


void FCustomizableObjectNodeTableDetails::OnOpenThumbnailComboBox()
{
	TSharedPtr<FString> CurrentSelection = GenerateThumbnailColumnComboBoxOptions();

	if (ThumbnailComboBox.IsValid())
	{
		ThumbnailComboBox->ClearSelection();
		ThumbnailComboBox->RefreshOptions();
		ThumbnailComboBox->SetSelectedItem(CurrentSelection);
	}
}


void FCustomizableObjectNodeTableDetails::OnThumbnailColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection && Node->ThumbnailColumn != FName(*Selection)
		&& (SelectInfo == ESelectInfo::OnKeyPress || SelectInfo == ESelectInfo::OnMouseClick))
	{
		Node->ThumbnailColumn = FName(*Selection);
		Node->MarkPackageDirty();
	}
}


void FCustomizableObjectNodeTableDetails::OnThumbnailColumnComboBoxSelectionReset()
{
	Node->ThumbnailColumn = NAME_None;

	if (ThumbnailComboBox.IsValid())
	{
		GenerateThumbnailColumnComboBoxOptions();
		ThumbnailComboBox->ClearSelection();
		ThumbnailComboBox->RefreshOptions();
	}
}


// Compilation Restrictions Category --------------------------------------------------------------------------------

TSharedPtr<FString> FCustomizableObjectNodeTableDetails::GenerateVersionColumnComboBoxOptions()
{
	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();
	TSharedPtr<FString> CurrentSelection;
	VersionColumnsOptionNames.Reset();

	if (!TableStruct)
	{
		return CurrentSelection;
	}

	// Iterating struct Options
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		TSharedPtr<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
		VersionColumnsOptionNames.Add(Option);

		if (*Option == Node->VersionColumn)
		{
			CurrentSelection = VersionColumnsOptionNames.Last();
		}
	}

	if (!Node->VersionColumn.IsNone() && !CurrentSelection)
	{
		VersionColumnsOptionNames.Add(MakeShareable(new FString(Node->VersionColumn.ToString())));
		CurrentSelection = VersionColumnsOptionNames.Last();
	}

	return CurrentSelection;
}


void FCustomizableObjectNodeTableDetails::OnOpenVersionColumnComboBox()
{
	TSharedPtr<FString> CurrentSelection = GenerateVersionColumnComboBoxOptions();

	if (VersionColumnsComboBox.IsValid())
	{
		VersionColumnsComboBox->ClearSelection();
		VersionColumnsComboBox->RefreshOptions();
		VersionColumnsComboBox->SetSelectedItem(CurrentSelection);
	}
}


void FCustomizableObjectNodeTableDetails::OnVersionColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection && Node->VersionColumn != FName(*Selection)
		&& (SelectInfo == ESelectInfo::OnKeyPress || SelectInfo == ESelectInfo::OnMouseClick))
	{
		Node->VersionColumn = FName(*Selection);
		Node->MarkPackageDirty();
	}
}


FSlateColor FCustomizableObjectNodeTableDetails::GetVersionColumnComboBoxTextColor(TArray<TSharedPtr<FString>>* CurrentOptions) const
{
	if (Node->FindColumnProperty(Node->VersionColumn) || Node->VersionColumn.IsNone())
	{
		return FSlateColor::UseForeground();
	}

	// Table Struct null or does not contain the selected property anymore
	return FSlateColor(FLinearColor(0.9f, 0.05f, 0.05f, 1.0f));
}


void FCustomizableObjectNodeTableDetails::OnVersionColumnComboBoxSelectionReset()
{
	Node->VersionColumn = NAME_None;

	if (VersionColumnsComboBox.IsValid())
	{
		GenerateVersionColumnComboBoxOptions();
		VersionColumnsComboBox->ClearSelection();
		VersionColumnsComboBox->RefreshOptions();
	}
}

#undef LOCTEXT_NAMESPACE
