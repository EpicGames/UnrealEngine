// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectNodeDetails.h"
#include "IDetailCustomization.h"

#include "UObject/WeakObjectPtr.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class ISinglePropertyView;
class FReply;
class SButton;
class SSearchableComboBox;
class STextBlock;
class STextComboBox;
class SWidget;
class UCustomizableObjectLayout;
class UCustomizableObjectNodeTable;
struct EVisibility;
struct FSlateColor;
struct FLayoutEditorMeshSection;

enum class EAnimColumnType 
{
	EACT_BluePrintColumn,
	EACT_SlotColumn,
	EACT_TagsColumn
};


/** Copy Material node details panel. Hides all properties from the inheret Material node. */
class FCustomizableObjectNodeTableDetails : public FCustomizableObjectNodeDetails
{
public:
	
	static TSharedRef<IDetailCustomization> MakeInstance();

	// ILayoutDetails interface
	/** Hides details copied from CustomizableObjectNodeMaterial. */
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

private:

	// Details -------------
	// Generates Mesh columns combobox options
	void GenerateMeshColumnComboBoxOptions();
	void GenerateMeshSectionOptions(TArray<FLayoutEditorMeshSection>& OutMeshSections);
	
	// Function called when the table node has been refreshed
	void OnNodePinValueChanged();


	// Anim Category -------------
	// Generates Animation Instance combobox options
	void GenerateAnimInstanceComboBoxOptions();

	// OnComboBoxSelectionChanged Callback for Layout ComboBox
	void OnAnimMeshColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// OnComboBoxSelectionChanged Callback for AnimInstance ComboBox
	void OnAnimInstanceComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// OnComboBoxSelectionChanged Callback for Anim Slot ComboBox
	void OnAnimSlotComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// OnComboBoxSelectionChanged Callback for Anim Tags ComboBox
	void OnAnimTagsComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// Set the visibility of the animation column selector widgets
	EVisibility AnimWidgetsVisibility() const;

	// Callback to clear the mesh column to edit its animation properties
	void OnAnimMeshCustomRowResetButtonClicked();

	// Callback to clear the animation combobox selections
	void OnAnimCustomRowResetButtonClicked(EAnimColumnType ColumnType);


	// Mutable UI Metadata Category -------------

	// Generates MutableMetadata columns combobox options
	// Returns the current selected option or a null pointer
	TSharedPtr<FString> GenerateMutableMetaDataColumnComboBoxOptions();

	// Generates Thumbnail columns combobox options
	// Returns the current selected option or a null pointer
	TSharedPtr<FString> GenerateThumbnailColumnComboBoxOptions();

	// Callbacks to regenerate the combobox options
	void OnOpenMutableMetadataComboBox();
	void OnOpenThumbnailComboBox();

	// OnComboBoxSelectionChanged Callbacks
	void OnMutableMetaDataColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	void OnThumbnailColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// If the property selected in the combobox does not exist anymore, returns a red color.
	FSlateColor GetComboBoxTextColor(TArray<TSharedPtr<FString>>* CurrentOptions, const FName ColumnName) const;

	// OnComboBoxSelectionChanged Callbacks for ComboBox
	void OnMutableMetaDataColumnComboBoxSelectionReset();
	void OnThumbnailColumnComboBoxSelectionReset();


	// Compilation Restrictions Category -------------

	// Generates MutableMetadata columns combobox options
	// Returns the current selected option or a null pointer
	TSharedPtr<FString> GenerateVersionColumnComboBoxOptions();

	// Callback to regenerate the combobox options
	void OnOpenVersionColumnComboBox();
	
	// OnComboBoxSelectionChanged Callback for Layout ComboBox
	void OnVersionColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);
	
	// Sets the combo box selection color
	FSlateColor GetVersionColumnComboBoxTextColor(TArray<TSharedPtr<FString>>* CurrentOptions) const;
	
	// OnComboBoxSelectionChanged Callback for Layout ComboBox
	void OnVersionColumnComboBoxSelectionReset();

private:

	// Details -------------
	// Pointer to the node represented in this details
	TWeakObjectPtr<UCustomizableObjectNodeTable> Node;

	// Anim -------------
	// ComboBox widget to select a column from the NodeTable
	TSharedPtr<STextComboBox> AnimMeshColumnComboBox;

	// Array with the name of the table columns as combobox options
	TArray<TSharedPtr<FString>> AnimMeshColumnOptionNames;

	// ComboBox widget to select an Animation Instance column from the NodeTable
	TSharedPtr<STextComboBox> AnimComboBox;

	// Array with the name of the Animation Instance columns as combobox options
	TArray<TSharedPtr<FString>> AnimOptionNames;

	// ComboBox widget to select an Animation Slot column from the NodeTable
	TSharedPtr<STextComboBox> AnimSlotComboBox;

	// Array with the name of the Animation Slot columns as combobox options
	TArray<TSharedPtr<FString>> AnimSlotOptionNames;

	// ComboBox widget to select an Animation Tags column from the NodeTable
	TSharedPtr<STextComboBox> AnimTagsComboBox;

	// Array with the name of the Animation Tags columns as combobox options
	TArray<TSharedPtr<FString>> AnimTagsOptionNames;

	// Mutable UI Metadata -------------
	// Array with the name of the MutableMetaData columns
	TArray<TSharedPtr<FString>> MutableMetaDataColumnsOptionNames;
	
	TArray<TSharedPtr<FString>> ThumbnailColumnOptionNames;

	// ComboBox widget to select a MutableMetaDatacolumn from the NodeTable
	TSharedPtr<STextComboBox> MutableMetaDataComboBox;
	
	// ComboBox widget to select a Thumbnail Column from the NodeTable
	TSharedPtr<STextComboBox> ThumbnailComboBox;


	// Version Bridge -------------
	// Array with the name of the Version columns
	TArray<TSharedPtr<FString>> VersionColumnsOptionNames;
	
	// ComboBox widget to select a VersionColumn from the NodeTable
	TSharedPtr<STextComboBox> VersionColumnsComboBox;

};
