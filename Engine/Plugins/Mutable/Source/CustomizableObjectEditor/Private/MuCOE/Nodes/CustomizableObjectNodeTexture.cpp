// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTexture.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "ISinglePropertyView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTexture)

class UCustomizableObjectNodeRemapPins;
struct FGeometry;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


TSharedPtr<SGraphNode> UCustomizableObjectNodeTextureBase::CreateVisualWidget()
{
	return SNew(SGraphNodeTexture, this);
}


void UCustomizableObjectNodeTexture::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	TexturePin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
	PassthroughTexturePin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough);
}


void UCustomizableObjectNodeTexture::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Image"))) {
			Pin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::AddedAnyTextureTypeToPassThroughTextures)
	{
		// Was UCustomizableObjectNodePassThroughTexture::Texture_DEPRECATED -> PassThroughTexture.
		// After the class redirect that data is loaded into this->Texture; migrate it only if the node was a passthrough.
		if (UEdGraphPin* OutputPin = FindPin(TEXT("Texture")))
		{
			if (OutputPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
			{
				if (Texture)
				{
					PassThroughTexture_DEPRECATED = Texture;
				}
			}
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		// Pin rename from "Texture" to "PassThrough Texture" only applies to nodes that were originally passthrough.
		if (UEdGraphPin* FoundPin = FindPin(TEXT("Texture")))
		{
			if (FoundPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
			{
				FoundPin->PinName = TEXT("PassThrough Texture");
				FoundPin->PinFriendlyName = LOCTEXT("PassThrough_Image_Pin_Category", "PassThrough Texture");
			}
		}
	}
	
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::MergerTextureNodes)
	{
		{
			UEdGraphPin* Pin = FindPin(TEXT("Texture"));
			if (!Pin)
			{
				Pin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
			}
			
			TexturePin = Pin;
		}
		
		{
			UEdGraphPin* Pin = FindPin(TEXT("PassThrough Texture"));
			if (!Pin)
			{
				Pin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough);
			}
			
			PassthroughTexturePin = Pin; 
		}
		
		if (PassThroughTexture_DEPRECATED)
		{
			Texture = PassThroughTexture_DEPRECATED;
		}
	}
}


FText UCustomizableObjectNodeTexture::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Texture)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("TextureName"), FText::FromString(Texture->GetName()));

		return FText::Format(LOCTEXT("Texture_Title", "{TextureName}\nTexture"), Args);
	}
	else
	{
		return LOCTEXT("Texture", "Texture");
	}
}


FLinearColor UCustomizableObjectNodeTexture::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


TObjectPtr<UTexture> UCustomizableObjectNodeTexture::GetTexture()
{
	return Texture;
}


void SGraphNodeTexture::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	NodeTexture = Cast<UCustomizableObjectNodeTextureBase>(InGraphNode);

	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FSinglePropertyParams SingleDetails;
	SingleDetails.NamePlacement = EPropertyNamePlacement::Hidden;
	SingleDetails.bHideAssetThumbnail = true;

	if (Cast<UCustomizableObjectNodeTexture>(InGraphNode))
	{
		TextureSelector = PropPlugin.CreateSingleProperty(NodeTexture, "Texture", SingleDetails);
	}
	else
	{
		// Node type not supported.
		ensure(false);
	}

	TextureBrush.SetResourceObject(NodeTexture->GetTexture());
	TextureBrush.ImageSize.X = 128.0f;
	TextureBrush.ImageSize.Y = 128.0f;
	TextureBrush.DrawAs = ESlateBrushDrawType::Image;

	SCustomizableObjectNode::Construct({}, InGraphNode);
}


void SGraphNodeTexture::UpdateGraphNode()
{
	SGraphNode::UpdateGraphNode();
}


void SGraphNodeTexture::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	DefaultTitleAreaWidget->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(5))
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SGraphNodeTexture::OnExpressionPreviewChanged)
			.IsChecked(IsExpressionPreviewChecked())
			.Cursor(EMouseCursor::Default)
			.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(GetExpressionPreviewArrow())
				]
			]
		];
}


void SGraphNodeTexture::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	LeftNodeBox->AddSlot()
	.AutoHeight()
	[
		SNew(SVerticalBox)
		.Visibility(ExpressionPreviewVisibility())
		
		+ SVerticalBox::Slot()
		.Padding(5.0f,5.0f,0.0f,2.5f)
		.AutoHeight()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SImage)
			.Image(&TextureBrush)
		]
	];

	MainBox->AddSlot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		.Visibility(ExpressionPreviewVisibility())

		+ SHorizontalBox::Slot()
		.Padding(1.0f, 5.0f, 5.0f, 5.0f)
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			TextureSelector.ToSharedRef()
		]
	];
}


void SGraphNodeTexture::OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState)
{
	NodeTexture->bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	UpdateGraphNode();
}


ECheckBoxState SGraphNodeTexture::IsExpressionPreviewChecked() const
{
	return NodeTexture->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


const FSlateBrush* SGraphNodeTexture::GetExpressionPreviewArrow() const
{
	return FCustomizableObjectEditorStyle::Get().GetBrush(NodeTexture->bCollapsed ? TEXT("Nodes.ArrowDown") : TEXT("Nodes.ArrowUp"));
}


EVisibility SGraphNodeTexture::ExpressionPreviewVisibility() const
{
	return NodeTexture->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible;
}


void SGraphNodeTexture::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (NodeTexture)
	{
		if (NodeTexture->GetTexture() != TextureBrush.GetResourceObject())
		{
			TextureBrush.SetResourceObject(NodeTexture->GetTexture());
		}
	}
}


#undef LOCTEXT_NAMESPACE
