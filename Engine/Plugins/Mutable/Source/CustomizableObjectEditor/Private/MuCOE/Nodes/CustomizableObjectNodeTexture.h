// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CustomizableObjectNodeTexture.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class SOverlay;
class SVerticalBox;
class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeTexture;
class UObject;
class UTexture2D;
struct FGeometry;

// Class to render the Texture thumbnail of a CustomizableObjectNodeTexture
class SGraphNodeTexture : public SCustomizableObjectNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeTexture) {}
	SLATE_END_ARGS();

	SGraphNodeTexture() : SCustomizableObjectNode() {}

	// Builds the SGraphNodeTexture when needed
	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// Calls the needed functions to build the SGraphNode widgets
	void UpdateGraphNode();

	// Overriden functions to build the SGraphNode widgets
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	// Callbacks for the widget
	void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsExpressionPreviewChecked() const;
	const FSlateBrush* GetExpressionPreviewArrow() const;
	EVisibility ExpressionPreviewVisibility() const;

	// Pointer to the NodeTexture that owns this SGraphNode
	UCustomizableObjectNodeTextureBase* NodeTexture = nullptr;

	// Single property that only draws the combo box widget of the Texture
	TSharedPtr<class ISinglePropertyView> TextureSelector;

	// Brush to draw the texture to the widget
	FSlateBrush TextureBrush;
};


UCLASS(MinimalAPI, Abstract)
class UCustomizableObjectNodeTextureBase : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Determines if the Node is collapsed or not
	bool bCollapsed = true;

	// Creates the SGraph Node widget for the thumbnail
	UE_API TSharedPtr<SGraphNode> CreateVisualWidget() override;

	/** */
	virtual TObjectPtr<UTexture> GetTexture() PURE_VIRTUAL(UCustomizableObjectNodeTextureBase::GetTexture, return {};);
};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTexture : public UCustomizableObjectNodeTextureBase
{
public:
	GENERATED_BODY()

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// UCustomizableObjectNodeTextureBase interface
	UE_API virtual TObjectPtr<UTexture> GetTexture() override;

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	
	UPROPERTY(EditAnywhere, Category = Texture)
	TObjectPtr<UTexture> Texture = nullptr;

	UPROPERTY()
	TObjectPtr<UTexture> PassThroughTexture_DEPRECATED = nullptr;
	
	UPROPERTY()
	FEdGraphPinReference TexturePin;
	
	UPROPERTY()
	FEdGraphPinReference PassthroughTexturePin;
};

#undef UE_API
