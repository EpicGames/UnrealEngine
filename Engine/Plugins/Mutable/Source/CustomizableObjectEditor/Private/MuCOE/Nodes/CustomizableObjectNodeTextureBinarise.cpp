// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureBinarise.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureBinarise)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureBinarise::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	UEdGraphPin* OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
	OutputPin->bDefaultValueIsIgnored = true;
	
	UEdGraphPin* ImagePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture);
	BaseImagePinReference = FEdGraphPinReference(ImagePin);
	
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float,  TEXT("Threshold"),  LOCTEXT("Threshold", "Threshold"));
}


void UCustomizableObjectNodeTextureBinarise::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Image")))
		{
			TexturePin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}

		if (UEdGraphPin* TexturePin = FindPin(TEXT("Base Image")))
		{
			TexturePin->PinName = TEXT("Base Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* InputTexturePin = FindPin(TEXT("Base Texture"), EGPD_Input))
		{
			InputTexturePin->PinName = TEXT("Texture");
			InputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}

		if (UEdGraphPin* OutputTexturePin = FindPin(TEXT("Texture"), EGPD_Output))
		{
			OutputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeTextureBinarise::GetBaseImagePin() const
{
	return BaseImagePinReference.Get();
}


FText UCustomizableObjectNodeTextureBinarise::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Binarise", "Texture Binarise");
}


FLinearColor UCustomizableObjectNodeTextureBinarise::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureBinarise::GetTooltipText() const
{
	return LOCTEXT("Texture_Binarise_Tooltip", "Turns a base texture into black and white using a threshold.");
}


void UCustomizableObjectNodeTextureBinarise::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PinsNamesImageToTexture)
	{
		const FName TargetPinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_Texture);
		BaseImagePinReference = FEdGraphPinReference(FindPin(TargetPinName, EGPD_Input));
	}
}


#undef LOCTEXT_NAMESPACE
