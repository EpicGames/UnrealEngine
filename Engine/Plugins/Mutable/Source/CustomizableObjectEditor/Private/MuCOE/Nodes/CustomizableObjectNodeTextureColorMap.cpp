// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureColorMap.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureColorMap)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureColorMap::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Texture"), LOCTEXT("Texture", "Texture"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Base"), LOCTEXT("Base", "Base"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Mask"), LOCTEXT("Mask", "Mask"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Map"), LOCTEXT("Map", "Map"));
}


void UCustomizableObjectNodeTextureColorMap::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Image")))
		{
			TexturePin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}
}


FText UCustomizableObjectNodeTextureColorMap::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Map", "Texture Color Map");
}


FLinearColor UCustomizableObjectNodeTextureColorMap::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureColorMap::GetTooltipText() const
{
	return LOCTEXT("Texture_Gradient_Sample_Tooltip", "Map colors of map using values form image.");
}

#undef LOCTEXT_NAMESPACE

