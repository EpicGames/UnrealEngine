// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureColorMap.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureColorMap : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	UEdGraphPin* GetMapPin() const
	{
		return FindPin(TEXT("Map"));
	}

	UEdGraphPin* GetBasePin() const
	{
		return FindPin(TEXT("Base"));
	}

	UEdGraphPin* GetMaskPin() const
	{
		return FindPin(TEXT("Mask"));
	}

};

#undef UE_API
