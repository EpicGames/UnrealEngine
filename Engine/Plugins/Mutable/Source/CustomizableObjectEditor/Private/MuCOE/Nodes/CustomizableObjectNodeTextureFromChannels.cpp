// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromChannels.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureFromChannels)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureFromChannels::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Texture"), LOCTEXT("Texture", "Texture"));

	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("R"), LOCTEXT("R", "R"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("G"), LOCTEXT("G", "G"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("B"), LOCTEXT("B", "B"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("A"), LOCTEXT("A", "A"));
}


void UCustomizableObjectNodeTextureFromChannels::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Image")))
		{
			TexturePin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}
}


FText UCustomizableObjectNodeTextureFromChannels::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_From_Channels", "Texture Make");
}


FLinearColor UCustomizableObjectNodeTextureFromChannels::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureFromChannels::GetTooltipText() const
{
	return LOCTEXT("Texture_From_Channels_Tooltip", "Make a colored texture with transparency from four grayscale textures that represent the values of each RGBA channel.");
}

#undef LOCTEXT_NAMESPACE

