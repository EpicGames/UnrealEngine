// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromColor.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureFromColor)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureFromColor::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	TexturePin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
	ColorPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Color);
}


void UCustomizableObjectNodeTextureFromColor::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* FoundTexturePin = FindPin(TEXT("Image")))
		{
			FoundTexturePin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::TextureFromColorPinCaching)
	{
		ColorPin = FindPin(TEXT("Color"));
		check(ColorPin.Get());
		
		TexturePin = FindPin(TEXT("Texture"));
		check(TexturePin.Get());
	}
}


FText UCustomizableObjectNodeTextureFromColor::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_From_Color", "Texture From Color");
}


FLinearColor UCustomizableObjectNodeTextureFromColor::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureFromColor::GetTooltipText() const
{
	return LOCTEXT("Texture_From_Color_Tooltip", "Creates a flat color texture from the color provided.");
}

#undef LOCTEXT_NAMESPACE
