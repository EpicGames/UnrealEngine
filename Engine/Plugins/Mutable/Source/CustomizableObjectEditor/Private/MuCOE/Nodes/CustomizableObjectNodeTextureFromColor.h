// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureFromColor.generated.h"

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;


UCLASS()
class UCustomizableObjectNodeTextureFromColor : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	UPROPERTY()
	FEdGraphPinReference ColorPin;

	UPROPERTY()
	FEdGraphPinReference TexturePin;
};
