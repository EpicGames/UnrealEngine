// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureFromFloats.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureFromFloats)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureFromFloats::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Texture"), LOCTEXT("Texture", "Texture"));

	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("R"), LOCTEXT("R", "R"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("G"), LOCTEXT("G", "G"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("B"), LOCTEXT("B", "B"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("A"), LOCTEXT("A", "A"));
}


FText UCustomizableObjectNodeTextureFromFloats::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_From_Floats", "Texture From Float Channels");
}


FLinearColor UCustomizableObjectNodeTextureFromFloats::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureFromFloats::GetTooltipText() const
{
	return LOCTEXT("Texture_From_Floats_Tooltip", "Creates a flat color texture from the float channels provided.");
}

#undef LOCTEXT_NAMESPACE
