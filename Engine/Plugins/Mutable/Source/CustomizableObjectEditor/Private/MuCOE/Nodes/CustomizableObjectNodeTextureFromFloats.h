// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureFromFloats.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureFromFloats : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	UEdGraphPin* RPin() const
	{
		return FindPin(TEXT("R"));
	}

	UEdGraphPin* GPin() const
	{
		return FindPin(TEXT("G"));
	}

	UEdGraphPin* BPin() const
	{
		return FindPin(TEXT("B"));
	}

	UEdGraphPin* APin() const
	{
		return FindPin(TEXT("A"));
	}
};

#undef UE_API
