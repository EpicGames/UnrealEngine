// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureInterpolate.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureInterpolate)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeTextureInterpolate::UCustomizableObjectNodeTextureInterpolate()
	: Super()
{
	NumTargets = 2;
}


void UCustomizableObjectNodeTextureInterpolate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if ( PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("NumTargets") )
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeTextureInterpolate::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	CustomCreatePin(EGPD_Output, Schema->PC_Texture, TEXT("Texture"), LOCTEXT("Texture", "Texture"));
	
	CustomCreatePin(EGPD_Input, Schema->PC_Float, TEXT("Factor"), LOCTEXT("Factor", "Factor"));

	for ( int LayerIndex = 0; LayerIndex < NumTargets; ++LayerIndex )
	{
		FString PinName = FString::Printf(TEXT("Target %d"), LayerIndex);
		CustomCreatePin(EGPD_Input, Schema->PC_Texture, FName(*PinName), FText::FromString(PinName));
	}
}


FText UCustomizableObjectNodeTextureInterpolate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Interpolate", "Texture Interpolate");
}


FLinearColor UCustomizableObjectNodeTextureInterpolate::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureInterpolate::GetTooltipText() const
{
	return LOCTEXT("Texture_Interpolate_Tooltip", "Changes between textures gradually, allowing to fully apply one texture, the next one, or any proportion between them.");
}

#undef LOCTEXT_NAMESPACE

