// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureInterpolate.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureInterpolate : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeTextureInterpolate();

	// UObject interface.
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;

	UEdGraphPin* FactorPin() const
	{
		return FindPin(TEXT("Factor"));
	}

	UEdGraphPin* Targets(int32 Index) const
	{
		return FindPin(FString::Printf(TEXT("Target %d"), Index));
	}

	int32 GetNumTargets() const
	{
		int32 Count = 0;

		for (UEdGraphPin* Pin : GetAllNonOrphanPins())
		{
			if (Pin->GetName().StartsWith(TEXT("Target ")))
			{
				Count++;
			}
		}

		return Count;
	}

	UPROPERTY(EditAnywhere, Category=CustomizableObject, meta = (ClampMin = 2))
	int32 NumTargets;
};

#undef UE_API
