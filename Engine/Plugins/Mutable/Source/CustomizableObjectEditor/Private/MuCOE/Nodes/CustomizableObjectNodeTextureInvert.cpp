// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureInvert.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureInvert)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureInvert::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	UEdGraphPin* OutputPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
	OutputPin->bDefaultValueIsIgnored = true;
	
	UEdGraphPin* ImagePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture);
	BaseImagePinReference = FEdGraphPinReference(ImagePin);
}


void UCustomizableObjectNodeTextureInvert::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PinsNamesImageToTexture)
	{
		BaseImagePinReference = FEdGraphPinReference(FindPin(TEXT("Base Image")));
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		bool Replaced = false;
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Image")))
		{
			TexturePin->PinName = TEXT("Texture");
			Replaced = true;
		}

		if (UEdGraphPin* TexturePin = FindPin(TEXT("Base Image")))
		{
			TexturePin->PinName = TEXT("Base Texture");
			Replaced = true;
		}

		if(Replaced)
		{
			UCustomizableObjectNode::ReconstructNode();
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* InputTexturePin = FindPin(TEXT("Base Texture"), EGPD_Input))
		{
			InputTexturePin->PinName = TEXT("Texture");
			InputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}

		if (UEdGraphPin* OutputTexturePin = FindPin(TEXT("Texture"), EGPD_Output))
		{
			OutputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeTextureInvert::GetBaseImagePin() const
{
	return BaseImagePinReference.Get();
}


FText UCustomizableObjectNodeTextureInvert::GetNodeTitle(ENodeTitleType::Type TitleType)const
{
	return LOCTEXT("Texture_Invert", "Texture Invert");
}


FLinearColor UCustomizableObjectNodeTextureInvert::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureInvert::GetTooltipText() const
{
	return LOCTEXT("Texture_Invert_Tooltip", "Inverts the colors of a base texture.");
}


#undef LOCTEXT_NAMESPACE
