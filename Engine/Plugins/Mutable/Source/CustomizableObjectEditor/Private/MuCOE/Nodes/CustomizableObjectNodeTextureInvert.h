// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureInvert.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UObject;

UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureInvert : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TittleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	
	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Own interface
	UE_API UEdGraphPin* GetBaseImagePin() const;

private:
	UPROPERTY()
	FEdGraphPinReference BaseImagePinReference;
};

#undef UE_API
