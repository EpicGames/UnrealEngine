// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureLayer.h"

#include "CustomizableObjectNodeTextureFromColor.h"
#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectSchemaActions.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureLayer)

class UCustomizableObjectNodeRemapPins;
class UEdGraph;
class UEdGraphNode;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeTextureLayer::UCustomizableObjectNodeTextureLayer()
	: Super()
{
	Layers.Add(FCustomizableObjectTextureLayer());
}


void UCustomizableObjectNodeTextureLayer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if (PropertyThatChanged && 
		PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UCustomizableObjectNodeTextureLayer, Layers))
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeTextureLayer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PinsNamesImageToTexture)
	{
		OutputPinReference = FEdGraphPinReference(FindPin(TEXT("Image")));
	}
}


void UCustomizableObjectNodeTextureLayer::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	OutputPinReference = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Texture"), LOCTEXT("Texture", "Texture"));

	BasePin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, TEXT("Base"), LOCTEXT("Base", "Base"));
	
	for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
	{
		FCustomizableObjectTextureLayer& CurrentLayerData = Layers[LayerIndex];
	
		// Determine the type of node the new layer pin will be. We use the old type so we keep the type of operation and tye pin type in sync
		{
			FName NewLayerPinType = UEdGraphSchema_CustomizableObject::PC_Texture;

			// If possible use the same pin type as the pin this Layer was representing before resizing the array
			const FEdGraphPinReference LayerPinReference = CurrentLayerData.LayerPin;
			if (const UEdGraphPin* LayerPin = LayerPinReference.Get())
			{
				NewLayerPinType = LayerPin->PinType.PinCategory;
			}
		
			const FText LayerPinName = FText::Format(LOCTEXT("Texture_Layer_LayerPinName", "Layer {0}"), LayerIndex);
			CurrentLayerData.LayerPin = CustomCreatePin(EGPD_Input, NewLayerPinType, *LayerPinName.ToString(), LayerPinName);
		}

		const FText MaskPinName = FText::Format(LOCTEXT("Texture_Layer_MaskPinName", "Mask {0}"), LayerIndex);
		CurrentLayerData.MaskPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture, *MaskPinName.ToString(), MaskPinName);
	}
}


void UCustomizableObjectNodeTextureLayer::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Image")))
		{
			TexturePin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}

	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::TextureLayerNodePinCaching)
	{
		// Cache base input pin
		BasePin = FindPin(TEXT("Base"), EEdGraphPinDirection::EGPD_Input);
		check(BasePin.Get());

		// Cache layer pin data
		for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
		{
			FCustomizableObjectTextureLayer& CurrentLayer = Layers[LayerIndex];

			const FString LayerPinName = FString::Printf(TEXT("Layer %d "), LayerIndex);
			CurrentLayer.LayerPin = FindPin(LayerPinName, EGPD_Input);
			check(CurrentLayer.LayerPin.Get());

			const FString MaskPinName = FString::Printf(TEXT("Mask %d "), LayerIndex);
			CurrentLayer.MaskPin = FindPin(MaskPinName, EGPD_Input);
			check(CurrentLayer.MaskPin.Get());
		}
	}

	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::TextureLayerNodeExplicitLayerTypePins)
	{
		bool bNodeWasModified = false;

		// Iterate over all Layer pins as those are the ones that could have a color connected. Then create a new TextureFromColor node and connect it
		for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
		{
			FCustomizableObjectTextureLayer& CurrentLayer = Layers[LayerIndex];

			UEdGraphPin* LayerPin = CurrentLayer.LayerPin.Get();
			check(LayerPin);
			check(LayerPin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture);
				
			// Who is connected to this Layer Pin?
			if (LayerPin->LinkedTo.IsEmpty())
			{
				continue;
			}

			const UEdGraphPin* ConnectedNodePin = LayerPin->LinkedTo[0];
			LayerPin->PinType = ConnectedNodePin->PinType;

			bNodeWasModified = true;
		}

		if (bNodeWasModified)
		{
			UEdGraph* Graph = GetGraph();
			check(Graph)
			Graph->NotifyNodeChanged(this);
		}
	}
}


TArray<FName> UCustomizableObjectNodeTextureLayer::GetPinAllowedTypes(const UEdGraphPin& Pin) const
{
	// If this is a layer pin then expose the texture and color options
	const int32 LayerCount = Layers.Num();
	for (int32 LayerIndex = 0; LayerIndex < LayerCount; ++LayerIndex)
	{
		const UEdGraphPin* LayerPin = this->GetLayerPin(LayerIndex);
		check(LayerPin);

		const FEdGraphPinReference LayerRef = LayerPin;
		if (LayerRef == FEdGraphPinReference(&Pin))
	{
			return { 
				UEdGraphSchema_CustomizableObject::PC_Texture, 
				UEdGraphSchema_CustomizableObject::PC_Color, 
			};
		}
	}

	return Super::GetPinAllowedTypes(Pin);
}


FText UCustomizableObjectNodeTextureLayer::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Layer", "Texture Layer");
}


FLinearColor UCustomizableObjectNodeTextureLayer::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureLayer::GetTooltipText() const
{
	return LOCTEXT("Texture_Layer_Tooltip", "Combines multiple textures into one.");
}

#undef LOCTEXT_NAMESPACE

