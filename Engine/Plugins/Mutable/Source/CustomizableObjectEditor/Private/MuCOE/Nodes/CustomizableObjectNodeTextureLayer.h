// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureLayer.generated.h"

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraph;
class UObject;
struct FPropertyChangedEvent;


UENUM()
enum ECustomizableObjectTextureLayerEffect : int
{
	COTLE_MODULATE			= 0 UMETA(DisplayName = "MODULATE"), 
	COTLE_MULTIPLY				UMETA(DisplayName = "MULTIPLY"), 
	COTLE_SOFTLIGHT				UMETA(DisplayName = "SOFTLIGHT"),
	COTLE_HARDLIGHT				UMETA(DisplayName = "HARDLIGHT"),
	COTLE_DODGE					UMETA(DisplayName = "DODGE"),
	COTLE_BURN					UMETA(DisplayName = "BURN"),
	COTLE_SCREEN				UMETA(DisplayName = "SCREEN"),
	COTLE_OVERLAY				UMETA(DisplayName = "OVERLAY"),
	COTLE_ALPHA_OVERLAY			UMETA(DisplayName = "LIGHTEN"),
	COTLE_NORMAL_COMBINE		UMETA(DisplayName = "BLEND NORMALS")
};


USTRUCT()
struct FCustomizableObjectTextureLayer
{
	GENERATED_USTRUCT_BODY()

	FCustomizableObjectTextureLayer()
	{
		Effect = COTLE_SOFTLIGHT;
	}

	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TEnumAsByte<ECustomizableObjectTextureLayerEffect> Effect;
	
	UPROPERTY()
	FEdGraphPinReference LayerPin;
	
	UPROPERTY()
	FEdGraphPinReference MaskPin;
};


UCLASS()
class UCustomizableObjectNodeTextureLayer : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UCustomizableObjectNodeTextureLayer();

	// UObject interface.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void Serialize(FArchive& Ar) override;
	
	// Begin EdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	virtual TArray<FName> GetPinAllowedTypes(const UEdGraphPin& Pin) const override;
		
	UEdGraphPin* GetLayerPin(int32 Index) const
	{
		if (Layers.IsValidIndex(Index))
		{
			return Layers[Index].LayerPin.Get();
		}

		return nullptr;
	}

	UEdGraphPin* GetMaskPin(int32 Index) const
	{
		if (Layers.IsValidIndex(Index))
		{
			return Layers[Index].MaskPin.Get();
		}

		return nullptr;
	}

	
	int32 GetNumLayers() const
	{
		int32 Count = 0;

		for (const FCustomizableObjectTextureLayer& LayerObject : Layers)
		{
			const UEdGraphPin* LayerPin = LayerObject.LayerPin.Get();
			check(LayerPin);
			if (!LayerPin->bOrphanedPin)
			{
				Count++;
			}
		}

		return Count;
	}
	
	UPROPERTY()
	FEdGraphPinReference BasePin;
	
	/**  */
	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TArray<FCustomizableObjectTextureLayer> Layers;

private:
	UPROPERTY()
	FEdGraphPinReference OutputPinReference;
};

