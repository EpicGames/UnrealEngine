// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureParameter.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureParameter)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureParameter::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	Super::AllocateDefaultPins(RemapPins);

	PassthroughPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture_Object);
}


void UCustomizableObjectNodeTextureParameter::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeTextureParameterDefaultToReferenceValue)
	{
		ReferenceValue = DefaultValue;
		DefaultValue = {};
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Texture");
			Pin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::NodeTextureParameterPassthroughPin)
	{
		if (!PassthroughPin.Get())
		{
			PassthroughPin = CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough);
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::PassthroughParameter)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinType.PinCategory ==  UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough)
			{
				Pin->PinType.PinCategory =  UEdGraphSchema_CustomizableObject::PC_Texture_Object;
			}
		}
	}
}


FName UCustomizableObjectNodeTextureParameter::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Texture;
}

#undef LOCTEXT_NAMESPACE

