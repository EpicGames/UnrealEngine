// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "CustomizableObjectNodeTextureParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UObject;
class UTexture2D;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureParameter : public UCustomizableObjectNodeParameter
{
public:
	GENERATED_BODY()

	/** Default value of the parameter. */
	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TObjectPtr<UTexture2D> DefaultValue;

	/** Reference Texture where this parameter copies some properties from. */
	UPROPERTY(EditAnywhere, Category=CustomizableObject)
	TObjectPtr<UTexture2D> ReferenceValue;

	/** Set the width of the Texture when there is no texture reference.*/
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	int32 TextureSizeX = 0;

	/** Set the height of the Texture when there is no texture reference.*/
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	int32 TextureSizeY = 0;

	UPROPERTY()
	FEdGraphPinReference PassthroughPin;
	
	// UCustomizableObjectNode interface
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	virtual FName GetCategory() const override;
};

#undef UE_API
