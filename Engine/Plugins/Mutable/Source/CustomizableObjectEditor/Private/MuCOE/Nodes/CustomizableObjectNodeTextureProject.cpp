// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureProject.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureProject)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


UCustomizableObjectNodeTextureProject::UCustomizableObjectNodeTextureProject()
	: Super()
{
	Textures = 1;
}


void UCustomizableObjectNodeTextureProject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if ( PropertyThatChanged && PropertyThatChanged->GetName() == TEXT("Textures") )
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeTextureProject::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PinsNamesImageToTexture)
	{
		{
			uint32 TexturePinsCount = 0;
			uint32 OutputPinsCount = 0;

			TArray<UEdGraphPin*> NonOrphanPins = GetAllNonOrphanPins();
			
			for (const UEdGraphPin* Pin : NonOrphanPins)
			{
				if (Pin->GetName().StartsWith(TEXT("Image ")) && Pin->Direction == EGPD_Input)
				{
					TexturePinsCount++;
				}
				else if (Pin->GetName().StartsWith(TEXT("Image ")) && Pin->Direction == EGPD_Output)
				{
					OutputPinsCount++;
				}
			}

			TexturePinsReferences.SetNum(OutputPinsCount);
			for (uint32 Index = 0; Index < TexturePinsCount; ++Index)
			{
				TexturePinsReferences.Add(FEdGraphPinReference(FindPin(FString::Printf(TEXT("Image %d "), Index), EGPD_Input)));
			}
			
			OutputPinsReferences.SetNum(TexturePinsCount);
			for (uint32 Index = 0; Index < TexturePinsCount; ++Index)
			{
				OutputPinsReferences.Add(FEdGraphPinReference(FindPin(FString::Printf(TEXT("Image %d "), Index), EGPD_Output)));
			}
		}
	}
}


void UCustomizableObjectNodeTextureProject::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	TexturePinsReferences.Reset(0);
	OutputPinsReferences.Reset(0);
	
	for ( uint32 LayerIndex = 0; LayerIndex < Textures; ++LayerIndex )
	{
		FString PinName = FString::Printf(TEXT("Texture %u"), LayerIndex);
		UEdGraphPin* TexturePin = CustomCreatePin(EGPD_Input, Schema->PC_Texture, FName(*PinName), FText::FromString(PinName));
		
		TexturePinsReferences.Add(FEdGraphPinReference(TexturePin));
		
		PinName = FString::Printf(TEXT("Texture %u"), LayerIndex);
		UEdGraphPin* OutputPin = CustomCreatePin(EGPD_Output, Schema->PC_Texture, FName(*PinName), FText::FromString(PinName));

		OutputPinsReferences.Add(FEdGraphPinReference(OutputPin));
	}

	CustomCreatePin(EGPD_Input, Schema->PC_Mesh, TEXT("Mesh"), LOCTEXT("Mesh", "Mesh"));

	CustomCreatePin(EGPD_Input, Schema->PC_Texture, TEXT("Mesh Mask"), LOCTEXT("MeshMask", "Mesh Mask"));

	UEdGraphPin* AngleFadeStartPin = CustomCreatePin(EGPD_Input, Schema->PC_Float, TEXT("Fade Start Angle"), LOCTEXT("FadeStartAngle", "Fade Start Angle"));
	AngleFadeStartPin->DefaultValue = "90";

	UEdGraphPin* AngleFadeEndPin = CustomCreatePin(EGPD_Input, Schema->PC_Float, TEXT("Fade End Angle"), LOCTEXT("FadeEndAngle", "Fade End Angle"));
	AngleFadeEndPin->DefaultValue = "95";
	
	CustomCreatePin(EGPD_Input, Schema->PC_Projector, TEXT("Projector"), LOCTEXT("Projector", "Projector"));
}


void UCustomizableObjectNodeTextureProject::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		TArray<UEdGraphPin*> AllPins = GetAllPins();
		uint32 TexturePinsCount = 0;
		uint32 OutputPinsCount = 0;

		for (UEdGraphPin* Pin : AllPins)
		{
			if (Pin && Pin->GetName().StartsWith(TEXT("Image "))) {
				if (Pin->Direction == EGPD_Input)
				{
					Pin->PinName = FName(*FString::Printf(TEXT("Texture %d"), TexturePinsCount));
					TexturePinsCount++;
				}
				else if (Pin->Direction == EGPD_Output)
				{
					Pin->PinName = FName(*FString::Printf(TEXT("Texture %d"), OutputPinsCount));
					OutputPinsCount++;
				}
			}
		}
		UCustomizableObjectNode::ReconstructNode();
	}
}


UEdGraphPin* UCustomizableObjectNodeTextureProject::TexturePins(int32 Index) const
{
	return TexturePinsReferences[Index].Get();	
}


UEdGraphPin* UCustomizableObjectNodeTextureProject::OutputPins(int32 Index) const
{
	return OutputPinsReferences[Index].Get();
}


int32 UCustomizableObjectNodeTextureProject::GetNumTextures() const
{
	return TexturePinsReferences.Num();
}

int32 UCustomizableObjectNodeTextureProject::GetNumOutputs() const
{
	return OutputPinsReferences.Num();
}


FText UCustomizableObjectNodeTextureProject::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Project", "Texture Project");
}


FLinearColor UCustomizableObjectNodeTextureProject::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureProject::GetTooltipText() const
{
	return LOCTEXT("Texture_Project_Tooltip",
	"Projects one or more textures on a mesh. Transforms a flat image to a layed out version usable as texture for the input mesh. This node does not modify the mesh section or material.");
}


#undef LOCTEXT_NAMESPACE

