// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureProject.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class UTexture2D;
namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FEdGraphPinReference;
struct FPropertyChangedEvent;

UENUM()
enum class ETextureProjectSamplingMethod
{
	Point,
	BiLinear
};

UENUM()
enum class ETextureProjectMinFilterMethod
{
	None,
	TotalAreaHeuristic
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureProject : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UE_API UCustomizableObjectNodeTextureProject();

	// Layout to use for the generated images.
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	uint8 Layout = 0;

	/** When this is enable, additional operations will happen to correct projections that go over a texture UV seam to prevent interpolation artifacts. 
	* This is not necessary if the projection is guaranteed to not go over a seam.
	*/
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bEnableTextureSeamCorrection = true;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bEnableAngleFadeOutForRGB = true;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bEnableAngleFadeOutForAlpha = true;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	ETextureProjectSamplingMethod SamplingMethod = ETextureProjectSamplingMethod::Point;

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	ETextureProjectMinFilterMethod MinFilterMethod = ETextureProjectMinFilterMethod::None;

	/** Set the width of the Texture. If greater than zero, it overrides the Reference Texture width. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	uint32 TextureSizeX = 0;

	/** Set the height of the Texture. If greater than zero, it overrides the Reference Texture height. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	uint32 TextureSizeY = 0;

	UPROPERTY(EditAnywhere, Category = CustomizableObject, meta=( ClampMin = 1))
	uint32 Textures = 0;

	/** Reference Texture used to decide the texture properties of the mutable-generated textures
	* connected to this material (e.g. LODBias, Size X,...). If null, mutable default texture properties will be applied. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TObjectPtr<UTexture2D> ReferenceTexture = nullptr;

	// UObject interface.
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	
	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	UEdGraphPin* MeshPin() const
	{
		return FindPin(TEXT("Mesh"));
	}

	UEdGraphPin* MeshMaskPin() const
	{
		return FindPin(TEXT("Mesh Mask"));
	}

	UEdGraphPin* AngleFadeStartPin() const
	{
		return FindPin(TEXT("Fade Start Angle"));
	}

	UEdGraphPin* AngleFadeEndPin() const
	{
		return FindPin(TEXT("Fade End Angle"));
	}

	UEdGraphPin* ProjectorPin() const
	{
		return FindPin(TEXT("Projector"));
	}

	UE_API UEdGraphPin* TexturePins(int32 Index) const;

	UE_API UEdGraphPin* OutputPins(int32 Index) const;

	UE_API int32 GetNumTextures() const;

	UE_API int32 GetNumOutputs() const;

private:
	UPROPERTY()
	TArray<FEdGraphPinReference> TexturePinsReferences;

	UPROPERTY()
	TArray<FEdGraphPinReference> OutputPinsReferences;
};

#undef UE_API
