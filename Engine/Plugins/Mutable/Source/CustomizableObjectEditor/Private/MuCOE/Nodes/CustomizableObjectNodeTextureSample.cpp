// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureSample.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureSample)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureSample::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	// Output
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Color);
	
	// Inputs
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture);
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float,  TEXT("X"), LOCTEXT("X", "X"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Y"), LOCTEXT("Y", "Y"));
}

void UCustomizableObjectNodeTextureSample::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* ColorOutput = FindPin(TEXT("Color"), EGPD_Output))
		{
			ColorOutput->PinFriendlyName = LOCTEXT("Color_Pin_Category", "Color");;
		}
		
		if (UEdGraphPin* ImageInput = FindPin(TEXT("Texture"), EGPD_Input))
		{
			ImageInput->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}
}


FText UCustomizableObjectNodeTextureSample::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Sample_Texture", "Color Sample Texture");
}


FLinearColor UCustomizableObjectNodeTextureSample::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Color);
}


FText UCustomizableObjectNodeTextureSample::GetTooltipText() const
{
	return LOCTEXT("Texture_Sample_Tooltip","Get the color found in a texture at the targeted X and Y position (from 0.0 to 1.0, both included).");
}

#undef LOCTEXT_NAMESPACE

