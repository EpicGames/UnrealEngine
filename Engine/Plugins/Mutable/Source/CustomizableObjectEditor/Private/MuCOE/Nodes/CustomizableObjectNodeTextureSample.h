// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureSample.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureSample : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	
	const UEdGraphPin* TexturePin() const
	{
		const FName PinName = UEdGraphSchema_CustomizableObject::GetPinCategoryName(UEdGraphSchema_CustomizableObject::PC_Texture);
		return FindPin(PinName);
	}

	const UEdGraphPin* XPin() const
	{
		return FindPin(TEXT("X"));
	}

	const UEdGraphPin* YPin() const
	{
		return FindPin(TEXT("Y"));
	}
};

#undef UE_API
