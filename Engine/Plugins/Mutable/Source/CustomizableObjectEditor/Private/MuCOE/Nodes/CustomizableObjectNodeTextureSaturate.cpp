// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureSaturate.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureSaturate)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureSaturate::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
	
	BaseImagePinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture);
	
	FactorPinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Factor"), LOCTEXT("Factor", "Factor"));
}


void UCustomizableObjectNodeTextureSaturate::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* InputTexturePin = FindPin(TEXT("Base Texture"), EGPD_Input))
		{
			InputTexturePin->PinName = TEXT("Texture");
			InputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}

		if (UEdGraphPin* OutputTexturePin = FindPin(TEXT("Texture"), EGPD_Output))
		{
			OutputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeTextureSaturate::GetBaseImagePin() const
{
	return BaseImagePinReference.Get();
}


UEdGraphPin* UCustomizableObjectNodeTextureSaturate::GetFactorPin() const
{
	return FactorPinReference.Get();
}


FText UCustomizableObjectNodeTextureSaturate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Saturate", "Texture Saturate");
}


FLinearColor UCustomizableObjectNodeTextureSaturate::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureSaturate::GetTooltipText() const
{
	return LOCTEXT("Texture_Saturate_Tooltip", "Get the provided texture with its saturation adjusted based on the numerical input provided where 1 equals to full saturation and 0 to no saturation.");
}


#undef LOCTEXT_NAMESPACE
