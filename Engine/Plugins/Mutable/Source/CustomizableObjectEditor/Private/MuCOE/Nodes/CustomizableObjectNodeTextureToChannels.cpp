// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureToChannels.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureToChannels)

class UCustomizableObjectNodeRemapPins;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void UCustomizableObjectNodeTextureToChannels::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeTextureToChannels::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectCustomVersion::GUID);

	if (Ar.CustomVer(FCustomizableObjectCustomVersion::GUID) < FCustomizableObjectCustomVersion::PinsNamesImageToTexture)
	{
		InputPinReference = FEdGraphPinReference(FindPin(TEXT("Image")));
	}
}


void UCustomizableObjectNodeTextureToChannels::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	InputPinReference = CustomCreatePin(EGPD_Input, Schema->PC_Texture, TEXT("Texture"), LOCTEXT("Texture", "Texture"));

	CustomCreatePin(EGPD_Output, Schema->PC_Texture, TEXT("R"), LOCTEXT("R", "R"));

	CustomCreatePin(EGPD_Output, Schema->PC_Texture, TEXT("G"), LOCTEXT("G", "G"));
	
	CustomCreatePin(EGPD_Output, Schema->PC_Texture, TEXT("B"), LOCTEXT("B", "B"));
	
	CustomCreatePin(EGPD_Output, Schema->PC_Texture, TEXT("A"), LOCTEXT("A", "A"));
}


void UCustomizableObjectNodeTextureToChannels::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::FixPinsNamesImageToTexture2)
	{
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Image"))) {
			TexturePin->PinName = TEXT("Texture");
			UCustomizableObjectNode::ReconstructNode();
		}
	}
}


UEdGraphPin* UCustomizableObjectNodeTextureToChannels::InputPin() const
{
	return InputPinReference.Get();
}


FText UCustomizableObjectNodeTextureToChannels::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_To_Channels", "Texture Break");
}


FLinearColor UCustomizableObjectNodeTextureToChannels::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureToChannels::GetTooltipText() const
{
	return LOCTEXT("Texture_To_Channels_Tooltip", "Get the red, green, blue and alpha channels of a texture as four separate grayscale textures.");
}


#undef LOCTEXT_NAMESPACE

