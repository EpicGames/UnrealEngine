// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureToChannels.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UObject;
struct FPropertyChangedEvent;


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureToChannels : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()
	
	// UObject interface.
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	
	// Begin EdGraphNode interface
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API FText GetTooltipText() const override;

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	UE_API UEdGraphPin* InputPin() const;

	UEdGraphPin* RPin() const
	{
		return FindPin(TEXT("R"));
	}

	UEdGraphPin* GPin() const
	{
		return FindPin(TEXT("G"));
	}

	UEdGraphPin* BPin() const
	{
		return FindPin(TEXT("B"));
	}

	UEdGraphPin* APin() const
	{
		return FindPin(TEXT("A"));
	}

private:
	UPROPERTY()
	FEdGraphPinReference InputPinReference;
};

#undef UE_API
