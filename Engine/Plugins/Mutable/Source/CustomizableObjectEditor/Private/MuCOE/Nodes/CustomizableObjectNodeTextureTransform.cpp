// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureTransform.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureTransform)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCustomizableObjectNodeTextureTransform::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Texture);
	
	BaseImagePinReference = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Texture);
	
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Offset X"), LOCTEXT("OffsetX", "Offset X"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Offset Y"), LOCTEXT("OffsetY", "Offset Y"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Scale X"), LOCTEXT("ScaleX", "Scale X"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Scale Y"), LOCTEXT("ScaleY", "Scale Y"));
	CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_Float, TEXT("Rotation"), LOCTEXT("Rotation", "Rotation"));
}

void UCustomizableObjectNodeTextureTransform::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* InputTexturePin = FindPin(TEXT("Base Texture"), EGPD_Input))
		{
			InputTexturePin->PinName = TEXT("Texture");
			InputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}

		if (UEdGraphPin* OutputTexturePin = FindPin(TEXT("Texture"), EGPD_Output))
		{
			OutputTexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}
}

UEdGraphPin* UCustomizableObjectNodeTextureTransform::GetBaseImagePin() const
{
	return BaseImagePinReference.Get();
}

FText UCustomizableObjectNodeTextureTransform::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Transform", "Texture Transform");
}

FLinearColor UCustomizableObjectNodeTextureTransform::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}

FText UCustomizableObjectNodeTextureTransform::GetTooltipText() const
{
	return LOCTEXT("Texture_Transform_Tooltip", 
			"Applies a linear transform, rotation and scale around the center of the image plus translation, "
			"to the content of Base Texture. Rotation is in the range [0 .. 1], 1 being full rotation, offset " 
			"and scale are in output image normalized coordinates with origin at the center of the image. " 
			"If Keep Aspect Ratio is set, an scaling factor preserving aspect ratio will be used as identity.");
}



#undef LOCTEXT_NAMESPACE
