// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeTextureTransform.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }

class FArchive;
class UCustomizableObjectNodeRemapPins;
class UObject;

UENUM()
enum class ETextureTransformAddressMode
{
	Wrap,
	ClampToEdge,
	ClampToBlack
};

UCLASS(MinimalAPI)
class UCustomizableObjectNodeTextureTransform : public UCustomizableObjectNode
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	ETextureTransformAddressMode AddressMode = ETextureTransformAddressMode::Wrap;


	/** If true, the operation will keep the source aspect ratio regardless of the output image size */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	bool bKeepAspectRatio = false;

	/** Set the width of the Texture. If greater than zero, it overrides the Reference Texture width. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	uint32 TextureSizeX = 0;

	/** Set the height of the Texture. If greater than zero, it overrides the Reference Texture height. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	uint32 TextureSizeY = 0;

	/** Reference Texture used to decide the texture properties of the mutable-generated textures
	* connected to this material (e.g. LODBias, Size X,...). If null, mutable default texture properties will be applied. */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TObjectPtr<UTexture2D> ReferenceTexture = nullptr;


	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TittleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	//virtual void Serialize(FArchive& Ar) override;
	
	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	UE_API UEdGraphPin* GetBaseImagePin() const;

	UEdGraphPin* GetOffsetXPin() const
	{
		return FindPin(TEXT("Offset X"));
	}

	UEdGraphPin* GetOffsetYPin() const
	{
		return FindPin(TEXT("Offset Y"));
	}

	UEdGraphPin* GetScaleXPin() const
	{
		return FindPin(TEXT("Scale X"));
	}

	UEdGraphPin* GetScaleYPin() const
	{
		return FindPin(TEXT("Scale Y"));
	}

	UEdGraphPin* GetRotationPin() const
	{
		return FindPin(TEXT("Rotation"));
	}

private:
	UPROPERTY()
	FEdGraphPinReference BaseImagePinReference;
};

#undef UE_API
