// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTextureVariation)

class UCustomizableObjectNodeRemapPins;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


FName UCustomizableObjectNodeTextureVariation::GetCategory() const
{
	return UEdGraphSchema_CustomizableObject::PC_Texture;
}


void UCustomizableObjectNodeTextureVariation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FProperty* PropertyThatChanged = PropertyChangedEvent.Property;
	if ( PropertyThatChanged  )
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeTextureVariation::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName3)
	{
		if (UEdGraphPin* TexturePin = FindPin(TEXT("Texture")))
		{
			TexturePin->PinFriendlyName = LOCTEXT("Image_Pin_Category", "Texture");
		}
	}

	// Converting TextureVariations to VariationNodes
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::TextureVariationsToVariations)
	{
		VariationsPins.SetNum(Variations_DEPRECATED.Num());
		VariationsData.SetNum(Variations_DEPRECATED.Num());

		for (int32 VariationIndex = 0; VariationIndex < Variations_DEPRECATED.Num(); ++VariationIndex)
		{
			if (UEdGraphPin* VariationPin = FindPin(FString::Printf(TEXT("Variation %d"), VariationIndex)))
			{
				VariationsPins[VariationIndex] = VariationPin;
				VariationsData[VariationIndex].Tag = Variations_DEPRECATED[VariationIndex].Tag;
			}
			else
			{
				check(false);
			}
		}
	}
}


FText UCustomizableObjectNodeTextureVariation::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Texture_Variation", "Texture Variation");
}


FLinearColor UCustomizableObjectNodeTextureVariation::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Texture);
}


FText UCustomizableObjectNodeTextureVariation::GetTooltipText() const
{
	return LOCTEXT("Texture_Variation_Tooltip", "Select a texture depending on what tags are active.");
}

#undef LOCTEXT_NAMESPACE

