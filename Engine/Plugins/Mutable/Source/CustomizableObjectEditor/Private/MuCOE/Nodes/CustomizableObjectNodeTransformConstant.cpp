// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTransformConstant.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SGridPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTransformConstant)


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SGraphNodeTransformConstant::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	NodeTransformConstant = Cast<UCustomizableObjectNodeTransformConstant>(InGraphNode);

	SCustomizableObjectNode::Construct({}, InGraphNode);
}


void SGraphNodeTransformConstant::SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget)
{
	// Collapsing arrow of the title area
	DefaultTitleAreaWidget->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(FMargin(5))
	[
		SNew(SCheckBox)
		.OnCheckStateChanged(this, &SGraphNodeTransformConstant::OnExpressionPreviewChanged)
		.IsChecked(IsExpressionPreviewChecked())
		.Cursor(EMouseCursor::Default)
		.Style(FAppStyle::Get(), "Graph.Node.AdvancedView")
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(GetExpressionPreviewArrow())
			]
		]
	];
}


void SGraphNodeTransformConstant::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	constexpr float TransformBoxMinWidth = 250.0f;
	
	MainBox->AddSlot()
	.AutoHeight()
	.Padding(10.0f, 0.0f, 10.0f, 10.0f)
	[
		SNew(SHorizontalBox)
		.Visibility(NodeTransformConstant->bCollapsed ? EVisibility::Collapsed : EVisibility::Visible)

		+ SHorizontalBox::Slot()
		.MinWidth(TransformBoxMinWidth)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1)
			+ SGridPanel::Slot(0, 0)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Transform_Location", "Location"))
			]
			+ SGridPanel::Slot(1, 0)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Fill)
			[
				SNew(SNumericVectorInputBox<double>)
				.bColorAxisLabels(true)
				.AllowSpin(true)
				.X_Lambda([Node=NodeTransformConstant]() { return Node->Value.GetLocation().X; })
				.Y_Lambda([Node=NodeTransformConstant]() { return Node->Value.GetLocation().Y; })
				.Z_Lambda([Node=NodeTransformConstant]() { return Node->Value.GetLocation().Z; })
				.OnXChanged(this, &SGraphNodeTransformConstant::OnLocationChanged, ETextCommit::Default, EAxis::X)
				.OnYChanged(this, &SGraphNodeTransformConstant::OnLocationChanged, ETextCommit::Default, EAxis::Y)
				.OnZChanged(this, &SGraphNodeTransformConstant::OnLocationChanged, ETextCommit::Default, EAxis::Z)
				.OnXCommitted(this, &SGraphNodeTransformConstant::OnLocationChanged, EAxis::X)
				.OnYCommitted(this, &SGraphNodeTransformConstant::OnLocationChanged, EAxis::Y)
				.OnZCommitted(this, &SGraphNodeTransformConstant::OnLocationChanged, EAxis::Z)
			]
			
			+ SGridPanel::Slot(0, 1)
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Transform_Rotation", "Rotation"))
			]
			+ SGridPanel::Slot(1, 1)
			.HAlign(HAlign_Fill)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SNumericRotatorInputBox<double>)
				.bColorAxisLabels(true)
				.AllowSpin(true)
				.Roll_Lambda([Node=NodeTransformConstant]() { return Node->Value.Rotator().Roll; })
				.Pitch_Lambda([Node=NodeTransformConstant]() { return Node->Value.Rotator().Pitch; })
				.Yaw_Lambda([Node=NodeTransformConstant]() { return Node->Value.Rotator().Yaw; })
				.OnRollChanged(this, &SGraphNodeTransformConstant::OnRotationChanged, ETextCommit::Default, EAxis::X)
				.OnPitchChanged(this, &SGraphNodeTransformConstant::OnRotationChanged, ETextCommit::Default, EAxis::Y)
				.OnYawChanged(this, &SGraphNodeTransformConstant::OnRotationChanged, ETextCommit::Default, EAxis::Z)
				.OnRollCommitted(this, &SGraphNodeTransformConstant::OnRotationChanged, EAxis::X)
				.OnPitchCommitted(this, &SGraphNodeTransformConstant::OnRotationChanged, EAxis::Y)
				.OnYawCommitted(this, &SGraphNodeTransformConstant::OnRotationChanged, EAxis::Z)
			]

			+ SGridPanel::Slot(0, 2)
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Transform_Scale", "Scale"))
			]
			+ SGridPanel::Slot(1, 2)
			.HAlign(HAlign_Fill)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SNumericVectorInputBox<double>)
				.bColorAxisLabels(true)
				.AllowSpin(true)
				.X_Lambda([Node=NodeTransformConstant]() { return Node->Value.GetScale3D().X; })
				.Y_Lambda([Node=NodeTransformConstant]() { return Node->Value.GetScale3D().Y; })
				.Z_Lambda([Node=NodeTransformConstant]() { return Node->Value.GetScale3D().Z; })
				.OnXChanged(this, &SGraphNodeTransformConstant::OnScaleChanged, ETextCommit::Default, EAxis::X)
				.OnYChanged(this, &SGraphNodeTransformConstant::OnScaleChanged, ETextCommit::Default, EAxis::Y)
				.OnZChanged(this, &SGraphNodeTransformConstant::OnScaleChanged, ETextCommit::Default, EAxis::Z)
				.OnXCommitted(this, &SGraphNodeTransformConstant::OnScaleChanged, EAxis::X)
				.OnYCommitted(this, &SGraphNodeTransformConstant::OnScaleChanged, EAxis::Y)
				.OnZCommitted(this, &SGraphNodeTransformConstant::OnScaleChanged, EAxis::Z)
			]
		]
	];
}


void SGraphNodeTransformConstant::OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState)
{
	NodeTransformConstant->bCollapsed = (NewCheckedState != ECheckBoxState::Checked);
	UpdateGraphNode();
}


ECheckBoxState SGraphNodeTransformConstant::IsExpressionPreviewChecked() const
{
	return NodeTransformConstant->bCollapsed ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


const FSlateBrush* SGraphNodeTransformConstant::GetExpressionPreviewArrow() const
{
	return FCustomizableObjectEditorStyle::Get().GetBrush(NodeTransformConstant->bCollapsed ? TEXT("Nodes.ArrowDown") : TEXT("Nodes.ArrowUp"));
}


void SGraphNodeTransformConstant::OnLocationChanged(double Value, ETextCommit::Type, EAxis::Type Axis)
{
	FVector Location = NodeTransformConstant->Value.GetLocation();
	if (!FMath::IsNearlyEqualByULP(Value, Location.GetComponentForAxis(Axis)))
	{
		Location.SetComponentForAxis(Axis, Value);
		NodeTransformConstant->Value.SetLocation(Location);

		(void)NodeTransformConstant->MarkPackageDirty();
	}
}


void SGraphNodeTransformConstant::OnRotationChanged(double Value, ETextCommit::Type, EAxis::Type Axis)
{
	FRotator Rotation = NodeTransformConstant->Value.Rotator();
	Rotation.SetComponentForAxis(Axis, Value);
	
	if (!NodeTransformConstant->Value.Rotator().Equals(Rotation))
	{
		NodeTransformConstant->Value.SetRotation(Rotation.Quaternion());

		(void)NodeTransformConstant->MarkPackageDirty();
	}
}


void SGraphNodeTransformConstant::OnScaleChanged(double Value, ETextCommit::Type, EAxis::Type Axis)
{
	FVector Scale = NodeTransformConstant->Value.GetScale3D();
	if (!FMath::IsNearlyEqualByULP(Value, Scale.GetComponentForAxis(Axis)))
	{
		Scale.SetComponentForAxis(Axis, Value);
		NodeTransformConstant->Value.SetScale3D(Scale);

		(void)NodeTransformConstant->MarkPackageDirty();
	}
}


// ============================================
// === UCustomizableObjectNodeTransformConstant


FText UCustomizableObjectNodeTransformConstant::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Transform_Constant", "Transform");
}


FLinearColor UCustomizableObjectNodeTransformConstant::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(UEdGraphSchema_CustomizableObject::PC_Transform);
}


FText UCustomizableObjectNodeTransformConstant::GetTooltipText() const
{
	return LOCTEXT("Transform_Constant_Tooltip", "Define a constant transform value.");
}


void UCustomizableObjectNodeTransformConstant::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UEdGraphSchema_CustomizableObject::PC_Transform);
}


void UCustomizableObjectNodeTransformConstant::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);

	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::UpdatedNodesPinName2)
	{
		if (UEdGraphPin* Pin = FindPin(TEXT("Value")))
		{
			Pin->PinName = TEXT("Transform");
			Pin->PinFriendlyName =  LOCTEXT("Transform_Pin_Category", "Transform");
		}
	}
}


TSharedPtr<SGraphNode> UCustomizableObjectNodeTransformConstant::CreateVisualWidget()
{
	return SNew(SGraphNodeTransformConstant, this);
}

#undef LOCTEXT_NAMESPACE
