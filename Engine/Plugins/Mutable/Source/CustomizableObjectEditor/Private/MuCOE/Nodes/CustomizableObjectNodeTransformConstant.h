// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/SCustomizableObjectNode.h"

#include "CustomizableObjectNodeTransformConstant.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


class UCustomizableObjectNodeTransformConstant;


// Class create color inputs in the NodeColorConstant
class SGraphNodeTransformConstant : public SCustomizableObjectNode
{
public:

	SLATE_BEGIN_ARGS(SGraphNodeTransformConstant) {}
	SLATE_END_ARGS();

	SGraphNodeTransformConstant() = default;

	// Builds the SGraphNodeFloatConstant when needed
	void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// Overriden functions to build the SGraphNode widgets
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual bool ShouldAllowCulling() const override { return false; }

	// Callbacks for the widget
	void OnExpressionPreviewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsExpressionPreviewChecked() const;
	const FSlateBrush* GetExpressionPreviewArrow() const;

private:

	// Callback for when the location part of the transform changes.
	void OnLocationChanged(double Value, ETextCommit::Type, EAxis::Type Axis);

	// Callback for when the rotation part of the transform changes.
	void OnRotationChanged(double Value, ETextCommit::Type, EAxis::Type Axis);

	// Callback for when the scale part of the transform changes.
	void OnScaleChanged(double Value, ETextCommit::Type, EAxis::Type Axis);

private:
	// Pointer to the transform constant node that owns this SGraphNode
	UCustomizableObjectNodeTransformConstant* NodeTransformConstant = nullptr;

};


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTransformConstant : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:	
	/**  */
	UPROPERTY(EditAnywhere, Category=CustomizableObject, meta = (DontUpdateWhileEditing))
	FTransform Value = FTransform::Identity;

	// Begin EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	// End EdGraphNode interface

	// UCustomizableObjectNode interface
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool IsAffectedByLOD() const override { return false; }
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// Creates the SGraph Node widget for the Color Editor
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;

	// Determines if the Node is collapsed or not
	bool bCollapsed = true;
};

#undef UE_API
