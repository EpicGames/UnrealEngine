// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNodeParameter.h"

#include "CustomizableObjectNodeTransformParameter.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType { enum Type : int; }


UCLASS(MinimalAPI)
class UCustomizableObjectNodeTransformParameter : public UCustomizableObjectNodeParameter
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=CustomizableObject, meta = (DontUpdateWhileEditing))
	FTransform DefaultValue = FTransform::Identity;

	// Begin EdGraphNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;

	// CustomizableObjectNodeParameter interface
	UE_API virtual FName GetCategory() const override;
	
};

#undef UE_API
