// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeTunnel.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeTunnel)

#define LOCTEXT_NAMESPACE "CustomizableObjectNodeTunnel"


bool UCustomizableObjectNodeTunnelRemapPins::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const
{
	const UCustomizableObjectNodeTunnelPinData* PinDataOldPin = Cast<UCustomizableObjectNodeTunnelPinData>(Node.GetPinData(OldPin));
	const UCustomizableObjectNodeTunnelPinData* PinDataNewPin = Cast<UCustomizableObjectNodeTunnelPinData>(Node.GetPinData(NewPin));

	return PinDataOldPin->VariableId == PinDataNewPin->VariableId && OldPin.PinType.PinCategory == NewPin.PinType.PinCategory;
}


void UCustomizableObjectNodeTunnelRemapPins::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	for (UEdGraphPin* OldPin : OldPins)
	{
		bool bFound = false;

		for (UEdGraphPin* NewPin : NewPins)
		{
			if (Equal(Node, *OldPin, *NewPin))
			{
				bFound = true;

				PinsToRemap.Add(OldPin, NewPin);
				break;
			}
		}

		if (!bFound && OldPin->LinkedTo.Num())
		{
			PinsToOrphan.Add(OldPin);
		}
	}
}


UCustomizableObjectNodeTunnelRemapPins* UCustomizableObjectNodeTunnel::CreateRemapPinsDefault() const
{
	return NewObject<UCustomizableObjectNodeTunnelRemapPins>();
}


FText UCustomizableObjectNodeTunnel::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (bIsInputNode)
	{
		return LOCTEXT("TunnelInput_Text", "Inputs");
	}
	else
	{
		return LOCTEXT("TunnelOutput_Text", "Outputs");
	}
}


FText UCustomizableObjectNodeTunnel::GetTooltipText() const
{
	return LOCTEXT("TunnelTooltipText", "Node Tunnel");
}


FLinearColor UCustomizableObjectNodeTunnel::GetNodeTitleColor() const
{
	return FLinearColor(0.15f, 0.15f, 0.15f);
}


bool UCustomizableObjectNodeTunnel::CanUserDeleteNode() const
{
	return false;
}


bool UCustomizableObjectNodeTunnel::CanDuplicateNode() const
{
	return false;
}


void UCustomizableObjectNodeTunnel::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	for (const UCustomizableObjectMacroInputOutput* Variable : ParentMacro->InputOutputs)
	{
		ECOMacroIOType NodeType = bIsInputNode ? ECOMacroIOType::COMVT_Input : ECOMacroIOType::COMVT_Output;
		EEdGraphPinDirection PinDirection = bIsInputNode ? EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input;

		if (Variable->Type == NodeType)
		{
			UCustomizableObjectNodeTunnelPinData* PinData = NewObject<UCustomizableObjectNodeTunnelPinData>(this);
			PinData->VariableId = Variable->UniqueId;

			CustomCreatePin(PinDirection, Variable->PinCategoryType, Variable->Name, FText::FromName(Variable->Name), PinData);
		}
	}
}


#undef LOCTEXT_NAMESPACE
