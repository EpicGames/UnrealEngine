// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"

#include "CustomizableObjectNodeTunnel.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API


/** Base class for all Tunnel Pins. */
UCLASS(MinimalAPI)
class UCustomizableObjectNodeTunnelPinData : public UCustomizableObjectNodePinData
{
	GENERATED_BODY()

public:

	/** Id of the variable associated to a tunnel node pin */
	UPROPERTY()
	FGuid VariableId;

};


UCLASS()
class UCustomizableObjectNodeTunnelRemapPins : public UCustomizableObjectNodeRemapPinsByName
{
	GENERATED_BODY()
public:

	// Specific method to decide when two pins are equal
	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const override;

	// Method to use in the RemapPins step of the node reconstruction process
	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};



UCLASS(MinimalAPI)
class UCustomizableObjectNodeTunnel : public UCustomizableObjectNode
{
public:

	/** Indicates if this node is an input or output node. */
	UPROPERTY()
	bool bIsInputNode = false;

	/** Pointer to the macro where this node belongs. */
	UPROPERTY()
	TObjectPtr<UCustomizableObjectMacro> ParentMacro;

public:

	GENERATED_BODY()

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual bool CanDuplicateNode() const override;


	// UCustomizableObjectNode interface
	UE_API void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual UCustomizableObjectNodeTunnelRemapPins* CreateRemapPinsDefault() const override;
};

#undef UE_API
