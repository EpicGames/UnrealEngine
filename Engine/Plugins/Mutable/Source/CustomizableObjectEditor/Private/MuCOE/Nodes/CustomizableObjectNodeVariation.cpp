// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"

#include "MuCO/CustomizableObjectCustomVersion.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNodeStaticString.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeVariation)

class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeRemapPinsByName;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void UCustomizableObjectNodeVariation::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void UCustomizableObjectNodeVariation::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

	const FName Category = GetCategory();
	const bool bIsInputPinArray = IsInputPinArray();

	{
		CustomCreatePin(EGPD_Output, Category);
	}
	
	VariationsPins.SetNum(VariationsData.Num());
	VariationTagPins.SetNum(VariationsData.Num());

	for (int32 VariationIndex = VariationsData.Num() - 1; VariationIndex >= 0; --VariationIndex)
	{
		const FName PinName = FName(FString::Printf( TEXT("Variation %d"), VariationIndex));
		UEdGraphPin* VariationPin = CustomCreatePin(EGPD_Input, Category, PinName, FText::FromName(PinName));
		if (bIsInputPinArray)
		{
			VariationPin->PinType.ContainerType = EPinContainerType::Array;
		}
		
		VariationsPins[VariationIndex] = VariationPin;

		const FText VariationTagSuffix = LOCTEXT("TagSuffix", " - Tag");
		const FName TagPinName = FName(PinName.ToString() + VariationTagSuffix.ToString());
		UEdGraphPin* VariationTagPin = CustomCreatePin(EGPD_Input, Schema->PC_String, TagPinName, FText::FromName(TagPinName));

		VariationTagPins[VariationIndex] = VariationTagPin;
	}

	UEdGraphPin* Pin = CustomCreatePin(EGPD_Input, Category, TEXT("Default"), LOCTEXT("Default", "Default"));
	if (bIsInputPinArray)
	{
		Pin->PinType.ContainerType = EPinContainerType::Array;
	}
}


bool UCustomizableObjectNodeVariation::CanRenamePin(const UEdGraphPin& Pin) const
{
	return Pin.PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_String;
}


FText UCustomizableObjectNodeVariation::GetPinEditableName(const UEdGraphPin& Pin) const
{
	for (int32 VariationIndex = 0; VariationIndex < GetNumVariations(); ++VariationIndex)
	{
		if (VariationTagPin(VariationIndex) == &Pin)
		{
			return FText::FromString(GetVariationTag(VariationIndex));
		}
	}

	return {};
}


void UCustomizableObjectNodeVariation::SetPinEditableName(const UEdGraphPin& Pin, const FText& InValue)
{
	for (int32 VariationIndex = 0; VariationIndex < GetNumVariations(); ++VariationIndex)
	{
		if (VariationTagPin(VariationIndex) == &Pin && VariationsData.IsValidIndex(VariationIndex))
		{
			FScopedTransaction LocalTransaction(LOCTEXT("UCustomizableObjectNodeVariation_SetVariationTagPinEditableNameTransaction", "Change Variation Tag PinName"));
			Modify();

			VariationsData[VariationIndex].Tag = InValue.ToString();
			PinEditableNameChangedDelegate.Broadcast(Pin, InValue);
			break;
		}
	}
}


bool UCustomizableObjectNodeVariation::IsInputPinArray() const
{
	return false;
}


int32 UCustomizableObjectNodeVariation::GetNumVariations() const
{
	return VariationsData.Num();
}


const FCustomizableObjectVariation& UCustomizableObjectNodeVariation::GetVariation(int32 Index) const
{
	return VariationsData[Index];
}


FString UCustomizableObjectNodeVariation::GetVariationTag(int32 Index, TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext) const
{
	if (const UEdGraphPin* TagPin = VariationTagPin(Index))
	{
		if (const UEdGraphPin* ConnectedStringPin = FollowInputPin(*TagPin))
		{
			if (const UEdGraphPin* SourceStringPin = GraphTraversal::FindIOPinSourceThroughMacroContext(*ConnectedStringPin, MacroContext))
			{
				if (const UCustomizableObjectNodeStaticString* StringNode = Cast<UCustomizableObjectNodeStaticString>(SourceStringPin->GetOwningNode()))
				{
					return StringNode->Value;
				}
			}
		}
	}

	return GetVariation(Index).Tag;
}


UEdGraphPin* UCustomizableObjectNodeVariation::DefaultPin() const
{
	return FindPin(TEXT("Default"));
}


UEdGraphPin* UCustomizableObjectNodeVariation::VariationPin(int32 Index) const
{
	if (VariationsPins.IsValidIndex(Index))
	{
		return VariationsPins[Index].Get();
	}

	return nullptr;
}


UEdGraphPin* UCustomizableObjectNodeVariation::VariationTagPin(int32 Index) const
{
	if (VariationTagPins.IsValidIndex(Index))
	{
		return VariationTagPins[Index].Get();
	}

	return nullptr;
}


FText UCustomizableObjectNodeVariation::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::Format(LOCTEXT("Variation_Node_Title", "{0} Variation"), UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetCategory()));
}


FLinearColor UCustomizableObjectNodeVariation::GetNodeTitleColor() const
{
	return UEdGraphSchema_CustomizableObject::GetPinTypeColor(GetCategory());
}


FText UCustomizableObjectNodeVariation::GetTooltipText() const
{
	return FText::Format(LOCTEXT("Variation_Tooltip", "Select a {0} depending on what tags are active."), UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetCategory()));
}


void UCustomizableObjectNodeVariation::BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion)
{
	Super::BackwardsCompatibleFixup(CustomizableObjectCustomVersion);
	
	if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::EnableMutableMacrosNewVersion)
	{
		if (VariationTagPins.IsEmpty())
		{
			VariationTagPins.SetNum(VariationsData.Num());

			for (int32 VariationIndex = VariationsData.Num() - 1; VariationIndex >= 0; --VariationIndex)
			{
				const FName PinName = FName(FString::Printf(TEXT("Variation %d"), VariationIndex));
				FText VariationTagSuffix = LOCTEXT("TagSuffix", " - Tag");
				const FName TagPinName = FName(PinName.ToString() + VariationTagSuffix.ToString());
				UEdGraphPin* VariationTagPin = CustomCreatePin(EGPD_Input, UEdGraphSchema_CustomizableObject::PC_String, TagPinName, FText::FromName(TagPinName));
				VariationTagPins[VariationIndex] = VariationTagPin;
			}
		}
	}
	else if (CustomizableObjectCustomVersion == FCustomizableObjectCustomVersion::WrongVariationNodeInputOrderAfterSerializationFix)
	{
		UCustomizableObjectNodeRemapPinsByName* Remapper =  NewObject<UCustomizableObjectNodeRemapPinsByName>();

		FixupReconstructPins(Remapper, [](UCustomizableObjectNode* Node, UCustomizableObjectNodeRemapPins* Action)
		{
			// Code extracted from AllocateDefaultPins
				
			UCustomizableObjectNodeVariation* CastedNode = Cast<UCustomizableObjectNodeVariation>(Node);
			check(CastedNode)
			
			const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();

			const FName Category = CastedNode->GetCategory();
			const bool bIsInputPinArray = CastedNode->IsInputPinArray();

			{
				CastedNode->CustomCreatePin(EGPD_Output, Category);
			}
			
			CastedNode->VariationsPins.SetNum(CastedNode->VariationsData.Num());
			CastedNode->VariationTagPins.SetNum(CastedNode->VariationsData.Num());

			for (int32 VariationIndex = CastedNode->VariationsData.Num() - 1; VariationIndex >= 0; --VariationIndex)
			{
				const FName PinName = FName(FString::Printf( TEXT("Variation %d"), VariationIndex));
				UEdGraphPin* VariationPin = CastedNode->CustomCreatePin(EGPD_Input, Category, PinName, FText::FromName(PinName));
				if (bIsInputPinArray)
				{
					VariationPin->PinType.ContainerType = EPinContainerType::Array;
				}
				
				CastedNode->VariationsPins[VariationIndex] = VariationPin;

				const FText VariationTagSuffix = LOCTEXT("TagSuffix", " - Tag");
				const FName TagPinName = FName(PinName.ToString() + VariationTagSuffix.ToString());
				UEdGraphPin* VariationTagPin = CastedNode->CustomCreatePin(EGPD_Input, Schema->PC_String, TagPinName, FText::FromName(TagPinName));

				CastedNode->VariationTagPins[VariationIndex] = VariationTagPin;
			}

			UEdGraphPin* Pin = CastedNode->CustomCreatePin(EGPD_Input, Category, TEXT("Default"), LOCTEXT("Default", "Default"));
			if (bIsInputPinArray)
			{
				Pin->PinType.ContainerType = EPinContainerType::Array;
			}
		});
	}
}


#undef LOCTEXT_NAMESPACE

