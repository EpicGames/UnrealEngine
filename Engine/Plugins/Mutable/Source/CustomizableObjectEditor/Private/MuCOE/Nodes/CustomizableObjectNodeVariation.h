// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeVariation.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

namespace ENodeTitleType
{
	enum Type : int;
}

class UCustomizableObjectNodeRemapPins;
class UEdGraphPin;
class UObject;
struct FPropertyChangedEvent;


USTRUCT()
struct FCustomizableObjectVariation
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FString Tag;
};


UCLASS(MinimalAPI, Abstract)
class UCustomizableObjectNodeVariation : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	// UObject interface.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// EdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	
	// UCustomizableObjectNode interface
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) override;
	UE_API virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	UE_API virtual bool CanRenamePin(const UEdGraphPin& Pin) const override;
	UE_API virtual FText GetPinEditableName(const UEdGraphPin& Pin) const override;
	UE_API virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value) override;
	
	// Own interface
	/** Return the pin category of this node. */
	virtual FName GetCategory() const PURE_VIRTUAL(UCustomizableObjectNodeVariation::GetCategory, return {}; );

	/** Return true if all inputs pins should be array. */
	UE_API virtual bool IsInputPinArray() const;

	/** Return the number of variations (input pins excluding the Default Pin). */
	UE_API int32 GetNumVariations() const;

	/** Get the variation at the given index. */
	UE_API const FCustomizableObjectVariation& GetVariation(int32 Index) const;

	/** Get the variation tag at the given index */
	UE_API FString GetVariationTag(int32 Index, TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr) const;

	/** Get the Default Input Pin. */
	UE_API UEdGraphPin* DefaultPin() const;

	/** Get the Variation Input Pin. */
	UE_API UEdGraphPin* VariationPin(int32 Index) const;

	/** Get the Variation Tag Input Pin */
	UE_API UEdGraphPin* VariationTagPin(int32 Index) const;
	
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	TArray<FCustomizableObjectVariation> VariationsData; // The variable name can not be Variations due issues with the UObject Serialization system

protected:

	UPROPERTY()
	TArray<FEdGraphPinReference> VariationsPins;

	UPROPERTY()
	TArray<FEdGraphPinReference> VariationTagPins;
};

#undef UE_API
