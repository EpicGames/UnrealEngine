// Copyright Epic Games, Inc. All Rights Reserved.

// MyConfigCustomization.cpp

#include "ExternalTypeCustomization.h"

#include "DetailWidgetRow.h"
#include "InstancedStructDetails.h"
#include "StructUtilsMetadata.h"
#include "Widgets/Layout/SBox.h"


bool FExternalTypeTypeIdentifier::IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const
{
	return InPropertyHandle.HasMetaData(TEXT("MutableExternalType"));
}

TSharedRef<IPropertyTypeCustomization> FExternalTypeCustomization::MakeInstance()
{
	return MakeShareable(new FExternalTypeCustomization());
}

void FExternalTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Make the type constant.
	PropertyHandle->SetInstanceMetaData(UE::StructUtils::Metadata::StructTypeConstName, TEXT(""));

	FInstancedStructDetails::CustomizeHeader(PropertyHandle, HeaderRow, CustomizationUtils);

	// Disable reset button since it allows to remove the type.
	HeaderRow.OverrideResetToDefault(FResetToDefaultOverride::Create(TAttribute<bool>(false), false));

	// Remove type picker.
	HeaderRow.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
		];
}

