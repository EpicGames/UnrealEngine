// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyEditorModule.h"

#include "InstancedStructDetails.h"
#include "IPropertyTypeCustomization.h"


class FExternalTypeTypeIdentifier : public IPropertyTypeIdentifier 
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override;
};


class FExternalTypeCustomization : public FInstancedStructDetails
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};

