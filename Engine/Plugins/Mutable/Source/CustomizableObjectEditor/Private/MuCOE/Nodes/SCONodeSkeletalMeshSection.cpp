// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/SCONodeSkeletalMeshSection.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/SCONodeSkeletalMeshSectionPinImage.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"

class SGraphPin;


void SCONodeSkeletalMeshSection::Construct(const FArguments& InArgs, UCONodeSkeletalMeshSection* InGraphNode)
{
	SCustomizableObjectNode::Construct({}, InGraphNode);
}


TSharedPtr<SGraphPin> SCONodeSkeletalMeshSection::CreatePinWidget(UEdGraphPin* Pin) const
{
	if ((Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture || Pin->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture_Passthrough) &&
		Pin->Direction == EGPD_Input)
	{
		return SNew(SCONodeSkeletalMeshSectionPinImage, Pin);
	}

	return SGraphNode::CreatePinWidget(Pin);
}
