// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCustomizableObjectNode.h"
#include "SGraphNode.h"

class UCONodeSkeletalMeshSection;
class SGraphPin;
class UEdGraphPin;


/** Custom widget for the Material node. */
class SCONodeSkeletalMeshSection : public SCustomizableObjectNode
{
public:
	SLATE_BEGIN_ARGS(SCONodeSkeletalMeshSection) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UCONodeSkeletalMeshSection* InGraphNode);

	// SGraphNode interface
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
};
