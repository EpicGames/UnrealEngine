// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Nodes/SCONodeSkeletalMeshSectionPinImage.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SToolTip.h"

class IToolTip;
class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SCONodeSkeletalMeshSectionPinImage::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SCustomizableObjectNodePin::Construct(SCustomizableObjectNodePin::FArguments(), InGraphPinObj);

	// Override previously defined tool tip.
	TSharedPtr<IToolTip> TooltipWidget = SNew(SToolTip)
		.Text(this, &SCONodeSkeletalMeshSectionPinImage::GetPinTooltipText);

	SetToolTip(TooltipWidget);
}


FText SCONodeSkeletalMeshSectionPinImage::GetPinTooltipText() const
{
	if (GraphPinObj->bOrphanedPin)
	{
		return LOCTEXT("PinModeMutableOrpahan", "Pin not disapearing due to being connected or having a property modified.");
	}
	
	if (GraphPinObj->PinType.PinCategory == UEdGraphSchema_CustomizableObject::PC_Texture)
	{
		return LOCTEXT("PinModeMutableTooltip", "Texture Parameter goes through Mutable.");
	}
	else
	{
		return LOCTEXT("PinModePassthroughTooltip", "Texture Parameter is ignored by Mutable.");
	}
}


#undef LOCTEXT_NAMESPACE
