// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCustomizableObjectNodePin.h"
#include "SGraphPin.h"

class SWidget;
class UEdGraphPin;
struct FSlateBrush;


/** Implements the "MUTABLE" and "PASSTHROUGH" text next to the pin name. */
class SCONodeSkeletalMeshSectionPinImage : public SCustomizableObjectNodePin
{
public:

	SLATE_BEGIN_ARGS(SCONodeSkeletalMeshSectionPinImage) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

private:
	/** Return pin tool tip. */
	FText GetPinTooltipText() const;
};
