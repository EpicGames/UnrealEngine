// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomizableObjectNode.h"

#include "CustomizableObjectNodeDetails.h"
#include "DetailLayoutBuilder.h"
#include "GraphEditorSettings.h"
#include "SCustomizableObjectNodePin.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "Widgets/SBoxPanel.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SCustomizableObjectNode::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
    GraphNode = InGraphNode;
	
    UpdateGraphNode();
}


TSharedPtr<SGraphPin> SCustomizableObjectNode::CreatePinWidget(UEdGraphPin* Pin) const
{
	check(Pin->GetOwningNode());

	return SNew(SCustomizableObjectNodePin, Pin);
}


void SCustomizableObjectNode::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	SGraphNode::CreateBelowPinControls(MainBox);

	if (const UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(GraphNode))
	{
		if (Node->IsExperimental())
		{
			MainBox->AddSlot()
			.Padding(FMargin(0.0, 2.0, 0.0, 0.0))
			.AutoHeight()
			[
				SNew(SErrorText)
				.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
				.ErrorText(LOCTEXT("SCONode_Experimental", "EXPERIMENTAL"))
			];
		}
		
		if (Node->IsDeprecated())
		{
			MainBox->AddSlot()
			.Padding(FMargin(0.0, 2.0, 0.0, 0.0))
			.AutoHeight()
			[
				SNew(SErrorText)
				.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
				.ErrorText(LOCTEXT("SCONode_Deprecated", "DEPRECATED"))
			];
		}
		
		if (!Node->IsLoaded())
		{
			MainBox->AddSlot()
			.Padding(FMargin(0.0, 2.0, 0.0, 0.0))
			.AutoHeight()
			[
				SNew(SErrorText)
				.BackgroundColor(FAppStyle::GetColor("ErrorReporting.BackgroundColor"))
				.ErrorText(LOCTEXT("SCONodeExternal_NotLoaded", "NOT LOADED"))
			];
		}
	}
}


FSlateColor SCustomizableObjectNode::GetNodeTitleColor() const
{
	FLinearColor ReturnTitleColor = GetNodeObj()->GetNodeTitleColor();

	if (!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || GraphNode->IsNodeUnrelated())
	{
		ReturnTitleColor *= FLinearColor(0.5f, 0.5f, 0.5f, 0.4f);
	}
	else
	{
		ReturnTitleColor.A = FadeCurve.GetLerp();
	}
	
	return ReturnTitleColor;
}


void SCustomizableObjectNode::CreateInputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	SGraphNode::CreateInputSideAddButton(OutputBox);

	if (const UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(GraphNode))
	{
		if (Node->GetAddPinButtonNodeSide() == EAddPinNodeButtonLocation::INPUT)
		{
			TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
			NSLOCTEXT("SequencerNode", "SequencerNodeAddPinButton", "Add pin"),
			NSLOCTEXT("SequencerNode", "SequencerNodeAddPinButton_ToolTip", "Add new pin"),
			false);

			FMargin AddPinPadding = Settings->GetOutputPinPadding();
			AddPinPadding.Top += 6.0f;

			OutputBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(AddPinPadding)
			[
				AddPinButton
			];
		}
	}
}


void SCustomizableObjectNode::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	SGraphNode::CreateOutputSideAddButton(OutputBox);

	if (const UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(GraphNode))
	{
		if (Node->GetAddPinButtonNodeSide() == EAddPinNodeButtonLocation::OUTPUT)
		{
			TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
			NSLOCTEXT("SequencerNode", "SequencerNodeAddPinButton", "Add pin"),
			NSLOCTEXT("SequencerNode", "SequencerNodeAddPinButton_ToolTip", "Add new pin"));

			FMargin AddPinPadding = Settings->GetOutputPinPadding();
			AddPinPadding.Top += 6.0f;

			OutputBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(AddPinPadding)
			[
				AddPinButton
			];
		}
	}
}


FReply SCustomizableObjectNode::OnAddPin()
{
	if (UCustomizableObjectNode* Node = Cast<UCustomizableObjectNode>(GraphNode))
	{
		Node->AddPinFromUI();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}


#undef LOCTEXT_NAMESPACE

