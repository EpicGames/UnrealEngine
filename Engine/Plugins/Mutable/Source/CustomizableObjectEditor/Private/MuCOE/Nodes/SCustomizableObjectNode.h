// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"

class SGraphPin;
class UEdGraphNode;
class UEdGraphPin;


class SCustomizableObjectNode : public SGraphNode
{
public:
    SLATE_BEGIN_ARGS(SCustomizableObjectNode) {}
    SLATE_END_ARGS();

    void Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode);

	// SGraphNode interface
    virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual FSlateColor GetNodeTitleColor() const override;

protected:
	virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
    virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
    virtual FReply OnAddPin() override;
};
