// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomizableObjectNodePin.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "Widgets/Input/SEditableTextBox.h"


void SCustomizableObjectNodePin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	const SGraphPin::FArguments SGraphPinArguments = SGraphPin::FArguments();
	SGraphPin::Construct(SGraphPinArguments, InGraphPinObj);

	// Snippet from SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj); of the creation and setup of the ValueWidget
	// Since the base class does no longer support having an editable text box for the outputs of the node we do create our own
	if (InGraphPinObj->Direction == EEdGraphPinDirection::EGPD_Output)
	{
		ValueWidget = GetDefaultValueWidget();

		if (ValueWidget != SNullWidget::NullWidget)
		{
			TSharedPtr<SBox> ValueBox;
			LabelAndValue->AddSlot()
				.Padding(FMargin(SGraphPinArguments._SideToSideMargin, 0.0f, 0, 0.0f))
				.VAlign(VAlign_Center)
				[
					SAssignNew(ValueBox, SBox)
					.Padding(0.0f)
					[
						ValueWidget.ToSharedRef()
					]
				];

			if (!DoesWidgetHandleSettingEditingEnabled())
			{
				ValueBox->SetEnabled(TAttribute<bool>(this, &SGraphPin::IsEditingEnabled));
			}
		}
	}

	// Cache pin icons.
	PassThroughImageConnected = UE_MUTABLE_GET_BRUSH(TEXT("Graph.ExecPin.Connected"));
	PassThroughImageDisconnected = UE_MUTABLE_GET_BRUSH(TEXT("Graph.ExecPin.Disconnected"));
}


TSharedRef<SWidget> SCustomizableObjectNodePin::GetDefaultValueWidget()
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(GetPinObj()->GetOwningNode());
		if (Node && Node->CanRenamePin(*GraphPin))
		{
			switch (Node->GetEditablePinNameVisibilityPolicy(*GraphPin))
			{
			case EEditablePinNameBoxVisibilityPolicy::ALWAYS_VISIBLE:
				{
					return SNew(SEditableTextBox)
						.Text(this, &SCustomizableObjectNodePin::GetNodeStringValue)
						.OnTextCommitted(this, &SCustomizableObjectNodePin::OnTextCommited)
						.Visibility(EVisibility::Visible);
				}
			case EEditablePinNameBoxVisibilityPolicy::HIDE_IF_LINKED:
				{
					return SNew(SEditableTextBox)
						.Text(this, &SCustomizableObjectNodePin::GetNodeStringValue)
						.OnTextCommitted(this, &SCustomizableObjectNodePin::OnTextCommited)
						.Visibility(this, &SCustomizableObjectNodePin::GetHideIfLinkedWidgetVisibility);
				}
			}
		}	
	}

	return SGraphPin::GetDefaultValueWidget();
}


const FSlateBrush* SCustomizableObjectNodePin::GetPinIcon() const
{
	if (UEdGraphSchema_CustomizableObject::IsPassthrough(GraphPinObj->PinType.PinCategory))
	{
		return GraphPinObj->LinkedTo.Num() ?
			PassThroughImageConnected :
			PassThroughImageDisconnected;
	}
	else
	{
		return SGraphPin::GetPinIcon();
	}
}


FSlateColor SCustomizableObjectNodePin::GetPinColor() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		if (bIsDiffHighlighted)
		{
			return FSlateColor(FLinearColor(0.9f, 0.2f, 0.15f));
		}
		if (GraphPin->bOrphanedPin)
		{
			return FSlateColor(FLinearColor::Red);
		}
		
		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(GetPinObj()->GetOwningNode());
		if (Node)
		{
			const FLinearColor Color = Node->GetPinColor(*GraphPin);
			
			if (!Node->IsNodeEnabled() || Node->IsDisplayAsDisabledForced() || !IsEditingEnabled() || Node->IsNodeUnrelated())
			{
				return Color * FLinearColor(1.0f, 1.0f, 1.0f, 0.5f);
			}

			return Color * PinColorModifier;
		}
	}

	return FLinearColor::White;
}


FText SCustomizableObjectNodePin::GetNodeStringValue() const
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(GetPinObj()->GetOwningNode());
		if (Node)
		{
			return Node->GetPinEditableName(*GraphPin);
		}
	}

	return {};
}


void SCustomizableObjectNodePin::OnTextCommited(const FText& InValue, ETextCommit::Type InCommitInfo)
{
	UEdGraphPin* GraphPin = GetPinObj();
	if (GraphPin && !GraphPin->IsPendingKill())
	{
		UCustomizableObjectNode* Node = CastChecked<UCustomizableObjectNode>(GetPinObj()->GetOwningNode());
		if (Node)
		{
			Node->SetPinEditableName(*GraphPin, InValue);
		}
	}
}


EVisibility SCustomizableObjectNodePin::GetHideIfLinkedWidgetVisibility() const
{
	return GraphPinObj->LinkedTo.Num() ? EVisibility::Collapsed : EVisibility::Visible;
}
