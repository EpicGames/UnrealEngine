// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

class SWidget;
class UEdGraphPin;
struct FSlateBrush;


class SCustomizableObjectNodePin : public SGraphPin
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjectNodePin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	
	// SGraphPin interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	virtual const FSlateBrush* GetPinIcon() const override;
	virtual FSlateColor GetPinColor() const override;

private:
	FText GetNodeStringValue() const;
	
	void OnTextCommited(const FText& InValue, ETextCommit::Type InCommitInfo);
	
	EVisibility GetHideIfLinkedWidgetVisibility() const;


	const FSlateBrush* PassThroughImageConnected = nullptr;
	const FSlateBrush* PassThroughImageDisconnected = nullptr;
};
