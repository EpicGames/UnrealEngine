// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomizableObjectNodeReroute.h"

#include "SCustomizableObjectNodeReroutePin.h"


void SCustomizableObjectNodeReroute::Construct(const FArguments& InArgs, UEdGraphNode* InGraphNode)
{
	SGraphNodeKnot::Construct({}, InGraphNode);
}


TSharedPtr<SGraphPin> SCustomizableObjectNodeReroute::CreatePinWidget(UEdGraphPin* Pin) const
{
	return SNew(SCustomizableObjectNodeReroutePin, Pin);
}

