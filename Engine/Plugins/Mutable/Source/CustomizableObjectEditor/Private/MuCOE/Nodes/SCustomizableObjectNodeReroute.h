// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNodeKnot.h"
#include "Templates/SharedPointer.h"

class SGraphPin;
class UEdGraphPin;


// To avoid multiple inheritance SCustomizableObjectNodeReroute does not inherit from SCustomizableObjectNode but reimplements its functionalities.
class SCustomizableObjectNodeReroute : public SGraphNodeKnot
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeKnot) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphNode* InKnot);
	
	// SGraphNode interface
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
};

