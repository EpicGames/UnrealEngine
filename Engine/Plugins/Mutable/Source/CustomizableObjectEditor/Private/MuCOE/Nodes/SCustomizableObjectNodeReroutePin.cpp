// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomizableObjectNodeReroutePin.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"


void SCustomizableObjectNodeReroutePin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPinKnot::Construct({}, InPin);

	// Cache pin icons.
	PassThroughImageConnected = UE_MUTABLE_GET_BRUSH(TEXT("Graph.ExecPin.Connected"));
	PassThroughImageDisconnected = UE_MUTABLE_GET_BRUSH(TEXT("Graph.ExecPin.Disconnected"));	
}


const FSlateBrush* SCustomizableObjectNodeReroutePin::GetPinIcon() const
{
	if (UEdGraphSchema_CustomizableObject::IsPassthrough(GraphPinObj->PinType.PinCategory))
	{
		return GraphPinObj->LinkedTo.Num() ?
			PassThroughImageConnected :
			PassThroughImageDisconnected;
	}
	else
	{
		return SGraphPin::GetPinIcon();
	}
}
