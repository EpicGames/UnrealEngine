// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNodeKnot.h"


// To avoid multiple inheritance SCustomizableObjectNodeReroutePin does not inherit from SCustomizableObjectNodePin but reimplements its functionalities.
class SCustomizableObjectNodeReroutePin : public SGraphPinKnot
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectNodeReroutePin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

protected:
	// SGraphPin interface
	virtual const FSlateBrush* GetPinIcon() const override;

	const FSlateBrush* PassThroughImageConnected = nullptr;
	const FSlateBrush* PassThroughImageDisconnected = nullptr;
};

