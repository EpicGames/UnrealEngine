// Copyright Epic Games, Inc. All Rights Reserved.

#include "PassthroughObjectFactory.h"


namespace UE::Mutable::Private
{
	PASSTHROUGH_ID FPassthroughObjectFactory::Add(UObject& Object, bool bDuplicate)
	{
		TSoftObjectPtr<UObject> SoftObjectPtr(&Object);
		
		PASSTHROUGH_ID Id = GetPassthroughId(SoftObjectPtr);
		while (TSoftObjectPtr<UObject>* PassthroughObject = PassthroughObjects.Find(Id))
		{
			if (*PassthroughObject == &Object)
			{
				return Id;
			}
			else
			{
				++Id;
			}
		}
	
		PassthroughObjects.Add(Id, SoftObjectPtr);
		
		if (bDuplicate)
		{
			DuplicateObjects.Add(Id);
		}

		return Id;
	}


	TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>>& FPassthroughObjectFactory::GetAll()
	{
		return PassthroughObjects;
	}


	TArray<PASSTHROUGH_ID>& FPassthroughObjectFactory::GetDuplicate()
	{
		return DuplicateObjects;
	}
}
