// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MuR/PassthroughObject.h"


namespace UE::Mutable::Private
{
	class FPassthroughObjectFactory
	{
	public:
		PASSTHROUGH_ID Add(UObject& Object, bool bDuplicate);
		
		TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>>& GetAll();
		
		TArray<PASSTHROUGH_ID>& GetDuplicate();
		
	private:
		TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>> PassthroughObjects;
		
		TArray<PASSTHROUGH_ID> DuplicateObjects;
	};	
}
