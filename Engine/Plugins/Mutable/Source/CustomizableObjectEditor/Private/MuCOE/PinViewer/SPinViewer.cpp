// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/PinViewer/SPinViewer.h"

#include "DetailWidgetRow.h"
#include "GraphEditAction.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/PinViewer/SPinViewerListRow.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class UObject;
struct EVisibility;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


const FName SPinViewer::COLUMN_NAME(TEXT("Name"));

const FName SPinViewer::COLUMN_TYPE(TEXT("Type"));

const FName SPinViewer::COLUMN_SUBTYPE(TEXT("Subtype"));

const FName SPinViewer::COLUMN_VISIBILITY(TEXT("Visible"));


FText SPinViewer::GetPinName(const UEdGraphPin& Pin)
{
	FString PinComposedName;
	
	if (!Pin.PinFriendlyName.IsEmpty())
	{
		PinComposedName = Pin.PinFriendlyName.ToString();
	}
	else
	{
		PinComposedName =  Pin.PinName.ToString();
	}
	
	return FText::FromString(PinComposedName);
}


/** Sort pin references by name. The pin references to be valid. */
bool SortPinsByNameAsc(const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
{
	const FText PinNameA = SPinViewer::GetPinName(*PinReferenceA->Get());
	const FText PinNameB = SPinViewer::GetPinName(*PinReferenceB->Get());

	return PinNameA.CompareTo(PinNameB, ETextComparisonLevel::Default) < 0;
}

bool SortPinsByNameDesc(const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
{
	const FText PinNameA = SPinViewer::GetPinName(*PinReferenceA->Get());
	const FText PinNameB = SPinViewer::GetPinName(*PinReferenceB->Get());

	return PinNameA.CompareTo(PinNameB, ETextComparisonLevel::Default) >= 0;
}


/** Sort pin references by the provided method. The pin references have to be valid. */
void Sort(const FName& Method, const EColumnSortMode::Type& SortMode, TArray<TSharedPtr<FEdGraphPinReference>>& Pins)
{
	if (Method == SPinViewer::COLUMN_NAME || SortMode == EColumnSortMode::None )
	{
		if (SortMode == EColumnSortMode::Descending)
		{
			Pins.Sort(&SortPinsByNameDesc);
		}
		else
		{
			Pins.Sort(&SortPinsByNameAsc);
		}
	}
	else if (Method == SPinViewer::COLUMN_TYPE)
	{
		if (SortMode == EColumnSortMode::Descending)
		{
			Pins.Sort([](const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
				{
					if (PinReferenceA->Get()->PinType.PinCategory == PinReferenceB->Get()->PinType.PinCategory)
					{
						return SortPinsByNameDesc(PinReferenceA, PinReferenceB);
					}
					else if (PinReferenceA->Get()->PinType.PinCategory.FastLess(PinReferenceB->Get()->PinType.PinCategory))
					{
						return false;
					}
					else
					{
						return true;
					}
				});
		}
		else
		{
			Pins.Sort([](const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
				{
					if (PinReferenceA->Get()->PinType.PinCategory == PinReferenceB->Get()->PinType.PinCategory)
					{
						return SortPinsByNameAsc(PinReferenceA, PinReferenceB);
					}
					else if (PinReferenceA->Get()->PinType.PinCategory.FastLess(PinReferenceB->Get()->PinType.PinCategory))
					{
						return true;
					}
					else
					{
						return false;
					}
				});
		}
	}
	else if (Method == SPinViewer::COLUMN_SUBTYPE)
	{
		if (SortMode == EColumnSortMode::Descending)
		{
			Pins.Sort([](const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
				{
					if (PinReferenceA->Get()->PinType.PinSubCategory == PinReferenceB->Get()->PinType.PinSubCategory)
					{
						return SortPinsByNameDesc(PinReferenceA, PinReferenceB);
					}
					else if (PinReferenceA->Get()->PinType.PinSubCategory.FastLess(PinReferenceB->Get()->PinType.PinSubCategory))
					{
						return false;
					}
					else
					{
						return true;
					}
				});
		}
		else
		{
			Pins.Sort([](const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
				{
					if (PinReferenceA->Get()->PinType.PinSubCategory == PinReferenceB->Get()->PinType.PinSubCategory)
					{
						return SortPinsByNameAsc(PinReferenceA, PinReferenceB);
					}
					else if (PinReferenceA->Get()->PinType.PinSubCategory.FastLess(PinReferenceB->Get()->PinType.PinSubCategory))
					{
						return true;
					}
					else
					{
						return false;
					}
				});
		}
	}
	else if (Method == SPinViewer::COLUMN_VISIBILITY)
	{
		if (SortMode == EColumnSortMode::Descending)
		{
			Pins.Sort([](const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
				{
					if (PinReferenceA->Get()->bHidden == PinReferenceB->Get()->bHidden)
					{
						return SortPinsByNameDesc(PinReferenceA, PinReferenceB);
					}
					else if (PinReferenceA->Get()->bHidden < PinReferenceB->Get()->bHidden)
					{
						return true;
					}
					else
					{
						return false;
					}
				});
		}
		else
		{
			Pins.Sort([](const TSharedPtr<FEdGraphPinReference>& PinReferenceA, const TSharedPtr<FEdGraphPinReference>& PinReferenceB)
				{
					if (PinReferenceA->Get()->bHidden == PinReferenceB->Get()->bHidden)
					{
						return SortPinsByNameAsc(PinReferenceA, PinReferenceB);
					}
					else if (PinReferenceA->Get()->bHidden < PinReferenceB->Get()->bHidden)
					{
						return false;
					}
					else
					{
						return true;
					}
				});
		}
	}
	else
	{
		check(false); // Unknown method.
	}
}


void SPinViewer::GeneratePinInfoList()
{
	PinReferences.Reset();
	for (UEdGraphPin* Pin : Node->GetAllPins())
	{
		if (CurrentFilter.IsEmpty() || GetPinName(*Pin).ToString().Contains(CurrentFilter))
		{
			PinReferences.Add(MakeShareable(new FEdGraphPinReference(Pin)));
		}
	}

	Sort(CurrentSortColumn, SortMode, PinReferences);
}


void SPinViewer::PinsRemapped(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap)
{
	for (const TTuple<UEdGraphPin*, UEdGraphPin*>& Pair : PinsToRemap)
	{
		if (const EVisibility* Result = AdditionalWidgetVisibility.Find(Pair.Key->PinId))
		{
			AdditionalWidgetVisibility.Add(Pair.Value->PinId, *Result);
		}
	}
}


void SPinViewer::Construct(const FArguments& InArgs)
{
	Node = TStrongObjectPtr<UCustomizableObjectNode>{InArgs._Node};
	check(Node)
	
	Node->RemapPinsDelegate.AddSP(this, &SPinViewer::PinsRemapped);
	Node->NodeConnectionListChangedDelegate.AddSP(this, &SPinViewer::UpdateWidget);
	Node->PinEditableNameChangedDelegate.AddSP(this, &SPinViewer::OnPinEditableNameChange);
	GeneratePinInfoList();

	Graph = TStrongObjectPtr(Node->GetGraph());
	if (Graph)
	{
		// bind the node refresh to a change in the graph
		Graph->AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateSP(this, &SPinViewer::OnGraphChanged));
	}
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.0f,5.0f,0.0f,0.0f)
		.AutoHeight()
		[
			SNew(SSearchBox)
			.SelectAllTextWhenFocused(true)
			.OnTextChanged(this, &SPinViewer::OnFilterTextChanged)
		]

		+ SVerticalBox::Slot()
		.Padding(0.0f,5.0f,0.0f,5.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			[
				SNew(SButton)
				.Text(LOCTEXT("Text","Show All"))
				.HAlign(HAlign_Center)
				.ToolTipText(LOCTEXT("ShowAllTooltip","Show all pins"))
				.OnClicked(this, &SPinViewer::OnShowAllPressed)
			]
			
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.Text(LOCTEXT("HideAllText", "Hide All"))
				.HAlign(HAlign_Center)
				.ToolTipText(LOCTEXT("ShowLinkedTooltip", "Hide all non connected pins"))
				.OnClicked(this, &SPinViewer::OnHideAllPressed)
			]
		]

		+SVerticalBox::Slot()
		[
			SAssignNew(ListView, SListView<TSharedPtr<FEdGraphPinReference>>)
			.ListItemsSource(&PinReferences)
			.OnGenerateRow(this, &SPinViewer::GenerateNodePinRow)
			.SelectionMode(ESelectionMode::None)
			.IsFocusable(false)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(COLUMN_NAME)
				.DefaultLabel(LOCTEXT("PinNameLabel", "Name"))
				.SortMode(this, &SPinViewer::GetColumnSortMode, COLUMN_NAME)
				.OnSort(this, &SPinViewer::SortListView)
				.FillWidth(0.5)

				+ SHeaderRow::Column(COLUMN_TYPE)
				.DefaultLabel(LOCTEXT("TypeLabel", "Type"))
				.SortMode(this, &SPinViewer::GetColumnSortMode, COLUMN_TYPE)
				.OnSort(this, &SPinViewer::SortListView)
				.FillWidth(0.20)
				
				// todo: since it is not the norm having a pin with a SubCategory set, maybe hide this column is no pin has a subcategory set.
				+ SHeaderRow::Column(COLUMN_SUBTYPE)
				.DefaultLabel(LOCTEXT("SubTypeLabel", "Subtype"))
				.SortMode(this, &SPinViewer::GetColumnSortMode, COLUMN_SUBTYPE)
				.OnSort(this, &SPinViewer::SortListView)
				.FillWidth(0.20)

				+ SHeaderRow::Column(COLUMN_VISIBILITY)
				.DefaultLabel(LOCTEXT("VisibilityLabel", "Visibility"))
				.SortMode(this, &SPinViewer::GetColumnSortMode, COLUMN_VISIBILITY)
				.OnSort(this, &SPinViewer::SortListView)
				.FillWidth(0.10)
			)
		]
	];

	ListView->SetIsRightClickScrollingEnabled(false);
}


void SPinViewer::UpdateWidget()
{
	GeneratePinInfoList();
	ListView->RequestListRefresh();
}


void SPinViewer::OnPinEditableNameChange(const UEdGraphPin& EdGraphPin, FText Text)
{
	// TODO: OPTIMIZATION : it would be nice to take into consideration what pin has changed so we do not update the whole widget but only the affected row.
	UpdateWidget();
}


void SPinViewer::OnGraphChanged(const FEdGraphEditAction& EdGraphEditAction)
{
	if (EdGraphEditAction.Action == GRAPHACTION_EditNode && EdGraphEditAction.Nodes.Contains(Node.Get()))
	{
		UpdateWidget();
	}
}


TSharedRef<ITableRow> SPinViewer::GenerateNodePinRow(TSharedPtr<FEdGraphPinReference> PinReference, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SPinViewerListRow, OwnerTable)
		.PinViewer(this)
		.PinReference(*PinReference);
}


void SPinViewer::OnFilterTextChanged(const FText& SearchText)
{
	CurrentFilter = SearchText.ToString();
	UpdateWidget();
}


FReply SPinViewer::OnShowAllPressed() const
{
	Node->SetPinHidden(Node->GetAllNonOrphanPins(), false);
	return FReply::Handled();
}


FReply SPinViewer::OnHideAllPressed() const
{
	Node->SetPinHidden(Node->GetAllNonOrphanPins(), true);
	return FReply::Handled();
}


void SPinViewer::SortListView(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	CurrentSortColumn = ColumnId;
	SortMode = NewSortMode;
	UpdateWidget();
}


EColumnSortMode::Type SPinViewer::GetColumnSortMode(const FName ColumnId) const
{
	if (CurrentSortColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}


#undef LOCTEXT_NAMESPACE
