// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SHeaderRow.h"
#include "GraphEditAction.h"

template <typename ItemType> class SListView;

class IDetailLayoutBuilder;
class ITableRow;
class STableViewBase;
class UCustomizableObjectNode;
class UEdGraphPin;
struct EVisibility;
struct FEdGraphPinReference;
struct FGuid;

/**
 * Pin Viewer widget. Shows a list of pins a node contains.
 * In addition, a pin can provide a custom details widget which will be shown as an expandable menu.
 * 
 * This widget will work as long as pins are only created and destroyed during the node reconstruction.
 */
class SPinViewer : public SCompoundWidget
{
	friend class SPinViewerListRow;

public:
	SLATE_BEGIN_ARGS(SPinViewer) {}
	SLATE_ARGUMENT(UCustomizableObjectNode*, Node)
	SLATE_END_ARGS()

	static const FName COLUMN_NAME;
	static const FName COLUMN_TYPE;
	static const FName COLUMN_SUBTYPE;
	static const FName COLUMN_VISIBILITY;
	
	void Construct(const FArguments& InArgs);

	/** Regenerate the list contents and update the widget. */
	void UpdateWidget();

	/** Callback executed each time a graph node does change. */
	void OnGraphChanged(const FEdGraphEditAction& EdGraphEditAction);

	/** Callback to Generate a Node Material Pin row */
	TSharedRef<ITableRow> GenerateNodePinRow(TSharedPtr<FEdGraphPinReference> PinReference, const TSharedRef<STableViewBase>& OwnerTable);

	/** SearchBox callback. */
	void OnFilterTextChanged(const FText& SearchText);

	/** Buttons callbacks. */
	FReply OnShowAllPressed() const;
	FReply OnHideAllPressed() const;

	// Columns sorting methods
	/** Sets the sorting method to be applied */
	void SortListView(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Get the pin name which is actually displayed. */
	static FText GetPinName(const UEdGraphPin& Pin);

	/** Callback method invoked each time a pin of the node we keep a reference of does get the Editable Name changed. */
	void OnPinEditableNameChange(const UEdGraphPin& EdGraphPin, FText Text);
	
private:
	/** Regenerate the list contents. */
	void GeneratePinInfoList();

	void PinsRemapped(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap);

	/** Pointer to the Node */
	TStrongObjectPtr<UCustomizableObjectNode> Node = nullptr;

	TStrongObjectPtr<UEdGraph> Graph = nullptr;
	
	/** ListView elements. */
	TArray<TSharedPtr<FEdGraphPinReference>> PinReferences;

	/** Data structure used to save SPinViewerMultiColumnTableRow additional widget visibility state between reconstructs.
	 * Key is a pin id. */
	TMap<FGuid, EVisibility> AdditionalWidgetVisibility;
	
	/** Current column. */
	FName CurrentSortColumn = COLUMN_TYPE;
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::None;

	FString CurrentFilter;

	/** Widget List of the Node Material Pins */
	TSharedPtr<SListView<TSharedPtr<FEdGraphPinReference>>> ListView;

	/** Scrollbox widget needed to fix some scrolling problems */
	TSharedPtr<class SScrollBox> Scrollbox;
};

