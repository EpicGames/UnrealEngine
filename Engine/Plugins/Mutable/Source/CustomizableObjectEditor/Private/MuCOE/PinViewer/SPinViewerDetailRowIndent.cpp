// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/PinViewer/SPinViewerDetailRowIndent.h"

#include "MuCOE/Widgets/SMutableExpandableTableRow.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FSlateBrush;


void SPinViewerDetailRowIndent::Construct(const FArguments& InArgs, TSharedRef<SWidget> DetailsRow)
{
	Row = DetailsRow;
	
	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(16.0f)
	];
}


int32 SPinViewerDetailRowIndent::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{

	const FSlateBrush* BackgroundBrush = FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
	const FSlateBrush* DropShadowBrush = FAppStyle::Get().GetBrush("DetailsView.ArrayDropShadow");

	const TSharedPtr<SWidget> RowPtr = Row.Pin();
	const FSlateColor BackgroundColor = GetRowBackgroundColor(0, RowPtr.IsValid() && RowPtr->IsHovered());
	
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(FVector2D(16, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform()),
		BackgroundBrush,
		ESlateDrawEffect::None,
		BackgroundColor.GetColor(InWidgetStyle)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(FVector2D(16, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform()),
		DropShadowBrush
	);

	return LayerId + 1;
}

