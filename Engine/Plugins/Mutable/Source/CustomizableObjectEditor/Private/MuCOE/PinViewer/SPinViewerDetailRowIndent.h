// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class SWidget;
struct FGeometry;


/** Pin Viewer custom details row left indent.
 * Draws a shadow to differentiate the different indent levels.
 * 
 * Copy of  SDetailRowIndent.h. */
class SPinViewerDetailRowIndent : public SCompoundWidget
{ 
public:
	SLATE_BEGIN_ARGS(SPinViewerDetailRowIndent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SWidget> Row);

private:
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	TWeakPtr<SWidget> Row;
};
