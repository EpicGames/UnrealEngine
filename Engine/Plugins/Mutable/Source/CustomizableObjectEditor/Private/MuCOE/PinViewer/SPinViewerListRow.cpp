// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/PinViewer/SPinViewerListRow.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/PinViewer/SPinViewer.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "IDetailsView.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SWidget.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "MuCOE/Widgets/SMutablePinTypeSelector.h"
#include "Widgets/Input/SEditableTextBox.h"

class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SPinViewerListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	PinViewer = InArgs._PinViewer;
	PinReference = InArgs._PinReference;
	
	if (UCustomizableObjectNode* OwningNode = Cast<UCustomizableObjectNode>(PinReference.Get()->GetOwningNode()))
	{
		PinOwner = MakeWeakObjectPtr(OwningNode);
	}
	else
	{
		checkNoEntry()
	}
	
	PinId = PinReference.Get()->PinId;
	
	SMutableExpandableTableRow<TSharedPtr<FEdGraphPinReference>>::Construct(FSuperRowType::FArguments(), OwnerTableView);
}


TSharedRef<SWidget> SPinViewerListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	const UEdGraphPin* Pin = PinReference.Get();
	
	if (ColumnName == SPinViewer::COLUMN_NAME)
	{
		const TSharedRef<STextBlock> PinNameTextBlock = SNew(STextBlock)
		.Text(SPinViewer::GetPinName(*Pin));
				
		if (Pin->bOrphanedPin)
		{
			PinNameTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		
		if (TStrongObjectPtr<UCustomizableObjectNode> PinnedPinOwner = PinOwner.Pin())
		{
			if (PinnedPinOwner->CanRenamePin(*Pin))
			{
				return SNew(SVerticalBox)
				+ SVerticalBox::Slot().AutoHeight().Padding(4).VAlign(VAlign_Center)
					.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.VAlign(VAlign_Center)
					[
						PinNameTextBlock
					]

					+ SHorizontalBox::Slot()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(4, 0)
					.AutoWidth()
					[
						SNew(SEditableTextBox)
						.OnTextCommitted(this, &SPinViewerListRow::OnEditableNameCommited)
						.Text(this, &SPinViewerListRow::GetEditableName)
					]
				];
			}
			else
			{
				return SNew(SVerticalBox)
					+ SVerticalBox::Slot().AutoHeight().Padding(4).VAlign(VAlign_Center)
					[
						PinNameTextBlock
					];
			}
		}
		
	}
	else if (ColumnName == SPinViewer::COLUMN_TYPE)
	{
		if (TStrongObjectPtr<UCustomizableObjectNode> PinnedPinOwner = PinOwner.Pin())
		{
			return SNew(SVerticalBox)
				
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SMutablePinTypeSelector)
				.PinReference(PinReference)
			];
		}
	}
	else if (ColumnName == SPinViewer::COLUMN_SUBTYPE)
	{
		if (TStrongObjectPtr<UCustomizableObjectNode> PinnedPinOwner = PinOwner.Pin())
		{
			return SNew(SVerticalBox)
				
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SMutablePinSubTypeSelector)
				.PinReference(PinReference)
			];
		}
	}
	else if (ColumnName == SPinViewer::COLUMN_VISIBILITY)
	{
		return SNew(SVerticalBox)
			
		+ SVerticalBox::Slot().AutoHeight().Padding(4).VAlign(VAlign_Center).HAlign(HAlign_Center)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SPinViewerListRow::OnPinVisibilityCheckStateChanged)
			.IsChecked(this, &SPinViewerListRow::IsVisibilityChecked)
			.IsEnabled(PinViewer->Node->CanPinBeHidden(*Pin))
		];
	}

	return SNullWidget::NullWidget;
}


TSharedPtr<SWidget> SPinViewerListRow::GenerateAdditionalWidgetForRow()
{
	return PinViewer->Node->CustomizePinDetails(*PinReference.Get());
}


EVisibility SPinViewerListRow::GetAdditionalWidgetDefaultVisibility() const
{
	if (const EVisibility* Result = PinViewer->AdditionalWidgetVisibility.Find(PinId))
	{
		return *Result;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}


void SPinViewerListRow::SetAdditionalWidgetVisibility(const EVisibility InVisibility)
{
	SMutableExpandableTableRow<TSharedPtr<FEdGraphPinReference, ESPMode::ThreadSafe>>::SetAdditionalWidgetVisibility(InVisibility);

	PinViewer->AdditionalWidgetVisibility.Add(PinId, InVisibility);
}


void SPinViewerListRow::OnPinVisibilityCheckStateChanged(ECheckBoxState NewRadioState)
{
	PinViewer->Node->SetPinHidden(*PinReference.Get(), NewRadioState == ECheckBoxState::Unchecked);
}


ECheckBoxState SPinViewerListRow::IsVisibilityChecked() const
{
	return PinReference.Get()->bHidden ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}


void SPinViewerListRow::OnEditableNameCommited(const FText& NewPinEditableName, ETextCommit::Type Arg) const
{
	if (TStrongObjectPtr<UCustomizableObjectNode> PinnedOwner = PinOwner.Pin())
	{
		if (const UEdGraphPin* Pin = PinReference.Get())
		{
			PinnedOwner->SetPinEditableName(*Pin, NewPinEditableName);
		}
	}
}


FText SPinViewerListRow::GetEditableName() const
{
	if (TStrongObjectPtr<UCustomizableObjectNode> PinnedOwner = PinOwner.Pin())
	{
		if (const UEdGraphPin* Pin = PinReference.Get())
		{
			return PinnedOwner->GetPinEditableName(*Pin);
		}
	}
	
	return {};
}


FName SPinViewerListRow::GetInitiallySelectedPinType() const
{
	if (UEdGraphPin* PinPointer = PinReference.Get())
	{
		return PinPointer->PinType.PinCategory;
	}
	else 
	{
		return NAME_Name;
	}
}


FName SPinViewerListRow::GetInitiallySelectedPinSubType() const
{
	if (UEdGraphPin* PinPointer = PinReference.Get())
	{
		return PinPointer->PinType.PinSubCategory;
	}
	else 
	{
		return NAME_Name;
	}
}


#undef LOCTEXT_NAMESPACE
