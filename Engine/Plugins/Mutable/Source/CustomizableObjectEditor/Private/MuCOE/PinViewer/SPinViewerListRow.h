// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphPin.h"
#include "MuCOE/Widgets/SMutableExpandableTableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

class SPinViewer;
class STableViewBase;
class SWidget;
class UCustomizableObjectNode;

/** Custom row for a SListView that allows to add custom widgets as a row element. */
class SPinViewerListRow : public SMutableExpandableTableRow<TSharedPtr<FEdGraphPinReference>>
{
public:
	SLATE_BEGIN_ARGS(SPinViewerListRow) {}
	SLATE_ARGUMENT(SPinViewer*, PinViewer)
	SLATE_ARGUMENT(FEdGraphPinReference, PinReference)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	// SPinViewerMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	virtual TSharedPtr<SWidget> GenerateAdditionalWidgetForRow() override;
	virtual EVisibility GetAdditionalWidgetDefaultVisibility() const override;
	virtual void SetAdditionalWidgetVisibility(EVisibility Visibility) override;
	
	// Own interface
	/** CheckBox callback. */
	void OnPinVisibilityCheckStateChanged(ECheckBoxState NewRadioState);
	
	/** Returns the state of the check box in function of the visibility of the pin */
	ECheckBoxState IsVisibilityChecked() const;

	void OnEditableNameCommited(const FText& NewPinEditableName, ETextCommit::Type Arg) const;
	FText GetEditableName() const;
	

private:
	
	FName GetInitiallySelectedPinType() const;
	FName GetInitiallySelectedPinSubType() const;

	SPinViewer* PinViewer = nullptr;
	
	/** Weak pointer to the Customizable Object node that contains this property */
	TWeakObjectPtr<UCustomizableObjectNode> PinOwner;
	
	/** Pin representing this row. */
	FEdGraphPinReference PinReference;
	FGuid PinId; // Temporal performance fix. Avoids using the slow FEdPinReference hash function.
};
