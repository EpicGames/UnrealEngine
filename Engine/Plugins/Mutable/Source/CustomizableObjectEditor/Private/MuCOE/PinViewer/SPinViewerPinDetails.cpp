// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/PinViewer/SPinViewerPinDetails.h"

#include "MuCOE/PinViewer/SPinViewerDetailRowIndent.h"
#include "MuCOE/Widgets/SMutableExpandableTableRow.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SPinViewerPinDetails::Construct(const FArguments& InArgs)
{
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
		.BorderBackgroundColor(this, &SPinViewerPinDetails::GetBackgroundColor)
		.Padding(0)
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPinViewerDetailRowIndent, SharedThis(this))
			]
			+SHorizontalBox::Slot()
			[
				SAssignNew(VerticalBox, SVerticalBox)
			]
		]);
}


TSharedPtr<SWidget> SPinViewerPinDetails::AddRow(const FText& Text, TSharedRef<SWidget> Widget, const FText* Tooltip)
{
	TSharedPtr<SWidget> Root;
	TSharedPtr<STextBlock> TextBlock;
	
	VerticalBox->AddSlot()
	.AutoHeight()
	[
		SAssignNew(Root, SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
		.Padding(FMargin(0, 0, 0, 1))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
			.BorderBackgroundColor(this, &SPinViewerPinDetails::GetBackgroundColor)
			.Padding(0)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SSplitter)
				.Style(FAppStyle::Get(), "DetailsView.Splitter")
				.PhysicalSplitterHandleSize(1.0f)
				.HitDetectionSplitterHandleSize(5.0f)
				+SSplitter::Slot()
				.Resizable(false)
				.SizeRule(SSplitter::ESizeRule::FractionOfParent)
				[
					 SNew(SBorder)
					 .BorderImage(UE_MUTABLE_GET_BRUSH("NoBorder"))
					 .VAlign(VAlign_Center)
					 .Padding(FMargin(16, 0, 0, 0))
					 [
						 SAssignNew(TextBlock, STextBlock)
						 .Text(Text)
					 ]
				]
				+SSplitter::Slot()
				[
					SNew(SBorder)
					.BorderImage(UE_MUTABLE_GET_BRUSH("NoBorder"))
					.Padding(FMargin(4, 2, 0, 2))
					[
						Widget
					]
				]
			]
		]
	];

	if (Tooltip)
	{
		TextBlock->SetToolTipText(*Tooltip);
	}
	
	return Root;
}


FSlateColor SPinViewerPinDetails::GetBackgroundColor() const
{
	return GetRowBackgroundColor(0, this->IsHovered());
}


#undef LOCTEXT_NAMESPACE
