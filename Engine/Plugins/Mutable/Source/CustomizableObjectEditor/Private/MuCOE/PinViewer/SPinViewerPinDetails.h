// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBorder.h"

class FText;
class SVerticalBox;
class SWidget;


/** Pin Viewer custom details multiple row style.
 * Base class which allows a custom pin viewer details to have multiple properties. */
class SPinViewerPinDetails : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SPinViewerPinDetails) {}
	SLATE_END_ARGS()

	/** Slate constructor. */
	void Construct(const FArguments& InArgs);

	/** Adds a new property row.
	 *
	 * @param Text Property title.
	 * @param Widget Property widget.
	 * @param Tooltip Optional property tooltip.
	 * @return Row widget. */
	TSharedPtr<SWidget> AddRow(const FText& Text, TSharedRef<SWidget> Widget, const FText* Tooltip = nullptr);
	
private:
	/** Vertical box which contains all rows. */
	TSharedPtr<SVerticalBox> VerticalBox;

	/** Returns custom details background color. */
	FSlateColor GetBackgroundColor() const;
};

