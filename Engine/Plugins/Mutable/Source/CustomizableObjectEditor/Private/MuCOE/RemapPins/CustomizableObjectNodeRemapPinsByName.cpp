// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"

#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeRemapPinsByName)


bool UCustomizableObjectNodeRemapPinsByName::Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const
{
	return Helper_GetPinName(&OldPin) == Helper_GetPinName(&NewPin) &&
				OldPin.PinType == NewPin.PinType &&
				OldPin.Direction == NewPin.Direction;
}


void UCustomizableObjectNodeRemapPinsByName::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	for (UEdGraphPin* OldPin : OldPins)
	{
		bool bFound = false;

		for (UEdGraphPin* NewPin : NewPins)
		{
			if (Equal(Node, *OldPin, *NewPin))
			{
				bFound = true;

				PinsToRemap.Add(OldPin, NewPin);
				break;
			}
		}

		if (!bFound && OldPin->LinkedTo.Num())
		{
			PinsToOrphan.Add(OldPin);
		}
	}
}
