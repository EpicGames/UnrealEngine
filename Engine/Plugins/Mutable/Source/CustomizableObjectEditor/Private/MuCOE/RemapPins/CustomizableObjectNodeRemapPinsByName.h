// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPins.h"

#include "CustomizableObjectNodeRemapPinsByName.generated.h"

class UEdGraphPin;
class UObject;


/** 
 * Remap pins by name.
 * 
 * Remap pins by the field Pin->PinName. Output pins and input pins get remapped independently.
 * 
 * Use inheritance is a node requires a set of special rules when remapping by name.
 */
UCLASS()
class UCustomizableObjectNodeRemapPinsByName : public UCustomizableObjectNodeRemapPins
{
public:
	GENERATED_BODY()

	virtual bool Equal(const UCustomizableObjectNode& Node, const UEdGraphPin& OldPin, const UEdGraphPin& NewPin) const;
	
	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};
