// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByNameDefaultPin.h"

#include "EdGraph/EdGraphPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeRemapPinsByNameDefaultPin)


void UCustomizableObjectNodeRemapPinsByNameDefaultPin::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	if (DefaultPin)
	{
		check(DefaultPin->Direction == EGPD_Output);

		// Remap the default pin to the first mesh pin
		// Orphan all pins except the default
		for (UEdGraphPin* Pin : OldPins)
		{
			if (Pin->LinkedTo.Num() && Pin != DefaultPin)
			{
				PinsToOrphan.Add(Pin);
			}
		}

		// Find the first mesh pin
		UEdGraphPin* FirstMeshPin = nullptr;
		for (UEdGraphPin* Pin : NewPins)
		{
			if (Pin->Direction == EGPD_Output && Pin->PinType == DefaultPin->PinType)
			{
				FirstMeshPin = Pin;
				break;
			}
		}

		if (FirstMeshPin)
		{
			PinsToRemap.Add(DefaultPin, FirstMeshPin);
		}
	}
	else
	{
		Super::RemapPins(Node, OldPins, NewPins, PinsToRemap, PinsToOrphan);
	}
}

