// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByName.h"

#include "CustomizableObjectNodeRemapPinsByNameDefaultPin.generated.h"

class UCustomizableObjectNode;
class UEdGraphPin;
class UObject;


/** If a default pin is given, remap it to the first pin. Otherwise, remap pins by name as usual. */
UCLASS()
class UCustomizableObjectNodeRemapPinsByNameDefaultPin : public UCustomizableObjectNodeRemapPinsByName
{
	GENERATED_BODY()

public:
	/** Default pin to remap with. */
	UEdGraphPin* DefaultPin = nullptr;

	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};

