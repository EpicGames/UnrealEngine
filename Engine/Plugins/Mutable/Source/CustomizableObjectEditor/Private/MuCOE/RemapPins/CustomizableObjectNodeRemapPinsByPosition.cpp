// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPinsByPosition.h"

#include "EdGraph/EdGraphPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeRemapPinsByPosition)


/** Remap pins of a given direction by position. */
void RemapPinsInternal(const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan, EEdGraphPinDirection Direction)
{
	uint64 NewPinsIndex = 0;
	uint64 OldPinsIndex = 0;
	while (NewPinsIndex < NewPins.Num() && OldPinsIndex < OldPins.Num())
	{
		UEdGraphPin* const NewPin = NewPins[NewPinsIndex]; // Const pointer
		UEdGraphPin* const OldPin = OldPins[OldPinsIndex];

		if (NewPin->Direction == Direction)
		{
			if (OldPin->Direction == Direction)
			{
				PinsToRemap.Add(OldPin, NewPin);
				++NewPinsIndex;
			}
		
			++OldPinsIndex;
		}
		else 
		{
			++NewPinsIndex;
		}
	}

	// If we have not reach the end of OldPins means that there are remaining pins in this direction that has not been mapped
	while (OldPinsIndex < OldPins.Num())
	{
		UEdGraphPin* const OldPin = OldPins[OldPinsIndex];  // Const pointer
		if (OldPin->Direction == Direction && OldPin->LinkedTo.Num())
		{
			PinsToOrphan.Add(OldPins[OldPinsIndex]);
		}

		++OldPinsIndex;
	}
}


void UCustomizableObjectNodeRemapPinsByPosition::RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan)
{
	RemapPinsInternal(OldPins, NewPins, PinsToRemap, PinsToOrphan, EGPD_Input);
	RemapPinsInternal(OldPins, NewPins, PinsToRemap, PinsToOrphan, EGPD_Output);
}
