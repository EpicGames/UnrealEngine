// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPins.h"

#include "CustomizableObjectNodeRemapPinsByPosition.generated.h"

class UEdGraphPin;
class UObject;


/**
 * Remap pins by position.
 *
 * Remap pins by their relative order. Output pins and input pins get remapped independently.
 *
 * Use inheritance is a node requires a set of special rules when remapping by position.
 */
UCLASS()
class UCustomizableObjectNodeRemapPinsByPosition : public UCustomizableObjectNodeRemapPins
{
public:
	GENERATED_BODY()

	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) override;
};
