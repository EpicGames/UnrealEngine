// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectEditorAdvancedPreviewSettings.h"

#include "AssetViewerSettings.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectEditorLevelProfile.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

class FAdvancedPreviewScene;
class SWidget;

#define LOCTEXT_NAMESPACE "LevelSettingsPreview"


void SLevelSelectWindow::Construct(const FArguments& InArgs)
{
	ProfileAddedSuccessfully = false;

	CustomizableObjectEditor = InArgs._CustomizableObjectEditor.Get();
	DefaultSettings = InArgs._DefaultSettings.Get();

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("World")));
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &SLevelSelectWindow::OnLevelAssetSelected);
	AssetPickerConfig.ThumbnailScale = 0.4f;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SNewProfileWindow_Title_1", "Select a level asset with UObjects corresponding the the filled names"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SLevelSelectWindow::OnButtonClick, EAppReturnType::Ok)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SLevelSelectWindow::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]
	);
}


FReply SLevelSelectWindow::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	bool Result = true;

	if (UserResponse == EAppReturnType::Ok)
	{
		if (LevelAssetData.AssetClassPath != FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("World")))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("PostProcessNameValueMissing", "Please select a Level asset"));
			return FReply::Handled();
		}

		LevelAssetPath = LevelAssetData.GetObjectPathString();

		Result = LevelProfileManage::LoadProfileUObjectNames(LevelAssetPath, ArrayDirectionalLightName, ArrayPostProcessName, ArraySkyLightName);

		if (Result)
		{
			TSharedRef<SUObjectSelecWindow> UObjectSelectDialog =
				SNew(SUObjectSelecWindow)
				.LevelAssetPath(LevelAssetPath)
				.LevelSelectWindow(this)
				.CustomizableObjectEditor(CustomizableObjectEditor)
				.DefaultSettings(DefaultSettings);

			UObjectSelectDialog->ShowModal();
		}
	}

	if ((UserResponse == EAppReturnType::Cancel) || ProfileAddedSuccessfully)
	{
		RequestDestroyWindow();
	}

	return FReply::Handled();
}


void SLevelSelectWindow::OnLevelAssetSelected(const FAssetData& AssetData)
{
	LevelAssetData = AssetData;
}


EAppReturnType::Type SLevelSelectWindow::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}


void SLevelSelectWindow::SetProfileAddedSuccessfully(bool Value)
{
	ProfileAddedSuccessfully = Value;
}


void SUObjectSelecWindow::Construct(const FArguments& InArgs)
{
	UserResponse = EAppReturnType::Cancel;
	LevelAssetPath = InArgs._LevelAssetPath.Get();
	LevelSelectWindow = InArgs._LevelSelectWindow.Get();
	CustomizableObjectEditor = InArgs._CustomizableObjectEditor.Get();
	DefaultSettings = InArgs._DefaultSettings.Get();

	TArray<FString> ArrayDirectionalLight;
	TArray<FString> ArrayPostProcess;
	TArray<FString> ArraySkyLight;

	int DirectionalLightProfileValueIndex = -1;
	int PostProcessVolumeProfileValueIndex = -1;
	int SkyLightProfileValueIndex = -1;

	TSharedPtr<FString> InitiallySelectedDirectionalLightItem = nullptr;
	TSharedPtr<FString> InitiallySelectedPostProcessItem = nullptr;
	TSharedPtr<FString> InitiallySelectedSkyLightItem = nullptr;

	LevelProfileManage::LoadProfileUObjectNames(LevelAssetPath,
											    ArrayDirectionalLight,
											    ArrayPostProcess,
											    ArraySkyLight);

	for (int32 i = 0; i < ArrayDirectionalLight.Num(); ++i)
	{
		ArrayDirectionalLightName.Add(MakeShareable(new FString(ArrayDirectionalLight[i])));
	}

	for (int32 i = 0; i < ArrayPostProcess.Num(); ++i)
	{
		ArrayPostProcessName.Add(MakeShareable(new FString(ArrayPostProcess[i])));
	}

	for (int32 i = 0; i < ArraySkyLight.Num(); ++i)
	{
		ArraySkyLightName.Add(MakeShareable(new FString(ArraySkyLight[i])));
	}

	if (ArrayDirectionalLightName.Num() > 0)
	{
		InitiallySelectedDirectionalLightItem = ArrayDirectionalLightName[0];
	}

	if (ArrayPostProcessName.Num() > 0)
	{
		InitiallySelectedPostProcessItem = ArrayPostProcessName[0];
	}

	if (ArraySkyLightName.Num() > 0)
	{
		InitiallySelectedSkyLightItem = ArraySkyLightName[0];
	}

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SNewProfileWindow_Title_2", "Select a level asset with UObjects corresponding the the filled names"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
							+ SUniformGridPanel::Slot(0, 0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ProfileName", "Profile Name"))
							]
							+ SUniformGridPanel::Slot(1, 0)
							[
								SAssignNew(ProfileName, SEditableTextBox)
								.SelectAllTextWhenFocused(true)
								.RevertTextOnEscape(true)
								.MinDesiredWidth(30.f)
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
							+ SUniformGridPanel::Slot(0, 0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("DirectionalLightName", "Directional Light Scene Element Name"))
							]
							+ SUniformGridPanel::Slot(1, 0)
							[
								SAssignNew(DirectionalLight, STextComboBox)
								.OptionsSource(&ArrayDirectionalLightName)
								.InitiallySelectedItem(InitiallySelectedDirectionalLightItem)
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
							+ SUniformGridPanel::Slot(0, 0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("PostProcessName", "Post Process Scene Element Name"))
							]
							+ SUniformGridPanel::Slot(1, 0)
							[
								SAssignNew(PostProcess, STextComboBox)
								.OptionsSource(&ArrayPostProcessName)
								.InitiallySelectedItem(InitiallySelectedPostProcessItem)
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
							+ SUniformGridPanel::Slot(0, 0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SkyLightName", "Sky Light Scene Element Name"))
							]
							+ SUniformGridPanel::Slot(1, 0)
							[
								SAssignNew(SkyLight, STextComboBox)
								.OptionsSource(&ArraySkyLightName)
								.InitiallySelectedItem(InitiallySelectedSkyLightItem)
							]
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SUObjectSelecWindow::OnButtonClick, EAppReturnType::Ok)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SUObjectSelecWindow::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]
	);
}


EAppReturnType::Type SUObjectSelecWindow::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}


FReply SUObjectSelecWindow::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	bool Result = true;

	if (UserResponse == EAppReturnType::Ok)
	{
		// Retrieve Profile and selected UObject names
		FString ProfileNameValue = ProfileName->GetText().ToString();
		FString DirectionalLightValue = *DirectionalLight->GetSelectedItem().Get();
		FString PostProcessVolumeValue = *PostProcess->GetSelectedItem().Get();
		FString SkyLightNameValue = *SkyLight->GetSelectedItem().Get();

		if (ProfileNameValue.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ProfileNameEmpty", "Profile name is blank, please enter a profile name"));
			Result = false;
		}
		else if (LevelProfileManage::ProfileNameExist(DefaultSettings->Profiles, ProfileNameValue))
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ProfileNameAlreadyExists", "Profile name already exists, please enter a new profile name"));
			Result = false;
		}
		else
		{
			FPreviewSceneProfile PreviewSceneProfile;

			Result = LevelProfileManage::FillPreviewSceneProfile(ProfileNameValue,
																 LevelAssetPath,
																 DirectionalLightValue,
																 PostProcessVolumeValue,
																 SkyLightNameValue,
																 PreviewSceneProfile);
			// Make new entry in profile
			CustomizableObjectEditor.Pin()->GetAdvancedPreviewSettings()->AddProfileToEditor(PreviewSceneProfile);

			if (LevelSelectWindow != nullptr)
			{
				LevelSelectWindow->SetProfileAddedSuccessfully(true);
			}
		}
	}

	if (Result)
	{
		RequestDestroyWindow();
	}

	return FReply::Handled();
}


SCustomizableObjectEditorAdvancedPreviewSettings::SCustomizableObjectEditorAdvancedPreviewSettings()
{
	AdditionalSettings = nullptr;
	DetailCustomizations = TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>();
	PropertyTypeCustomizations = TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>();
}


void SCustomizableObjectEditorAdvancedPreviewSettings::Construct(const FArguments& InArgs, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene)
{
	SAdvancedPreviewDetailsTab::FArguments ParentArgs;
	ParentArgs.AdditionalSettings( InArgs._CustomSettings );
	SAdvancedPreviewDetailsTab::Construct(ParentArgs, InPreviewScene);
	
	CustomizableObjectEditor = InArgs._CustomizableObjectEditor.Get();

	if (CustomizableObjectEditor.Pin()->ShowProfileManagementOptions())
	{
		TSharedRef< SWidget > ActualContent = ChildSlot.GetWidget();
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				ActualContent
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(STextBlock)
						.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 12, "Regular"))
						.Text(LOCTEXT("LevelPreviewSettings", "Profile from Level Asset"))
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.OnClicked(this, &SCustomizableObjectEditorAdvancedPreviewSettings::ShowAddProfileWindow)
						.Text(LOCTEXT("AddProfile", "Add new Profile"))
						.ToolTipText(LOCTEXT("AddProfileToolTip", "Add new Profile reading information from an existing level"))
					]
				]
			]
		];	
	}
}


FReply SCustomizableObjectEditorAdvancedPreviewSettings::ShowAddProfileWindow()
{
	TSharedRef<SLevelSelectWindow> LevelSelectWindow =
		SNew(SLevelSelectWindow)
		.CustomizableObjectEditor(CustomizableObjectEditor)
		.DefaultSettings(DefaultSettings);
	
	LevelSelectWindow->ShowModal();
	return FReply::Handled();
}


void SCustomizableObjectEditorAdvancedPreviewSettings::LoadProfileEnvironment()
{
	for (int32 i = 0; i < DefaultSettings->Profiles.Num(); ++i)
	{
		if (!DefaultSettings->Profiles[i].EnvironmentCubeMapPath.IsEmpty())
		{
			UObject* LoadedObject = nullptr;
			LoadedObject = UE::Mutable::Private::LoadObject<UObject>(nullptr, *DefaultSettings->Profiles[i].EnvironmentCubeMapPath);
			while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(LoadedObject))
			{
				LoadedObject = Redirector->DestinationObject;
			}

			if (LoadedObject)
			{
				DefaultSettings->Profiles[i].EnvironmentCubeMap = CastChecked<UTextureCube>(LoadedObject);
			}
		}
	}
}


void SCustomizableObjectEditorAdvancedPreviewSettings::AddProfileToEditor(const FPreviewSceneProfile& PreviewSceneProfile)
{
	DefaultSettings->Profiles.Add(PreviewSceneProfile);
	DefaultSettings->PostEditChange();
	Refresh();
	ProfileComboBox->SetSelectedItem(ProfileNames.Last());
}

#undef LOCTEXT_NAMESPACE
