// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectInstanceEditor.h"
#include "AssetRegistry/AssetData.h"
#include "SAdvancedPreviewDetailsTab.h"
#include "Widgets/SWindow.h"

class FAdvancedPreviewScene;
class FCustomizableObjectEditor;
class SEditableTextBox;
class STextComboBox;
class UAssetViewerSettings;
class UObject;
struct FPreviewSceneProfile;


/** Simple window with a content browser filtered for ULevel assets to select a ULevel with a
* DirectionalLight, PostProcess and SkyLight elements to form a profile */
class SLevelSelectWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SLevelSelectWindow) {}
		SLATE_ATTRIBUTE(TWeakPtr<ICustomizableObjectInstanceEditor>, CustomizableObjectEditor)
		SLATE_ATTRIBUTE(UAssetViewerSettings*, DefaultSettings)
	SLATE_END_ARGS()

	SLevelSelectWindow()
		: UserResponse(EAppReturnType::Cancel)
	{
	}

	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	void SetProfileAddedSuccessfully(bool Value);

private:
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	/** Callback for when the user selects an asset in the asset picker widget */
	void OnLevelAssetSelected(const FAssetData& AssetData);

	EAppReturnType::Type UserResponse;
	FAssetData LevelAssetData; // Current ULevel selected asset

	TArray<FString> ArrayDirectionalLightName;
	TArray<FString> ArrayPostProcessName;
	TArray<FString> ArraySkyLightName;
	FString LevelAssetPath;
	bool ProfileAddedSuccessfully;
	TWeakPtr<ICustomizableObjectInstanceEditor> CustomizableObjectEditor;
	UAssetViewerSettings* DefaultSettings;
};


/** In this window, all the directional light, post process volume and sky light elements of the
* selected / edited profile are shown in drop down lists */
class SUObjectSelecWindow : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SUObjectSelecWindow) {}
		SLATE_ATTRIBUTE(FString, LevelAssetPath)
		SLATE_ATTRIBUTE(SLevelSelectWindow*, LevelSelectWindow)
		SLATE_ATTRIBUTE(TWeakPtr<ICustomizableObjectInstanceEditor>, CustomizableObjectEditor)
		SLATE_ATTRIBUTE(UAssetViewerSettings*, DefaultSettings)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

private:
	FReply OnButtonClick(EAppReturnType::Type ButtonID);

	EAppReturnType::Type UserResponse;
	TSharedPtr<SEditableTextBox> ProfileName;
	TSharedPtr<STextComboBox> DirectionalLight;
	TSharedPtr<STextComboBox> PostProcess;
	TSharedPtr<STextComboBox> SkyLight;
	FString LevelAssetPath;
	SLevelSelectWindow* LevelSelectWindow; // Pointer to parent window, needed to close it if new profile added successfully
	TWeakPtr<ICustomizableObjectInstanceEditor> CustomizableObjectEditor;
	TArray<TSharedPtr<FString>> ArrayDirectionalLightName;
	TArray<TSharedPtr<FString>> ArrayPostProcessName;
	TArray<TSharedPtr<FString>> ArraySkyLightName;
	UAssetViewerSettings* DefaultSettings;
};


/** This widget extends the SAdvancedPreviewDetailsTab tab adding the profile management options at
* the bottom of the tab */
class SCustomizableObjectEditorAdvancedPreviewSettings : public SAdvancedPreviewDetailsTab//,
														 //public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectEditorAdvancedPreviewSettings) {}
		SLATE_ARGUMENT(UCustomSettings*, CustomSettings) // From SAdvancedPreviewDetailsTab
		SLATE_ATTRIBUTE(TWeakPtr<ICustomizableObjectInstanceEditor>, CustomizableObjectEditor)
	SLATE_END_ARGS()

	SCustomizableObjectEditorAdvancedPreviewSettings();

	void Construct(const FArguments& InArgs, const TSharedRef<FAdvancedPreviewScene>& InPreviewScene);

	FReply ShowAddProfileWindow();

	/** Applies the preview scene profile to the LevelProfileProfile profile used to applyt this kind of profiles */
	void AddProfileToEditor(const FPreviewSceneProfile& PreviewSceneProfile);
	
	/** Load the environment map of each prodile in DefaultSettings::Profiles since default constructor loads a predetermined environment */
	void LoadProfileEnvironment();

	TWeakPtr<ICustomizableObjectInstanceEditor> CustomizableObjectEditor; // Pointer to editor instance
};

