// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectEditorTagExplorer.h"

#include "DetailLayoutBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "HAL/PlatformApplicationMisc.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeModifierEditSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSectionVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipDeform.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipMorph.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithMesh.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierClipWithUVMask.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"


class ITableRow;
class STableViewBase;
class SWidget;
class UObject;

const FName SCustomizableObjectEditorTagExplorer::COLUMN_OBJECT(TEXT("Customizable Object"));
const FName SCustomizableObjectEditorTagExplorer::COLUMN_TYPE(TEXT("Node Type"));

#define LOCTEXT_NAMESPACE "SCustomizableObjectEditorTagExplorer"


void SCustomizableObjectEditorTagExplorer::Construct(const FArguments & InArgs)
{
	CustomizableObjectEditorPtr = InArgs._CustomizableObjectEditor;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 2.0f, 0, 0)
			[
				SNew(STextBlock).Text(FText::FromString("Selected Tag:"))
			]
	
			+ SHorizontalBox::Slot()
			.Padding(5.0f, 0, 0, 0)
			[
				SAssignNew(TagComboBox, SComboButton)
				.OnGetMenuContent(this, &SCustomizableObjectEditorTagExplorer::OnGetTagsMenuContent)
				.VAlign(VAlign_Center)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &SCustomizableObjectEditorTagExplorer::GetCurrentItemLabel)
				]
			]
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton).Text(LOCTEXT("CopyToClipboard", "Copy to Clipboard")).OnClicked(this, &SCustomizableObjectEditorTagExplorer::CopyTagToClipboard)
				.ToolTipText(LOCTEXT("CopyToClipboardToolTip", "Copy tag name to clipboard."))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		[
			SNew(STextBlock).Text(FText::FromString("Used in:"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SAssignNew(ListViewWidget, SListView<TWeakObjectPtr<UCustomizableObjectNode>>)
			.ListItemsSource(&Nodes)
			.OnGenerateRow(this, &SCustomizableObjectEditorTagExplorer::OnGenerateTableRow)
			.OnSelectionChanged(this, &SCustomizableObjectEditorTagExplorer::OnTagTableSelectionChanged)
			.SelectionMode(ESelectionMode::Single)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(COLUMN_OBJECT)
				.DefaultLabel(LOCTEXT("CustomizableObject_ColumnName", "Customizable Object"))
				.FillWidth(0.5f)
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.SortMode(this, &SCustomizableObjectEditorTagExplorer::GetColumnSortMode, COLUMN_OBJECT)
				.OnSort(this, &SCustomizableObjectEditorTagExplorer::SortListView)

				+ SHeaderRow::Column(COLUMN_TYPE)
				.DefaultLabel(LOCTEXT("NodeType_ColumnName", "Node Type"))
				.FillWidth(0.5f)
				.HAlignHeader(EHorizontalAlignment::HAlign_Center)
				.HAlignCell(EHorizontalAlignment::HAlign_Left)
				.SortMode(this, &SCustomizableObjectEditorTagExplorer::GetColumnSortMode, COLUMN_TYPE)
				.OnSort(this, &SCustomizableObjectEditorTagExplorer::SortListView)
			)
		]
	];
}


TSharedRef<SWidget> SCustomizableObjectEditorTagExplorer::OnGetTagsMenuContent()
{
	NodeTags.Empty();

	if (UCustomizableObject* CustomizableObject = CustomizableObjectEditorPtr->GetCustomizableObject())
	{
		TArray<FString> Tags;
		TSet<UCustomizableObject*> CustomizableObjectTree;

		GraphTraversal::GetAllObjectsInGraph(CustomizableObject, CustomizableObjectTree);

		for (const UCustomizableObject* CustObject : CustomizableObjectTree)
		{
			if (CustObject)
			{
				FillTagInformation(*CustObject, Tags);
			}
		}

		if (Tags.Num())
		{
			FMenuBuilder MenuBuilder(true, NULL);

			for (int32 TagIndex = 0; TagIndex < Tags.Num(); ++TagIndex)
			{
				FText TagText = FText::FromString(Tags[TagIndex]);
				FUIAction Action(FExecuteAction::CreateSP(this, &SCustomizableObjectEditorTagExplorer::OnComboBoxSelectionChanged, Tags[TagIndex]));
				MenuBuilder.AddMenuEntry(TagText, FText::GetEmpty(), FSlateIcon(), Action);
			}

			return MenuBuilder.MakeWidget();
		}
	}

	return SNullWidget::NullWidget;
}


void SCustomizableObjectEditorTagExplorer::FillTagInformation(const UCustomizableObject& Object, TArray<FString>& Tags)
{
	if (Object.GetPrivate()->GetSource())
	{
		for (const TObjectPtr<UEdGraphNode>& Node : Object.GetPrivate()->GetSource()->Nodes)
		{
			// Gather tags from nodes that activate them
			if (UCustomizableObjectNode* TypedNodeMat = Cast<UCustomizableObjectNode>(Node))
			{
				const TArray<FString> MaterialTags = TypedNodeMat->GetEnableTags();
				
				if (MaterialTags.Num())
				{
					for (const FString& CurrentTag : MaterialTags)
					{
						NodeTags.Add(CurrentTag, TypedNodeMat);
						Tags.AddUnique(CurrentTag);
					}
				}
			}

			// Gather tags from nodes that require them

			// TODO: All other variations?
			if (UCONodeSkeletalMeshSectionVariation* TypedNodeVariations = Cast<UCONodeSkeletalMeshSectionVariation>(Node))
			{
				for (int32 i = 0; i < TypedNodeVariations->GetNumVariations(); ++i)
				{
					const FString& VariationTag = TypedNodeVariations->GetVariation(i).Tag; 
					NodeTags.Add(VariationTag, TypedNodeVariations);					
					Tags.AddUnique(VariationTag);
				}
			}

			if (UCustomizableObjectNodeModifierBase* TypedNodeModifier = Cast<UCustomizableObjectNodeModifierBase>(Node))
			{
				const TArray<FString>& RequiredTags = TypedNodeModifier->RequiredTags;
				for (const FString& CurrentTag : RequiredTags)
				{
					NodeTags.Add(CurrentTag, TypedNodeModifier);
					Tags.AddUnique(CurrentTag);
				}
			}
		}
	}
}


TSharedRef<SWidget> SCustomizableObjectEditorTagExplorer::MakeComboButtonItemWidget(TSharedPtr<FString> StringItem)
{
	return SNew(STextBlock).Text(FText::FromString(*StringItem));
}


FText SCustomizableObjectEditorTagExplorer::GetCurrentItemLabel() const
{
	if (!SelectedTag.IsEmpty())
	{
		return FText::FromString(*SelectedTag);
	}

	return LOCTEXT("InvalidComboEntryText", "None");
}


FReply SCustomizableObjectEditorTagExplorer::CopyTagToClipboard()
{
	if (!SelectedTag.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*SelectedTag);
	}

	return FReply::Handled();
}


void SCustomizableObjectEditorTagExplorer::OnComboBoxSelectionChanged(const FString NewValue)
{
	SelectedTag = NewValue;

	if (!NewValue.IsEmpty())
	{
		TArray<UCustomizableObjectNode*> AllNodes;
		NodeTags.MultiFind(NewValue, AllNodes, false);

		Nodes.Empty();

		for (UCustomizableObjectNode* Node : AllNodes)
		{
			if (Node)
			{
				Nodes.Add(MakeWeakObjectPtr(Node));
			}
		}
	}

	if (ListViewWidget.IsValid())
	{
		ListViewWidget->RequestListRefresh();
	}
}


TSharedRef<ITableRow> SCustomizableObjectEditorTagExplorer::OnGenerateTableRow(TWeakObjectPtr<UCustomizableObjectNode> Node, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STagExplorerTableRow, OwnerTable).CustomizableObjectNode(Node);
}


void SCustomizableObjectEditorTagExplorer::OnTagTableSelectionChanged(TWeakObjectPtr<UCustomizableObjectNode> Entry, ESelectInfo::Type SelectInfo) const
{
	if (Entry.IsValid())
	{
		UObject* Object = Entry->GetCustomizableObjectGraph()->GetOuter();

		// Make sure the editor exists for this asset
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Object);

		// Find it
		TSharedPtr<FCustomizableObjectGraphEditorToolkit> Editor = Entry->GetGraphEditor();
		Editor->SelectNode(Entry.Get());

		if (ListViewWidget.IsValid())
		{
			ListViewWidget->ClearSelection();
			ListViewWidget->RequestListRefresh();
		}
	}
}


void SCustomizableObjectEditorTagExplorer::SortListView(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	CurrentSortColumn = ColumnId;
	SortMode = NewSortMode;

	if (ColumnId == SCustomizableObjectEditorTagExplorer::COLUMN_OBJECT)
	{
		Nodes.Sort([&](const TWeakObjectPtr<UCustomizableObjectNode>& NodeA, const TWeakObjectPtr<UCustomizableObjectNode>& NodeB)
		{
			if (NodeA.IsValid() && NodeB.IsValid())
			{
				FString NameA = GetNameSafe(NodeA->GetOutermostObject());
				FString NameB = GetNameSafe(NodeB->GetOutermostObject());

				return NewSortMode == EColumnSortMode::Ascending ? NameA < NameB : NameA > NameB;
			}

			return NodeA.IsValid();
		});
	}
	else if (ColumnId == SCustomizableObjectEditorTagExplorer::COLUMN_TYPE)
	{
		Nodes.Sort([&](const TWeakObjectPtr<UCustomizableObjectNode>& NodeA, const TWeakObjectPtr<UCustomizableObjectNode>& NodeB)
		{
			if (NodeA.IsValid() && NodeB.IsValid())
			{
				const FString NodeTypeA = NodeA->GetNodeTitle(ENodeTitleType::ListView).ToString();
				const FString NodeTypeB = NodeB->GetNodeTitle(ENodeTitleType::ListView).ToString();

				return NewSortMode == EColumnSortMode::Ascending ? NodeTypeA < NodeTypeB : NodeTypeA > NodeTypeB;
			}

			return NodeA.IsValid();
		});
	}
	else
	{
		check(false); // Unknown method.
	}

	ListViewWidget->RequestListRefresh();
}


EColumnSortMode::Type SCustomizableObjectEditorTagExplorer::GetColumnSortMode(const FName ColumnName) const
{
	if (CurrentSortColumn != ColumnName)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}


void STagExplorerTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	Node = InArgs._CustomizableObjectNode;

	SMultiColumnTableRow<TWeakObjectPtr<UCustomizableObjectNode> >::Construct(FSuperRowType::FArguments(), OwnerTableView);
}


TSharedRef<SWidget> STagExplorerTableRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (InColumnName == SCustomizableObjectEditorTagExplorer::COLUMN_OBJECT)
	{
		if (Node.IsValid())
		{
			if (UCustomizableObject* CustomizableObject = Cast<UCustomizableObject>(Node->GetOutermostObject()))
			{
				return SNew(SBox).Padding(5.0f,0.0f,0.0f,0.0f)
				[
					SNew(STextBlock).Text(FText::FromString(CustomizableObject->GetName()))
				];
			}
		}
	}
	
	else if (InColumnName == SCustomizableObjectEditorTagExplorer::COLUMN_TYPE)
	{
		if (Node.IsValid())
		{
			return SNew(SBox).Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock).Text(Node->GetNodeTitle(ENodeTitleType::ListView))
			];
		}
	}
	
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE 
