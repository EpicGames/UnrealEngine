// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class SListView;

class FCustomizableObjectEditor;
class ITableRow;
class STableViewBase;
class SComboButton;
class SWidget;
class UCustomizableObject;


class STagExplorerTableRow : public SMultiColumnTableRow<TWeakObjectPtr<UCustomizableObjectNode>>
{
public:

	SLATE_BEGIN_ARGS(STagExplorerTableRow) {}
		SLATE_ARGUMENT(TWeakObjectPtr<UCustomizableObjectNode>, CustomizableObjectNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:

	/** The info about the widget that we are visualizing */
	TWeakObjectPtr<UCustomizableObjectNode> Node;

};


class SCustomizableObjectEditorTagExplorer : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjectEditorTagExplorer){}
		SLATE_ARGUMENT(FCustomizableObjectEditor*, CustomizableObjectEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	static const FName COLUMN_OBJECT;
	static const FName COLUMN_TYPE;

private:

	/** Callback to fill the combobox options */
	TSharedRef<SWidget> OnGetTagsMenuContent();

	/** Fills a list with all the tags found in the nodes of a graph */
	void FillTagInformation(const UCustomizableObject& Object, TArray<FString>& Tags);

	/** Generates the combobox with all the tags  */
	TSharedRef<SWidget> MakeComboButtonItemWidget(TSharedPtr<FString> StringItem);

	/** Generates the text of the tags combobox */
	FText GetCurrentItemLabel() const;

	/** Copies the tag name to the clipboard */
	FReply CopyTagToClipboard();

	/** OnSelectionChanged callback of the tags combobox */
	void OnComboBoxSelectionChanged(const FString NewValue);

	/** Tags table callbacks*/
	TSharedRef<ITableRow> OnGenerateTableRow(TWeakObjectPtr<UCustomizableObjectNode> Node, const TSharedRef<STableViewBase>& OwnerTable);
	void OnTagTableSelectionChanged(TWeakObjectPtr<UCustomizableObjectNode> Entry, ESelectInfo::Type SelectInfo) const;

	/** Sorts the content of the list view alphabetically */
	void SortListView(const EColumnSortPriority::Type SortPriority, const FName& ColumnName, const EColumnSortMode::Type NewSortMode);

	/** Returns the sorting mode of the specified column */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnName) const;

private:

	/** Pointer back to the editor tool that owns us */
	FCustomizableObjectEditor* CustomizableObjectEditorPtr;

	/** Combobox for the Customizable Object Tags */
	TSharedPtr<SComboButton> TagComboBox;

	/** Combobox Selection */
	FString SelectedTag;

	/** List views for the nodes that contain the same tag */
	TSharedPtr<SListView<TWeakObjectPtr<UCustomizableObjectNode>>> ListViewWidget;

	/** Arrays for each type of node that contains a tag */
	TArray<TWeakObjectPtr<UCustomizableObjectNode>> Nodes;

	/** Multimap to relate nodes with tags */
	TMultiMap<FString, UCustomizableObjectNode*> NodeTags;

	/** Stores name of the column to sort */
	FName CurrentSortColumn;

	/** Stores the sorting mode of the selected column */
	EColumnSortMode::Type SortMode = EColumnSortMode::Type::None;
};
