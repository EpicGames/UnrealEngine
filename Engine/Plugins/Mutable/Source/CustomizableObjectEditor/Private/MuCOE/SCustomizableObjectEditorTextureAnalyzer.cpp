// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectEditorTextureAnalyzer.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/Texture.h"
#include "Engine/TextureDefines.h"
#include "Framework/Views/ITypedTableView.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCOE/CustomizableObjectEditor.h"
#include "MuCOE/CustomizableObjectInstanceEditor.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SCustomizableObjectEditorTextureAnalyzer)

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SCustomizableObjecEditorTextureAnalyzer"


void SCustomizableObjecEditorTextureAnalyzer::Construct(const FArguments& InArgs)
{
	CustomizableObjectEditor = InArgs._CustomizableObjectEditor;
	CustomizableObjectInstanceEditor = InArgs._CustomizableObjectInstanceEditor;

	TotalSizeTextures = SNew(STextBlock);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoHeight()
		.Padding(1.0f, 5.0f, 0.0f, 10.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("RefreshTable", "Refresh Table"))
				.OnClicked(this, &SCustomizableObjecEditorTextureAnalyzer::RefreshTextureAnalyzerTable)
				.Visibility(CustomizableObjectEditor || CustomizableObjectInstanceEditor ? EVisibility::Visible : EVisibility::Collapsed)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 10.0f, 0.0f, 0.0f)
			[
				TotalSizeTextures.ToSharedRef()
			]
		]

		+ SVerticalBox::Slot()
		[
			BuildTextureAnalyzerTable()
		]
	];
}


void SCustomizableObjecEditorTextureAnalyzer::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& i : TabTextures)
	{
		Collector.AddReferencedObject(i);
	}
}

void SCustomizableObjecEditorTextureAnalyzer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (TextureAnalyzerTable.IsValid())
	{
		if (TextureAnalyzerTable->GetLastClickedCell().IsValid() && TextureAnalyzerTable->GetLastClickedCell()->InEditMode())
		{
			OnTextureTableSelectionChanged(TextureAnalyzerTable->GetLastClickedCell());
		}
	}

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


TSharedRef<SWidget> SCustomizableObjecEditorTextureAnalyzer::BuildTextureAnalyzerTable()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TextureAnalyzerTable = PropertyEditorModule.CreatePropertyTable();
	TextureAnalyzerTable->SetIsUserAllowedToChangeRoot(false);
	TextureAnalyzerTable->SetSelectionMode(ESelectionMode::Single);
	TextureAnalyzerTable->SetSelectionUnit(EPropertyTableSelectionUnit::Cell);
	TextureAnalyzerTable->SetShowObjectName(false);

	if (TextureAnalyzerTable.IsValid())
	{
		FillTextureAnalyzerTable();
	}

	return PropertyEditorModule.CreatePropertyTableWidget(TextureAnalyzerTable.ToSharedRef());
}


void SCustomizableObjecEditorTextureAnalyzer::FillTextureAnalyzerTable(UCustomizableObjectInstance* PreviewInstance)
{
	TabTextures.Empty();

	uint32 TotalSize = 0;

	if(!PreviewInstance)
	{
		if (CustomizableObjectEditor)
		{
			PreviewInstance = CustomizableObjectEditor->GetPreviewInstance();
		}

		if (CustomizableObjectInstanceEditor)
		{
			PreviewInstance = CustomizableObjectInstanceEditor->GetPreviewInstance();
		}
	}

	if (PreviewInstance && PreviewInstance->HasAnySkeletalMesh())
	{
		const UCustomizableObject* CustomizableObject = PreviewInstance->GetCustomizableObject();
		check(CustomizableObject)

		for (const TObjectPtr<UTexture>& GeneratedTexture : PreviewInstance->GetPrivate()->Textures)
		{
			UTexture2D* Texture = Cast<UTexture2D>(GeneratedTexture);
			if (!Texture || !Texture->HasAllFlags(EObjectFlags::RF_Transient))
			{
				continue;
			}

			//New Object for the table
			UCustomizableObjectEditorTextureStats* Entry = NewObject<UCustomizableObjectEditorTextureStats>();
			TabTextures.Add(Entry);

			//Texture Info
			Entry->Texture = Texture;
			Entry->TextureName = Texture->GetName();
			Entry->ResolutionX = Texture->GetSizeX();
			Entry->ResolutionY = Texture->GetSizeY();
			Entry->LODBias = Texture->GetCachedLODBias();
			uint32 TextureSize = Texture->CalcTextureMemorySizeEnum(TMC_AllMips);
			Entry->Size = TextureSize / 1024.0f;
			Entry->Format = Texture->GetPixelFormat();
			Entry->IsStreamed = Texture->IsCurrentlyVirtualTextured() ? "Virtual Streamed" : (!Texture->IsStreamable() ? "Not Streamed" : "Streamed");
			Entry->LODGroup = Texture->LODGroup;
			
			//Total size
			TotalSize += TextureSize;
		}

		TextureAnalyzerTable->SetObjects(ObjectPtrDecay(TabTextures));
	}

	// Updating Texture Size Text
	if (TotalSizeTextures.IsValid())
	{
		TotalSizeTextures->SetText(FText::FromString(FString::Printf(TEXT("Total Size (All LODs, only Mutable): %.2f Mb "), TotalSize / (1024.0 * 1024.0))));
	}
}


void SCustomizableObjecEditorTextureAnalyzer::OnTextureTableSelectionChanged(TSharedPtr<class IPropertyTableCell> Cell)
{
	FString SelectedColumn = Cell->GetColumn()->GetDisplayName().ToString();

	if (UCustomizableObjectEditorTextureStats* Current = Cast<UCustomizableObjectEditorTextureStats>(Cell->GetObject().Get()))
	{
		//Open Texture Editor Tab
		if ((SelectedColumn == "Texture") && Current->Texture)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Current->Texture);
		}
	}

	//Avoid edit mode (not needed)
	Cell->ExitEditMode();
}


FReply SCustomizableObjecEditorTextureAnalyzer::RefreshTextureAnalyzerTable()
{
	return RefreshTextureAnalyzerTable(nullptr);
}

FReply SCustomizableObjecEditorTextureAnalyzer::RefreshTextureAnalyzerTable(UCustomizableObjectInstance* PreviewInstance)
{
	FillTextureAnalyzerTable(PreviewInstance);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE 
