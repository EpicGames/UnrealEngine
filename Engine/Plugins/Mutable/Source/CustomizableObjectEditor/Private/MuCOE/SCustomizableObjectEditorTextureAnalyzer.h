// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "SCustomizableObjectEditorTextureAnalyzer.generated.h"


class FCustomizableObjectEditor;
class FCustomizableObjectInstanceEditor;
class FReferenceCollector;
class IPropertyTable;
class STextBlock;
class SWidget;
class UCustomizableObjectInstance;
class UMaterialInterface;
class UObject;
class UTexture;

enum TextureGroup : int;

struct FGeometry;


/** Statistics for the Texture Analyzer */
UCLASS(Transient, MinimalAPI, meta = (DisplayName = "Texture Stats"))
class UCustomizableObjectEditorTextureStats : public UObject
{
	GENERATED_BODY()

public:
	/** Texture - double click to open */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Texture", ColumnWidth = "40", NoResetToDefault))
	FString TextureName;
	
	/** Used to open the texture in the editor*/
	UPROPERTY(meta = (NoResetToDefault))
	TObjectPtr<UTexture> Texture;

	/** Resolution of the texture */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Resolution X", ColumnWidth = "40", DisplayRight = "true", NoResetToDefault))
	int32 ResolutionX;

	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Resolution Y", ColumnWidth = "40", DisplayRight = "true", NoResetToDefault))
	int32 ResolutionY;

	/** The memory used in KB */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Size Kb", ColumnWidth = "90", NoResetToDefault, Units = "Kb"))
	float Size;

	/** The texture format, e.g. PF_DXT1 */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (ColumnWidth = "96", NoResetToDefault))
	TEnumAsByte<EPixelFormat> Format;

	/** LOD Bias for this texture. (Texture LODBias + Texture group) */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "LOD Bias", ColumnWidth = "70", NoResetToDefault))
	int32 LODBias;

	/** Says if the texture is being streamed */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Streamed", ColumnWidth = "70", NoResetToDefault))
	FString IsStreamed;

	/** The Level of detail group of the texture */
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Stats", meta = (DisplayName = "Texture Group", ColumnWidth = "70", NoResetToDefault))
	TEnumAsByte<TextureGroup> LODGroup;
};


class SCustomizableObjecEditorTextureAnalyzer : public SCompoundWidget, public FGCObject
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjecEditorTextureAnalyzer){}
		SLATE_ARGUMENT(FCustomizableObjectEditor*,CustomizableObjectEditor)
		SLATE_ARGUMENT(FCustomizableObjectInstanceEditor*, CustomizableObjectInstanceEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// FSerializableObject interface
	void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SCustomizableObjecEditorTextureAnalyzer");
	}

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Force the Update of the Texture Analyzer Table information */
	FReply RefreshTextureAnalyzerTable();
	/** Force the Update of the Texture Analyzer Table information and provides the instance to be analyzed*/
	FReply RefreshTextureAnalyzerTable(UCustomizableObjectInstance* PreviewInstance);

private:

	/** Build the Texture Analyzer table with the information of the textures */
	TSharedRef<SWidget> BuildTextureAnalyzerTable();

	/** Generate the information of the Texture Analyzer Table */
	void FillTextureAnalyzerTable(UCustomizableObjectInstance* PreviewInstance = nullptr);

	/** Callback for a Texture Analyzer Table Cell selection */
	void OnTextureTableSelectionChanged(TSharedPtr<class IPropertyTableCell> Cell);

public:

	/** Texture Analyzer table widget which shows the information of the transient textures used in the customizable object instance */
	TSharedPtr<IPropertyTable> TextureAnalyzerTable;

private:

	/** Pointer back to the editor tool that owns us */
	FCustomizableObjectEditor* CustomizableObjectEditor = nullptr;

	/** Pointer back to the editor tool that owns us */
	FCustomizableObjectInstanceEditor* CustomizableObjectInstanceEditor = nullptr;

	/** The sum of all the textures shown in the Texture Analyzer */
	TSharedPtr< STextBlock > TotalSizeTextures;

	/** Array of Objects used in the Texture Analyzer Table */
	TArray<TObjectPtr<UObject>> TabTextures;

};
