// Copyright Epic Games, Inc. All Rights Reserved.	

#include "MuCOE/SCustomizableObjectEditorTransformViewportToolbar.h"

#include "Framework/Commands/UICommandList.h"
#include "MuCOE/CustomizableObjectEditorViewportLODCommands.h"

#define LOCTEXT_NAMESPACE "SCustomizableObjectEditorTransformViewportToolbar"

void SCustomizableObjectEditorTransformViewportToolbar::Construct( const FArguments& InArgs )
{
	Viewport = InArgs._Viewport;
	ViewportTapBody = InArgs._ViewportTapBody;

	const FCustomizableObjectEditorViewportLODCommands& ViewportLODMenuCommands = FCustomizableObjectEditorViewportLODCommands::Get();
	UICommandList = MakeShareable(new FUICommandList);

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

#undef LOCTEXT_NAMESPACE 
