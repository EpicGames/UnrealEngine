// Copyright Epic Games, Inc. All Rights Reserved.	

#pragma once

#include "SViewportToolBar.h"

class SCustomizableObjectEditorTransformViewportToolbar : public SViewportToolBar
{

public:
	SLATE_BEGIN_ARGS(SCustomizableObjectEditorTransformViewportToolbar){}
		SLATE_ARGUMENT(TSharedPtr<class SEditorViewport>, Viewport)
		SLATE_ARGUMENT(TSharedPtr<class SCustomizableObjectEditorViewportTabBody>, ViewportTapBody)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

protected:
	/** The editor viewport that we are in */
	TWeakPtr<class SEditorViewport> Viewport;

	/** Viewport we are in - derived class */
	TWeakPtr<class SCustomizableObjectEditorViewportTabBody> ViewportTapBody;

	/** Command list for the widgets to be added to the toolbar */
	TSharedPtr<class FUICommandList> UICommandList;
};
