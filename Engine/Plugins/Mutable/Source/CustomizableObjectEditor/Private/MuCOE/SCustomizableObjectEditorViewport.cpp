// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectEditorViewport.h"

#include "CustomizableObjectInstanceEditor.h"
#include "Animation/AnimationAsset.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorViewportCommands.h"
#include "ObjectEditorUtils.h"
#include "SMutableScrubPanel.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCOE/CustomizableObjectEditorActions.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "MuCOE/CustomizableObjectEditorViewportLODCommands.h"
#include "MuCOE/CustomizableObjectEditorViewportMenuCommands.h"
#include "MuCOE/CustomizableObjectPreviewScene.h"
#include "MuCOE/ICustomizableObjectInstanceEditor.h"
#include "MuCOE/SCustomizableObjectEditorViewportToolBar.h"
#include "MuCOE/SCustomizableObjectHighresScreenshot.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "UnrealEdGlobals.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/LoadUtils.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "Widgets/Input/STextComboBox.h"

class UCustomizableObject;
class UCustomizableObjectNodeProjectorConstant;
class UStaticMesh;
struct FCustomizableObjectProjector;
struct FGeometry;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditorViewportToolBar"


void SCustomizableObjectEditorViewport::Construct(const FArguments& InArgs,
	const TWeakPtr<FCustomizableObjectPreviewScene>& PreviewScene,
	const TWeakPtr<SCustomizableObjectEditorViewportTabBody>& TabBody,
	const TWeakPtr<ICustomizableObjectInstanceEditor>& Editor)
{
	PreviewScenePtr = PreviewScene;
	TabBodyPtr = TabBody;
	WeakEditor = Editor;
	
	SEditorViewport::Construct(
		SEditorViewport::FArguments()
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		.AddMetaData<FTagMetaData>(TEXT("Persona.Viewport"))
	);

	Client->VisibilityDelegate.BindSP(this, &SCustomizableObjectEditorViewport::IsVisible);
}

TSharedRef<FEditorViewportClient> SCustomizableObjectEditorViewport::MakeEditorViewportClient()
{
	FEditorModeTools* ModeTools = nullptr;
	if (TSharedPtr<ICustomizableObjectInstanceEditor> Editor = TabBodyPtr.Pin()->WeakEditor.Pin())
	{
		ModeTools = Editor->GetEditorModeTools();
	}

	LevelViewportClient = MakeShareable(new FCustomizableObjectEditorViewportClient(TabBodyPtr.Pin()->WeakEditor, ModeTools, PreviewScenePtr.Pin().Get(), SharedThis(this)));

	LevelViewportClient->ViewportType = LVT_Perspective;
	LevelViewportClient->bSetListenerPosition = false;
	LevelViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	LevelViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

	return LevelViewportClient.ToSharedRef();
}


TSharedPtr<FSceneViewport>& SCustomizableObjectEditorViewport::GetSceneViewport()
{
	return SceneViewport;
}


void SCustomizableObjectEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	FTextBlockStyle NormalTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	FTextBlockStyle CompileOverlayText = FTextBlockStyle(NormalTextStyle).SetFontSize(18);
	
	Overlay->AddSlot()
	.VAlign(VAlign_Top)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SCustomizableObjectEditorViewportToolBar, TabBodyPtr.Pin(), SharedThis(this)).Cursor(EMouseCursor::Default)
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		.Padding(4.0, 16.0, 0.0, 0.0)
		[
			SNew(STextBlock)
			.Text(this, &SCustomizableObjectEditorViewport::GetWarningText)
			.Visibility(this, &SCustomizableObjectEditorViewport::GetWarningTextVisibility)
			.ColorAndOpacity(FLinearColor::Yellow)
		]
		+SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		.Padding(4.0, 16.0, 0.0, 0.0)
		[
			SNew(STextBlock)
			.Text(this, &SCustomizableObjectEditorViewport::GetMeshInfoText)
			.Visibility(this, &SCustomizableObjectEditorViewport::GetMeshInfoTextVisibility)
			.ColorAndOpacity(FLinearColor::White)
		]
	];
	Overlay->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SNew(STextBlock)
		.Visibility(this, &SCustomizableObjectEditorViewport::GetShowCompileErrorOverlay)
		.Text_Raw(this, &SCustomizableObjectEditorViewport::GetCompileErrorOverlayText)
		.TextStyle(&CompileOverlayText)
		.ColorAndOpacity(FLinearColor::White)
		.ShadowOffset(FVector2D(1.5, 1.5))
		.ShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f))
	];
}


EVisibility SCustomizableObjectEditorViewport::GetShowCompileErrorOverlay() const
{
	return GetCompileErrorOverlayText().IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;
}


FText SCustomizableObjectEditorViewport::GetCompileErrorOverlayText() const
{
	const TSharedPtr<ICustomizableObjectInstanceEditor> Editor = WeakEditor.Pin();
	if (!Editor)
	{
		return {};
	}

	const UObject* EditingObject = Editor->GetObjectBeingEdited();

	const UCustomizableObject* Object = nullptr;
	const UCustomizableObjectInstance* Instance = nullptr;

	if (auto* CastObject = Cast<UCustomizableObject>(EditingObject))
	{
		Object = CastObject;
		Instance = Editor->GetPreviewInstance();
	}
	else if (auto* CastInstance = Cast<UCustomizableObjectInstance>(EditingObject))
	{
		Object = CastInstance->GetCustomizableObject();
		Instance = CastInstance;
	}
	else
	{
		check(false);
	}

	if (!Object)
	{
		return LOCTEXT("NoPreviewInstance", "No Customizable Object");
	}

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsGathering())
	{
		return LOCTEXT("LoadingAssetRegistry", "Loading Asset Registry...");
	}
	
	if (Object->GetPrivate()->Status.Get() == FCustomizableObjectStatusTypes::EState::Loading)
	{
		return LOCTEXT("Loading", "Loading...");
	}

	ICustomizableObjectEditorModule& Module = ICustomizableObjectEditorModule::GetChecked();
	if (Module.IsCompiling(*Object))
	{
		return LOCTEXT("Compiling", "Compiling...");
	}
	
	if (!Instance) // Only happens if Mutable has compilations disabled and the CO was not compiled.
	{
		return LOCTEXT("EmptyPreview", "Empty Preview");
	}
	
	const UCustomizableObjectSystem* System = UCustomizableObjectSystem::GetInstanceChecked();
	
	if (System->IsUpdating(Instance))
	{
		return LOCTEXT("Updating", "Updating...");
	}

	if (Object->GetPrivate()->CompilationResult == ECompilationResultPrivate::Errors) // Compilation errors have more priority than Update errors
	{
		return LOCTEXT("ErrorCompiling", "Error Compiling");
	}

	if (Instance->GetPrivate()->SkeletalMeshStatus == ESkeletalMeshStatus::Error)
	{
		return LOCTEXT("ErrorUpdating", "Error Updating");
	}

	if (!Instance->HasAnySkeletalMesh())
	{
		return LOCTEXT("EmptyPreview", "Empty Preview");
	}
	
	return {};
}


FText SCustomizableObjectEditorViewport::GetWarningText() const
{
	if (const UCustomizableObjectInstance* Instance = WeakEditor.Pin()->GetPreviewInstance())
	{
		if (const UCustomizableObject* Object = Instance->GetCustomizableObject())
		{
			if (const UModelResources* ModelResources = Object->GetPrivate()->GetModelResources())
			{
				if (!ModelResources->bIsCompiledWithOptimization)
				{
					return LOCTEXT("CompiledWithoutOptimization", "Compiled without maximum optimization. Updates will be slower!");
				}
			}
		}
	}

	return {};
}


EVisibility SCustomizableObjectEditorViewport::GetWarningTextVisibility() const
{
	return !GetWarningText().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}


FText SCustomizableObjectEditorViewport::GetMeshInfoText() const
{
	return LevelViewportClient->GetMeshInfoText();
}


EVisibility SCustomizableObjectEditorViewport::GetMeshInfoTextVisibility() const
{
	return LevelViewportClient->IsShowingMeshInfo() ? EVisibility::Visible : EVisibility::Collapsed;
}


void SCustomizableObjectEditorViewport::OnUndoRedo()
{
	LevelViewportClient->Invalidate();
}


//////////////////////////////////////////////////////////////////////////


void SCustomizableObjectEditorViewportTabBody::Construct(const FArguments& InArgs)
{
	UICommandList = MakeShareable(new FUICommandList);

	WeakEditor = InArgs._CustomizableObjectEditor;

	FCustomizableObjectEditorViewportMenuCommands::Register();
	FCustomizableObjectEditorViewportLODCommands::Register();

	FPreviewScene::ConstructionValues SceneConstructValues;
	SceneConstructValues.bShouldSimulatePhysics = true;

	PreviewScenePtr = MakeShareable(new FCustomizableObjectPreviewScene(SceneConstructValues));
	
	ViewportWidget = SNew(SCustomizableObjectEditorViewport, PreviewScenePtr.ToSharedRef(), SharedThis(this), WeakEditor);

	LevelViewportClient = StaticCastSharedPtr<FCustomizableObjectEditorViewportClient>(ViewportWidget->GetViewportClient());

	this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				BuildToolBar()
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				ViewportWidget.ToSharedRef()
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ScrubPanel, SMutableScrubPanel, LevelViewportClient.ToSharedRef())
			]
		];
	
	BindCommands();
}


void SCustomizableObjectEditorViewportTabBody::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Needed for the forced "LOD 0, 1, 2..." display mode to work in the preview
	PreviewScenePtr->GetWorld()->Tick(LEVELTICK_All, InDeltaTime);

	const TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& PreviewSkeletalMeshComponents = LevelViewportClient->GetPreviewMeshComponents();

	// Update the material list. Not ideal to do it every tick, but tracking changes on materials in the current instance is not easy right now.
	if (PreviewSkeletalMeshComponents.Num()>0)
	{
		MaterialNames.Empty();

		for (TPair<FName,TWeakObjectPtr<UDebugSkelMeshComponent>> Entry : PreviewSkeletalMeshComponents)
		{
			if (!Entry.Value.IsValid())
			{
				continue;
			}
			
			const TArray<UMaterialInterface*> Materials = Entry.Value->GetMaterials();
			for (UMaterialInterface* m : Materials)
			{
				if (m)
				{
					const UMaterial* BaseMaterial = m->GetBaseMaterial();
					MaterialNames.Add(MakeShareable(new FString(BaseMaterial->GetName())));
				}
			}
		}
	}
	else
	{
		MaterialNames.Empty();
	}

}


void SCustomizableObjectEditorViewportTabBody::ShowGizmoClipMorph(UCustomizableObjectNodeModifierClipMorph& ClipPlainNode) const
{
	SetTransformWidgetVisibility(EVisibility::Visible);
	LevelViewportClient->ShowGizmoClipMorph(ClipPlainNode);
}


void SCustomizableObjectEditorViewportTabBody::HideGizmoClipMorph() const
{
	SetTransformWidgetVisibility(EVisibility::Hidden);
	LevelViewportClient->HideGizmoClipMorph();
}


void SCustomizableObjectEditorViewportTabBody::ShowGizmoClipMesh(UCustomizableObjectNode& Node, FTransform* Transform, UObject& ClipMesh, int32 LODIndex, int32 SectionIndex, int32 MaterialSlotIndex) const
{
	SetTransformWidgetVisibility(EVisibility::Visible);		
	LevelViewportClient->ShowGizmoClipMesh(Node, Transform, ClipMesh, LODIndex, SectionIndex, MaterialSlotIndex);
}


void SCustomizableObjectEditorViewportTabBody::SetTransformWidgetVisibility(const EVisibility NewVisibilityState) const
{
	if (const TSharedPtr<SWidget> Toolbar = ViewportToolbarTransformWidget.Pin())
	{
		Toolbar->SetVisibility(NewVisibilityState);		
	}
}


void SCustomizableObjectEditorViewportTabBody::HideGizmoClipMesh() const
{
	SetTransformWidgetVisibility(EVisibility::Hidden);
	
	LevelViewportClient->HideGizmoClipMesh();
}


void SCustomizableObjectEditorViewportTabBody::ShowGizmoProjector(
	const FWidgetLocationDelegate& WidgetLocationDelegate, const FOnWidgetLocationChangedDelegate& OnWidgetLocationChangedDelegate,
	const FWidgetDirectionDelegate& WidgetDirectionDelegate, const FOnWidgetDirectionChangedDelegate& OnWidgetDirectionChangedDelegate,
	const FWidgetUpDelegate& WidgetUpDelegate, const FOnWidgetUpChangedDelegate& OnWidgetUpChangedDelegate,
	const FWidgetScaleDelegate& WidgetScaleDelegate, const FOnWidgetScaleChangedDelegate& OnWidgetScaleChangedDelegate,
	const FWidgetAngleDelegate& WidgetAngleDelegate,
	const FProjectorTypeDelegate& ProjectorTypeDelegate,
	const FWidgetColorDelegate& WidgetColorDelegate,
	const FWidgetTrackingStartedDelegate& WidgetTrackingStartedDelegate) const
{
	SetTransformWidgetVisibility(EVisibility::Visible);
	
	LevelViewportClient->ShowGizmoProjector(WidgetLocationDelegate, OnWidgetLocationChangedDelegate,
		WidgetDirectionDelegate, OnWidgetDirectionChangedDelegate,
		WidgetUpDelegate, OnWidgetUpChangedDelegate,
		WidgetScaleDelegate, OnWidgetScaleChangedDelegate,
		WidgetAngleDelegate,
		ProjectorTypeDelegate,
		WidgetColorDelegate,
		WidgetTrackingStartedDelegate);
}


void SCustomizableObjectEditorViewportTabBody::HideGizmoProjector() const
{
	SetTransformWidgetVisibility(EVisibility::Hidden);
	
	LevelViewportClient->HideGizmoProjector();
}


void SCustomizableObjectEditorViewportTabBody::ShowGizmoLight(ULightComponent& SelectedLight) const
{
	SetTransformWidgetVisibility(EVisibility::Visible);
	
	LevelViewportClient->ShowGizmoLight(SelectedLight);
}


void SCustomizableObjectEditorViewportTabBody::HideGizmoLight() const
{
	SetTransformWidgetVisibility(EVisibility::Hidden);
	
	LevelViewportClient->HideGizmoLight();
}


void SCustomizableObjectEditorViewportTabBody::CreatePreviewActor(const TWeakObjectPtr<UCustomizableObjectInstance>& InInstance)
{
	LevelViewportClient->CreatePreviewActor(InInstance);
	
	LODSelection = 0;
}


bool SCustomizableObjectEditorViewportTabBody::IsVisible() const
{
	return ViewportWidget.IsValid();
}


void SCustomizableObjectEditorViewportTabBody::OnSetPlaybackSpeed(int32 PlaybackSpeedMode)
{
	LevelViewportClient->SetPlaybackSpeedMode(static_cast<EMutableAnimationPlaybackSpeeds::Type>(PlaybackSpeedMode));
}


bool SCustomizableObjectEditorViewportTabBody::IsPlaybackSpeedSelected(int32 PlaybackSpeedMode)
{
	return PlaybackSpeedMode == LevelViewportClient->GetPlaybackSpeedMode();
}


void SCustomizableObjectEditorViewportTabBody::BindCommands()
{
	FUICommandList& CommandList = *UICommandList;

	const FCustomizableObjectEditorViewportCommands& Commands = FCustomizableObjectEditorViewportCommands::Get();

	// Viewport commands
	TSharedRef<FCustomizableObjectEditorViewportClient> EditorViewportClientRef = LevelViewportClient.ToSharedRef();

	CommandList.MapAction(
		Commands.SetCameraLock,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetCameraLock ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsCameraLocked ) );

	CommandList.MapAction(
		Commands.SetDrawUVs,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetDrawUVOverlay ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsSetDrawUVOverlayChecked ) );

	CommandList.MapAction(
		Commands.SetShowGrid,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::UpdateShowGridFromButton),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsShowGridChecked ) );

	CommandList.MapAction(
		Commands.SetShowSky,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::UpdateShowSkyFromButton),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsShowSkyChecked));

	CommandList.MapAction(
		Commands.SetShowBounds,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetShowBounds ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsSetShowBoundsChecked ) );

	CommandList.MapAction(
		Commands.SetShowCollision,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetShowCollision ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsSetShowCollisionChecked ) );

	// Menu
	CommandList.MapAction(
		Commands.BakeInstance,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::BakeInstance),
		FCanExecuteAction(),
		FIsActionChecked());
	
	//Create a menu item for each playback speed in EMutableAnimationPlaybackSpeeds
	for (int32 Index = 0; Index < static_cast<int32>(EMutableAnimationPlaybackSpeeds::NumPlaybackSpeeds); ++Index)
	{
		CommandList.MapAction( 
			Commands.PlaybackSpeedCommands[Index],
			FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::OnSetPlaybackSpeed, Index),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsPlaybackSpeedSelected, Index));
	}
	
	//Bind LOD preview menu commands
	const FCustomizableObjectEditorViewportLODCommands& ViewportLODMenuCommands = FCustomizableObjectEditorViewportLODCommands::Get();

	//LOD Auto
	CommandList.MapAction(
		ViewportLODMenuCommands.LODAuto,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::OnSetLODModel, 0),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsLODModelSelected, 0));

	// LOD 0
	CommandList.MapAction(
		ViewportLODMenuCommands.LOD0,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::OnSetLODModel, 1),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsLODModelSelected, 1));

	CommandList.MapAction(
		ViewportLODMenuCommands.TranslateMode,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::ProjectorCheckboxStateChanged, UE::Widget::WM_Translate),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsProjectorCheckboxState, UE::Widget::WM_Translate));

	CommandList.MapAction(
		ViewportLODMenuCommands.RotateMode,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::ProjectorCheckboxStateChanged, UE::Widget::WM_Rotate),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsProjectorCheckboxState, UE::Widget::WM_Rotate));

	CommandList.MapAction(
		ViewportLODMenuCommands.ScaleMode,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::ProjectorCheckboxStateChanged, UE::Widget::WM_Scale),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsProjectorCheckboxState, UE::Widget::WM_Scale));

	CommandList.MapAction(
		ViewportLODMenuCommands.RotationGridSnap,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::RotationGridSnapClicked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::RotationGridSnapIsChecked)
	);

	CommandList.MapAction(
		ViewportLODMenuCommands.HighResScreenshot,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::OnTakeHighResScreenshot),
		FCanExecuteAction()
	);

	//Orbital Camera Mode
	CommandList.MapAction(
		ViewportLODMenuCommands.OrbitalCamera,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::SetCameraMode, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsCameraModeActive,0)
	);

	//Free Camera Mode
	CommandList.MapAction(
		ViewportLODMenuCommands.FreeCamera,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::SetCameraMode, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsCameraModeActive,1)
	);

	// Bones Mode
	CommandList.MapAction(
		ViewportLODMenuCommands.ShowBones,
		FExecuteAction::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::SetShowBones),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::IsShowingBones)
	);

	const FEditorViewportCommands& ViewportCommands = FEditorViewportCommands::Get();

	// Camera Views
	CommandList.MapAction(
		ViewportCommands.Perspective,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_Perspective),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_Perspective));

	CommandList.MapAction(
		ViewportCommands.Front,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_OrthoNegativeYZ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_OrthoNegativeYZ));

	CommandList.MapAction(
		ViewportCommands.Left,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_OrthoNegativeXZ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_OrthoNegativeXZ));

	CommandList.MapAction(
		ViewportCommands.Top,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_OrthoXY),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_OrthoXY));

	CommandList.MapAction(
		ViewportCommands.Back,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_OrthoYZ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_OrthoYZ));

	CommandList.MapAction(
		ViewportCommands.Right,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_OrthoXZ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_OrthoXZ));

	CommandList.MapAction(
		ViewportCommands.Bottom,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::SetViewportType, LVT_OrthoNegativeXY),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsActiveViewportType, LVT_OrthoNegativeXY));
	CommandList.MapAction( 
		Commands.ShowDisplayInfo,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::OnShowDisplayInfo),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsShowingMeshInfo));

	CommandList.MapAction( 
		Commands.EnableClothSimulation,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::OnEnableClothSimulation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsClothSimulationEnabled));

	CommandList.MapAction( 
		Commands.DebugDrawPhysMeshWired,
		FExecuteAction::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::OnDebugDrawPhysMeshWired),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsDebugDrawPhysMeshWired));
	
	CommandList.MapAction(
		Commands.SetShowNormals,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::ToggleShowNormals),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsSetShowNormalsChecked));

	CommandList.MapAction(
		Commands.SetShowTangents,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::ToggleShowTangents),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsSetShowTangentsChecked)); 

	CommandList.MapAction(
		Commands.SetShowBinormals,
		FExecuteAction::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::ToggleShowBinormals),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( EditorViewportClientRef, &FCustomizableObjectEditorViewportClient::IsSetShowBinormalsChecked));
	
	// all other LODs will be added dynamically 
}


void SCustomizableObjectEditorViewportTabBody::OnTakeHighResScreenshot()
{
	// TODO: Fix for multicomponents
	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Comps = LevelViewportClient->GetPreviewMeshComponents();
	if (!Comps.IsEmpty())
	{
		CustomizableObjectHighresScreenshot = SCustomizableObjectHighresScreenshot::OpenDialog(ViewportWidget->GetSceneViewport(), LevelViewportClient, Comps.FindArbitraryElement()->Value.Get(), PreviewScenePtr);
	}
}


void SCustomizableObjectEditorViewportTabBody::SetCameraMode(bool Value)
{
	LevelViewportClient->SetCameraMode(Value);
}


bool SCustomizableObjectEditorViewportTabBody::IsCameraModeActive(int Value)
{
	bool IsOrbitalCemareaActive = LevelViewportClient->IsOrbitalCameraActive();
	return (Value == 0) ? IsOrbitalCemareaActive : !IsOrbitalCemareaActive;
}


void SCustomizableObjectEditorViewportTabBody::SetDrawDefaultUVMaterial()
{
	GenerateUVSectionOptions();
	GenerateUVChannelOptions();
	
	if (!SelectedUVSection ||
		!SelectedUVChannel)
	{
		LevelViewportClient->SetDrawUV(NAME_None, -1, -1, -1);
	}
	else
	{
		const int32 SectionOptionIndex = UVSectionOptionString.IndexOfByKey(SelectedUVSection);
		check(SectionOptionIndex != INDEX_NONE);
		const FSection& Section = UVSectionOption[SectionOptionIndex];

		const int32 UVIndex = UVChannelOptionString.IndexOfByKey(SelectedUVChannel);
		check(UVIndex != INDEX_NONE);
	
		LevelViewportClient->SetDrawUV(Section.ComponentName, Section.LODIndex, Section.SectionIndex, UVIndex);
	}
}


void SCustomizableObjectEditorViewportTabBody::SetShowBones()
{
	LevelViewportClient->SetShowBones();
}


bool SCustomizableObjectEditorViewportTabBody::IsShowingBones()
{
	return LevelViewportClient->IsShowingBones();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportTabBody::BuildToolBar()
{
	FSlimHorizontalToolBarBuilder CommandToolbarBuilder(UICommandList, FMultiBoxCustomization::None);
	{
		CommandToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportCommands::Get().SetShowGrid);
		CommandToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportCommands::Get().SetShowSky);
	}
	CommandToolbarBuilder.BeginSection("Material UVs");
	{
		CommandToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportCommands::Get().SetDrawUVs);

		CommandToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SCustomizableObjectEditorViewportTabBody::GenerateUVMaterialOptionsMenuContent),
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(),
			true);
	}
	CommandToolbarBuilder.EndSection();

	// Utilities
	CommandToolbarBuilder.BeginSection("Utilities");
	CommandToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportCommands::Get().BakeInstance);
	CommandToolbarBuilder.EndSection();

	return CommandToolbarBuilder.MakeWidget();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportTabBody::GenerateUVMaterialOptionsMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);
	MenuBuilder.BeginSection("ShowUV");
	{
		// Generating an array with all the options of the combobox 
		GenerateUVSectionOptions();
		
		UVSectionOptionCombo = SNew(STextComboBox)
			.OptionsSource(&UVSectionOptionString)
			.InitiallySelectedItem(SelectedUVSection)
			.OnSelectionChanged(this, &SCustomizableObjectEditorViewportTabBody::OnSectionChanged);

		// Generating an array with all the options of the combobox 
		GenerateUVChannelOptions();
		
		UVChannelOptionCombo = SNew(STextComboBox)
			.OptionsSource(&UVChannelOptionString)
			.InitiallySelectedItem(SelectedUVChannel)
			.OnSelectionChanged(this, &SCustomizableObjectEditorViewportTabBody::OnUVChannelChanged);
		
		MenuBuilder.AddWidget(UVSectionOptionCombo.ToSharedRef(), FText::FromString(TEXT("Section")));
		MenuBuilder.AddWidget(UVChannelOptionCombo.ToSharedRef(), FText::FromString(TEXT("UV Channel")));
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportTabBody::ShowStateTestData()
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	MenuBuilder.BeginSection("Objects to include");
	{
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().StateChangeShowData);
		MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().StateChangeShowGeometryData);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


void SCustomizableObjectEditorViewportTabBody::GenerateUVSectionOptions()
{
	ON_SCOPE_EXIT
	{
		if (UVSectionOptionCombo)
		{
			UVSectionOptionCombo->RefreshOptions();
			UVSectionOptionCombo->SetSelectedItem(SelectedUVSection);
		}
	};
		
	UVSectionOptionString.Empty();
	UVSectionOption.Empty();
	
	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Comps = LevelViewportClient->GetPreviewMeshComponents();
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : Comps)
	{
		const TWeakObjectPtr<UDebugSkelMeshComponent>& PreviewSkeletalMeshComponent = Entry.Value;
		
		if (PreviewSkeletalMeshComponent.IsValid() && PreviewSkeletalMeshComponent->GetSkinnedAsset() != nullptr && PreviewSkeletalMeshComponent->GetSkinnedAsset()->GetResourceForRendering() != nullptr)
		{
			const TArray<UMaterialInterface*> Materials = PreviewSkeletalMeshComponent->GetMaterials();
			const FSkeletalMeshRenderData* MeshRes = PreviewSkeletalMeshComponent->GetSkinnedAsset()->GetResourceForRendering();
			for (int32 LODIndex = 0; LODIndex < MeshRes->LODRenderData.Num(); ++LODIndex)
			{
				for (int32 SectionIndex = 0; SectionIndex < MeshRes->LODRenderData[LODIndex].RenderSections.Num(); ++SectionIndex)
				{
					const FSkelMeshRenderSection& Section = MeshRes->LODRenderData[LODIndex].RenderSections[SectionIndex];

					FString BaseMaterialName = FString::Printf(TEXT("Section %i"), SectionIndex);

					if (Materials.IsValidIndex(Section.MaterialIndex))
					{
						if (UMaterialInterface* MaterialInterface = Materials[Section.MaterialIndex])
						{
							if (const UMaterial* BaseMaterial = MaterialInterface->GetBaseMaterial())
							{
								BaseMaterialName += " - " + BaseMaterial->GetName();
							}
						}
					}

					UVSectionOptionString.Add(MakeShared<FString>(BaseMaterialName));

					FSection SectionOption;
					SectionOption.ComponentName = Entry.Key;
					SectionOption.SectionIndex = SectionIndex;
					SectionOption.LODIndex = LODIndex;

					UVSectionOption.Add(SectionOption);
				}
			}
		}
	}

	if (SelectedUVSection)
	{
		if (const TSharedPtr<FString>* Result = UVSectionOptionString.FindByPredicate([this](const TSharedPtr<FString>& Other){ return *SelectedUVSection == *Other; }))
		{
			SelectedUVSection = *Result;
			return;
		}	
	}
	
	SelectedUVSection = UVSectionOptionString.IsEmpty() ? nullptr : UVSectionOptionString[0];
}


void SCustomizableObjectEditorViewportTabBody::OnSectionChanged(TSharedPtr<FString> Selected, ESelectInfo::Type SelectInfo)
{
	SelectedUVSection = Selected;
	
	// We need to update options for the new section
	GenerateUVChannelOptions();

	// Reset the UVChannel selection
	SelectedUVChannel = UVChannelOptionString.IsEmpty() ? nullptr : UVChannelOptionString[0];
	UVChannelOptionCombo->SetSelectedItem(SelectedUVChannel);
	
	if (!LevelViewportClient)
	{
		return;
	}

	if (SelectedUVSection)
	{
		const int32 SectionOptionIndex = UVSectionOptionString.IndexOfByKey(SelectedUVSection);
		check(SectionOptionIndex != INDEX_NONE);
		const FSection& Section = UVSectionOption[SectionOptionIndex];

		const int32 UVIndex = UVChannelOptionString.IndexOfByKey(SelectedUVChannel);
		check(UVIndex != INDEX_NONE);
	
		LevelViewportClient->SetDrawUV(Section.ComponentName, Section.LODIndex, Section.SectionIndex, UVIndex);
	}
	else
	{
		LevelViewportClient->SetDrawUV(NAME_None, -1, -1, -1);
	}
}


void SCustomizableObjectEditorViewportTabBody::GenerateUVChannelOptions()
{
	ON_SCOPE_EXIT
	{
		if (UVChannelOptionCombo)
		{
			UVChannelOptionCombo->RefreshOptions();
			UVChannelOptionCombo->SetSelectedItem(SelectedUVChannel);
		}
	};
	
	UVChannelOptionString.Empty();
	
	if (!SelectedUVSection)
	{
		SelectedUVChannel = nullptr;
		return;
	}
	
	const int32 Index = UVSectionOptionString.IndexOfByKey(SelectedUVSection);
	check(Index != INDEX_NONE);
	const FSection& Section = UVSectionOption[Index];

	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Comps = LevelViewportClient->GetPreviewMeshComponents();
	const TWeakObjectPtr<UDebugSkelMeshComponent>* PreviewSkeletalMeshComponent = Comps.Find(Section.ComponentName);
	
	if (PreviewSkeletalMeshComponent && PreviewSkeletalMeshComponent->IsValid() && (*PreviewSkeletalMeshComponent)->GetSkinnedAsset() != nullptr
		&& (*PreviewSkeletalMeshComponent)->GetSkinnedAsset()->GetResourceForRendering() != nullptr)
	{
		const FSkeletalMeshRenderData* MeshRes = (*PreviewSkeletalMeshComponent)->GetSkinnedAsset()->GetResourceForRendering();
		
		const int32 UVChannels = MeshRes->LODRenderData[Section.LODIndex].GetNumTexCoords();
		for (int32 UVChan = 0; UVChan < UVChannels; ++UVChan)
		{
			UVChannelOptionString.Add(MakeShareable(new FString(FString::FromInt(UVChan))));
		}
	}

	if (SelectedUVChannel)
	{
		if (const TSharedPtr<FString>* Result = UVChannelOptionString.FindByPredicate([this](const TSharedPtr<FString>& Other) { return *SelectedUVChannel == *Other; }))
		{
			SelectedUVChannel = *Result;
			return;
		}
	}

	SelectedUVChannel = UVChannelOptionString.IsEmpty() ? nullptr : UVChannelOptionString[0];
}


void SCustomizableObjectEditorViewportTabBody::OnUVChannelChanged(TSharedPtr<FString> Selected, ESelectInfo::Type SelectInfo)
{
	SelectedUVChannel = Selected;

	if (!LevelViewportClient)
	{
		return;
	}

	if (SelectedUVChannel)
	{
		const int32 SectionOptionIndex = UVSectionOptionString.IndexOfByKey(SelectedUVSection);
		check(SectionOptionIndex != INDEX_NONE);
		const FSection& Section = UVSectionOption[SectionOptionIndex];

		const int32 UVIndex = UVChannelOptionString.IndexOfByKey(SelectedUVChannel);
		check(UVIndex != INDEX_NONE);
	
		LevelViewportClient->SetDrawUV(Section.ComponentName, Section.LODIndex, Section.SectionIndex, UVIndex);
	}
	else
	{
		LevelViewportClient->SetDrawUV(NAME_None, -1, -1, -1);
	}
}


FReply SCustomizableObjectEditorViewportTabBody::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (FAssetDragDropOp* DragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>().Get())
	{
		if (DragDropOp->GetAssets().Num())
		{
			// This cast also includes UPoseAsset assets.
			UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(UE::Mutable::Private::LoadObject(DragDropOp->GetAssets()[0]));
			if (AnimationAsset)
			{
				FObjectEditorUtils::SetPropertyValue(WeakEditor.Pin()->GetCustomSettings(), GET_MEMBER_NAME_CHECKED(UCustomSettings, Animation), AnimationAsset);

				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}


int32 SCustomizableObjectEditorViewportTabBody::GetLODModelCount() const
{
	int32 LODModelCount = 0;

	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Comps = LevelViewportClient->GetPreviewMeshComponents();
	for (const TPair<FName,TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : Comps)
	{
		if (Entry.Value.IsValid() && Entry.Value->GetSkinnedAsset())
		{
			const TIndirectArray<FSkeletalMeshLODRenderData>& LODModels = Entry.Value->GetSkinnedAsset()->GetResourceForRendering()->LODRenderData;
			LODModelCount = FMath::Max(LODModelCount, LODModels.Num());
		}
	}

	return LODModelCount;
}

bool SCustomizableObjectEditorViewportTabBody::IsLODModelSelected(int32 LODSelectionType) const
{
	return LODSelection == LODSelectionType;
}


void SCustomizableObjectEditorViewportTabBody::ProjectorCheckboxStateChanged(const UE::Widget::EWidgetMode Mode)
{
	GetViewportClient()->SetWidgetMode(Mode);
}


bool SCustomizableObjectEditorViewportTabBody::IsProjectorCheckboxState(const UE::Widget::EWidgetMode Mode) const
{
	return GetViewportClient()->GetWidgetMode() == Mode;	
}


void SCustomizableObjectEditorViewportTabBody::RotationGridSnapClicked()
{
	GUnrealEd->Exec(GEditor->GetEditorWorldContext().World(), *FString::Printf(TEXT("MODE ROTGRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? 1 : 0));
}


bool SCustomizableObjectEditorViewportTabBody::RotationGridSnapIsChecked() const
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled;
}


void SCustomizableObjectEditorViewportTabBody::SetRotationGridSize(int32 InIndex, ERotationGridMode InGridMode)
{
	GEditor->SetRotGridSize(InIndex, InGridMode);
}


bool SCustomizableObjectEditorViewportTabBody::IsRotationGridSizeChecked(int32 GridSizeIndex, ERotationGridMode GridMode)
{
	const ULevelEditorViewportSettings* ViewportSettings = GetDefault<ULevelEditorViewportSettings>();
	return (ViewportSettings->CurrentRotGridSize == GridSizeIndex) && (ViewportSettings->CurrentRotGridMode == GridMode);
}


void SCustomizableObjectEditorViewportTabBody::OnSetLODModel(int32 LODSelectionType)
{
	LODSelection = LODSelectionType;

	TMap<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Comps = LevelViewportClient->GetPreviewMeshComponents();
	for (const TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : Comps)
	{
		if (Entry.Value.IsValid())
		{
			Entry.Value.Get()->SetForcedLOD(LODSelection);
		}
	}
}


TSharedPtr<FCustomizableObjectPreviewScene> SCustomizableObjectEditorViewportTabBody::GetPreviewScene() const
{
	return PreviewScenePtr;
}


TSharedPtr<FCustomizableObjectEditorViewportClient> SCustomizableObjectEditorViewportTabBody::GetViewportClient() const
{
	return LevelViewportClient;
}


void SCustomizableObjectEditorViewportTabBody::SetViewportToolbarTransformWidget(TWeakPtr<class SWidget> InTransformWidget)
{
	ViewportToolbarTransformWidget = InTransformWidget;
}


void SCustomizableObjectEditorViewportTabBody::SetCustomizableObject(UCustomizableObject* CustomizableObjectParameter)
{
	LevelViewportClient->SetCustomizableObject(CustomizableObjectParameter);
}


#undef LOCTEXT_NAMESPACE