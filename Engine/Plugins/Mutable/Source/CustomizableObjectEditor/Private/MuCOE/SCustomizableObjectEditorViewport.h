// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "UObject/GCObject.h"

class SMutableScrubPanel;
class FCustomizableObjectPreviewScene;
class SCustomizableObjectEditorViewportTabBody;
enum ERotationGridMode : int;
enum EViewModeIndex : int;
namespace EAnimationMode { enum Type : int; }
namespace ESelectInfo { enum Type : int; }

class FDragDropEvent;
class FEditorViewportClient;
class FReferenceCollector;
class FSceneViewport;
class FUICommandList;
class STextComboBox;
class SWidget;
class UCustomizableObject;
class UCustomizableObjectNode;
class UCustomizableObjectNodeProjectorConstant;
class UCustomizableObjectNodeProjectorParameter;
class UDebugSkelMeshComponent;
class UStaticMesh;
class SCustomizableObjectEditorToolBar;
class SCustomizableObjectViewportToolBar;
class FCustomizableObjectEditorViewportClient;
struct FCustomizableObjectProjector;
struct FGeometry;
enum class ECustomizableObjectProjectorType : uint8;


//////////////////////////////////////////////////////////////////////////
// SCustomizableObjectEditorViewport: the pure viewport widget

class SCustomizableObjectEditorViewport : public SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectEditorViewport)	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs,
		const TWeakPtr<FCustomizableObjectPreviewScene>& PreviewScene,
		const TWeakPtr<SCustomizableObjectEditorViewportTabBody>& TabBody,
		const TWeakPtr<ICustomizableObjectInstanceEditor>& Editor);

	// SEditorViewport interface
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	// End of SEditorViewport interface

	TSharedPtr<FSceneViewport>& GetSceneViewport();

private:
	/**  Handle undo/redo by refreshing the viewport */
	void OnUndoRedo();

	EVisibility GetShowCompileErrorOverlay() const;

	FText GetCompileErrorOverlayText() const;

	FText GetWarningText() const;

	EVisibility GetWarningTextVisibility() const;

	FText GetMeshInfoText() const;

	EVisibility GetMeshInfoTextVisibility() const;
	
	// Viewport client
	TSharedPtr<FCustomizableObjectEditorViewportClient> LevelViewportClient;

	// Pointer to the compound widget that owns this viewport widget
	TWeakPtr<SCustomizableObjectEditorViewportTabBody> TabBodyPtr;

	// The preview scene that we are viewing
	TWeakPtr<FCustomizableObjectPreviewScene> PreviewScenePtr;

	TWeakPtr<SCustomizableObjectViewportToolBar> ToolbarPtr;

	/** Weak pointer of the open editor */
	TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor;
};


/**
 * CustomizableObject Editor Preview viewport widget with toolbars, etc.
 */
class SCustomizableObjectEditorViewportTabBody : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectEditorViewportTabBody){}
		SLATE_ARGUMENT(TWeakPtr<class ICustomizableObjectInstanceEditor>, CustomizableObjectEditor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	
	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

	/**
	* @return The list of commands on the viewport that are bound to delegates
	*/
	const TSharedPtr<FUICommandList>& GetCommandList() const { return UICommandList; }

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoClipMorph(UCustomizableObjectNodeModifierClipMorph& ClipPlainNode) const;

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoClipMorph() const;

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoClipMesh(UCustomizableObjectNode& Node, FTransform* Transform, UObject& ClipMesh, int32 LODIndex, int32 SectionIndex, int32 MaterialSlotIndex) const;

	void SetTransformWidgetVisibility(const EVisibility NewVisibilityState) const;
	
	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoClipMesh() const;

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoProjector(
		const FWidgetLocationDelegate& WidgetLocationDelegate, const FOnWidgetLocationChangedDelegate& OnWidgetLocationChangedDelegate,
		const FWidgetDirectionDelegate& WidgetDirectionDelegate, const FOnWidgetDirectionChangedDelegate& OnWidgetDirectionChangedDelegate,
		const FWidgetUpDelegate& WidgetUpDelegate, const FOnWidgetUpChangedDelegate& OnWidgetUpChangedDelegate,
		const FWidgetScaleDelegate& WidgetScaleDelegate, const FOnWidgetScaleChangedDelegate& OnWidgetScaleChangedDelegate,
		const FWidgetAngleDelegate& WidgetAngleDelegate,
	    const FProjectorTypeDelegate& ProjectorTypeDelegate,
	    const FWidgetColorDelegate& WidgetColorDelegate,
	    const FWidgetTrackingStartedDelegate& WidgetTrackingStartedDelegate) const;

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoProjector() const;

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void ShowGizmoLight(ULightComponent& SelectedLight) const;

	/** Do not call directly. Use ICustomizableObjectEditor functions instead. */
	void HideGizmoLight() const;
	
	/** Binds commands associated with the viewport client. */
	void BindCommands();
	TSharedRef<SWidget> BuildToolBar();

	TSharedRef<SWidget> GenerateUVMaterialOptionsMenuContent();

	/** Callback to show / hide the state and runtime parameter test in the viewport */
	TSharedRef<SWidget> ShowStateTestData();
	
	void CreatePreviewActor(const TWeakObjectPtr<UCustomizableObjectInstance>& InInstance);
	
	/** Function to get the number of LOD models associated with the preview skeletal mesh*/
	int32 GetLODModelCount() const;

	/** LOD model selection checking function*/
	bool IsLODModelSelected(int32 LODSelectionType) const;
	int32 GetLODSelection() const { return LODSelection; };

	/** Callback for when the projector checkbox changes */
	void ProjectorCheckboxStateChanged(UE::Widget::EWidgetMode Mode);

	bool IsProjectorCheckboxState(UE::Widget::EWidgetMode Mode) const;
	
	/** Called when the rotation grid snap is toggled off and on */
	void RotationGridSnapClicked();

	/** Returns whether or not rotation grid snap is enabled */
	bool RotationGridSnapIsChecked() const;

	/** Sets the rotation grid size */
	static void SetRotationGridSize(int32 InIndex, ERotationGridMode InGridMode);

	/** Checks to see if the specified rotation grid angle is the current rotation grid angle */
	static bool IsRotationGridSizeChecked(int32 GridSizeIndex, ERotationGridMode GridMode);

	/** Function to set LOD model selection*/
	void OnSetLODModel(int32 LODSelectionType);

	// Getter of PreviewScenePtr
	TSharedPtr<FCustomizableObjectPreviewScene> GetPreviewScene() const;

	TSharedPtr<FCustomizableObjectEditorViewportClient> GetViewportClient() const;

	// Pointer back to the editor tool that owns us.
	TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor;

	//void SetEditorTransformViewportToolbar(TWeakPtr<class SCustomizableObjectEditorTransformViewportToolbar> EditorTransformViewportToolbarParam);
	void SetViewportToolbarTransformWidget(TWeakPtr<class SWidget> TransformWidget);

	/** Setter of CustomizableObject */
	void SetCustomizableObject(UCustomizableObject* CustomizableObjectParameter);

	/** Called to bring up the screenshot UI */
	void OnTakeHighResScreenshot();

	/** Sets camera mode of the LevelViewportClient::bActivateOrbitalCamera */
	void SetCameraMode(bool Value);

	/** Returns whether or not the a camera mode is active (Orbital or Free) */
	bool IsCameraModeActive(int Value);

	/** Sets the first material of the instance as the default section to draw in the UVs Overlay (when there is no material selected in the combobox)
		Param bIsCompilation: true if this function is called during a CO compilation, false when is called from an instance update	*/
	void SetDrawDefaultUVMaterial();

	/** Sets the bones visibility */
	void SetShowBones();

	/** Returns true if the skeletal mesh bones are visible in the viewport */
	bool IsShowingBones();

private:
	/** Determines the visibility of the viewport. */
	bool IsVisible() const;

	void OnSetPlaybackSpeed(int32 PlaybackSpeedMode);
	
	bool IsPlaybackSpeedSelected(int32 PlaybackSpeedMode);

	// The scene for this viewport.
	TSharedPtr<FCustomizableObjectPreviewScene> PreviewScenePtr;

	// Level viewport client
	TSharedPtr<FCustomizableObjectEditorViewportClient> LevelViewportClient;

	// Viewport widget
	TSharedPtr<SCustomizableObjectEditorViewport> ViewportWidget;
	TSharedPtr<SCustomizableObjectEditorToolBar> ViewportToolbar;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;

	// The currently selected view mode.
	EViewModeIndex CurrentViewMode = VMI_Lit;

	/** Section identifier. */
	struct FSection
	{
		FName ComponentName = NAME_None;
		int32 LODIndex = -1;
		int32 SectionIndex = -1;
	};
	
	// Selection for the section to filter when showing UVs
	TArray<TSharedPtr<FString>> UVSectionOptionString;
	TArray<FSection> UVSectionOption;

	// Combo box for UV section selection
	TSharedPtr<STextComboBox> UVSectionOptionCombo;

	// Selection for the channel to filter when showing UVs
	TArray<TSharedPtr<FString>> UVChannelOptionString;

	// Combo box for UV channel selection
	TSharedPtr<STextComboBox> UVChannelOptionCombo;

	TSharedPtr<SMutableScrubPanel> ScrubPanel;

	// Generates the Combobox options for each UV Channel
	void GenerateUVSectionOptions();

	// Generates the Combobox options for each UV Channel
	void GenerateUVChannelOptions();

	// Selected option of the UV Material ComboBox
	TSharedPtr<FString> SelectedUVSection;
	
	// Selected option of the UV Channel ComboBox
	TSharedPtr<FString> SelectedUVChannel;

	// Options for the above combo.
	TArray< TSharedPtr< FString > > MaterialNames;

	// Slate callbacks to manage the material combo
	void OnSectionChanged(TSharedPtr<FString> Selected, ESelectInfo::Type SelectInfo);
	void OnUVChannelChanged(TSharedPtr<FString> Selected, ESelectInfo::Type SelectInfo);

	/** Current LOD selection*/
	int32 LODSelection = 0;

	/** To be able to show and hide the constant projector RTS controls */
	TWeakPtr<SWidget> ViewportToolbarTransformWidget;

	/** Flag to know when the asset registry initial loading has completed, value set by the Customizable Object / Customizable Object Instance editor */
	bool AssetRegistryLoaded = false;

	/** Weak pointer to the high resolution screenshot dialog if it's open. Will become invalid if UI is closed whilst the viewport is open */
	TWeakPtr<SWindow> CustomizableObjectHighresScreenshot;
};
