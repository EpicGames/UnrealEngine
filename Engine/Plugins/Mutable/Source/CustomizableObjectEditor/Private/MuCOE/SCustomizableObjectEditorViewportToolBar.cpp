// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectEditorViewportToolBar.h"

#include "AssetViewerSettings.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorViewportCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "MuCOE/CustomizableObjectEditorViewportLODCommands.h"
#include "MuCOE/CustomizableObjectEditorViewportMenuCommands.h"
#include "MuCOE/ICustomizableObjectInstanceEditor.h"
#include "MuCOE/SCustomizableObjectEditorViewport.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Preferences/PersonaOptions.h"
#include "SEditorViewportViewMenu.h"
#include "SViewportToolBarComboMenu.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "UnrealEdGlobals.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCOE/CustomizableObjectEditorActions.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SSlider.h"
#include "MuCOE/CustomizableObjectEditorActions.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditorViewportToolBar"


void SCustomizableObjectEditorViewportToolBar::Construct(const FArguments& InArgs, TSharedPtr<SCustomizableObjectEditorViewportTabBody> InViewport, TSharedPtr<SEditorViewport> InRealViewport)
{
	Viewport = InViewport;

	TSharedRef<SCustomizableObjectEditorViewportTabBody> ViewportRef = Viewport.Pin().ToSharedRef();

	WeakEditor = ViewportRef->WeakEditor;
	
	TSharedRef<SHorizontalBox> LeftToolbar = SNew(SHorizontalBox)

	//// Camera Type (Perspective/Top/etc...)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.0f, 2.0f)
	[
		SNew(SEditorViewportToolbarMenu)
		.ParentToolBar(SharedThis(this))
		.Label(this, &SCustomizableObjectEditorViewportToolBar::GetCameraMenuLabel)
		.LabelIcon(this, &SCustomizableObjectEditorViewportToolBar::GetCameraMenuLabelIcon)
		.OnGetMenuContent(this, &SCustomizableObjectEditorViewportToolBar::GenerateViewportTypeMenu)
	]

	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 2.0f))
		[
			SNew( SEditorViewportToolbarMenu )
			.ParentToolBar( SharedThis( this ) )
			.Cursor( EMouseCursor::Default )
			.Image( "EditorViewportToolBar.MenuDropdown" )
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EditorViewportToolBar.MenuDropdown")))
			.OnGetMenuContent( this, &SCustomizableObjectEditorViewportToolBar::GenerateOptionsMenu )
		]

	// View menu (lit, unlit, etc...)
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			SNew(SEditorViewportViewMenu, InRealViewport.ToSharedRef(), SharedThis(this))
		]

	// LOD menu
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			//LOD
			SNew(SEditorViewportToolbarMenu)
				.ParentToolBar(SharedThis(this))
				.Label(this, &SCustomizableObjectEditorViewportToolBar::GetLODMenuLabel)
				.OnGetMenuContent(this, &SCustomizableObjectEditorViewportToolBar::GenerateLODMenu)
		]

	// View Options Menu (Camera options, Bones...)
	+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			//Show Bones
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Label(LOCTEXT("ViewOptionsMenuLabel","View Options"))
			.OnGetMenuContent(this, &SCustomizableObjectEditorViewportToolBar::GenerateViewportOptionsMenu)
		]
	
	// Character Menu
	+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			//Show Bones
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Label(LOCTEXT("CharacterMenuLabel","Character"))
			.OnGetMenuContent(this, &SCustomizableObjectEditorViewportToolBar::GenerateCharacterMenu)
		]

	// Playback Menu
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			SNew(SEditorViewportToolbarMenu)
			.ToolTipText(LOCTEXT("PlaybackSpeedMenuTooltip", "Playback Speed Options. Control the time dilation of the scene's update."))
			.ParentToolBar(SharedThis(this))
			.Label(this, &SCustomizableObjectEditorViewportToolBar::GetPlaybackMenuLabel)
			.LabelIcon(FAppStyle::GetBrush("AnimViewportMenu.PlayBackSpeed"))
			.OnGetMenuContent(this, &SCustomizableObjectEditorViewportToolBar::GeneratePlaybackMenu)
		];
	
	TSharedRef<SWidget> RTSButtons = GenerateRTSButtons();
	ViewportRef->SetViewportToolbarTransformWidget(RTSButtons);

	LeftToolbar->AddSlot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			RTSButtons
		];

	static const FName DefaultForegroundName("DefaultForeground");

	FLinearColor ButtonColor1 = FLinearColor(0.1f, 0.1f, 0.1f, 1.0f);
	FLinearColor ButtonColor2 = FLinearColor(0.2f, 0.2f, 0.2f, 0.75f);
	FLinearColor TextColor1 = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	FLinearColor TextColor2 = FLinearColor(0.8f, 0.8f, 0.8f, 0.8f);
	FSlateFontInfo Info = UE_MUTABLE_GET_FONTSTYLE("BoldFont");
	Info.Size += 26;

	ChildSlot
		[
		SNew(SBorder)
			.BorderImage(UE_MUTABLE_GET_BRUSH("NoBorder"))
			.ForegroundColor(UE_MUTABLE_GET_SLATECOLOR(DefaultForegroundName))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					[
						LeftToolbar
					]
				]
			]
		];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateViewMenu() const
{
	const FCustomizableObjectEditorViewportMenuCommands& Actions = FCustomizableObjectEditorViewportMenuCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ViewMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList());

	return ViewMenuBuilder.MakeWidget();
}


FText SCustomizableObjectEditorViewportToolBar::GetLODMenuLabel() const
{
	FText Label = LOCTEXT("LODMenu_AutoLabel", "LOD Auto");
	if (Viewport.IsValid())
	{
		int32 LODSelectionType = Viewport.Pin()->GetLODSelection();

		if (LODSelectionType > 0)
		{
			FString TitleLabel = FString::Printf(TEXT("LOD %d"), LODSelectionType - 1);
			Label = FText::FromString(TitleLabel);
		}
	}
	return Label;
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateRTSButtons()
{
	const FCustomizableObjectEditorViewportLODCommands& Actions = FCustomizableObjectEditorViewportLODCommands::Get();

	TSharedPtr< FExtender > InExtenders;
	FToolBarBuilder ToolbarBuilder(Viewport.Pin()->GetCommandList(), FMultiBoxCustomization::None, InExtenders);

	// Use a custom style
	FName ToolBarStyle = "EditorViewportToolBar";
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	// Transform controls cannot be focusable as it fights with the press space to change transform mode feature
	ToolbarBuilder.SetIsFocusable(false);

	ToolbarBuilder.BeginSection("Transform");
	ToolbarBuilder.BeginBlockGroup();
	{
		// Move Mode
		static FName TranslateModeName = FName(TEXT("TranslateMode"));
		ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportLODCommands::Get().TranslateMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), TranslateModeName);

		// Rotate Mode
		static FName RotateModeName = FName(TEXT("RotateMode"));
		ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportLODCommands::Get().RotateMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), RotateModeName);

		// Scale Mode
		static FName ScaleModeName = FName(TEXT("ScaleMode"));
		ToolbarBuilder.AddToolBarButton(FCustomizableObjectEditorViewportLODCommands::Get().ScaleMode, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>(), ScaleModeName);

	}
	ToolbarBuilder.EndBlockGroup();
	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("RotationGridSnap");
	{
		// Grab the existing UICommand 
		FUICommandInfo* Command = FCustomizableObjectEditorViewportLODCommands::Get().RotationGridSnap.Get();

		static FName RotationSnapName = FName(TEXT("RotationSnap"));

		// Setup a GridSnapSetting with the UICommand
		ToolbarBuilder.AddWidget(SNew(SViewportToolBarComboMenu)
			.Cursor(EMouseCursor::Default)
			.IsChecked(this, &SCustomizableObjectEditorViewportToolBar::IsRotationGridSnapChecked)
			.OnCheckStateChanged(this, &SCustomizableObjectEditorViewportToolBar::HandleToggleRotationGridSnap)
			.Label(this, &SCustomizableObjectEditorViewportToolBar::GetRotationGridLabel)
			.OnGetMenuContent(this, &SCustomizableObjectEditorViewportToolBar::FillRotationGridSnapMenu)
			.ToggleButtonToolTip(Command->GetDescription())
			.MenuButtonToolTip(LOCTEXT("RotationGridSnap_ToolTip", "Set the Rotation Grid Snap value"))
			.Icon(Command->GetIcon())
			.ParentToolBar(SharedThis(this))
			, RotationSnapName);
	}

	ToolbarBuilder.EndSection();

	ToolbarBuilder.SetIsFocusable(true);

	return ToolbarBuilder.MakeWidget();
}

TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateLODMenu() const
{
	const FCustomizableObjectEditorViewportLODCommands& Actions = FCustomizableObjectEditorViewportLODCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList());
	{
		// LOD Models
		ShowMenuBuilder.BeginSection("AnimViewportPreviewLODs", LOCTEXT("ShowLOD_PreviewLabel", "Preview LODs"));
		{
			ShowMenuBuilder.AddMenuEntry(Actions.LODAuto);
			ShowMenuBuilder.AddMenuEntry(Actions.LOD0);

			int32 LODCount = Viewport.Pin()->GetLODModelCount();
			for (int32 LODId = 1; LODId < LODCount; ++LODId)
			{
				FString TitleLabel = FString::Printf(TEXT("LOD %d"), LODId);

				FUIAction Action(FExecuteAction::CreateSP(Viewport.Pin().ToSharedRef(), &SCustomizableObjectEditorViewportTabBody::OnSetLODModel, LODId + 1),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(Viewport.Pin().ToSharedRef(), &SCustomizableObjectEditorViewportTabBody::IsLODModelSelected, LODId + 1));

				ShowMenuBuilder.AddMenuEntry(FText::FromString(TitleLabel), FText::GetEmpty(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::RadioButton);
			}
		}
		ShowMenuBuilder.EndSection();
	}

	return ShowMenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateCharacterMenu() const
{
	FMenuBuilder MenuBuilder(true, Viewport.Pin()->GetCommandList());
	
	MenuBuilder.BeginSection("Mesh", LOCTEXT("Mesh", "Mesh"));
	MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().ShowDisplayInfo);

	// Uncomment once UE-217529 fixed.
	// MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().SetShowNormals);
	// MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().SetShowTangents);
	// MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().SetShowBinormals);

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Bones", LOCTEXT("Bones", "Bones"));
	MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportLODCommands::Get().ShowBones);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Clothing", LOCTEXT("Clothing", "Clothing"));
	MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().EnableClothSimulation);
	MenuBuilder.AddMenuEntry(FCustomizableObjectEditorViewportCommands::Get().DebugDrawPhysMeshWired);
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateViewportTypeMenu() const
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder CameraMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList());

	// Camera types
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Perspective);

	CameraMenuBuilder.BeginSection("LevelViewportCameraType_Ortho", LOCTEXT("CameraTypeHeader_Ortho", "Orthographic"));
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Top);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Bottom);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Left);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Right);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Front);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Back);
	CameraMenuBuilder.EndSection();

	return CameraMenuBuilder.MakeWidget();
}


class SCusttomAnimationSpeedSetting : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnCustomSpeedChanged, float);

	SLATE_BEGIN_ARGS(SCusttomAnimationSpeedSetting) {}
		SLATE_ATTRIBUTE(float, CustomSpeed)
		SLATE_EVENT(FOnCustomSpeedChanged, OnCustomSpeedChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs )
	{
		CustomSpeed = InArgs._CustomSpeed;
		OnCustomSpeedChanged = InArgs._OnCustomSpeedChanged;

		this->ChildSlot
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SSpinBox<float>)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.ToolTipText(LOCTEXT("AnimationCustomSpeed", "Set Custom Speed."))
					.MinValue(0.f)
					.MaxSliderValue(10.f)
					.SupportDynamicSliderMaxValue(true)
					.Value(CustomSpeed)
					.OnValueChanged(OnCustomSpeedChanged)
				]
			]
		];
	}

private:
    TAttribute<float> CustomSpeed = 1.0f;
	FOnCustomSpeedChanged OnCustomSpeedChanged;
};


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GeneratePlaybackMenu() const
{
	const FCustomizableObjectEditorViewportCommands& Actions = FCustomizableObjectEditorViewportCommands::Get();
	
	FMenuBuilder InMenuBuilder(true, Viewport.Pin()->GetCommandList());

	InMenuBuilder.BeginSection("AnimViewportPlaybackSpeed", LOCTEXT("PlaybackMenu_SpeedLabel", "Playback Speed") );

	for(int32 PlaybackSpeedIndex = 0; PlaybackSpeedIndex < EMutableAnimationPlaybackSpeeds::NumPlaybackSpeeds; ++PlaybackSpeedIndex)
	{
		InMenuBuilder.AddMenuEntry( Actions.PlaybackSpeedCommands[PlaybackSpeedIndex] );
	}
	
	TSharedPtr<SWidget> AnimSpeedWidget = SNew(SCusttomAnimationSpeedSetting)
		.CustomSpeed_Lambda([Viewport = Viewport]()
			{
				return Viewport.Pin()->GetViewportClient()->GetCustomAnimationSpeed();
			})
		.OnCustomSpeedChanged_Lambda([Viewport = Viewport](float CustomSpeed)
			{
				return Viewport.Pin()->GetViewportClient()->SetCustomAnimationSpeed(CustomSpeed);
			});
	
	InMenuBuilder.AddWidget(AnimSpeedWidget.ToSharedRef(), LOCTEXT("PlaybackMenu_Speed_Custom", "Custom Speed:"));
	InMenuBuilder.EndSection();

	return InMenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateViewportOptionsMenu() const
{
	const FCustomizableObjectEditorViewportLODCommands& Actions = FCustomizableObjectEditorViewportLODCommands::Get();

	const bool bInShouldCloseWindowAfterMenuSelection = true;

	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList());
	ShowMenuBuilder.BeginSection("Camera", LOCTEXT("Camera", "Camera"));

	ShowMenuBuilder.AddMenuEntry(Actions.OrbitalCamera);
	ShowMenuBuilder.AddMenuEntry(Actions.FreeCamera);
	
	TSharedPtr<SWidget> BoneSizeWidget = SNew(SVerticalBox)
	+ SVerticalBox::Slot().AutoHeight()
	.HAlign(HAlign_Left)
	.Padding(FMargin(20.0f, 5.0f, 0.0f, 0.0f))
	[
		SNew(STextBlock)
		.Text(LOCTEXT("OptionsMenu_CameraOptions_CameraSpeed_Text", "Camera Speed"))
		.Font(UE_MUTABLE_GET_FONTSTYLE(TEXT("MenuItem.Font")))
	]

	+SVerticalBox::Slot().AutoHeight()
	.HAlign(HAlign_Left)
	.Padding(FMargin(20.0f, 0.0f, 0.0f, 0.0f))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot().AutoWidth()
		[
			SNew(SBox).MinDesiredWidth(100.0f)
			[
				UE::UnrealEd::CreateCameraSpeedWidget(Viewport.Pin()->GetViewportClient())
			]
		]
		+ SHorizontalBox::Slot().AutoWidth()
		[
			SNew(STextBlock).Text_Lambda([this]() { return UE::UnrealEd::GetCameraSpeedLabel(Viewport.Pin()->GetViewportClient()); })
		]
	];
	
	ShowMenuBuilder.AddWidget(BoneSizeWidget.ToSharedRef(), FText());
	
	ShowMenuBuilder.EndSection();

	return ShowMenuBuilder.MakeWidget();
}


FText SCustomizableObjectEditorViewportToolBar::GetPlaybackMenuLabel() const
{
	FText Label = LOCTEXT("PlaybackError", "Error");
	if (Viewport.IsValid())
	{
		for(int Index = 0; Index < EMutableAnimationPlaybackSpeeds::NumPlaybackSpeeds; ++Index)
		{
			if (Viewport.Pin()->GetViewportClient()->GetPlaybackSpeedMode() == Index)
			{
				const int32 NumFractionalDigits = (Index == EMutableAnimationPlaybackSpeeds::Quarter || Index == EMutableAnimationPlaybackSpeeds::ThreeQuarters) ? 2 : 1;

				const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
					.SetMinimumFractionalDigits(NumFractionalDigits)
					.SetMaximumFractionalDigits(NumFractionalDigits);

				Label = FText::Format(LOCTEXT("AnimViewportPlaybackMenuLabel", "x{0}"), FText::AsNumber(EMutableAnimationPlaybackSpeeds::Values[Index], &FormatOptions));
			}
		}
	}
	
	return Label;
}


FText SCustomizableObjectEditorViewportToolBar::GetCameraMenuLabel() const
{
	FText Label = LOCTEXT("Viewport_Default", "Camera");
	TSharedPtr< SCustomizableObjectEditorViewportTabBody > PinnedViewport(Viewport.Pin());
	if (PinnedViewport.IsValid())
	{
		switch (PinnedViewport->GetViewportClient()->ViewportType)
		{
		case LVT_Perspective:
			Label = LOCTEXT("CameraMenuTitle_Perspective", "Perspective");
			break;

		case LVT_OrthoXY:
			Label = LOCTEXT("CameraMenuTitle_Top", "Top");
			break;

		case LVT_OrthoNegativeXZ:
			Label = LOCTEXT("CameraMenuTitle_Left", "Left");
			break;

		case LVT_OrthoNegativeYZ:
			Label = LOCTEXT("CameraMenuTitle_Front", "Front");
			break;

		case LVT_OrthoNegativeXY:
			Label = LOCTEXT("CameraMenuTitle_Bottom", "Bottom");
			break;

		case LVT_OrthoXZ:
			Label = LOCTEXT("CameraMenuTitle_Right", "Right");
			break;

		case LVT_OrthoYZ:
			Label = LOCTEXT("CameraMenuTitle_Back", "Back");
			break;
		case LVT_OrthoFreelook:
			break;
		}
	}

	return Label;
}

const FSlateBrush* SCustomizableObjectEditorViewportToolBar::GetCameraMenuLabelIcon() const
{
	FName Icon = NAME_None;
	TSharedPtr< SCustomizableObjectEditorViewportTabBody > PinnedViewport(Viewport.Pin());
	if (PinnedViewport.IsValid())
	{
		switch (PinnedViewport->GetViewportClient()->ViewportType)
		{
		case LVT_Perspective:
			Icon = FName("EditorViewport.Perspective");
			break;

		case LVT_OrthoXY:
			Icon = FName("EditorViewport.Top");
			break;

		case LVT_OrthoYZ:
			Icon = FName("EditorViewport.Back");
			break;

		case LVT_OrthoXZ:
			Icon = FName("EditorViewport.Right");
			break;

		case LVT_OrthoNegativeXY:
			Icon = FName("EditorViewport.Bottom");
			break;

		case LVT_OrthoNegativeYZ:
			Icon = FName("EditorViewport.Front");
			break;

		case LVT_OrthoNegativeXZ:
			Icon = FName("EditorViewport.Left");
			break;
		case LVT_OrthoFreelook:
			break;
		}
	}

	return UE_MUTABLE_GET_BRUSH(Icon);
}

float SCustomizableObjectEditorViewportToolBar::OnGetFOVValue() const
{
	return Viewport.Pin()->GetViewportClient()->ViewFOV;
}


void SCustomizableObjectEditorViewportToolBar::OnFOVValueChanged(float NewValue) const
{
	TSharedPtr<FCustomizableObjectEditorViewportClient> ViewportClient = Viewport.Pin()->GetViewportClient();

	ViewportClient->FOVAngle = NewValue;
	// \todo: this editor name should be somewhere else.
	FString EditorName("CustomizableObjectEditor");
	int ViewportIndex=0;
	ViewportClient->ConfigOption->SetViewFOV(FName(*EditorName),NewValue,ViewportIndex);

	ViewportClient->ViewFOV = NewValue;
	ViewportClient->Invalidate();
}


ECheckBoxState SCustomizableObjectEditorViewportToolBar::IsRotationGridSnapChecked() const
{
	return GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


void SCustomizableObjectEditorViewportToolBar::HandleToggleRotationGridSnap(ECheckBoxState InState)
{
	GUnrealEd->Exec(GEditor->GetEditorWorldContext().World(), *FString::Printf(TEXT("MODE ROTGRID=%d"), !GetDefault<ULevelEditorViewportSettings>()->RotGridEnabled ? 1 : 0));
}


FText SCustomizableObjectEditorViewportToolBar::GetRotationGridLabel() const
{
	return FText::Format(LOCTEXT("GridRotation - Number - DegreeSymbol", "{0}\u00b0"), FText::AsNumber(GEditor->GetRotGridSize().Pitch));
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::FillRotationGridSnapMenu()
{
	UE::UnrealEd::FRotationGridCheckboxListExecuteActionDelegate ExecuteDelegate =
		UE::UnrealEd::FRotationGridCheckboxListExecuteActionDelegate::CreateStatic(
			&SCustomizableObjectEditorViewportTabBody::SetRotationGridSize
		);

	UE::UnrealEd::FRotationGridCheckboxListIsCheckedDelegate IsCheckedDelegate =
		UE::UnrealEd::FRotationGridCheckboxListIsCheckedDelegate::CreateStatic(
			&SCustomizableObjectEditorViewportTabBody::IsRotationGridSizeChecked
		);

	return UE::UnrealEd::CreateRotationGridSnapMenu(ExecuteDelegate, IsCheckedDelegate, Viewport.Pin()->GetCommandList());
}

FReply SCustomizableObjectEditorViewportToolBar::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	if (MenuAnchor->ShouldOpenDueToClick())
	{
		MenuAnchor->SetIsOpen(true);
		this->SetOpenMenu(MenuAnchor);
	}
	else
	{
		MenuAnchor->SetIsOpen(false);
		TSharedPtr<SMenuAnchor> NullAnchor;
		this->SetOpenMenu(MenuAnchor);
	}

	return FReply::Handled();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateOptionsMenu() const
{
	const FCustomizableObjectEditorViewportLODCommands& LevelViewportActions = FCustomizableObjectEditorViewportLODCommands::Get();

	// Get all menu extenders for this context menu from the level editor module
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
	TArray<FLevelEditorModule::FLevelEditorMenuExtender> MenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportOptionsMenuExtenders();
	
	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute(Viewport.Pin()->GetCommandList().ToSharedRef()));
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	const bool bIsPerspective = Viewport.Pin()->GetViewportClient()->IsPerspective();
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder OptionsMenuBuilder( bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList(), MenuExtender );
	{
		OptionsMenuBuilder.AddWidget( GenerateFOVMenu(), LOCTEXT("FOVAngle", "Field of View (H)") );
		OptionsMenuBuilder.AddMenuEntry( LevelViewportActions.HighResScreenshot );
	}

	return OptionsMenuBuilder.MakeWidget();
}


TSharedRef<SWidget> SCustomizableObjectEditorViewportToolBar::GenerateFOVMenu() const
{
	const float FOVMin = 5.f;
	const float FOVMax = 170.f;

	return
		SNew( SBox )
		.HAlign( HAlign_Right )
		[
			SNew( SBox )
			.Padding( FMargin(4.0f, 0.0f, 0.0f, 0.0f) )
			.WidthOverride( 100.0f )
			[
				SNew(SSpinBox<float>)
				.Font( UE_MUTABLE_GET_FONTSTYLE( TEXT( "MenuItem.Font" ) ) )
				.MinValue(FOVMin)
				.MaxValue(FOVMax)
				.Value( this, &SCustomizableObjectEditorViewportToolBar::OnGetFOVValue )
				.OnValueChanged( this, &SCustomizableObjectEditorViewportToolBar::OnFOVValueChanged )
			]
		];
}


#undef LOCTEXT_NAMESPACE
