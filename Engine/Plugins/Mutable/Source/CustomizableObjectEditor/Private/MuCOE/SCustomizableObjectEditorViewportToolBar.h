// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SViewportToolBar.h"

class ICustomizableObjectInstanceEditor;
enum ERotationGridMode : int;
enum class ECheckBoxState : uint8;
namespace ETextCommit { enum Type : int; }

class FMenuBuilder;
class SMenuAnchor;
class SWidget;
class SButton;
struct FSlateBrush;


/**
* A level viewport toolbar widget that is placed in a viewport
*/
class SCustomizableObjectEditorViewportToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectEditorViewportToolBar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class SCustomizableObjectEditorViewportTabBody> InViewport, TSharedPtr<class SEditorViewport> InRealViewport);

private:
	/**
	* Generates the toolbar view menu content
	*
	* @return The widget containing the view menu content
	*/
	TSharedRef<SWidget> GenerateViewMenu() const;
	

	/** Add the projector Rotation, traslation and scale buttons to viewport toolbar */
	TSharedRef<SWidget> GenerateRTSButtons();

	/**
	* Generates the toolbar LOD menu content
	*
	* @return The widget containing the LOD menu content based on LOD model count
	*/
	TSharedRef<SWidget> GenerateLODMenu() const;

	/**
	* Returns the label for the "LOD" tool bar menu, which changes depending on the current LOD selection
	*
	* @return	Label to use for this LOD label
	*/
	FText GetLODMenuLabel() const;

	/**
	* Generates the toolbar Character menu content
	*
	* @return The widget containing the view menu content
	*/
	TSharedRef<SWidget> GenerateCharacterMenu() const;
	
	/**
	* Generates the toolbar viewport type menu content
	*
	* @return The widget containing the viewport type menu content
	*/
	TSharedRef<SWidget> GenerateViewportTypeMenu() const;

	/**
	* Generates the toolbar playback menu content
	*
	* @return The widget containing the playback menu content
	*/
	TSharedRef<SWidget> GeneratePlaybackMenu() const;

	/**
	* Generates the toolbar Options Mode menu content
	*
	* @return The widget containing the Options menu content
	*/
	TSharedRef<SWidget> GenerateViewportOptionsMenu() const;

	/**
	* Returns the label for the Playback tool bar menu, which changes depending on the current playback speed
	*
	* @return	Label to use for this Menu
	*/
	FText GetPlaybackMenuLabel() const;

	/**
	* Returns the label for the Viewport type tool bar menu, which changes depending on the current selected type
	*
	* @return	Label to use for this Menu
	*/
	FText GetCameraMenuLabel() const;
	const FSlateBrush* GetCameraMenuLabelIcon() const;

	/** Called by the FOV slider in the perspective viewport to get the FOV value */
	float OnGetFOVValue() const;
	/** Called when the FOV slider is adjusted in the perspective viewport */
	void OnFOVValueChanged(float NewValue) const;

	ECheckBoxState IsRotationGridSnapChecked() const;
	void HandleToggleRotationGridSnap(ECheckBoxState InState);
	FText GetRotationGridLabel() const;
	TSharedRef<SWidget> FillRotationGridSnapMenu();

	/** Callback for drop-down menu with FOV and high resolution screenshot options currently */
	FReply OnMenuClicked();

	/** Generates drop-down menu with FOV and high resolution screenshot options currently */
	TSharedRef<SWidget> GenerateOptionsMenu() const;

	/** Generates widgets for viewport camera FOV control */
	TSharedRef<SWidget> GenerateFOVMenu() const;

	/** The viewport that we are in */
	TWeakPtr<SCustomizableObjectEditorViewportTabBody> Viewport;

	TWeakPtr<ICustomizableObjectInstanceEditor> WeakEditor;

	// Layout to show information about instance skeletal mesh update / CO asset data
	TSharedPtr<SButton> CompileErrorLayout;

	TSharedPtr<SMenuAnchor> MenuAnchor;
};
