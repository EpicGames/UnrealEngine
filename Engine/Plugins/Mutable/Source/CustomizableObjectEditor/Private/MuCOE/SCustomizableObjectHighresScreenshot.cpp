// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectHighresScreenshot.h"

#include "Components/SkeletalMeshComponent.h"
#include "Framework/Docking/TabManager.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "MuCOE/CustomizableObjectPreviewScene.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SSplitter.h"

TWeakPtr<class SWindow> SCustomizableObjectHighresScreenshot::CurrentWindow = NULL;
TWeakPtr<class SCustomizableObjectHighresScreenshot> SCustomizableObjectHighresScreenshot::CurrentDialog = NULL;
bool SCustomizableObjectHighresScreenshot::bMaskVisualizationWasEnabled = false;

SCustomizableObjectHighresScreenshot::SCustomizableObjectHighresScreenshot()
: Config(GetHighResScreenshotConfig())
{
}


SCustomizableObjectHighresScreenshot::~SCustomizableObjectHighresScreenshot()
{
	Window.Reset();
}


void SCustomizableObjectHighresScreenshot::Construct( const FArguments& InArgs )
{
	this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.Padding(5)
					[
						SNew(SSplitter)
						.Orientation(Orient_Horizontal)
						+SSplitter::Slot()
						.Value(1)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew( STextBlock )
								.Text( NSLOCTEXT("CustomizableObjectEditor", "ScreenshotSizeMultiplier", "Screenshot Size Multiplier") )
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew( STextBlock )
								.Text( NSLOCTEXT("CustomizableObjectEditor", "IncludeBufferVisTargets", "Include Buffer Visualization Targets") )
							]
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SAssignNew(HDRLabel, STextBlock)
								.Text(NSLOCTEXT("CustomizableObjectEditor", "CaptureHDR", "Write HDR format visualization targets"))
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SAssignNew(Force128BitRenderingLabel, STextBlock)
								.Text(NSLOCTEXT("CustomizableObjectEditor", "Force128BitPipeline", "Force 128-bit buffers for rendering pipeline"))
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew( STextBlock )
								.Text( NSLOCTEXT("CustomizableObjectEditor", "UseCustomDepth", "Use custom depth as mask") )
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew( STextBlock )
								.Text( NSLOCTEXT("CustomizableObjectEditor", "ShowHideBackground", "Show / hide background") )
							]
						]
						+SSplitter::Slot()
						.Value(1)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew (SHorizontalBox)
								+SHorizontalBox::Slot()
								.FillWidth(1)
								[
									SNew( SNumericEntryBox<float> )
									.Value(this, &SCustomizableObjectHighresScreenshot::GetResolutionMultiplier)
									.OnValueCommitted(this, &SCustomizableObjectHighresScreenshot::OnResolutionMultiplierChanged)
								]
								+SHorizontalBox::Slot()
								.HAlign(HAlign_Fill)
								.Padding(5,0,0,0)
								.FillWidth(3)
								[
									SNew( SSlider )
									.Value(this, &SCustomizableObjectHighresScreenshot::GetResolutionMultiplierSlider)
									.OnValueChanged(this, &SCustomizableObjectHighresScreenshot::OnResolutionMultiplierSliderChanged)
								]
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew( SCheckBox )
								.OnCheckStateChanged(this, &SCustomizableObjectHighresScreenshot::OnBufferVisualizationDumpEnabledChanged)
								.IsChecked(this, &SCustomizableObjectHighresScreenshot::GetBufferVisualizationDumpEnabled)
							]
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SAssignNew(HDRCheckBox, SCheckBox)
								.OnCheckStateChanged(this, &SCustomizableObjectHighresScreenshot::OnHDREnabledChanged)
								.IsChecked(this, &SCustomizableObjectHighresScreenshot::GetHDRCheckboxUIState)
							]
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SAssignNew(Force128BitRenderingCheckBox, SCheckBox)
								.OnCheckStateChanged(this, &SCustomizableObjectHighresScreenshot::OnForce128BitRenderingChanged)
								.IsChecked(this, &SCustomizableObjectHighresScreenshot::GetForce128BitRenderingCheckboxUIState)
							]
							+SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SNew( SCheckBox )
								.OnCheckStateChanged(this, &SCustomizableObjectHighresScreenshot::OnMaskEnabledChanged)
								.IsChecked(this, &SCustomizableObjectHighresScreenshot::GetMaskEnabled)
							]
							+ SVerticalBox::Slot()
							.VAlign(VAlign_Center)
							[
								SAssignNew(ShowFloor, SCheckBox)
								.OnCheckStateChanged(this, &SCustomizableObjectHighresScreenshot::OnShowFloorChanged)
								.IsChecked(this, &SCustomizableObjectHighresScreenshot::GetShowFloorState)
							]
						]
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SImage)
					.Image(UE_MUTABLE_GET_BRUSH("WarningStripe"))
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SBorder )
					.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
					[
						SNew( STextBlock )
						.Text( NSLOCTEXT("CustomizableObjectEditor", "CaptureWarningText", "Due to the high system requirements of a high resolution screenshot, very large multipliers might cause the graphics driver to become unresponsive and possibly crash. In these circumstances, please try using a lower multiplier") )
						.AutoWrapText(true)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SImage)
					.Image(UE_MUTABLE_GET_BRUSH("WarningStripe"))
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
						// for padding
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew( SButton )
						.ToolTipText( NSLOCTEXT("CustomizableObjectEditor", "ScreenshotCaptureTooltop", "Take a screenshot") )
						.OnClicked( this, &SCustomizableObjectHighresScreenshot::OnCaptureClicked )
						[
							SNew(SImage)
							.Image(UE_MUTABLE_GET_BRUSH("ClassIcon.SceneCaptureComponent"))
						]
					]
				]
			]
		];

	SetHDRUIEnableState(Config.bDumpBufferVisualizationTargets);
	SetForce128BitRenderingState(Config.bDumpBufferVisualizationTargets);
	bCaptureRegionControlsVisible = false;
	ExposureLogOffsetCached = -1;
	ExposureLogFixedCached = false;
	FloorVisibilityCached = false;
	GridVisibilityCached = false;
	SkeletalMeshComponent = nullptr;
}


ECheckBoxState SCustomizableObjectHighresScreenshot::GetShowFloorState() const
{
	if (LevelViewportClient.IsValid())
	{
		return (LevelViewportClient.Pin()->GetFloorVisibility() || LevelViewportClient.Pin()->GetEnvironmentMeshVisibility()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}


void SCustomizableObjectHighresScreenshot::OnMaskEnabledChanged(ECheckBoxState NewValue)
{
	Config.bMaskEnabled = (NewValue == ECheckBoxState::Checked);
	TSharedPtr<FSceneViewport> ConfigViewport = Config.TargetViewport.Pin();
	if (ConfigViewport.IsValid())
	{
		if (SkeletalMeshComponent != nullptr)
		{
			if (NewValue == ECheckBoxState::Checked)
			{
				if (SkeletalMeshComponent->GetWorld() != nullptr)
				{
					GEngine->Exec(SkeletalMeshComponent->GetWorld(), TEXT("r.CustomDepth 1"));
				}
				SkeletalMeshComponent->SetRenderCustomDepth(true);
			}
			else
			{
				SkeletalMeshComponent->SetRenderCustomDepth(false);
			}
		}

		ConfigViewport->GetClient()->GetEngineShowFlags()->SetHighResScreenshotMask(Config.bMaskEnabled);
		ConfigViewport->Invalidate();
	}
}


void SCustomizableObjectHighresScreenshot::OnShowFloorChanged(ECheckBoxState NewValue)
{
	if (LevelViewportClient.IsValid())
	{
		if (NewValue == ECheckBoxState::Unchecked)
		{
			ExposureLogOffsetCached = LevelViewportClient.Pin()->ExposureSettings.FixedEV100;
			LevelViewportClient.Pin()->ExposureSettings.FixedEV100 = 0;

			ExposureLogFixedCached = LevelViewportClient.Pin()->ExposureSettings.bFixed;
			LevelViewportClient.Pin()->ExposureSettings.bFixed = true;
			FloorVisibilityCached = LevelViewportClient.Pin()->GetFloorVisibility();
			GridVisibilityCached = LevelViewportClient.Pin()->GetGridVisibility();
			LevelViewportClient.Pin()->SetFloorVisibility(false);
			if (GridVisibilityCached)
			{
				LevelViewportClient.Pin()->UpdateShowGrid(true);
			}
		}
		else
		{
			if ((LevelViewportClient.Pin()->ExposureSettings.FixedEV100 == 0) &&
				(LevelViewportClient.Pin()->ExposureSettings.bFixed == true))
			{
				LevelViewportClient.Pin()->ExposureSettings.FixedEV100 = ExposureLogOffsetCached;
				LevelViewportClient.Pin()->ExposureSettings.bFixed = ExposureLogFixedCached;
			}

			LevelViewportClient.Pin()->SetFloorVisibility(FloorVisibilityCached);
			LevelViewportClient.Pin()->UpdateShowGrid(true);
		}

		LevelViewportClient.Pin()->SetEnvironmentMeshVisibility(NewValue == ECheckBoxState::Checked ? 1 : 0);
	}
}


void SCustomizableObjectHighresScreenshot::RestoreViewportValues()
{
	if (LevelViewportClient.IsValid())
	{
		// Restore exposure values if modified with the "Show / hide background" option
		if ((LevelViewportClient.Pin()->ExposureSettings.FixedEV100 == 0) &&
			(LevelViewportClient.Pin()->ExposureSettings.bFixed == true) &&
			(ExposureLogOffsetCached != -1))
		{
			LevelViewportClient.Pin()->ExposureSettings.FixedEV100 = ExposureLogOffsetCached;
			LevelViewportClient.Pin()->ExposureSettings.bFixed = ExposureLogFixedCached;
		}

		// Restore floor and environment mesh visibility
		if (!LevelViewportClient.Pin()->GetEnvironmentMeshVisibility())
		{
			LevelViewportClient.Pin()->UpdateShowGrid(false);
			LevelViewportClient.Pin()->SetEnvironmentMeshVisibility(1);
		}
	}
}


void SCustomizableObjectHighresScreenshot::WindowClosedHandler(const TSharedRef<SWindow>& InWindow)
{
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();

	if (CurrentDialog.IsValid())
	{
		if (CurrentDialog.Pin()->GetConfig().bMaskEnabled)
		{
			CurrentDialog.Pin()->OnMaskEnabledChanged(ECheckBoxState::Unchecked);
		}
	}

	if (CurrentDialog.IsValid() && CurrentDialog.Pin()->PreviewScenePtr.IsValid())
	{
		CurrentDialog.Pin()->PreviewScenePtr->GetWorld()->WorldType = EWorldType::EditorPreview; // Needed for custom depth pass for screenshot with no background
	}

	// Cleanup the config after each usage as it is a static and we don't want it to keep pointers or settings around between runs.
	Config.bDisplayCaptureRegion = false;
	Config.ChangeViewport(TWeakPtr<FSceneViewport>());
	CurrentDialog.Pin()->RestoreViewportValues();
	CurrentWindow.Reset();
	CurrentDialog.Reset();
}


TWeakPtr<class SWindow> SCustomizableObjectHighresScreenshot::OpenDialog(
	TSharedPtr<FSceneViewport>& InSceneViewport,
	TSharedPtr<FCustomizableObjectEditorViewportClient>& InLevelViewportClient,
	USkeletalMeshComponent* SkeletalMeshComponentParameter,
	TSharedPtr<FCustomizableObjectPreviewScene> InPreviewScenePtr)
{
	FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();
	TSharedPtr<FSceneViewport> ConfigViewport = Config.TargetViewport.Pin();
	TSharedPtr<SWindow> CurrentWindowPinned = CurrentWindow.Pin();

	bool bInitializeDialog = false;

	if (CurrentWindowPinned.IsValid())
	{
		// Dialog window is already open - if it is being pointed at a new viewport, reset the old one
		//if (InViewport != ConfigViewport)
		if (InSceneViewport != ConfigViewport)
		{
			bInitializeDialog = true;
		}
	}
	else
	{
		// No dialog window currently open - need to create one
		TSharedRef<SCustomizableObjectHighresScreenshot> Dialog = SNew(SCustomizableObjectHighresScreenshot);
		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title( NSLOCTEXT("CustomizableObjectEditor", "HighResolutionScreenshot", "High Resolution Screenshot") )
			.ClientSize(FVector2D(484,231))
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.FocusWhenFirstShown(true)
			[
				Dialog
			];

		Window->SetOnWindowClosed(FOnWindowClosed::CreateStatic(&WindowClosedHandler));
		Dialog->SetWindow(Window);
		Dialog->LevelViewportClient = TWeakPtr<FCustomizableObjectEditorViewportClient>(InLevelViewportClient);
		Dialog->SetSkeletalMeshComponent(SkeletalMeshComponentParameter);

		TSharedPtr<SWindow> ParentWindow = FGlobalTabmanager::Get()->GetRootWindow();

		if (ParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(Window, ParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(Window);
		}

		CurrentWindow = TWeakPtr<SWindow>(Window);
		CurrentDialog = TWeakPtr<SCustomizableObjectHighresScreenshot>(Dialog);

		Config.bDisplayCaptureRegion = true;

		bInitializeDialog = true;

		Dialog->PreviewScenePtr = InPreviewScenePtr;

		if (InPreviewScenePtr.IsValid())
		{
			InPreviewScenePtr->GetWorld()->WorldType = EWorldType::Editor; // Needed for custom depth pass for screenshot with no background
		}
	}

	if (bInitializeDialog)
	{
		CurrentWindow.Pin()->BringToFront();
		Config.ChangeViewport(InSceneViewport);

		// Enable mask visualization if the mask is enabled
		if (InSceneViewport.IsValid())
		{
			bMaskVisualizationWasEnabled = InSceneViewport->GetClient()->GetEngineShowFlags()->HighResScreenshotMask;
			InSceneViewport->GetClient()->GetEngineShowFlags()->SetHighResScreenshotMask(Config.bMaskEnabled);
		}
	}

	return CurrentWindow;
}


void SCustomizableObjectHighresScreenshot::SetSkeletalMeshComponent(USkeletalMeshComponent* SkeletalMeshComponentParameter)
{
	SkeletalMeshComponent = SkeletalMeshComponentParameter;
}


FReply SCustomizableObjectHighresScreenshot::OnCaptureClicked()
{
	if (!GIsHighResScreenshot)
	{
		TSharedPtr<FSceneViewport> ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			GScreenshotResolutionX = ConfigViewport->GetSizeXY().X * Config.ResolutionMultiplier;
			GScreenshotResolutionY = ConfigViewport->GetSizeXY().Y * Config.ResolutionMultiplier;
			FIntRect ScaledCaptureRegion = Config.UnscaledCaptureRegion;

			if (ScaledCaptureRegion.Area() > 0)
			{
				ScaledCaptureRegion.Clip(FIntRect(FIntPoint::ZeroValue, ConfigViewport->GetSizeXY()));
				ScaledCaptureRegion *= Config.ResolutionMultiplier;
			}

			Config.CaptureRegion = ScaledCaptureRegion;

			// Trigger the screenshot on the owning viewport
			ConfigViewport->TakeHighResScreenShot();
		}
	}

	return FReply::Handled();
}


bool SCustomizableObjectHighresScreenshot::IsSetCameraSafeAreaCaptureRegionEnabled() const
{
	if (Config.TargetViewport.IsValid())
	{
		FViewportClient* Client = Config.TargetViewport.Pin()->GetClient();
		if (Client)
		{
			FIntRect Rect;
			if (Client->OverrideHighResScreenshotCaptureRegion(Rect))
			{
				return true;
			}
		}
	}

	return false;
}
