// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HighResScreenshot.h"
#include "Slate/SceneViewport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

class SButton;
class SHorizontalBox;
class USkeletalMeshComponent;

class SCustomizableObjectHighresScreenshot : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectHighresScreenshot){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	SCustomizableObjectHighresScreenshot();

	virtual ~SCustomizableObjectHighresScreenshot();

	void SetWindow( TSharedPtr<SWindow> InWindow )
	{
		Window = InWindow;
	}

	void SetCaptureRegion(const FIntRect& InCaptureRegion)
	{
		Config.UnscaledCaptureRegion = InCaptureRegion;
		TSharedPtr<FSceneViewport> ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}
	}

	FHighResScreenshotConfig& GetConfig()
	{
		return Config;
	}
	
	static TWeakPtr<class SWindow> OpenDialog(
		TSharedPtr<class FSceneViewport>& InSceneViewport,
		TSharedPtr<class FCustomizableObjectEditorViewportClient>& InLevelViewportClient,
		USkeletalMeshComponent* SkeletalMeshComponentParameter,
		TSharedPtr<class FCustomizableObjectPreviewScene> PreviewScenePtr);

	void SetSkeletalMeshComponent(USkeletalMeshComponent* SkeletalMeshComponentParameter);

	TSharedPtr<class FCustomizableObjectPreviewScene> PreviewScenePtr;

private:

	FReply OnCaptureClicked();

	bool IsSetCameraSafeAreaCaptureRegionEnabled() const;

	void OnResolutionMultiplierChanged( float NewValue, ETextCommit::Type CommitInfo )
	{
		NewValue = FMath::Clamp(NewValue, FHighResScreenshotConfig::MinResolutionMultipler, FHighResScreenshotConfig::MaxResolutionMultipler);
		Config.ResolutionMultiplier = NewValue;
		Config.ResolutionMultiplierScale = (NewValue - FHighResScreenshotConfig::MinResolutionMultipler) / (FHighResScreenshotConfig::MaxResolutionMultipler - FHighResScreenshotConfig::MinResolutionMultipler);
	}

	void OnResolutionMultiplierSliderChanged( float NewValue )
	{
		Config.ResolutionMultiplierScale = NewValue;
		Config.ResolutionMultiplier = FMath::RoundToFloat(FMath::Lerp(FHighResScreenshotConfig::MinResolutionMultipler, FHighResScreenshotConfig::MaxResolutionMultipler, NewValue));
	}

	void OnMaskEnabledChanged(ECheckBoxState NewValue);

	void OnHDREnabledChanged(ECheckBoxState NewValue)
	{
		Config.SetHDRCapture(NewValue == ECheckBoxState::Checked);
		TSharedPtr<FSceneViewport> ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}
	}

	void OnShowFloorChanged(ECheckBoxState NewValue);

	void OnForce128BitRenderingChanged(ECheckBoxState NewValue)
	{
		Config.SetForce128BitRendering(NewValue == ECheckBoxState::Checked);
		TSharedPtr<FSceneViewport> ConfigViewport = Config.TargetViewport.Pin();
		if (ConfigViewport.IsValid())
		{
			ConfigViewport->Invalidate();
		}
	}

	void OnBufferVisualizationDumpEnabledChanged(ECheckBoxState NewValue)
	{
		bool bEnabled = (NewValue == ECheckBoxState::Checked);
		Config.bDumpBufferVisualizationTargets = bEnabled;
		SetHDRUIEnableState(bEnabled);
		SetForce128BitRenderingState(bEnabled);
	}

	EVisibility GetSpecifyCaptureRegionVisibility() const
	{
		return bCaptureRegionControlsVisible ? EVisibility::Hidden : EVisibility::Visible;
	}

	EVisibility GetCaptureRegionControlsVisibility() const
	{
		return bCaptureRegionControlsVisible ? EVisibility::Visible : EVisibility::Hidden;
	}

	void SetCaptureRegionControlsVisibility(bool bVisible)
	{
		bCaptureRegionControlsVisible = bVisible;
	}

	TOptional<float> GetResolutionMultiplier() const
	{
		return TOptional<float>(Config.ResolutionMultiplier);
	}

	float GetResolutionMultiplierSlider() const
	{
		return Config.ResolutionMultiplierScale;
	}

	ECheckBoxState GetMaskEnabled() const
	{
		return Config.bMaskEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetHDRCheckboxUIState() const
	{
		return Config.bCaptureHDR ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetShowFloorState() const;

	ECheckBoxState GetForce128BitRenderingCheckboxUIState() const
	{
		return Config.bForce128BitRendering ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	ECheckBoxState GetBufferVisualizationDumpEnabled() const
	{
		return Config.bDumpBufferVisualizationTargets ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	
	void SetHDRUIEnableState(bool bEnable)
	{
		HDRCheckBox->SetEnabled(bEnable);
		HDRLabel->SetEnabled(bEnable);
	}

	void SetForce128BitRenderingState(bool bEnable)
	{
		Force128BitRenderingCheckBox->SetEnabled(bEnable);
		Force128BitRenderingLabel->SetEnabled(bEnable);
	}

	void RestoreViewportValues();

	static void WindowClosedHandler(const TSharedRef<SWindow>& InWindow);

	TSharedPtr<SWindow> Window;
	TSharedPtr<SButton> CaptureRegionButton;
	TSharedPtr<SHorizontalBox> RegionCaptureActiveControlRoot;
	TSharedPtr<SCheckBox> HDRCheckBox;
	TSharedPtr<STextBlock> HDRLabel;
	TSharedPtr<SCheckBox> Force128BitRenderingCheckBox;
	TSharedPtr<STextBlock> Force128BitRenderingLabel;
	TSharedPtr<SCheckBox> ShowFloor;

	FHighResScreenshotConfig& Config;
	bool bCaptureRegionControlsVisible;

	static TWeakPtr<SWindow> CurrentWindow;
	static TWeakPtr<SCustomizableObjectHighresScreenshot> CurrentDialog;
	static bool bMaskVisualizationWasEnabled;

	TWeakPtr<class FCustomizableObjectEditorViewportClient> LevelViewportClient;
	int32 ExposureLogOffsetCached;
	bool ExposureLogFixedCached;
	bool FloorVisibilityCached;
	bool GridVisibilityCached;
	USkeletalMeshComponent* SkeletalMeshComponent;
};
