// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectLayoutEditor.h"

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/SCustomizableObjectLayoutGrid.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "ScopedTransaction.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailsView.h"

class ISlateStyle;
class SWidget;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

/**
 * 
 */
class FLayoutEditorCommands : public TCommands<FLayoutEditorCommands>
{

public:
	FLayoutEditorCommands() : TCommands<FLayoutEditorCommands>
	(
		"LayoutEditorCommands", // Context name for fast lookup
		NSLOCTEXT( "CustomizableObjectEditor", "LayoutEditorCommands", "Layout Editor" ), // Localized context name for displaying
		NAME_None, // Parent
		FCustomizableObjectEditorStyle::GetStyleSetName()
	)
	{
	}	
	
	/**  */
	TSharedPtr< FUICommandInfo > AddBlock;
	TSharedPtr< FUICommandInfo > RemoveBlock;
	TSharedPtr< FUICommandInfo > ConsolidateBlocks;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override
	{
		UI_COMMAND( AddBlock, "Add Block", "Add a new block to the layout.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( RemoveBlock, "Remove Block", "Remove a block from the layout.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND(ConsolidateBlocks, "Consolidate Blocks", "Convert automatic blocks into user blocks.", EUserInterfaceActionType::Button, FInputChord() );
	}
};


void FCustomizableObjectLayoutEditorDetailsBuilder::CustomizeDetails(IDetailLayoutBuilder& DetailsBuilder)
{
	TWeakObjectPtr<UObject> Node = nullptr;

	TSharedPtr<const IDetailsView> DetailsView = DetailsBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		Node = DetailsView->GetSelectedObjects()[0].Get();
	}

	// Layout category
	IDetailCategoryBuilder& LayoutCategory = DetailsBuilder.EditCategory("Layout Editor");

	if (!Node.IsValid())
	{
		LayoutCategory.AddCustomRow(LOCTEXT("LayoutEditor_MissingNode", "NodeNotFound"))
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_MissingNodeText", "Node not found"))
			];
		return;
	}

	if (!LayoutEditor)
	{
		check(false);
		return;
	}

	if (bShowLayoutSelector)
	{
		LayoutCategory.AddCustomRow(LOCTEXT("LayoutEditor_MeshSectionRow", "Mesh Section Selector"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_MeshSectionText", "Mesh Section: "))
					.ToolTipText(LOCTEXT("LayoutEditor_MeshSectionTooltip", "Select the mesh section to visualize."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.9f)
					[
						SAssignNew(LayoutEditor->MeshSectionComboBox, STextComboBox)
							.OptionsSource(&LayoutEditor->MeshSectionNames)
							.InitiallySelectedItem(LayoutEditor->MeshSectionNames[0])
							.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnMeshSectionChanged)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					// TODO UE-223305: Get the tooltip text from a function
					//+ SHorizontalBox::Slot()
					//.FillWidth(0.1f)
					//[
					//	SNew(SBox)
					//		.HAlign(EHorizontalAlignment::HAlign_Center)
					//		.VAlign(EVerticalAlignment::VAlign_Center)
					//		.Content()
					//		[
					//			SNew(SImage)
					//				.Image(UE_MUTABLE_GET_BRUSH(TEXT("Icons.Info")))
					//				.ToolTipText(FText(LOCTEXT("LaytoutEditor_MeshNoteTooltip", "Note:"
					//					"\nAs all meshes of a Data Table column share the same layout, the UVs shown"
					//					"\nin the editor are from the Default Skeletal Mesh of the Structure.")))
					//		]
					//]

			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnResetSelection)));


		LayoutCategory.AddCustomRow(LOCTEXT("LayoutEditor_UVChannelRow", "UV Channel Selector"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_UVChannelText", "UV Channel"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->UVChannelComboBox, STextComboBox)
					.InitiallySelectedItem(LayoutEditor->UVChannels[0])
					.OptionsSource(&LayoutEditor->UVChannels)
					.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnUVChannelChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	IDetailGroup* BaseLayoutOptionsGroup = &LayoutCategory.AddGroup(TEXT("LayoutEditor_OptionsGroupRow"), LOCTEXT("LayoutEditor_OptionsGroupRow", "Layout Options"), false, true);
	IDetailGroup* LayoutOptionsGroup = BaseLayoutOptionsGroup;

	BaseLayoutOptionsGroup->HeaderRow()
		.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::LayoutOptionsVisibility))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("LayoutEditor_OptionsGroupText", "Layout Options"))
				.ToolTipText(LOCTEXT("LayoutEditor_OptionsGroupTooltip", "Selects the packing strategy in case of a layout merge."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	if (bShowPackagingStrategy)
	{
		// Layout strategy selector group widget
		LayoutOptionsGroup = &LayoutOptionsGroup->AddGroup(TEXT("LayoutEditor_LayoutStrategyGroup"), LOCTEXT("LayoutEditor_LayoutStrategyGroup", "Layout Strategy Group"), true);
		LayoutOptionsGroup->HeaderRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::LayoutOptionsVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_LayoutStrategyText", "Layout Strategy:"))
					.ToolTipText(LOCTEXT("LayoutEditor_LayoutStrategyTooltip", "Selects the packing strategy in case of a layout merge."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->StrategyComboBox, SSearchableComboBox)
					.OptionsSource(&LayoutEditor->LayoutPackingStrategies)
					.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnPackagingStrategyChanged)
					.OnGenerateWidget(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GenerateLayoutPackagingStrategyComboBox)
					.ToolTipText(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetLayoutPackagingStrategyToolTip)
					[
						SNew(STextBlock)
							.Text(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetLayoutPackagingStrategyName)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}


	if (bShowAutomaticGenerationSettings)
	{
		LayoutOptionsGroup->AddWidgetRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::AutoBlocksStrategyVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_AutoBlockStrategyText", "Automatic Blocks Strategy:"))
					.ToolTipText(LOCTEXT("AutoBlockStrategyTooltip", "Selects the strategy to create layout blocks from unassigned UVs."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->AutoBlocksComboBox, SSearchableComboBox)
					.InitiallySelectedItem(LayoutEditor->AutoBlocksStrategies[0])
					.OptionsSource(&LayoutEditor->AutoBlocksStrategies)
					.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnAutoBlocksChanged)
					.OnGenerateWidget(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GenerateAutoBlocksComboBox)
					.ToolTipText(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetAutoBlocksTooltip)
					[
						SNew(STextBlock)
							.Text(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetAutoBlocksName)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];

		// Option to merge child automatic blocks
		LayoutOptionsGroup->AddWidgetRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::AutoBlocksMergeStrategyVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_AutoBlockMergeStrategyText", "Automatic Blocks Merge Strategy:"))
					.ToolTipText(LOCTEXT("LayoutEditor_AutoBlockMergeStrategyTooltip", "Selects the strategy to merge blocks during automatic generation."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->AutoBlocksMergeComboBox, SSearchableComboBox)
					.InitiallySelectedItem(LayoutEditor->AutoBlocksMergeStrategies[0])
					.OptionsSource(&LayoutEditor->AutoBlocksMergeStrategies)
					.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnAutoBlocksMergeChanged)
					.OnGenerateWidget(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GenerateAutoBlocksMergeComboBox)
					.ToolTipText(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetAutoBlocksMergeTooltip)
					[
						SNew(STextBlock)
							.Text(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetAutoBlocksMergeName)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}

	// Grid size combo
	if (bShowGridSize)
	{
		LayoutOptionsGroup->AddWidgetRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GridSizeVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_GridSizeText", "Grid Size:"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.5f)
					[
						SAssignNew(LayoutEditor->GridSizeXComboBox, STextComboBox)
							.InitiallySelectedItem(LayoutEditor->LayoutGridSizes[0])
							.OptionsSource(&LayoutEditor->LayoutGridSizes)
							.OnComboBoxOpening(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnOpenGridSizeComboBox)
							.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnGridSizeChanged, true)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					+ SHorizontalBox::Slot()
					.FillWidth(0.5f)
					[
						SAssignNew(LayoutEditor->GridSizeYComboBox, STextComboBox)
							.InitiallySelectedItem(LayoutEditor->LayoutGridSizes[0])
							.OptionsSource(&LayoutEditor->LayoutGridSizes)
							.OnComboBoxOpening(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnOpenGridSizeComboBox)
							.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnGridSizeChanged, false)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}

	if (bShowMaxGridSize)
	{
		LayoutOptionsGroup->AddWidgetRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::FixedStrategyOptionsVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_MaxGridSizeText", "Max Grid Size:"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->MaxGridSizeComboBox, STextComboBox)
					.InitiallySelectedItem(LayoutEditor->MaxLayoutGridSizes[0])
					.OptionsSource(&LayoutEditor->MaxLayoutGridSizes)
					.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnMaxGridSizeChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	if (bShowReductionMethods)
	{

		// Reduction method selector widget
		LayoutOptionsGroup->AddWidgetRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::FixedStrategyOptionsVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_ReductionMethodText", "Reduction Method:"))
					.ToolTipText(LOCTEXT("LayoutEditor_ReductionMethodTooltip", "Select how blocks will be reduced in case that they do not fit in the layout."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->ReductionMethodComboBox, SSearchableComboBox)
					.InitiallySelectedItem(LayoutEditor->BlockReductionMethods[0])
					.OptionsSource(&LayoutEditor->BlockReductionMethods)
					.OnSelectionChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnReductionMethodChanged)
					.OnGenerateWidget(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GenerateReductionMethodComboBox)
					.ToolTipText(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetLayoutReductionMethodTooltip)
					[
						SNew(STextBlock)
							.Text(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::GetLayoutReductionMethodName)
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
			];
	}

	if (bShowWarningSettings)
	{
		// Warning selector group widget
		IDetailGroup* IgnoreWarningsGroup = &LayoutCategory.AddGroup(TEXT("LayoutEditor_IgnoreWarningsGroup"), LOCTEXT("LayoutEditor_IgnoreWarningsGroup", "Ignore Unassigned Vertives Warning group"), false, true);
		IgnoreWarningsGroup->HeaderRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::WarningOptionsVisibility))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LayoutEditor_IgnoreLodsCheckBoxText", "Ignore Unassigned Vertices Warning:"))
					.ToolTipText(LOCTEXT("LayoutEditor_IgnoreLodsCheckBoxTooltip",
						"If true, warning message \"Source mesh has vertices not assigned to any layout block\" will be ignored."
						"\n Note:"
						"\n This warning can appear when a CO has more than one LOD using the same Layout Block node and these LODs have been generated using the automatic LOD generation."
						"\n (At high LODs, some vertices may have been displaced from their original position which means they could have been displaced outside their layout blocks.)"
						"\n Ignoring these warnings can cause some visual artifacts that may or may not be visually important at higher LODs."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SCheckBox)
					.IsChecked(ECheckBoxState::Unchecked)
					.OnCheckStateChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnIgnoreErrorsCheckStateChanged)
			];

		// LOD selector widget
		IgnoreWarningsGroup->AddWidgetRow()
			.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::WarningOptionsVisibility))
			.NameContent()
			[
				SAssignNew(LayoutEditor->LODSelectorTextWidget, STextBlock)
					.Text(LOCTEXT("LayoutEditor_IgnoreLodText", "First LOD to ignore"))
					.ToolTipText(LOCTEXT("LayoutEditor_IgnoreLodTooltip", "LOD from which vertex warning messages will be ignored."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.IsEnabled(false)
			]
			.ValueContent()
			[
				SAssignNew(LayoutEditor->LODSelectorWidget, SSpinBox<int32>)
					//.Value_Lambda(0)
					.IsEnabled(false)
					.OnValueChanged(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::OnIgnoreErrorsLODBoxValueChanged)
					.MinValue(0)
					.Delta(1)
					.AlwaysUsesDeltaSnap(true)
					.MinDesiredWidth(40.0f)
					.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	IDetailGroup& BlockEditorGroup = LayoutCategory.AddGroup(TEXT("LayoutEditor_BlockEditorGroup"), LOCTEXT("LayoutEditor_Layout", "Layout"), false, true);
	BlockEditorGroup.HeaderRow()
		.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::LayoutOptionsVisibility))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("LayoutEditor_BlockEditorText", "Layout"))
				.ToolTipText(LOCTEXT("LayoutEditor_BlockEditorTooltip", "Selected editable layout.")) 
				.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	BlockEditorGroup.AddWidgetRow()
		.Visibility(TAttribute<EVisibility>(LayoutEditor.Get(), &SCustomizableObjectLayoutEditor::LayoutOptionsVisibility))
		[
			SNew(SBox)
				.HeightOverride(700.0f)
				.WidthOverride(700.0f)
				[
					LayoutEditor.ToSharedRef()
				]
		];
}


SCustomizableObjectLayoutEditor::SCustomizableObjectLayoutEditor() : UICommandList(new FUICommandList())
{
}


void SCustomizableObjectLayoutEditor::Construct(const FArguments& InArgs)
{
	CurrentLayout = nullptr;
	Node = InArgs._Node;
	check(Node.Get());

	BindCommands();

	NothingSelectedString = MakeShareable(new FString("- Nothing Selected -"));

	MeshSections = InArgs._MeshSections.Get();
	OnPreUpdateLayoutDelegate = InArgs._OnPreUpdateLayoutDelegate;

	// Layout selector to select the mesh section and uv channel to edit
	MeshSectionNames.Empty();
	MeshSectionNames.Add(NothingSelectedString);

	for (FLayoutEditorMeshSection& MeshSection : MeshSections)
	{
		MeshSectionNames.Add(MeshSection.MeshName);
	}


	// UV Channel combo (for now hardcoded to a maximum of 4)
	UVChannels.Empty();

	TSharedPtr<FString> CurrentUVChannel;
	for (int32 Index = 0; Index < 4; ++Index)
	{
		UVChannels.Add(MakeShareable(new FString(FString::FromInt(Index))));
	}

	{
		// Layout Strategy options. Hardcoded: we should get names and tooltips from the enum property
		LayoutPackingStrategies.Empty();
		LayoutPackingStrategiesOptions.Empty();

		LayoutPackingStrategies.Add(MakeShareable(new FString("Resizable")));
		LayoutPackingStrategiesOptions.Add({ ECustomizableObjectTextureLayoutPackingStrategy::Resizable, LOCTEXT("LayoutEditor_ResizableStrategyTooltip", "In a layout merge, Layout size will increase if blocks don't fit inside.") });

		LayoutPackingStrategies.Add(MakeShareable(new FString("Fixed")));
		LayoutPackingStrategiesOptions.Add({ ECustomizableObjectTextureLayoutPackingStrategy::Fixed,LOCTEXT("LayoutEditor_FixedStrategyTooltip", "In a layout merge, the layout will increase its size until the maximum layout grid size"
			"\nBlock sizes will be reduced if they don't fit inside the layout."
			"\nSet the reduction priority of each block to control which blocks are reduced first and how they are reduced.") });

		LayoutPackingStrategies.Add(MakeShareable(new FString("Overlay")));
		LayoutPackingStrategiesOptions.Add({ ECustomizableObjectTextureLayoutPackingStrategy::Overlay,LOCTEXT("LayoutEditor_OverlayStrategyTooltip", "In a layout merge, the layout will not be modified and blocks will be ignored."
			"\nExtend material nodes just add their layouts on top of the base one") });
	}

	{
		AutoBlocksStrategies.Empty();
		AutoBlocksStrategiesOptions.Empty();

		AutoBlocksStrategies.Add(MakeShareable(new FString("Rectangles")));
		AutoBlocksStrategiesOptions.Add(
			{
				ECustomizableObjectLayoutAutomaticBlocksStrategy::Rectangles ,
				LOCTEXT("AutoBlockDetails_RectanglesStrategyTooltip", "Try to build rectangles splitting the UVs.")
			});

		AutoBlocksStrategies.Add(MakeShareable(new FString("UV islands")));
		AutoBlocksStrategiesOptions.Add(
			{
				ECustomizableObjectLayoutAutomaticBlocksStrategy::UVIslands,
				LOCTEXT("AutoBlockDetails_UVIslandsStrategyTooltip", "Try to build rectangles around each UV island, with a mask.")
			});

		AutoBlocksStrategies.Add(MakeShareable(new FString("Ignore (legacy)")));
		AutoBlocksStrategiesOptions.Add(
			{
				ECustomizableObjectLayoutAutomaticBlocksStrategy::Ignore,
				LOCTEXT("AutoBlockDetails_IgnoreStrategyTooltip", "Legacy behavior: assign to first block, or ignore if none.")
			});
	}

	{
		AutoBlocksMergeStrategies.Empty();
		AutoBlocksMergeStrategiesOptions.Empty();

		AutoBlocksMergeStrategies.Add(MakeShareable(new FString("Don't merge")));
		AutoBlocksMergeStrategiesOptions.Add(
			{
				ECustomizableObjectLayoutAutomaticBlocksMergeStrategy::DontMerge ,
				LOCTEXT("AutoBlockMerge_DontMergeTooltip", "Don't merge and make each UV island a unique block.")
			});

		AutoBlocksMergeStrategies.Add(MakeShareable(new FString("Merge child blocks")));
		AutoBlocksMergeStrategiesOptions.Add(
			{
				ECustomizableObjectLayoutAutomaticBlocksMergeStrategy::MergeChildBlocks,
				LOCTEXT("AutoBlockMerge_ChildBlocksTooltip", "Merge the blocks that are already fully included in another block.")
			});
	}


	// Array of available grid size options
	LayoutGridSizes.Empty();
	MaxLayoutGridSizes.Empty();

	for (int32 Size = 1; Size <= MaxPossibleGridSize; Size *= 2)
	{
		LayoutGridSizes.Add(MakeShareable(new FString(FString::FromInt(Size))));
		MaxLayoutGridSizes.Add(MakeShareable(new FString(FString::Printf(TEXT("%d x %d"), Size, Size))));
	}

	{
		// Block reduction methods options
		BlockReductionMethods.Empty();
		BlockReductionMethods.Add(MakeShareable(new FString("Halve")));
		BlockReductionMethodsTooltips.Add(LOCTEXT("LayoutEditor_HalveRedMethodTooltip", "Blocks will be reduced by half each time."));

		BlockReductionMethods.Add(MakeShareable(new FString("Unitary")));
		BlockReductionMethodsTooltips.Add(LOCTEXT("LayoutEditor_UnitaryRedMethodTooltip", "Blocks will be reduced by one unit each time."));
	}
}


void SCustomizableObjectLayoutEditor::SetLayout( UCustomizableObjectLayout* InLayout )
{
	if (!Node.IsValid())
	{
		return;
	}

	CurrentLayout = InLayout;

	FillLayoutComboBoxOptions();

	UCustomizableObjectLayout* LayoutForUVs = UVOverrideLayout ? UVOverrideLayout : CurrentLayout;

	TArray<FVector2f> UVs;
	TArray<FVector2f> UnassignedUVs;
	if (CurrentLayout)
	{
		LayoutForUVs->GetUVs(UVs);

		UnassignedUVs = TArray<FVector2f>();
		
		if (CurrentLayout->UnassignedUVs.Num())
		{
			UnassignedUVs = LayoutForUVs->UnassignedUVs[0];
		}
	}

	// Save some layout widget state to persist between updates
	bool bHadWidget = false;
	SCustomizableObjectLayoutGrid::FPointOfView OldView;
	if (LayoutGridWidget)
	{
		bHadWidget = true;
		OldView = LayoutGridWidget->PointOfView;
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(0.0f,2.0f,0.0f,0.0f )
		.AutoHeight()
		[
			BuildLayoutToolBar()
		]

		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SAssignNew(LayoutGridWidget, SCustomizableObjectLayoutGrid)
			.Mode(this, &SCustomizableObjectLayoutEditor::GetGridMode)
			.GridSize(this, &SCustomizableObjectLayoutEditor::GetGridSize)
			.Blocks(this, &SCustomizableObjectLayoutEditor::GetBlocks)
			.UVLayout(UVs)
			.UnassignedUVLayoutVertices(UnassignedUVs)
			.SelectionColor(FColor(75, 106, 230, 155))
			.OnBlockChanged(this, &SCustomizableObjectLayoutEditor::OnBlockChanged)
			.OnDeleteBlocks(this, &SCustomizableObjectLayoutEditor::OnRemoveBlock)
			.OnAddBlockAt(this, &SCustomizableObjectLayoutEditor::OnAddBlockAt)
			.OnSetBlockPriority(this, &SCustomizableObjectLayoutEditor::OnSetBlockPriority)
			.OnSetReduceBlockSymmetrically(this, &SCustomizableObjectLayoutEditor::OnSetBlockReductionSymmetry)
			.OnSetReduceBlockByTwo(this, &SCustomizableObjectLayoutEditor::OnSetBlockReductionByTwo)
			.OnSetBlockMask(this, &SCustomizableObjectLayoutEditor::OnSetBlockMask)
		]
	];	

	if (CurrentLayout)
	{
		CurrentLayout->GenerateAutomaticBlocksFromUVs();
	}

	if (bHadWidget)
	{
		LayoutGridWidget->PointOfView = OldView;
	}
}


void SCustomizableObjectLayoutEditor::SetUVsOverride(UCustomizableObjectLayout* InUVOverrideLayout)
{
	UVOverrideLayout = InUVOverrideLayout;
}


void SCustomizableObjectLayoutEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( CurrentLayout );
}


TSharedRef<SWidget> SCustomizableObjectLayoutEditor::BuildLayoutToolBar()
{
	FSlimHorizontalToolBarBuilder LayoutToolbarBuilder(UICommandList, FMultiBoxCustomization::None, nullptr, true);
	LayoutToolbarBuilder.SetLabelVisibility(EVisibility::Visible);

	//Getting toolbar style
	const ISlateStyle* const StyleSet = &FCoreStyle::Get();
	const FName& StyleName = "ToolBar";

	if (CurrentLayout && CurrentLayout->PackingStrategy!=ECustomizableObjectTextureLayoutPackingStrategy::Overlay)
	{
		LayoutToolbarBuilder.BeginSection("Blocks");
		{
			LayoutToolbarBuilder.AddToolBarButton(FLayoutEditorCommands::Get().AddBlock);
			LayoutToolbarBuilder.AddToolBarButton(FLayoutEditorCommands::Get().RemoveBlock);
			// Disable block consolidation if we are defininf blocks on top of another layout
			bool bCanConsolidate = !UVOverrideLayout;

			// Disable block consolidation if the cuyrrent automatic strategy doesn't generate blocks.
			if (CurrentLayout->AutomaticBlocksStrategy != ECustomizableObjectLayoutAutomaticBlocksStrategy::Rectangles)
			{
				bCanConsolidate = false;
			}

			if (bCanConsolidate)
			{
				LayoutToolbarBuilder.AddToolBarButton(FLayoutEditorCommands::Get().ConsolidateBlocks);
			}
		}
		LayoutToolbarBuilder.EndSection();
	}

	LayoutToolbarBuilder.BeginSection("Info");
	{
		LayoutToolbarBuilder.AddWidget
		(
			SNew(SBox)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(SImage)
				.Image(UE_MUTABLE_GET_BRUSH(TEXT("Icons.Info")))
				.ToolTip(GenerateInfoToolTip())
			]
		);
	}
	LayoutToolbarBuilder.EndSection();

	return
	SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.Padding(4,0)
	[
		SNew(SBorder)
		.Padding(2.0f)
		.BorderImage(UE_MUTABLE_GET_BRUSH("NoBorder"))
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		[
			LayoutToolbarBuilder.MakeWidget()
		]
	];
}


void SCustomizableObjectLayoutEditor::OnAddBlock()
{
	if (CurrentLayout)
	{
		const FScopedTransaction Transaction(LOCTEXT("OnAddBlock", "Add Block"));
		CurrentLayout->Modify();

		FCustomizableObjectLayoutBlock Block;
		CurrentLayout->Blocks.Add(Block);

		CurrentLayout->GenerateAutomaticBlocksFromUVs();

		if (LayoutGridWidget.IsValid())
		{
			LayoutGridWidget->SetSelectedBlock(Block.Id);
		}
	}
}


void SCustomizableObjectLayoutEditor::OnAddBlockAt(const FIntPoint Min, const FIntPoint Max)
{
	if (CurrentLayout)
	{
		const FScopedTransaction Transaction(LOCTEXT("OnAddBlockAt", "Add Block"));
		CurrentLayout->Modify();
		
		FCustomizableObjectLayoutBlock block(Min,Max);
		CurrentLayout->Blocks.Add(block);

		CurrentLayout->GenerateAutomaticBlocksFromUVs();
	}
}


void SCustomizableObjectLayoutEditor::OnRemoveBlock()
{
	if (CurrentLayout)
	{
		if (LayoutGridWidget.IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("OnRemoveBlock", "Remove Block"));
			CurrentLayout->Modify();

			const TArray<FGuid>& Selected = LayoutGridWidget->GetSelectedBlocks();
			for (TArray<FCustomizableObjectLayoutBlock>::TIterator It = CurrentLayout->Blocks.CreateIterator(); It; ++It)
			{
				if (Selected.Contains(It->Id))
				{
					It.RemoveCurrent();
				}
			}

			CurrentLayout->GenerateAutomaticBlocksFromUVs();
		}
	}
}


void SCustomizableObjectLayoutEditor::OnMeshSectionChanged(TSharedPtr<FString> MeshSection, ESelectInfo::Type SelectInfo)
{
	if (!Node.IsValid())
	{
		return;
	}

	TSharedPtr<FString> UVChannel = UVChannelComboBox->GetSelectedItem();
	UpdateLayout(FindSelectedLayout(MeshSection, UVChannel));
}


void SCustomizableObjectLayoutEditor::OnUVChannelChanged(TSharedPtr<FString> UVChannel, ESelectInfo::Type SelectInfo)
{
	if (!Node.IsValid())
	{
		return;
	}

	TSharedPtr<FString> MeshSection = MeshSectionComboBox->GetSelectedItem();
	UpdateLayout(FindSelectedLayout(MeshSection, UVChannel));
}


void SCustomizableObjectLayoutEditor::OnOpenGridSizeComboBox()
{
	if (!CurrentLayout)
	{
		return;
	}

	const int32 MaxGridSize = FixedStrategyOptionsVisibility() == EVisibility::Visible ? CurrentLayout->GetMaxGridSize().X : MaxPossibleGridSize;
	const int32 NumOptions = FMath::CountTrailingZeros(MaxGridSize) + 1;

	LayoutGridSizes.SetNum(NumOptions);

	int32 AuxIndex = 0;
	for (TSharedPtr<FString>& GridSizeString : LayoutGridSizes)
	{
		if (!GridSizeString)
		{
			GridSizeString = MakeShareable(new FString(FString::FromInt(1 << AuxIndex)));
		}

		AuxIndex++;
	}
}


void SCustomizableObjectLayoutEditor::OnGridSizeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo, bool bIsGridSizeX)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	const int32 Size = 1 << LayoutGridSizes.Find(NewSelection);

	FIntPoint GridSize = CurrentLayout->GetGridSize();
	GridSize.X = bIsGridSizeX ? Size : GridSize.X;
	GridSize.Y = !bIsGridSizeX ? Size : GridSize.Y;

	if (GridSize != CurrentLayout->GetGridSize())
	{
		CurrentLayout->SetGridSize(GridSize);

		Node->Modify();

		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnMaxGridSizeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{

	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	const int32 Size = 1 << MaxLayoutGridSizes.Find(NewSelection);

	if (CurrentLayout->GetMaxGridSize().X != Size || CurrentLayout->GetMaxGridSize().Y != Size)
	{
		CurrentLayout->SetMaxGridSize(FIntPoint(Size));

		// GridSize must be equal or smaller than MaxGridSize.
		FIntPoint GridSize = CurrentLayout->GetGridSize();
		GridSize.X = FMath::Clamp(GridSize.X, 1, Size);
		GridSize.Y = FMath::Clamp(GridSize.Y, 1, Size);
		CurrentLayout->SetGridSize(GridSize);
		
		Node->Modify();

		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnAutoBlocksChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	uint32 Selection = AutoBlocksStrategies.IndexOfByKey(NewSelection);
	
	if (CurrentLayout->AutomaticBlocksStrategy != AutoBlocksStrategiesOptions[Selection].Value)
	{
		CurrentLayout->AutomaticBlocksStrategy = AutoBlocksStrategiesOptions[Selection].Value;

		Node->Modify();

		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnAutoBlocksMergeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	uint32 Selection = AutoBlocksMergeStrategies.IndexOfByKey(NewSelection);
	
	if (CurrentLayout->AutomaticBlocksMergeStrategy != AutoBlocksMergeStrategiesOptions[Selection].Value)
	{
		CurrentLayout->AutomaticBlocksMergeStrategy = AutoBlocksMergeStrategiesOptions[Selection].Value;

		Node->Modify();
	
		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnReductionMethodChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	uint32 ReductionMethod = BlockReductionMethods.IndexOfByKey(NewSelection);
	
	if (CurrentLayout->BlockReductionMethod != (ECustomizableObjectLayoutBlockReductionMethod)ReductionMethod)
	{
		CurrentLayout->BlockReductionMethod = (ECustomizableObjectLayoutBlockReductionMethod)ReductionMethod;

		Node->Modify();

		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnPackagingStrategyChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	const uint32 Index = LayoutPackingStrategies.IndexOfByKey(NewSelection);
	if (CurrentLayout->PackingStrategy != LayoutPackingStrategiesOptions[Index].Value)
	{
		CurrentLayout->PackingStrategy = LayoutPackingStrategiesOptions[Index].Value;
		
		// Update max size when changing strategies
		if (CurrentLayout->PackingStrategy == ECustomizableObjectTextureLayoutPackingStrategy::Fixed)
		{
			// GridSize must be equal or smaller than MaxGridSize.
			const int32 MaxGridSize = CurrentLayout->GetMaxGridSize().X;
			
			FIntPoint GridSize = CurrentLayout->GetGridSize();
			GridSize.X = FMath::Clamp(GridSize.X, 1, MaxGridSize);
			GridSize.Y = FMath::Clamp(GridSize.Y, 1, MaxGridSize);
			CurrentLayout->SetGridSize(GridSize);
		}

		Node->Modify();
	
		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnIgnoreErrorsCheckStateChanged(ECheckBoxState CheckedBoxState)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	const bool bIsChecked = CheckedBoxState == ECheckBoxState::Checked;
	if (CurrentLayout->GetIgnoreVertexLayoutWarnings() != bIsChecked)
	{
		CurrentLayout->SetIgnoreVertexLayoutWarnings(bIsChecked);

		Node->Modify();

		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnIgnoreErrorsLODBoxValueChanged(int32 Value)
{
	if (!Node.IsValid() || !CurrentLayout)
	{
		return;
	}

	if (CurrentLayout->GetFirstLODToIgnoreWarnings() != Value)
	{
		CurrentLayout->SetIgnoreWarningsLOD(Value);

		Node->Modify();

		// Reset to update the UI.
		UpdateLayout(CurrentLayout);
	}
}


void SCustomizableObjectLayoutEditor::OnResetSelection()
{
	if (MeshSectionComboBox.IsValid())
	{
		MeshSectionComboBox->SetSelectedItem(NothingSelectedString);
	}

	UpdateLayout(nullptr);
}


void SCustomizableObjectLayoutEditor::OnConsolidateBlocks()
{
	if (CurrentLayout)
	{
		const FScopedTransaction Transaction(LOCTEXT("OnConsolidateBlocks", "Consolidate Blocks"));
		CurrentLayout->Modify();
				
		CurrentLayout->GenerateAutomaticBlocksFromUVs();

		CurrentLayout->ConsolidateAutomaticBlocks();
	}
}


ELayoutGridMode SCustomizableObjectLayoutEditor::GetGridMode() const
{
	if (CurrentLayout && CurrentLayout->PackingStrategy != ECustomizableObjectTextureLayoutPackingStrategy::Overlay)
	{
		return ELayoutGridMode::ELGM_Edit;
	}

	return ELayoutGridMode::ELGM_ShowUVsOnly;
}


FIntPoint SCustomizableObjectLayoutEditor::GetGridSize() const
{
	if (CurrentLayout)
	{
		return CurrentLayout->GetGridSize();
	}
	return FIntPoint(1);
}


void SCustomizableObjectLayoutEditor::OnBlockChanged( FGuid BlockId, FIntRect Block )
{
	if (CurrentLayout)
	{
		const FScopedTransaction Transaction(LOCTEXT("OnBlockChanged", "Edit Block"));
		CurrentLayout->Modify();
		
		for (FCustomizableObjectLayoutBlock& B : CurrentLayout->Blocks)
		{
			if (B.Id == BlockId)
			{
				B.Min = Block.Min;
				B.Max = Block.Max;

				break;
			}
		}

		CurrentLayout->GenerateAutomaticBlocksFromUVs();
	}
}


void SCustomizableObjectLayoutEditor::OnSetBlockPriority(int32 InValue)
{
	if (CurrentLayout)
	{
		const FScopedTransaction Transaction(LOCTEXT("OnSetBlockPriority", "Change Block Priority"));
		CurrentLayout->Modify();
		
		if (LayoutGridWidget.IsValid())
		{
			const TArray<FGuid>& SelectedBlocks = LayoutGridWidget->GetSelectedBlocks();

			for (FCustomizableObjectLayoutBlock& Block : CurrentLayout->Blocks)
			{
				if (SelectedBlocks.Contains(Block.Id))
				{
					Block.Priority = InValue;
				}
			}
		}
	}
}


void SCustomizableObjectLayoutEditor::OnSetBlockReductionSymmetry(bool bInValue)
{
	if (CurrentLayout)
	{
		if (LayoutGridWidget.IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("OnSetBlockReductionSymmetry", "Change Block Symetry"));
			CurrentLayout->Modify();
			
			const TArray<FGuid>& SelectedBlocks = LayoutGridWidget->GetSelectedBlocks();

			for (FCustomizableObjectLayoutBlock& Block : CurrentLayout->Blocks)
			{
				if (SelectedBlocks.Contains(Block.Id))
				{
					Block.bReduceBothAxes = bInValue;
				}
			}
		}
	}
}


void SCustomizableObjectLayoutEditor::OnSetBlockReductionByTwo(bool bInValue)
{
	if (CurrentLayout)
	{
		if (LayoutGridWidget.IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("OnSetBlockReductionByTwo", "Change Block Reduction By Two"));
			CurrentLayout->Modify();
			
			const TArray<FGuid>& SelectedBlocks = LayoutGridWidget->GetSelectedBlocks();

			for (FCustomizableObjectLayoutBlock& Block : CurrentLayout->Blocks)
			{
				if (SelectedBlocks.Contains(Block.Id))
				{
					Block.bReduceByTwo = bInValue;
				}
			}
		}
	}
}


void SCustomizableObjectLayoutEditor::OnSetBlockMask(UTexture2D* InValue)
{
	if (CurrentLayout)
	{
		const FScopedTransaction Transaction(LOCTEXT("OnSetBlockMask", "Change Block Mask"));
		CurrentLayout->Modify();

		if (LayoutGridWidget.IsValid())
		{
			const TArray<FGuid>& SelectedBlocks = LayoutGridWidget->GetSelectedBlocks();

			for (FCustomizableObjectLayoutBlock& Block : CurrentLayout->Blocks)
			{
				if (SelectedBlocks.Contains(Block.Id))
				{
					Block.Mask = InValue;
				}
			}
		}

		CurrentLayout->GenerateAutomaticBlocksFromUVs();
	}
}


void SCustomizableObjectLayoutEditor::UpdateLayout(UCustomizableObjectLayout* Layout)
{
	OnPreUpdateLayoutDelegate.ExecuteIfBound();

	SetLayout(Layout);
}


TArray<FCustomizableObjectLayoutBlock> SCustomizableObjectLayoutEditor::GetBlocks() const
{
	TArray<FCustomizableObjectLayoutBlock> Blocks;

	if (CurrentLayout)
	{
		Blocks = CurrentLayout->Blocks;
		Blocks.Append(CurrentLayout->AutomaticBlocks);
	}

	return Blocks;
}


void SCustomizableObjectLayoutEditor::BindCommands()
{
	// Register our commands. This will only register them if not previously registered
	FLayoutEditorCommands::Register();

	const FLayoutEditorCommands& Commands = FLayoutEditorCommands::Get();

	UICommandList->MapAction(
		Commands.AddBlock,
		FExecuteAction::CreateSP( this, &SCustomizableObjectLayoutEditor::OnAddBlock ),
		FCanExecuteAction(),
		FIsActionChecked() );

	UICommandList->MapAction(
		Commands.RemoveBlock,
		FExecuteAction::CreateSP( this, &SCustomizableObjectLayoutEditor::OnRemoveBlock ),
		FCanExecuteAction(),
		FIsActionChecked() );

	UICommandList->MapAction(
		Commands.ConsolidateBlocks,
		FExecuteAction::CreateSP(this, &SCustomizableObjectLayoutEditor::OnConsolidateBlocks),
		FCanExecuteAction(),
		FIsActionChecked());
}


TSharedPtr<IToolTip> SCustomizableObjectLayoutEditor::GenerateInfoToolTip() const
{
	TSharedPtr<SGridPanel> ToolTipWidget = SNew(SGridPanel);
	int32 SlotCount = 0;

	auto BuildShortcutAndTooltip = [ToolTipWidget, &SlotCount](const FText& Shortcut, const FText& Tooltip)
	{
		// Command Shortcut
		ToolTipWidget->AddSlot(0, SlotCount)
		[
			SNew(STextBlock)
			.Text(Shortcut)
		];

		// Command Explanation
		ToolTipWidget->AddSlot(1, SlotCount)
		.Padding(15.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(Tooltip)
		];

		++SlotCount;
	};

	// Duplicate command
	if (CurrentLayout && CurrentLayout->PackingStrategy != ECustomizableObjectTextureLayoutPackingStrategy::Overlay)
	{
		BuildShortcutAndTooltip(LOCTEXT("ShortCut_DuplicateBlocks", "CTRL + D"), LOCTEXT("Tooltip_DuplicateBlocks", "Duplicate selected block/s"));
		BuildShortcutAndTooltip(LOCTEXT("ShortCut_CreateNewBlock", "CTRL + N"), LOCTEXT("Tooltip_CreateNewBlock", "Create new block"));
		BuildShortcutAndTooltip(LOCTEXT("ShortCut_FillGridSize", "CTRL + F"), LOCTEXT("Tooltip_FillGridSize", "Resize selected block/s to grid size"));
		BuildShortcutAndTooltip(LOCTEXT("ShortCut_DeleteSelectedBlock", "DEL"), LOCTEXT("Tooltip_DeleteSelectedBlock", "Delete selected block/s"));
		BuildShortcutAndTooltip(LOCTEXT("ShortCut_SelectMultipleBlocksOneByOne", "SHIFT + L Click"), LOCTEXT("Tooltip_SelectMultipleBlocksOneByOne", "Select multiple blocks one by one"));
		BuildShortcutAndTooltip(LOCTEXT("ShortCut_SelectMultipleBlocks", "L Click + Drag"), LOCTEXT("Tooltip_SelectMultipleBlocks", "Select blocks that intersect with the yellow rectangle"));
	}
	BuildShortcutAndTooltip(LOCTEXT("ShortCut_Pan", "M Click + Drag"), LOCTEXT("Tooltip_Pan", "Pan the UV view."));
	BuildShortcutAndTooltip(LOCTEXT("ShortCut_Zoom", "M Wheel"), LOCTEXT("Tooltip_Zoom", "Zoom in and out the UV view."));

	return SNew(SToolTip)
	[
		ToolTipWidget.ToSharedRef()
	];
}


TSharedRef<SWidget> SCustomizableObjectLayoutEditor::GenerateAutoBlocksComboBox(TSharedPtr<FString> InItem) const
{	
	//A list of tool tips should have been populated in a 1 to 1 correspondance
	check(AutoBlocksStrategies.Num() == AutoBlocksStrategiesOptions.Num());

	FText AutoBlocksName;
	FText AutoBlocksTooltip;

	if (InItem.IsValid())
	{
		AutoBlocksName = FText::FromString(*InItem.Get());
		int32 TooltipIndex = AutoBlocksStrategies.IndexOfByKey(InItem);

		if (ensure(AutoBlocksStrategiesOptions.IsValidIndex(TooltipIndex)))
		{
			AutoBlocksTooltip = AutoBlocksStrategiesOptions[TooltipIndex].Tooltip;
		}
	}

	return SNew(STextBlock)
		.Text(AutoBlocksName)
		.ToolTipText(AutoBlocksTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}


FText SCustomizableObjectLayoutEditor::GetAutoBlocksName() const
{
	if (CurrentLayout)
	{
		const uint32 ReductionIndex = (uint32)CurrentLayout->AutomaticBlocksStrategy;
		if (ensure(AutoBlocksStrategies.IsValidIndex(ReductionIndex)))
		{
			return FText::FromString(*AutoBlocksStrategies[ReductionIndex]);
		}
	}

	return FText();
}


FText SCustomizableObjectLayoutEditor::GetAutoBlocksTooltip() const
{
	if (CurrentLayout)
	{
		const uint32 ReductionIndex = (uint32)CurrentLayout->AutomaticBlocksStrategy;
		if (ensure(AutoBlocksStrategiesOptions.IsValidIndex(ReductionIndex)))
		{
			return AutoBlocksStrategiesOptions[ReductionIndex].Tooltip;
		}
	}

	return FText();
}


TSharedRef<SWidget> SCustomizableObjectLayoutEditor::GenerateAutoBlocksMergeComboBox(TSharedPtr<FString> InItem) const
{	
	//A list of tool tips should have been populated in a 1 to 1 correspondance
	check(AutoBlocksMergeStrategies.Num() == AutoBlocksMergeStrategiesOptions.Num());

	FText AutoBlocksMergeName;
	FText AutoBlocksMergeTooltip;

	if (InItem.IsValid())
	{
		AutoBlocksMergeName = FText::FromString(*InItem.Get());
		int32 TooltipIndex = AutoBlocksMergeStrategies.IndexOfByKey(InItem);

		if (ensure(AutoBlocksMergeStrategiesOptions.IsValidIndex(TooltipIndex)))
		{
			AutoBlocksMergeTooltip = AutoBlocksMergeStrategiesOptions[TooltipIndex].Tooltip;
		}
	}

	return SNew(STextBlock)
		.Text(AutoBlocksMergeName)
		.ToolTipText(AutoBlocksMergeTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}


FText SCustomizableObjectLayoutEditor::GetAutoBlocksMergeName() const
{
	if (CurrentLayout)
	{
		const uint32 AutoMergeIndex = (uint32)CurrentLayout->AutomaticBlocksMergeStrategy;
		if (ensure(AutoBlocksMergeStrategies.IsValidIndex(AutoMergeIndex)))
		{
			return FText::FromString(*AutoBlocksMergeStrategies[AutoMergeIndex]);
		}
	}

	return FText();
}


FText SCustomizableObjectLayoutEditor::GetAutoBlocksMergeTooltip() const
{
	if (CurrentLayout)
	{
		const uint32 AutoMergeIndex = (uint32)CurrentLayout->AutomaticBlocksMergeStrategy;
		if (ensure(AutoBlocksMergeStrategiesOptions.IsValidIndex(AutoMergeIndex)))
		{
			return AutoBlocksMergeStrategiesOptions[AutoMergeIndex].Tooltip;
		}
	}

	return FText();
}


TSharedRef<SWidget> SCustomizableObjectLayoutEditor::GenerateReductionMethodComboBox(TSharedPtr<FString> InItem) const
{
	//A list of tool tips should have been populated in a 1 to 1 correspondance
	check(BlockReductionMethods.Num() == BlockReductionMethodsTooltips.Num());

	FText ReductionMethodName;
	FText ReductionMethodTooltip;
	
	if (InItem.IsValid())
	{
		ReductionMethodName = FText::FromString(*InItem.Get());
		int32 TooltipIndex = BlockReductionMethods.IndexOfByKey(InItem);
	
		if (ensure(BlockReductionMethodsTooltips.IsValidIndex(TooltipIndex)))
		{
			ReductionMethodTooltip = BlockReductionMethodsTooltips[TooltipIndex];
		}
	}
	
	return SNew(STextBlock)
		.Text(ReductionMethodName)
		.ToolTipText(ReductionMethodTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}


FText SCustomizableObjectLayoutEditor::GetLayoutReductionMethodName() const
{
	if (CurrentLayout)
	{
		const uint32 ReductionIndex = (uint32)CurrentLayout->BlockReductionMethod;
		if (ensure(BlockReductionMethods.IsValidIndex(ReductionIndex)))
		{
			return FText::FromString(*BlockReductionMethods[ReductionIndex]);
		}
	}

	return FText();
}


FText SCustomizableObjectLayoutEditor::GetLayoutReductionMethodTooltip() const
{
	if (CurrentLayout)
	{
		const uint32 ReductionIndex = (uint32)CurrentLayout->BlockReductionMethod;
		if (ensure(BlockReductionMethods.IsValidIndex(ReductionIndex)))
		{
			return BlockReductionMethodsTooltips[ReductionIndex];
		}
	}
	
	return FText();
}


TSharedRef<SWidget> SCustomizableObjectLayoutEditor::GenerateLayoutPackagingStrategyComboBox(TSharedPtr<FString> InItem) const
{
	//A list of tool tips should have been populated in a 1 to 1 correspondance
	check(LayoutPackingStrategies.Num() == LayoutPackingStrategiesOptions.Num());
	
	FText StrategyTooltip;
	FText StrategyName;

	if (InItem.IsValid())
	{
		StrategyName = FText::FromString(*InItem.Get());
		int32 TooltipIndex = LayoutPackingStrategies.IndexOfByKey(InItem);

		if (ensure(LayoutPackingStrategiesOptions.IsValidIndex(TooltipIndex)))
		{
			StrategyTooltip = LayoutPackingStrategiesOptions[TooltipIndex].Tooltip;
		}
	}

	return SNew(STextBlock)
		.Text(StrategyName)
		.ToolTipText(StrategyTooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont());
}


FText SCustomizableObjectLayoutEditor::GetLayoutPackagingStrategyName() const
{
	if (CurrentLayout)
	{
		const uint32 StrategyIndex = (uint32)CurrentLayout->PackingStrategy;
		if (ensure(LayoutPackingStrategies.IsValidIndex(StrategyIndex)))
		{
			return FText::FromString(*LayoutPackingStrategies[StrategyIndex]);
		}
	}

	return FText();
}


FText SCustomizableObjectLayoutEditor::GetLayoutPackagingStrategyToolTip() const
{
	if (CurrentLayout)
	{
		const uint32 StrategyIndex = (uint32)CurrentLayout->PackingStrategy;
		if (ensure(LayoutPackingStrategiesOptions.IsValidIndex(StrategyIndex)))
		{
			return LayoutPackingStrategiesOptions[StrategyIndex].Tooltip;
		}
	}

	return FText();
}


UCustomizableObjectLayout* SCustomizableObjectLayoutEditor::FindSelectedLayout(TSharedPtr<FString> MeshSectionName, TSharedPtr<FString> UVChannel)
{
	const int32 Index = UVChannels.Find(UVChannel);

	for (FLayoutEditorMeshSection& MeshSection : MeshSections)
	{
		if (MeshSection.MeshName != MeshSectionName)
		{
			continue;
		}

		if (MeshSection.Layouts.IsValidIndex(Index))
		{
			return MeshSection.Layouts[Index].Get();
		}
		else
		{
			return nullptr;
		}
	}

	return nullptr;
}


void SCustomizableObjectLayoutEditor::FillLayoutComboBoxOptions()
{
	if (CurrentLayout)
	{
		// Update LayoutGridSizes
		if (CurrentLayout->PackingStrategy == ECustomizableObjectTextureLayoutPackingStrategy::Fixed)
		{
			const int32 MaxGridSize = CurrentLayout->GetMaxGridSize().GetMax();
			LayoutGridSizes.Empty();

			for (int32 Size = 1; Size <= MaxGridSize; Size *= 2)
			{
				LayoutGridSizes.Add(MakeShareable(new FString(FString::FromInt(Size))));
			}

			FInt32Point GridSize = CurrentLayout->GetGridSize();
			if (GridSize.X > MaxGridSize || GridSize.Y > MaxGridSize)
			{
				GridSize.X = FMath::Clamp(GridSize.X, 1, MaxGridSize);
				GridSize.Y = FMath::Clamp(GridSize.Y, 1, MaxGridSize);
				CurrentLayout->SetGridSize(GridSize);
			}
		}

		if (StrategyComboBox)
		{
			const uint32 StrategyIndex = (uint32)CurrentLayout->PackingStrategy;
			StrategyComboBox->SetSelectedItem(LayoutPackingStrategies[StrategyIndex]);
		}

		if (AutoBlocksComboBox)
		{
			const uint32 AutoBlockIndex = (uint32)CurrentLayout->AutomaticBlocksStrategy;
			AutoBlocksComboBox->SetSelectedItem(AutoBlocksStrategies[AutoBlockIndex]);
		}

		if (AutoBlocksMergeComboBox)
		{
			const uint32 AutoBlockMergeIndex = (uint32)CurrentLayout->AutomaticBlocksMergeStrategy;
			AutoBlocksMergeComboBox->SetSelectedItem(AutoBlocksMergeStrategies[AutoBlockMergeIndex]);
		}

		if (GridSizeXComboBox)
		{
			const int32 Index = FMath::CountTrailingZeros(CurrentLayout->GetGridSize().X);
			GridSizeXComboBox->SetSelectedItem(LayoutGridSizes[Index]);
		}

		if (GridSizeYComboBox)
		{
			const int32 Index = FMath::CountTrailingZeros(CurrentLayout->GetGridSize().Y);
			GridSizeYComboBox->SetSelectedItem(LayoutGridSizes[Index]);
		}

		if (MaxGridSizeComboBox)
		{
			const int32 Index = FMath::CountTrailingZeros(CurrentLayout->GetMaxGridSize().X);
			MaxGridSizeComboBox->SetSelectedItem(MaxLayoutGridSizes[Index]);
		}

		if (ReductionMethodComboBox)
		{
			const uint32 ReductionMethodIndex = (uint32)CurrentLayout->BlockReductionMethod;
			ReductionMethodComboBox->SetSelectedItem(BlockReductionMethods[ReductionMethodIndex]);
		}

		if (LODSelectorWidget)
		{
			LODSelectorWidget->SetEnabled(CurrentLayout->GetIgnoreVertexLayoutWarnings());
			LODSelectorWidget->SetValue(CurrentLayout->GetFirstLODToIgnoreWarnings());
		}

		if (LODSelectorTextWidget)
		{
			LODSelectorTextWidget->SetEnabled(CurrentLayout->GetIgnoreVertexLayoutWarnings());
		}
	}
}


EVisibility SCustomizableObjectLayoutEditor::GridSizeVisibility() const
{
	return CurrentLayout && CurrentLayout->PackingStrategy != ECustomizableObjectTextureLayoutPackingStrategy::Overlay ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility SCustomizableObjectLayoutEditor::LayoutOptionsVisibility() const
{
	return CurrentLayout ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility SCustomizableObjectLayoutEditor::AutoBlocksStrategyVisibility() const
{
	return CurrentLayout && CurrentLayout->PackingStrategy != ECustomizableObjectTextureLayoutPackingStrategy::Overlay ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility SCustomizableObjectLayoutEditor::AutoBlocksMergeStrategyVisibility() const
{
	const bool bIsVisible = CurrentLayout && 
		CurrentLayout->PackingStrategy != ECustomizableObjectTextureLayoutPackingStrategy::Overlay &&
		CurrentLayout->AutomaticBlocksStrategy == ECustomizableObjectLayoutAutomaticBlocksStrategy::UVIslands;
	
	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility SCustomizableObjectLayoutEditor::FixedStrategyOptionsVisibility() const
{
	return  CurrentLayout && CurrentLayout->PackingStrategy == ECustomizableObjectTextureLayoutPackingStrategy::Fixed ? EVisibility::Visible : EVisibility::Collapsed;
}


EVisibility SCustomizableObjectLayoutEditor::WarningOptionsVisibility() const
{
	return CurrentLayout && CurrentLayout->PackingStrategy != ECustomizableObjectTextureLayoutPackingStrategy::Overlay ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE

