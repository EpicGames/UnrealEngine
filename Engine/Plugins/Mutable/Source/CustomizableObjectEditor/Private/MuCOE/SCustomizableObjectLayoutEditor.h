// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SSpinBox.h"
#include "MuCOE/SCustomizableObjectLayoutGrid.h"

namespace ESelectInfo { enum Type : int; }

class FReferenceCollector;
class ICustomizableObjectInstanceEditor;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class ISlateStyle;
class SWidget;
class STextComboBox;
class SSearchableComboBox;
class SVerticalBox;
class SHorizontalBox;
class FUICommandList;
class SCustomizableObjectLayoutEditor;
class SCustomizableObjectLayoutGrid;
class UCustomizableObjectLayout;
class UTexture2D;

struct FCustomizableObjectLayoutBlock;
struct FGuid;

struct FLayoutEditorMeshSection
{
	TSharedPtr<FString> MeshName;
	TArray<TWeakObjectPtr<UCustomizableObjectLayout>> Layouts;
};

struct FCustomizableObjectLayoutEditorDetailsBuilder
{
	void CustomizeDetails(IDetailLayoutBuilder& Details);

	bool bShowLayoutSelector = false;
	bool bShowPackagingStrategy = false;
	bool bShowAutomaticGenerationSettings = false;
	bool bShowGridSize = false;
	bool bShowMaxGridSize = false;
	bool bShowReductionMethods = false;
	bool bShowWarningSettings = false;

	TSharedPtr<SCustomizableObjectLayoutEditor> LayoutEditor;
};

/**
 * CustomizableObject Editor Preview viewport widget
 */
class SCustomizableObjectLayoutEditor : public SCompoundWidget, public FGCObject
{

	friend FCustomizableObjectLayoutEditorDetailsBuilder;

public:

	DECLARE_DELEGATE(FOnPreUpdateLayout);

	SLATE_BEGIN_ARGS( SCustomizableObjectLayoutEditor ){}
		SLATE_ATTRIBUTE(TArray<FLayoutEditorMeshSection>, MeshSections)
		SLATE_ARGUMENT_DEFAULT(UObject*, Node) { nullptr };
		SLATE_EVENT(FOnPreUpdateLayout, OnPreUpdateLayoutDelegate)
	SLATE_END_ARGS()

	SCustomizableObjectLayoutEditor();

	void Construct(const FArguments& InArgs);

	// FSerializableObject interface
	void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SCustomizableObjectLayoutEditor");
	}
	// End of FSerializableObject interface


	/** Binds commands associated with the viewport client. */
	void BindCommands();

	void SetLayout(UCustomizableObjectLayout* Layout);

	/**
	* UVOverrideLayout parameter can be speicifed to show different UVs in the widget instead of the ones in Layout.
	*/
	void SetUVsOverride(UCustomizableObjectLayout* UVOverrideLayout);

	void UpdateLayout(UCustomizableObjectLayout* Layout);

private:

	FOnPreUpdateLayout OnPreUpdateLayoutDelegate;

	TWeakObjectPtr<UObject> Node;

	TArray<FLayoutEditorMeshSection> MeshSections;
	TArray<TSharedPtr<FString>> MeshSectionNames;

	TArray<TSharedPtr<FString>> UVChannels;

	/** Layout whose blocksa are being edited. */
	TObjectPtr<UCustomizableObjectLayout> CurrentLayout;

	/** If valid, layout use to show the UVs instead of CurrentLayout. */
	TObjectPtr<UCustomizableObjectLayout> UVOverrideLayout;
	
	/** List of available layout grid sizes. */
	TArray<TSharedPtr<FString>> LayoutGridSizes;
	TArray<TSharedPtr<FString>> MaxLayoutGridSizes;

	TSharedPtr<FString> NothingSelectedString;

	struct FAutoBlocksStrategyOption
	{
		ECustomizableObjectLayoutAutomaticBlocksStrategy Value;
		FText Tooltip;
	};
	TArray<TSharedPtr<FString>> AutoBlocksStrategies;
	TArray<FAutoBlocksStrategyOption> AutoBlocksStrategiesOptions;

	struct FAutoBlocksMergeStrategyOption
	{
		ECustomizableObjectLayoutAutomaticBlocksMergeStrategy Value;
		FText Tooltip;
	};
	TArray<TSharedPtr<FString>> AutoBlocksMergeStrategies;
	TArray<FAutoBlocksMergeStrategyOption> AutoBlocksMergeStrategiesOptions;

	/** List of available block reduction methods. */
	TArray<TSharedPtr<FString>> BlockReductionMethods;
	TArray<FText> BlockReductionMethodsTooltips;

	/** List of available layout packing strategies. */
	struct FPackingStrategyOption
	{
		ECustomizableObjectTextureLayoutPackingStrategy Value;
		FText Tooltip;
	};
	TArray<TSharedPtr<FString>> LayoutPackingStrategies;
	TArray<FPackingStrategyOption> LayoutPackingStrategiesOptions;

	/** The list of UI Commands executable */
	TSharedRef<FUICommandList> UICommandList;

	// Layout -------------
	// ComboBox widget to select a column from the NodeTable
	TSharedPtr<STextComboBox> MeshSectionComboBox;
	TSharedPtr<STextComboBox> UVChannelComboBox;


	// ComboBox widget to select a Strategy from the Selected Layout. SSearchableComboBox allows us to set a custom tooltip per option.
	TSharedPtr<SSearchableComboBox> StrategyComboBox;

	TSharedPtr<SSearchableComboBox> AutoBlocksComboBox;
	TSharedPtr<SSearchableComboBox> AutoBlocksMergeComboBox;

	// ComboBox widget to select a Grid Size from the Selected Layout
	TSharedPtr<STextComboBox> GridSizeXComboBox;
	TSharedPtr<STextComboBox> GridSizeYComboBox;

	// ComboBox widget to select a Max Grid Size from the Selected Layout
	TSharedPtr<STextComboBox> MaxGridSizeComboBox;

	// ComboBox widget to select a Reduction Method from the Selected Layout
	TSharedPtr<SSearchableComboBox> ReductionMethodComboBox;

	// Widget to select at which LOD layout vertex warnings will start to be ignored
	TSharedPtr<SSpinBox<int32>> LODSelectorWidget;
	TSharedPtr<STextBlock> LODSelectorTextWidget;

	TSharedPtr<SCustomizableObjectLayoutGrid> LayoutGridWidget;

private:

	/** */
	ELayoutGridMode GetGridMode() const;
	FIntPoint GetGridSize() const;
	void OnBlockChanged(FGuid BlockId, FIntRect Block );
	TArray<FCustomizableObjectLayoutBlock> GetBlocks() const;

	/** Callbacks from the layout block editor. */
	TSharedRef<SWidget> BuildLayoutToolBar();

	void OnAddBlock();
	void OnAddBlockAt(const FIntPoint Min, const FIntPoint Max);
	void OnRemoveBlock();

	/** */
	void OnMeshSectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OnUVChannelChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OnOpenGridSizeComboBox();
	void OnGridSizeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo, bool bIsGridSizeX);
	void OnMaxGridSizeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	void OnAutoBlocksChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OnAutoBlocksMergeChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OnReductionMethodChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OnPackagingStrategyChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	void OnIgnoreErrorsCheckStateChanged(ECheckBoxState NewSelection);
	void OnIgnoreErrorsLODBoxValueChanged(int32 Value);

	void OnResetSelection();

	/** Turn the automatic layout blocks into user-created blocks. */
	void OnConsolidateBlocks();

	/** Sets the block priority from the input text. */
	void OnSetBlockPriority(int32 InValue);

	/** Sets the block reduction symmetry option. */
	void OnSetBlockReductionSymmetry(bool bInValue);

	/** Sets the block reduction ReduceByTwo option. */
	void OnSetBlockReductionByTwo(bool bInValue);

	/** Callback for block mask change. */
	void OnSetBlockMask(UTexture2D* InValue);

	UCustomizableObjectLayout* FindSelectedLayout(TSharedPtr<FString> MeshSection, TSharedPtr<FString> UVChannel);

	void FillLayoutComboBoxOptions();

	TSharedPtr<IToolTip> GenerateInfoToolTip() const;
	
	TSharedRef<SWidget> GenerateAutoBlocksComboBox(TSharedPtr<FString> InItem) const;
	FText GetAutoBlocksName() const;
	FText GetAutoBlocksTooltip() const;	

	TSharedRef<SWidget> GenerateAutoBlocksMergeComboBox(TSharedPtr<FString> InItem) const;
	FText GetAutoBlocksMergeName() const;
	FText GetAutoBlocksMergeTooltip() const;

	TSharedRef<SWidget> GenerateReductionMethodComboBox(TSharedPtr<FString> InItem) const;
	FText GetLayoutReductionMethodName() const;
	FText GetLayoutReductionMethodTooltip() const;

	TSharedRef<SWidget> GenerateLayoutPackagingStrategyComboBox(TSharedPtr<FString> InItem) const;
	FText GetLayoutPackagingStrategyName() const;
	FText GetLayoutPackagingStrategyToolTip() const;

	EVisibility GridSizeVisibility() const;
	EVisibility LayoutOptionsVisibility() const;
	EVisibility AutoBlocksStrategyVisibility() const;
	EVisibility AutoBlocksMergeStrategyVisibility() const;
	EVisibility FixedStrategyOptionsVisibility() const;
	EVisibility WarningOptionsVisibility() const;
	
	const static int32 MaxPossibleGridSize = 128;
};
