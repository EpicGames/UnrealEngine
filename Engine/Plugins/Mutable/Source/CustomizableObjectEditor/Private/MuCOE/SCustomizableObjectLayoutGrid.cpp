// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectLayoutGrid.h"

#include "MuCOE/SStandAloneAssetPicker.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "BatchedElements.h"
#include "CanvasTypes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Rendering/SlateRenderer.h"
#include "RenderingThread.h"
#include "Styling/SlateTypes.h"
#include "UnrealClient.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "RenderGraphBuilder.h"
#include "MuCO/LoadUtils.h"

class FExtender;
class FPaintArgs;
class FRHICommandListImmediate;
class FSlateRect;
class FWidgetStyle;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

/** Simple representation of the backbuffer for drawing UVs. */
class FSlateCanvasRenderTarget : public FRenderTarget
{
public:
	FIntPoint GetSizeXY() const override
	{
		return ViewRect.Size();
	}

	/** Sets the texture that this target renders to */
	void SetRenderTargetTexture(FRDGTexture* InTexture)
	{
		RDGTexture = InTexture;
	}

	FRDGTexture* GetRenderTargetTexture(FRDGBuilder&) const override
	{
		return RDGTexture;
	}

	/** Clears the render target texture */
	void ClearRenderTargetTexture()
	{
		RDGTexture = nullptr;
	}

	/** Sets the viewport rect for the render target */
	void SetViewRect(const FIntRect& InViewRect)
	{
		ViewRect = InViewRect;
	}

	/** Gets the viewport rect for the render target */
	const FIntRect& GetViewRect() const
	{
		return ViewRect;
	}

	/** Sets the clipping rect for the render target */
	void SetClippingRect(const FIntRect& InClippingRect)
	{
		ClippingRect = InClippingRect;
	}

	/** Gets the clipping rect for the render target */
	const FIntRect& GetClippingRect() const
	{
		return ClippingRect;
	}

private:
	FRDGTexture* RDGTexture = nullptr;
	FIntRect ViewRect;
	FIntRect ClippingRect;
};


/** Custom Slate drawing element. Holds a copy of all information required to draw UVs. 
*/
class FUVCanvasDrawer : public ICustomSlateElement
{
public:
	virtual ~FUVCanvasDrawer() override;

	/** Set the canvas area and all required data to paint the UVs.
	 * 
	 * All data will be copied.
	 */
	void Initialize(const FIntRect& InCanvasRect, const FIntRect& InClippingRect, const FVector2D& InOrigin, const FVector2D& InSize, const FIntPoint& InGridSize, const float InCellSize);
	void InitializeDrawingData(const TArray<FVector2f>& InUVLayout, const TArray<FVector2f>& InUnassignedUVs, const TArray<FCustomizableObjectLayoutBlock>& InBlocks, const TArray<FGuid>& InSelectedBlocks);

	/** Sets the layout mode to know what to draw */
	void SetLayoutMode(ELayoutGridMode Mode);

private:

	virtual void Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs) override;

	/** Basic function to draw a block in the canvas */
	void DrawBlock(FBatchedElements* BatchedElements, const FHitProxyId HitProxyId, const FRect2D& BlockRect, FColor Color, UTexture2D* Mask = nullptr);

	/** SlateElement initialized, can Draw during the DrawRenderThread call. */
	bool Initialized = false;

	/** Drawing origin. */
	FVector2D Origin;

	/** Drawing size. */
	FVector2D Size;

	/** Size of the Layout Grid */
	FIntPoint GridSize = FIntPoint(0, 0);

	/** Cell Size */
	float CellSize = 0.0f;

	/** Drawing Data. */
	TArray<FVector2D> UVLayout;
	TArray<FVector2D> UnassignedUVs;
	TArray<FCustomizableObjectLayoutBlock> Blocks;
	TArray<FGuid> SelectedBlocks;

	/** Layout Mode */
	ELayoutGridMode LayoutMode = ELGM_Show;

	FSlateCanvasRenderTarget* RenderTarget = new FSlateCanvasRenderTarget();

	/** Default colors. */
	FColor SelectedBlockColor = FColor(75, 106, 230, 155);
	FColor UnselectedBlockColor = FColor(230, 199, 75, 155);
	FColor AutomaticBlockColor = FColor(125, 125, 125, 125);
};


void SCustomizableObjectLayoutGrid::Construct( const FArguments& InArgs )
{
	GridSize = InArgs._GridSize;
	Blocks = InArgs._Blocks;
	UVLayout = InArgs._UVLayout;
	UnassignedUVLayoutVertices = InArgs._UnassignedUVLayoutVertices;
	Mode = InArgs._Mode;
	BlockChangedDelegate = InArgs._OnBlockChanged;
	SelectionChangedDelegate = InArgs._OnSelectionChanged;
	SelectionColor = InArgs._SelectionColor;
	DeleteBlocksDelegate = InArgs._OnDeleteBlocks;
	AddBlockAtDelegate = InArgs._OnAddBlockAt;
	OnSetBlockPriority = InArgs._OnSetBlockPriority;
	OnSetReduceBlockSymmetrically = InArgs._OnSetReduceBlockSymmetrically;
	OnSetReduceBlockByTwo = InArgs._OnSetReduceBlockByTwo;
	OnSetBlockMask = InArgs._OnSetBlockMask;

	for (int32 BufferIndex=0;BufferIndex< UE_MUTABLE_UI_DRAWBUFFERS; ++BufferIndex)
	{
		UVCanvasDrawers[BufferIndex] = TSharedPtr<FUVCanvasDrawer>(new FUVCanvasDrawer());
	}
}


SCustomizableObjectLayoutGrid::~SCustomizableObjectLayoutGrid()
{
	// UVCanvasDrawer can only be destroyed after drawing the last command
	for (int32 BufferIndex = 0; BufferIndex < UE_MUTABLE_UI_DRAWBUFFERS; ++BufferIndex)
	{
		TSharedPtr<class FUVCanvasDrawer> UVCanvasDrawer = UVCanvasDrawers[BufferIndex];

		ENQUEUE_RENDER_COMMAND(SafeDeletePreviewElement)(
			[UVCanvasDrawer](FRHICommandListImmediate& RHICmdList) mutable
			{
				UVCanvasDrawer.Reset();
			}
		);
	}
}


int32 SCustomizableObjectLayoutGrid::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 RetLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyClippingRect, OutDrawElements, LayerId,InWidgetStyle, bParentEnabled );

	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	// Paint inside the border only. 
	const FVector2D BorderPadding = FVector2D(2,2);
	FPaintGeometry ForegroundPaintGeometry = AllottedGeometry.ToInflatedPaintGeometry( -BorderPadding );
	
	const FIntPoint GridSizePoint = GridSize.Get();
	const float OffsetX = BorderPadding.X;
	const FVector2D AreaSize =  AllottedGeometry.GetLocalSize() - 2.0f * BorderPadding;
	const float GridRatio = float(GridSizePoint.X) / float(GridSizePoint.Y);
	FVector2D Size;
	if ( AreaSize.X/GridRatio > AreaSize.Y )
	{
		Size.Y = AreaSize.Y;
		Size.X = AreaSize.Y*GridRatio;
	}
	else
	{
		Size.X =  AreaSize.X;
		Size.Y =  AreaSize.X/GridRatio;
	}

	FVector2D OldSize = Size;

	double ZoomFactor = PointOfView.GetZoomFactor();
	Size *= ZoomFactor;

	float AuxCellSize = Size.X / GridSizePoint.X;
	
	// Drawing Offsets
	FVector2D Offset = FVector2D((AreaSize - Size).X / 2.0f, 0.0f);
	
	// Drawing Origin
	FVector2D Origin = BorderPadding + Offset + PointOfView.PaddingAmount;

	// Setting Canvas Drawing Rectangles
	FSlateRect SlateCanvasRect = AllottedGeometry.GetLayoutBoundingRect();
	FSlateRect ClippedCanvasRect = SlateCanvasRect.IntersectionWith(MyClippingRect);

	FIntRect CanvasRect(
		FMath::TruncToInt(FMath::Max(0.0f, SlateCanvasRect.Left)),
		FMath::TruncToInt(FMath::Max(0.0f, SlateCanvasRect.Top)),
		FMath::TruncToInt(FMath::Max(0.0f, SlateCanvasRect.Right)),
		FMath::TruncToInt(FMath::Max(0.0f, SlateCanvasRect.Bottom)));

	FIntRect ClippingRect(
		FMath::TruncToInt(FMath::Max(0.0f, ClippedCanvasRect.Left)),
		FMath::TruncToInt(FMath::Max(0.0f, ClippedCanvasRect.Top)),
		FMath::TruncToInt(FMath::Max(0.0f, ClippedCanvasRect.Right)),
		FMath::TruncToInt(FMath::Max(0.0f, ClippedCanvasRect.Bottom)));

	TSharedPtr<class FUVCanvasDrawer> UVCanvasDrawer = UVCanvasDrawers[CurrentDrawBuffer];
	
	ELayoutGridMode GridMode = Mode.Get();
	UVCanvasDrawer->SetLayoutMode(GridMode);
	UVCanvasDrawer->InitializeDrawingData(UVLayout.Get(), UnassignedUVLayoutVertices, Blocks.Get(), SelectedBlocks);
	UVCanvasDrawer->Initialize(CanvasRect, ClippingRect, Origin * AllottedGeometry.Scale, Size * AllottedGeometry.Scale, GridSizePoint, AuxCellSize * AllottedGeometry.Scale);

	FSlateDrawElement::MakeCustom(OutDrawElements, RetLayerId, UVCanvasDrawer);

	const auto MakeYellowSquareLine = [&](const TArray<FVector2D>& Points) -> void
	{
		FSlateDrawElement::MakeLines(OutDrawElements, RetLayerId, AllottedGeometry.ToPaintGeometry(),
			Points, ESlateDrawEffect::None, FColor(250, 230, 43, 255), true, 2.0);
	};

	// Drawing Multi-Selection rect
	if (GridMode == ELGM_Edit && bIsSelecting)
	{
		TArray<FVector2D> SelectionSquarePoints;
		SelectionSquarePoints.SetNum(2);

		FVector2D RectMin = FVector2D(SelectionRect.Min);
		FVector2D RectSize = FVector2D(SelectionRect.Size);

		FVector2D TopLeft = RectMin;
		FVector2D TopRight = RectMin + FVector2D(RectSize.X, 0.0f);
		FVector2D BottomRight = RectMin + RectSize;
		FVector2D BottomLeft = RectMin + FVector2D(0.0f, RectSize.Y);

		SelectionSquarePoints[0] = TopLeft;
		SelectionSquarePoints[1] = TopRight;
		MakeYellowSquareLine(SelectionSquarePoints);

		SelectionSquarePoints[0] = BottomRight;
		MakeYellowSquareLine(SelectionSquarePoints);

		SelectionSquarePoints[1] = BottomLeft;
		MakeYellowSquareLine(SelectionSquarePoints);

		SelectionSquarePoints[0] = TopLeft;
		MakeYellowSquareLine(SelectionSquarePoints);
	}

	RetLayerId++;

	return RetLayerId - 1;
}


void SCustomizableObjectLayoutGrid::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Swap the rendering buffer.
	CurrentDrawBuffer = FMath::Modulo(CurrentDrawBuffer+1, UE_MUTABLE_UI_DRAWBUFFERS);

	const FVector2D BorderPadding = FVector2D(2,2);
	const FVector2D AreaSize =  AllottedGeometry.Size - 2.0f * BorderPadding;
	const float GridRatio = float(GridSize.Get().X)/float(GridSize.Get().Y);
	FVector2D Size;
	if ( AreaSize.X/GridRatio > AreaSize.Y )
	{
		Size.Y = AreaSize.Y;
		Size.X = AreaSize.Y*GridRatio;
	}
	else
	{
		Size.X =  AreaSize.X;
		Size.Y =  AreaSize.X/GridRatio;
	}

	FVector2D OldSize = Size;
	double ZoomFactor = PointOfView.GetZoomFactor();
	Size *= ZoomFactor;

	CellSize = Size.X/GridSize.Get().X;
	FVector2D Offset = FVector2D((AreaSize - Size).X / 2.0f, 0.0f);
	FVector2D Origin = BorderPadding + Offset + PointOfView.PaddingAmount;
	DrawOrigin = Origin;

	BlockRects.Empty();

	const TArray<FCustomizableObjectLayoutBlock>& CurrentBlocks = Blocks.Get();
	for (const FCustomizableObjectLayoutBlock& Block : CurrentBlocks)
	{
		const FVector2f BlockMin(Block.Min);
		const FVector2f BlockMax(Block.Max);

		FBlockWidgetData BlockData;
		BlockData.Rect.Min = FVector2f(Origin) + BlockMin * CellSize + CellSize * 0.1f;
		BlockData.Rect.Size = (BlockMax - BlockMin) * CellSize - CellSize * 0.2f;

		float HandleRectSize = FMath::Log2(float(GridSize.Get().X))/10.0f;
		BlockData.HandleRect.Size = FVector2f(CellSize) * HandleRectSize;
		BlockData.HandleRect.Min = BlockData.Rect.Min + BlockData.Rect.Size - BlockData.HandleRect.Size;

		BlockRects.Add(Block.Id, BlockData);
	}

	// Update selection list
	for (int32 SelectedBlockIndex=0; SelectedBlockIndex<SelectedBlocks.Num();)
	{
		bool bFound = false;
		for (const FCustomizableObjectLayoutBlock& Block : CurrentBlocks)
		{
			if (Block.Id == SelectedBlocks[SelectedBlockIndex])
			{
				bFound = true;
			}
		}

		if ( !bFound )
		{
			SelectedBlocks.RemoveAt(SelectedBlockIndex);
		}
		else
		{
			++SelectedBlockIndex;
		}
	}

	if (bIsSelecting)
	{
		CalculateSelectionRect();
	}

	SCompoundWidget::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );
}


FReply SCustomizableObjectLayoutGrid::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();

	ELayoutGridMode GridMode = Mode.Get();
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && GridMode == ELGM_Edit)
		{
			bHasDragged = false;
			bIsDragging = false;
			bIsResizing = false;

			// To know if we clicked on a block
			bool ClickOnBlock = false;

			FVector2D Pos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			InitSelectionRect = Pos;

			//Reset Selection Rect
			SelectionRect.Size = FVector2f::Zero();
			SelectionRect.Min = FVector2f(Pos);

			// Handles selection must be detected on mouse down
			// We also check if we click on a block
			TArray<FGuid> SelectedBlockHandles;

			for (const FGuid& BlockId : SelectedBlocks)
			{
				if (MouseOnBlock(BlockId, Pos, true))
				{
					SelectedBlockHandles.Add(BlockId);
				}

				if (MouseOnBlock(BlockId, Pos))
				{
					if (SelectedBlocks.Contains(BlockId))
					{
						ClickOnBlock = true;
					}
				}
			}

			if (SelectedBlocks.Num() && ClickOnBlock)
			{
				bIsDragging = true;
				DragStart = Pos;

				if (SelectedBlocks.Num() == 1 && SelectedBlockHandles.Contains(SelectedBlocks[0]))
				{
					bIsResizing = true;
				}
			}
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Mouse position
			FVector2D Pos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			FVector2D CellDelta = (Pos - DrawOrigin) / CellSize;

			// Create context menu
			const bool CloseAfterSelection = true;
			FMenuBuilder MenuBuilder(CloseAfterSelection, NULL, TSharedPtr<FExtender>(), false, &FCoreStyle::Get(), false);

			MenuBuilder.BeginSection("View", LOCTEXT("ViewActionsTitle", "View"));
			{
				FUIAction ResetViewAction(FExecuteAction::CreateSP(this, &SCustomizableObjectLayoutGrid::ResetView));
				MenuBuilder.AddMenuEntry(LOCTEXT("ResetViewLabel", "Reset View"), LOCTEXT("ResetViewLabelTooltip", "Set the view to the unit UV space."), FSlateIcon(), ResetViewAction);
			}
			MenuBuilder.EndSection();

			if (GridMode == ELGM_Edit)
			{
				MenuBuilder.BeginSection("Block Management", LOCTEXT("BlockActionsTitle", "Block Actions"));
				{
					if (SelectedBlocks.Num())
					{
						FUIAction DeleteAction(FExecuteAction::CreateSP(this, &SCustomizableObjectLayoutGrid::DeleteSelectedBlocks));
						MenuBuilder.AddMenuEntry(LOCTEXT("DeleteBlocksLabel", "Delete"), LOCTEXT("DeleteBlocksTooltip", "Delete Selected Blocks"), FSlateIcon(), DeleteAction);

						FUIAction DuplicateAction(FExecuteAction::CreateSP(this, &SCustomizableObjectLayoutGrid::DuplicateBlocks));
						MenuBuilder.AddMenuEntry(LOCTEXT("DuplicateBlocksLabel", "Duplicate"), LOCTEXT("DuplicateBlocksTooltip", "Duplicate Selected Blocks"), FSlateIcon(), DuplicateAction);
					}
					else
					{
						FUIAction AddNewBlockAction(FExecuteAction::CreateSP(this, &SCustomizableObjectLayoutGrid::GenerateNewBlock, CellDelta));
						MenuBuilder.AddMenuEntry(LOCTEXT("AddNewBlockLabel", "Add Block"), LOCTEXT("AddNewBlockTooltip", "Add New Block"), FSlateIcon(), AddNewBlockAction);
					}
				}
				MenuBuilder.EndSection();

				MenuBuilder.BeginSection("Block Properties for Fixed Layout", LOCTEXT("BlockPropertiesFixedTitle", "Block Properties for Fixed Layout"));
				{
					if (SelectedBlocks.Num())
					{
						MenuBuilder.AddWidget(
							SNew(SBox)
							.WidthOverride(125.0f)
							.ToolTipText(LOCTEXT("SetBlockPriority_Tooltip", "Sets the block priority for a Fixed Layout Strategy."))
							[
								SNew(SNumericEntryBox<int32>)
									.MinValue(0)
									.MaxValue(INT_MAX)
									.MaxSliderValue(100)
									.AllowSpin(SelectedBlocks.Num() == 1)
									.Value(this, &SCustomizableObjectLayoutGrid::GetBlockPriorityValue)
									.UndeterminedString(LOCTEXT("MultipleValues", "Multiples Values"))
									.OnValueChanged(this, &SCustomizableObjectLayoutGrid::OnBlockPriorityChanged)
									.EditableTextBoxStyle(&UE_MUTABLE_GET_WIDGETSTYLE<FEditableTextBoxStyle>("NormalEditableTextBox"))
							]
							, FText::FromString("Block Priority"), true);

						MenuBuilder.AddWidget(
							SNew(SBox)
							.WidthOverride(125.0f)
							.ToolTipText(LOCTEXT("SetBlockSymmetry_Tooltip", "If true, this block will be reduced in both axes at the same time in a Fixed Layout Strategy."))
							[
								SNew(SCheckBox)
									.IsChecked(this, &SCustomizableObjectLayoutGrid::GetReductionMethodBoolValue, EFRO_Symmetry)
									.OnCheckStateChanged(this, &SCustomizableObjectLayoutGrid::OnReduceBlockSymmetricallyChanged)
							]
							, FText::FromString("Reduce Symmetrically"), true);

						MenuBuilder.AddWidget(
							SNew(SBox)
							.WidthOverride(125.0f)
							.ToolTipText(LOCTEXT("SetBlockReduceByTwo_Tooltip", "Only for Unitary reduction. If true, this option reduces each time the block by two block units."))
							[
								SNew(SCheckBox)
									.IsChecked(this, &SCustomizableObjectLayoutGrid::GetReductionMethodBoolValue, EFRO_RedyceByTwo)
									.OnCheckStateChanged(this, &SCustomizableObjectLayoutGrid::OnReduceBlockByTwoChanged)
							]
							, FText::FromString("Reduce by Two"), true);
					}
				}
				MenuBuilder.EndSection();


				MenuBuilder.BeginSection("Block Properties for Masks", LOCTEXT("BlockPropertiesMaskTitle", "Block Mask"));
				{
					if (SelectedBlocks.Num())
					{
						MenuBuilder.AddWidget(
							SNew(SBox)
							.WidthOverride(125.0f)
							.ToolTipText(LOCTEXT("SetBlockMask_Tooltip", "Sets the UV mask texture for the block."))
							[
								SNew(SStandAloneAssetPicker)
									.OnAssetSelected(this, &SCustomizableObjectLayoutGrid::OnMaskAssetSelected)
									.OnGetAllowedClasses(FOnGetAllowedClasses::CreateLambda([](TArray<const UClass*>& OutClasses) {OutClasses.Add(UTexture2D::StaticClass()); }))
									.InitialAsset(GetBlockMaskValue())
							]
							, FText::FromString("Block Mask"), true);

						// TODO: Additional properties: color used for preview?
					}
				}
				MenuBuilder.EndSection();
			}

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

			Reply = FReply::Handled();
		}

		else if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
		{
			bIsPadding = true;
			PaddingStart = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		}
	}
	
	if (!Reply.IsEventHandled())
	{
		Reply = SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	return Reply;
}


void SCustomizableObjectLayoutGrid::CloseMenu()
{
	FSlateApplication::Get().DismissAllMenus();
}


void SCustomizableObjectLayoutGrid::OnMaskAssetSelected(const FAssetData& AssetData)
{
	UTexture2D* Mask = Cast<UTexture2D>(UE::Mutable::Private::LoadObject(AssetData));
	if (SelectedBlocks.Num())
	{
		OnSetBlockMask.ExecuteIfBound(Mask);
	}
}



UTexture2D* SCustomizableObjectLayoutGrid::GetBlockMaskValue() const
{
	if (SelectedBlocks.Num())
	{
		TArray<FCustomizableObjectLayoutBlock> CurrentSelectedBlocks;

		for (const FCustomizableObjectLayoutBlock& Block : Blocks.Get())
		{
			if (SelectedBlocks.Contains(Block.Id))
			{
				CurrentSelectedBlocks.Add(Block);
			}
		}

		UTexture2D* BlockMask = CurrentSelectedBlocks[0].Mask;
		bool bSameMask = true;

		for (const FCustomizableObjectLayoutBlock& Block : CurrentSelectedBlocks)
		{
			if (Block.Mask != BlockMask)
			{
				bSameMask = false;
				break;
			}
		}

		if (bSameMask)
		{
			return BlockMask;
		}
	}

	return nullptr;
}


FReply SCustomizableObjectLayoutGrid::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply Reply = FReply::Unhandled();

	ELayoutGridMode GridMode = Mode.Get();

	if ( MouseEvent.GetEffectingButton()==EKeys::LeftMouseButton && GridMode == ELGM_Edit)
	{
		bIsDragging = false;
		bIsResizing = false;

		// Left Shif is pressed for multi selection
		bool bLeftShift = MouseEvent.GetModifierKeys().IsLeftShiftDown();

		// Screen to Widget Position
		FVector2D Pos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		
		// Selection before reset
		TArray<FGuid> OldSelection = SelectedBlocks;

		TArray<FGuid> OldPossibleSelection = PossibleSelectedBlocks;
		PossibleSelectedBlocks.Reset();

		// Reset selection if multi selection is not enabled
		if (GridMode == ELGM_Edit && !bLeftShift && !bHasDragged)
		{
			// Only one selected block allowed in edit mode.
			SelectedBlocks.Reset();
		}

		if (!bIsSelecting)
		{
			if (!bHasDragged)
			{
				// Backward iteration to select the block rendered in front of the rest
				const TArray<FCustomizableObjectLayoutBlock>& CurrentBlocks = Blocks.Get();
				for (int32 i = CurrentBlocks.Num() - 1; i > -1; --i)
				{
					if ( (!CurrentBlocks[i].bIsAutomatic) && MouseOnBlock(CurrentBlocks[i].Id, Pos))
					{
						PossibleSelectedBlocks.Add(CurrentBlocks[i].Id);
					}
				}

				bool bSameSelection = PossibleSelectedBlocks == OldPossibleSelection;

				for (int32 i = 0; i < PossibleSelectedBlocks.Num(); ++i)
				{
					if (bLeftShift)
					{
						if (PossibleSelectedBlocks.Num() == 1)
						{
							if (SelectedBlocks.Contains(PossibleSelectedBlocks[i]))
							{
								SelectedBlocks.Remove(PossibleSelectedBlocks[i]);
							}
							else
							{
								SelectedBlocks.Add(PossibleSelectedBlocks[i]);
								break;
							}
						}
						else
						{
							if (!SelectedBlocks.Contains(PossibleSelectedBlocks[i]))
							{
								SelectedBlocks.Add(PossibleSelectedBlocks[i]);
								break;
							}
						}
					}
					else
					{
						if (OldSelection.Num() == 0)
						{
							SelectedBlocks.Add(PossibleSelectedBlocks[0]);
						}

						if (bSameSelection)
						{
							if (OldSelection.Contains(PossibleSelectedBlocks[i]))
							{
								SelectedBlocks.Remove(PossibleSelectedBlocks[i]);

								if (i == PossibleSelectedBlocks.Num() - 1)
								{
									SelectedBlocks.Add(PossibleSelectedBlocks[0]);
									break;
								}
								else
								{
									SelectedBlocks.Add(PossibleSelectedBlocks[i + 1]);
								}
							}
						}
						else
						{
							if (OldSelection.Contains(PossibleSelectedBlocks[i]) && PossibleSelectedBlocks.Num() > 1)
							{
								SelectedBlocks.Remove(PossibleSelectedBlocks[i]);
							}
							else
							{
								SelectedBlocks.AddUnique(PossibleSelectedBlocks[i]);
								break;
							}
						}
					}
				}
			}
		}
		else
		{
			FBox2D SelectRect(FVector2D(SelectionRect.Min), FVector2D(SelectionRect.Min + SelectionRect.Size) );
			
			const TArray<FCustomizableObjectLayoutBlock>& CurrentBlocks = Blocks.Get();
			for (int32 i = 0; i < CurrentBlocks.Num(); ++i)
			{
				if (CurrentBlocks[i].bIsAutomatic)
				{
					continue;
				}

				FBox2D CurrentBlock(FVector2D(BlockRects[CurrentBlocks[i].Id].Rect.Min), FVector2D(BlockRects[CurrentBlocks[i].Id].Rect.Min + BlockRects[CurrentBlocks[i].Id].Rect.Size));
				
				if (SelectedBlocks.Contains(CurrentBlocks[i].Id))
				{
					if (!SelectRect.Intersect(CurrentBlock) && !bLeftShift)
					{
						SelectedBlocks.Remove(CurrentBlocks[i].Id);
					}
				}
				else
				{
					if (SelectRect.Intersect(CurrentBlock))
					{
						SelectedBlocks.Add(CurrentBlocks[i].Id);
					}
				}
			}
		}

		// Executing selection delegate
		if (OldSelection != SelectedBlocks)
		{
			SelectionChangedDelegate.ExecuteIfBound(SelectedBlocks);
		}

		bHasDragged = false;
		bIsSelecting = false;
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		bIsPadding = false;
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		Reply = FReply::Handled();
	}

	if (!Reply.IsEventHandled())
	{
		Reply = SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	return Reply;
}


FReply SCustomizableObjectLayoutGrid::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	CurrentMousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	ELayoutGridMode GridMode = Mode.Get();

	if(MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && GridMode == ELGM_Edit)
	{
		FVector2D Pos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

		if (bIsDragging && SelectedBlocks.Num())
		{
			FVector2D CellDelta = (Pos - DragStart) / CellSize;

			int32 CellDeltaX = CellDelta.X;
			int32 CellDeltaY = CellDelta.Y;

			DragStart += FVector2D(CellDeltaX * CellSize, CellDeltaY * CellSize);

			if (CellDeltaX || CellDeltaY)
			{
				bHasDragged = true;

				const TArray<FCustomizableObjectLayoutBlock>& CurrentBlocks = Blocks.Get();

				if (!bIsResizing)
				{
					// Bounding box of all selected blocks in grid units.
					FIntRect TotalBlock;
					bool bFirstBlock = true;

					for (const FCustomizableObjectLayoutBlock& B : CurrentBlocks)
					{
						FIntRect Block(B.Min, B.Max);

						if (SelectedBlocks.Contains(B.Id))
						{
							if (bFirstBlock)
							{
								TotalBlock = Block;
								bFirstBlock = false;
							}

							TotalBlock.Min.X = FMath::Min(TotalBlock.Min.X, Block.Min.X);
							TotalBlock.Min.Y = FMath::Min(TotalBlock.Min.Y, Block.Min.Y);
							TotalBlock.Max.X = FMath::Max(TotalBlock.Max.X, Block.Max.X);
							TotalBlock.Max.Y = FMath::Max(TotalBlock.Max.Y, Block.Max.Y);
						}
					}

					FIntPoint Grid = GridSize.Get();
					FIntRect BlockMovement = TotalBlock;

					// Block movement in layouts is restricted to the positive quadrant.
					//BlockMovement.Min.X = FMath::Max(0, FMath::Min(TotalBlock.Min.X + CellDeltaX, Grid.X - TotalBlock.Size().X));
					//BlockMovement.Min.Y = FMath::Max(0, FMath::Min(TotalBlock.Min.Y + CellDeltaY, Grid.Y - TotalBlock.Size().Y));
					BlockMovement.Min.X = FMath::Max(0, TotalBlock.Min.X + CellDeltaX);
					BlockMovement.Min.Y = FMath::Max(0, TotalBlock.Min.Y + CellDeltaY);
					//BlockMovement.Min.X = TotalBlock.Min.X + CellDeltaX;
					//BlockMovement.Min.Y = TotalBlock.Min.Y + CellDeltaY;

					BlockMovement.Max = BlockMovement.Min + TotalBlock.Size();

					FIntRect AddMovement = BlockMovement - TotalBlock;

					for (const FCustomizableObjectLayoutBlock& B : CurrentBlocks)
					{
						if (SelectedBlocks.Find(B.Id) != INDEX_NONE)
						{
							FIntRect ResultBlock(B.Min, B.Max);
							ResultBlock.Max += AddMovement.Max;
							ResultBlock.Min += AddMovement.Min;

							BlockChangedDelegate.ExecuteIfBound(B.Id, ResultBlock);
						}
					}
				}
				else
				{
					for (const FCustomizableObjectLayoutBlock& B : CurrentBlocks)
					{
						FIntRect Block;
						for (const FGuid& Id : SelectedBlocks)
						{
							if (B.Id != Id)
							{
								continue;
							}

							Block.Min = B.Min;
							Block.Max = B.Max;

							FIntRect InitialBlock = Block;

							FIntPoint Grid = GridSize.Get();

							FIntPoint BlockSize = Block.Size();
							// Block movement in layouts is restricted to the positive quadrant.
							//Block.Max.X = FMath::Max(Block.Min.X + 1, FMath::Min(Block.Max.X + CellDeltaX, Grid.X));
							//Block.Max.Y = FMath::Max(Block.Min.Y + 1, FMath::Min(Block.Max.Y + CellDeltaY, Grid.Y));
							Block.Max.X = Block.Max.X + CellDeltaX;
							Block.Max.Y = Block.Max.Y + CellDeltaY;

							if (Block != InitialBlock)
							{
								BlockChangedDelegate.ExecuteIfBound(Id, Block);
							}

							break;
						}
					}
				}
			}
		}

		if (!bIsSelecting && !bIsDragging)
		{
			bool ClickOnBlock = false;

			for (const FGuid& BlockId : SelectedBlocks)
			{
				if (MouseOnBlock(BlockId, Pos))
				{
					if (SelectedBlocks.Contains(BlockId))
					{
						ClickOnBlock = true;
					}
				}
			}

			int32 MovementSensitivity = 4;
			FVector2D MouseDiference = InitSelectionRect - Pos;
			MouseDiference = MouseDiference.GetAbs();

			if (!ClickOnBlock && (MouseDiference.X > MovementSensitivity || MouseDiference.Y > MovementSensitivity))
			{
				bHasDragged = true;
				bIsSelecting = true;
			}
		}
	}
	
	if (!bIsDragging && !bIsResizing && SelectedBlocks.Num()==1 && GridMode == ELGM_Edit)
	{
		const TArray<FCustomizableObjectLayoutBlock>& CurrentBlocks = Blocks.Get();
		for (int32 i = CurrentBlocks.Num() - 1; i > -1; --i)
		{
			// Check for new created blocks
			if (BlockRects.Contains(CurrentBlocks[i].Id) && SelectedBlocks.Contains(CurrentBlocks[i].Id))
			{
				FVector2D Pos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
				if (MouseOnBlock(CurrentBlocks[i].Id, Pos, true))
				{
					bIsResizeCursor = true;
					break;
				}
			}

			bIsResizeCursor = false;
		}
	}

	// In case we lose focus
	if (bIsPadding)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton))
		{
			FVector2D Pos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			PointOfView.PaddingAmount += Pos - PaddingStart;
			PaddingStart = Pos;
		}
		else
		{
			bIsPadding = false;
		}
	}

	if (!MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		bIsSelecting = false;
		bIsDragging = false;

		if (bIsResizing)
		{
			bIsResizeCursor = false;
			bIsResizing = false;
		}
	}

	return SCompoundWidget::OnMouseMove( MyGeometry, MouseEvent );
}


FReply SCustomizableObjectLayoutGrid::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	constexpr int32 MinZoomLevel = -2;
	constexpr int32 MaxZoomLevel = 3;

	{
		double OldZoomFactor = PointOfView.GetZoomFactor();
		FVector2D UnzoomedPadding = PointOfView.PaddingAmount * (1.0f / OldZoomFactor);

		if (MouseEvent.GetWheelDelta() > 0)
		{
			int32 NewZoomLevel = FMath::Min(PointOfView.Zoom + 1, MaxZoomLevel);
			if (PointOfView.Zoom != NewZoomLevel)
			{
				//FVector2D GridCenter = DrawOrigin + (FVector2D((float)GridSize.Get().X, (float)GridSize.Get().Y) / 2.0f) * CellSize;
				//DistanceFromOrigin = CurrentMousePosition - GridCenter;

				PointOfView.Zoom = NewZoomLevel;
			}
		}
		else
		{
			int32 NewZoomLevel = FMath::Max(PointOfView.Zoom - 1, MinZoomLevel);
			if (PointOfView.Zoom != NewZoomLevel)
			{
				//DistanceFromOrigin = FVector2D::Zero();
				//PaddingAmount = FVector2D::Zero();

				PointOfView.Zoom = NewZoomLevel;
			}
		}

		double NewZoomFactor = PointOfView.GetZoomFactor();
		FVector2D RezoomedPadding = UnzoomedPadding * NewZoomFactor;
		PointOfView.PaddingAmount = RezoomedPadding;

		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Mouse, true);
	}
}


FReply SCustomizableObjectLayoutGrid::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	ELayoutGridMode GridMode = Mode.Get();
	if (GridMode != ELGM_Edit)
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}
	
	if (InKeyEvent.IsLeftControlDown())
	{
		if (InKeyEvent.GetKey() == EKeys::D)
		{
			DuplicateBlocks();
		}
		else if (InKeyEvent.GetKey() == EKeys::N)
		{
			FVector2D MouseToCellPosition = (CurrentMousePosition - DrawOrigin) / CellSize;
			GenerateNewBlock(MouseToCellPosition);
		}
		else if (InKeyEvent.GetKey() == EKeys::F)
		{
			SetBlockSizeToMax();
		}
	}

	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		DeleteSelectedBlocks();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}


FCursorReply SCustomizableObjectLayoutGrid::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (bIsResizeCursor)
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeSouthEast);
	}
	else
	{
		return FCursorReply::Cursor(EMouseCursor::Default);
	}
}


FVector2D SCustomizableObjectLayoutGrid::ComputeDesiredSize(float NotUsed) const
{
	return FVector2D(200.0f, 200.0f);
}


void SCustomizableObjectLayoutGrid::SetSelectedBlock(FGuid block )
{
	SelectedBlocks.Reset();
	SelectedBlocks.Add( block );
}


const TArray<FGuid>& SCustomizableObjectLayoutGrid::GetSelectedBlocks() const
{
	return SelectedBlocks;
}


void SCustomizableObjectLayoutGrid::DeleteSelectedBlocks()
{
	DeleteBlocksDelegate.ExecuteIfBound();
}


void SCustomizableObjectLayoutGrid::ResetView()
{
	PointOfView.Zoom = 1;
	PointOfView.PaddingAmount = { 0,0 };
}


void SCustomizableObjectLayoutGrid::GenerateNewBlock(FVector2D MousePosition)
{
	if (MousePosition.X > 0 && MousePosition.Y > 0 && MousePosition.X < GridSize.Get().X && MousePosition.Y < GridSize.Get().Y)
	{
		FIntPoint Min = FIntPoint(MousePosition.X, MousePosition.Y);
		FIntPoint Max = Min + FIntPoint(1, 1);

		AddBlockAtDelegate.ExecuteIfBound(Min, Max);

		SelectedBlocks.Add(Blocks.Get().Last().Id);
	}
}


void SCustomizableObjectLayoutGrid::DuplicateBlocks()
{
	if (SelectedBlocks.Num())
	{
		for (const FCustomizableObjectLayoutBlock& Block : Blocks.Get())
		{
			if (SelectedBlocks.Find(Block.Id) != INDEX_NONE)
			{
				AddBlockAtDelegate.ExecuteIfBound(Block.Min, Block.Max);
			}
		}
	}
}


void SCustomizableObjectLayoutGrid::SetBlockSizeToMax()
{
	if (SelectedBlocks.Num())
	{
		for (const FCustomizableObjectLayoutBlock& Block : Blocks.Get())
		{
			if (SelectedBlocks.Find(Block.Id) != INDEX_NONE)
			{
				FIntRect FinalBlock;

				FinalBlock.Min = FIntPoint(0, 0);
				FinalBlock.Max = GridSize.Get();

				BlockChangedDelegate.ExecuteIfBound(Block.Id, FinalBlock);
			}
		}
	}
}


void SCustomizableObjectLayoutGrid::CalculateSelectionRect()
{
	if (InitSelectionRect.X <= CurrentMousePosition.X)
	{
		if (InitSelectionRect.Y <= CurrentMousePosition.Y)
		{
			SelectionRect.Min = FVector2f(InitSelectionRect);
			SelectionRect.Size = FVector2f(CurrentMousePosition - InitSelectionRect);
		}
		else
		{
			SelectionRect.Min = FVector2f(InitSelectionRect.X, CurrentMousePosition.Y);

			FVector2f AuxVector(CurrentMousePosition.X, InitSelectionRect.Y);
			SelectionRect.Size = AuxVector - SelectionRect.Min;
		}
	}
	else
	{
		if (InitSelectionRect.Y <= CurrentMousePosition.Y)
		{
			SelectionRect.Min = FVector2f(CurrentMousePosition.X, InitSelectionRect.Y);

			FVector2f AuxVector(InitSelectionRect.X, CurrentMousePosition.Y);
			SelectionRect.Size = AuxVector - SelectionRect.Min;
		}
		else
		{
			SelectionRect.Min = FVector2f(CurrentMousePosition);
			SelectionRect.Size = FVector2f(InitSelectionRect - CurrentMousePosition);
		}
	}

}


void SCustomizableObjectLayoutGrid::SetBlocks(const FIntPoint& InGridSize, const TArray<FCustomizableObjectLayoutBlock>& InBlocks)
{
	GridSize = InGridSize;
	Blocks = InBlocks;
}


bool SCustomizableObjectLayoutGrid::MouseOnBlock(FGuid BlockId, FVector2D MousePosition, bool CheckResizeBlock) const
{
	FVector2f Min, Max;
	if (CheckResizeBlock)
	{
		Min = BlockRects[BlockId].HandleRect.Min;
		Max = Min + BlockRects[BlockId].HandleRect.Size;
	}
	else
	{
		Min = BlockRects[BlockId].Rect.Min;
		Max = Min + BlockRects[BlockId].Rect.Size;
	}

	if (MousePosition.X > Min.X && MousePosition.X<Max.X && MousePosition.Y>Min.Y && MousePosition.Y < Max.Y)
	{
		return true;
	}

	return false;
}


TOptional<int32> SCustomizableObjectLayoutGrid::GetBlockPriorityValue() const
{
	if (SelectedBlocks.Num())
	{
		TArray<FCustomizableObjectLayoutBlock> CurrentSelectedBlocks;

		for (const FCustomizableObjectLayoutBlock& Block : Blocks.Get())
		{
			if (SelectedBlocks.Contains(Block.Id))
			{
				CurrentSelectedBlocks.Add(Block);
			}
		}

		int32 BlockPriority = CurrentSelectedBlocks[0].Priority;
		bool bSamePriority = true;

		for (const FCustomizableObjectLayoutBlock& Block : CurrentSelectedBlocks)
		{
			if (Block.Priority != BlockPriority)
			{
				bSamePriority = false;
				break;
			}
		}

		if (bSamePriority)
		{
			return BlockPriority;
		}
	}

	return TOptional<int32>();
}


ECheckBoxState SCustomizableObjectLayoutGrid::GetReductionMethodBoolValue(EFixedReductionOptions Option) const
{
	if (SelectedBlocks.Num())
	{
		TArray<FCustomizableObjectLayoutBlock> CurrentSelectedBlocks;

		// Getting all selected blocks
		for (const FCustomizableObjectLayoutBlock& Block : Blocks.Get())
		{
			if (SelectedBlocks.Contains(Block.Id))
			{
				CurrentSelectedBlocks.Add(Block);
			}
		}

		switch (Option)
		{
		case EFRO_Symmetry:
		{
			const bool bReduceBothAxes = CurrentSelectedBlocks[0].bReduceBothAxes;

			// If one or more blocks have a different value than the rest of selected blocks return Undetermined
			for (const FCustomizableObjectLayoutBlock& Block : CurrentSelectedBlocks)
			{
				if (Block.bReduceBothAxes != bReduceBothAxes)
				{
					return ECheckBoxState::Undetermined;
				}
			}

			return bReduceBothAxes ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		case EFRO_RedyceByTwo:
		{
			const bool bReduceByTwo = CurrentSelectedBlocks[0].bReduceByTwo;

			// If one or more blocks have a different value than the rest of selected blocks return Undetermined
			for (const FCustomizableObjectLayoutBlock& Block : CurrentSelectedBlocks)
			{
				if (Block.bReduceByTwo != bReduceByTwo)
				{
					return ECheckBoxState::Undetermined;
				}
			}

			return bReduceByTwo ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		default:
			break;
		}
	}

	return ECheckBoxState::Undetermined;
}


void SCustomizableObjectLayoutGrid::OnBlockPriorityChanged(int32 InValue)
{
	if (SelectedBlocks.Num())
	{
		OnSetBlockPriority.ExecuteIfBound(InValue);
	}
}


void SCustomizableObjectLayoutGrid::OnReduceBlockSymmetricallyChanged(ECheckBoxState InCheckboxState)
{
	if (SelectedBlocks.Num())
	{
		OnSetReduceBlockSymmetrically.ExecuteIfBound(InCheckboxState == ECheckBoxState::Checked);
	}
}


void SCustomizableObjectLayoutGrid::OnReduceBlockByTwoChanged(ECheckBoxState InCheckboxState)
{
	if (SelectedBlocks.Num())
	{
		OnSetReduceBlockByTwo.ExecuteIfBound(InCheckboxState == ECheckBoxState::Checked);
	}
}


// Canvas Drawer --------------------------------------------------------------


FUVCanvasDrawer::~FUVCanvasDrawer()
{
	delete RenderTarget;
}


void FUVCanvasDrawer::Initialize(const FIntRect& InCanvasRect, const FIntRect& InClippingRect, const FVector2D& InOrigin, const FVector2D& InSize, const FIntPoint& InGridSize, const float InCellSize)
{
	Initialized = InCanvasRect.Size().X > 0 && InCanvasRect.Size().Y > 0;
	if (Initialized)
	{
		RenderTarget->SetViewRect(InCanvasRect);
		RenderTarget->SetClippingRect(InClippingRect);

		Origin = InOrigin;
		Size = InSize;
		CellSize = InCellSize;
		GridSize = InGridSize;
	}
}


void FUVCanvasDrawer::InitializeDrawingData(const TArray<FVector2f>& InUVLayout, const TArray<FVector2f>& InUnassignedUVs, const TArray<FCustomizableObjectLayoutBlock>& InBlocks, const TArray<FGuid>& InSelectedBlocks)
{
	Blocks = InBlocks;
	SelectedBlocks = InSelectedBlocks;

	// Convert data
	UVLayout.SetNum(InUVLayout.Num());
	for (int32 Index = 0; Index < InUVLayout.Num(); ++Index)
	{
		UVLayout[Index] = FVector2D(InUVLayout[Index]);
	}

	UnassignedUVs.SetNum(InUnassignedUVs.Num());
	for (int32 Index = 0; Index < UnassignedUVs.Num(); ++Index)
	{
		UnassignedUVs[Index] = FVector2D(InUnassignedUVs[Index]);
	}
}


void FUVCanvasDrawer::SetLayoutMode(ELayoutGridMode Mode)
{
	LayoutMode = Mode;
}


void FUVCanvasDrawer::Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs)
{
	if (!Initialized)
	{
		return;
	}

	RenderTarget->SetRenderTargetTexture(Inputs.OutputTexture);

	FCanvas& Canvas = *GraphBuilder.AllocObject<FCanvas>(RenderTarget, nullptr, FGameTime(), GMaxRHIFeatureLevel);

	Canvas.SetRenderTargetRect(RenderTarget->GetViewRect());
	Canvas.SetRenderTargetScissorRect(RenderTarget->GetClippingRect());

	// Number of tiles to render in each axis including the unit tile.
	constexpr int32 NumTiles = 4;

	// Num Lines
	const uint32 NumAxisLines = 2;
	const uint32 NumEdges = UVLayout.Num() / 2;
	const uint32 NumUnitGridLines = GridSize.X + GridSize.Y + 2;
	const uint32 NumTileLines = NumTiles + NumTiles + 2;
	const uint32 NumExtendedGridLines = NumTiles * (GridSize.X + GridSize.Y) + 2;
	const uint32 NumUnasignedUVs = UnassignedUVs.Num() * 4;

	// Num Vertices and Triangles including blocks and unit tile quad.
	const uint32 RectCount = (LayoutMode == ELayoutGridMode::ELGM_Edit) ? Blocks.Num() * 2 : Blocks.Num();
	const uint32 NumVertices = RectCount*4 + 4;
	const uint32 NumTriangles = RectCount*2 + 2;

	const int32 NumLines = NumAxisLines + NumEdges + NumUnitGridLines + NumTileLines + NumExtendedGridLines + NumUnasignedUVs;

	FBatchedElements* BatchedElements = Canvas.GetBatchedElements(FCanvas::ET_Line);
	BatchedElements->AddReserveLines(NumLines);
	BatchedElements->AddReserveVertices(NumVertices);
	BatchedElements->AddReserveTriangles(NumTriangles, GWhiteTexture, ESimpleElementBlendMode::SE_BLEND_Translucent);

	// Color Definitions
	const FColor ExtendedGridLineColor = FColor(150, 150, 150, 32);
	const FColor GridLineColor = FColor(150, 150, 150, 64);
	const FColor TileLineColor = FColor(200, 200, 150, 48);
	const FColor UVLineColor = FColor(255, 255, 255, 255);
	const FColor UnassignedUVsColor = FColor::Yellow;
	const FColor ResizeBlockColor = FColor(255, 96, 96, 255);
	const FColor UnitTileBlockColor = FColor(96, 96, 96, 128);

	const FHitProxyId HitProxyId = Canvas.GetHitProxyId();

	int32 FullTilesSize = NumTiles * Size.X;
	//FVector2D TilesOrigin = Origin - FVector2D(FullTilesSize / 2, FullTilesSize / 2);
	FVector2D TilesOrigin = Origin;

	// Create lines as pairs of points
	FVector LinePoints[2];

	// Unit Tile
	{
		FRect2D TileBlock;
		TileBlock.Min = FVector2f(Origin);
		TileBlock.Size = FVector2f(Size);

		DrawBlock(BatchedElements, HitProxyId, TileBlock, UnitTileBlockColor, nullptr);
	}


	// Drawing Extended Grid
	if (LayoutMode!=ELGM_ShowUVsOnly)
	{
		// Vertical Lines
		for (int32 LineIndex = 0; LineIndex < NumTiles*GridSize.X + 1; LineIndex++)
		{
			LinePoints[0] = FVector(TilesOrigin.X + LineIndex * CellSize, TilesOrigin.Y, 0.0f);
			LinePoints[1] = FVector(TilesOrigin.X + LineIndex * CellSize, TilesOrigin.Y + FullTilesSize, 0.0f);

			BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], ExtendedGridLineColor, HitProxyId, 2.0f);
		}

		// Drawing Unit Grid Horizontal Lines
		for (int32 LineIndex = 0; LineIndex < NumTiles*GridSize.Y + 1; LineIndex++)
		{
			LinePoints[0] = FVector(TilesOrigin.X, TilesOrigin.Y + LineIndex * CellSize, 0.0f);
			LinePoints[1] = FVector(TilesOrigin.X + FullTilesSize, TilesOrigin.Y + LineIndex * CellSize, 0.0f);

			BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], ExtendedGridLineColor, HitProxyId, 2.0f);
		}
	}

	// Drawing Unit Grid
	if (LayoutMode != ELGM_ShowUVsOnly)
	{
		// Vertical Lines
		for (int32 LineIndex = 0; LineIndex < GridSize.X + 1; LineIndex++)
		{
			LinePoints[0] = FVector(Origin.X + LineIndex * CellSize, Origin.Y, 0.0f);
			LinePoints[1] = FVector(Origin.X + LineIndex * CellSize, Origin.Y + Size.Y, 0.0f);

			BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], GridLineColor, HitProxyId, 2.0f);
		}

		// Drawing Unit Grid Horizontal Lines
		for (int32 LineIndex = 0; LineIndex < GridSize.Y + 1; LineIndex++)
		{
			LinePoints[0] = FVector(Origin.X, Origin.Y + LineIndex * CellSize, 0.0f);
			LinePoints[1] = FVector(Origin.X + Size.X, Origin.Y + LineIndex * CellSize, 0.0f);

			BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], GridLineColor, HitProxyId, 2.0f);
		}
	}

	// Drawing Tiles 
	{
		// Vertical Lines
		for (int32 LineIndex = 0; LineIndex < NumTiles + 1; LineIndex++)
		{
			LinePoints[0] = FVector(TilesOrigin.X + LineIndex * Size.X, TilesOrigin.Y, 0.0f);
			LinePoints[1] = FVector(TilesOrigin.X + LineIndex * Size.X, TilesOrigin.Y + FullTilesSize, 0.0f);

			BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], TileLineColor, HitProxyId, 2.0f);
		}

		// Horizontal Lines
		for (int32 LineIndex = 0; LineIndex < NumTiles + 1; LineIndex++)
		{
			LinePoints[0] = FVector(TilesOrigin.X, TilesOrigin.Y + LineIndex * Size.Y, 0.0f);
			LinePoints[1] = FVector(TilesOrigin.X + FullTilesSize, TilesOrigin.Y + LineIndex * Size.Y, 0.0f);

			BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], TileLineColor, HitProxyId, 2.0f);
		}
	}

	// Axes
	{
		LinePoints[0] = FVector(TilesOrigin.X, Origin.Y, 0.0f);
		LinePoints[1] = FVector(TilesOrigin.X + FullTilesSize, Origin.Y, 0.0f);
		BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], FColor(255, 150, 150, 200), HitProxyId, 2.0f);

		LinePoints[0] = FVector(Origin.X, TilesOrigin.Y, 0.0f);
		LinePoints[1] = FVector(Origin.X, TilesOrigin.Y + FullTilesSize, 0.0f);
		BatchedElements->AddTranslucentLine(LinePoints[0], LinePoints[1], FColor(150, 255, 150, 200), HitProxyId, 2.0f);
	}


	// Drawing UV Lines
	for (uint32 LineIndex = 0; LineIndex < NumEdges; ++LineIndex)
	{
		LinePoints[0] = FVector(Origin + UVLayout[LineIndex * 2 + 0] * Size, 0.0f);
		LinePoints[1] = FVector(Origin + UVLayout[LineIndex * 2 + 1] * Size, 0.0f);

		BatchedElements->AddLine(LinePoints[0], LinePoints[1], UVLineColor, HitProxyId);
	}

	// Drawing Unassigned UVs
	const FVector2D CrossSize = Size * 0.01;
	for (const FVector2d& Vertex : UnassignedUVs)
	{
		LinePoints[0] = FVector(Origin + FVector2D(Vertex) * Size + FVector2D(CrossSize), 0.0f);
		LinePoints[1] = FVector(Origin + FVector2D(Vertex) * Size - FVector2D(CrossSize) * FVector2D(1.0f, -1.0f), 0.0f);
		BatchedElements->AddLine(LinePoints[0], LinePoints[1], UVLineColor, HitProxyId);

		LinePoints[0] = FVector(Origin + FVector2D(Vertex) * Size - FVector2D(CrossSize), 0.0f);
		BatchedElements->AddLine(LinePoints[0], LinePoints[1], UVLineColor, HitProxyId);

		LinePoints[1] = FVector(Origin + FVector2D(Vertex) * Size + FVector2D(CrossSize) * FVector2D(1.0f, -1.0f), 0.0f);
		BatchedElements->AddLine(LinePoints[0], LinePoints[1], UVLineColor, HitProxyId);

		LinePoints[0] = FVector(Origin + FVector2D(Vertex) * Size + FVector2D(CrossSize), 0.0f);
		BatchedElements->AddLine(LinePoints[0], LinePoints[1], UVLineColor, HitProxyId);
	}

	// Drawing Blocks
	if (LayoutMode != ELGM_ShowUVsOnly)
	{
		for (const FCustomizableObjectLayoutBlock& Block : Blocks)
		{
			FColor BlockColor = AutomaticBlockColor;
			if (!Block.bIsAutomatic)
			{
				BlockColor = SelectedBlocks.Contains(Block.Id) ? SelectedBlockColor : UnselectedBlockColor;
			}


			const FVector2f BlockMin(Block.Min);
			const FVector2f BlockMax(Block.Max);

			// Selection Block
			FRect2D BlockRect;
			BlockRect.Min = FVector2f(Origin) + BlockMin * CellSize + CellSize * 0.1f;
			BlockRect.Size = (BlockMax - BlockMin) * CellSize - CellSize * 0.2f;

			DrawBlock(BatchedElements, HitProxyId, BlockRect, BlockColor, Block.Mask.Get());

			if (LayoutMode == ELayoutGridMode::ELGM_Edit && !Block.bIsAutomatic)
			{
				// Resize Block
				FRect2D ResizeBlock;;
				float HandleRectSize = FMath::Log2(float(GridSize.X)) / 10.0f;
				ResizeBlock.Size = FVector2f(CellSize) * HandleRectSize;
				ResizeBlock.Min = BlockRect.Min + BlockRect.Size - ResizeBlock.Size;

				DrawBlock(BatchedElements, HitProxyId, ResizeBlock, ResizeBlockColor);
			}
		}
	}

	Canvas.Flush_RenderThread(GraphBuilder, true);

	RenderTarget->ClearRenderTargetTexture();
}


void FUVCanvasDrawer::DrawBlock(FBatchedElements* BatchedElements, const FHitProxyId HitProxyId, const FRect2D& BlockRect, FColor Color, UTexture2D* Mask)
{
	// Vertex positions
	FVector4 Vert0(BlockRect.Min.X, BlockRect.Min.Y, 0, 1);
	FVector4 Vert1(BlockRect.Min.X, BlockRect.Min.Y + BlockRect.Size.Y, 0, 1);
	FVector4 Vert2(BlockRect.Min.X + BlockRect.Size.X, BlockRect.Min.Y, 0, 1);
	FVector4 Vert3(BlockRect.Min.X + BlockRect.Size.X, BlockRect.Min.Y + BlockRect.Size.Y, 0, 1);

	auto VertexToMaskUVs = [this](const FVector4& V)
		{
			FVector2D Result = ( FVector2D(V.X,V.Y) - Origin ) / (FVector2D(CellSize) * FVector2D(GridSize.X, GridSize.X));
			// TODO Modulo doesn't work with cross-tile blocks: use tiling.
			Result.X = FMath::Fmod(Result.X, 1.0);
			Result.Y = FMath::Fmod(Result.Y, 1.0);
			return Result;
		};

	// Brush Paint triangle
	{
		int32 V0 = BatchedElements->AddVertex(Vert0, VertexToMaskUVs(Vert0), Color, HitProxyId);
		int32 V1 = BatchedElements->AddVertex(Vert1, VertexToMaskUVs(Vert1), Color, HitProxyId);
		int32 V2 = BatchedElements->AddVertex(Vert2, VertexToMaskUVs(Vert2), Color, HitProxyId);
		int32 V3 = BatchedElements->AddVertex(Vert3, VertexToMaskUVs(Vert3), Color, HitProxyId);

		EBlendMode Mode = EBlendMode::BLEND_Translucent;
		BatchedElements->AddTriangle(V0, V1, V2, GWhiteTexture, Mode);
		BatchedElements->AddTriangle(V1, V3, V2, GWhiteTexture, Mode);

		if (Mask)
		{
			Mode = EBlendMode::BLEND_Additive;
			FTexture* Texture = Mask->GetResource();
			BatchedElements->AddTriangle(V0, V1, V2, Texture, Mode);
			BatchedElements->AddTriangle(V1, V3, V2, Texture, Mode);
		}
	}

	// Drawing Outline to selected Blocks
	if (Color == SelectedBlockColor)
	{
		BatchedElements->AddLine(Vert0, Vert1, UnselectedBlockColor, HitProxyId, 4.0f);
		BatchedElements->AddLine(Vert1, Vert3, UnselectedBlockColor, HitProxyId, 4.0f);
		BatchedElements->AddLine(Vert3, Vert2, UnselectedBlockColor, HitProxyId, 4.0f);
		BatchedElements->AddLine(Vert2, Vert0, UnselectedBlockColor, HitProxyId, 4.0f);
	}
}

#undef LOCTEXT_NAMESPACE
