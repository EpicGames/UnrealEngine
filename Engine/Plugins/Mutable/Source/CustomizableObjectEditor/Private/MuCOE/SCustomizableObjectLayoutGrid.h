// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/CustomizableObjectLayout.h"
#include "Widgets/SCompoundWidget.h"

#define UE_MUTABLE_UI_DRAWBUFFERS 2

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class UTexture2D;
struct FGeometry;
struct FGuid;
struct FKeyEvent;
struct FPointerEvent;

enum class ECheckBoxState : uint8;

struct FRect2D
{
	FVector2f Min;
	FVector2f Size;
};

typedef enum
{
	ELGM_Show,
	ELGM_Edit,
	ELGM_ShowUVsOnly,
} ELayoutGridMode;

struct FBlockWidgetData
{
	FRect2D Rect;
	FRect2D HandleRect;
};


enum EFixedReductionOptions
{
	EFRO_Symmetry,
	EFRO_RedyceByTwo
};


class SCustomizableObjectLayoutGrid : public SCompoundWidget
{

public:
	DECLARE_DELEGATE_TwoParams(FBlockChangedDelegate, FGuid /*BlockId*/, FIntRect /*Block*/);
	DECLARE_DELEGATE_OneParam(FBlockSelectionChangedDelegate, const TArray<FGuid>& );
	DECLARE_DELEGATE(FDeleteBlockDelegate);
	DECLARE_DELEGATE_TwoParams(FAddBlockAtDelegate, FIntPoint, FIntPoint);
	DECLARE_DELEGATE_OneParam(FSetBlockPriority, int32);
	DECLARE_DELEGATE_OneParam(FSetReduceBlockSymmetrically, bool);
	DECLARE_DELEGATE_OneParam(FSetReduceBlockByTwo, bool);
	DECLARE_DELEGATE_OneParam(FSetBlockMask, UTexture2D*);

	SLATE_BEGIN_ARGS( SCustomizableObjectLayoutGrid ){}

		SLATE_ATTRIBUTE(ELayoutGridMode, Mode)
		SLATE_ATTRIBUTE( FIntPoint, GridSize )
		SLATE_ATTRIBUTE(TArray<FCustomizableObjectLayoutBlock>, Blocks)
		SLATE_ATTRIBUTE( TArray<FVector2f>, UVLayout )
		SLATE_ARGUMENT( TArray<FVector2f>, UnassignedUVLayoutVertices )
		SLATE_ARGUMENT( FColor, SelectionColor )
		SLATE_EVENT( FBlockChangedDelegate, OnBlockChanged )
		SLATE_EVENT( FBlockSelectionChangedDelegate, OnSelectionChanged )
		SLATE_EVENT(FDeleteBlockDelegate, OnDeleteBlocks)
		SLATE_EVENT(FAddBlockAtDelegate, OnAddBlockAt)
		SLATE_EVENT(FSetBlockPriority, OnSetBlockPriority)
		SLATE_EVENT(FSetReduceBlockSymmetrically, OnSetReduceBlockSymmetrically)
		SLATE_EVENT(FSetReduceBlockByTwo, OnSetReduceBlockByTwo)
		SLATE_EVENT(FSetBlockMask, OnSetBlockMask)

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );
	virtual ~SCustomizableObjectLayoutGrid() override;

	// SWidgetInterface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	// Own interface

	/** Set the currently selected block */
	void SetSelectedBlock( FGuid block );

	/** */
	const TArray<FGuid>& GetSelectedBlocks() const;

	/** Calls the delegate to delete the selected blocks */
	void DeleteSelectedBlocks();

	/** Reset the view zoom and pan to show the unit UV space. */
	void ResetView();

	/** Generates a new block at mouse position */
	void GenerateNewBlock(FVector2D MousePosition);

	/** Duplicates the selected blocks */
	void DuplicateBlocks();

	/** Sets the size of the selected blocks to the size of the Grid */
	void SetBlockSizeToMax();

	void CalculateSelectionRect();

	FColor SelectionColor;

	/** Set the grid and blocks to show in the widget. */
	void SetBlocks( const FIntPoint& GridSize, const TArray<FCustomizableObjectLayoutBlock>& Blocks);

	struct FPointOfView
	{
		/** Amount of padding since start dragging */
		FVector2D PaddingAmount = FVector2D::Zero();

		/** Level of zoom */
		int32 Zoom = 1;

		/** */
		double GetZoomFactor() const
		{
			return FMath::Pow(2.0, double(Zoom - 1));
		}

	};

	/** Current point of view. */
	FPointOfView PointOfView;

private:

	/** Gets the priority value of the selected blocks */
	TOptional<int32> GetBlockPriorityValue() const;

	/** Callback when the priority of a block changes */
	void OnBlockPriorityChanged(int32 InValue);

	/** Callback when symmetry block reduction option changes */
	void OnReduceBlockSymmetricallyChanged(ECheckBoxState InCheckboxState);

	/** Callback when ReduceByTwo block reduction option changes */
	void OnReduceBlockByTwoChanged(ECheckBoxState InCheckboxState);

	/** Gets block reduction value of the selected blocks */
	ECheckBoxState GetReductionMethodBoolValue(EFixedReductionOptions Option) const;

	bool MouseOnBlock(FGuid BlockId, FVector2D MousePosition, bool CheckResizeBlock = false) const;

	/** Callback for an asset selection popup menu close. */
	void CloseMenu();

	/** Callback for an actual mask asset selection in the context menu. */
	void OnMaskAssetSelected(const FAssetData& AssetData);

	UTexture2D* GetBlockMaskValue() const;

private:

	/** A delegate to report block changes */
	FBlockChangedDelegate BlockChangedDelegate;
	FBlockSelectionChangedDelegate SelectionChangedDelegate;
	FDeleteBlockDelegate DeleteBlocksDelegate;
	FAddBlockAtDelegate AddBlockAtDelegate;
	FSetBlockPriority OnSetBlockPriority;
	FSetReduceBlockSymmetrically OnSetReduceBlockSymmetrically;
	FSetReduceBlockByTwo OnSetReduceBlockByTwo;
	FSetBlockMask OnSetBlockMask;

	/** Size of the grid in blocks */
	TAttribute<FIntPoint> GridSize;

	/** Array with all the editable blocks of the layout */
	TAttribute< TArray<FCustomizableObjectLayoutBlock> > Blocks;

	/** Array with all the UVs to draw in the layout */
	TAttribute< TArray<FVector2f> > UVLayout;

	/** Array with all the unassigned UVs */
	TArray<FVector2f> UnassignedUVLayoutVertices;

	/** Layout mode */
	TAttribute<ELayoutGridMode> Mode = ELGM_Show;

	float CellSize = 0.0f;

	/** Map to relate Block ids with blocks data */
	TMap<FGuid,FBlockWidgetData> BlockRects;

	/** Interaction status. */
	TArray<FGuid> SelectedBlocks;
	TArray<FGuid> PossibleSelectedBlocks;

	/** Booleans needed for the Block Management */
	/** Indicates when we have dragged the mouse after click */
	bool bHasDragged = false;

	/** Indicates when we are dragging the mouse */
	bool bIsDragging = false;
	
	/** Indicates when we are resizing a block */
	bool bIsResizing = false;
	
	/** Indicates when we have to change the mouse cursor */
	bool bIsResizeCursor = false;
	
	/** Indicates when we are making a selection */
	bool bIsSelecting = false;
	
	/** Indicates when we are padding */
	bool bIsPadding = false;

	/** Position where the drag started */
	FVector2D DragStart;

	/** Position where the layout grid starts to be drawn */
	FVector2D DrawOrigin;

	/** Position where the padding started */
	FVector2D PaddingStart;

	/** Selection Rectangle */
	FRect2D SelectionRect;

	/** Position where the Selection Rectangle started */
	FVector2D InitSelectionRect;

	/** Current mouse position */
	FVector2D CurrentMousePosition;

	/** Custom Slate drawing element. Used to improve the UVs drawing performance. 
	* * This is multi-buffered because it is read and written simultaneously from the render and the game threads.
	*/
	int32 CurrentDrawBuffer = 0;
	TSharedPtr<class FUVCanvasDrawer> UVCanvasDrawers[UE_MUTABLE_UI_DRAWBUFFERS];
};
