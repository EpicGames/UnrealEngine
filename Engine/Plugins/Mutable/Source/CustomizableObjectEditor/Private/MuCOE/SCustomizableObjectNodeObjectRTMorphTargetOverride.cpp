// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectNodeObjectRTMorphTargetOverride.h"

#include "Framework/Views/TableViewMetadata.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "SEnumCombo.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/SListView.h"
#include "DetailLayoutBuilder.h"

class ITableRow;
class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


namespace CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal
{

namespace ViewRows 
{
    static const FName NameColumn(TEXT("Name"));
}

namespace SkeletalMeshListViewWidget_Internal
{
   
void SRowWidget::Construct(
        const FArguments& InArgs, 
        const TSharedRef<STableViewBase>& OwnerTableView)
{
    Item = InArgs._Item;
    MainWidget = InArgs._MainWidget;

    check(Item.IsValid());

    SMultiColumnTableRow< TSharedPtr<FItemInfo> >::Construct(
            FSuperRowType::FArguments(), 
            OwnerTableView);
}

int32 SRowWidget::OverrideCurrentValue() const
{
	TSharedPtr<MainWidgetType> PinnedMainWidget = MainWidget.Pin();

    if (!PinnedMainWidget.IsValid())
    {
        return static_cast<int32>(ECustomizableObjectSelectionOverride::NoOverride);
    }

    ECustomizableObjectSelectionOverride* const EnumPtr = GetItemEnumPtr(PinnedMainWidget->Node);

    return EnumPtr 
        ? static_cast<int32>(*EnumPtr)
        : static_cast<int32>(ECustomizableObjectSelectionOverride::NoOverride);
}

TSharedRef<SWidget> SRowWidget::GenerateWidgetForColumn(const FName& ColumnName)
{
    using ThisType = TDecay<decltype(*this)>::Type;

    if (ColumnName == ViewRows::NameColumn)
    {
        return
            SNew(SVerticalBox)
            + SVerticalBox::Slot().AutoHeight().VAlign(VAlign_Center)
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot().Padding(4,0,0,0).VAlign(VAlign_Center)
                [
                    SNew(STextBlock).Text(FText::FromName(Item->SkeletalMeshName))
                        .Font(IDetailLayoutBuilder::GetDetailFont())
                ]
                +SHorizontalBox::Slot().Padding(4,0,0,0).HAlign(HAlign_Right)
                [
                    SNew(SEnumComboBox, StaticEnum<ECustomizableObjectSelectionOverride>())
                    .CurrentValue(this, &ThisType::OverrideCurrentValue)
                    .OnEnumSelectionChanged(this, &ThisType::OnEnumSelectionChanged)
                    .Font(IDetailLayoutBuilder::GetDetailFont())
                ]
            ];
    }

    return SNullWidget::NullWidget;
}

ECustomizableObjectSelectionOverride* SRowWidget::GetItemEnumPtr(UCustomizableObjectNodeObject* Node) const
{
    if (!Node)
    {
        return nullptr; 
    }

    TArray<FRealTimeMorphSelectionOverride>& Overrides = Node->RealTimeMorphSelectionOverrides;

    FRealTimeMorphSelectionOverride* MorphFound = Overrides.FindByPredicate( 
            [MorphName = Item->MorphName](FRealTimeMorphSelectionOverride& O) { return O.MorphName == MorphName; });

    if (!MorphFound)
    {
        return nullptr;
    }

    FSkeletalMeshMorphTargetOverride* SkeletalMeshOverride = MorphFound->SkeletalMeshes.FindByPredicate(
            [SkeletalMeshName = Item->SkeletalMeshName](FSkeletalMeshMorphTargetOverride& O) 
            { 
                return O.SkeletalMeshName == SkeletalMeshName; 
            });
    
    if (!SkeletalMeshOverride)
    {
        return nullptr;
    }

    return &SkeletalMeshOverride->SelectionOverride;
}

void SRowWidget::OnEnumSelectionChanged(int32 InEnumValue, ESelectInfo::Type)
{
	TSharedPtr<MainWidgetType> PinnedMainWidget = MainWidget.Pin();

    if (!PinnedMainWidget.IsValid())
    {
        return;
    }

	ECustomizableObjectSelectionOverride* const EnumPtr = GetItemEnumPtr(PinnedMainWidget->Node);

    if (EnumPtr)
    {
        const FScopedTransaction Transaction(LOCTEXT("ChangedRealTimeMorphTarget", "Changed Real Time Morph Target"));
        PinnedMainWidget->Node->Modify();
        *EnumPtr = static_cast<ECustomizableObjectSelectionOverride>(InEnumValue);
    }
}

FItemInfo::FItemInfo( 
            const FName& InMorphName, 
            const FName& InSkeletalMeshName)
    : MorphName(InMorphName)
    , SkeletalMeshName(InSkeletalMeshName)
{
}


} //namespace SkeletalMeshListViewWidget_Internal

void SSkeletalMeshListViewWidget::Construct(const FArguments& InArgs)
{
    MorphName = InArgs._MorphName;
    MainWidget = InArgs._MainWidget;
    UpdateWidget();
}

void SSkeletalMeshListViewWidget::UpdateWidget()
{
    using ThisType = TDecay<decltype(*this)>::Type;

    PopulateItems();
    
    ChildSlot
    [
        //SNew(SBorder)
        //[
            SNew(SVerticalBox)
            +SVerticalBox::Slot()
            [
                SAssignNew(View, SListView< TSharedPtr<ItemType> >)
                .ListItemsSource(&Items)
                .OnGenerateRow(this, &ThisType::OnGenerateRow)
                .SelectionMode(ESelectionMode::None)
                .IsFocusable(true)
                .HeaderRow
                (
                    SNew(SHeaderRow)
                    + SHeaderRow::Column(ViewRows::NameColumn)
                    .DefaultLabel(LOCTEXT("SkeletalMeshLabel", "Skeletal Meshes"))
                ) 
            ]
        //]
    ];

    View->RequestListRefresh();
}

TSharedRef<ITableRow> SSkeletalMeshListViewWidget::OnGenerateRow( 
        TSharedPtr<SSkeletalMeshListViewWidget::ItemType> InItem, 
        const TSharedRef<STableViewBase>& OwnerTable)
{
    check(InItem.IsValid());

    namespace Internal = SkeletalMeshListViewWidget_Internal;

    return SNew(Internal::SRowWidget, OwnerTable)
        .Item(InItem)
        .MainWidget(MainWidget);
}

void SSkeletalMeshListViewWidget::PopulateItems()
{
    namespace Internal = SkeletalMeshListViewWidget_Internal;
    
    Items.Reset();

	TSharedPtr<MainWidgetType> PinnedMainWidget = MainWidget.Pin();

    if (!(PinnedMainWidget.IsValid() && PinnedMainWidget->Node))
    {
        return;
    } 

    FRealTimeMorphSelectionOverride* Found = PinnedMainWidget->Node->RealTimeMorphSelectionOverrides.FindByPredicate(
            [this](const FRealTimeMorphSelectionOverride& O) { return O.MorphName == MorphName; });

    if (!Found)
    {
        return; 
    }

    const FString& FilterText = PinnedMainWidget->SearchBoxFilter;
   
    // If the morph filter passes, all skeletal meshes are displayed. 
    const bool bMorphFilterPass = 
            FilterText.IsEmpty() || 
            MorphName.ToString().Contains(FilterText);

    for (const FSkeletalMeshMorphTargetOverride& SkeletalMeshOverride : Found->SkeletalMeshes)
    {
        const bool bFilterPass =
            bMorphFilterPass || SkeletalMeshOverride.SkeletalMeshName.ToString().Contains(FilterText);

        if (bFilterPass)
        {
            Items.Emplace(MakeShareable(new Internal::FItemInfo(MorphName, SkeletalMeshOverride.SkeletalMeshName)));
        }
    } 
}

FItemInfo::FItemInfo(const FName& InName)
    : Name(InName)
{
}

void SRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
    Item = InArgs._Item;
    MainWidget = InArgs._MainWidget;

    check(Item.IsValid());

    SMultiColumnTableRow< TSharedPtr<FItemInfo> >::Construct(
            FSuperRowType::FArguments(), 
            OwnerTableView);
}

int32 SRowWidget::SelectionCurrentValue() const
{
	TSharedPtr<MainWidgetType> PinnedMainWidget = MainWidget.Pin();

    if (!(PinnedMainWidget.IsValid() && PinnedMainWidget->Node))
    {
        return static_cast<int32>(ECustomizableObjectSelectionOverride::NoOverride);
    }

	FRealTimeMorphSelectionOverride* Override = PinnedMainWidget->Node->RealTimeMorphSelectionOverrides.FindByPredicate(
            [this](const FRealTimeMorphSelectionOverride& O){ return O.MorphName == Item->Name; });

    return Override
            ? static_cast<int32>(Override->SelectionOverride) 
            : static_cast<int32>(ECustomizableObjectSelectionOverride::NoOverride);
}

TSharedRef<SWidget> SRowWidget::GenerateWidgetForColumn(const FName& ColumnName)
{
    using ThisType = TDecay<decltype(*this)>::Type;

    if (ColumnName == ViewRows::NameColumn)
    {
        return 
            SNew(SBorder)
            [ 
                SNew(SVerticalBox)
                + SVerticalBox::Slot().AutoHeight().VAlign(VAlign_Center)
                [
                    SNew(SExpandableArea)
                    .BorderImage(FAppStyle::GetBrush("DetailsView.GridLine"))
                    .InitiallyCollapsed(true)
                    .HeaderContent()
                    [
                    SNew(SHorizontalBox)
                    + SHorizontalBox::Slot().VAlign(VAlign_Center)
                    [
                        SNew(STextBlock)
                        .Text(FText::FromName(Item->Name))
                        .Font(IDetailLayoutBuilder::GetDetailFont())
                    ]
                    + SHorizontalBox::Slot().HAlign(HAlign_Right)
                    [
                        SNew(SEnumComboBox, StaticEnum<ECustomizableObjectSelectionOverride>())
                        .CurrentValue(this, &ThisType::SelectionCurrentValue)
                        .OnEnumSelectionChanged(this, &ThisType::OnEnumSelectionChanged)
                        .Font(IDetailLayoutBuilder::GetDetailFont())
                    ]
                    ]
                    .BodyContent()
                    [
                    SNew(SSkeletalMeshListViewWidget)
                    .MorphName(Item->Name)
                    .MainWidget(MainWidget)
                    ]
                ]
            ];
    }

    return SNullWidget::NullWidget;
}

void SRowWidget::OnEnumSelectionChanged(int32 InEnumValue, ESelectInfo::Type)
{
	ECustomizableObjectSelectionOverride EnumValue = static_cast<ECustomizableObjectSelectionOverride>(InEnumValue);
    
	TSharedPtr<MainWidgetType> PinnedMainWidget = MainWidget.Pin();

    if (!(PinnedMainWidget.IsValid() && PinnedMainWidget->Node))
    {
        return;
    }

	FRealTimeMorphSelectionOverride* Found = PinnedMainWidget->Node->RealTimeMorphSelectionOverrides.FindByPredicate(
            [this](const FRealTimeMorphSelectionOverride& O) { return O.MorphName == Item->Name; });
   
    if (!Found)
    {
        return;
    } 

    const FScopedTransaction Transaction(LOCTEXT("ChangedRealTimeMorphTarget", "Changed Real Time Morph Target"));
    PinnedMainWidget->Node->Modify();
    Found->SelectionOverride = EnumValue;
}

} //namespace CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal

void SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::Construct(const FArguments& InArgs)
{
    Node = InArgs._Node;

    UpdateWidget();
}

void SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::AddReferencedObjects(
        FReferenceCollector& Collector)
{
    if (Node)
    {
        Collector.AddReferencedObject(Node);
    }   
}

void SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::UpdateWidget()
{
    namespace Internal = CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal;
    
    using ThisType = TDecay<decltype(*this)>::Type;
    
    ChildSlot
    [
        SNew(SVerticalBox)
        + SVerticalBox::Slot()
        .AutoHeight()
        [
            SAssignNew(SearchBox, SSearchBox)
            .SelectAllTextWhenFocused(true)
            .OnTextChanged(this, &ThisType::OnFilterTextChanged)
            .OnTextCommitted(this, &ThisType::OnFilterTextCommitted)
        ]
        +SVerticalBox::Slot()
        .FillHeight(1.0f)
        [
            SAssignNew(View, ViewType)
            .ListItemsSource(&Items)
            .OnGenerateRow(this, &ThisType::OnGenerateRow)
            .SelectionMode(ESelectionMode::None)
            //.ScrollbarVisibility(EVisibility::Visible)
            //.IsFocusable(true)
            .HeaderRow
            (
                SNew(SHeaderRow)
                + SHeaderRow::Column(Internal::ViewRows::NameColumn)
                .DefaultLabel(LOCTEXT("MorphNameLabel", "Morph Targets"))
            )
        ]
    ];

    SearchBoxFilter.Empty();
    PopulateItems();

    View->RequestListRefresh();
}

void SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::PopulateItems()
{
    namespace Internal = CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal;
    
    Items.Reset();
    
    if (!Node)
    {
        return;
    }

    const TArray<FRealTimeMorphSelectionOverride>& MorphTargetsOverrides = Node->RealTimeMorphSelectionOverrides;
    for (const FRealTimeMorphSelectionOverride& Override : MorphTargetsOverrides)
    {
        FName MorphName = Override.MorphName;

        const bool bSearchBoxContainsSkeletalMeshName = [&]()
            {
                for (const FSkeletalMeshMorphTargetOverride& SkeletalMeshOverride : Override.SkeletalMeshes)
                {
                    if (SkeletalMeshOverride.SkeletalMeshName.ToString().Contains(SearchBoxFilter))
                    {
                        return true;
                    }
                }

                return false;
            }();

        const bool bFilterEntryPass = 
                SearchBoxFilter.IsEmpty()                      ||
                MorphName.ToString().Contains(SearchBoxFilter) ||
                bSearchBoxContainsSkeletalMeshName;

        if (bFilterEntryPass)
        {
            Items.Emplace(MakeShareable(new Internal::FItemInfo(MorphName)));
        }
    }

    // Sort alphabetically
    Items.Sort([](
            const TSharedPtr<Internal::FItemInfo>& A, 
            const TSharedPtr<Internal::FItemInfo>& B)
    {
        return A->Name.ToString().Compare(B->Name.ToString()) < 0;
    });
}

TSharedRef<ITableRow> SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::OnGenerateRow(
        TSharedPtr<SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::ItemType> InItem, 
        const TSharedRef<STableViewBase>& OwnerTable)
{
    check(InItem.IsValid());

    using ThisType = TDecay<decltype(*this)>::Type;
    
    namespace Internal = 
            CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal;

    return SNew(Internal::SRowWidget, OwnerTable)
        .Item(InItem)
        .MainWidget(TWeakPtr<ThisType>(SharedThis(this)));
}

void SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::OnFilterTextChanged(
        const FText& SearchText)
{
    SearchBoxFilter = SearchText.ToString();
    PopulateItems();
    View->RequestListRefresh();
}

void SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride::OnFilterTextCommitted(
        const FText& SearchText, 
        ETextCommit::Type CommitInfo)
{
    if (CommitInfo == ETextCommit::OnEnter)
    {
        SearchBoxFilter = SearchText.ToString();
        PopulateItems();
        View->RequestListRefresh();
    }
}

#undef LOCTEXT_NAMESPACE

