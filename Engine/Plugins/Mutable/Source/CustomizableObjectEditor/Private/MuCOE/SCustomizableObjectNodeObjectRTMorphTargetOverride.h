// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableRow.h"

class FReferenceCollector;
class FText;
class ITableRow;
class SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride;
class SSearchBox;
class STableViewBase;
class SWidget;
class UCustomizableObjectNodeObject;

enum class ECustomizableObjectSelectionOverride : uint8;

namespace CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal
{

using MainWidgetType = SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride;

class SSkeletalMeshListViewWidget;
namespace SkeletalMeshListViewWidget_Internal
{

class SRowWidget;

class FItemInfo
{
    friend SSkeletalMeshListViewWidget;
    friend SRowWidget; 

    // Keep the information needed to loacte the NodeObject Morph Entry.
    // For robustness keep names so we don't depend on the order or life
    // of the observed element.
    FName MorphName;
    FName SkeletalMeshName;

    FItemInfo( 
            const FName& InMorphName, 
            const FName& InSkeletalMeshName);
};

class SRowWidget : public SMultiColumnTableRow< TSharedPtr<FItemInfo> >
{
    friend SSkeletalMeshListViewWidget;

    SLATE_BEGIN_ARGS(SRowWidget) {}
        SLATE_ARGUMENT(TSharedPtr<FItemInfo>, Item)
        SLATE_ARGUMENT(TWeakPtr<MainWidgetType>, MainWidget)
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);
    virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

    void OnEnumSelectionChanged(int32 InEnumValue, ESelectInfo::Type);
    int32 OverrideCurrentValue() const;

    ECustomizableObjectSelectionOverride* GetItemEnumPtr(UCustomizableObjectNodeObject* Node) const;

    TSharedPtr<FItemInfo> Item;
    TWeakPtr<MainWidgetType> MainWidget;

};

} // namespace SkeletalMeshListViewWidget_Internal

class SSkeletalMeshListViewWidget : public SCompoundWidget
{
    SLATE_BEGIN_ARGS(SSkeletalMeshListViewWidget) {}
        SLATE_ARGUMENT(FName, MorphName)
        SLATE_ARGUMENT(TWeakPtr<MainWidgetType>, MainWidget)
    SLATE_END_ARGS()

    ~SSkeletalMeshListViewWidget()= default;

    using ItemType = SkeletalMeshListViewWidget_Internal::FItemInfo;

    FName MorphName; 
    TWeakPtr<MainWidgetType> MainWidget;

    TSharedPtr< SListView< TSharedPtr<ItemType> > > View;
    TArray< TSharedPtr<ItemType> > Items;

    void Construct(const FArguments& InArgs);

    TSharedRef<ITableRow> OnGenerateRow(
            TSharedPtr<ItemType> InItem, 
            const TSharedRef<STableViewBase>& OwnerTable);

    void UpdateWidget();
    void PopulateItems();
};

class SRowWidget;

using FChildItemInfo = SkeletalMeshListViewWidget_Internal::FItemInfo;
using SChildRowWidget = SkeletalMeshListViewWidget_Internal::SRowWidget;

class FItemInfo
{
    friend SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride;
    friend SRowWidget;

    FItemInfo(const FName& InName); 

    FName Name;
};

class SRowWidget : public SMultiColumnTableRow< TSharedPtr<FItemInfo> > 
{ 
public:
    SLATE_BEGIN_ARGS(SRowWidget) {}
        SLATE_ARGUMENT(TSharedPtr<FItemInfo>, Item)
        SLATE_ARGUMENT(TWeakPtr<MainWidgetType>, MainWidget)
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

    virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

    void OnEnumSelectionChanged(int32 InEnumValue, ESelectInfo::Type);
    int32 SelectionCurrentValue() const;
private:
    TSharedPtr<FItemInfo> Item;
    TWeakPtr<MainWidgetType> MainWidget;
};

} //namespace CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal

class SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride : public SCompoundWidget, public FGCObject
{
    using ItemType = 
            CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal::FItemInfo;
    using ChildItemType =
            CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal::FChildItemInfo;

    using RowWidgetType = 
            CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal::SRowWidget;
    using ChildRowWidgetType = 
            CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal::SChildRowWidget;

    using SSkeletalMeshListViewWidget = 
            CustomizableObjectNodeSkeletalMeshRTMorphTargetOverride_Internal::SSkeletalMeshListViewWidget;

    using ViewType = SListView<TSharedPtr<ItemType>>; 
    //using ViewType = typename STreeView<TSharedPtr<ItemType>>; 

    // Child widgets will access data from the main widget 
    friend RowWidgetType;
    friend ChildRowWidgetType;
    friend SSkeletalMeshListViewWidget;

public:

    SLATE_BEGIN_ARGS(SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride) {}
        SLATE_ARGUMENT(UCustomizableObjectNodeObject*, Node)
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs);
    ~SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride() = default;

    void AddReferencedObjects(FReferenceCollector& Collector) override;

    void UpdateWidget();
    void PopulateItems();

    TSharedRef<ITableRow> OnGenerateRow(
            TSharedPtr<ItemType> InItem, 
            const TSharedRef<STableViewBase>& OwnerTable);

    void OnFilterTextChanged(const FText& SearchText);
    void OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo);

	virtual FString GetReferencerName() const override
	{
		return TEXT("SCustomizableObjectNodeSkeletalMeshRTMorphTargetOverride");
	}

private:
    /** Pointer to the Current selected Node SkeletalMesh */
		TObjectPtr<UCustomizableObjectNodeObject> Node = nullptr;

    /** Widget List of the SkeletalMesh MorphTargets */
    TArray< TSharedPtr<ItemType> > Items;
    TSharedPtr< ViewType > View;

    /** SerachBox for the morphs List */
    TSharedPtr<SSearchBox> SearchBox;

    /** Text filter of the Search Box */
    FString SearchBoxFilter;

};
