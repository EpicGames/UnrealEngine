// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectNodeSkeletalMeshRTMorphSelector.h"

#include "Engine/SkeletalMesh.h"
#include "Framework/Views/TableViewMetadata.h"
#include "MuCOE/Nodes/CustomizableObjectNodeSkeletalMesh.h"
#include "ScopedTransaction.h"
#include "MuCO/LoadUtils.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"

class ITableRow;
class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


namespace CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal
{

FRowItemInfo::FRowItemInfo(const FName& InName)
    : Name(InName)
{
}

namespace ViewRows 
{
    static const FName NameColumn(TEXT("Name"));
    static const FName SelectionColumn(TEXT("Selected"));
}

void SRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
    Item = InArgs._Item;
    OwnerWidget = InArgs._OwnerWidget;

    check(Item.IsValid());

    SMultiColumnTableRow< TSharedPtr<FRowItemInfo> >::Construct(
            FSuperRowType::FArguments(), 
            OwnerTableView);
}

TSharedRef<SWidget> SRowWidget::GenerateWidgetForColumn(const FName& ColumnName)
{
    using ThisType = TDecay<decltype(*this)>::Type;

    if (ColumnName == ViewRows::NameColumn)
    {
        return SNew(SVerticalBox)
            + SVerticalBox::Slot().AutoHeight().Padding(4).VAlign(VAlign_Center)
            [
                SNew(STextBlock).Text(FText::FromName(Item->Name))
            ];
    }
    else if (ColumnName == ViewRows::SelectionColumn)
    {
        return  SNew(SVerticalBox)
            + SVerticalBox::Slot().AutoHeight().Padding(4).VAlign(VAlign_Center).HAlign(HAlign_Center)
            [
                SNew(SCheckBox)
                .OnCheckStateChanged(this, &ThisType::OnMorphSelectionCheckStateChanged)
                .IsChecked(this, &ThisType::IsMorphSelected)
            ];
    }

    return SNullWidget::NullWidget;
}

void SRowWidget::OnMorphSelectionCheckStateChanged(ECheckBoxState CheckBoxState)
{
    TSharedPtr<OwnerWidgetType> PinnedOwner = OwnerWidget.Pin();

    if (!(PinnedOwner.IsValid() && PinnedOwner->Node))
    {
        return;
    }

    const FScopedTransaction Transaction(LOCTEXT("ChangeSelectMorphTransaction", "Change Select Morph"));
    PinnedOwner->Node->Modify();
    
    if (CheckBoxState == ECheckBoxState::Checked)
    {
        PinnedOwner->Node->UsedRealTimeMorphTargetNames.AddUnique(Item->Name.ToString());
    }
    else if (CheckBoxState == ECheckBoxState::Unchecked)
    {
       PinnedOwner->Node->UsedRealTimeMorphTargetNames.Remove(Item->Name.ToString());
    }
}

ECheckBoxState SRowWidget::IsMorphSelected() const
{
	TSharedPtr<OwnerWidgetType> PinnedOwner = OwnerWidget.Pin();

    if (!PinnedOwner.IsValid())
    {
        return ECheckBoxState::Unchecked;
    }

    int32 FoundIndex = PinnedOwner->Node->UsedRealTimeMorphTargetNames.Find(Item->Name.ToString());

    return FoundIndex == INDEX_NONE 
            ? ECheckBoxState::Unchecked 
            : ECheckBoxState::Checked; 
}

} //namespace CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal

void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::Construct(const FArguments& InArgs)
{
    Node = InArgs._Node;
    UpdateWidget();
}


void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::AddReferencedObjects(
        FReferenceCollector& Collector)
{
    if (Node)
    {
        Collector.AddReferencedObject(Node);
    }   
}

void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::UpdateWidget()
{
    SearchBoxFilter.Empty();
    GenerateMorphsInfoList();

    using ThisType = TDecay<decltype(*this)>::Type;

    namespace Internal = CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal;

    ChildSlot
    [
        SNew(SVerticalBox)
        + SVerticalBox::Slot()
        .AutoHeight()
        [
            SAssignNew(SearchBox, SSearchBox)
            .SelectAllTextWhenFocused(true)
            .OnTextChanged(this, &ThisType::OnFilterTextChanged)
            .OnTextCommitted(this, &ThisType::OnFilterTextCommitted)
        ]

        + SVerticalBox::Slot()
        .Padding(0.0f, 5.0f, 0.0f, 5.0f)
        .AutoHeight()
        [
            SNew(SHorizontalBox)
            +SHorizontalBox::Slot()
            [
                SNew(STextBlock)
                .Text(LOCTEXT("UseAllText","Select All Morphs"))
            ]
            +SHorizontalBox::Slot()
            [
                SNew(SCheckBox)
                .OnCheckStateChanged(this, &ThisType::OnSelectAllCheckStateChanged)
                .IsChecked(this, &ThisType::IsSelectAllChecked)
            ]
        ]

        +SVerticalBox::Slot()
        [
            SAssignNew(ListWidget, ListViewType)
	        .IsEnabled(this, &ThisType::IsRuntimeMorphsListEnabled)
            .ListItemsSource(&Items)
            .OnGenerateRow(this, &ThisType::GenerateRow)
            .SelectionMode(ESelectionMode::Single)
            .IsFocusable(true)
            .HeaderRow
            (
                SNew(SHeaderRow)
                + SHeaderRow::Column(Internal::ViewRows::NameColumn)
                .DefaultLabel(LOCTEXT("MorphNameLabelSelector", "Name"))

                + SHeaderRow::Column(Internal::ViewRows::SelectionColumn)
                .DefaultLabel(LOCTEXT("SelectionLabelSelector", "Selection"))
            )
        ]
    ];

    ListWidget->RequestListRefresh();
}


void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::GenerateMorphsInfoList()
{
    Items.Reset();

    const USkeletalMesh* SkeletalMesh = UE::Mutable::Private::LoadObject(Node->SkeletalMesh);
    if (!SkeletalMesh)
    {
        return;
    }

    namespace Internal = CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal;
    
	const TArray<UMorphTarget*>& MorphTargets = SkeletalMesh->GetMorphTargets();
    for (const UMorphTarget* MorphTarget : MorphTargets)
    {
        FName MorphName = MorphTarget->GetFName(); 
        if (SearchBoxFilter.IsEmpty() || MorphName.ToString().Contains(SearchBoxFilter))
        {
            Items.Emplace(MakeShareable(new Internal::FRowItemInfo(MorphName)));
        }
    }

    // Sort parameter alphabetically
    Items.Sort([](
            const TSharedPtr<Internal::FRowItemInfo>& A, 
            const TSharedPtr<Internal::FRowItemInfo>& B)
    {
        return A->Name.ToString().Compare(B->Name.ToString()) < 0;
    });
}


TSharedRef<ITableRow> SCustomizableObjectNodeSkeletalMeshRTMorphSelector::GenerateRow(
    TSharedPtr<SCustomizableObjectNodeSkeletalMeshRTMorphSelector::ItemType> InItem, 
    const TSharedRef<STableViewBase>& OwnerTable)
{
    check(InItem.IsValid());

    using SRowWidget = 
            CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal::SRowWidget;

    return SNew(SRowWidget, OwnerTable)
        .Item(InItem)
        .OwnerWidget(SharedThis(this));
}


bool SCustomizableObjectNodeSkeletalMeshRTMorphSelector::IsRuntimeMorphsListEnabled() const
{
	return Node ? !Node->bUseAllRealTimeMorphs : true;
}


ECheckBoxState SCustomizableObjectNodeSkeletalMeshRTMorphSelector::IsSelectAllChecked() const
{
    if (!Node)
    {
        return ECheckBoxState::Unchecked;
    }

    return Node->bUseAllRealTimeMorphs 
            ? ECheckBoxState::Checked 
            : ECheckBoxState::Unchecked;
}


void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::OnSelectAllCheckStateChanged(
        ECheckBoxState CheckBoxState)
{
    if (!Node)
    {
        return;
    }

    const FScopedTransaction Transaction(LOCTEXT("ChangeSelectAllMorphsTransaction", "Change Select All Morphs"));
    Node->Modify();
    Node->bUseAllRealTimeMorphs = CheckBoxState == ECheckBoxState::Checked;
}


void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::OnFilterTextChanged(const FText& SearchText)
{
    SearchBoxFilter = SearchText.ToString();
    GenerateMorphsInfoList();
    ListWidget->RequestListRefresh();
}

void SCustomizableObjectNodeSkeletalMeshRTMorphSelector::OnFilterTextCommitted(
        const FText& SearchText, 
        ETextCommit::Type CommitInfo)
{
    if (CommitInfo == ETextCommit::OnEnter)
    {
        SearchBoxFilter = SearchText.ToString();
        GenerateMorphsInfoList();
        ListWidget->RequestListRefresh();
    }
}

#undef LOCTEXT_NAMESPACE

