// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h" // IWYU pragma: keep

class FReferenceCollector;
class FText;
class ITableRow;
class SCustomizableObjectNodeSkeletalMeshRTMorphSelector;
class SSearchBox;
class STableViewBase;
class SWidget;
class UCustomizableObjectNodeSkeletalMesh;

namespace CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal
{
    
    class FRowItemInfo
    {
        friend class ::SCustomizableObjectNodeSkeletalMeshRTMorphSelector;
        friend class SRowWidget;

        FRowItemInfo(const FName& InName); 

        FName Name;
    };

    class SRowWidget : public SMultiColumnTableRow< TSharedPtr<FRowItemInfo> > 
    {
        using OwnerWidgetType = SCustomizableObjectNodeSkeletalMeshRTMorphSelector;

    public:
        SLATE_BEGIN_ARGS(SRowWidget) {}
            SLATE_ARGUMENT(TSharedPtr<FRowItemInfo>, Item)
            SLATE_ARGUMENT(TWeakPtr<OwnerWidgetType>, OwnerWidget)
        SLATE_END_ARGS()

        void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

        virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
    
        void OnMorphSelectionCheckStateChanged(ECheckBoxState CheckBoxState); 
        ECheckBoxState IsMorphSelected() const;

    private:
        TSharedPtr<FRowItemInfo> Item;
        TWeakPtr<OwnerWidgetType> OwnerWidget;
    };
} //namespace CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal

class SCustomizableObjectNodeSkeletalMeshRTMorphSelector 
        : public SCompoundWidget, public FGCObject
{
    using ItemType = 
        CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal::FRowItemInfo;
    
    using ListViewType = SListView<TSharedPtr<ItemType>>;

    friend class CustomizableObjectNodeSkeletalMeshRTMorphSelector_Internal::SRowWidget;

public:

    SLATE_BEGIN_ARGS(SCustomizableObjectNodeSkeletalMeshRTMorphSelector) {}
        SLATE_ARGUMENT(UCustomizableObjectNodeSkeletalMesh*, Node)
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs);
    ~SCustomizableObjectNodeSkeletalMeshRTMorphSelector() = default;

    void AddReferencedObjects(FReferenceCollector& Collector) override;

    void UpdateWidget();

    void GenerateMorphsInfoList();

    TSharedRef<ITableRow> GenerateRow(
            TSharedPtr<ItemType> InItem, 
            const TSharedRef<STableViewBase>& OwnerTable);
    bool IsRuntimeMorphsListEnabled() const;

    ECheckBoxState IsSelectAllChecked() const;
    void OnSelectAllCheckStateChanged(ECheckBoxState CheckBoxState);

    void OnFilterTextChanged(const FText& SearchText);
    void OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo);

	virtual FString GetReferencerName() const override
	{
		return TEXT("SCustomizableObjectNodeSkeletalMeshRTMorphSelector");
	}

private:
		TObjectPtr<UCustomizableObjectNodeSkeletalMesh> Node = nullptr;

    TArray< TSharedPtr<ItemType> > Items;
    TSharedPtr< ListViewType > ListWidget;

    TSharedPtr<SSearchBox> SearchBox;
    FString SearchBoxFilter;
};
