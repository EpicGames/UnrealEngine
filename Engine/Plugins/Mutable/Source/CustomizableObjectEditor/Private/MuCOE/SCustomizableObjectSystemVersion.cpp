// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SCustomizableObjectSystemVersion.h"

#include "MuCO/ICustomizableObjectModule.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;


void SCustomizableObjectSystemVersion::Construct( const FArguments& InArgs )
{
	FString Version = ICustomizableObjectModule::Get().GetPluginVersion();

	ChildSlot
	[
		SNew( SVerticalBox )

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SGridPanel)
			+ SGridPanel::Slot(0, 0)
		.Padding(2)
		[
			SNew(STextBlock)
				.Text(FText::FromString("v:"))
			]
			+SGridPanel::Slot(1,0)
			.Padding(2)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Version))
			]			
		]
	];

}


void SCustomizableObjectSystemVersion::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SCompoundWidget::Tick( AllottedGeometry, InCurrentTime, InDeltaTime );
}




