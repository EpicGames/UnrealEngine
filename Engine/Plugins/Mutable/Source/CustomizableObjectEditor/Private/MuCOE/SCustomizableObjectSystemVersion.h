// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

struct FGeometry;


class SCustomizableObjectSystemVersion : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjectSystemVersion)
		{}
	SLATE_END_ARGS()

	/** */
	UE_API void Construct( const FArguments& InArgs );

public:

	// SWidget interface
	UE_API void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

private:


};

#undef UE_API
