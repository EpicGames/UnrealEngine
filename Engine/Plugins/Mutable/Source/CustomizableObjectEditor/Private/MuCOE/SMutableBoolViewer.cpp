// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableBoolViewer.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void SMutableBoolViewer::Construct(const FArguments& InArgs)
{
	const FText BoolValueTitle = LOCTEXT("BoolValueTitle", "Bool Value : ");

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Bool title
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(BoolValueTitle)
		]

		// Bool value
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(this, &SMutableBoolViewer::GetValue)
			.ColorAndOpacity(this, &SMutableBoolViewer::GetColorForValue)
		]
	];
}


void SMutableBoolViewer::SetBool(const bool& bInBool)
{
	this->bBoolValue = bInBool;
}

FText SMutableBoolViewer::GetValue() const
{
	return this->bBoolValue ? FText(INVTEXT("TRUE")) : FText(INVTEXT("FALSE"));
}

FSlateColor SMutableBoolViewer::GetColorForValue() const
{
	return this->bBoolValue ? this->TrueValueColor : this->FalseValueColor;
}

#undef LOCTEXT_NAMESPACE
