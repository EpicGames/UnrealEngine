// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

/*
* Mutable viewer to show a representation of a bool variable held by mutable.
*/
class SMutableBoolViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableBoolViewer) {}
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Bool to be used for this widget */
	void SetBool(const bool& bInBool);

private:

	/** The value being monitorized */
	bool bBoolValue;

	/** Colors used to paint the TRUE or FALSE value text */
	const FSlateColor TrueValueColor{ FLinearColor{0,1,0,1} };
	const FSlateColor FalseValueColor{ FLinearColor{1,0,0,1} };

	/*
	* Callback method used by the UI to get the value in a format it can display (FText) 
	* @return The FText that could be either TRUE or FALSE depending on the value of bBoolValue
	*/
	FText GetValue() const;

	/*
	* Callback that returns the color that the UI text should be using for TRUE or FALSE 
	* @return The FSlateColor to be used by the FText shown in the UI
	*/
	FSlateColor GetColorForValue() const;

};
