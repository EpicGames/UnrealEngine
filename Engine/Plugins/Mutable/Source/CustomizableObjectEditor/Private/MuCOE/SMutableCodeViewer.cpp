// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableCodeViewer.h"

#include "PropertyCustomizationHelpers.h"
#include "SListViewSelectorDropdownMenu.h"
#include "SMutableMaterialViewer.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "MuCO/UnrealConversionUtils.h"
#include "MuCO/UnrealToMutableTextureConversionUtils.h"
#include "MuCOE/SMutableBoolViewer.h"
#include "MuCOE/SMutableColorViewer.h"
#include "MuCOE/SMutableConstantsWidget.h"
#include "MuCOE/SMutableCurveViewer.h"
#include "MuCOE/SMutableImageViewer.h"
#include "MuCOE/SMutableIntViewer.h"
#include "MuCOE/SMutableLayoutViewer.h"
#include "MuCOE/SMutableMeshViewer.h"
#include "MuCOE/SMutableProjectorViewer.h"
#include "MuCOE/SMutableScalarViewer.h"
#include "MuCOE/SMutableSkeletonViewer.h"
#include "MuCOE/SMutableStringViewer.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "MuCOE/Widgets/MutableExpanderArrow.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/UnrealMutableImageProvider.h"
#include "MuT/ErrorLog.h"
#include "MuT/TypeInfo.h"
#include "MuR/System.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Internationalization/Regex.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuR/PassthroughObject.h"

class FExtender;
class FReferenceCollector;
class FUICommandList;
class SWidget;
namespace UE::Mutable::Private { struct FProjector; }
namespace UE::Mutable::Private { struct FShape; }
struct FGeometry;
struct FSlateBrush;


#define LOCTEXT_NAMESPACE "SMutableDebugger"


namespace MutableCodeTreeViewColumns
{
	static const FName OperationsColumnID("Operations");
	static const FName AdditionalDataColumnID("Flags");
};

/**
 * Mutable tree row used to display the operations held on the Mutable model object. 
 */
class SMutableCodeTreeRow final : public SMultiColumnTableRow<TSharedPtr<FMutableCodeTreeElement>>
{
public:

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableCodeTreeElement>& InRowItem)
	{
		RowItem = InRowItem;
		
		SMultiColumnTableRow< TSharedPtr<FMutableCodeTreeElement> >::Construct(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	FLinearColor OnGetExtraDataBoxColor() const
	{
		if (RowItem->bIsDynamicResource)
		{
			return DynamicResourceBoxColor;
		}
		else if (RowItem->bIsStateConstant)
		{
			return StateConstantBoxColor;
		}
		else
		{
			return ExtraDataBackgroundBoxDefaultColor;
		}
	}

	FText OnGetExtraDataText() const
	{
		// DEBUG :Uncomment the next line in order to debug the current state being used by the element
		// return FText::FromString(FString::FromInt(RowItem->GetStateIndex()));
		
		if (RowItem->bIsDynamicResource)
		{
			return  DynamicResourceText;
		}
		else if (RowItem->bIsStateConstant)
		{
			return StateConstantText;
		}
		else
		{
			return FText::FromString(FString(""));
		}
	}

	/** Depending on the state of the row returns one color or another to be used by the highlighting system */
	FLinearColor GetHighlightColor() const
	{
		if (bShouldBeHighlighted)
		{
			if (RowItem->DuplicatedOf)
			{
				return HighlightedDuplicatedBoxColor;
			}
			else
			{
				return HighlightedUniqueRowBoxColor;
			}
		}

		return HighlightBoxDefaultColor;
	}
	

	FSlateColor GetLabelColor() const
	{
		return RowItem->GetLabelColor();
	}
	
	/** Method intended with the generation of the wanted objects for each column*/
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		// Primary column showing the name of the operation and tye type
		if (ColumnName == MutableCodeTreeViewColumns::OperationsColumnID)
		{
			// Prepare a ui container for all the UI objects required by this row element
			TSharedRef<SHorizontalBox> RowContainer = SNew(SHorizontalBox)
				
			// First coll showing operation name and type
			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.AutoWidth()
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SAssignNew(this->HighlightingColorBox, SColorBlock)
					.Color(this, &SMutableCodeTreeRow::GetHighlightColor)
				]

				+ SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SMutableExpanderArrow, SharedThis(this))
					]
					
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(FText::FromString(RowItem->MainLabel))
						.ColorAndOpacity(this, &SMutableCodeTreeRow::GetLabelColor)
					]
				]
			];

			return RowContainer;
		}

		// Second column showing some extra data related with the operation being displayed
		if (ColumnName == MutableCodeTreeViewColumns::AdditionalDataColumnID)
		{
			TSharedRef<SHorizontalBox> RowContainer =  SNew(SHorizontalBox)
			
			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.AutoWidth()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.MaxWidth(4.0f)
				[
					SNew(SColorBlock)
					.Color(this,&SMutableCodeTreeRow::OnGetExtraDataBoxColor)
				]

				+ SHorizontalBox::Slot()
				.Padding(4,1)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this,&SMutableCodeTreeRow::OnGetExtraDataText)
				]
			];
			
			return RowContainer;
		}

		// Invalid column name so no widget will be produced 
		return SNullWidget::NullWidget;
	}
	
	/** Marks the row to be highlighted */
	void Highlight()
	{
		bShouldBeHighlighted = true;
	}

	/** Resets the highlighting status */
	void ResetHighlight()
	{
		bShouldBeHighlighted = false;
	}

	/** Returns a reference to the Element this row is representing */
	TSharedPtr<FMutableCodeTreeElement>& GetItem()
	{
		return RowItem;
	}

private:

	/** Pointer to the element that did spawn this row */
	TSharedPtr<FMutableCodeTreeElement> RowItem = nullptr;

	/** Transparent color */
	const FLinearColor TransparentColor = FLinearColor(0,0,0,0);

	/*
	 * Operation Highlighting 
	 */
	
	/** Custom Widget used to display a color. Used as the background of the text on the row to serve as highlighting Visual Element*/
	TSharedPtr<SColorBlock> HighlightingColorBox = nullptr;
	
	/** The color used to highlight the row if duplicated from another row */
	const FLinearColor HighlightedDuplicatedBoxColor = FLinearColor(1, 1, 1, 0.15f);

	/** The color used to highlight elements that are originals (not duplicates)  */
	const FLinearColor HighlightedUniqueRowBoxColor = FLinearColor(1, 1, 1, 0.28f);

	/** Default color used when the row is not highlighted */
	const FLinearColor HighlightBoxDefaultColor = TransparentColor;

	/*
	 * Extra data objects
	 */

	// Text used to set the width of the color area in front of the extra data
	const FText EmptyText = FText(INVTEXT(" "));
	
	/** String printed on the UI when the operation is shown to be dynamic resource */
	const FText DynamicResourceText = FText::FromString(FString("dyn"));

	/** String printed on the UI when the operation is shown to be state constant */
	const FText StateConstantText = FText::FromString(FString("const"));
	
	/** Color used on the extra data column when no extra data is shown */
	const FLinearColor ExtraDataBackgroundBoxDefaultColor =  TransparentColor;

	/** Color shown on the extra data column when the resource is found to be Dynamic */
	const FLinearColor DynamicResourceBoxColor = FLinearColor(0,0,1,0.8f);

	/** Color shown on the extra data column when the resource is found to be State Constant */
	const FLinearColor StateConstantBoxColor = FLinearColor(1,0,0,0.8f);

	bool bShouldBeHighlighted = false;
};


void SMutableCodeViewer::ClearSelectedTreeRow() const
{
	check(TreeView);
	TreeView->ClearSelection();
}


void SMutableCodeViewer::Construct(const FArguments& InArgs, UCustomizableObject& InObject, UCustomizableObjectInstance& InInstance, const TSharedPtr<UE::Mutable::Private::FModel>& InMutableModel)
{
	// Min width allowed for the column. Needed to avoid having issues with the constants space being to small
	// and then getting too tall on the y axis crashing the UI drawer.
	constexpr float MinParametersCollWidth = 200;

	Object = TStrongObjectPtr(&InObject);
	Instance = TStrongObjectPtr(&InInstance);

	MutableModel = InMutableModel;
	ExternalResourceProvider = InArgs._ExternalResourceProvider;

	PreviewParameters = Instance->GetPrivate()->GetDescriptor().GetParameters();
	if (!PreviewParameters)
	{
		PreviewParameters = UE::Mutable::Private::FModel::NewParameters(MutableModel);
	}
	
	RootNodes.Empty();
	RootNodeAddresses.Empty();
	ItemCache.Empty();
	MainItemPerOp.Empty();
	TreeElements.Empty();
	ExpandedElements.Empty();
	FoundModelOperationTypeElements.Empty();
	ModelOperationTypes.Empty();
	ModelOperationTypeNames.Empty();

	// Reset navigation by type / constant resource
	NavigationElements.Empty();
	NavigationIndex = -1;

	// Reset navigation by string
	NameBasedNavigationElements.Empty();
	StringNavigationIndex = -1;

	// Generate all elements before starting the tree UI so we have a deterministic set of unique and duplicated elements
	GenerateAllTreeElements();

	// Setup Navigation system
	{
		// Store the addresses of the root nodes so they can be used by operation search methods
		CacheRootNodeAddresses();

		// Cache the operation types that are present on the model
		CacheOperationTypesPresentOnModel();

		// Get an array of mutable types as an array of FStrings for the UI
		GenerateNavigationOpTypeStrings();

		// Generate list elements for the found operation types so we are able to search over them on our type dropdown
		GenerateNavigationDropdownElements();

		// Check we did find types (witch should always happen in a normal run) and select the NONE option as the default value
		check(FoundModelOperationTypeElements.Num())
		CurrentlySelectedOperationTypeElement = NoneOperationEntry;
	}
	
	FToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None, TSharedPtr<FExtender>(), true);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Visible);
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "SlimToolBar");

	TSharedRef<SScrollBar> TreeVertScrollBar =
		SNew(SScrollBar).
		Orientation(EOrientation::Orient_Vertical).
		AlwaysShowScrollbar(false);
	
	// Ensure the color policy does have a valid default value.
	check(EntryLabelColoringPolicyNames.IsValidIndex((int32)EntryLabelColoringPolicy));
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		[
			ToolbarBuilder.MakeWidget()
		]
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Fill)
		.Padding(5,2)
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)
			
			+ SSplitter::Slot()
			.Value(0.35f)
			.MinSize(520)
			.Resizable(true)
			[
				SNew(SVerticalBox)

				// Search box for tree operations
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				[
					SNew(SHorizontalBox)
				
					// Search by name
					+ SHorizontalBox::Slot()
					.Padding(2,2)
					.AutoWidth()
					[
						SNew(SHorizontalBox)
				
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SelectedOperationByStringLabel","Search Operation by String "))
						]
					
						+ SHorizontalBox::Slot()
						.MaxWidth(250)
						.VAlign(VAlign_Center)
						[
							SNew(SSearchBox)
							.HintText(LOCTEXT("OperationToSearchHintText","Search OP"))
							.SearchResultData(this,&SMutableCodeViewer::SearchResultsData)
							.OnSearch(this, &SMutableCodeViewer::OnTreeStringSearch)
							.OnTextChanged(this, &SMutableCodeViewer::OnTreeSearchTextChanged)
							.OnTextCommitted(this, &SMutableCodeViewer::OnTreeSearchTextCommitted)
						]
					]
					
					
					// Regex control for search by name
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4,2)
					[
						SNew(SHorizontalBox)
				
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Regex", "Regex"))
						]
				
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4, 0, 0, 0)
						.VAlign(VAlign_Center)
						[
							SNew(SCheckBox)
							.OnCheckStateChanged(this,&SMutableCodeViewer::OnRegexToggleChanged)
						]
					]	
				]
				
				// Operation type filtering slot
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				[
					// Box containing navigation elements
					SNew(SVerticalBox)
				
					+ SVerticalBox::Slot()
					.Padding(2,4)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
				
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SHorizontalBox)
				
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SelectedOperationTypeLabel","Search Operation Type "))
							]
				
							// ComboBox used to select one or another Op_Type for tree navigation purposes
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SAssignNew(TargetedTypeSelector,SComboBox<TSharedPtr<const FMutableOperationElement>>)
								.OptionsSource(&FoundModelOperationTypeElements)
								.InitiallySelectedItem(CurrentlySelectedOperationTypeElement)
								[
									SNew(STextBlock)
									.Text(this,&SMutableCodeViewer::GetCurrentNavigationOpTypeText)
									.ColorAndOpacity(this,&SMutableCodeViewer::GetCurrentNavigationOpTypeColor)
								]
								.OnGenerateWidget(this,&SMutableCodeViewer::OnGenerateOpNavigationDropDownWidget)
								.OnSelectionChanged(this,&SMutableCodeViewer::OnNavigationSelectedOperationChanged)
							]
						]
				
						+ SHorizontalBox::Slot()
						.Padding(4,0)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.Text(LOCTEXT("GoToPreviousOperationButton"," < "))
							.OnClicked(this,&SMutableCodeViewer::OnGoToPreviousOperationButtonPressed)
							.IsEnabled(this,&SMutableCodeViewer::CanInteractWithPreviousOperationButton)
						]
				
						+ SHorizontalBox::Slot()
						.Padding(4,0)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this,&SMutableCodeViewer::OnPrintNavigableObjectAddressesCount)
							.Justification(ETextJustify::Center)
						]
				
						+ SHorizontalBox::Slot()
						.Padding(4,0)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.Text(LOCTEXT("GoToNextOperationButton"," > "))
							.OnClicked(this,&SMutableCodeViewer::OnGoToNextOperationButtonPressed)
							.IsEnabled(this,&SMutableCodeViewer::CanInteractWithNextOperationButton)
						]
					]
				]
				
				// Tree operations slot
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SBorder)
					.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
					.Padding(FMargin(4.0f, 4.0f))
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.FillContentWidth(1)
						[
							SNew(SScrollBox)		
							.Orientation(EOrientation::Orient_Horizontal)
							.ConsumeMouseWheel(EConsumeMouseWheel::Never)
							
							+ SScrollBox::Slot()
							.HAlign(HAlign_Fill)
							.FillContentSize(1)
							[
								SAssignNew(TreeView, STreeView<TSharedPtr<FMutableCodeTreeElement>>)
								.TreeItemsSource(&RootNodes)
								.OnGenerateRow(this, &SMutableCodeViewer::GenerateRowForNodeTree)
								.OnRowReleased(this, &SMutableCodeViewer::OnRowReleased)
								.OnGetChildren(this, &SMutableCodeViewer::GetChildrenForInfo)
								.OnSelectionChanged(this, &SMutableCodeViewer::OnSelectionChanged)
								.OnSetExpansionRecursive(this, &SMutableCodeViewer::TreeExpandRecursive)
								.OnContextMenuOpening(this, &SMutableCodeViewer::OnTreeContextMenuOpening)
								.OnExpansionChanged(this, &SMutableCodeViewer::OnExpansionChanged)
								.SelectionMode(ESelectionMode::Single)
								.ExternalScrollbar(TreeVertScrollBar)
								.HeaderRow
								(
									SNew(SHeaderRow)
									.ResizeMode(ESplitterResizeMode::Fill)

									+ SHeaderRow::Column(MutableCodeTreeViewColumns::OperationsColumnID)
										.DefaultLabel(LOCTEXT("Operation", "Operation"))

									+ SHeaderRow::Column(MutableCodeTreeViewColumns::AdditionalDataColumnID)
										.DefaultLabel(LOCTEXT("OperationFlags", "Flags"))
										.FixedWidth(50.0f)
								)
							]
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							TreeVertScrollBar
						]
					]
				]
				
				// Label coloring policy
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OPLabelColoringStrategy", "Operation Coloring Strategy: "))
			]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4,0)
					[
						SNew(SComboBox<FName>)
						.OptionsSource(&EntryLabelColoringPolicyNames)
						.InitiallySelectedItem(EntryLabelColoringPolicyNames[(int32)EntryLabelColoringPolicy])
						.OnSelectionChanged(this, &SMutableCodeViewer::OnLabelColorPolicyChange)
						.OnGenerateWidget(this, &SMutableCodeViewer::OnLabelColorPolicyGenerateWidget)
						[
							SNew(STextBlock)
							.Text(this, &SMutableCodeViewer::GetCurrentLabelColoringPolicyText)
						]
					]
				]
			]
			+ SSplitter::Slot()
			.Value(0.75f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Horizontal)
				
				+ SSplitter::Slot()
				.Value(0.28f)
				.MinSize(MinParametersCollWidth)
				[
					// Splitter managing both parameter and constant panels
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.Padding(4, 4, 4,0)
						.AutoHeight()
						[
							SNew(SButton)
							.Text(LOCTEXT("RefreshParameters", "Refresh Parameters"))
							.OnClicked(this, &SMutableCodeViewer::UpdateParameters)
						]
						
						+ SVerticalBox::Slot()
						.Padding(4, 8, 4, 0)
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.Padding(4, 0, 0, 0)
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SkipMipsLabel", "Skip mips on generate "))
								.Visibility(this, &SMutableCodeViewer::IsMipSkipVisible)
							]
							+ SHorizontalBox::Slot()
							.Padding(4, 0, 0, 0)
							[
								SNew(SNumericEntryBox<int32>)
								.Visibility(this, &SMutableCodeViewer::IsMipSkipVisible)
								.AllowSpin(true)
								.MinValue(0)
								.MaxValue(16)
								.MinSliderValue(0)
								.MaxSliderValue(16)
								.Value(this, &SMutableCodeViewer::GetCurrentMipSkip)
								.OnValueChanged(this, &SMutableCodeViewer::OnCurrentMipSkipChanged)
							]
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8, 8, 0, 0)
					.VAlign(VAlign_Top)
					[
						// Generate a new Constants panel to show the data stored on the current mutable program
						SAssignNew(ConstantsWidget, SMutableConstantsWidget,
							MutableModel,
							SharedThis(this))
					]
				]
				+ SSplitter::Slot()
				.Value(0.72f)
				[
					SAssignNew(PreviewBorder, SBorder)
					.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
					.Padding(FMargin(4.0f, 4.0f))					
				]
			]
		]
	];
	
	// Set the tree expanded by default
	// It does not recalculate states since the expansion of the instance will NOT expand duplicates witch means the widget position
	// of the children of duplicated (or the original of an operation with duplicates) will not change.
	TreeExpandInstance();

	// Enable the recalculation of states once the tree has already been initially expanded since now we do not control
	// how the user is point to interact with the view.
	bShouldRecalculateStates = true;
	// Now, on expansion or contraction the states will get recalculated
}


FReply SMutableCodeViewer::UpdateParameters()
{
	PreviewParameters = Instance->GetPrivate()->GetDescriptor().GetParameters();
	if (!PreviewParameters)
	{
		PreviewParameters = UE::Mutable::Private::FModel::NewParameters(MutableModel);
	}

	bIsPreviewPendingUpdate = true;

	return FReply::Handled();
}


EVisibility SMutableCodeViewer::IsMipSkipVisible() const 
{ 
	return bSelectedOperationIsImage ? EVisibility::Visible : EVisibility::Hidden;
}


TOptional<int32> SMutableCodeViewer::GetCurrentMipSkip() const 
{ 
	return MipsToSkip; 
}


void SMutableCodeViewer::OnCurrentMipSkipChanged(int32 NewValue)
{
	MipsToSkip = NewValue;
	bIsPreviewPendingUpdate = true;
}

#pragma region CodeTree operation name search


void SMutableCodeViewer::OnRegexToggleChanged(ECheckBoxState CheckBoxState)
{
	const bool bPreChangeValue = bIsSearchStringRegularExpression;
	bIsSearchStringRegularExpression = CheckBoxState == ECheckBoxState::Checked ? true : false;
	
	if (bPreChangeValue != bIsSearchStringRegularExpression)
	{
		CacheOperationsMatchingStringPattern();
		GoToNextOperation();
	}
}

void SMutableCodeViewer::OnTreeStringSearch(SSearchBox::SearchDirection SearchDirection)
{
	if (SearchDirection==SSearchBox::SearchDirection::Next)
	{
		GoToNextOperation();
	}
	else
	{
		GoToPreviousOperation();
	}
}

void SMutableCodeViewer::GoToNextOperation()
{
	// Contingency : Prevent a second scroll operation from being performed if still we do not have the first target in view
	if (bWasScrollToTargetRequested)
	{
		return;
	}
	
	if (NameBasedNavigationElements.Num())
	{
		const int32 PreviousIndex = StringNavigationIndex;

		// Focus on next target
		StringNavigationIndex = StringNavigationIndex >= NameBasedNavigationElements.Num() - 1
										   ? 0
										   : StringNavigationIndex + 1;

		if (StringNavigationIndex != PreviousIndex)
		{
			FocusViewOnNavigationTarget(NameBasedNavigationElements[StringNavigationIndex]);
		}
	}
}


void SMutableCodeViewer::GoToPreviousOperation()
{
	// Contingency : Prevent a second scroll operation from being performed if still we do not have the first target in view
	if (bWasScrollToTargetRequested)
	{
		return;
	}
	
	if (NameBasedNavigationElements.Num())
	{
		const int32 PreviousIndex = StringNavigationIndex;
		
		// Focus on previous target
		StringNavigationIndex = StringNavigationIndex <= 0 ?  NameBasedNavigationElements.Num() -1 : StringNavigationIndex -1;

		if (PreviousIndex != StringNavigationIndex)
		{
			FocusViewOnNavigationTarget(NameBasedNavigationElements[StringNavigationIndex]);
		}
	}
}

void SMutableCodeViewer::GoToTargetOperation(const int32& InTargetIndex)
{
	if (InTargetIndex == StringNavigationIndex)
	{
		return;
	}
	
	if (NameBasedNavigationElements.Num() && InTargetIndex > 0 && InTargetIndex <= NameBasedNavigationElements.Num()-1)
	{
		// Focus on the target index
		StringNavigationIndex = InTargetIndex;
		FocusViewOnNavigationTarget(NameBasedNavigationElements[StringNavigationIndex]);
	}
}


void SMutableCodeViewer::OnTreeSearchTextChanged(const FText& InUpdatedText)
{
	SearchString = InUpdatedText.ToString();
}


TOptional<SSearchBox::FSearchResultData> SMutableCodeViewer::SearchResultsData() const
{
	if (NameBasedNavigationElements.Num() == 0)
	{
		return TOptional<SSearchBox::FSearchResultData>();
	}
	return TOptional<SSearchBox::FSearchResultData>({ NameBasedNavigationElements.Num(), StringNavigationIndex + 1});
}


void SMutableCodeViewer::OnTreeSearchTextCommitted(const FText& InUpdatedText, ETextCommit::Type TextCommitType)
{
	if (TextCommitType == ETextCommit::OnEnter)
	{
		check (InUpdatedText.ToString() == SearchString);
		CacheOperationsMatchingStringPattern();	
		GoToNextOperation();
	}
}


void SMutableCodeViewer::CacheOperationsMatchingStringPattern()
{
	check(MutableModel);
	check(RootNodeAddresses.Num());
	
	if ( LastSearchedString == SearchString &&
		bWasLastSearchRegEx == bIsSearchStringRegularExpression && 
		LastSearchedModel == MutableModel )
	{
		// Do not perform a search again since the context has not changed
		return;
	} 
	
	if (!SearchString.IsEmpty())
	{
		UE_LOGF(LogMutable,Display, "Starting string search with target string ""\"%ls""\" ",*SearchString);
	
		// Object containing all data required by the search operation to be able to be called recursively
		FElementsSearchCache SearchPayload;
		// Initialize the Search Payload with the root node addresses. This way the search will use them as the root nodes where
		// to start searching
		SearchPayload.SetupRootBatch(RootNodeAddresses);

		const UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram();
		GetOperationsMatchingStringPattern(SearchString,bIsSearchStringRegularExpression,SearchPayload, Program);
	
		// Dump the located resources array onto the navigation array
		NameBasedNavigationElements = MoveTemp(SearchPayload.FoundElements);
		SortElementsByTreeIndex(NameBasedNavigationElements);
		
		UE_LOGF(LogMutable, Display, "Operations found with matching pattern ""\"%ls""\" is  %i", *SearchString, NameBasedNavigationElements.Num());
	}
	else
	{
		NameBasedNavigationElements.Reset();
	}

	// Reset the search index
	StringNavigationIndex = -1;

	// Keep track of what context was used to perform the search to avoid doing it again if the context has not changed
	LastSearchedString = SearchString;
	bWasLastSearchRegEx = bIsSearchStringRegularExpression;
	LastSearchedModel = MutableModel;
}


void SMutableCodeViewer::GetOperationsMatchingStringPattern(const FString& InStringPattern,const bool bIsRegularExpression ,FElementsSearchCache& SearchPayload,const UE::Mutable::Private::FProgram& InProgram)
{
	// next batch of addresses to be explored 
	TArray<FItemCacheKey> NextBatchAddressesData;
	
	for (int32 ParentIndex = 0; ParentIndex < SearchPayload.BatchData.Num(); ParentIndex++)
	{	
		const FItemCacheKey CacheKey = SearchPayload.BatchData[ParentIndex];
		const FString OperationDescriptiveText = GetOperationDescriptiveText(CacheKey);
		
		bool bMatchesPattern = false;
		if (!bIsRegularExpression)
		{
			// Check if the provided text is contained over the element identification text
			bMatchesPattern = OperationDescriptiveText.Contains(InStringPattern);
		}
		else
		{
			FRegexPattern Pattern{InStringPattern};
			FRegexMatcher RegexMatcher{Pattern,OperationDescriptiveText};
			bMatchesPattern = RegexMatcher.FindNext();
		}
		
		// Get one of the previous run "children" and treat as a parent to get it's children and process them
		const UE::Mutable::Private::FOperation::ADDRESS& ParentAddress = SearchPayload.BatchData[ParentIndex].Child;
		
		if (bMatchesPattern)
		{
			SearchPayload.AddToFoundElements(ParentAddress,ParentIndex,ItemCache);
		}
		
		// Get all NON PROCESSED the children of this operation to later be able to process them (on next recursive call)
		SearchPayload.CacheChildrenOfAddressIfNotProcessed(ParentAddress, InProgram, NextBatchAddressesData);
	}

	// At this point all the addresses to be computed on the next batch have already been set and will be computed on
	// the next recursive call
	
	// Explore children if found 
	if (NextBatchAddressesData.Num())
	{
		// Cache next batch data so the next invocations is able to locate the provided addresses on the itemsCache
		SearchPayload.BatchData = MoveTemp(NextBatchAddressesData);
		
		GetOperationsMatchingStringPattern(InStringPattern,bIsRegularExpression, SearchPayload, InProgram);
	}
}


FString SMutableCodeViewer::GetOperationDescriptiveText(const FItemCacheKey& InItemCacheKey)
{
	FString OperationDescriptiveText;
		
	if (const TSharedPtr<FMutableCodeTreeElement>* Element = ItemCache.Find(InItemCacheKey))
	{
		OperationDescriptiveText = Element->Get()->MainLabel;
		check (!OperationDescriptiveText.IsEmpty());
	}
	
	return OperationDescriptiveText;
}

#pragma endregion 


#pragma region CodeTree operation search

FText SMutableCodeViewer::GetCurrentNavigationOpTypeText() const
{
	check (CurrentlySelectedOperationTypeElement);
	
	return CurrentlySelectedOperationTypeElement->OperationTypeText;
}

FSlateColor SMutableCodeViewer::GetCurrentNavigationOpTypeColor() const
{
	check (CurrentlySelectedOperationTypeElement);

	return CurrentlySelectedOperationTypeElement->bEntryColorWasOverriden ? CurrentlySelectedOperationTypeElement->OperationTextOverrideColor :  GetOperationColor(CurrentlySelectedOperationTypeElement->OperationType);
}

void SMutableCodeViewer::GenerateNavigationDropdownElements()
{
	const int32 OperationTypesCount = ModelOperationTypes.Num();
	
	// It must have at least one type, if not may be because we are running this before filling ModelOperationTypes
	FoundModelOperationTypeElements.Empty(OperationTypesCount);
	
	for	(int32 OperationTypeIndex = 0; OperationTypeIndex < OperationTypesCount;  OperationTypeIndex++)
	{
		// Get the type as a string to be able to print it on the UI
		const FText OperationTypeName = FText::FromString(ModelOperationTypeNames[OperationTypeIndex]);

		const UE::Mutable::Private::EOpType RepresentedType = ModelOperationTypes[OperationTypeIndex].Key;
		const uint32 OperationTypeInstancesCount = ModelOperationTypes[OperationTypeIndex].Value;
		
		// Generate an element to be used by the ComboBox handling the selection of the type to be used during navigation
		TSharedPtr<FMutableOperationElement> OperationElement = MakeShared<FMutableOperationElement>(RepresentedType, OperationTypeName, OperationTypeInstancesCount);
		FoundModelOperationTypeElements.Add(OperationElement);
	}

	// Add an entry for the NONE type of operation
	{
		const FText EntryName = FText::FromString("NONE");
		
		NoneOperationEntry = MakeShared<FMutableOperationElement>(UE::Mutable::Private::EOpType::NONE,EntryName,0);

		// @warn While not visible this element must be part of the collection for the ComboBox to be able to work
		// properly
		FoundModelOperationTypeElements.Add(NoneOperationEntry);
	}
	
	// Add an extra operation type that will represent the constant resource based navigation type
	{
		const FText EntryName = FText::FromString("Selected Constant");	
		TSharedRef<FMutableOperationElement> NewMutableOperationElement = MakeShared<FMutableOperationElement>(UE::Mutable::Private::EOpType::NONE, EntryName, 0);

		const FSlateColor EntryColor = FSlateColor(FLinearColor(0.35f ,0.35f,1.0f,1));
		NewMutableOperationElement->SetEntryColorOverride(EntryColor);
		
		ConstantBasedNavigationEntry = NewMutableOperationElement;
		
		// @warn While not visible this element must be part of the collection for the ComboBox to be able to work
		// properly
		FoundModelOperationTypeElements.Add(ConstantBasedNavigationEntry);
	}
}


TSharedRef<SWidget> SMutableCodeViewer::OnGenerateOpNavigationDropDownWidget(
	TSharedPtr<const FMutableOperationElement> InMutableOperationElement) const
{
	if (!InMutableOperationElement->bEntryColorWasOverriden)
{
	TSharedRef<STextBlock> NewSlateObject = SNew(STextBlock)
		.Text(InMutableOperationElement->OperationTypeText)
		.ColorAndOpacity(this, &SMutableCodeViewer::GetOperationColor, InMutableOperationElement->OperationType)
		.Visibility(InMutableOperationElement->GetEntryVisibility());

		return NewSlateObject;
	}
	else
	{
		TSharedRef<STextBlock> NewSlateObject = SNew(STextBlock)
		.Text(InMutableOperationElement->OperationTypeText)
		.ColorAndOpacity(InMutableOperationElement->OperationTextOverrideColor)
		.Visibility(InMutableOperationElement->GetEntryVisibility());
	
	return NewSlateObject;
}
}


void SMutableCodeViewer::OnNavigationSelectedOperationChanged(
	TSharedPtr<const FMutableOperationElement, ESPMode::ThreadSafe> MutableOperationElement, ESelectInfo::Type Arg)
{
	// Handle the case where we do not want an option selected, for example, when clearing the selected option.
	TSharedPtr<const FMutableOperationElement, ESPMode::ThreadSafe> NewSelectedElement = MutableOperationElement;
	if (!NewSelectedElement.IsValid())
	{
		NewSelectedElement = NoneOperationEntry;
	}
	
	check (NewSelectedElement.IsValid());

	// Cache the currently selected operation set on the UI by the user
	const UE::Mutable::Private::EOpType NewOperationType = NewSelectedElement->OperationType;
	OperationTypeToSearch = NewOperationType;
	CurrentlySelectedOperationTypeElement = NewSelectedElement;
	
	// Only do the internal work if the type is one that makes sense searching
	if (OperationTypeToSearch != UE::Mutable::Private::EOpType::NONE)
	{
		// Locate all operations on the mutable operations tree (not the visual one) that do share the same operation type
		// as the one selected. This will fill the array with the elements we should be looking for during the navigation operation
		CacheAddressesOfOperationsOfType();
	}
	// None can be set by the user or be an indication that we are navigating over constant related operations
	// todo: Separate both operations in some way on the UI to avoid complications in the code and in the UI's UX
	else
	{
		// Clear all the elements on the navigation addresses 
		NavigationElements.Empty();
	}
	
}


void SMutableCodeViewer::GenerateNavigationOpTypeStrings()
{
	// Grab only the names from the operation types located during the caching of operation types of the model
	for (const TTuple<UE::Mutable::Private::EOpType, uint32>& LocatedOperationType : ModelOperationTypes)
	{
		// Find the name of the Operation type
		const uint16 OperationIndex = static_cast<uint16>(LocatedOperationType.Key);
		const TCHAR* OpName = UE::Mutable::Private::s_opNames[OperationIndex];

		// Remove trailing whitespaces adding noise and messing up concatenations with other strings
		FString OperationNameString{OpName};
		OperationNameString.RemoveSpacesInline();

		// Save the name
		ModelOperationTypeNames.Add( OperationNameString);
	}
}

void SMutableCodeViewer::OnSelectedOperationTypeFromTree()
{
	// We require to have only 1 element selected to avoid having inconsistencies during operation
	check(TreeView->GetNumItemsSelected() == 1);
	
	const TSharedPtr<FMutableCodeTreeElement> ReferenceOperationElement = TreeView->GetSelectedItems()[0];
	
	const UE::Mutable::Private::EOpType OperationType =
		MutableModel->GetProgram().GetOpType(ReferenceOperationElement->MutableOperation);

	// Find the operation type directly in our array of operation elements (from the drop down)
	const TSharedPtr<const FMutableOperationElement>* RepresentativeElement = FoundModelOperationTypeElements.
		FindByPredicate([OperationType](const TSharedPtr<const FMutableOperationElement> Other)
		{
			return Other->OperationType == OperationType;
		});

	// Ensure an element was found. Failing the next check would mean that we are not caching all the types present on
	// the current operation's tree
	check(RepresentativeElement != nullptr);
	
	// Set the type operation type to be looking for -> Will invoke OnOptionTypeSelectionChanged
	TargetedTypeSelector->SetSelectedItem(*RepresentativeElement);
	
	// Reset the navigation index so it starts from scratch
	NavigationIndex = -1;
}


void SMutableCodeViewer::SortElementsByTreeIndex(TArray<TSharedPtr<FMutableCodeTreeElement>>& InElementsArrayToSort)
{
	// Sort the array from lower index to bigger index (0 , 1 , 2 ...)
	InElementsArrayToSort.Sort([](const TSharedPtr<FMutableCodeTreeElement> A , const TSharedPtr<FMutableCodeTreeElement> B)
	{
		return A->IndexOnTree < B->IndexOnTree;
	});
}


void SMutableCodeViewer::CacheAddressesOfOperationsOfType()
{
	// Clear previous data
	NavigationElements.Empty();
	check(RootNodeAddresses.Num());

	// Object containing all data required by the search operation to be able to be called recursively
	FElementsSearchCache SearchPayload;
	// Initialize the Search Payload with the root node addresses. This way the search will use them as the root nodes where
	// to start searching
	SearchPayload.SetupRootBatch(RootNodeAddresses);
	
	// Main update procedure run for the targeted state and the targeted parameter values
	const UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram();
	GetOperationsOfType(OperationTypeToSearch,SearchPayload, Program);
	
	if (!SearchPayload.FoundElements.IsEmpty())
	{
		// Cache the navigation addresses so we are able to navigate over them
		NavigationElements = MoveTemp(SearchPayload.FoundElements);
		SortElementsByTreeIndex(NavigationElements);
		
		// Reset the navigation index
		NavigationIndex = -1;
	}
}

void SMutableCodeViewer::GetOperationsOfType(const UE::Mutable::Private::EOpType& TargetOperationType,
                                             FElementsSearchCache& InSearchPayload,
                                             const UE::Mutable::Private::FProgram& InProgram)
{
	// next batch of addresses to be explored 
	TArray<FItemCacheKey> NextBatchAddressesData;
	
	for	(int32 ParentIndex = 0 ; ParentIndex < InSearchPayload.BatchData.Num(); ParentIndex++)
	{
		// Get one of the previous run "children" and treat as a parent to get it's children and process them
		const UE::Mutable::Private::FOperation::ADDRESS& CurrentAddress = InSearchPayload.BatchData[ParentIndex].Child;
		
		// Cache if same data type and we share the same address (means this op is pointing at the provided resource)
		// It will cache duplicated entries
		if ( InProgram.GetOpType(CurrentAddress) == TargetOperationType)
		{
			// Since this element is of the type we are looking for then cache it on InSearchPayload.FoundElements
			InSearchPayload.AddToFoundElements(CurrentAddress,ParentIndex,ItemCache);
		}
		
		// Get all NON PROCESSED the children of this operation to later be able to process them (on next recursive call)
		InSearchPayload.CacheChildrenOfAddressIfNotProcessed(CurrentAddress, InProgram, NextBatchAddressesData);
	}

	// Explore children if found 
	if (NextBatchAddressesData.Num())
	{
		// Cache next batch data so the next invocations are able to locate the provided addresses on the itemsCache
		InSearchPayload.BatchData = MoveTemp(NextBatchAddressesData);
		
		// Process the children of this object
		GetOperationsOfType(TargetOperationType, InSearchPayload, InProgram);
	}
 }


void SMutableCodeViewer::CacheOperationTypesPresentOnModel()
{
	// ModelOperationTypes is filled when the tree is generating.
	
	// Remove all operation types that do have no operations present on the model
	{
		ModelOperationTypes.RemoveAll(
		[](const TPair<UE::Mutable::Private::EOpType, uint32>& Current)
			{
				return Current.Value == 0;
			});
	}
	
	// Sort the contents of the array of mutable operation types alphabetically
	ModelOperationTypes.StableSort([&](const TPair<UE::Mutable::Private::EOpType,uint32>& A, const TPair<UE::Mutable::Private::EOpType,uint32>& B)
	{
		// Find the name
		FString AString;
		{
			const uint16 OperationIndex = static_cast<uint16>(A.Key);
			const TCHAR* OpName = UE::Mutable::Private::s_opNames[OperationIndex];
			AString = FString(OpName);
		}
		
		// Find out the name of the first element
		FString BString;
		{
			const uint16 OperationIndex = static_cast<uint16>(B.Key);
			const TCHAR* OpName = UE::Mutable::Private::s_opNames[OperationIndex];
			BString = FString(OpName);
		}
		
		// Then the name of the second element
		return AString < BString;
	});	
	
	// ModelOperationTypes is now an array with all the types found on the operations tree in alphabetical order
}



FText SMutableCodeViewer::OnPrintNavigableObjectAddressesCount() const
{
	FString OutputString = "";
	if (const int32 NavigationElementsCount = NavigationElements.Num())
	{
		// Show the index if the index showing adds information
		if (NavigationIndex >= 0)
		{
			OutputString.Append( FString::FromInt( NavigationIndex+1));
			OutputString.Append(" / ");
		}
		
		OutputString.Append( FString::FromInt(NavigationElementsCount));

		// Format : 1 / 12 or 12
	}
	
	// Depending on the amount of navigable objects (addresses, not actual elements) display the amount there are
	return FText::FromString(OutputString);
}


bool SMutableCodeViewer::CanInteractWithPreviousOperationButton() const
{
	// Only navigable if there are more than 0 elements to traverse and we are not scrolling
	return NavigationElements.Num() > 0 && NavigationIndex > 0 && (!bWasScrollToTargetRequested && !bWasUniqueExpansionInvokedForNavigation);
}

bool SMutableCodeViewer::CanInteractWithNextOperationButton() const
{
	// Only navigable if there are more than 0 elements to traverse and we are not scrolling
	return NavigationElements.Num() > 0 && NavigationIndex < NavigationElements.Num() -1 && (!bWasScrollToTargetRequested && !bWasUniqueExpansionInvokedForNavigation);
}


FReply SMutableCodeViewer::OnGoToPreviousOperationButtonPressed()
{
	// Focus on previous target
	NavigationIndex = NavigationIndex<=0 ? 0 : NavigationIndex - 1;
	FocusViewOnNavigationTarget(NavigationElements[NavigationIndex]);
	
	return FReply::Handled();
}

FReply SMutableCodeViewer::OnGoToNextOperationButtonPressed()
{
	// Focus on next target
	NavigationIndex = NavigationIndex>=NavigationElements.Num() -1 ? NavigationElements.Num() -1 : NavigationIndex + 1;
	FocusViewOnNavigationTarget(NavigationElements[NavigationIndex]);
	
	return FReply::Handled();
}

void SMutableCodeViewer::FocusViewOnNavigationTarget(TSharedPtr<FMutableCodeTreeElement> InTargetElement)
{
	// Stage 1 : Expand all tree so all navigable elements get to be reachable
	if (!bWasUniqueExpansionInvokedForNavigation && !bWasScrollToTargetRequested)
	{
		TreeExpandUnique();
		bWasUniqueExpansionInvokedForNavigation = true;
		
		// Cache the current navigation target so after the update we can focus it 
		ToFocusElement = InTargetElement;
		
		// Early exit, this method will get called again later after tree update
		return;		
	}
	
	// Stage 2 : Try to get to the targeted element. if not visible scroll into view
	check (InTargetElement.IsValid());
	
	// If required scroll to the area where we know the element is going to be in view
	// a way to ensure this happens is by calling 
	if (TreeView->IsItemVisible(InTargetElement))
	{
		// Stage 3-b : Select the element we have provided since now is sure to be in view
		
		// This line selects the element with at the same time updates the UI to show the row representing this element selected
		TreeView->SetSelection(InTargetElement);
		ToFocusElement.Reset();							// We have focused the target so we no longer need to keep a reference to it
		
		// Done!
		// We have the element in view and we have selected it!
	}
	else
	{
		// Stage 3-a (optional) : Ask for the provided element to be scrolled into view.
		
		// Failing this check would mean we have performed a scroll but we are still not able to view the element
		check (!bWasScrollToTargetRequested);
		
		// Request the tree to show us the target element we want to get focused
		TreeView->RequestScrollIntoView(InTargetElement);
		
		// Read this variable after the update and then select the object (easy at this point)
		// You may want to just call again this method after refresh since the element will be on view
		bWasScrollToTargetRequested = true;

		// Early exit, this method will get called again later after tree update once the scroll has been completed
		return;
	}

	// Reset the control flag so we do not expand all tree again if not required
	bWasUniqueExpansionInvokedForNavigation = false;
	bWasScrollToTargetRequested = false;
}

#pragma endregion 

#pragma region Operation Cost Color Hints

void SMutableCodeViewer::GenerateAllTreeElements()
{
	// By generating all tree elements prior to usage we are able to :
	//	- Compute the index of each one to aid on navigation
	//	- Remove non-deterministic assignation of the "Duplicated" state of elements. It was due to user interaction with the tree
	//	Only unique elements, their children and duplicated elements will be generated. Children of duplicates will
	// 	be ignored due to how we handle them when expanding and contracting elements (OnExpansionChanged)

	// Generate all root nodes
	const UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram();
	const uint32 StateCount = Program.States.Num();

	ModelOperationTypes.Init({ UE::Mutable::Private::EOpType::NONE, 0 }, (int32)UE::Mutable::Private::EOpType::COUNT);

	for ( uint32 StateIndex = 0; StateIndex < StateCount; ++StateIndex )
	{
		const UE::Mutable::Private::FProgram::FState& State = Program.States[StateIndex];
		const FString Caption = FString::Printf( TEXT("state [%s]"), *State.Name );

		// Locate the "original" tree element :
		// This may happen if for some reason the state is duplicated (should never happen)
		const TSharedPtr<FMutableCodeTreeElement>* MainItemPtr = MainItemPerOp.Find(State.Root);

		// Create a new root element and add it to the collection of root nodes
		TSharedPtr<FMutableCodeTreeElement> RootNodeElement = MakeShareable(new FMutableCodeTreeElement(ItemCache.Num(),StateIndex, MutableModel, State.Root, Caption, SharedThis(this), Program.GetOpType(State.Root), MainItemPtr));
		RootNodes.Add(RootNodeElement);
		
		// Add the element to the cache so we keep the indices straight.
		constexpr UE::Mutable::Private::FOperation::ADDRESS CommonParent = 0;
		const FItemCacheKey Key = { CommonParent, State.Root, StateIndex };
		ItemCache.Add(Key, RootNodeElement);

		if (!MainItemPtr)
		{
			// Cache this node as it may be duplicated of another state. Check the "MainItemPtr" initialization for more info
			MainItemPerOp.Add(State.Root, RootNodeElement);

			// Populate OperatationTypes with the root.
			UE::Mutable::Private::EOpType RootType = Program.GetOpType(State.Root);
			ModelOperationTypes[(int32)RootType].Key = RootType;
			++ModelOperationTypes[(int32)RootType].Value;

			// Iterate over each root node and generate all the elements in a human-readable pattern (Z Pattern)
			GenerateElement(StateIndex, State.Root, Program);
		}
	}
	
}

void SMutableCodeViewer::GenerateElement(const int32& InStateIndex, UE::Mutable::Private::FOperation::ADDRESS InParentAddress,  const UE::Mutable::Private::FProgram& InProgram)
{
	TQueue<UE::Mutable::Private::FOperation::ADDRESS> Addresses;
	Addresses.Enqueue(InParentAddress);
	
	// This will be used to add operations
	auto AddOpFunc = [this, &InProgram, &InStateIndex, &Addresses](UE::Mutable::Private::FOperation::ADDRESS ParentAddress, UE::Mutable::Private::FOperation::ADDRESS ChildAddress, const FString& Caption, uint32& ChildIndex)
	{
		const FItemCacheKey Key = { ParentAddress, ChildAddress, ChildIndex };
		const TSharedPtr<FMutableCodeTreeElement>* CachedItem = ItemCache.Find(Key);
			
		ChildIndex++;

		// If not already cached then process it
		if (ensure(!CachedItem))
		{
			// Locate the "original" tree element
			const TSharedPtr<FMutableCodeTreeElement>* MainItemPtr = MainItemPerOp.Find(ChildAddress);

			const TSharedPtr<FMutableCodeTreeElement> Item = MakeShareable(new FMutableCodeTreeElement(ItemCache.Num(),InStateIndex, MutableModel, ChildAddress, Caption, SharedThis(this), InProgram.GetOpType(ChildAddress), MainItemPtr));

			// Cache this element for later access
			ItemCache.Add(Key, Item);

			// It is not a duplicated of another one, then we can continue searching
			if (!MainItemPtr)
			{
				MainItemPerOp.Add(ChildAddress, Item);

				// Populate OperatationTypes with the root.
				UE::Mutable::Private::EOpType ChildType = InProgram.GetOpType(ChildAddress);
				ModelOperationTypes[(int32)ChildType].Key = ChildType;
				++ModelOperationTypes[(int32)ChildType].Value;
				
				Addresses.Enqueue( ChildAddress );
			}
		}
		else
		{
			UE_LOGF(LogMutable, Error, "An already processed operation is being re-processed in order to generate a tree row. "
							"ParentAddress : %u , ChildAddress : %u , ChildIndex : %u ", ParentAddress, ChildAddress, ChildIndex);
		}
	};
	
	while (!Addresses.IsEmpty())
	{
		UE::Mutable::Private::FOperation::ADDRESS Address;
		Addresses.Dequeue(Address);
		
		// For some specific parent operation types we create more detailed subtrees.
		bool bUseGeneric = false;
		const UE::Mutable::Private::EOpType ParentOperationType = InProgram.GetOpType(Address);
		switch (ParentOperationType)
		{
		case UE::Mutable::Private::EOpType::IM_CONDITIONAL:
		case UE::Mutable::Private::EOpType::LA_CONDITIONAL:
		case UE::Mutable::Private::EOpType::ME_CONDITIONAL:
		case UE::Mutable::Private::EOpType::CO_CONDITIONAL:
		case UE::Mutable::Private::EOpType::SC_CONDITIONAL:
		case UE::Mutable::Private::EOpType::NU_CONDITIONAL:
		case UE::Mutable::Private::EOpType::IN_CONDITIONAL:
		case UE::Mutable::Private::EOpType::ED_CONDITIONAL:
		case UE::Mutable::Private::EOpType::MI_CONDITIONAL:
		case UE::Mutable::Private::EOpType::IS_CONDITIONAL:
		case UE::Mutable::Private::EOpType::SK_CONDITIONAL:
		case UE::Mutable::Private::EOpType::LD_CONDITIONAL:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::ConditionalArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ConditionalArgs>(Address);
				AddOpFunc(Address, Args.condition, TEXT("cond "), ChildIndex);
				AddOpFunc(Address, Args.yes, TEXT("true "), ChildIndex);
				AddOpFunc(Address, Args.no, TEXT("false "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::IM_SWITCH:
		case UE::Mutable::Private::EOpType::LA_SWITCH:
		case UE::Mutable::Private::EOpType::ME_SWITCH:
		case UE::Mutable::Private::EOpType::CO_SWITCH:
		case UE::Mutable::Private::EOpType::SC_SWITCH:
		case UE::Mutable::Private::EOpType::NU_SWITCH:
		case UE::Mutable::Private::EOpType::IN_SWITCH:
		case UE::Mutable::Private::EOpType::ED_SWITCH:
		case UE::Mutable::Private::EOpType::SK_SWITCH:
		case UE::Mutable::Private::EOpType::IS_SWITCH:
		case UE::Mutable::Private::EOpType::MI_SWITCH:
		case UE::Mutable::Private::EOpType::LD_SWITCH:
			{
				uint32 ChildIndex = 0;

				const uint8* OpData = InProgram.GetOpArgsPointer(Address);

				UE::Mutable::Private::FOperation::ADDRESS VarAddress;
				FMemory::Memcpy(&VarAddress, OpData, sizeof(UE::Mutable::Private::FOperation::ADDRESS));
				OpData += sizeof(UE::Mutable::Private::FOperation::ADDRESS);
				AddOpFunc(Address, VarAddress, TEXT("var "), ChildIndex);

				UE::Mutable::Private::FOperation::ADDRESS DefAddress;
				FMemory::Memcpy(&DefAddress, OpData, sizeof(UE::Mutable::Private::FOperation::ADDRESS));
				OpData += sizeof(UE::Mutable::Private::FOperation::ADDRESS);
				AddOpFunc(Address, DefAddress, TEXT("def "), ChildIndex);

				UE::Mutable::Private::FOperation::FSwitchCaseDescriptor CaseDesc;
				FMemory::Memcpy(&CaseDesc, OpData, sizeof(UE::Mutable::Private::FOperation::FSwitchCaseDescriptor));
				OpData += sizeof(UE::Mutable::Private::FOperation::FSwitchCaseDescriptor);

				if (!CaseDesc.bUseRanges)
				{
					for (uint32 C = 0; C < CaseDesc.Count; ++C)
					{
						int32 Condition;
						FMemory::Memcpy(&Condition, OpData, sizeof(int32));
						OpData += sizeof(int32);

						UE::Mutable::Private::FOperation::ADDRESS At;
						FMemory::Memcpy(&At, OpData, sizeof(UE::Mutable::Private::FOperation::ADDRESS));
						OpData += sizeof(UE::Mutable::Private::FOperation::ADDRESS);

						FString Caption = FString::Printf(TEXT("case %d "), Condition);
						AddOpFunc(Address, At, Caption, ChildIndex);
					}
				}
				else
				{
					for (uint32 C = 0; C < CaseDesc.Count; ++C)
					{
						int32 ConditionStart;
						FMemory::Memcpy(&ConditionStart, OpData, sizeof(int32));
						OpData += sizeof(int32);

						uint32 RangeSize;
						FMemory::Memcpy(&RangeSize, OpData, sizeof(uint32));
						OpData += sizeof(uint32);

						UE::Mutable::Private::FOperation::ADDRESS At;
						FMemory::Memcpy(&At, OpData, sizeof(UE::Mutable::Private::FOperation::ADDRESS));
						OpData += sizeof(UE::Mutable::Private::FOperation::ADDRESS);

						FString Caption = FString::Printf(TEXT("case %d to %d "), ConditionStart, ConditionStart+RangeSize-1);
						AddOpFunc(Address, At, Caption, ChildIndex);
					}
				}

				break;
			}

		case UE::Mutable::Private::EOpType::IM_SWIZZLE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageSwizzleArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageSwizzleArgs>(Address);
				for (int32 Channel = 0; Channel < 4; ++Channel)
				{
					FString Caption = FString::Printf(TEXT("%d is %d from "), Channel, Args.sourceChannels[Channel]);
					AddOpFunc(Address, Args.sources[Channel], Caption, ChildIndex);
				}
				break;
			}

		case UE::Mutable::Private::EOpType::CO_SWIZZLE:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::ColorSwizzleArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ColorSwizzleArgs>(Address);
				for (int32 Channel = 0; Channel < 4; ++Channel)
				{
					FString Caption = FString::Printf(TEXT("%d is %d from "), Channel, Args.sourceChannels[Channel]);
					AddOpFunc(Address, Args.sources[Channel], Caption, ChildIndex);
				}
				break;
			}

		case UE::Mutable::Private::EOpType::IM_LAYER:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::ImageLayerArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageLayerArgs>(Address);
				AddOpFunc(Address, Args.base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.mask, TEXT("mask "), ChildIndex);
				AddOpFunc(Address, Args.blended, TEXT("blended "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::IM_LAYERCOLOR:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::ImageLayerColorArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageLayerColorArgs>(Address);
				AddOpFunc(Address, Args.base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.mask, TEXT("mask "), ChildIndex);
				AddOpFunc(Address, Args.color, TEXT("color "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::IM_MULTILAYER:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::ImageMultiLayerArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageMultiLayerArgs>(Address);
				AddOpFunc(Address, Args.rangeSize, TEXT("range "), ChildIndex);
				AddOpFunc(Address, Args.base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.mask, TEXT("mask "), ChildIndex);
				AddOpFunc(Address, Args.blended, TEXT("blended "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::ME_ADDMETADATA:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::MeshAddMetadataArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshAddMetadataArgs>(Address);

				using OpEnumFlags = UE::Mutable::Private::FOperation::MeshAddMetadataArgs::EnumFlags;

				int32 TagCount = EnumHasAnyFlags(Args.Flags, OpEnumFlags::HasGameplayTags) ? 1 : 0;
				if (EnumHasAnyFlags(Args.Flags, OpEnumFlags::IsGameplayTagList) &&
					ensure(InProgram.ConstantUInt32Lists.IsValidIndex(Args.GameplayTags.ListIndex)))
				{
					TagCount = InProgram.ConstantUInt32Lists.IsValidIndex(Args.GameplayTags.ListIndex) 
							? InProgram.ConstantUInt32Lists[Args.GameplayTags.ListIndex].Num() 
							: 0;
				}

				int32 ResourceCount = EnumHasAnyFlags(Args.Flags, OpEnumFlags::HasAnimationSlots) ? 1 : 0;
				if (EnumHasAnyFlags(Args.Flags, OpEnumFlags::IsAnimationSlotList) &&
					ensure(InProgram.ConstantUInt32Lists.IsValidIndex(Args.AnimSlotNames.ListIndex)))
				{
					ResourceCount = InProgram.ConstantUInt32Lists.IsValidIndex(Args.AnimSlotNames.ListIndex) 
							? InProgram.ConstantUInt32Lists[Args.AnimSlotNames.ListIndex].Num() 
							: 0;
				}

				int32 RealTimeMorphNamesCount = EnumHasAnyFlags(Args.Flags, OpEnumFlags::HasRealTimeMorphNames) ? 1 : 0;
				if (EnumHasAnyFlags(Args.Flags, OpEnumFlags::IsRealTimeMorphNamesList) &&
					ensure(InProgram.ConstantUInt32Lists.IsValidIndex(Args.RealTimeMorphNames.ListAddress)))
				{
					RealTimeMorphNamesCount = InProgram.ConstantUInt32Lists.IsValidIndex(Args.RealTimeMorphNames.ListAddress) 
							? InProgram.ConstantUInt32Lists[Args.RealTimeMorphNames.ListAddress].Num() 
							: 0;
				}

				const int32 SkeletonCount = Args.SkeletonId != UE::Mutable::Private::PASSTHROUGH_ID_INVALID ? 1 : 0;
				const int32 PhysicsAssetCount = Args.PhysicsAssetId != UE::Mutable::Private::PASSTHROUGH_ID_INVALID ? 1 : 0;

				int32 AdditionalPhysicsAssetCount = EnumHasAnyFlags(Args.Flags, OpEnumFlags::HasAdditionalPhysicsAsset) ? 1 : 0;
				if (EnumHasAnyFlags(Args.Flags, OpEnumFlags::IsAdditionalPhysicsAssetList) &&
					ensure(InProgram.ConstantUInt32Lists.IsValidIndex(Args.AdditionalPhysicsAssetsIds.ListAddress)))
				{
					AdditionalPhysicsAssetCount = InProgram.ConstantUInt32Lists[Args.AdditionalPhysicsAssetsIds.ListAddress].Num();
				}
			
				int32 AssetUserDataCount = EnumHasAnyFlags(Args.Flags, OpEnumFlags::HasAssetUserData) ? 1 : 0;
				if (EnumHasAnyFlags(Args.Flags, OpEnumFlags::IsAssetUserDataList) &&
					ensure(InProgram.ConstantUInt32Lists.IsValidIndex(Args.AssetUserDataIds.ListAddress)))
				{
					AssetUserDataCount = InProgram.ConstantUInt32Lists[Args.AssetUserDataIds.ListAddress].Num();
				}

				FString Caption = FString::Printf(TEXT("add %d Tags, %d Resources, %d Skeleton, %d PhysicsAsset, %d RealTimeMorphs, %d Additional PhysicsAssets, and %d AssetUserData to"),
					TagCount, ResourceCount, SkeletonCount, PhysicsAssetCount, RealTimeMorphNamesCount, AdditionalPhysicsAssetCount, AssetUserDataCount);
				AddOpFunc(Address, Args.Source, Caption, ChildIndex);

				break;
			}

		case UE::Mutable::Private::EOpType::ME_SETMATERIALSLOTID:
		{
			uint32 ChildIndex = 0;

			UE::Mutable::Private::FOperation::MeshSetMaterialSlotIdArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshSetMaterialSlotIdArgs>(Address);
			AddOpFunc(Address, Args.Mesh, FString::Printf(TEXT("Id % u : Mesh "), Args.MaterialSlotId), ChildIndex);
			break;
		}

		case UE::Mutable::Private::EOpType::ME_APPLYLAYOUT:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::MeshApplyLayoutArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshApplyLayoutArgs>(Address);
				AddOpFunc(Address, Args.Layout, TEXT("layout "), ChildIndex);
				AddOpFunc(Address, Args.Mesh, TEXT("mesh "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::ME_PREPARELAYOUT:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::MeshPrepareLayoutArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshPrepareLayoutArgs>(Address);
				AddOpFunc(Address, Args.Layout, TEXT("layout "), ChildIndex);
				AddOpFunc(Address, Args.Mesh, TEXT("mesh "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::ME_DIFFERENCE:
			{
				uint32 ChildIndex = 0;

				const uint8* data = InProgram.GetOpArgsPointer(Address);

				UE::Mutable::Private::FOperation::ADDRESS BaseAt = 0;
				FMemory::Memcpy(&BaseAt, data, sizeof(UE::Mutable::Private::FOperation::ADDRESS));
				AddOpFunc(Address, BaseAt, TEXT("base "), ChildIndex);
				data += sizeof(UE::Mutable::Private::FOperation::ADDRESS);

				UE::Mutable::Private::FOperation::ADDRESS TargetAt = 0;
				FMemory::Memcpy(&TargetAt, data, sizeof(UE::Mutable::Private::FOperation::ADDRESS));
				AddOpFunc(Address, TargetAt, TEXT("target "), ChildIndex);
				data += sizeof(UE::Mutable::Private::FOperation::ADDRESS);
		
				break;
			}
		
		case UE::Mutable::Private::EOpType::ME_MORPH:
			{
				uint32 ChildIndex = 0;

				const UE::Mutable::Private::FOperation::MeshMorphArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshMorphArgs>(Address);

				FString MorphName = InProgram.ConstantStrings.IsValidIndex(Args.Name) 
					? InProgram.ConstantStrings[Args.Name]
					: FString(TEXT("[Name Not Found]"), ChildIndex);
		
				AddOpFunc(Address, Args.Factor, TEXT("Factor "), ChildIndex);
				AddOpFunc(Address, Args.Base, FString::Printf(TEXT("%s : Base "), *MorphName), ChildIndex);
	
				break;
			}
		
		case UE::Mutable::Private::EOpType::SC_CURVE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ScalarCurveArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ScalarCurveArgs>(Address);
				AddOpFunc(Address, Args.time, TEXT("Time "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::CO_SAMPLEIMAGE:
			{
				uint32 ChildIndex = 0;
			
				UE::Mutable::Private::FOperation::ColorSampleImageArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ColorSampleImageArgs>(Address);
				AddOpFunc(Address, Args.Image, TEXT("image "), ChildIndex);
				AddOpFunc(Address, Args.X, TEXT("x "), ChildIndex);
				AddOpFunc(Address, Args.Y, TEXT("y "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::CO_LINEARTOSRGB:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ColorArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ColorArgs>(Address);
				AddOpFunc(Address, Args.Color, TEXT("Color "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_RESIZELIKE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageResizeLikeArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageResizeLikeArgs>(Address);
				AddOpFunc(Address, Args.Source, TEXT("src "), ChildIndex);
				AddOpFunc(Address, Args.SizeSource, TEXT("sizeSrc "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::LD_NEW:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::FLODNewArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::FLODNewArgs>(Address);

				const TArray<UE::Mutable::Private::FOperation::ADDRESS>& Meshes = InProgram.ConstantUInt32Lists[Args.Meshes];
				for (int32 SectionIndex = 0; SectionIndex < Meshes.Num(); ++SectionIndex)
				{
					AddOpFunc(Address, Meshes[SectionIndex], FString::Printf(TEXT("section %i"), SectionIndex), ChildIndex);
				}

				break;
			}
	
		case UE::Mutable::Private::EOpType::SK_NEW:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::FSkeletalMeshNewArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::FSkeletalMeshNewArgs>(Address);

				const TArray<UE::Mutable::Private::FOperation::CONSTANT_NAME>& MaterialNames = InProgram.ConstantUInt32Lists[Args.MaterialSlotNames];
				const TArray<UE::Mutable::Private::FOperation::ADDRESS>& Materials = InProgram.ConstantUInt32Lists[Args.MaterialSlotMaterials];
				for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
				{
					AddOpFunc(Address, Materials[MaterialIndex], FString::Printf(TEXT("material %s"), *InProgram.ConstantNames[MaterialNames[MaterialIndex]].ToString()), ChildIndex);
				}
			
				const TArray<UE::Mutable::Private::FOperation::ADDRESS>& LODs = InProgram.ConstantUInt32Lists[Args.LODs];
				for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
				{
					AddOpFunc(Address, LODs[LODIndex], FString::Printf(TEXT("lod %i"), LODIndex), ChildIndex);
				}

				break;
			}

		case UE::Mutable::Private::EOpType::SK_MERGE:
		{
			uint32 ChildIndex = 0;

			UE::Mutable::Private::FOperation::FSkeletalMeshMergeArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::FSkeletalMeshMergeArgs>(Address);
			AddOpFunc(Address, Args.BaseMesh, TEXT("Base "), ChildIndex);
			AddOpFunc(Address, Args.AddedMesh, TEXT("Added "), ChildIndex);

			break;
		}

		case UE::Mutable::Private::EOpType::IN_ADDSKELETALMESH:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::InstanceAddArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::InstanceAddArgs>(Address);
				AddOpFunc(Address, Args.instance, TEXT("instance "), ChildIndex);
				AddOpFunc(Address, Args.value, TEXT("value "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IN_ADDCOMPONENT:
			{
				uint32 ChildIndex = 0;
	
				UE::Mutable::Private::FOperation::InstanceAddComponentArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::InstanceAddComponentArgs>(Address);
				AddOpFunc(Address, Args.Instance, TEXT("instance "), ChildIndex);
				AddOpFunc(Address, Args.Value, TEXT("value "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IN_ADDOVERLAYMATERIAL:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::InstanceAddOverlayMaterialArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::InstanceAddOverlayMaterialArgs>(Address);
				AddOpFunc(Address, Args.Instance, TEXT("instance "), ChildIndex);
				AddOpFunc(Address, Args.Material, TEXT("value "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::IN_ADDOVERRIDEMATERIAL:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::InstanceAddOverrideMaterialArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::InstanceAddOverrideMaterialArgs>(Address);
				AddOpFunc(Address, Args.Instance, TEXT("instance "), ChildIndex);
				AddOpFunc(Address, Args.Material, TEXT("value "), ChildIndex);
				break;
			}
	
		case UE::Mutable::Private::EOpType::IM_COMPOSE:
		{
			uint32 ChildIndex = 0;

			UE::Mutable::Private::FOperation::ImageComposeArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageComposeArgs>(Address);
			AddOpFunc(Address, Args.layout, TEXT("Layout "), ChildIndex);
			AddOpFunc(Address, Args.base, TEXT("Base "), ChildIndex);
			AddOpFunc(Address, Args.blockImage, TEXT("BlockImage "), ChildIndex);
			AddOpFunc(Address, Args.mask, TEXT("Mask "), ChildIndex);
			break;
		}

		case UE::Mutable::Private::EOpType::IM_MULTICOMPOSE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageMultiComposeArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageMultiComposeArgs>(Address);
				AddOpFunc(Address, Args.Layout, TEXT("Layout "), ChildIndex);
				AddOpFunc(Address, Args.Base, TEXT("Base "), ChildIndex);
				AddOpFunc(Address, Args.SourceLayout, TEXT("SourceLayout"), ChildIndex);
				AddOpFunc(Address, Args.SourceImage, TEXT("SourceImage"), ChildIndex);
		
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_INTERPOLATE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageInterpolateArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageInterpolateArgs>(Address);
				AddOpFunc(Address, Args.Factor, TEXT("factor "), ChildIndex);
			
				int32 TargetIndex = 0;
				for (const UE::Mutable::Private::FOperation::ADDRESS& Operation : Args.Targets)
				{
					AddOpFunc(Address, Operation, FString::Printf(TEXT("target %i "), TargetIndex++), ChildIndex);
				}
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_SATURATE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageSaturateArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageSaturateArgs>(Address);
				AddOpFunc(Address, Args.Base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.Factor, TEXT("factor "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_COLORMAP:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageColorMapArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageColorMapArgs>(Address);
				AddOpFunc(Address, Args.Base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.Mask, TEXT("mask "), ChildIndex);
				AddOpFunc(Address, Args.Map, TEXT("map "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_BINARISE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageBinariseArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageBinariseArgs>(Address);
				AddOpFunc(Address, Args.Base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.Threshold, TEXT("threshold "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_PATCH:
			{
				uint32 ChildIndex = 0;
	
				UE::Mutable::Private::FOperation::ImagePatchArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImagePatchArgs>(Address);
				AddOpFunc(Address, Args.base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.patch, TEXT("patch "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_RASTERMESH:
			{
				uint32 ChildIndex = 0;
		
				UE::Mutable::Private::FOperation::ImageRasterMeshArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageRasterMeshArgs>(Address);
				AddOpFunc(Address, Args.mesh, TEXT("mesh "), ChildIndex);
				AddOpFunc(Address, Args.image, TEXT("image "), ChildIndex);
				AddOpFunc(Address, Args.mask, TEXT("mask "), ChildIndex);
				AddOpFunc(Address, Args.angleFadeProperties, TEXT("angleFadeProperties "), ChildIndex);
				AddOpFunc(Address, Args.projector, TEXT("projector "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_DISPLACE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageDisplaceArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageDisplaceArgs>(Address);
				AddOpFunc(Address, Args.Source, TEXT("src "), ChildIndex);
				AddOpFunc(Address, Args.DisplacementMap, TEXT("displacementMap "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_NORMALCOMPOSITE:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageNormalCompositeArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageNormalCompositeArgs>(Address);
				AddOpFunc(Address, Args.base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.normal, TEXT("normal "), ChildIndex);
				break;
			}
		
		case UE::Mutable::Private::EOpType::IM_TRANSFORM:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::ImageTransformArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ImageTransformArgs>(Address);
				AddOpFunc(Address, Args.Base, TEXT("base "), ChildIndex);
				AddOpFunc(Address, Args.OffsetX, TEXT("offsetX "), ChildIndex);
				AddOpFunc(Address, Args.OffsetY, TEXT("offsetY "), ChildIndex);
				AddOpFunc(Address, Args.ScaleX, TEXT("scaleX "), ChildIndex);
				AddOpFunc(Address, Args.ScaleY, TEXT("scaleY "), ChildIndex);
				AddOpFunc(Address, Args.Rotation, TEXT("rotation "), ChildIndex);
				break;
			}

		case UE::Mutable::Private::EOpType::SC_MATERIAL_BREAK:
		case UE::Mutable::Private::EOpType::CO_MATERIAL_BREAK:
		case UE::Mutable::Private::EOpType::IM_MATERIAL_BREAK:
			{
				uint32 ChildIndex = 0;

				UE::Mutable::Private::FOperation::MaterialBreakArgs Args = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MaterialBreakArgs>(Address);
				AddOpFunc(Address, Args.Material, TEXT("Material "), ChildIndex);
				break;
			}

			//TODO(Max) Add MI_Parameter and MI_CONSTANT

			// Add here more operation types to define how they are exposed in the tree (set a Caption)
		
		default:
			{
				// Generic list of child operations
				bUseGeneric = true;
				break;
			}
		}

		if (bUseGeneric)
		{
			// Find children of the provided element without adding any extra string to better identify what each of them represents
			uint32 ChildIndex = 0;
			UE::Mutable::Private::ForEachReference(InProgram, Address, [this, AddOpFunc, &ChildIndex, Address](UE::Mutable::Private::FOperation::ADDRESS ChildAddress)
			{
				AddOpFunc(Address, ChildAddress, TEXT(""), ChildIndex);
			});
		}
		else
		{
			// Validate in case there is a mismatch in the custom processing of children and the generic one, which would cause problems.
			uint32 ChildIndex = 0;

			auto ValidateOpFunc = [this, Address, &ChildIndex, &InProgram](UE::Mutable::Private::FOperation::ADDRESS ChildAddress)
				{
					const FItemCacheKey Key = { Address, ChildAddress, ChildIndex };
					const TSharedPtr<FMutableCodeTreeElement>* CachedItem = ItemCache.Find(Key);

					// If this check fails could mean that in the switch above one type of OP processes its children in one
					// order but in "ForEachReference" that same operation processes the same children in another order.
					check(CachedItem);
				
					++ChildIndex;
				};

			UE::Mutable::Private::ForEachReference(InProgram, Address, [this, ValidateOpFunc](UE::Mutable::Private::FOperation::ADDRESS ChildAddress)
			{
				ValidateOpFunc(ChildAddress);
			});
		}
	}
}


SMutableCodeViewer::EOperationComputationalCost SMutableCodeViewer::GetOperationTypeComputationalCost(UE::Mutable::Private::EOpType OperationType) const
{
	if (VeryExpensiveOperationTypes.Contains(OperationType))
	{
		return EOperationComputationalCost::VeryExpensive;
	}
	else if (ExpensiveOperationTypes.Contains(OperationType))
	{
		return EOperationComputationalCost::Expensive;
	}
	else
	{
		return EOperationComputationalCost::Standard;
	}
}


FSlateColor SMutableCodeViewer::GetOperationDataTypeColor(UE::Mutable::Private::EDataType DataType) const
{
	FSlateColor Output = FSlateColor(FColor::White);
	
	switch (DataType)
	{
		case UE::Mutable::Private::EDataType::Bool:
			Output = FSlateColor(FLinearColor(1,0.69f,0.69f, 1));
			break;
		case UE::Mutable::Private::EDataType::Int:
			Output = FSlateColor(FLinearColor(0.5,0.24f,0.45f, 1));
			break;
		case UE::Mutable::Private::EDataType::Scalar:
			Output = FSlateColor(FLinearColor(1,0.71f,0, 1));
			break;
		case UE::Mutable::Private::EDataType::Color:
			Output = FSlateColor(FLinearColor(0.64f,0.73f,0.83f, 1));
			break;
		case UE::Mutable::Private::EDataType::Image:
			Output = FSlateColor(FLinearColor(0.75f,0,0.12f, 1));
			break;
		case UE::Mutable::Private::EDataType::Layout:
			Output = FSlateColor(FLinearColor(0.80f,0.75f,0.38f, 1));
			break;
		case UE::Mutable::Private::EDataType::Mesh:
			Output = FSlateColor(FLinearColor(0.50f,0.43f,0.71f,1));
			break;
		case UE::Mutable::Private::EDataType::Instance:
			Output = FSlateColor(FLinearColor(0,0.48f,0.20f,1));
			break;
		case UE::Mutable::Private::EDataType::Projector:
			Output = FSlateColor(FLinearColor(0.96f,0.46f,0.55f,1));
			break;
		case UE::Mutable::Private::EDataType::String:
			Output = FSlateColor(FLinearColor(0,0.32f,0.53f,1));
			break;
		case UE::Mutable::Private::EDataType::ExtensionData:
			Output = FSlateColor(FLinearColor(1,0.55f,0,1));
			break;
		case UE::Mutable::Private::EDataType::Matrix:
			Output = FSlateColor(FLinearColor(0.69f,0.15f,0.31f,1));
			break;
		case UE::Mutable::Private::EDataType::Material:
			Output = FSlateColor(FLinearColor(0.95f,0.78f,0,1));
			break;
		case UE::Mutable::Private::EDataType::InstancedStruct:
			Output = FSlateColor(FLinearColor(0.49f,0.09f,0.05f,1));
			break;
		case UE::Mutable::Private::EDataType::SkeletalMesh:
			Output = FSlateColor(FLinearColor(0.94f,0.22f,0.84f, 1));
			break;
	}
	
	// Other colors that could be used if one of those does show to not be ideal:
	// Output = FSlateColor(FLinearColor(0.34f,0.19f,0.08f, 1));
	// Output = FSlateColor(FLinearColor(0.57f,0.66f,0,1));
	// Output = FSlateColor(FLinearColor(0.13f,0.17f,0.08f, 1));
	
	return Output;
}



FSlateColor SMutableCodeViewer::GetOperationColor(const UE::Mutable::Private::EOpType OperationType) const
{
	if (EntryLabelColoringPolicy == EEntryLabelColoringPolicy::BY_COST)
	{
		return ColorPerComputationalCost[StaticCast<uint8>(GetOperationTypeComputationalCost(OperationType))];
	}
	else if (EntryLabelColoringPolicy == EEntryLabelColoringPolicy::BY_DATATYPE)
	{
		return GetOperationDataTypeColor(UE::Mutable::Private::GetOpDataType(OperationType));
	}

	checkNoEntry();
	return FLinearColor::White;
}


void SMutableCodeViewer::OnLabelColorPolicyChange(FName PolicyName, ESelectInfo::Type Arg)
{
	check(EntryLabelColoringPolicyNames.Num() == (int32)EEntryLabelColoringPolicy::COUNT);
	
	const int32 IndexOnNameArray = EntryLabelColoringPolicyNames.Find(PolicyName);
	check(IndexOnNameArray != INDEX_NONE);
	check(EntryLabelColoringPolicyNames.IsValidIndex(IndexOnNameArray))

	EntryLabelColoringPolicy = StaticCast<EEntryLabelColoringPolicy>(IndexOnNameArray);
}


TSharedRef<SWidget> SMutableCodeViewer::OnLabelColorPolicyGenerateWidget(FName PolicyName)
{
	TSharedRef<STextBlock> NewSlateObject = SNew(STextBlock)
		.Text(FText::FromName(PolicyName));

	return NewSlateObject;
}


FText SMutableCodeViewer::GetCurrentLabelColoringPolicyText() const
{
	check(EntryLabelColoringPolicyNames.IsValidIndex((int32)EntryLabelColoringPolicy));
	const FName CurrentPolicyName = EntryLabelColoringPolicyNames[(int32)EntryLabelColoringPolicy];
	
	return FText::FromName(CurrentPolicyName);
}


#pragma endregion 



#pragma region CodeTree Callbacks


TSharedRef<ITableRow> SMutableCodeViewer::GenerateRowForNodeTree(TSharedPtr<FMutableCodeTreeElement> InTreeNode, const TSharedRef<STableViewBase>& InOwnerTable)
{
	// Save the node for later access
	TreeElements.Add(InTreeNode);
	
	// Generate a row element
	TSharedRef<SMutableCodeTreeRow> Row = SNew(SMutableCodeTreeRow, InOwnerTable, InTreeNode);

	// Determine if a row should be painted as highlighted based on the selected item
	if (TreeView->GetNumItemsSelected())
	{
		const TSharedPtr<FMutableCodeTreeElement> SelectedElement = TreeView->GetSelectedItems()[0];
		if (SelectedElement != InTreeNode &&
			SelectedElement->MutableOperation > 0 &&
			InTreeNode->MutableOperation == SelectedElement->MutableOperation)
		{
			Row->Highlight();
		}
	}
	
	return Row;
}


void SMutableCodeViewer::GetChildrenForInfo(TSharedPtr<FMutableCodeTreeElement> InInfo, TArray<TSharedPtr<FMutableCodeTreeElement>>& OutChildren)
{
	if (!InInfo->MutableModel)
	{
		return;
	}

	check(MutableModel);
	const UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram();

	UE::Mutable::Private::FOperation::ADDRESS ParentAddress = InInfo->MutableOperation;

	// Generic case for unnamed children traversal.
	uint32 ChildIndex = 0;
	UE::Mutable::Private::ForEachReference(Program, InInfo->MutableOperation, [this, ParentAddress, &ChildIndex, &OutChildren](UE::Mutable::Private::FOperation::ADDRESS ChildAddress)
	{
		{
			const FItemCacheKey Key = { ParentAddress, ChildAddress, ChildIndex };
			const TSharedPtr<FMutableCodeTreeElement>* CachedItem = ItemCache.Find(Key);

			if (CachedItem)
			{
				OutChildren.Add(*CachedItem);
			}
			else
			{
				// if all elements have been already cached this should never happen
				checkNoEntry();
			}
		}
		++ChildIndex;
	});
}

void SMutableCodeViewer::OnExpansionChanged(TSharedPtr<FMutableCodeTreeElement> InItem, bool bInExpanded)
{
	// Update expanded state of the provided element
	InItem->bIsExpanded = bInExpanded;
	
	// If an element gets expanded then contract (if found) the other element that uses the same address
	if (bInExpanded)
	{
		const UE::Mutable::Private::FOperation::ADDRESS MutableOperation = InItem->MutableOperation;
		const TSharedPtr<FMutableCodeTreeElement>* PreviouslyExpandedElement = ExpandedElements.Find(MutableOperation);
		if (PreviouslyExpandedElement)
		{
			TreeView->SetItemExpansion(*PreviouslyExpandedElement, false);
		}

		// Only do this if in a situation where it may be required (do not do it if the tree has not been interacted with yet)
		if (bShouldRecalculateStates)
		{
			// Find all the children (recursive) of this item.
			TSet<TSharedPtr<FMutableCodeTreeElement>> FoundChildren;
			GetVisibleChildren(InItem, FoundChildren);
			for (const TSharedPtr<FMutableCodeTreeElement>& Child : FoundChildren)
			{
				// For each of the children found set it's state to be the one found on the expanded element
				Child->SetElementCurrentState(InItem->GetStateIndex());
			}
		}
		
		// Cache this element as one currently expanded
		ExpandedElements.Add(MutableOperation, InItem);
	}
	else
	{
		// Remove this element from the cache of expanded elements
		ExpandedElements.Remove(InItem->MutableOperation);
	}
}

void SMutableCodeViewer::GetVisibleChildren(TSharedPtr<FMutableCodeTreeElement> InInfo, TSet<TSharedPtr<FMutableCodeTreeElement>>& OutChildren)
{
	check(MutableModel);
	const UE::Mutable::Private::FProgram& MutableProgram = MutableModel->GetProgram();
	
	TArray<TSharedPtr<FMutableCodeTreeElement>> ToSearchForChildren;
	ToSearchForChildren.Add(InInfo);
	while (!ToSearchForChildren.IsEmpty())
	{
		// Grab the first element in order to check for it's children
		const TSharedPtr<FMutableCodeTreeElement> ToCheck = ToSearchForChildren[0];
		ToSearchForChildren.RemoveAt(0);
		
		const UE::Mutable::Private::FOperation::ADDRESS ParentAddress = ToCheck->MutableOperation;
	
		// Generic case for unnamed children traversal.
		uint32 ChildIndex = 0;
		UE::Mutable::Private::ForEachReference(MutableProgram, ParentAddress, [this, ParentAddress, &ChildIndex, &OutChildren, &ToSearchForChildren ](UE::Mutable::Private::FOperation::ADDRESS ChildAddress)
		{
			{
				const FItemCacheKey Key = { ParentAddress, ChildAddress, ChildIndex };
				const TSharedPtr<FMutableCodeTreeElement> CachedItem = *ItemCache.Find(Key);

				// Since we have already generated all elements CachedItem should be therefore always a valid pointer
				check (CachedItem);
				
				// If the address has not been yet found then save it as one of the children affected
				if (!OutChildren.Contains(CachedItem))
				{
					OutChildren.Add(CachedItem);

					// And if the children is found to be expanded then also process it later to later return only the
					// elements that are expanded in the tree view (using data manually set on each tree element)
					if (CachedItem->bIsExpanded )
					{
						// Add for processing
						ToSearchForChildren.Add(CachedItem);
					}
				}
			}
			++ChildIndex;
		});
	}

	// Debug
	// UE_LOGF(LogTemp,Warning, "Found a total of %i children elements ",OutChildren.Num());
}


void SMutableCodeViewer::OnSelectionChanged(TSharedPtr<FMutableCodeTreeElement> InNode, ESelectInfo::Type InSelectInfo)
{
	if (bIsElementHighlighted)
	{
		ClearHighlightedItems();
	}

	TArray<TSharedPtr<FMutableCodeTreeElement>> SelectedNodes;
	TreeView->GetSelectedItems(SelectedNodes);

	PreviewBorder->ClearContent();

	SelectedOperationAddress = 0;
	bSelectedOperationIsImage = false;

	if (SelectedNodes.IsEmpty())
	{
		return;
	}

	// Clear all selected items in the constant resources widget
	ConstantsWidget->ClearSelectedConstantItems();
	
	// Find the duplicates for the selected tree element and highlight them
	if (InNode)
	{
		HighlightDuplicatesOfEntry(InNode);
	}

	bIsPreviewPendingUpdate = true;

	SelectedOperationAddress = SelectedNodes[0]->MutableOperation;
	const UE::Mutable::Private::EOpType OperationType = MutableModel->GetProgram().GetOpType(SelectedOperationAddress);
	const UE::Mutable::Private::EDataType OperationDataType = UE::Mutable::Private::GetOpDataType(OperationType);
	
	switch (OperationDataType)
	{
	case UE::Mutable::Private::EDataType::Layout:
	{
		// Create or reuse the UI
		PrepareLayoutViewer();
		break;
	}

	case UE::Mutable::Private::EDataType::Image:
	{
		// Create or reuse the UI
		bSelectedOperationIsImage = true;
		PrepareImageViewer();
		break;
	}

	case UE::Mutable::Private::EDataType::Mesh:
	{
		// Create or reuse the UI
		PrepareMeshViewer();
		break;
	}

	case UE::Mutable::Private::EDataType::Material:
	{
		// Create or reuse the UI
		PrepareMaterialViewer();
		break;
	}
	
	case UE::Mutable::Private::EDataType::Scalar:
	{
		// Create or reuse the UI
		if (!PreviewScalarViewer)
		{
			PreviewScalarViewer = SNew(SMutableScalarViewer);
		}

		PreviewBorder->SetContent(PreviewScalarViewer.ToSharedRef());
		break;
	}

	case UE::Mutable::Private::EDataType::String:
	{
		// Create or reuse the UI
		PrepareStringViewer();
		break;
	}

	case UE::Mutable::Private::EDataType::Color:
	{
		// Create or reuse the UI
		if (!PreviewColorViewer)
		{
			PreviewColorViewer = SNew(SMutableColorViewer);
		}

		PreviewBorder->SetContent(PreviewColorViewer.ToSharedRef());
		break;
	}

	case UE::Mutable::Private::EDataType::Int:
	{
		// Create or reuse the UI
		if (!PreviewIntViewer)
		{
			PreviewIntViewer = SNew(SMutableIntViewer);
		}

		PreviewBorder->SetContent(PreviewIntViewer.ToSharedRef());
		break;
	}

	case UE::Mutable::Private::EDataType::Bool:
	{
		// Create or reuse the UI
		if (!PreviewBoolViewer)
		{
			PreviewBoolViewer = SNew(SMutableBoolViewer);
		}

		PreviewBorder->SetContent(PreviewBoolViewer.ToSharedRef());
		break;
	}

	case UE::Mutable::Private::EDataType::Projector:
	{
		// Create or reuse the UI
		PrepareProjectorViewer();
		break;
	}

	
	default:
		// There is no viewer for this type yet.
		break;

	}
}

TSharedPtr<SWidget> SMutableCodeViewer::OnTreeContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	// Only show the Ui for operations different from "None" or "0"
	if (TreeView->GetSelectedItems().Num() && TreeView->GetSelectedItems()[0]->MutableOperation > 0)
	{
		if (TreeView->GetSelectedItems().Num() == 1)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("Set_as_search_operation_type","Set as search Operation"),
				LOCTEXT("Set_as_search_operation_type_Tooltip", "Sets the type of this operation as the type to be looking for when searching for operations on the tree view"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SMutableCodeViewer::OnSelectedOperationTypeFromTree)
				)	
			);
		}

		MenuBuilder.AddMenuSeparator();
		
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Code_Expand_Selected", "Expand Selected Operation"),
			LOCTEXT("Code_Expand_Selected_Tooltip", "Expands only the selected Operation and leaves the other as they are."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SMutableCodeViewer::TreeExpandSelected)
			)
		);
	}


	MenuBuilder.AddMenuEntry(
		LOCTEXT("Code_Expand_Instance", "Expand Instance-Level Operations"),
		LOCTEXT("Code_Expand_Instance_Tooltip", "Expands all the operations in the tree that are instance operations (not images, meshes, booleans, etc.)."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SMutableCodeViewer::TreeExpandInstance)
			//, FCanExecuteAction::CreateSP(this, &SMutableCodeViewer::HasAnyItemInPalette)
		)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Code_Expand_Unique", "Expand All Unique Operations"),
		LOCTEXT("Code_Expand_Unique_Tooltip", "Expands all the operations in the tree that have not been expanded yet."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SMutableCodeViewer::TreeExpandUnique)
		)
	);

	return MenuBuilder.MakeWidget();
}

void SMutableCodeViewer::TreeExpandRecursive(TSharedPtr<FMutableCodeTreeElement> InInfo, bool bExpand)
{
	if (bExpand)
	{
		TreeExpandUnique();
	}
}

void SMutableCodeViewer::OnRowReleased(const TSharedRef<ITableRow>& InTreeRow)
{
	SMutableCodeTreeRow* CastedTableRow = static_cast<SMutableCodeTreeRow*>(&InTreeRow.Get());
	const TSharedPtr<FMutableCodeTreeElement>& RowElement = CastedTableRow->GetItem();
	TreeElements.Remove(RowElement);
}
#pragma endregion


#pragma region Highlight Methods
void SMutableCodeViewer::HighlightDuplicatesOfEntry(const TSharedPtr<FMutableCodeTreeElement>& InTargetEntry)
{
	if (bIsElementHighlighted)
	{
		ClearHighlightedItems();
	}

	// Do not highlight empty entries
	if (InTargetEntry->MutableOperation == 0)
	{
		return;
	}
	
	// Highlight the elements related to the currently selected item of the tree
	HighlightedOperation = InTargetEntry->MutableOperation;

	for (const TSharedPtr<FMutableCodeTreeElement>& TreeItem : TreeElements)
	{
		if (TreeItem.Get() != InTargetEntry.Get() && TreeItem->MutableOperation == HighlightedOperation)
		{
			TSharedPtr<ITableRow> TableRow = TreeView->WidgetFromItem(TreeItem);
			SMutableCodeTreeRow* MutableRow = static_cast<SMutableCodeTreeRow*>(TableRow.Get());
			MutableRow->Highlight();
		}
	}

	bIsElementHighlighted = true;
}


void SMutableCodeViewer::ClearHighlightedItems()
{
	// Clear the previously highlighted elements
	for (const TSharedPtr<FMutableCodeTreeElement>& HighlightedElement : TreeElements)
	{
		if (HighlightedElement->MutableOperation == HighlightedOperation)
		{
			TSharedPtr<ITableRow> TableRow = TreeView->WidgetFromItem(HighlightedElement);

			if (TableRow.IsValid())
			{
				SMutableCodeTreeRow* MutableRow = static_cast<SMutableCodeTreeRow*>(TableRow.Get());
				MutableRow->ResetHighlight();
			}
		}
	}

	bIsElementHighlighted = false;
}
#pragma endregion

#pragma region Element Expansion Llogic


void SMutableCodeViewer::TreeExpandElements(TArray<TSharedPtr<FMutableCodeTreeElement>>& InElementsToExpand, 
	bool bForceExpandDuplicates /*= false*/,
	UE::Mutable::Private::EDataType FilteringDataType /*= UE::Mutable::Private::EDataType::None*/,
	TSharedPtr<FProcessedOperationsBuffer> InExpandedOperationsBuffer /* = nullptr */)
{
	if (InElementsToExpand.IsEmpty())
	{
		return;
	}
	
	// Initialization of recursive elements if this is the first invocation of method
	{
		if (!InExpandedOperationsBuffer)
		{
			InExpandedOperationsBuffer = MakeShared<FProcessedOperationsBuffer>();
		}
	}

	// Load references to the arrays containing all the operations already worked on during another recursive call to this
	// method
	TArray<UE::Mutable::Private::FOperation::ADDRESS>& AlreadyExpandedOriginalOperations = InExpandedOperationsBuffer->ExpandedOriginalOperations;
	TArray<UE::Mutable::Private::FOperation::ADDRESS>& AlreadyExpandedDuplicatedOperations = InExpandedOperationsBuffer->ExpandedDuplicatedOperations;
	
	// Array containing the children object found on Item.
	TArray<TSharedPtr<FMutableCodeTreeElement>> Children;
	
	// Index of the current element being processed
	int32 CurrentElementIndex = 0;
	while (CurrentElementIndex < InElementsToExpand.Num() )
	{
		// Grab the current element to process and move the index forward once
		TSharedPtr<FMutableCodeTreeElement> Item = InElementsToExpand[CurrentElementIndex++];
		check(Item);

		// Identifier of the element. May be repeated if there are elements duplicating another element
		const UE::Mutable::Private::FOperation::ADDRESS OperationAddress = Item->MutableOperation;
		
		// Filter the elements being expanded if the user has defined a desired EDataType
		if (FilteringDataType != UE::Mutable::Private::EDataType::None)
		{
			const UE::Mutable::Private::EOpType OperationType = Item->MutableModel->GetProgram().GetOpType(OperationAddress);
			const UE::Mutable::Private::EDataType OperationDataType = UE::Mutable::Private::GetOpDataType(OperationType);

			// If it is not of the desired type then ignore it and continue to the next pending element
			if (OperationDataType != FilteringDataType)
			{
				continue;
			}
		}

		// Reset the children array
		Children.SetNum(0);
		
		// If not duplicated expand it and grab the children to be also expanded on the next loop
		if (Item->DuplicatedOf == nullptr )
		{
			// Was this unique element expanded before (only valid if also expanding duplicates)
			bool bHasBeenExpandedPreviously = false;
			
			// Mind duplicated original elements if dealing with duplicated operation expansions.
			if (bForceExpandDuplicates)
			{
				// Make sure we have not already expanded this item to avoid recursive expansions of the same item and
				// children
				bHasBeenExpandedPreviously = AlreadyExpandedOriginalOperations.Contains(OperationAddress);
			}
			
			// Only check for duplicated original elements when working with duplicates
			if (!bHasBeenExpandedPreviously )
			{
				// Get the children of this unique element and prepare them for processing
				GetChildrenForInfo(Item, Children);

				// Call for the expansion of the children first
				TreeExpandElements(Children, bForceExpandDuplicates, FilteringDataType,InExpandedOperationsBuffer);

				// At this point all the children objects that needed expansion are already expanded so we can proceed with
				// the expansion of this element
				
				{
					// If we do expect to expand duplicates make sure we record this object as being expanded to be later able
					// to block the expansion of duplicates of this object
					if (bForceExpandDuplicates)
					{
						// Register this node as expanded so other nodes are able to check if it has already been worked with
						AlreadyExpandedOriginalOperations.Add(OperationAddress);
					}

					// Only ask for the expansion of the element if we know it can be expanded due to it having
					// children
					if (!Children.IsEmpty())
					{
						// Expand this unique element
						TreeView->SetItemExpansion(Item, true);
					}
				}
			}
			
		}
		// If it is a duplicated node
		else
		{
			// Special behavior where we expand duplicates if parent is not found to be expanded
			if (bForceExpandDuplicates)
			{
				// Was this element expanded as an original operation? We only want to expand the duplicate if the original
				// was not duplicated before
				bool bOriginalElementHasBeenExpanded = false;

				// Was this element expanded on a duplicated element? we only want to expand the first duplicate!
				const bool bOtherDuplicateOfSameOpWasExpandedBefore = AlreadyExpandedDuplicatedOperations.Contains(OperationAddress);

				// Only check if there is another original element with the same operation if we know that there is not another
				// duplicated element using this operation
				if (!bOtherDuplicateOfSameOpWasExpandedBefore)
				{
					bOriginalElementHasBeenExpanded = AlreadyExpandedOriginalOperations.Contains(OperationAddress);
				}
				
				// Was this operation expanded before?
				const bool bWasOperationExpandedPreviously = bOriginalElementHasBeenExpanded || bOtherDuplicateOfSameOpWasExpandedBefore;

				// If this operation have not yet been expanded then expand it!
				// Duplicates do not have priority over original elements.
				if ( !bWasOperationExpandedPreviously )
				{
					// Mark the children to be expanded later if conditions are met
					GetChildrenForInfo(Item, Children);

					// Expand the children objects
					TreeExpandElements(Children, bForceExpandDuplicates, FilteringDataType,InExpandedOperationsBuffer);

					// At this point all the children objects that needed expansion are already expanded so we can proceed with
					// the expansion of this element
					
					{
						// Record this node being expanded
						AlreadyExpandedDuplicatedOperations.Add(OperationAddress);

						// Only ask for the expansion of the element if we know it can be expanded due to it having
						// children
						if (!Children.IsEmpty())
						{
							// Expand the current element since we know it is from a operation not yet expanded
							TreeView->SetItemExpansion(Item, true);
						}
					}
				}
			}
		}
	}
}


void SMutableCodeViewer::TreeExpandSelected()
{
	// Get the selected items and expand them excluding the duplicates
	TArray<TSharedPtr<FMutableCodeTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	TreeExpandElements(SelectedItems,true);
}


void SMutableCodeViewer::TreeExpandUnique()
{
	// Expand the tree from the root and do not expand the duplicated elements
	TreeExpandElements(RootNodes);
}

void SMutableCodeViewer::TreeExpandInstance()
{
	// Expand only the items that match the datatype provided
	TreeExpandElements(RootNodes,false, UE::Mutable::Private::EDataType::Instance);
}


void SMutableCodeViewer::CacheRootNodeAddresses()
{
	check (MutableModel);
	check (RootNodeAddresses.IsEmpty())

	TArray<UE::Mutable::Private::FOperation::ADDRESS> FoundRootNodeAddresses;
	
	const int32 StateCount = MutableModel->GetProgram().States.Num();
	for ( int32 StateIndex=0; StateIndex < StateCount; ++StateIndex )
	{
		const UE::Mutable::Private::FProgram::FState& State = MutableModel->GetProgram().States[StateIndex];
		FoundRootNodeAddresses.Add(State.Root);
	}

	RootNodeAddresses = MoveTemp(FoundRootNodeAddresses);
}

 void SMutableCodeViewer::CacheAddressesRelatedWithConstantResource(const UE::Mutable::Private::EDataType ConstantDataType,
	const int32 IndexOnConstantsArray)
{
	check(MutableModel);
	check(RootNodeAddresses.Num());

	if (IndexOnConstantsArray < 0)
	{
		// Not valid index.
		UE_LOGF(LogTemp,Error, "The provided index [%d] is not valid.",IndexOnConstantsArray );
		return;
	}

	// Object containing all data required by the search operation to be able to be called recursively
	FElementsSearchCache SearchPayload;
	// Initialize the Search Payload with the root node addresses. This way the search will use them as the root nodes where
	// to start searching
	SearchPayload.SetupRootBatch(RootNodeAddresses);
	
	// Main update procedure run for the targeted state and the targeted parameter values
	const UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram(); 
	GetOperationsReferencingConstantResource(ConstantDataType,IndexOnConstantsArray,SearchPayload, Program);
	
	// At this point we did get all the addresses of operations that do involve the usage of our resource
	if (SearchPayload.FoundElements.Num() > 0)
	{
		// Set the type operation type to CONST_BASED_NAVIGATION (used to tell the user what is happening)
		TargetedTypeSelector->SetSelectedItem(ConstantBasedNavigationEntry);

		// Dump the located resources array onto the navigation array since we have content to navigate over
		NavigationElements = MoveTemp(SearchPayload.FoundElements);
		SortElementsByTreeIndex(NavigationElements);
	}
	else
	{
		TargetedTypeSelector->SetSelectedItem(NoneOperationEntry);
		NavigationElements.Reset();
		UE_LOGF(LogTemp,Error, "The provided constant index does not seem to be used anywhere : Make sure the index is valid and that IsConstantResourceUsedByOperation() switch is up to date");
	}

	// Reset the navigation index
	NavigationIndex = -1;
}


void SMutableCodeViewer::GetOperationsReferencingConstantResource(
	const UE::Mutable::Private::EDataType ConstantDataType,
	const int32 IndexOnConstantsArray,
	FElementsSearchCache& InSearchPayload,
	const UE::Mutable::Private::FProgram& InProgram)
{
	// next batch of addresses to be explored 
	TArray<FItemCacheKey> NextBatchAddressesData;
	
	for	(int32 ParentIndex = 0 ; ParentIndex < InSearchPayload.BatchData.Num(); ParentIndex++)
	{
		// Get one of the previous run "children" and treat as a parent to get it's children and process them
		const UE::Mutable::Private::FOperation::ADDRESS& ParentAddress = InSearchPayload.BatchData[ParentIndex].Child;
		
		// Cache if same data type and we share the same address (means this op is pointing at the provided resource)
		// It will cache duplicated entries
		if (IsConstantResourceUsedByOperation(IndexOnConstantsArray, ConstantDataType, ParentAddress,InProgram))
		{
			// Since this element is related with the provided constant resource cache it on InSearchPayload.FoundElements
			InSearchPayload.AddToFoundElements(ParentAddress,ParentIndex,ItemCache);
		}
		
		// Get all NON PROCESSED the children of this operation to later be able to process them (on next recursive call)
		InSearchPayload.CacheChildrenOfAddressIfNotProcessed(ParentAddress, InProgram, NextBatchAddressesData);
	}

	// At this point all the addresses to be computed on the next batch have already been set and will be computed on
	// the next recursive call
	
	// Explore children if found 
	if (NextBatchAddressesData.Num())
	{
		// Cache next batch data so the next invocations is able to locate the provided addresses on the itemsCache
		InSearchPayload.BatchData = MoveTemp(NextBatchAddressesData);
		
		GetOperationsReferencingConstantResource(ConstantDataType, IndexOnConstantsArray, InSearchPayload, InProgram);
	}
}

bool SMutableCodeViewer::IsConstantResourceUsedByOperation(const int32 IndexOnConstantsArray,
	const UE::Mutable::Private::EDataType ConstantDataType, const UE::Mutable::Private::FOperation::ADDRESS OperationAddress, const UE::Mutable::Private::FProgram& InProgram) const
{
	// Cache the current operation type to know where to look and what to check
	const UE::Mutable::Private::EOpType OperationType = InProgram.GetOpType(OperationAddress);

	// Making usage of the operation data type is not valid since some operations while return one type do, in fact,
	// contain data from other types (like the mesh constant for example that contains mesh, skeleton and physics asset)
	// const UE::Mutable::Private::EDataType DataType = UE::Mutable::Private::GetOpDataType(OperationType);
	// if (DataType != ConstantDataType)
	// {
	// 	return false;
	// }

	// Is this operation referencing (by an index) the index we are providing from a constants array
	bool bResourceLocated = false;
	
	// Check if the operation data type is compatible with the type of resources we are providing
	switch (ConstantDataType)
	{
		case UE::Mutable::Private::EDataType::String:
			{
				// TIP: To know if they represent a constant value check the code on code runner to see if they read from the constants array
				if (OperationType == UE::Mutable::Private::EOpType::ST_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ResourceConstantArgs>(OperationAddress).value;
				}
				else if (OperationType == UE::Mutable::Private::EOpType::IN_CONDITIONAL)
				{
					UE::Mutable::Private::FOperation::ConditionalArgs Arguments = InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ConditionalArgs>(OperationAddress);
					
					if (IndexOnConstantsArray == Arguments.condition)
					{
						bResourceLocated = true;
						break;
					}

					if (IndexOnConstantsArray == Arguments.yes)
					{
						bResourceLocated = true;
						break;
					}

					if (IndexOnConstantsArray == Arguments.no)
					{
						bResourceLocated = true;
						break;
					}
				}

				break;
			}

		case UE::Mutable::Private::EDataType::Image:
			{
				if (OperationType == UE::Mutable::Private::EOpType::IM_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ResourceConstantArgs>(OperationAddress).value;
				} 
				
				break;
			}

		case UE::Mutable::Private::EDataType::Mesh:
			{
				if (OperationType == UE::Mutable::Private::EOpType::ME_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshConstantArgs>(OperationAddress).Value;
				}
				break;
			}

		case UE::Mutable::Private::EDataType::Layout:
			{
				if (OperationType == UE::Mutable::Private::EOpType::LA_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ResourceConstantArgs>(OperationAddress).value;
				}
				break;
			}

		case UE::Mutable::Private::EDataType::Projector:
			{
				if (OperationType == UE::Mutable::Private::EOpType::PR_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ResourceConstantArgs>(OperationAddress).value;
				}
				break;
			}

		case UE::Mutable::Private::EDataType::Matrix:
			{
				if (OperationType == UE::Mutable::Private::EOpType::ME_TRANSFORM)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshTransformArgs>(OperationAddress).matrix;
				}
				else if (OperationType == UE::Mutable::Private::EOpType::ME_TRANSFORMWITHMESH)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshTransformWithinMeshArgs>(OperationAddress).matrix;
				}
				else if (OperationType == UE::Mutable::Private::EOpType::ME_TRANSFORMWITHBONE)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshTransformWithBoneArgs>(OperationAddress).Matrix;
				}
				else if (OperationType == UE::Mutable::Private::EOpType::MA_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MatrixConstantArgs>(OperationAddress).value;
				}
				break;
			}

		case UE::Mutable::Private::EDataType::Shape:
			{
				if (OperationType == UE::Mutable::Private::EOpType::ME_CLIPMORPHPLANE)
				{
					const UE::Mutable::Private::FOperation::MeshClipMorphPlaneArgs Arguments =  InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshClipMorphPlaneArgs>(OperationAddress);
					
					// Morph shape
					bResourceLocated = IndexOnConstantsArray == Arguments.MorphShape;
					if (bResourceLocated)
					{
						break;
					}

					if (Arguments.VertexSelectionType == EClipVertexSelectionType::Shape)
					{
						// Selection Shape
						bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshClipMorphPlaneArgs>(OperationAddress).VertexSelectionShapeOrBone;
					}
				}
				break;
			}
		
		case UE::Mutable::Private::EDataType::Curve:
			{
				if (OperationType == UE::Mutable::Private::EOpType::SC_CURVE)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::ScalarCurveArgs>(OperationAddress).curve;
				}
				break;
			}

		case UE::Mutable::Private::EDataType::Skeleton:
			{
				if (OperationType == UE::Mutable::Private::EOpType::ME_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshConstantArgs>(OperationAddress).Skeleton;
				}
				break;
			}
		case UE::Mutable::Private::EDataType::PhysicsAsset:
			{
				if (OperationType == UE::Mutable::Private::EOpType::ME_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MeshConstantArgs>(OperationAddress).Value;
				}
				break;
			}

		case UE::Mutable::Private::EDataType::Material:
			{
				if (OperationType == UE::Mutable::Private::EOpType::MI_CONSTANT)
				{
					bResourceLocated = IndexOnConstantsArray == InProgram.GetOpArgs<UE::Mutable::Private::FOperation::MaterialConstantArgs>(OperationAddress).Value;
				}
				break;
			}

		// Invalid types
		case UE::Mutable::Private::EDataType::None:
		default:
			{
				checkNoEntry();
			}
	}


	return bResourceLocated;
}


void SMutableCodeViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	// After the tick we do know the tree has been refreshed, so all expansion and contraction operations have been
	// completed and the new data has been loaded onto our listening arrays. Then its safe to expect the widgets to be
	// there to be selected or inspected.
	if (!TreeView->IsPendingRefresh())
	{
		/** If we have expanded the tree elements in order to reach one of them then continue the operation */
		if (bWasUniqueExpansionInvokedForNavigation || bWasScrollToTargetRequested)
		{
			FocusViewOnNavigationTarget(ToFocusElement);
		}
	}
	
	if (!bIsPreviewPendingUpdate)
	{
		return;
	}

	bIsPreviewPendingUpdate = false;

	const UE::Mutable::Private::EOpType OperationType = MutableModel->GetProgram().GetOpType(SelectedOperationAddress);
	const UE::Mutable::Private::EDataType OperationDataType = UE::Mutable::Private::GetOpDataType(OperationType);

	UE::Mutable::Private::FSettings Settings;
	const TSharedPtr<UE::Mutable::Private::FSystem> System = MakeShared<UE::Mutable::Private::FSystem>(Settings);
	const TSharedRef<UE::Mutable::Private::FModelReader> ModelReader = UCustomizableObjectSystem::GetInstance()->GetPrivate()->MutableSystem->StreamInterface.ToSharedRef();

	System->SetStreamingInterface(ModelReader);
	TSharedRef<UE::Mutable::Private::FLiveInstance> TempLiveInstance = System->NewBuildInstance(MutableModel, PreviewParameters, ExternalResourceProvider);
	
	switch (OperationDataType)
	{
	case UE::Mutable::Private::EDataType::Layout:
	{
		check(PreviewLayoutViewer);
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout> MutableLayout = System->BuildLayout(TempLiveInstance, SelectedOperationAddress);
		PreviewLayoutViewer->SetLayout(MutableLayout);
		break;
	}

	case UE::Mutable::Private::EDataType::Image:
	{
		check(PreviewImageViewer);
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage = System->BuildImage(TempLiveInstance, SelectedOperationAddress, MipsToSkip);
		PreviewImageViewer->SetImage(MutableImage, 0);
		break;
	}

	case UE::Mutable::Private::EDataType::Mesh:
	{
		check(PreviewMeshViewer);
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> MutableMesh = System->BuildMesh(TempLiveInstance, SelectedOperationAddress, UE::Mutable::Private::EMeshContentFlags::AllFlags);
		PreviewMeshViewer->SetMesh(MutableMesh);
		break;
	}
	
	case UE::Mutable::Private::EDataType::Bool:
	{
		check(PreviewBoolViewer);
		const bool MutableBool = System->BuildBool(TempLiveInstance, SelectedOperationAddress);
		PreviewBoolViewer->SetBool(MutableBool);
		break;
	}

	case UE::Mutable::Private::EDataType::Int:
	{
		check(PreviewIntViewer);
		const int32 MutableInt = System->BuildInt(TempLiveInstance, SelectedOperationAddress);
		PreviewIntViewer->SetInt(MutableInt);
		break;
	}

	case UE::Mutable::Private::EDataType::Scalar:
	{
		check(PreviewScalarViewer);
		const float MutableScalar = System->BuildScalar(TempLiveInstance, SelectedOperationAddress);
		PreviewScalarViewer->SetScalar(MutableScalar);
		break;
	}

	case UE::Mutable::Private::EDataType::String:
	{
		check(PreviewStringViewer);
		UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::String> MutableString = System->BuildString(TempLiveInstance, SelectedOperationAddress);
		const FText MutableText = FText::FromString(FString(MutableString->GetValue()));
		PreviewStringViewer->SetString(MutableText);
		break;
	}

	case UE::Mutable::Private::EDataType::Color:
	{
		check(PreviewColorViewer);
		FVector4f Color = System->BuildColor(TempLiveInstance, SelectedOperationAddress);
		PreviewColorViewer->SetColor(Color);
		break;
	}

	case UE::Mutable::Private::EDataType::Projector:
	{
		check (PreviewProjectorViewer);
		UE::Mutable::Private::FProjector Projector = System->BuildProjector(TempLiveInstance, SelectedOperationAddress);
		PreviewProjectorViewer->SetProjector(Projector);
		break;
	}
	
	case UE::Mutable::Private::EDataType::Material:
	{
		check (PreviewMaterialViewer);
		// todo: Evaluate and feed the FMaterial into the viewer or make the viewer evaluate the contents and display them appropriately (mind the constants pannel)
		PreviewMaterialViewer->SetMaterial(nullptr);
		break;
	}
	
	default:
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
		UE_LOGF(LogMutable, Log, "There is no previewer for the selected type of Mutable object")
#endif
		PreviewBorder->SetContent(SNullWidget::NullWidget);
		// There is no viewer for this type.
		break;
	}
}


UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> SMutableCodeViewer::BuildImage(UE::Mutable::Private::FOperation::ADDRESS TargetOpAddress) const
{
	UE::Mutable::Private::FSettings Settings;
	const TSharedPtr<UE::Mutable::Private::FSystem> System = MakeShared<UE::Mutable::Private::FSystem>(Settings);
	
	
	TSharedRef<UE::Mutable::Private::FLiveInstance> TempLiveInstance = System->NewBuildInstance(MutableModel, PreviewParameters, ExternalResourceProvider);
	
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage = System->BuildImage(TempLiveInstance, TargetOpAddress, 0);	
	return MutableImage;
}


void SMutableCodeViewer::OnPreviewParameterValueChanged(int32 ParamIndex)
{
	// This is deferred to the tick to avoid multiple updates per frame.
	bIsPreviewPendingUpdate = true;
}


void SMutableCodeViewer::PrepareStringViewer()
{
	if (!PreviewStringViewer)
	{
		PreviewStringViewer = SNew(SMutableStringViewer);
	}

	PreviewBorder->SetContent(PreviewStringViewer.ToSharedRef());
}


void SMutableCodeViewer::PrepareImageViewer()
{
	if (!PreviewImageViewer)
	{
		PreviewImageViewer = SNew(SMutableImageViewer)
			.GridSize(FIntPoint(8, 8));
	}

	PreviewBorder->SetContent(PreviewImageViewer.ToSharedRef());
}


void SMutableCodeViewer::PrepareMeshViewer()
{
	if (!PreviewMeshViewer)
	{
		PreviewMeshViewer = SNew(SMutableMeshViewer);
	}

	PreviewBorder->SetContent(PreviewMeshViewer.ToSharedRef());
}


void SMutableCodeViewer::PrepareMaterialViewer()
{
	if (!PreviewMaterialViewer)
	{
		PreviewMaterialViewer = 
			SNew(SMutableMaterialViewer)
			.HostCodeViewer(TWeakPtr<SMutableCodeViewer>(SharedThis(this)));		// So it can invoke the BuildImage member method WIP
	}
	
	PreviewBorder->SetContent(PreviewMaterialViewer.ToSharedRef());
}


void SMutableCodeViewer::PrepareLayoutViewer()
{
	if (!PreviewLayoutViewer)
	{
		PreviewLayoutViewer = SNew(SMutableLayoutViewer);
	}

	PreviewBorder->SetContent(PreviewLayoutViewer.ToSharedRef());
}


void SMutableCodeViewer::PrepareProjectorViewer()
{
	if (!PreviewProjectorViewer)
	{
		PreviewProjectorViewer = SNew(SMutableProjectorViewer);
	}

	PreviewBorder->SetContent(PreviewProjectorViewer.ToSharedRef());
}


void SMutableCodeViewer::PreviewMutableString(const FString& InString)
{
	// Prepare the previewer object to receive data 
	PrepareStringViewer();
	
	//  Provide the desired data to the previewer object
	const FText TextToShow = FText::FromString(InString);
	PreviewStringViewer->SetString(TextToShow);
}


void SMutableCodeViewer::PreviewMutableImage(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> InImagePtr)
{
	PrepareImageViewer();
	PreviewImageViewer->SetImage(InImagePtr,0);
}


void SMutableCodeViewer::PreviewMutableMesh(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> InMeshPtr)
{
	PrepareMeshViewer();
	PreviewMeshViewer->SetMesh(InMeshPtr);
}


void SMutableCodeViewer::PreviewMutableMaterial(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial> InMaterialPtr)
{
	PrepareMaterialViewer();
	PreviewMaterialViewer->SetMaterial(InMaterialPtr);
}


void SMutableCodeViewer::PreviewMutableLayout(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout> Layout)
{
	PrepareLayoutViewer();
	PreviewLayoutViewer->SetLayout(Layout);
}


void SMutableCodeViewer::PreviewMutableProjector(const UE::Mutable::Private::FProjector* Projector)
{
	if (!Projector)
	{
		UE_LOGF(LogTemp,Error, "Unable to preview data on null Projector pointer.")
		return;
	}
	
	PrepareProjectorViewer();
	PreviewProjectorViewer->SetProjector(*Projector);
}


void SMutableCodeViewer::PreviewMutableSkeleton(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> Skeleton)
{
	if (!PreviewSkeletonViewer)
	{
		PreviewSkeletonViewer = SNew(SMutableSkeletonViewer);
	}

	PreviewBorder->SetContent(PreviewSkeletonViewer.ToSharedRef());
	
	PreviewSkeletonViewer->SetSkeleton(Skeleton);
}


void SMutableCodeViewer::PreviewMutableCurve(const FRichCurve& Curve)
{
	if (!PreviewCurveViewer)
	{
		PreviewCurveViewer = SNew(SMutableCurveViewer);
	}

	PreviewBorder->SetContent(PreviewCurveViewer.ToSharedRef());
	
	PreviewCurveViewer->SetCurve(Curve);
}

// TODO: Implement physics viewer
void SMutableCodeViewer::PreviewMutablePhysics(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FPhysicsBody> Physics)
{
	UE_LOGF(LogMutable, Warning, "Previewer for Mutable Physics not yet implemented")
}

// TODO: Implement matrix viewer
void SMutableCodeViewer::PreviewMutableMatrix(const FMatrix44f& Mat)
{
	UE_LOGF(LogMutable, Warning, "Previewer for Mutable Matrices not yet implemented")
}

// TODO: Implement shape viewer
void SMutableCodeViewer::PreviewMutableShape(const UE::Mutable::Private::FShape* Shape)
{
	UE_LOGF(LogMutable, Warning, "Previewer for Mutable Shapes not yet implemented")
}


FMutableCodeTreeElement::FMutableCodeTreeElement(int32 InIndexOnTree, const int32& InMutableStateIndex, const TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe>& InModel, UE::Mutable::Private::FOperation::ADDRESS InOperation, const FString& InCaption, TWeakPtr<SMutableCodeViewer> InCodeViewer, UE::Mutable::Private::EOpType InOperationType, const TSharedPtr<FMutableCodeTreeElement>* InDuplicatedOf)
{
	MutableModel = InModel;
	MutableOperation = InOperation;
	Caption = InCaption;
	HostCodeViewer = InCodeViewer;
	OperationType = InOperationType;
	
	IndexOnTree = InIndexOnTree;

	if (InDuplicatedOf)
	{
        bHasDuplicate = true;

		DuplicatedOf = *InDuplicatedOf;
		check(DuplicatedOf);
			   
		if (!DuplicatedOf->bHasDuplicate)
		{
			DuplicatedOf->bHasDuplicate = true;
			DuplicatedOf->GenerateLabelText();
	}
	}

	// Generate the label to be used to display this operation in the operation tree
	GenerateLabelText();

	// Process the data that can be extracted from the current state
	SetElementCurrentState(InMutableStateIndex);
}


void FMutableCodeTreeElement::SetElementCurrentState(const int32& InStateIndex)
{
	// Skip operation if state is the same
	if (InStateIndex == CurrentMutableStateIndex)
	{
		return;
	}

	// Check for an out of bounds value
	check(MutableModel);
	UE::Mutable::Private::FProgram& MutableProgram = MutableModel->GetProgram();
	check(InStateIndex >= 0 && InStateIndex < MutableProgram.States.Num());

	CurrentMutableStateIndex = InStateIndex;
	const UE::Mutable::Private::FProgram::FState& CurrentState = MutableProgram.States[CurrentMutableStateIndex];

	// Check if it is a dynamic resource
	for (auto& DynamicResource : CurrentState.m_dynamicResources)
	{
		// If the operation gets located then mark it as dynamic resource
		if (DynamicResource.Key == MutableOperation)
		{
			bIsDynamicResource = true;
			break;
		}
	}
	// Early exit: A dynamic resource can not be at the same time a state constant
	if (bIsDynamicResource)
	{
		return;
	}

	// Check if it is a state constant
	bIsStateConstant = CurrentState.m_updateCache.Contains(MutableOperation);
}


void FMutableCodeTreeElement::GenerateLabelText()
{
	const UE::Mutable::Private::FProgram& Program = MutableModel->GetProgram();
	FString OpName = UE::Mutable::Private::s_opNames[static_cast<int32>(OperationType)];
	OpName.TrimEndInline();

	// See if the operation type accepts additional information in the label
	switch (OperationType)
	{
	case UE::Mutable::Private::EOpType::BO_PARAMETER:
	case UE::Mutable::Private::EOpType::NU_PARAMETER:
	case UE::Mutable::Private::EOpType::SC_PARAMETER:
	case UE::Mutable::Private::EOpType::CO_PARAMETER:
	case UE::Mutable::Private::EOpType::PR_PARAMETER:
	case UE::Mutable::Private::EOpType::ST_PARAMETER:
	{
		UE::Mutable::Private::FOperation::ParameterArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ParameterArgs>(MutableOperation);
		OpName += TEXT(" ");
		OpName += Program.Parameters[int32(Args.variable)].Name;
		break;
	}

	case UE::Mutable::Private::EOpType::ME_SKELETALMESH_BREAK:
	{
		UE::Mutable::Private::FOperation::MeshSkeletalMeshBreakArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::MeshSkeletalMeshBreakArgs>(MutableOperation);
		OpName += FString::Printf( TEXT("BaseLOD %d BaseSection %d LODBias %d Flags %d"), Args.BaseLOD, Args.BaseSection, Args.LODBias, Args.Flags );
		break;
	}

	case UE::Mutable::Private::EOpType::IM_SWIZZLE:
	{
		UE::Mutable::Private::FOperation::ImageSwizzleArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageSwizzleArgs>(MutableOperation);
		OpName += TEXT(" ");
		OpName += StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[int32(Args.format)]).Get();
		break;
	}

	case UE::Mutable::Private::EOpType::IM_PIXELFORMAT:
	{
		UE::Mutable::Private::FOperation::ImagePixelFormatArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImagePixelFormatArgs>(MutableOperation);
		OpName += TEXT(" ");
		OpName += StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[int32(Args.format)]).Get();
		OpName += TEXT(" or ");
		OpName += StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[int32(Args.formatIfAlpha)]).Get();
		break;
	}

	case UE::Mutable::Private::EOpType::IM_CONFORM:
	{
		UE::Mutable::Private::FOperation::ImageConformArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageConformArgs>(MutableOperation);
		OpName += FString::Printf(TEXT(" format: %s or %s - size: %dx%d - lods: %d"), 
				StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[int32(Args.ConformDescriptor.m_format)]).Get(),
				StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[int32(Args.ConformDescriptor.FormatIfAlpha)]).Get(),
				(int32)Args.ConformDescriptor.m_size.X, (int32)Args.ConformDescriptor.m_size.Y,
				(int32)Args.ConformDescriptor.m_lods); 
		break;
	}

	case UE::Mutable::Private::EOpType::IM_MIPMAP:
	{
		UE::Mutable::Private::FOperation::ImageMipmapArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageMipmapArgs>(MutableOperation);
		OpName += FString::Printf(TEXT(" levels: %d-%d tail: %d"), Args.levels, Args.blockLevels, int32(Args.onlyTail));
		break;
	}

	case UE::Mutable::Private::EOpType::IM_RESIZE:
	{
		UE::Mutable::Private::FOperation::ImageResizeArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageResizeArgs>(MutableOperation);
		OpName += FString::Printf(TEXT(" %d x %d"), int32(Args.Size[0]), int32(Args.Size[1]));
		break;
	}

	case UE::Mutable::Private::EOpType::IM_RESIZEREL:
	{
		UE::Mutable::Private::FOperation::ImageResizeRelArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageResizeRelArgs>(MutableOperation);
		OpName += FString::Printf(TEXT(" %.3f x %.3f"), Args.Factor[0], Args.Factor[1]);
		break;
	}

	case UE::Mutable::Private::EOpType::IM_MULTILAYER:
	{
		UE::Mutable::Private::FOperation::ImageMultiLayerArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageMultiLayerArgs>(MutableOperation);
		OpName += TEXT(" rgb: ");
		OpName += UE::Mutable::Private::TypeInfo::s_blendModeName[int32(Args.blendType)];
		OpName += TEXT(", a: ");
		OpName += UE::Mutable::Private::TypeInfo::s_blendModeName[int32(Args.blendTypeAlpha)];
		OpName += FString::Printf(TEXT(" a from %d "), Args.BlendAlphaSourceChannel);
		OpName += FString::Printf(TEXT(" range-id: %d"), Args.rangeId);
		OpName += FString::Printf(TEXT(" mask-from-alpha: %d"), int32(Args.bUseMaskFromBlended));
		break;
	}

	case UE::Mutable::Private::EOpType::IM_LAYER:
	{
		UE::Mutable::Private::FOperation::ImageLayerArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageLayerArgs>(MutableOperation);
		OpName += TEXT(" rgb: ");
		OpName += UE::Mutable::Private::TypeInfo::s_blendModeName[int32(Args.blendType)];
		OpName += TEXT(", a: ");
		OpName += UE::Mutable::Private::TypeInfo::s_blendModeName[int32(Args.blendTypeAlpha)];
		OpName += FString::Printf(TEXT(" a from %d "), Args.BlendAlphaSourceChannel);
		OpName += FString::Printf(TEXT(" flags %d"), Args.flags);
		break;
	}

	case UE::Mutable::Private::EOpType::IM_LAYERCOLOR:
	{
		UE::Mutable::Private::FOperation::ImageLayerColorArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImageLayerColorArgs>(MutableOperation);
		OpName += TEXT(" rgb: ");
		OpName += UE::Mutable::Private::TypeInfo::s_blendModeName[int32(Args.blendType)];
		OpName += TEXT(" a: ");
		OpName += UE::Mutable::Private::TypeInfo::s_blendModeName[int32(Args.blendTypeAlpha)];
		OpName += TEXT(" a from ");
		OpName += FString::Printf(TEXT(" a from %d "), Args.BlendAlphaSourceChannel);
		OpName += FString::Printf(TEXT(" flags %d"), Args.flags);
		break;
	}

	case UE::Mutable::Private::EOpType::IM_PLAINCOLOR:
	{
		UE::Mutable::Private::FOperation::ImagePlainColorArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::ImagePlainColorArgs>(MutableOperation);
		OpName += TEXT(" format: ");
		OpName += StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[int32(Args.Format)]).Get();
		OpName += FString::Printf(TEXT(" size %d x %d"), Args.Size[0], Args.Size[1]);
		OpName += FString::Printf(TEXT(" mips %d"), Args.LODs);
		break;
	}
		
	case UE::Mutable::Private::EOpType::BO_EQUAL_INT_CONST:
	{
		UE::Mutable::Private::FOperation::BoolEqualScalarConstArgs Args = Program.GetOpArgs<UE::Mutable::Private::FOperation::BoolEqualScalarConstArgs>(MutableOperation);
		OpName += TEXT(" const: ");
		OpName += FString::FromInt(Args.Constant);
		break;
	}

	default:
		break;
	}

	OpName = OpName.TrimStartAndEnd();
	
	// Prepare the text shown on the UI side of the operation tree
	if (!Caption.IsEmpty())
	{
		Caption = Caption.TrimStartAndEnd();
		MainLabel = FString::Printf(TEXT("%d: [%s] %s"), int32(MutableOperation), *Caption, *OpName);
	}
	else
	{
		MainLabel = FString::Printf(TEXT("%d: %s"), int32(MutableOperation), *OpName);
	}
	

	// DEBUG : 
	// FString IndexOnTree = FString::FromInt(IndexOnTree);
	// IndexOnTree.Append(TEXT("- "));
	// MainLabel.InsertAt(0,IndexOnTree);

	// DEBUG : 
	// FString RowStateIndex = FString::FromInt(GetStateIndex());
	// RowStateIndex.Append(TEXT(" st "));
	// MainLabel.InsertAt(0,RowStateIndex);

	// Ignore the special case of operations of type "None"
	if (MutableOperation > 0 && bHasDuplicate)
	{
		MainLabel.Append(TEXT(" (duplicated)"));
	}
}


int32 FMutableCodeTreeElement::GetStateIndex() const
{
	return CurrentMutableStateIndex;
}


FSlateColor FMutableCodeTreeElement::GetLabelColor() const
{
	if (MutableOperation == 0)
	{
		return ZeroOperationColor;
	}
	
	if (TSharedPtr<SMutableCodeViewer> PinnedViewer = HostCodeViewer.Pin())
	{
		return PinnedViewer->GetOperationColor(OperationType);
	}
	
	// The HostCodeViewer may no longer exist, the color returned will not be used.
	return FColor::Black;
}


#undef LOCTEXT_NAMESPACE
