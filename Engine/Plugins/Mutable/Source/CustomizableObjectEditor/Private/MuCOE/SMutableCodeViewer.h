// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Model.h"
#include "MuT/TypeInfo.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Engine/Texture.h"
#include "MuR/Mesh.h"
#include "UObject/SoftObjectPtr.h"

class UCustomizableObjectInstance;
class UCustomizableObject;
class FUnrealMutableResourceProvider;
class STableViewBase;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class STreeView;
template <typename OptionType> class SComboBox;


class FMutableCodeTreeElement;
class FMutableOperationElement;
class FReferenceCollector;
class ITableRow;
class SBorder;
class SMutableBoolViewer;
class SMutableColorViewer;
class SMutableConstantsWidget;
class SMutableCurveViewer;
class SMutableImageViewer;
class SMutableIntViewer;
class SMutableLayoutViewer;
class SMutableMeshViewer;
class SMutableInstanceViewer;
class SMutableParametersWidget;
class SMutableProjectorViewer;
class SMutableScalarViewer;
class SMutableSkeletonViewer;
class SMutableStringViewer;
class SMutableMaterialViewer;
class STextComboBox;
class SWidget;
namespace UE::Mutable::Private { struct FProjector; }
namespace UE::Mutable::Private { struct FShape; }
struct FGeometry;

/** This widget shows the internal Mutable Code for debugging purposes. 
 * This is not the Unreal source graph in the UCustomizableObject, but the actual Mutable virtual machine graph.
 */
class SMutableCodeViewer : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMutableCodeViewer) {}

	SLATE_ARGUMENT(TSharedPtr<FUnrealMutableResourceProvider>, ExternalResourceProvider)
		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UCustomizableObject& InObject, UCustomizableObjectInstance& InInstance, const TSharedPtr<UE::Mutable::Private::FModel>& InMutableModel);

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	/** Clear the selected operation of the tree view.*/
	void ClearSelectedTreeRow() const;
	
	/**
	 * Evaluates the graph starting at the TargetOPAddress and returns an FImage resulting from that
	 * @param TargetOpAddress The OP Address to use as the root to be computed.
	 * @return A pointer to the FImage generated. May be null if the operation does not evaluate to an FImage
	 * @note This method is experimental and should not be used as example
	 */
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage>  BuildImage(UE::Mutable::Private::FOperation::ADDRESS TargetOpAddress) const;
	
private:
	TStrongObjectPtr<UCustomizableObject> Object;
	TStrongObjectPtr<UCustomizableObjectInstance> Instance;
	
	/** The Mutable Model that we are showing. */
	TSharedPtr<UE::Mutable::Private::FModel> MutableModel;

	TSharedPtr<FUnrealMutableResourceProvider> ExternalResourceProvider;
	
	/** Array of external referenced textures in MutableModel, indexed by id. */
	TArray<TSoftObjectPtr<const UTexture>> ReferencedTextures;
	TArray<TSoftObjectPtr<const UStreamableRenderAsset>> ReferencedMeshes;

	/** Selected model operation for preview. */
	UE::Mutable::Private::FOperation::ADDRESS SelectedOperationAddress = 0;

	/** Mutable parameters used in the preview. */
	TSharedPtr<UE::Mutable::Private::FParameters> PreviewParameters;

	/** Widget showing the parameters that affect the current preview. */
	TSharedPtr<SMutableParametersWidget> ParametersWidget;

	/** Widget showing the constants found on the mutable Model program */
	TSharedPtr<SMutableConstantsWidget> ConstantsWidget;

	/** If true, the parameters have changed and we need to update the preview. */
	bool bIsPreviewPendingUpdate = false;

	/** Widget container where different previews will be created. */
	TSharedPtr<SBorder> PreviewBorder;

	/*
	* Preview windows for the data types exposed by Mutable
	*/

	/** Widget used to show the preview of layout operation results. Once created it is reused to preserve the settings. */
	TSharedPtr<SMutableLayoutViewer> PreviewLayoutViewer;

	/** Widget used to show the preview of image operation results. Once created it is reused to preserve the settings. */
	TSharedPtr<SMutableImageViewer> PreviewImageViewer;
	
	/** Widget used to show a preview of a mesh and the metadata it holds */
	TSharedPtr<SMutableMeshViewer> PreviewMeshViewer;

	/** Widget used to show a preview of a mutable FMaterial and the data it holds */
	TSharedPtr<SMutableMaterialViewer> PreviewMaterialViewer;
	
	/** Widget used to show a preview of a mutable bool value */
	TSharedPtr<SMutableBoolViewer> PreviewBoolViewer;

	/** Widget used to show a preview of a mutable int value */
	TSharedPtr<SMutableIntViewer> PreviewIntViewer;

	/** Widget used to show a preview of a mutable float value */
	TSharedPtr<SMutableScalarViewer> PreviewScalarViewer;

	/** Widget used to show a preview of a mutable string value */
	TSharedPtr<SMutableStringViewer> PreviewStringViewer;
	
	/** Widget used to show a preview of a mutable color value */
	TSharedPtr<SMutableColorViewer> PreviewColorViewer;

	/** Widget used to display the data held on the mutable projector objects */
	TSharedPtr<SMutableProjectorViewer> PreviewProjectorViewer;

	/** Widget used to display the data held on the mutable skeleton objects */
	TSharedPtr<SMutableSkeletonViewer> PreviewSkeletonViewer;

	/** Widget used to display the data held on the mutable curve objects */
	TSharedPtr<SMutableCurveViewer> PreviewCurveViewer;
	
	/** Tree widget showing the code hierarchically. */
	TSharedPtr<STreeView<TSharedPtr<FMutableCodeTreeElement>>> TreeView;

	/** Root nodes of the tree widget. */
	TArray<TSharedPtr<FMutableCodeTreeElement>> RootNodes;

	/** Cache of tree elements matching the operations that have been generated so far. 
	* We store both the parent and the operation in the key, because a single operation may appear multiple times if it has different parents.
	*/
	struct FItemCacheKey
	{
		UE::Mutable::Private::FOperation::ADDRESS Parent=0;
		UE::Mutable::Private::FOperation::ADDRESS Child=0;
		uint32 ChildIndexInParent=0;

		friend FORCEINLINE bool operator == (const FItemCacheKey& A, const FItemCacheKey& B)
		{
			return A.Parent == B.Parent && A.Child == B.Child && A.ChildIndexInParent == B.ChildIndexInParent;
		}

		friend FORCEINLINE uint32 GetTypeHash(const FItemCacheKey& Key)
		{
			return HashCombine(Key.Parent, HashCombine(Key.Child, Key.ChildIndexInParent));
		}
	};

	/*
	* Tree widget objects
	*/

	/** Map with all the generated elements of the tree. Unique and duplicated elements will be present on this list and
	 * also the children of the unique elements.
	 * @note The children of duplicated elements will only be present once, as children of the unique element they duplicate.
	 * This is to avoid having identical elements on the tree (witch would cause a crash) while also being pointless due to
	 * how we manage expansion of duplicated elements.
	 */
	TMap< FItemCacheKey, TSharedPtr<FMutableCodeTreeElement>> ItemCache;

	/** Main tree item for each op. An op can be represented with multiple tree nodes if it is reachable from different paths. */
	TMap< UE::Mutable::Private::FOperation::ADDRESS, TSharedPtr<FMutableCodeTreeElement>> MainItemPerOp;

	/** List with all the elements related to nodes displayed on the tree. */
	TArray<TSharedPtr<FMutableCodeTreeElement>> TreeElements;

	/** Array with all the elements that have been manually expanded by the user */
	TMap< UE::Mutable::Private::FOperation::ADDRESS, TSharedPtr<FMutableCodeTreeElement>> ExpandedElements;

	/** Before any UI operation generate all the elements that may be navigable over the tree. No children of duplicated 
	 * addresses will be generated.
	 */
	void GenerateAllTreeElements();

	/** Support GenerateAllTreeElements by generating the elements recursively */
	void GenerateElement( const int32& InStateIndex, UE::Mutable::Private::FOperation::ADDRESS InParentAddress, const UE::Mutable::Private::FProgram& InProgram);

	
	/** The addresses of the root operations. Cached when this object gets loaded on the Construct method of this slate */
	TArray<UE::Mutable::Private::FOperation::ADDRESS> RootNodeAddresses;
	
	/** Utility method that provides you with the addresses of the root nodes of the program */
	void CacheRootNodeAddresses();
	
	/** Control boolean to know if there are any highlighted elements on the tree */
	bool bIsElementHighlighted = false;

	/** Operation that is currently being highlighted */
	UE::Mutable::Private::FOperation::ADDRESS HighlightedOperation{};

	/*
	 * Operations computational cost reference collections
	 */

	enum class EEntryLabelColoringPolicy : int32
	{
		BY_COST = 0,
		BY_DATATYPE,

		COUNT,
	};

	const TArray<FName> EntryLabelColoringPolicyNames
	{
		"BY_COST",
		"BY_DATATYPE",
	};
	
	/** Determines what should be used to paint the labels */
	EEntryLabelColoringPolicy EntryLabelColoringPolicy = EEntryLabelColoringPolicy::BY_DATATYPE;
	
	/** Collection with all very expensive to run operation types */
	const TArray<UE::Mutable::Private::EOpType> VeryExpensiveOperationTypes
	{
		UE::Mutable::Private::EOpType::ME_BINDSHAPE,
		UE::Mutable::Private::EOpType::ME_MASKCLIPMESH,
		UE::Mutable::Private::EOpType::ME_MASKCLIPUVMASK,
		UE::Mutable::Private::EOpType::ME_FORMAT,
		UE::Mutable::Private::EOpType::ME_MASKDIFF,
		UE::Mutable::Private::EOpType::ME_DIFFERENCE,
		UE::Mutable::Private::EOpType::ME_EXTRACTLAYOUTBLOCK,
		UE::Mutable::Private::EOpType::IM_MAKEGROWMAP,
	};

	/** Collection with all expensive to run operation types */
	const TArray<UE::Mutable::Private::EOpType> ExpensiveOperationTypes
	{
		UE::Mutable::Private::EOpType::IM_PIXELFORMAT,
		UE::Mutable::Private::EOpType::IM_SWIZZLE,
		UE::Mutable::Private::EOpType::ME_PROJECT,
		UE::Mutable::Private::EOpType::ME_TRANSFORMWITHMESH,
	};
	
	/** Enum designed to be able to notify the row generation of the type of operation being generated */
	enum class EOperationComputationalCost : uint8
	{
		Standard =			0,			// All other operation types
		Expensive =			1,			// The ones located on ExpensiveOperationTypes
		VeryExpensive =		2			// The ones located on VeryExpensiveOperationTypes
	};

	/** Array holding the relation between each computational cost category and the color to be used to display elements related
	 * with it. 
	 */
	const TArray<FSlateColor> ColorPerComputationalCost
	{
		FSlateColor(FLinearColor(1,1,1,1)),			// Standard cost color
		FSlateColor(FLinearColor(1,0.4f,0.2f,1)),		// Expensive cost color
		FSlateColor(FLinearColor(1,0.1f,0.1f,1))		// Very Expensive cost color
	};
	
	/** Provided an operation type it returns the category representing how much costs to run an operation of this type
	 * @param OperationType The operation type you want to know the computational cost of
	 * @return The enum value representing the computational cost of the operation type provided.
	 */
	EOperationComputationalCost GetOperationTypeComputationalCost(UE::Mutable::Private::EOpType OperationType) const;
	
	FSlateColor GetOperationDataTypeColor(UE::Mutable::Private::EDataType DataType) const;
	
public:
	FSlateColor GetOperationColor(const UE::Mutable::Private::EOpType OperationType) const;

private:
	void OnLabelColorPolicyChange(FName PolicyName, ESelectInfo::Type Arg);
	TSharedRef<SWidget>  OnLabelColorPolicyGenerateWidget(FName PolicyName);
	FText GetCurrentLabelColoringPolicyText() const;

	/*
	 * Navigation : Operation type navigation Type selection object
	 */
	
	/** Slate object that provides the user a way of selecting what kind of operation it wants to navigate.*/
	TSharedPtr<SComboBox<TSharedPtr<const FMutableOperationElement>>> TargetedTypeSelector;

	/** Data backend for the list displayed for the navigation type selection */
	TArray<TSharedPtr<const FMutableOperationElement>> FoundModelOperationTypeElements;

	/** Navigation operation entry representing the NONE type */
	TSharedPtr<const FMutableOperationElement> NoneOperationEntry;

	/** Navigation operation entry used to be able to set the UI to show we are performing a constant resource based 
	 * navigation
	 */
	TSharedPtr<const FMutableOperationElement> ConstantBasedNavigationEntry;
	
	/** Currently selected element on the TargetedTypeSelector slate. Actively used by the ui */
	TSharedPtr<const FMutableOperationElement> CurrentlySelectedOperationTypeElement;
	
	/** Operation type we are using to search for tree nodes. Driven primarily by the UI */
	UE::Mutable::Private::EOpType OperationTypeToSearch = UE::Mutable::Private::EOpType::NONE;

	/** Operation types present on the currently set mutable model.
	 * The value of the TPair represents the amount of times operations using it have been found (not duplicates) */
	TArray< TPair< UE::Mutable::Private::EOpType,uint32>> ModelOperationTypes;
	
	/** Array with all the names for each of the operations available on mutable. Used by the UI  */
	TArray<FString> ModelOperationTypeNames;
	
	/** Stores a list of strings based on the possible types of operations mutable define to be used by the UI */
	void GenerateNavigationOpTypeStrings();

	/** Generate a list of elements that will be used as backend for the navigation type selection dropdown */
	void GenerateNavigationDropdownElements();
	
	
	/** Fills ModelOperationTypes with all the types present on the current model.
	 * It also makes sure we have a NONE operation and that the operations are sorted alphabetically.
	 * @note Method designed to be called once per model.*/
	void CacheOperationTypesPresentOnModel();
	
	
	/*
	 * Navigation : UI callback methods
	 */

	/** Generate the text to be used by the navigation operation selector */
	FText GetCurrentNavigationOpTypeText() const;

	/** Returns the color to be used by the text being currently displayed as selected on the operation selector*/
	FSlateColor GetCurrentNavigationOpTypeColor() const;

	/** Callback invoked by the ComboBox used for displaying and selecting operation types for navigation. it gets invoked
	 * each time the slate object requires to draw a line representing one of the elements set on FoundModelOperationTypeElements
	 */
	TSharedRef<SWidget> OnGenerateOpNavigationDropDownWidget(TSharedPtr<const FMutableOperationElement> InMutableOperationElement) const;

	/** Callback invoked each time the selected operation on our navigation slate changes. It can change due to UI interaction
	 * or also due to direct change by invoking the SetSelectedOption on the SComboBox TargetedTypeSelector
	 */
	void OnNavigationSelectedOperationChanged(TSharedPtr<const FMutableOperationElement, ESPMode::ThreadSafe> MutableOperationElement, ESelectInfo::Type Arg);

	/** Callback used to print on screen the amount of operations found on the tree that share the same operation type that
	 * the one currently selected on the navigation system.
	 */
	FText OnPrintNavigableObjectAddressesCount() const;
	
	/** Method that tells the UI if the bottom to go to the previous element can be interacted */
	bool CanInteractWithPreviousOperationButton() const;

	/** Method that tells the UI if the bottom to go to the next element can be interacted */
	bool CanInteractWithNextOperationButton() const;
	

	/** Used by the UI and internally does what is necessary to get to the previous element of the targeted search type.
	 * It is designed to work alongside GoToPreviousOperationAfterRefresh(...)
	 * @return A reply to tell if the UI action has been handled or not.
	 */
	FReply OnGoToPreviousOperationButtonPressed();

	/** Used by the UI and internally does what is necessary to get to the next element of the targeted search type.
	 * It is designed to work alongside GoToNextOperationAfterRefresh(...)
	 * @return A reply to tell if the UI action has been handled or not.
	 */
	FReply OnGoToNextOperationButtonPressed();
	
	
	/** Callback method used to allow the user to directly set the type of operation to be scanning directly by selecting a
	 * operation on the tree and using that operation type as the type to search for
	 */
	void OnSelectedOperationTypeFromTree();
	
	/*
	 * Navigation : control flags
	 */
	
	/** Flag monitoring if we have requested a scroll operation to reach the targeted element */
	bool bWasScrollToTargetRequested = false;

	/** Flag designed to tell the system when the expansion of unique elements have been performed as part of the navigation operation */
	bool bWasUniqueExpansionInvokedForNavigation = false;
	
	/*
	 * Navigation : Shared objects between navigation search types 
	 */

	/** Array with all the elements of the type we are looking for (shared type of constant resource). */
	TArray<TSharedPtr<FMutableCodeTreeElement>> NavigationElements;
	
	/** Operation types present on the currently set mutable model. */
	int64 NavigationIndex = -1;
	
	/** Sort the contents of NavigationElements to follow a sequential pattern using the indices of the elements from 0 to +n*/
	void SortElementsByTreeIndex(TArray<TSharedPtr<FMutableCodeTreeElement>>& InElementsArrayToSort);
	
	/** If the focus could not be performed directly then cache the target until it is in view and can therefore be focused.
	 * It will be valid if we are trying to focus something (it) and invalid if the focus operation has already been resolved. */
	TSharedPtr<FMutableCodeTreeElement> ToFocusElement;
	
	/** Focus the current NavigationElement and places it into view. It also selects it so the previewer for the
	 * element gets invoked
	 */
	void FocusViewOnNavigationTarget(TSharedPtr<FMutableCodeTreeElement> InTargetElement);
	
	/** Wrapper struct designed to be used as cache for all elements found during the navigation system search for elements
	 * of X Type or relation with a targeted constant resource. It is designed to be used and then destroyed once the
	 * search operation has been completed */
	struct FElementsSearchCache
	{
		/** Set of addresses that have already been searched for relevant data */
		TSet<UE::Mutable::Private::FOperation::ADDRESS> ProcessedAddresses;

		/** Collection of elements that have been found during the search. They may be related by OP_Type or used constant resource*/
		TArray<TSharedPtr<FMutableCodeTreeElement>> FoundElements;
		
		/** Array containing all the next addresses to be processed.
		 * The child is the address itself to be later processed
		 * The parent is the parent address of the child.
		 * The ChildIndexInParent is the index (or child position) o the child address on Child as part of the child set of the parent address.
		 */
		TArray<FItemCacheKey> BatchData;
		

		/** Generates the structures to be able to start the search of elements. It uses the root addresses as the
		 * start of the search operation.
		 * @param InRootNodeAddresses The addresses of the root operations of the operations tree
		 */
		void SetupRootBatch(const TArray<UE::Mutable::Private::FOperation::ADDRESS>& InRootNodeAddresses)
		{
			check (InRootNodeAddresses.Num());
			// This method should only be called once when no data is present on this cache
			check (BatchData.IsEmpty());
			
			BatchData.Reserve(InRootNodeAddresses.Num());
			// The child address of each parent operation on it's parent
			for (int32 RootIndex = 0; RootIndex < InRootNodeAddresses.Num(); RootIndex++)
			{
				FItemCacheKey Key;
				{
					Key.Child = InRootNodeAddresses[RootIndex];
					// Store the parent of this object as 0 to have a "virtual" address witch all root addresses are
					// children of 
					Key.Parent = 0;
					// And also the index of the parent on it's parent "virtual" structure
					Key.ChildIndexInParent = RootIndex;		
				}

				// Add this new entry point for the search of addresses 
				BatchData.Add(Key);
			}
		}
		
		/** Method to cache the provided mutable address as one of the addresses that are of the type we are looking for or maybe
		* is related with the constant resource we are looking for operations related with.
		* @param OpAddress The address to save as one related with a operation type or constant resource
		* @param IndexAsChildOfInputAddress The index on BatchData that represents the parent of the provided operation address
		* @param InItemCache Cache with all the elements of the tree. Here is where we search for the element based on
		* the OpAddress and the parent and ChildIndexInParent provided thanks to IndexAsChildOfInputAddress
		*/
		void AddToFoundElements(const UE::Mutable::Private::FOperation::ADDRESS OpAddress,const int32 IndexAsChildOfInputAddress,
		                        const TMap<FItemCacheKey, TSharedPtr<FMutableCodeTreeElement>>& InItemCache)
		{
			FItemCacheKey Key;
			{
				Key.Child = OpAddress;
				// Store the parent address of this object
				Key.Parent = BatchData[IndexAsChildOfInputAddress].Parent;
				// And also the index of the parent on it's parent structure
				Key.ChildIndexInParent = BatchData[IndexAsChildOfInputAddress].ChildIndexInParent;		
			}
			// Generate a key for this element in order to search in on the map with all the elements
								
			// Find that element on the tree, (check error if not found since all elements should be there)
			const TSharedPtr<FMutableCodeTreeElement>* PtrFoundElement = InItemCache.Find(Key);
			check(PtrFoundElement);
			TSharedPtr<FMutableCodeTreeElement> FoundElement = *PtrFoundElement;
			check(FoundElement);
			
			// Store this element on our temp array of elements
			FoundElements.Add(FoundElement);
		}

		/** Caches the provided parent address to the Search payload so they can be later read and processed on another batch of the
		 * method tasked with finding related operations to operation type or constant resource.
		 * @param InParentAddress The parent address to search children of.
		 * @param InProgram The mutable program that will be used to perform the children search.
		 * @param OutFoundChildrenData Array with all the ItemCacheKeys that represent all the children found
		 * @note It will not add the children of any provided ParentAddress for the next batch if the parent address provided
		 * has already been processed and therefore whose children have already been searched or prepared for searching.
		 */
		void CacheChildrenOfAddressIfNotProcessed(UE::Mutable::Private::FOperation::ADDRESS InParentAddress,
		                                          const UE::Mutable::Private::FProgram& InProgram,
		                                          TArray<FItemCacheKey>& OutFoundChildrenData)
		{
			if (!ProcessedAddresses.Contains(InParentAddress))
			{
				// Cache to avoid processing it again later
				ProcessedAddresses.Add(InParentAddress);
	
				// Generic case for unnamed children traversal.
				uint32 ChildIndex = 0;
				UE::Mutable::Private::ForEachReference(InProgram, InParentAddress, [this, &InParentAddress, &ChildIndex,&OutFoundChildrenData]( UE::Mutable::Private::FOperation::ADDRESS ChildAddress)
				{
					// If the parent does have a child then process it
					// Ignoring child address 0 since those can not have children
					if (ChildAddress) 
					{
						FItemCacheKey Key;
						{
							Key.Child = ChildAddress;
							Key.Parent = InParentAddress;
							Key.ChildIndexInParent = ChildIndex;		
						}

						// Save it to the output so can later be placed onto BatchData safely 
						OutFoundChildrenData.Add(Key);
					}
					ChildIndex++;
				});
			}
		}
	};
	
	
	/*
	 * Navigation : Operation type based navigation
	 */
	
	/** Provided an Operation type store as navigation addresses all operations of the targeted navigation Op type
	 * @note The operations found get saved on the collection  NavigationOPAddresses.
	 */
	void CacheAddressesOfOperationsOfType();

	/** Fills an array with all the addresses of operations that do have in common the same targeted operation type
	 * @param TargetOperationType The operation type used to discriminate what operation addresses we want to retrieve.
	 * processing them more than once.
	 * @param InSearchPayload A caching structure designed to hold the data that gets passed from one recursive call to
	 * the other. It also stores the found elements and other data.
	 * @param InProgram The mutable program holding the data to be searched over
	 */
	void GetOperationsOfType(const UE::Mutable::Private::EOpType& TargetOperationType,
	                         FElementsSearchCache& InSearchPayload,
	                         const UE::Mutable::Private::FProgram& InProgram);

	/*
	 * Navigation : Navigation based on constant resource relation with addresses
	 */

public:
	
	/**
	 * Provided a constant resource data type and index on its constant array this method sets all the operations that
	 * make usage of said constant to be navigable by the navigation system.
	 * @note The output of this operation will be cached onto NavigationOPAddresses so we can navigate over them.
	 * @param ConstantDataType The constant data type to locate operations referencing it: It is required for the
	 * system to know what operations should be checked for the usage of the provided constant resource. For example, a
	 * type of UE::Mutable::Private::EDataType::DT_Curve will try to locate operations that we know operate with constant curves.
	 * @param IndexOnConstantsArray The index of the constant on its constants array. If it is, for example, a constant mesh
	 * this value represents the index of the constant inside constantMeshes array on UE::Mutable::Private::program.
	 */
	void CacheAddressesRelatedWithConstantResource(const UE::Mutable::Private::EDataType ConstantDataType, const int32 IndexOnConstantsArray);

private:
	
	/**
	 * Provided a data type and the index of the constant object using said data type locate all operations that do
	 * directly make use of said constant resource.
	 * @param ConstantDataType The type of the resource we are providing an index of. 
	 * @param IndexOnConstantsArray The index of the constant resource we want to know what operations are using it.
	 * @param InSearchPayload Cache object that will end up containing all the elements related with the provided constant
	 * resource
	 * @param InProgram The mutable program containing all the operations of the graph.
	 */
	void GetOperationsReferencingConstantResource(const UE::Mutable::Private::EDataType ConstantDataType,
	                                              const int32 IndexOnConstantsArray,
	                                              FElementsSearchCache& InSearchPayload,
	                                              const UE::Mutable::Private::FProgram& InProgram);

	/**
	 * Provided an operation address this method tells us if that address is making use of our constant resource or not.
	 * It will take into account the datatype of the provided resource to discard or acknowledge the operation type of the
	 * provided operation address.
	 * @param IndexOnConstantsArray The index onto its respective constants array of the constant resource provided. We do
	 * not give a pointer to the constant resource but the index of it onto its hosting array.
	 * @param ConstantDataType The type of data we know the constant uses. It is used to check for ones or other operations since
	 * not all operations do access the same constant resources.
	 * @param OperationAddress The address of the operation currently being scanned for constant resources.
	 * @param InProgram The mutable program to use to check the type of operation that corresponds to that operation address.
	 * @return True if the operation at OperationAddress does make use of the provided constant resource. False otherwise
	 */
	bool IsConstantResourceUsedByOperation(const int32 IndexOnConstantsArray,
	                                       const UE::Mutable::Private::EDataType ConstantDataType, const UE::Mutable::Private::FOperation::ADDRESS OperationAddress,
	                                       const UE::Mutable::Private::FProgram& InProgram) const;
	

	/*
	* Navigation : Text based navigation
	*/

	/** Current text being used to perform a search by name. It gets updated each time the user changes the text in the SearchBox */
	FString SearchString;
	
	/** Flag that determines how to interpret the SearchString provided.
	 * If true it means it is a regular expression.
	 * If not it is just a fragment of text to seek for in the tree of operations. */
	bool bIsSearchStringRegularExpression = false;
	
	/** Last search configuration -> Prevents recurrent searches with the same search parameters */
	FString LastSearchedString;
	bool bWasLastSearchRegEx = false;
	TSharedPtr<UE::Mutable::Private::FModel> LastSearchedModel;

	/** A collection with all the elements we have found to be related with the SearchString. */
	TArray<TSharedPtr<FMutableCodeTreeElement>> NameBasedNavigationElements;

	/**  Index in the above array. Current navigation index for the navigation over elements related with the SearchString */
	int32 StringNavigationIndex = -1;
	
	/** UI Callback that sets the backend flag to mirror the UI toggle.
	* @param CheckBoxState The new state of the checkBox object being updated
	*/
	void OnRegexToggleChanged(ECheckBoxState CheckBoxState);

	/** Exposes arrows to allow navigation over the found elements and handles the navigation using those.
	 * @param SearchDirection The direction (up or down) being used to navigate the collection of tree elements
	 */
	void OnTreeStringSearch(SSearchBox::SearchDirection SearchDirection);

	/** Invoked each time the user changes the search text in the UI. NewText cached in SearchString variable 
	* @param InUpdatedText The new value set by the user for the TextBox used to provide the search string.
	*/
	void OnTreeSearchTextChanged(const FText& InUpdatedText);

	/** Get the search results for the current search. Used by the Ui to generate some context related elements.
	* @return Data for external search results to be shown in the search box.
	*/
	TOptional<SSearchBox::FSearchResultData> SearchResultsData() const;

	/** Invoked each time the user commits to the text provided. It also caches the text provided by the user 
	* @param InUpdatedText The current text being exposed in the UI textBox to be used for search. It is not being used
	* since we rely in the value cached in the global variable "SearchString"
	* @param TextCommitType Determines how the user did commit the search string.
	*/
	void OnTreeSearchTextCommitted(const FText& InUpdatedText, ETextCommit::Type TextCommitType);
	
	/** Move the focus to the next element in the collection of related elements to the cached SearchString. */
	void GoToNextOperation();

	/** Move the focus to the previous element in the collection of related elements to the cached SearchString. */
	void GoToPreviousOperation();

	/** Provided an index inside the NameBasedNavigationElements array change the focus to it. */
	void GoToTargetOperation(const int32& InTargetIndex);

	/** Provided a search string and a search type a series of elements to be navigated over */
	void CacheOperationsMatchingStringPattern();

	/** Searches the item cache in search of a string pattern that can be treated as a regular string or as a Regular expression.
	 * @param InStringPattern The string to be searched over. It will get compared with the "Main Label" of the comparing operations.
	 * @param bIsRegularExpression True if the InStringPattern is ought to be treated as a regular expression, false otherwise.
	 * @param SearchPayload Structure containing the operations to be used as the origin of the search. This object will also contain the results of the search.
	 * @param InProgram MutableProgram used to allow the search of the children of the operations being currently processed.
	 */
	void GetOperationsMatchingStringPattern(const FString& InStringPattern,const bool bIsRegularExpression, FElementsSearchCache& SearchPayload, const UE::Mutable::Private::FProgram& InProgram);

	/** Provided a ItemCacheKey this method returns the string that represents it. It will not take in consideration duplicated entries since those are not part
	 * of said ItemCache.
	 * @param InItemCacheKey The key representing the current operation being searched.
	 * @return The MainLabel used to display the provided operation.
	 */
	FString GetOperationDescriptiveText(const FItemCacheKey& InItemCacheKey);

	
	/*
	* Main callbacks from the tree widget standard operation. 
	*/

	/** Callback that generates a new Row object based on a Tree Node*/
 	TSharedRef<ITableRow> GenerateRowForNodeTree(TSharedPtr<FMutableCodeTreeElement> InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	
	/**
	* Provided an element it is returned a list with all the immediate children it has 
	* @param InInfo - The MutableCodeElement whose children objects we are searching
	* @param OutChildren - The children of the provided MutableCodeTreeElement
	*/
	void GetChildrenForInfo(TSharedPtr<FMutableCodeTreeElement> InInfo, TArray< TSharedPtr<FMutableCodeTreeElement> >& OutChildren);

	/** 
	* Callback invoked each time an element of the tree gets expanded or contracted 
	* @param InInfo - The MutableCodeElement to be contracted or expanded
	* @param bInExpanded - Determines if the action is of contraction (false) or expansion (true).
	*/
	void OnExpansionChanged(TSharedPtr<FMutableCodeTreeElement> InInfo, bool bInExpanded);


	/** 
	* Callback invoked each time the selected element changes used to generate the previews depending of the 
	* operation selected
	* @param InInfo - The MutableCodeElement that has been selected. Can be null if unselected
	* @param SelectInfo - Way in witch the selection changed
	*/
	void OnSelectionChanged(TSharedPtr<FMutableCodeTreeElement> InInfo, ESelectInfo::Type SelectInfo);

	/** 
	* Callback invoked when a tree row is getting removed 
	* @param InTreeRow - The tree row that is being removed from the tree view
	*/
	void OnRowReleased(const TSharedRef<ITableRow>& InTreeRow);

	/** Callback invoked when whe press the right mouse button. Useful for adding context menu objects */
	TSharedPtr<SWidget> OnTreeContextMenuOpening();

	void TreeExpandRecursive(TSharedPtr<FMutableCodeTreeElement> Info, bool bExpand );
	
	/** Temporal object designed to be used during the recursive operation of TreeExpandElements() and group
	 * strongly related data
	 */
	struct FProcessedOperationsBuffer
	{
		/** Array with the operation addresses of all original (not duplicates) expanded operations. */
		TArray<UE::Mutable::Private::FOperation::ADDRESS> ExpandedOriginalOperations;

		/** Array with the operation addresses of all duplicated expanded operations. */
		TArray<UE::Mutable::Private::FOperation::ADDRESS> ExpandedDuplicatedOperations;
	};
	
	/** 
	* Expands the provided elements and all the children they have. 
	* @note It performs the operation by going to the deepest children and expanding from then upwards to the origin before
	* proceeding to the next parent. This way we make sure that the expansion of the elements is following the order
	* of left to right and top to bottom in a "Z" like pattern like the way the tree gets read by humans.
	* @param InElementsToExpand - The root info objects where to start the expansion
	* @param bForceExpandDuplicates - (Optional) Determines if the duplicated objects must be expanded. 
	*	Used for certain situations where we want to expand them. By default they do not get expanded
	* @param FilteringDataType - (Optional) determines what kind of operations should be expanded. By default no
	* filtering is applied
	* @param InExpandedOperationsBuffer (Optional) Object containing all the duplicated and original elements already processed
	* during the subsequent recursive calls to this method.
	*/
	void TreeExpandElements(TArray<TSharedPtr<FMutableCodeTreeElement>>& InElementsToExpand,
		bool bForceExpandDuplicates = false,
		UE::Mutable::Private::EDataType FilteringDataType = UE::Mutable::Private::EDataType::None,
		TSharedPtr<FProcessedOperationsBuffer> InExpandedOperationsBuffer = nullptr);
	

	/** Expand only the unique elements (mot duplicates) */
	void TreeExpandUnique();

	/** Expand only the elements using as operation type Instance */
	void TreeExpandInstance();
	
	/** Grabs the selected element and expands all the elements inside the selected branch. Duplicates are ignored */
	void TreeExpandSelected();
	
	/** 
	* Highlights all tree elements that share the same operation as the element provided
	* @param InTargetElement - The info object to be used as blueprint to search for similar objects and highlight them
	*/
	void HighlightDuplicatesOfEntry(const TSharedPtr<FMutableCodeTreeElement>& InTargetElement);

	/** Clear all the highlighted elements and set them to their default visual state */
	void ClearHighlightedItems();

	/** Called when any parameter value has changed, with the parameter index as argument.  */
	void OnPreviewParameterValueChanged( int32 ParamIndex );

	/*
	* Element state recalculation 
	*/
	
	/** FLag that determines if the expansion of elements should require the computation of the state of it's child elements.
	 * Useful to avoid unnecessary calls for state resting of movable branches like the children of operations that are repeated
	 * in the tree slate */
	bool bShouldRecalculateStates = false;
	
	/** Provided an element of the tree view return all the children of it (direct and indirect). The search will stop (depth)
	 * if a children is found to not be expanded. This way we avoid infinite searches (since we do not stop at duplicates).
	 * @note It DOES NOT check for visibility by asking the TreeView. It only gets the elements that could be seen in theory.
	 * @note This method will return expanded and not expanded children but only the expanded children will get it's children searched and therefore
	 * provided by this method. In conclusion, only the elements that could be seen by the view will get provide.
	 * @param InInfo The element whose children we want to grab. It can be expanded or not.
	 * @param OutChildren A collection of children elements. Only the elements that could be seen in the tree view (expanded and the root element of an
	 * unexpanded branch) wll get added to this collection.
	 */
	void GetVisibleChildren(TSharedPtr<FMutableCodeTreeElement> InInfo, TSet<TSharedPtr<FMutableCodeTreeElement>>& OutChildren);
	

	/*
	 * Control of "Skip Mips" for image operations.
	 */

	/** */
	int32 MipsToSkip = 0;

	/** */
	bool bSelectedOperationIsImage = false;

	FReply UpdateParameters();
	
	/** Stores a list of strings based on the possible types of operations mutable define to be used by the UI */
	EVisibility IsMipSkipVisible() const;

	/** Stores a list of strings based on the possible types of operations mutable define to be used by the UI */
	TOptional<int32> GetCurrentMipSkip() const;

	/** */
	void OnCurrentMipSkipChanged(int32 NewValue);

	/*
	 * Remote previewer invocation methods
	 */

public:
	void PreviewMutableImage (UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> InImagePtr);
	void PreviewMutableMesh (UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> InMeshPtr);
	void PreviewMutableMaterial(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial> InMaterialPtr);
	void PreviewMutableLayout(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout> Layout);
	void PreviewMutableSkeleton(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> Skeleton);
	void PreviewMutablePhysics(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FPhysicsBody> Physics);
	void PreviewMutableString(const FString& InString);
	void PreviewMutableProjector(const UE::Mutable::Private::FProjector* Projector);
	void PreviewMutableMatrix(const FMatrix44f& Mat);
	void PreviewMutableShape(const UE::Mutable::Private::FShape* Shape);
	void PreviewMutableCurve(const FRichCurve& Curve);
	
private:
	void PrepareStringViewer();
	void PrepareImageViewer();
	void PrepareMeshViewer();
	void PrepareMaterialViewer();
	void PrepareLayoutViewer();
	void PrepareProjectorViewer();
};

/** The data of a row on the operation type dropdown. */
class FMutableOperationElement : public TSharedFromThis<FMutableOperationElement>
{
public:
	
	FMutableOperationElement(UE::Mutable::Private::EOpType InOperationType,FText OperationTypeName,uint32 OperationTypeInstanceCount)
	{
		OperationType = InOperationType;

		// Show the amount of instances of the operation type is found on model.
		// @note This if block will not be triggered when working with an operation type that is not present on the model
		// but added manually to the list of operation types (currently UE::Mutable::Private::OP_Type::None is an example of this)
		FText OperationsCountText = FText::GetEmpty();
		if (OperationTypeInstanceCount > 0)
		{
			OperationsCountText = FText::Format(INVTEXT(" - ({0})"), OperationTypeInstanceCount);
		}
		OperationTypeText = FText::Format(INVTEXT("{0}{1}"), OperationTypeName,OperationsCountText);
	}

	
	/** Set the color to be used by this entry so it gets used instead of the one defined by the coloring policy 
	 * @param InEntryColorOverride The color to be used for this entry.
	 */
	void SetEntryColorOverride(const FSlateColor& InEntryColorOverride)
	{
		bEntryColorWasOverriden = true;
		OperationTextOverrideColor = InEntryColorOverride;
	}

	
	UE::Mutable::Private::EOpType OperationType;
	
	FText OperationTypeText;
	
	FSlateColor OperationTextOverrideColor;
	bool bEntryColorWasOverriden = false;

	/** Get the type of visibility a slate representing this objects should have. */
	EVisibility GetEntryVisibility() const
	{
		return OperationType == UE::Mutable::Private::EOpType::NONE ? EVisibility::Collapsed : EVisibility::Visible;
	}
};



/** An row of the code tree in the SMutableCodeViewer. */
class FMutableCodeTreeElement : public TSharedFromThis<FMutableCodeTreeElement>
{
public:
	FMutableCodeTreeElement(int32 InIndexOnTree, const int32& InMutableStateIndex, const TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe>& InModel, UE::Mutable::Private::FOperation::ADDRESS InOperation, const FString& InCaption, TWeakPtr<SMutableCodeViewer> InCodeViewer, UE::Mutable::Private::EOpType InOperationType, const TSharedPtr<FMutableCodeTreeElement>* InDuplicatedOf = nullptr);
	
	void SetElementCurrentState(const int32& InStateIndex);

	void GenerateLabelText();
	
	int32 GetStateIndex() const;
	
	FSlateColor GetLabelColor() const;

public:

	/** Model this operation is part of */
	TSharedPtr<UE::Mutable::Private::FModel, ESPMode::ThreadSafe> MutableModel;
	
	/** Mutable Graph Node represented in this tree row. */
	UE::Mutable::Private::FOperation::ADDRESS MutableOperation;

	/** If this tree element is a duplicated of another op, this is the op. */
	TSharedPtr<FMutableCodeTreeElement> DuplicatedOf;

	/** Label to be used to represent this operation in the tree */
	FString MainLabel;

	/*
	 * Navigation metadata
	 */
	
	/** The current position of this element on the tree view. Used for navigation */
	int32 IndexOnTree;
	
	/*
	 * Dynamic data : Can and will change during the standard operation of the tree view object.
	 */

	/** If true means that it will not update when a runtime parameter on the state gets updated */
	bool bIsStateConstant = false;

	/** If true then the mesh or image of this operation may change during the state update */
	bool bIsDynamicResource = false;

	/** Flag that reflects the expanded state of the element in the tree view. It is used in order to know when an element
	 * should or should not has it's set state index updated (and the data that comes with it).
	 * @note It does not mean the element can be seen at the moment in the view, just that, in case of expansion of a parent
	 * element then this could be part of the view (if inside view space).
	 */
	bool bIsExpanded = false;

	bool bHasDuplicate = false;

private:
	/** Represents as part of what state this element is currently part of. Will change for elements that are children
	 * of duplicated elements (duplicated) since children of duplicated elements are still unique and therefore they can
	 * be present in more than one state (but can just appear in one state at any given time)
	 */
	int32 CurrentMutableStateIndex = -1;

	/** String representing additional data like the state this operation is from. It may be empty. */
	FString Caption;

	/** Code Viewer hosting the tree that uses this object. Will be used to evaluate the code to be displayed by the tree entry. */
	TWeakPtr<SMutableCodeViewer> HostCodeViewer;

	/** The type of the mutable operation. */
	UE::Mutable::Private::EOpType OperationType;
	
	/** The color to be used by the row representing this object */
	FSlateColor ZeroOperationColor = FColor(FColorList::DimGrey);
};
