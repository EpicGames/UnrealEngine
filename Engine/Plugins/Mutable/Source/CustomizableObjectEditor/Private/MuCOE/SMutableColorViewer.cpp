// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableColorViewer.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SMutableColorViewer::Construct(const FArguments& InArgs)
{
	const FText ColorValueTitle = LOCTEXT("ColorValuesTitle", "Color Values : ");

	const FText RedColorValueTitle = LOCTEXT("RedColorValueTitle", "Red : ");
	const FText GreenColorValueTitle = LOCTEXT("GreenColorValueTitle", "Green : ");
	const FText BlueColorValueTitle = LOCTEXT("BlueColorValueTitle", "Blue : ");
	const FText AlphaColorValueTitle = LOCTEXT("AlphaColorValueTitle", "Alpha : ");

	constexpr int32 IndentationSpace = 16;
	constexpr int32 AfterTitleSpacing = 4;
	constexpr int32 ColorPreviewHorizontalPadding = 18;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[	
			SNew(STextBlock).
			Text(ColorValueTitle)
		]
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(IndentationSpace, AfterTitleSpacing)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				// Red channel -----------------------------------------
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(RedColorValueTitle)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableColorViewer::GetRedValue)
					]
				]
				// -----------------------------------------------------

				// Green channel ---------------------------------------
				+SVerticalBox::Slot()
				.Padding(0, 1)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(GreenColorValueTitle)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableColorViewer::GetGreenValue)
					]
				]
				// -----------------------------------------------------

				// Blue channel ----------------------------------------
				+SVerticalBox::Slot()
				.Padding(0, 1)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(BlueColorValueTitle)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableColorViewer::GetBlueValue)
					]
				]
				// -----------------------------------------------------

				// Alpha channel ----------------------------------------
				+SVerticalBox::Slot()
				.Padding(0, 1)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(AlphaColorValueTitle)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableColorViewer::GetAlphaValue)
					]
				]
				// -----------------------------------------------------
			]
		
			// Color preview widget --------------------------------
			+ SHorizontalBox::Slot()
			.Padding(ColorPreviewHorizontalPadding,0)
			.MaxWidth(120)
			[
				SNew(SBorder)
				[
					SAssignNew(ColorPreview, SColorBlock)
					.UseSRGB(false)
					.Color(this, &SMutableColorViewer::GetColor)
				]
			]
			// -----------------------------------------------------
		]
	];
}

void SMutableColorViewer::SetColor(FVector4f InColor)
{
	Color = InColor;
}


FLinearColor SMutableColorViewer::GetColor() const
{
	FLinearColor BoxColor(Color);
	
	return BoxColor;
}

FText SMutableColorViewer::GetRedValue() const
{
	return FText::AsNumber(Color[0]);
}

FText SMutableColorViewer::GetGreenValue() const
{
	return FText::AsNumber(Color[1]);
}

FText SMutableColorViewer::GetBlueValue() const
{
	return FText::AsNumber(Color[2]);
}

FText SMutableColorViewer::GetAlphaValue() const
{
	return FText::AsNumber(Color[3]);
}

#undef LOCTEXT_NAMESPACE
