// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SColorBlock;


class SMutableColorViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableColorViewer) {}
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Bool to be used for this widget */
	void SetColor(FVector4f Color);

private:

	/** Color being displayed */
	FVector4f Color = FVector4f(0,0,0,1);

	/** Color box widget designed to serve as a preview of the color reported by mutable */
	TSharedPtr<SColorBlock> ColorPreview;

	/** Callback method invoked by the ColorPreview slate to get a FLinearColor object to display. */
	FLinearColor GetColor() const;
	
	/** Retrieve the values of each color component for the UI Texts to be updated */
	FText GetRedValue() const;
	FText GetGreenValue() const;
	FText GetBlueValue() const;
	FText GetAlphaValue() const;
};
