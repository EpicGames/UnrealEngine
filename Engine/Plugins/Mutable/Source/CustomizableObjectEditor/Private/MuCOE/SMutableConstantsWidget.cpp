// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableConstantsWidget.h"

#include "AssetViewUtils.h"
#include "Editor.h"
#include "Algo/StableSort.h"
#include "Framework/Views/TableViewMetadata.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCOE/SMutableCodeViewer.h"
#include "MuT/TypeInfo.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/STileView.h"
#include "MuCOE/Widgets/MutableMultiPageListView.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SRichTextHyperlink.h"

class ITableRow;
class SWidget;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

// Private namespace with utility functionality used by this slate object
namespace
{
	/** Provided a byte count this function proceeds to output that byte value as text alongside with it's unit of mesure (Bytes, KB...).
	 * @param SizeInBytes The amount of bytes to convert to a formatted text that represents it.
	 * @return A text representing the value provided as Bytes, Kilobytes, Megabytes or Gigabytes. 
	 */
	FText GenerateTextForSize(const uint64 SizeInBytes)
	{
		FString Unit = TEXT("Bytes");
		double Value = SizeInBytes;
		
		// B to KB
		if (SizeInBytes >= 1024)
		{
			Unit = TEXT("KB");
			Value = SizeInBytes / 1024.0;
			
			// KB to MB
			if (Value >= 1024.0)
			{
				Unit = TEXT("MB");
				Value = Value / 1024.0;
				
				// MB to GB
				if (Value >= 1024.0)
				{
					Unit = TEXT("GB");
					Value = Value / 1024.0;
				}
			}
		}

		FString OutputString = FString::Printf(TEXT("%.2f"), Value);
		OutputString.Append(" ");
		OutputString.Append(Unit);

		return FText::FromString(OutputString);
	}

	
	/**
	* Get the amount of channels for all the buffers in the provided BuffSetet
	* @param InBufferSet : The bufferset whose channels we want to count.
	* @return : the amount of channels found 
	*/
	int32 GetChannelCountOfBufferSet(const UE::Mutable::Private::FMeshBufferSet& InBufferSet)
	{
		int32 MeshBufferChannelsCount = 0;
		for (const UE::Mutable::Private::FMeshBuffer& MeshBuffer : InBufferSet.Buffers)
		{
			MeshBufferChannelsCount += MeshBuffer.Channels.Num();
		}
		return MeshBufferChannelsCount;
	}


	/**
	 * Get the amount of channels found in all buffers found in the provided mutable mesh
	 * @param MeshPtr Pointer to the mesh whose channels we want to count
	 * @return : the amount of channels found.
	 */
	int32 GetMeshChannelCount (UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> MeshPtr)
	{
		check(MeshPtr)

		int32 ChannelCount = 0;
		ChannelCount += GetChannelCountOfBufferSet(MeshPtr->GetVertexBuffers());
		ChannelCount += GetChannelCountOfBufferSet(MeshPtr->GetIndexBuffers());
		return ChannelCount;
	}
}


#pragma region SUPPORT_CLASSES


namespace MeshConstantTitles
{
	static const FName MeshID("ID");
	static const FName MeshVertices("Vertices");
	static const FName MeshIndices("Indices");
	static const FName MeshChannels("Channels");
	static const FName MeshMemory("Size");
}

class SMutableConstantMeshRow final : public SMultiColumnTableRow<TSharedPtr<FMutableConstantMeshElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantMeshElement>& InRowElement)
	{
		check(InRowElement);
		RowElement = InRowElement;

		SMultiColumnTableRow< TSharedPtr<FMutableConstantMeshElement> >::Construct(
		STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(RowElement);
		check(RowElement->MeshPtr)
		
		// Index
		if (ColumnName == MeshConstantTitles::MeshID)
		{
			const FText IndexAsText = FText::AsNumber(RowElement->IndexOnSourceVector);
			return SNew(STextBlock).Text(IndexAsText);
		}

		// The amount of mesh vertex buffer and image buffer channels
		if (ColumnName == MeshConstantTitles::MeshChannels)
		{
			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh = RowElement->MeshPtr;
			const int32 ChannelCount =  GetMeshChannelCount(Mesh);
			const FText ChannelCountText = FText::AsNumber(ChannelCount);
			return SNew(STextBlock).Text(ChannelCountText);
		}
		
		// The amount of indices of the mesh
		if (ColumnName == MeshConstantTitles::MeshIndices)
		{
			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh = RowElement->MeshPtr;
			const FText IndexCount = FText::AsNumber(Mesh->GetIndexCount());
			return SNew(STextBlock).Text(IndexCount);
		}

		// The amount of vertices of the mesh
		if (ColumnName == MeshConstantTitles::MeshVertices)
		{
			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh = RowElement->MeshPtr;
			const FText VertexCount = FText::AsNumber(Mesh->GetVertexCount());
			return SNew(STextBlock).Text(VertexCount);
		}

		// Memory used by the mesh 
		if (ColumnName == MeshConstantTitles::MeshMemory)
		{
			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> Mesh = RowElement->MeshPtr;
			const FText SizeAsText = GenerateTextForSize(Mesh->GetDataSize());
			return SNew(STextBlock).Text(SizeAsText);
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FMutableConstantMeshElement> RowElement;
	
};

class SMutableConstantStringRow final : public STableRow<TSharedPtr<FMutableConstantStringElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantStringElement>& InRowElement)
	{
		check(InRowElement);
		
		// Generate the text to be displayed taking in mind the string value held by the constant to be able to "preview"
		// it for easier navigation
		const FString MainString = FString::FromInt(InRowElement->IndexOnSourceVector) + FString(TEXT("_STR "));
		const FString ConstantStringText = InRowElement->MutableString;
		const FString GlimpseConstantText = ConstantStringText.Left(GlimpseCharacterCount);
		
		// Compose the FStrings to produce the UI text to be displayed
		FString UiString = MainString + "\"" + GlimpseConstantText;
		if (ConstantStringText.Len() > GlimpseConstantText.Len())
		{
			// Shortening occured, adding "..." to show it to the user
			UiString.Append("...");
		}
		UiString.Append("\"");
		
		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(FText::FromString(UiString))
		];
		
		STableRow< TSharedPtr<FMutableConstantStringElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}


private:

	/** Determines the amount of characters to be displayed on the string constant UI text as a preview of the value
	 * of the actual string constant.
	 */
	const uint32 GlimpseCharacterCount = 8;
};

namespace ImageConstantTitles
{
	static const FName ImageID("ID");
	static const FName ImageSize("Resolution");
	static const FName ImageMipMaps("Mip Maps");
	static const FName ImageFormat("Format");
	static const FName ImageTotalMemory("Size");
}

class SMutableConstantImageRow final : public SMultiColumnTableRow<TSharedPtr<FMutableConstantImageElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantImageElement>& InRowElement)
	{
		check(InRowElement);
		RowElement = InRowElement;
		
		SMultiColumnTableRow< TSharedPtr<FMutableConstantImageElement> >::Construct(
	STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(RowElement);

		const UE::Mutable::Private::FImage* Image = RowElement->ImagePtr.Get();
		
		// Index
		if (ColumnName == ImageConstantTitles::ImageID)
		{
			const FText IndexAsText = FText::AsNumber(RowElement->IndexOnSourceVector);
			return SNew(STextBlock).Text(IndexAsText);
		}

		// ImageSize (Resolution size)
		if (ColumnName == ImageConstantTitles::ImageSize)
		{
			const FString XSizeString = FString::FromInt( Image->GetSizeX());
			const FString YSizeString = FString::FromInt( Image->GetSizeY());
			const FString XSign = FString("x");
			const FText ImageResolution = FText::FromString( XSizeString + XSign + YSizeString);
			
			return SNew(STextBlock).Text(ImageResolution);
		}

		// Image Mip maps (LODs)
		if (ColumnName == ImageConstantTitles::ImageMipMaps)
		{
			const FText LodCount = FText::AsNumber(Image->GetLODCount());
			return SNew(STextBlock).Text(LodCount);
		}

		// Image format
		if (ColumnName == ImageConstantTitles::ImageFormat)
		{
			const UE::Mutable::Private::EImageFormat ImageFormat = Image->GetFormat();
			const uint8 ImageFormatValue = static_cast<uint8>(ImageFormat);
			const FText FormatAsText = FText::FromString(
				FString(UE::Mutable::Private::TypeInfo::s_imageFormatName[ImageFormatValue]));
			
			return SNew(STextBlock).Text(FormatAsText);
		}

		// Memory
		if (ColumnName == ImageConstantTitles::ImageTotalMemory)
		{
			const uint32 SizeInBytes = Image->GetDataSize();
			const FText SizeAsText = GenerateTextForSize(SizeInBytes);

			// Return the text with the size
			return SNew(STextBlock).Text(SizeAsText);
		}

		checkNoEntry();
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FMutableConstantImageElement> RowElement;
};


namespace MaterialConstantTitles
{
	static const FName MaterialID("ID");
	static const FName Material("Material");
	static const FName ImageParametersCount("Images");
	static const FName ScalarParametersCount("Scalars");
	static const FName ColorParametersCount("Colors");
}

class SMutableConstantMaterialRow final : public SMultiColumnTableRow<TSharedPtr<FMutableConstantMaterialElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantMaterialElement>& InRowElement)
	{
		check(InRowElement);
		RowElement = InRowElement;
		
		SMultiColumnTableRow< TSharedPtr<FMutableConstantMaterialElement> >::Construct(
	STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}
	
	
	void OnMaterialNavigation() const
	{
		if (RowElement)
		{
			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial> MutableMaterial = RowElement->MutableMaterial;
			if (MutableMaterial && MutableMaterial->PassthroughObject.IsResolved())
			{
				const UMaterialInterface* Material = MutableMaterial->PassthroughObject.Get();
				check(Material);
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Material);
			}
		}
	}

	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(RowElement);

		const UE::Mutable::Private::FMaterial* MutableMaterial = RowElement->MutableMaterial.Get();
		if (!MutableMaterial)
		{
			return SNullWidget::NullWidget;
		}
		
		// Index
		if (ColumnName == MaterialConstantTitles::MaterialID)
		{
			const FText IndexAsText = FText::AsNumber(RowElement->IndexOnSourceVector);
			return SNew(STextBlock).Text(IndexAsText);
		}
		
		// Material
		if (ColumnName == MaterialConstantTitles::Material)
		{
			// Try to get the actual object and if null then display the ID as fallback
			if (MutableMaterial->PassthroughObject.IsResolved())
			{
				if (const UMaterialInterface* Material = MutableMaterial->PassthroughObject.Get())
				{
					return SNew(SHyperlink)
						.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
						.Text(FText::FromString(Material->GetName()))
						.OnNavigate(this, &SMutableConstantMaterialRow::OnMaterialNavigation);
				}
			}
			else
			{
				FText MaterialIDText;
				
				UE::Mutable::Private::PASSTHROUGH_ID PassthroughMaterialID = MutableMaterial->PassthroughObject.GetId();
				if (PassthroughMaterialID != UE::Mutable::Private::PASSTHROUGH_ID_INVALID)
				{
					MaterialIDText = FText::AsNumber(PassthroughMaterialID);
				}
				else
				{
					MaterialIDText = FText::FromString(TEXT("INVALID_ID"));
				}
				
				return SNew(STextBlock).Text(MaterialIDText);
			}
		}
		
		// Image parameters count
		if (ColumnName == MaterialConstantTitles::ImageParametersCount)
		{
			const int32 ImageParametersCount = MutableMaterial->ImageParameters.Num();
			return SNew(STextBlock).Text(FText::AsNumber(ImageParametersCount));
		}
		
		// Color Parameters Count
		if (ColumnName == MaterialConstantTitles::ColorParametersCount)
		{
			const int32 ColorParametersCount = MutableMaterial->ColorParameters.Num();
			return SNew(STextBlock).Text(FText::AsNumber(ColorParametersCount));
		}
		
		// Scalar Parameters Count
		if (ColumnName == MaterialConstantTitles::ScalarParametersCount)
		{
			const int32 ScalarParametersCount = MutableMaterial->ScalarParameters.Num();
			return SNew(STextBlock).Text(FText::AsNumber(ScalarParametersCount));
		}

		checkNoEntry();
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FMutableConstantMaterialElement> RowElement;
};


class SMutableConstantLayoutRow final : public STableRow<TSharedPtr<FMutableConstantLayoutElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantLayoutElement>& InRowElement)
	{
		check(InRowElement);
		const FText LayoutProxyText = FText::Format( LOCTEXT("LayoutConstantProxyLabel", "{0}_LAYOUT "),InRowElement->IndexOnSourceVector);

		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(LayoutProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantLayoutElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
	
};

class SMutableConstantProjectorRow final : public STableRow<TSharedPtr<FMutableConstantProjectorElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantProjectorElement>& InRowElement)
	{
		check(InRowElement);
		const FText ProjectorProxyText = FText::Format( LOCTEXT("ProjectorConstantProxyLabel", "{0}_PROJECTOR "),InRowElement->IndexOnSourceVector);
		
		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(ProjectorProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantProjectorElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
	
};

class SMutableConstantMatrixRow final : public STableRow<TSharedPtr<FMutableConstantMatrixElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantMatrixElement>& InRowElement)
	{
		check(InRowElement);
		const FText MatrixProxyText = FText::Format( LOCTEXT("MatrixConstantProxyLabel", "{0}_MATRIX "),InRowElement->IndexOnSourceVector);

		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(MatrixProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantMatrixElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
};

class SMutableConstantShapeRow final : public STableRow<TSharedPtr<FMutableConstantShapeElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantShapeElement>& InRowElement)
	{
		check(InRowElement);
		const FText ShapeProxyText = FText::Format( LOCTEXT("ShapeConstantProxyLabel", "{0}_SHAPE "),InRowElement->IndexOnSourceVector);

		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(ShapeProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantShapeElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
};

class SMutableConstantCurveRow final : public STableRow<TSharedPtr<FMutableConstantCurveElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantCurveElement>& InRowElement)
	{
		check(InRowElement);
		const FText CurveProxyText = FText::Format( LOCTEXT("CurveConstantProxyLabel", "{0}_CURVE "),InRowElement->IndexOnSourceVector);

		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(CurveProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantCurveElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
};


class SMutableConstantSkeletonRow final : public STableRow<TSharedPtr<FMutableConstantSkeletonElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantSkeletonElement>& InRowElement)
	{
		check(InRowElement);
		const FText SkeletonProxyText = FText::Format( LOCTEXT("SkeletonConstantProxyLabel", "{0}_SKELETON "),InRowElement->IndexOnSourceVector);

		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(SkeletonProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantSkeletonElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
};

class SMutableConstantPhysicsRow final : public STableRow<TSharedPtr<FMutableConstantPhysicsElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableConstantPhysicsElement>& InRowElement)
	{
		check(InRowElement);
		const FText PhysicsProxyText = FText::Format( LOCTEXT("PhysicsConstantProxyLabel", "{0}_PHYSICS "),InRowElement->IndexOnSourceVector);

		this->ChildSlot
		[
			SNew(STextBlock)
			.Text(PhysicsProxyText)
		];

		STableRow< TSharedPtr<FMutableConstantPhysicsElement> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView);
	}
};

#pragma endregion SUPPORT_CLASSES



void SMutableConstantsWidget::Construct(const FArguments& InArgs, TSharedPtr<UE::Mutable::Private::FModel> InModel,  TSharedPtr<SMutableCodeViewer> InMutableCodeViewerPtr)
{
	// A pointer to MutableCodeViewerPtr is required in order to be able to invoke the preview of our constants
	check (InMutableCodeViewerPtr);
	MutableCodeViewerPtr = InMutableCodeViewerPtr;
	
	// A pointer to the mutable program object is required to get the constants data
	check(InModel);
	SetModel(InModel);
	
	// Formatting constants
	constexpr uint32 InBetweenListsVerticalPadding = 4;
	
	// Vertical size for each entry 
	constexpr float ProxyEntryHeight = 20;
	
	// Panel title
	const FText ConstantsPanelTitle = LOCTEXT("ConstantsPanelName", "Constants : ");

	// Hack to allow us to later set the array to be used. if no array is provided then the children slate to contain them later will not exist
	TArray<TSharedPtr<FMutableConstantImageElement>> DummyImageElementsArray;
	
	// Image Constants List View object and Handler
	TSharedPtr<SListView<TSharedPtr<FMutableConstantImageElement>>> ConstantImagesListView =
		SNew(SListView<TSharedPtr<FMutableConstantImageElement>>)
	.OnGenerateRow(this,&SMutableConstantsWidget::OnGenerateImageRow)
		// We do require to provide something here or the slate to contain the children will not get generated
	.ListItemsSource(&DummyImageElementsArray)		
	.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedImageChanged)
	.SelectionMode(ESelectionMode::Single)
	.HeaderRow
	(
		SNew(SHeaderRow)
		+ SHeaderRow::Column(ImageConstantTitles::ImageID)
		.DefaultLabel(FText::FromName(ImageConstantTitles::ImageID))
		.OnSort(this, &SMutableConstantsWidget::OnImageTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetImageListColumnSortMode, ImageConstantTitles::ImageID )
		.FillWidth(0.28f)
	
		+ SHeaderRow::Column(ImageConstantTitles::ImageSize)
		.DefaultLabel(FText::FromName(ImageConstantTitles::ImageSize))
		.DefaultTooltip(FText(LOCTEXT("ImageResolutionColumnToolTip","Pixel resolution")))
		.OnSort(this, &SMutableConstantsWidget::OnImageTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetImageListColumnSortMode, ImageConstantTitles::ImageSize )
		
		+ SHeaderRow::Column(ImageConstantTitles::ImageMipMaps)
		.DefaultLabel(FText::FromName(ImageConstantTitles::ImageMipMaps))
		.DefaultTooltip(FText(LOCTEXT("ImageMipMapsColumnToolTip","Amount of Mip maps")))
		.OnSort(this, &SMutableConstantsWidget::OnImageTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetImageListColumnSortMode, ImageConstantTitles::ImageMipMaps )

		+ SHeaderRow::Column(ImageConstantTitles::ImageFormat)
		.DefaultLabel(FText::FromName(ImageConstantTitles::ImageFormat))
		.DefaultTooltip(FText(LOCTEXT("ImageFormatColumnToolTip","Image Format")))
		.OnSort(this, &SMutableConstantsWidget::OnImageTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetImageListColumnSortMode, ImageConstantTitles::ImageFormat )

		+ SHeaderRow::Column(ImageConstantTitles::ImageTotalMemory)
		.DefaultLabel(FText::FromName(ImageConstantTitles::ImageTotalMemory))
		.DefaultTooltip(FText(LOCTEXT("ImageMemorySizeColumnToolTip","Memory size")))
		.OnSort(this, &SMutableConstantsWidget::OnImageTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetImageListColumnSortMode, ImageConstantTitles::ImageTotalMemory )

	);


	TArray<TSharedPtr<FMutableConstantMeshElement>> DummyMeshElementsArray;

	// Handled list view for mutable constant meshes
	TSharedPtr<SListView<TSharedPtr<FMutableConstantMeshElement>>> ConstantMeshesListView =
		SNew(SListView<TSharedPtr<FMutableConstantMeshElement>>)
	.OnGenerateRow(this,&SMutableConstantsWidget::OnGenerateMeshRow)
		// We do require to provide something here or the slate to contain the children will not get generated
	.ListItemsSource(&DummyMeshElementsArray)		
	.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedMeshChanged)
	.SelectionMode(ESelectionMode::Single)
	.HeaderRow
	(
		SNew(SHeaderRow)
		+ SHeaderRow::Column(MeshConstantTitles::MeshID)
		.DefaultLabel(FText::FromName(MeshConstantTitles::MeshID))
		.OnSort(this, &SMutableConstantsWidget::OnMeshTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMeshListColumnSortMode, MeshConstantTitles::MeshID)
		.FillWidth(0.28f)
	
		+ SHeaderRow::Column(MeshConstantTitles::MeshVertices)
		.DefaultLabel(FText::FromName(MeshConstantTitles::MeshVertices))
		.DefaultTooltip(FText(LOCTEXT("MeshVerticesCountColumnToolTip","Amount of vertices")))
		.OnSort(this, &SMutableConstantsWidget::OnMeshTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMeshListColumnSortMode, MeshConstantTitles::MeshVertices)

		+ SHeaderRow::Column(MeshConstantTitles::MeshIndices)
		.DefaultLabel(FText::FromName(MeshConstantTitles::MeshIndices))
		.DefaultTooltip(FText(LOCTEXT("MeshIndicesCountColumnToolTip","Amount of indices")))
		.OnSort(this, &SMutableConstantsWidget::OnMeshTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMeshListColumnSortMode, MeshConstantTitles::MeshIndices)

		+ SHeaderRow::Column(MeshConstantTitles::MeshChannels)
		.DefaultLabel(FText::FromName(MeshConstantTitles::MeshChannels))
		.DefaultTooltip(FText(LOCTEXT("MeshVertexChannelsCountColumnToolTip","Amount of channels")))
		.OnSort(this, &SMutableConstantsWidget::OnMeshTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMeshListColumnSortMode, MeshConstantTitles::MeshChannels)
		
		+ SHeaderRow::Column(MeshConstantTitles::MeshMemory)
		.DefaultLabel(FText::FromName(MeshConstantTitles::MeshMemory))
		.DefaultTooltip(FText(LOCTEXT("MeshMemoryColumnToolTip","Memory size")))
		.OnSort(this, &SMutableConstantsWidget::OnMeshTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMeshListColumnSortMode, MeshConstantTitles::MeshMemory)
	);
	

	// Hack to allow us to later set the array to be used. if no array is provided then the children slate to contain them later will not exist
	TArray<TSharedPtr<FMutableConstantMaterialElement>> DummyMaterialElementsArray;
	
	// Image Constants List View object and Handler
	TSharedPtr<SListView<TSharedPtr<FMutableConstantMaterialElement>>> ConstantMaterialsListView =
		SNew(SListView<TSharedPtr<FMutableConstantMaterialElement>>)
	.OnGenerateRow(this,&SMutableConstantsWidget::OnGenerateMaterialRow)
		// We do require to provide something here or the slate to contain the children will not get generated
	.ListItemsSource(&DummyMaterialElementsArray)		
	.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedMaterialChanged)
	.SelectionMode(ESelectionMode::Single)
	.HeaderRow
	(
		SNew(SHeaderRow)
		+ SHeaderRow::Column(MaterialConstantTitles::MaterialID)
		.DefaultLabel(FText::FromName(MaterialConstantTitles::MaterialID))
		.OnSort(this, &SMutableConstantsWidget::OnMaterialsTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMaterialListColumnSortMode, MaterialConstantTitles::MaterialID )
		.FillWidth(0.20f)
		
		+ SHeaderRow::Column(MaterialConstantTitles::Material)
		.DefaultLabel(FText::FromName(MaterialConstantTitles::Material))
		.FillWidth(0.40f)
	
		+ SHeaderRow::Column(MaterialConstantTitles::ImageParametersCount)
		.DefaultLabel(FText::FromName(MaterialConstantTitles::ImageParametersCount))
		.OnSort(this, &SMutableConstantsWidget::OnMaterialsTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMaterialListColumnSortMode, MaterialConstantTitles::ImageParametersCount )
		.FillWidth(0.20f)

		+ SHeaderRow::Column(MaterialConstantTitles::ColorParametersCount)
		.DefaultLabel(FText::FromName(MaterialConstantTitles::ColorParametersCount))
		.OnSort(this, &SMutableConstantsWidget::OnMaterialsTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMaterialListColumnSortMode, MaterialConstantTitles::ColorParametersCount )
		.FillWidth(0.20f)

		+ SHeaderRow::Column(MaterialConstantTitles::ScalarParametersCount)
		.DefaultLabel(FText::FromName(MaterialConstantTitles::ScalarParametersCount))
		.OnSort(this, &SMutableConstantsWidget::OnMaterialsTableSortRequested)
		.SortMode(this, &SMutableConstantsWidget::GetMaterialListColumnSortMode, MaterialConstantTitles::ScalarParametersCount )
		.FillWidth(0.20f)
	);
	
	// Child structure
	this->ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)
		
		// String constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(StringsExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnStringsRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawStringsAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantStringsSlate,STileView<TSharedPtr<FMutableConstantStringElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedStringChanged)
				.ListItemsSource(&ConstantStringElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this, &SMutableConstantsWidget::OnGenerateStringRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]

		// Image constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(ImagesExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnImagesRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawImagesAreaTitle)
			.BodyContent()
			[
				// Custom slate that will handle the updating of the elements displayed by the ConstantImagesListView object
				SAssignNew(ImageListViewHandler,SMutableMultiPageListView<TSharedPtr<FMutableConstantImageElement>>)
					.HostedListView(ConstantImagesListView)
					.ElementsToSeparateInPages(ConstantImageElements)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Mesh Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(MeshesExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnMeshesRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawMeshesAreaTitle)
			.BodyContent()
			[
				// Custom slate that will handle the updating of the elements displayed by the ConstantMeshesListView object
				SAssignNew(MeshListViewHandler, SMutableMultiPageListView<TSharedPtr<FMutableConstantMeshElement>>)
				.HostedListView(ConstantMeshesListView)	
				.ElementsToSeparateInPages(ConstantMeshElements)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Layout Constants		
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(LayoutsExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnLayoutsRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawLayoutsAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantLayoutsSlate,STileView<TSharedPtr<FMutableConstantLayoutElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedLayoutChanged)
				.ListItemsSource(&ConstantLayoutElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGenerateLayoutRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Projector Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(ProjectorsExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnProjectorsRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawProjectorsAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantProjectorsSlate,STileView<TSharedPtr<FMutableConstantProjectorElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedProjectorChanged)
				.ListItemsSource(&ConstantProjectorElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGenerateProjectorRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Matrix Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(MatricesExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnMatricesRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawMatricesAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantMatricesSlate,STileView<TSharedPtr<FMutableConstantMatrixElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedMatrixChanged)
				.ListItemsSource(&ConstantMatrixElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGenerateMatrixRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Shape Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(ShapesExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnShapesRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawShapesAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantShapesSlate,STileView<TSharedPtr<FMutableConstantShapeElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedShapeChanged)
				.ListItemsSource(&ConstantShapeElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGenerateShapeRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Curve Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(CurvesExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnCurvesRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawCurvesAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantCurvesSlate,STileView<TSharedPtr<FMutableConstantCurveElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedCurveChanged)
				.ListItemsSource(&ConstantCurveElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGenerateCurveRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Skeleton Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(SkeletonsExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnSkeletonsRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawSkeletonsAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantSkeletonsSlate,STileView<TSharedPtr<FMutableConstantSkeletonElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedSkeletonChanged)
				.ListItemsSource(&ConstantSkeletonElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGenerateSkeletonRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]

		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]
		
		// Physics Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(PhysicsExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnPhysicsRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawPhysicsAreaTitle)
			.BodyContent()
			[
				SAssignNew(ConstantPhysicsSlate,STileView<TSharedPtr<FMutableConstantPhysicsElement>>)
				.OnSelectionChanged(this,&SMutableConstantsWidget::OnSelectedPhysicsChanged)
				.ListItemsSource(&ConstantPhysicsElements)
				.ItemHeight(ProxyEntryHeight)
				.OnGenerateTile(this,&SMutableConstantsWidget::OnGeneratePhysicsRow)
				.SelectionMode(ESelectionMode::Single)
			]
		]
		
		+ SScrollBox::Slot()
		[
			SNew(SSeparator)
		]

		// Material Constants
		+ SScrollBox::Slot()
		.Padding(0,InBetweenListsVerticalPadding)
		[
			SAssignNew(MaterialsExpandableArea,SExpandableArea)
			.OnAreaExpansionChanged(this,&SMutableConstantsWidget::OnMaterialsRegionExpansionChanged)
			.InitiallyCollapsed(true)
			.AreaTitle(this,&SMutableConstantsWidget::OnDrawMaterialsAreaTitle)
			.BodyContent()
			[
				// Custom slate that will handle the updating of the elements displayed by the ConstantMeshesListView object
				SAssignNew(MaterialistViewHandler ,SMutableMultiPageListView<TSharedPtr<FMutableConstantMaterialElement>>)
				.HostedListView(ConstantMaterialsListView)	
				.ElementsToSeparateInPages(ConstantMaterialElements)
			]
		]
	];

	// Store all the expandable areas so they are later reachable using loops
	ExpandableAreas.Add(StringsExpandableArea);
	ExpandableAreas.Add(ImagesExpandableArea);
	ExpandableAreas.Add(MeshesExpandableArea);
	ExpandableAreas.Add(LayoutsExpandableArea);
	ExpandableAreas.Add(ProjectorsExpandableArea);
	ExpandableAreas.Add(ShapesExpandableArea);
	ExpandableAreas.Add(CurvesExpandableArea);
	ExpandableAreas.Add(MatricesExpandableArea);
	ExpandableAreas.Add(SkeletonsExpandableArea);
	ExpandableAreas.Add(PhysicsExpandableArea);
	ExpandableAreas.Add(MaterialsExpandableArea);
}



void SMutableConstantsWidget::SetModel(const TSharedPtr<UE::Mutable::Private::FModel>& InModel)
{
	// Make sure we do not process the setting of the same program object as the one already set
	if (MutableModel == InModel)
	{
		return;
	}
	
	// Set only once the program that is being used. No further updates should be required
	MutableModel = InModel;
	
	// Generate the backend for the lists used in this object
	if (MutableModel != nullptr)
	{
		LoadConstantElements();
	}
}

#pragma region Row Generation

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateStringRow(
	TSharedPtr<FMutableConstantStringElement> MutableConstantStringElement, const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantStringRow> Row = SNew(SMutableConstantStringRow, OwnerTable, MutableConstantStringElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateImageRow(
	TSharedPtr<FMutableConstantImageElement> MutableConstantImageElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantImageRow> Row = SNew(SMutableConstantImageRow,OwnerTable,MutableConstantImageElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateMeshRow(
	TSharedPtr<FMutableConstantMeshElement> MutableConstantMeshElement, const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantMeshRow> Row = SNew(SMutableConstantMeshRow, OwnerTable, MutableConstantMeshElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateMaterialRow(TSharedPtr<FMutableConstantMaterialElement> MutableConstantMaterialElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantMaterialRow> Row = SNew(SMutableConstantMaterialRow, OwnerTable, MutableConstantMaterialElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateLayoutRow(
	TSharedPtr<FMutableConstantLayoutElement> MutableConstantLayoutElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantLayoutRow> Row = SNew(SMutableConstantLayoutRow,OwnerTable,MutableConstantLayoutElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateProjectorRow(
	TSharedPtr<FMutableConstantProjectorElement> MutableConstantProjectorElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantProjectorRow> Row = SNew(SMutableConstantProjectorRow,OwnerTable,MutableConstantProjectorElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateMatrixRow(
	TSharedPtr<FMutableConstantMatrixElement> MutableConstantMatrixElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantMatrixRow> Row = SNew(SMutableConstantMatrixRow,OwnerTable,MutableConstantMatrixElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateShapeRow(
	TSharedPtr<FMutableConstantShapeElement> MutableConstantShapeElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantShapeRow> Row = SNew(SMutableConstantShapeRow,OwnerTable,MutableConstantShapeElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateCurveRow(
	TSharedPtr<FMutableConstantCurveElement> MutableConstantCurveElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantCurveRow> Row = SNew(SMutableConstantCurveRow,OwnerTable,MutableConstantCurveElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGenerateSkeletonRow(
	TSharedPtr<FMutableConstantSkeletonElement> MutableConstantSkeletonElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantSkeletonRow> Row = SNew(SMutableConstantSkeletonRow,OwnerTable,MutableConstantSkeletonElement);
	return Row;
}

TSharedRef<ITableRow> SMutableConstantsWidget::OnGeneratePhysicsRow(
	TSharedPtr<FMutableConstantPhysicsElement> MutableConstantPhysicsElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableConstantPhysicsRow> Row = SNew(SMutableConstantPhysicsRow,OwnerTable,MutableConstantPhysicsElement);
	return Row;
}


void SMutableConstantsWidget::OnImageTableSortRequested(EColumnSortPriority::Type ColumnSortPriority, const FName& ColumnID,
                                                        EColumnSortMode::Type ColumnSortMode)
{
	ImageConstantsLastSortedColumnID = ColumnID;
	ImageListSortMode = ColumnSortMode;

	Algo::StableSort(*ConstantImageElements, [&](const TSharedPtr<FMutableConstantImageElement>& A, const TSharedPtr<FMutableConstantImageElement>& B)
	{
		// Sort by image id
		if (ColumnID == ImageConstantTitles::ImageID)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->IndexOnSourceVector < B->IndexOnSourceVector;
			}
			return A->IndexOnSourceVector > B->IndexOnSourceVector;
		}
		
		// Sort Image mip maps
		if (ColumnID == ImageConstantTitles::ImageMipMaps)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->ImagePtr->GetLODCount() < B->ImagePtr->GetLODCount();
			}
			return A->ImagePtr->GetLODCount() > B->ImagePtr->GetLODCount();
		}

		// Sort by image format
		if (ColumnID == ImageConstantTitles::ImageFormat)
		{
			const uint8 AImageFormatValue = static_cast<uint8>(A->ImagePtr->GetFormat());
			const FString AImageFormatString = FString(UE::Mutable::Private::TypeInfo::s_imageFormatName[AImageFormatValue]);
			
			const uint8 BImageFormatValue = static_cast<uint8>(B->ImagePtr->GetFormat());
			const FString BImageFormatString = FString(UE::Mutable::Private::TypeInfo::s_imageFormatName[BImageFormatValue]);
			
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return AImageFormatString.Compare(BImageFormatString) < 0;
			}
			return AImageFormatString.Compare(BImageFormatString) > 0;
		}
		
		
		// Sort by image size
		if (ColumnID == ImageConstantTitles::ImageSize)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->ImagePtr->GetSizeX() *  A->ImagePtr->GetSizeY() <
					B->ImagePtr->GetSizeX() *  B->ImagePtr->GetSizeY();
			}
			return A->ImagePtr->GetSizeX() *  A->ImagePtr->GetSizeY() >
					B->ImagePtr->GetSizeX() *  B->ImagePtr->GetSizeY();
		}
		
		
		// Sort by image used memory
		if (ColumnID == ImageConstantTitles::ImageTotalMemory)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->ImagePtr->GetDataSize() < B->ImagePtr->GetDataSize();
			}
			return A->ImagePtr->GetDataSize() > B->ImagePtr->GetDataSize();
		}
		
		return false;
	});
	
	if (ImageListViewHandler)
	{
		ImageListViewHandler->RegeneratePage();
	}
}


EColumnSortMode::Type SMutableConstantsWidget::GetImageListColumnSortMode(FName ColumnName) const
{
	if (ImageConstantsLastSortedColumnID != ColumnName)
	{
		return EColumnSortMode::None;
	}

	return ImageListSortMode;
}


void SMutableConstantsWidget::OnMeshTableSortRequested(EColumnSortPriority::Type ColumnSortPriority, const FName& ColumnID, EColumnSortMode::Type ColumnSortMode)
{
	// If the colum has been sorted on one way now do the inverse way of sorting
	MeshConstantsLastSortedColumnID = ColumnID;
	MeshListSortMode = ColumnSortMode;

	Algo::StableSort(*ConstantMeshElements, [&](const TSharedPtr<FMutableConstantMeshElement>& A, const TSharedPtr<FMutableConstantMeshElement>& B)
	{
		// Sort by mesh id
		if (ColumnID == MeshConstantTitles::MeshID)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->IndexOnSourceVector < B->IndexOnSourceVector;
			}
			return A->IndexOnSourceVector > B->IndexOnSourceVector;
		}
	
		// Sort by vertex count
		if (ColumnID == MeshConstantTitles::MeshVertices)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->MeshPtr->GetVertexCount() < B->MeshPtr->GetVertexCount();
			}
			return A->MeshPtr->GetVertexCount() > B->MeshPtr->GetVertexCount();
		}

		// Sort by index count
		if (ColumnID == MeshConstantTitles::MeshIndices)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->MeshPtr->GetIndexCount() < B->MeshPtr->GetIndexCount();
			}
			return A->MeshPtr->GetIndexCount() > B->MeshPtr->GetIndexCount();
		}

		// Sort by the amount of channels in the vertex and index buffers
		if (ColumnID == MeshConstantTitles::MeshChannels)
		{
			const int32 AChannelCount = GetMeshChannelCount(A->MeshPtr);
			const int32 BChannelCount = GetMeshChannelCount(B->MeshPtr);
			
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return AChannelCount < BChannelCount;
			}
			return AChannelCount > BChannelCount;
		}
		
		// Sort by the amount memory used by the mesh
		if (ColumnID == MeshConstantTitles::MeshMemory)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->MeshPtr->GetDataSize() < B->MeshPtr->GetDataSize();
			}
			return A->MeshPtr->GetDataSize() > B->MeshPtr->GetDataSize();
		}
	
		return false;
	});
	
	if (MeshListViewHandler)
	{
		MeshListViewHandler->RegeneratePage();
	}
}



EColumnSortMode::Type SMutableConstantsWidget::GetMeshListColumnSortMode(FName ColumnName) const
{
	if (MeshConstantsLastSortedColumnID != ColumnName)
	{
		return EColumnSortMode::None;
	}

	return MeshListSortMode;
}

void SMutableConstantsWidget::OnMaterialsTableSortRequested(EColumnSortPriority::Type ColumnSortPriority, const FName& ColumnID,
	EColumnSortMode::Type ColumnSortMode)
{
	// If the colum has been sorted on one way now do the inverse way of sorting
	MaterialConstantsLastSortedColumnID = ColumnID;
	MaterialListSortMode = ColumnSortMode;

	Algo::StableSort(*ConstantMaterialElements, [&](const TSharedPtr<FMutableConstantMaterialElement>& A, const TSharedPtr<FMutableConstantMaterialElement>& B)
	{
		if (!A || !B)
		{
			return false;
		}
			
		// Sort by material id
		if (ColumnID == MaterialConstantTitles::MaterialID)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->IndexOnSourceVector < B->IndexOnSourceVector;
			}
			return A->IndexOnSourceVector > B->IndexOnSourceVector;
		}
			
		// Sort by Image Parameters count
		if (ColumnID == MaterialConstantTitles::ImageParametersCount)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->MutableMaterial->ImageParameters.Num() < B->MutableMaterial->ImageParameters.Num();
			}
			return A->MutableMaterial->ImageParameters.Num() > B->MutableMaterial->ImageParameters.Num();
		}

		// Sort by Color Parameters count
		if (ColumnID == MaterialConstantTitles::ColorParametersCount)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->MutableMaterial->ColorParameters.Num() < B->MutableMaterial->ColorParameters.Num();
			}
			return A->MutableMaterial->ColorParameters.Num() > B->MutableMaterial->ColorParameters.Num();
		}
		
		// Sort by Scalar Parameters count
		if (ColumnID == MaterialConstantTitles::ScalarParametersCount)
		{
			if (ColumnSortMode == EColumnSortMode::Ascending)
			{
				return A->MutableMaterial->ScalarParameters.Num() < B->MutableMaterial->ScalarParameters.Num();
			}
			return A->MutableMaterial->ScalarParameters.Num() > B->MutableMaterial->ScalarParameters.Num();
		}
	
		return false;
	});
	
	if (MaterialistViewHandler)
	{
		MaterialistViewHandler->RegeneratePage();
	}
}


EColumnSortMode::Type SMutableConstantsWidget::GetMaterialListColumnSortMode(FName ColumnName) const
{
	if (MaterialConstantsLastSortedColumnID != ColumnName)
	{
		return EColumnSortMode::None;
	}

	return MaterialListSortMode;
}


#pragma endregion 

#pragma  region  Expansions Handling

void SMutableConstantsWidget::OnStringsRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(StringsExpandableArea);
	}
}

void SMutableConstantsWidget::OnImagesRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(ImagesExpandableArea);
	}
}

void SMutableConstantsWidget::OnMeshesRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(MeshesExpandableArea);
	}
}

void SMutableConstantsWidget::OnLayoutsRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(LayoutsExpandableArea);
	}
}

void SMutableConstantsWidget::OnProjectorsRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(ProjectorsExpandableArea);
	}
}

void SMutableConstantsWidget::OnMatricesRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(MatricesExpandableArea);
	}
}

void SMutableConstantsWidget::OnShapesRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(ShapesExpandableArea);
	}
}

void SMutableConstantsWidget::OnCurvesRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(CurvesExpandableArea);
	}
}

void SMutableConstantsWidget::OnSkeletonsRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(SkeletonsExpandableArea);
	}
}

void SMutableConstantsWidget::OnPhysicsRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(PhysicsExpandableArea);
	}
}


void SMutableConstantsWidget::OnMaterialsRegionExpansionChanged(bool bExpanded)
{
	if (bExpanded)
	{
		ContractExpandableAreas(MaterialsExpandableArea);
	}
}


void SMutableConstantsWidget::ContractExpandableAreas(const TSharedPtr<SExpandableArea>& InException)
{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	if(!InException)
	{
		UE_LOGF(LogTemp,Warning, "No valid expandable area has been provided as exception : All expandable areas will therefore get contracted");
	}
#endif
	
	for (TSharedPtr<SExpandableArea>& CurrentExpandableArea : ExpandableAreas)
	{
		if (CurrentExpandableArea == InException)
		{
			continue;
		}

		CurrentExpandableArea->SetExpanded(false);
	}
}

#pragma endregion 

#pragma  region Element caches loading

void SMutableConstantsWidget::LoadConstantElements()
{
	LoadConstantStrings();
	LoadConstantImages();
	LoadConstantMeshes();
	LoadConstantLayouts();
	LoadConstantProjectors();
	LoadConstantMatrices();
	LoadConstantShapes();
	LoadConstantCurves();
	LoadConstantSkeletons();
	LoadConstantPhysics();
	LoadConstantMaterials();
}

void SMutableConstantsWidget::LoadConstantStrings()
{
	check(MutableModel);

	const int32 ConstantsCount = MutableModel->GetProgram().ConstantStrings.Num();
	ConstantStringElements.Empty(ConstantsCount);
	uint64 ConstantStringsAccumulatedSize = 0;
	
	for (int32 StringAddressIndex = 0; StringAddressIndex < ConstantsCount; StringAddressIndex++)
	{
		TSharedPtr<FMutableConstantStringElement> ConstantStringElement = MakeShared<FMutableConstantStringElement>();
		ConstantStringElement->MutableString = MutableModel->GetProgram().ConstantStrings[StringAddressIndex];
		ConstantStringElement->IndexOnSourceVector = StringAddressIndex;
		
		// Cache resource size
		// in case we change the type of the contents of the UE::Mutable::Private::string we check its size as if it was a vector<>
		ConstantStringsAccumulatedSize += ConstantStringElement->MutableString.GetAllocatedSize();
		
		ConstantStringElements.Add(ConstantStringElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantStringsFormattedSize = GenerateTextForSize(ConstantStringsAccumulatedSize);
}

void SMutableConstantsWidget::LoadConstantImages()
{
	check(MutableModel);

	const int32 ConstantsCount = MutableModel->GetProgram().ConstantImages.Num();
	if (ConstantImageElements)
	{
		ConstantImageElements->Empty(ConstantsCount);
	}
	else
	{
		ConstantImageElements = MakeShared<TArray<TSharedPtr<FMutableConstantImageElement>>>();
		ConstantImageElements->Reserve(ConstantsCount);
	}
	
	uint64 ConstantImagesAccumulatedSize = 0;
	
	const TSharedRef<UE::Mutable::Private::FModelReader> ModelReader = UCustomizableObjectSystem::GetInstance()->GetPrivate()->MutableSystem->StreamInterface.ToSharedRef();

	for (int32 ImageIndex = 0; ImageIndex < ConstantsCount; ImageIndex++)
	{
		TSharedPtr<FMutableConstantImageElement> ConstantImageElement = MakeShared<FMutableConstantImageElement>();
		
		MutableModel->GetConstant(ImageIndex, ConstantImageElement->ImagePtr, 0,
		[this, ModelReader](int32 RomIndex)
		{
			return MutableModel->RomManager.GetRom<UE::Mutable::Private::FImage>(RomIndex, ModelReader).GetResult();
		},
		[this](int32 x, int32 y, int32 m, UE::Mutable::Private::EImageFormat f, UE::Mutable::Private::EInitializationType i) 
		{ 
			return UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FImage>(x, y, m, f, i); 
		});

		ConstantImageElement->IndexOnSourceVector = ImageIndex;
		
		ConstantImagesAccumulatedSize += ConstantImageElement->ImagePtr->GetDataSize();
		
		ConstantImageElements->Add(ConstantImageElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantImagesFormattedSize = GenerateTextForSize(ConstantImagesAccumulatedSize);
}

void SMutableConstantsWidget::LoadConstantMeshes()
{
	check(MutableModel);
	
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantMeshes.Num();
	if (ConstantMeshElements)
	{
		ConstantMeshElements->Empty(ConstantsCount);
	}
	else
	{
		ConstantMeshElements = MakeShared<TArray<TSharedPtr<FMutableConstantMeshElement>>>();
		ConstantMeshElements->Reserve(ConstantsCount);
	}
	
	uint64 ConstantMeshesAccumulatedSize = 0;
	
	const TSharedRef<UE::Mutable::Private::FModelReader> ModelReader = UCustomizableObjectSystem::GetInstance()->GetPrivate()->MutableSystem->StreamInterface.ToSharedRef();

	for (int32 MeshIndex = 0; MeshIndex < ConstantsCount; MeshIndex++)
	{
		TSharedPtr< FMutableConstantMeshElement> ConstantMeshElement = MakeShared<FMutableConstantMeshElement>();

		MutableModel->GetConstant(MeshIndex, INDEX_NONE, {}, ConstantMeshElement->MeshPtr, UE::Mutable::Private::EMeshContentFlags::AllFlags,
		[this, ModelReader](int32 RomIndex)
		{
			return MutableModel->RomManager.GetRom<UE::Mutable::Private::FMesh>(RomIndex, ModelReader).GetResult();
		},
		[](int32 BudgetReserve)
		{
			return UE::Mutable::Private::MakeManaged<UE::Mutable::Private::FMesh>();
		});
		
		ConstantMeshElement->IndexOnSourceVector = MeshIndex;

		// Actual core disk size would be:
		//UE::Mutable::Private::FOutputMemoryStream Stream;
		//UE::Mutable::Private::FOutputArchive Archive{ &Stream };
		//ConstantMeshElement->MeshPtr->Serialise(Archive);
		//uint64 OtherSize = Stream.GetBufferSize();

		const uint64 ThisMeshSize = ConstantMeshElement->MeshPtr->GetDataSize();
		ConstantMeshesAccumulatedSize += ThisMeshSize;
		
		ConstantMeshElements->Add(ConstantMeshElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantMeshesFormattedSize = GenerateTextForSize(ConstantMeshesAccumulatedSize);
}

void SMutableConstantsWidget::LoadConstantLayouts()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantLayouts.Num();
	ConstantLayoutElements.Empty(ConstantsCount);
	
	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 LayoutIndex = 0; LayoutIndex < ConstantsCount; LayoutIndex++)
	{		
		TSharedPtr<FMutableConstantLayoutElement> ConstantLayoutElement = MakeShared<FMutableConstantLayoutElement>();
		ConstantLayoutElement->Layout = MutableModel->GetProgram().ConstantLayouts[LayoutIndex];
		ConstantLayoutElement->IndexOnSourceVector = LayoutIndex;
		
		// Cache resource for later GetBufferSize() call
		ConstantLayoutElement->Layout->Serialise(Archive);

		ConstantLayoutElements.Add(ConstantLayoutElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantLayoutsFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}

void SMutableConstantsWidget::LoadConstantSkeletons()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantSkeletons.Num();
	ConstantSkeletonElements.Empty(ConstantsCount);

	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 SkeletonIndex = 0; SkeletonIndex < ConstantsCount; SkeletonIndex++)
	{
		TSharedPtr<FMutableConstantSkeletonElement> ConstantSkeletonElement = MakeShared<FMutableConstantSkeletonElement>();
		ConstantSkeletonElement->Skeleton = MutableModel->GetProgram().ConstantSkeletons[SkeletonIndex];
		ConstantSkeletonElement->IndexOnSourceVector = SkeletonIndex;
		
		ConstantSkeletonElement->Skeleton->Serialise(Archive);

		ConstantSkeletonElements.Add(ConstantSkeletonElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantSkeletonsFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}

void SMutableConstantsWidget::LoadConstantPhysics()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantPhysicsBodies.Num();
	ConstantPhysicsElements.Empty(ConstantsCount);

	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 PhysicsIndex = 0; PhysicsIndex < ConstantsCount; PhysicsIndex++)
	{
		TSharedPtr<FMutableConstantPhysicsElement> ConstantPhysicsElement = MakeShared<FMutableConstantPhysicsElement>();
		ConstantPhysicsElement->Physics = MutableModel->GetProgram().ConstantPhysicsBodies[PhysicsIndex];
		ConstantPhysicsElement->IndexOnSourceVector = PhysicsIndex;
	
		ConstantPhysicsElement->Physics->Serialise(Archive);

		ConstantPhysicsElements.Add(ConstantPhysicsElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantPhysicsFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}


void SMutableConstantsWidget::LoadConstantMaterials()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantMaterials.Num();
	
	if (ConstantMaterialElements)
	{
		ConstantMaterialElements->Empty(ConstantsCount);
	}
	else
	{
		ConstantMaterialElements = MakeShared<TArray<TSharedPtr<FMutableConstantMaterialElement>>>();
		ConstantMaterialElements->Reserve(ConstantsCount);
	}
	
	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 MaterialIndex = 0; MaterialIndex < ConstantsCount; MaterialIndex++)
	{
		TSharedPtr<FMutableConstantMaterialElement> ConstantMaterialElement = MakeShared<FMutableConstantMaterialElement>();
		ConstantMaterialElement->MutableMaterial = MutableModel->GetProgram().ConstantMaterials[MaterialIndex];
		ConstantMaterialElement->IndexOnSourceVector = MaterialIndex;
		ConstantMaterialElement->MutableMaterial->Serialise(Archive);

		ConstantMaterialElements->Add(ConstantMaterialElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantMaterialsFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}


void SMutableConstantsWidget::LoadConstantProjectors()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantProjectors.Num();
	ConstantProjectorElements.Empty(ConstantsCount);

	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 ProjectorIndex = 0; ProjectorIndex < ConstantsCount; ProjectorIndex++)
	{
		TSharedPtr<FMutableConstantProjectorElement> ConstantProjectorElement = MakeShared<FMutableConstantProjectorElement>();
		ConstantProjectorElement->Projector = &(MutableModel->GetProgram().ConstantProjectors[ProjectorIndex]);
		ConstantProjectorElement->IndexOnSourceVector = ProjectorIndex;
		
		ConstantProjectorElement->Projector->Serialise(Archive);
		
		ConstantProjectorElements.Add(ConstantProjectorElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantProjectorsFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}

void SMutableConstantsWidget::LoadConstantMatrices()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantMatrices.Num();
	ConstantMatrixElements.Empty(ConstantsCount);

	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 MatrixIndex = 0; MatrixIndex < ConstantsCount; MatrixIndex++)
	{
		TSharedPtr<FMutableConstantMatrixElement> ConstantMatrixElement = MakeShared<FMutableConstantMatrixElement>();
		ConstantMatrixElement->Matrix = MutableModel->GetProgram().ConstantMatrices[MatrixIndex];
		ConstantMatrixElement->IndexOnSourceVector = MatrixIndex;
		
		Archive << ConstantMatrixElement->Matrix;
		
		ConstantMatrixElements.Add(ConstantMatrixElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantMatricesFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}

void SMutableConstantsWidget::LoadConstantShapes()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantShapes.Num();
	ConstantShapeElements.Empty(ConstantsCount);

	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};	
	
	for (int32 ShapeIndex = 0; ShapeIndex < ConstantsCount; ShapeIndex++)
	{
		TSharedPtr<FMutableConstantShapeElement> ConstantShapeElement = MakeShared<FMutableConstantShapeElement>();
		ConstantShapeElement->Shape = &(MutableModel->GetProgram().ConstantShapes[ShapeIndex]);
		ConstantShapeElement->IndexOnSourceVector = ShapeIndex;
		
		ConstantShapeElement->Shape->Serialise(Archive);

		ConstantShapeElements.Add(ConstantShapeElement);
	}
	
	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantShapesFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}

void SMutableConstantsWidget::LoadConstantCurves()
{
	check(MutableModel);
	const int32 ConstantsCount = MutableModel->GetProgram().ConstantCurves.Num();
	ConstantCurveElements.Empty(ConstantsCount);
	
	UE::Mutable::Private::FOutputMemoryStream Stream;
	UE::Mutable::Private::FOutputArchive Archive{&Stream};
	
	for (int32 CurveIndex = 0; CurveIndex < ConstantsCount; CurveIndex++)
	{
		TSharedPtr<FMutableConstantCurveElement> ConstantCurveElement = MakeShared<FMutableConstantCurveElement>();
		ConstantCurveElement->Curve = MutableModel->GetProgram().ConstantCurves[CurveIndex];
		ConstantCurveElement->IndexOnSourceVector = CurveIndex;

		Archive << ConstantCurveElement->Curve;
		
		ConstantCurveElements.Add(ConstantCurveElement);
	}

	// Cache the size in memory of the constants as a formatted text so it is able to be later used by the UI
	ConstantCurvesFormattedSize = GenerateTextForSize(Stream.GetBufferSize());
}

#pragma  endregion


#pragma region Previewer invocation methods


void SMutableConstantsWidget::ClearSelectedConstantItems(UE::Mutable::Private::EDataType ExceptionDataType /* = UE::Mutable::Private::EDataType::None */) const
{
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Mesh)
	{
		MeshListViewHandler->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::String)
	{
		ConstantStringsSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Layout)
	{
		ConstantLayoutsSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Projector)
	{
		ConstantProjectorsSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Matrix)
	{
		ConstantMatricesSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Shape)
	{
		ConstantShapesSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Curve)
	{
		ConstantCurvesSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Skeleton)
	{
		ConstantSkeletonsSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::PhysicsAsset)
	{
		ConstantPhysicsSlate->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Image)
	{
		ImageListViewHandler->ClearSelection();
	}
	if (ExceptionDataType != UE::Mutable::Private::EDataType::Material)
	{
		MaterialistViewHandler->ClearSelection();
	}

	// todo: if you add more slates for the types update this to make sure they behave as the others
}

void SMutableConstantsWidget::OnSelectedStringChanged(
	TSharedPtr<FMutableConstantStringElement> MutableConstantStringElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantStringElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::String;

		// Clear the selected CodeViewer row and all other constant viewer slates that are not the type provided.
		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);
		
		// Ask the Code viewer to present the element held on your element on the previewer window
		MutableCodeViewerPtr->PreviewMutableString(MutableConstantStringElement->MutableString);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantStringElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedImageChanged(
	TSharedPtr<FMutableConstantImageElement> MutableConstantImageElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantImageElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Image;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);
		
		MutableCodeViewerPtr->PreviewMutableImage(MutableConstantImageElement->ImagePtr);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantImageElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedMeshChanged(
	TSharedPtr<FMutableConstantMeshElement> MutableConstantMeshElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantMeshElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Mesh;
		
		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableMesh(MutableConstantMeshElement->MeshPtr);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantMeshElement->IndexOnSourceVector);
	}
}


void SMutableConstantsWidget::OnSelectedMaterialChanged(TSharedPtr<FMutableConstantMaterialElement> MutableConstantMaterialElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantMaterialElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Material;
		
		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableMaterial(MutableConstantMaterialElement->MutableMaterial);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantMaterialElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedLayoutChanged(
	TSharedPtr<FMutableConstantLayoutElement> MutableConstantLayoutElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantLayoutElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Layout;
		
		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableLayout(MutableConstantLayoutElement->Layout);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantLayoutElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedProjectorChanged(
	TSharedPtr<FMutableConstantProjectorElement> MutableConstantProjectorElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantProjectorElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Projector;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableProjector(MutableConstantProjectorElement->Projector);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantProjectorElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedMatrixChanged(
	TSharedPtr<FMutableConstantMatrixElement> MutableConstantMatrixElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantMatrixElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Matrix;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableMatrix(MutableConstantMatrixElement->Matrix);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantMatrixElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedShapeChanged(
	TSharedPtr<FMutableConstantShapeElement> MutableConstantShapeElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantShapeElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Shape;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableShape(MutableConstantShapeElement->Shape);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantShapeElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedCurveChanged(
	TSharedPtr<FMutableConstantCurveElement> MutableConstantCurveElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantCurveElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Curve;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);
		
		MutableCodeViewerPtr->PreviewMutableCurve(MutableConstantCurveElement->Curve);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantCurveElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedSkeletonChanged(
	TSharedPtr<FMutableConstantSkeletonElement> MutableConstantSkeletonElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantSkeletonElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::Skeleton;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutableSkeleton(MutableConstantSkeletonElement->Skeleton);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantSkeletonElement->IndexOnSourceVector);
	}
}

void SMutableConstantsWidget::OnSelectedPhysicsChanged(
	TSharedPtr<FMutableConstantPhysicsElement> MutableConstantPhysicsElement,
	ESelectInfo::Type SelectionType) const
{
	if (MutableConstantPhysicsElement)
	{
		constexpr UE::Mutable::Private::EDataType SlateDataType = UE::Mutable::Private::EDataType::PhysicsAsset;

		MutableCodeViewerPtr->ClearSelectedTreeRow();
		ClearSelectedConstantItems(SlateDataType);

		MutableCodeViewerPtr->PreviewMutablePhysics(MutableConstantPhysicsElement->Physics);
		MutableCodeViewerPtr->CacheAddressesRelatedWithConstantResource(SlateDataType,MutableConstantPhysicsElement->IndexOnSourceVector);
	}
}

#pragma endregion 

#pragma region Region title drawing callback methods


FText SMutableConstantsWidget::OnDrawStringsAreaTitle() const
{
	return FText::Format( LOCTEXT("StringConstantsTitle", "String Constants ({0}) : {1} "),  ConstantStringElements.Num() , ConstantStringsFormattedSize);
}

FText SMutableConstantsWidget::OnDrawImagesAreaTitle() const
{
	return FText::Format( LOCTEXT("ImageConstantsTitle", "Image Constants ({0}) : {1} "), ConstantImageElements ? ConstantImageElements->Num() : 0, ConstantImagesFormattedSize);
}

FText SMutableConstantsWidget::OnDrawMeshesAreaTitle() const
{
	return FText::Format(LOCTEXT("MeshConstantsTitle", "Mesh Constants ({0}) : {1} "), ConstantMeshElements ? ConstantMeshElements->Num() : 0, ConstantMeshesFormattedSize) ;
}

FText SMutableConstantsWidget::OnDrawLayoutsAreaTitle() const
{
	return FText::Format(LOCTEXT("LayoutConstantsTitle", "Layout Constants ({0}) : {1} "), ConstantLayoutElements.Num(), ConstantLayoutsFormattedSize) ;
}

FText SMutableConstantsWidget::OnDrawProjectorsAreaTitle() const
{
	return  FText::Format( LOCTEXT("ProjectorConstantsTitle", "Projector Constants ({0}) : {1} "), ConstantProjectorElements.Num(),ConstantProjectorsFormattedSize);
}

FText SMutableConstantsWidget::OnDrawMatricesAreaTitle() const
{
	return FText::Format(LOCTEXT("MatrixConstantsTitle", "Matrix Constants ({0}) : {1} "), ConstantMatrixElements.Num(),ConstantMatricesFormattedSize) ;
}

FText SMutableConstantsWidget::OnDrawShapesAreaTitle() const
{
	return FText::Format(LOCTEXT("ShapeConstantsTitle", "Shape Constants ({0}) : {1} "),ConstantShapeElements.Num(), ConstantShapesFormattedSize) ;
}

FText SMutableConstantsWidget::OnDrawCurvesAreaTitle() const
{
	return FText::Format(LOCTEXT("CurveConstantsTitle", "Curve Constants ({0}) : {1} "), ConstantCurveElements.Num(), ConstantCurvesFormattedSize) ;
}

FText SMutableConstantsWidget::OnDrawSkeletonsAreaTitle() const
{
	return FText::Format( LOCTEXT("SkeletonConstantsTitle", "Skeleton Constants ({0}) : {1} "), ConstantSkeletonElements.Num(), ConstantSkeletonsFormattedSize);
}

FText SMutableConstantsWidget::OnDrawPhysicsAreaTitle() const
{
	return FText::Format( LOCTEXT("PhysicsConstantsTitle", "Physics Constants ({0}) : {1} "), ConstantPhysicsElements.Num(), ConstantPhysicsFormattedSize);
}

FText SMutableConstantsWidget::OnDrawMaterialsAreaTitle() const
{
	return FText::Format( LOCTEXT("MaterialConstantsTitle", "Material Constants ({0}) : {1} "), ConstantMaterialElements ? ConstantMaterialElements->Num() : 0, ConstantMaterialsFormattedSize);
}

#pragma endregion


#undef LOCTEXT_NAMESPACE
