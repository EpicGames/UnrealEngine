// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/Operations.h"
#include "MuR/Model.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STileView.h"


// Forward declarations
template <typename ItemType> class SMutableMultiPageListView;
class ITableRow;
class SExpandableArea;
class SMutableCodeViewer;
class STableViewBase;
class SWidget;
namespace UE::Mutable::Private { struct FProgram; }
namespace UE::Mutable::Private { struct FProjector; }
namespace UE::Mutable::Private { struct FShape; }

/**
 * Base Structure to define the different elements used by the lists on this object
 */
struct FMutableConstantElement
{
	/** The index of this element on the host vector. */
	uint32 IndexOnSourceVector;
};

/**
* Cache object used for the generation of the ui elements related to the constant images found on the model
*/
struct FMutableConstantImageElement : public FMutableConstantElement
{
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> ImagePtr;
};

/**
* Cache object used for the generation of the ui elements related to the constant meshes found on the model
*/
struct FMutableConstantMeshElement : public FMutableConstantElement
{
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> MeshPtr;
};

/**
* Cache object used for the generation of the ui elements related to the constant strings found on the model
*/
struct FMutableConstantStringElement : public FMutableConstantElement
{
	FString MutableString;
};

/**
* Cache object used for the generation of the ui elements related to the constant layouts found on the model
*/
struct FMutableConstantLayoutElement : public FMutableConstantElement
{
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout> Layout;
};

/**
* Cache object used for the generation of the ui elements related to the constant skeletons found on the model
*/
struct FMutableConstantSkeletonElement : public FMutableConstantElement
{
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> Skeleton;
};

/**
* Cache object used for the generation of the ui elements related to the constant skeletons found on the model
*/
struct FMutableConstantPhysicsElement : public FMutableConstantElement
{
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FPhysicsBody> Physics;
};

/**
* Cache object used for the generation of the ui elements related to the constant materials found on the model
*/
struct FMutableConstantMaterialElement : public FMutableConstantElement
{
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial> MutableMaterial;
};

/**
* Cache object used for the generation of the ui elements related to the constant projectors found on the model
*/
struct FMutableConstantProjectorElement : public FMutableConstantElement
{
	const UE::Mutable::Private::FProjector* Projector = nullptr;
};

/**
* Cache object used for the generation of the ui elements related to the constant matrices found on the model
*/
struct FMutableConstantMatrixElement : public FMutableConstantElement
{ 
	FMatrix44f Matrix;
};

/**
* Cache object used for the generation of the ui elements related to the constant shapes found on the model
*/
struct FMutableConstantShapeElement : public FMutableConstantElement
{
	const UE::Mutable::Private::FShape* Shape = nullptr;
};

/**
* Cache object used for the generation of the ui elements related to the constant curves found on the model
*/ 
struct FMutableConstantCurveElement : public FMutableConstantElement
{
	FRichCurve Curve;
};


/**
* Slate panel object designed to hold all the model constants
*/
class SMutableConstantsWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMutableConstantsWidget) {}
	SLATE_END_ARGS()

public:

	/** Builds the widget
	 * @param InArgs - Arguments provided when generating this slate object
	 * @param InMutableProgramPtr - Pointer to the UE::Mutable::Private::FProgram object that holds the constants data.
	 * @param  InMutableCodeViewerPtr - Pointer to the MutableCodeViewer tasked with the previewing of the constant values
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<UE::Mutable::Private::FModel> InModel,  TSharedPtr<SMutableCodeViewer> InMutableCodeViewerPtr);

	/** It clears the selected element for all constant view slates (SListView and STileView) except the one that shows the data of the provided type
	 * @param ExceptionDataType The type of data of the slate container that we want to skip when clearing the selected elements.
	 * EX : We want to clear all selected elements but not the ones from the constant images. Provide as argument "UE::Mutable::Private::EDataType::Image"
	 * to clear all but the selected item/s of the Image type.
	 */
	void ClearSelectedConstantItems(UE::Mutable::Private::EDataType ExceptionDataType = UE::Mutable::Private::EDataType::None) const;

private:

	/** Mutable object containing the constants data */
	TSharedPtr<UE::Mutable::Private::FModel> MutableModel;
	
	/** Slate capable of accessing the previewer object */
	TSharedPtr<SMutableCodeViewer> MutableCodeViewerPtr = nullptr;
	
	/**
	 *Sets the back end for the operation of this widget. Each time this is done the ui backend gets updated
	 * @param InProgram - Mutable program object holding all the constants data
	 */
	void SetModel(const TSharedPtr<UE::Mutable::Private::FModel>& InModel);

	/*
	 * Pointers to all slates showing the constants
	 */
	
	TSharedPtr<STileView<TSharedPtr<FMutableConstantMeshElement>>> ConstantMeshesSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantStringElement>>> ConstantStringsSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantLayoutElement>>> ConstantLayoutsSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantProjectorElement>>> ConstantProjectorsSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantMatrixElement>>> ConstantMatricesSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantShapeElement>>> ConstantShapesSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantCurveElement>>> ConstantCurvesSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantSkeletonElement>>> ConstantSkeletonsSlate;
	TSharedPtr<STileView<TSharedPtr<FMutableConstantPhysicsElement>>> ConstantPhysicsSlate;
	
	/*
	* Data backend for the lists of constants
	*/

	TSharedPtr<SMutableMultiPageListView<TSharedPtr<FMutableConstantImageElement>>> ImageListViewHandler;
	TSharedPtr<TArray<TSharedPtr<FMutableConstantImageElement>>> ConstantImageElements;

	TSharedPtr<SMutableMultiPageListView<TSharedPtr<FMutableConstantMeshElement>>> MeshListViewHandler;
	TSharedPtr<TArray<TSharedPtr<FMutableConstantMeshElement>>> ConstantMeshElements;
	
	TSharedPtr<SMutableMultiPageListView<TSharedPtr<FMutableConstantMaterialElement>>> MaterialistViewHandler;
	TSharedPtr<TArray<TSharedPtr<FMutableConstantMaterialElement>>> ConstantMaterialElements;
	
	TArray<TSharedPtr<FMutableConstantStringElement>> ConstantStringElements;
	TArray<TSharedPtr<FMutableConstantLayoutElement>> ConstantLayoutElements;
	TArray<TSharedPtr<FMutableConstantProjectorElement>> ConstantProjectorElements;
	TArray<TSharedPtr<FMutableConstantMatrixElement>> ConstantMatrixElements;
	TArray<TSharedPtr<FMutableConstantShapeElement>> ConstantShapeElements;
	TArray<TSharedPtr<FMutableConstantCurveElement>> ConstantCurveElements;
	TArray<TSharedPtr<FMutableConstantSkeletonElement>> ConstantSkeletonElements;
	TArray<TSharedPtr<FMutableConstantPhysicsElement>> ConstantPhysicsElements;

	/**
	 * Load up all the elements with the data found on the UE::Mutable::Private::FProgram object onto TArrays after parsing the data found.
	 */
	void LoadConstantElements();
	
	void LoadConstantStrings();
	void LoadConstantMeshes();
	void LoadConstantImages();
	void LoadConstantLayouts();
	void LoadConstantProjectors();
	void LoadConstantMatrices();
	void LoadConstantShapes();
	void LoadConstantCurves();
	void LoadConstantSkeletons();
	void LoadConstantPhysics();
	void LoadConstantMaterials();

	/*
	 * Proxy slates operation objects 
	 */
	
	void OnSelectedStringChanged(TSharedPtr<FMutableConstantStringElement> MutableConstantStringElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedImageChanged(TSharedPtr<FMutableConstantImageElement> MutableConstantImageElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedMeshChanged(TSharedPtr<FMutableConstantMeshElement> MutableConstantMeshElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedMaterialChanged(TSharedPtr<FMutableConstantMaterialElement> MutableConstantMaterialElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedLayoutChanged(TSharedPtr<FMutableConstantLayoutElement> MutableConstantLayoutElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedProjectorChanged(TSharedPtr<FMutableConstantProjectorElement> MutableConstantProjectorElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedMatrixChanged(TSharedPtr<FMutableConstantMatrixElement> MutableConstantMatrixElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedShapeChanged(TSharedPtr<FMutableConstantShapeElement> MutableConstantShapeElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedCurveChanged(TSharedPtr<FMutableConstantCurveElement> MutableConstantCurveElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedSkeletonChanged(TSharedPtr<FMutableConstantSkeletonElement> MutableConstantSkeletonElement, ESelectInfo::Type SelectionType) const;
	void OnSelectedPhysicsChanged(TSharedPtr<FMutableConstantPhysicsElement> MutableConstantPhysicsElement, ESelectInfo::Type SelectionType) const;
	
	/*
	* UI updating methods
	*/
	
	TSharedRef<ITableRow> OnGenerateStringRow(TSharedPtr<FMutableConstantStringElement> MutableConstantStringElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateImageRow(TSharedPtr<FMutableConstantImageElement> MutableConstantImageElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateMeshRow(TSharedPtr<FMutableConstantMeshElement> MutableConstantMeshElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateMaterialRow(TSharedPtr<FMutableConstantMaterialElement> MutableConstantMaterialElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateLayoutRow(TSharedPtr<FMutableConstantLayoutElement> MutableConstantLayoutElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateProjectorRow(TSharedPtr<FMutableConstantProjectorElement> MutableConstantProjectorElement, const TSharedRef<STableViewBase>& OwnerTable)const;
	TSharedRef<ITableRow> OnGenerateMatrixRow(TSharedPtr<FMutableConstantMatrixElement> MutableConstantMatrixElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateShapeRow(TSharedPtr<FMutableConstantShapeElement> MutableConstantShapeElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateCurveRow(TSharedPtr<FMutableConstantCurveElement> MutableConstantCurveElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGenerateSkeletonRow(TSharedPtr<FMutableConstantSkeletonElement> MutableConstantSkeletonElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	TSharedRef<ITableRow> OnGeneratePhysicsRow(TSharedPtr<FMutableConstantPhysicsElement> MutableConstantPhysicsElement, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/*
	 * Image List sorting methods
	 */
	
	/** Id of the last column the user decided to Sort. Usefully in order to interpolate ascending and descending sorting
	 * order 
	 */
	FName ImageConstantsLastSortedColumnID = "";
	/** Variable holding what kind of sorting has been used on the last sorting operation*/
	EColumnSortMode::Type ImageListSortMode = EColumnSortMode::Type::None;
	
	/** Callback method designed to sort the list of images. It sorts ConstantImageElements */
	void OnImageTableSortRequested(EColumnSortPriority::Type ColumnSortPriority, const FName& ColumnID, EColumnSortMode::Type ColumnSortMode);
	EColumnSortMode::Type GetImageListColumnSortMode(FName ColumnName) const;

	/*
	 * Mesh List sorting methods
	 */
	
	FName MeshConstantsLastSortedColumnID = "";
	EColumnSortMode::Type MeshListSortMode = EColumnSortMode::Type::None;
	
	void OnMeshTableSortRequested(EColumnSortPriority::Type ColumnSortPriority, const FName& ColumnID, EColumnSortMode::Type ColumnSortMode);
	EColumnSortMode::Type GetMeshListColumnSortMode(FName ColumnName) const;

	/**
	 * Materials List sorting methods
	 */
	
	FName MaterialConstantsLastSortedColumnID = "";
	EColumnSortMode::Type MaterialListSortMode = EColumnSortMode::Type::None;
	
	void OnMaterialsTableSortRequested(EColumnSortPriority::Type ColumnSortPriority, const FName& ColumnID, EColumnSortMode::Type ColumnSortMode);
	EColumnSortMode::Type GetMaterialListColumnSortMode(FName ColumnName) const;

	
	/*
	 * Expandable areas objects and behaviors
	 */

	// Pointers to all expandable areas part of this slate 
	TSharedPtr<SExpandableArea> StringsExpandableArea;
	TSharedPtr<SExpandableArea> ImagesExpandableArea;
	TSharedPtr<SExpandableArea> MeshesExpandableArea;
	TSharedPtr<SExpandableArea> LayoutsExpandableArea;
	TSharedPtr<SExpandableArea> ProjectorsExpandableArea;
	TSharedPtr<SExpandableArea> MatricesExpandableArea;
	TSharedPtr<SExpandableArea> ShapesExpandableArea;
	TSharedPtr<SExpandableArea> CurvesExpandableArea;
	TSharedPtr<SExpandableArea> SkeletonsExpandableArea;
	TSharedPtr<SExpandableArea> PhysicsExpandableArea;
	TSharedPtr<SExpandableArea> MaterialsExpandableArea;

	/** Array with all expandable areas set on this object. Used for dynamic expansion/contraction */
	TArray<TSharedPtr<SExpandableArea>> ExpandableAreas;

	/*
	 * Callback methods called each time one expandable area gets expanded or contracted
	 */
	
	void OnStringsRegionExpansionChanged(bool bExpanded);
	void OnImagesRegionExpansionChanged(bool bExpanded);
	void OnMeshesRegionExpansionChanged(bool bExpanded);
	void OnLayoutsRegionExpansionChanged(bool bExpanded);
	void OnProjectorsRegionExpansionChanged(bool bExpanded);
	void OnMatricesRegionExpansionChanged(bool bExpanded);
	void OnShapesRegionExpansionChanged(bool bExpanded);
	void OnCurvesRegionExpansionChanged(bool bExpanded);
	void OnSkeletonsRegionExpansionChanged(bool bExpanded);
	void OnPhysicsRegionExpansionChanged(bool bExpanded);
	void OnMaterialsRegionExpansionChanged(bool bExpanded);

	/** Method called each time one expandable area changes expansion state
	 * @param InException - Pointer to the expandable area that will not be contracted while all the others will
	 */
	void ContractExpandableAreas(const TSharedPtr<SExpandableArea>& InException);

	/*
	 * Methods used to get the size of each constant type collection on memory.  
	 */

	// Sizes decomposed on GB, MB, KB and B
	FText ConstantStringsFormattedSize;
	FText ConstantImagesFormattedSize;
	FText ConstantMeshesFormattedSize;
	FText ConstantLayoutsFormattedSize;
	FText ConstantProjectorsFormattedSize;
	FText ConstantMatricesFormattedSize;
	FText ConstantShapesFormattedSize;
	FText ConstantCurvesFormattedSize;
	FText ConstantSkeletonsFormattedSize;
	FText ConstantPhysicsFormattedSize;
	FText ConstantMaterialsFormattedSize;

	/*
	 * Callback methods used for drawing the titles of each of the constant expandable areas
	 */
	
	FText OnDrawStringsAreaTitle() const;
	FText OnDrawImagesAreaTitle() const;
	FText OnDrawMeshesAreaTitle() const;
	FText OnDrawLayoutsAreaTitle() const;
	FText OnDrawProjectorsAreaTitle() const;
	FText OnDrawMatricesAreaTitle() const;
	FText OnDrawShapesAreaTitle() const;
	FText OnDrawCurvesAreaTitle() const;
	FText OnDrawSkeletonsAreaTitle() const;
	FText OnDrawPhysicsAreaTitle() const;
	FText OnDrawMaterialsAreaTitle() const;

};



