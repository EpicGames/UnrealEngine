// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableConstrainedBox.h"


void SMutableConstrainedBox::Construct(const FArguments& InArgs)
{
	MinWidth = InArgs._MinWidth;
	MaxWidth = InArgs._MaxWidth;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

FVector2D SMutableConstrainedBox::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	const float MinWidthVal = MinWidth.Get().Get(0.0f);
	const float MaxWidthVal = MaxWidth.Get().Get(0.0f);

	if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f)
	{
		return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	}
	else
	{
		FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

		float XVal = FMath::Max(MinWidthVal, ChildSize.X);
		if (MaxWidthVal > MinWidthVal)
		{
			XVal = FMath::Min(MaxWidthVal, XVal);
		}

		return FVector2D(XVal, ChildSize.Y);
	}
}
