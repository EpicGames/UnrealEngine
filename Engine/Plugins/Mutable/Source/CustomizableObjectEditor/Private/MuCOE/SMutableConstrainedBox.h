// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SWidget;

/** Helper class to force a widget to fill in a space. Copied from SConstrainedBox.h */
class SMutableConstrainedBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableConstrainedBox)
		: _MinWidth()
		, _MaxWidth()
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
		SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;

private:
	TAttribute< TOptional<float> > MinWidth;
	TAttribute< TOptional<float> > MaxWidth;
};
