// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableCurveViewer.h"

#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "Framework/Views/TableViewMetadata.h"
#include "ICurveEditorModule.h"
#include "MuT/TypeInfo.h"
#include "SCurveEditorPanel.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "SMutableCurveViewer"

#pragma region FMutableCurveModel method definitions

void FMutablePreviewerCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace,
	TArray<TTuple<double, double>>& InterpolatingPoints) const
{
	const double StartTimeSeconds = ScreenSpace.GetInputMin();
	const double EndTimeSeconds = ScreenSpace.GetInputMax();
	const double TimeThreshold = FMath::Max(0.0001, 1.0 / ScreenSpace.PixelsPerInput());
	const double ValueThreshold = FMath::Max(0.0001, 1.0 / ScreenSpace.PixelsPerOutput());
		
	InterpolatingPoints.Add(MakeTuple(StartTimeSeconds, double(RichCurve.Eval(StartTimeSeconds))));

	const TArray<FRichCurveKey>& CurveKeys = RichCurve.GetConstRefOfKeys();
	for (const FRichCurveKey& Key : CurveKeys)
	{
		if (Key.Time > StartTimeSeconds && Key.Time < EndTimeSeconds)
		{
			InterpolatingPoints.Add(MakeTuple(double(Key.Time), double(Key.Value)));
		}
	}

	InterpolatingPoints.Add(MakeTuple(EndTimeSeconds, double(RichCurve.Eval(EndTimeSeconds))));


	// Generate the points between the key positions we have already set
	int32 OldSize = InterpolatingPoints.Num();
	do
	{
		OldSize = InterpolatingPoints.Num();
		RefineCurvePoints(RichCurve, TimeThreshold, ValueThreshold, InterpolatingPoints);
	} while (OldSize != InterpolatingPoints.Num());
		
}

void FMutablePreviewerCurveModel::RefineCurvePoints(const FRichCurve& InRichCurve, double TimeThreshold,
	float ValueThreshold, TArray<TTuple<double, double>>& InOutPoints) const
{
	const float InterpTimes[] = { 0.25f, 0.5f, 0.6f };

	for (int32 Index = 0; Index < InOutPoints.Num() - 1; ++Index)
	{
		TTuple<double, double> Lower = InOutPoints[Index];
		TTuple<double, double> Upper = InOutPoints[Index + 1];

		if ((Upper.Get<0>() - Lower.Get<0>()) >= TimeThreshold)
		{
			bool bSegmentIsLinear = true;

			TTuple<double, double> Evaluated[UE_ARRAY_COUNT(InterpTimes)] = { TTuple<double, double>(0, 0) };

			for (int32 InterpIndex = 0; InterpIndex < UE_ARRAY_COUNT(InterpTimes); ++InterpIndex)
			{
				double& EvalTime = Evaluated[InterpIndex].Get<0>();

				EvalTime = FMath::Lerp(Lower.Get<0>(), Upper.Get<0>(), InterpTimes[InterpIndex]);

				float Value = InRichCurve.Eval(EvalTime);

				const float LinearValue = FMath::Lerp(Lower.Get<1>(), Upper.Get<1>(), InterpTimes[InterpIndex]);
				if (bSegmentIsLinear)
				{
					bSegmentIsLinear = FMath::IsNearlyEqual(Value, LinearValue, ValueThreshold);
				}

				Evaluated[InterpIndex].Get<1>() = Value;
			}

			if (!bSegmentIsLinear)
			{
				// Add the point
				InOutPoints.Insert(Evaluated, UE_ARRAY_COUNT(Evaluated), Index + 1);
				--Index;
			}
		}
	}
}

void FMutablePreviewerCurveModel::AddKeys(TArrayView<const FKeyPosition> InPositions,
	TArrayView<const FKeyAttributes> InAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	check(InPositions.Num() == InAttributes.Num());
		
	// Iterate over the positions and store their data 
	for (int32 KeyIndex = 0; KeyIndex < InPositions.Num(); KeyIndex++)
	{
		// Add a new key on our rich curve and cache the handle
		const float KeyTime = InPositions[KeyIndex].InputValue;
		const float KeyValue = InPositions[KeyIndex].OutputValue;
		// RichCurve add key is the one that sets the index to whatever it wants!!
		KeyHandles.Add( RichCurve.AddKey(KeyTime,KeyValue));
			
		// Load attributes onto the key
		const FKeyAttributes KeyAttributes = InAttributes[KeyIndex];
		{
			// Set interpolation and weight modes
			RichCurve.SetKeyInterpMode(KeyHandles.Last(),KeyAttributes.GetInterpMode());
			RichCurve.SetKeyTangentMode(KeyHandles.Last(),KeyAttributes.GetTangentMode());
			RichCurve.SetKeyTangentWeightMode(KeyHandles.Last(),KeyAttributes.GetTangentWeightMode());

			// Set the tangent data
			FRichCurveKey& RichKey = RichCurve.GetKey(KeyHandles.Last());
			RichKey.ArriveTangentWeight = KeyAttributes.GetArriveTangentWeight();
			RichKey.LeaveTangentWeight = KeyAttributes.GetLeaveTangentWeight();
			RichKey.ArriveTangent = KeyAttributes.GetArriveTangent();
			RichKey.LeaveTangent = KeyAttributes.GetLeaveTangent();
		}
			
	}
}

void FMutablePreviewerCurveModel::GetKeyPositions(TArrayView<const FKeyHandle> InKeys,
	TArrayView<FKeyPosition> OutKeyPositions) const
{
	if (!InKeys.Num())
	{
		return;
	}

	TArray<FKeyPosition> LocatedPositions;
			
	// Retrieve the keys by looking for those same key handles on our array of key handles
	for (int32 InKeyIndex = 0; InKeyIndex < InKeys.Num(); InKeyIndex++)
	{
		// Index of the provided key on our array of keys
		const int32 ActualIndex = KeyHandles.IndexOfByKey(InKeys[InKeyIndex]);
			
		const FRichCurveKey Key =  RichCurve.GetKey(KeyHandles[ActualIndex] );
		FKeyPosition KeyPosition;
		{
			KeyPosition.InputValue = Key.Time;
			KeyPosition.OutputValue = Key.Value;
		}
		LocatedPositions.Add(KeyPosition);
	}

	OutKeyPositions = TArrayView<FKeyPosition>(LocatedPositions);
}

void FMutablePreviewerCurveModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	if (RichCurve.Keys.Num() > 1)
	{
		MinTime = RichCurve.Keys[0].Time;
		MaxTime = RichCurve.Keys[RichCurve.Keys.Num()-1].Time;
	}
	else
	{
		MinTime = 0;
		MaxTime = 1;
	}
}

void FMutablePreviewerCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	MinValue = 0;
	MaxValue = 1;
		
	// Get max and min values
	for (int32 RichCurveKeyIndex = 0; RichCurveKeyIndex < RichCurve.Keys.Num(); RichCurveKeyIndex++)
	{
		const double PositionValue = RichCurve.Keys[RichCurveKeyIndex].Value;

		if (PositionValue > MaxValue)
		{
			MaxValue = PositionValue;
		}

		if (PositionValue < MinValue)
		{
			MinValue = PositionValue;
		}
	}
}

void FMutablePreviewerCurveModel::GetNeighboringKeys(const FKeyHandle InKeyHandle,
	TOptional<FKeyHandle>& OutPreviousKeyHandle, TOptional<FKeyHandle>& OutNextKeyHandle) const
{
	const int32 IndexOfTargetKey = KeyHandles.IndexOfByKey(InKeyHandle);
	check(IndexOfTargetKey >= 0);

	if (IndexOfTargetKey > 0)
	{
		OutPreviousKeyHandle = KeyHandles[IndexOfTargetKey - 1];
	}

	if (IndexOfTargetKey < KeyHandles.Num() - 1)
	{
		OutNextKeyHandle = KeyHandles[IndexOfTargetKey + 1];
	}
}

#pragma endregion 


namespace MutableCurveKeyFramesListColumns
{
	static const FName KeyFrameIdColumnID("Keyframe");
	static const FName KeyFrameTimeColumnID ("Time");
	static const FName KeyFrameValueColumnID ("Value");
	
	static const FName KeyFrameInTangentColumnID ("In Tangent");
	static const FName KeyFrameInTangentWeightColumnID ("In Tangent Weight");
	
	static const FName KeyFrameOutTangentColumnID ("Out Tangent");
	static const FName KeyFrameOutTangentWeightColumnID ("Out Tangent Weight");

	static const FName KeyFrameInterpolationModeColumnID ("Interpolation Mode");
	static const FName KeyFrameTangentModeColumnID ("Tangent Mode");
	static const FName KeyFrameTangentWeightModeColumnID ("Weight Mode");
	
}

class SMutableCurveKeyFrameTableRow : public SMultiColumnTableRow<TSharedPtr<FMutableCurveElement>>
{
public:

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableCurveElement>& InRowItem)
	{
		RowItem = InRowItem;
		
		SMultiColumnTableRow< TSharedPtr<FMutableCurveElement> >::Construct(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);

	}

	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& InColumnName ) override
	{
		// INDEX
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameIdColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::FromInt(RowItem->KeyFrameIndex)))
			];
		}

		// time
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameTimeColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::SanitizeFloat(RowItem->CurveKeyFrame.Time) ))
			];
		}
		
		// value
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameValueColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::SanitizeFloat(RowItem->CurveKeyFrame.Value) ))
			];
		}
		
		// In tangent
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameInTangentColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::SanitizeFloat(RowItem->CurveKeyFrame.ArriveTangent) ))
			];
		}

		// In tangent weight
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameInTangentWeightColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::SanitizeFloat(RowItem->CurveKeyFrame.ArriveTangentWeight) ))
			];
		}

		// out tangent
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameOutTangentColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::SanitizeFloat(RowItem->CurveKeyFrame.LeaveTangent) ))
			];
		}

		// Out tangent weight
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameOutTangentWeightColumnID)
		{
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString( FString::SanitizeFloat(RowItem->CurveKeyFrame.LeaveTangentWeight) ))
			];
		}

		// interp_mode 
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameInterpolationModeColumnID)
		{
			const uint8 InterpolationMode = RowItem->CurveKeyFrame.InterpMode;
			
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromString(* FString::Printf(TEXT("%d"),InterpolationMode)))
			];
		}

		// tangent mode 
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameTangentModeColumnID)
		{
			const uint8 TangentMode = RowItem->CurveKeyFrame.TangentMode;
			
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
					Text(FText::FromString(*FString::Printf(TEXT("%d"), TangentMode)))
			];
		}

		// tangent weight mode 
		if (InColumnName == MutableCurveKeyFramesListColumns::KeyFrameTangentWeightModeColumnID)
		{
			const uint8 TangentWeightMode = RowItem->CurveKeyFrame.TangentWeightMode;
			
			return SNew(SHorizontalBox)+SHorizontalBox::Slot()
			[
				SNew(STextBlock).
					Text(FText::FromString(*FString::Printf(TEXT("%d"), TangentWeightMode)))
			];
		}
		
		// Invalid column name so no widget will be produced 
		checkNoEntry();
		return SNullWidget::NullWidget;
	}
	
private:

	TSharedPtr<FMutableCurveElement> RowItem;
};


void SMutableCurveViewer::Construct(const FArguments& InArgs)
{
	// Create the Curve Editor
	{
		this->CurveEditor = MakeShared<FCurveEditor>();

		CurveEditor = MakeShared<FCurveEditor>();
		FCurveEditorInitParams InitParams;
		CurveEditor->InitCurveEditor(InitParams);
		CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}");
	}
	
	// Formatting
	constexpr float VerticalPadding = 30.0f;
	
	ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.FillHeight(0.6f)
		[
			// Curve display
			SAssignNew(CurveEditorPanel, SCurveEditorPanel, CurveEditor.ToSharedRef())
		]

		+ SVerticalBox::Slot()
		.Padding(0,VerticalPadding)
		.AutoHeight()
		[
			// Curve table with all the mutable key data
			SAssignNew(CurveListView, SListView<TSharedPtr<FMutableCurveElement>>)
			.ListItemsSource(&CurveElements)
			.OnGenerateRow(this, &SMutableCurveViewer::OnGenerateCurveTableRow)
			.SelectionMode(ESelectionMode::None)
			.HeaderRow
			(
				SNew(SHeaderRow)
			
				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameIdColumnID)
				.DefaultLabel(LOCTEXT("KeyframeID", "ID"))
				.FillWidth(0.22)
				
				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameTimeColumnID)
				.DefaultLabel(LOCTEXT("KeyframeTime", "Time"))

				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameValueColumnID)
				.DefaultLabel(LOCTEXT("KeyframeValue", "Value"))
				
				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameInTangentColumnID)
				.DefaultLabel(LOCTEXT("InTangent", "In-Tangent"))

				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameInTangentWeightColumnID)
				.DefaultLabel(LOCTEXT("InTangentWeight", "In-Tangent Weight"))

				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameOutTangentColumnID)
				.DefaultLabel(LOCTEXT("OutTangent", "Out-Tangent"))

				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameOutTangentWeightColumnID)
				.DefaultLabel(LOCTEXT("OutTangentWeight", "Out-Tangent Weight"))

				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameInterpolationModeColumnID)
				.DefaultLabel(LOCTEXT("InterpolationMode", "Interpolation Mode"))

				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameTangentModeColumnID)
				.DefaultLabel(LOCTEXT("TangentWeight", "Tangent Mode"))
				
				+ SHeaderRow::Column(MutableCurveKeyFramesListColumns::KeyFrameTangentWeightModeColumnID)
				.DefaultLabel(LOCTEXT("TangentWeightMode", "Weight Mode"))

			)
		]
		
	];
	
	SetCurve(InArgs._MutableCurve);
}


void SMutableCurveViewer::SetCurve(const FRichCurve& InMutableCurve)
{
	this->MutableCurve = InMutableCurve;

	// Load the Curve Editor with the data found on the UE::Mutable::Private::Curve
	SetupMutableCurveGraph();
	
	// Load the data required by the SListView to display the data on the UE::Mutable::Private::Curve
	SetupMutableCurveListView();
}


void SMutableCurveViewer::SetupMutableCurveListView()
{
	const int32 KeyFrameCount = MutableCurve.Keys.Num() ;
	CurveElements.SetNum(KeyFrameCount);
	for	(int32 KeyFrameIndex = 0; KeyFrameIndex < KeyFrameCount; KeyFrameIndex++)
	{
		const FRichCurveKey& CurrentKeyFrame = MutableCurve.Keys[KeyFrameIndex];

		const TSharedPtr<FMutableCurveElement> NewCurveElement =
			MakeShareable(new FMutableCurveElement(KeyFrameIndex,CurrentKeyFrame));

		CurveElements[KeyFrameIndex] = NewCurveElement;
	}

	CurveListView->RequestListRefresh();
}


TSharedRef<ITableRow> SMutableCurveViewer::OnGenerateCurveTableRow(TSharedPtr<FMutableCurveElement> InElement,
	const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableCurveKeyFrameTableRow> Row = SNew(SMutableCurveKeyFrameTableRow, OwnerTable, InElement);
	return Row;
}


void SMutableCurveViewer::SetupMutableCurveGraph() const
{
	// Clear all possible curves set on previous iterations
	if (CurveEditor->GetCurves().Num())
	{
		CurveEditor->RemoveAllCurves();
	}
	
	// Add a default curve
	TUniquePtr<FMutablePreviewerCurveModel> CurveEditorModule = MakeUnique<FMutablePreviewerCurveModel>();
	CurveEditorModule->SetColor(FLinearColor(FColor::Yellow));

	// Cache the last key time to be able to compare it with the one being currently processed
	float PreviousFrameTime =  TNumericLimits<float>::Lowest();
	
	// Fill the curve with data
	const int32 KeyFrameCount = MutableCurve.Keys.Num() ;
	for (int32 KeyframeIndex = 0; KeyframeIndex < KeyFrameCount; KeyframeIndex++)
	{
		// Load the mutable data
		const FRichCurveKey& CurrentKeyframe = MutableCurve.Keys[KeyframeIndex];
		
		// Setup basic data (x and y axis)
		FKeyPosition KeyPosition;
		{
			// time and value
			KeyPosition.InputValue = CurrentKeyframe.Time;
			KeyPosition.OutputValue = CurrentKeyframe.Value;
		}
		
		/*
		 * Important : The time value for each key MUST be different. nearly equal values can produce ill-formed graphs
		 * depending on the order of the keys.
		 *
		 * Check out FRichCurve::AddKey for more information (for at the beginning of the method setting the index).
		*/
		{
			// Base amount of time delta to be applied to correct the time as index behaviour of FRichCurve
			// @note TNumericLimits<float>::Min() is too small;
			constexpr float ApplicableTimeDelta = 0.00000005f;

			// Shift the position of the current element if we are too close to the last processed element or if
			// the previous element for whatever reason is now in front of us (previousTime > currentTime>.
			if (FMath::IsNearlyEqual(PreviousFrameTime,CurrentKeyframe.Time,ApplicableTimeDelta)
				|| PreviousFrameTime > CurrentKeyframe.Time)
			{
				KeyPosition.InputValue += ApplicableTimeDelta;
			}
		}

		// Update previousFrameTime to represent the updated value of the current keyframe
		PreviousFrameTime = KeyPosition.InputValue;

		// Setup the curve behaviour for this key of the curve.
		FKeyAttributes KeyAttributes;
		{
			// tangents
			KeyAttributes.SetArriveTangent(CurrentKeyframe.ArriveTangent);
			KeyAttributes.SetLeaveTangent(CurrentKeyframe.LeaveTangent);

			// Weights
			KeyAttributes.SetArriveTangentWeight(CurrentKeyframe.ArriveTangentWeight);
			KeyAttributes.SetLeaveTangentWeight(CurrentKeyframe.LeaveTangentWeight);
			
			// Interp mode
			const TEnumAsByte UnrealInterpolationMode = static_cast<ERichCurveInterpMode>(CurrentKeyframe.InterpMode);
			KeyAttributes.SetInterpMode(UnrealInterpolationMode);
			
			// Tangent mode
			const TEnumAsByte UnrealTangentMode =
				static_cast<ERichCurveTangentMode>(CurrentKeyframe.TangentMode);
			KeyAttributes.SetTangentMode(UnrealTangentMode);

			// Tangent weight
			const TEnumAsByte UnrealTangentWeightMode =
				static_cast<ERichCurveTangentWeightMode>(CurrentKeyframe.TangentWeightMode);
			KeyAttributes.SetTangentWeightMode(UnrealTangentWeightMode);
		}

		CurveEditorModule->AddKey(KeyPosition,KeyAttributes);
	}

	// Add the curve to the editor
	const FCurveModelID CurveId = CurveEditor->AddCurve(MoveTemp(CurveEditorModule));
	CurveEditor->PinCurve(CurveId);
}

#undef LOCTEXT_NAMESPACE
