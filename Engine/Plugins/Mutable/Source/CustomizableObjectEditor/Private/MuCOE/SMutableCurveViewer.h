// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveModel.h"
#include "Curves/RichCurve.h"
#include "MuR/Parameters.h"
#include "Widgets/SCompoundWidget.h"

enum class ECurvePointType : uint8;
template <typename ItemType> class SListView;

class FCurveEditor;
class ITableRow;
class SCurveEditorPanel;
class STableViewBase;
struct FCurveEditorScreenSpace;
struct FKeyAttributes;
struct FKeyDrawInfo;
struct FKeyPosition;

/** Custom curve model designed to be able to work with our mutable curve. The overriding of the FCurveModel
 * methods is mandatory.
 * @note Methods with // are methods whose operation is not required for our usage case at the moment. Refer to
 * FCurveModel class to know more about them and the rest of methods that have been overriden.
 */
class FMutablePreviewerCurveModel : public FCurveModel 
{
private:
	/** Cache the handles being added onto curve. Allows us to access them externally to the RichCurve */
	TArray< FKeyHandle> KeyHandles;

	/** The curve object that will manage all the keys (time and value)  */
	FRichCurve RichCurve;
	
public:
	
	virtual const void* GetCurve() const override
	{
		return &RichCurve;
	}
	
	virtual void Modify() override {}
	
	// Based on DrawCurve from RichCurveEditorModel.cpp
	virtual void DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace,
		TArray<TTuple<double, double>>& InterpolatingPoints) const override;

	// Based on RefineCurvePoints from RichCurveEditorModel.cpp
	void RefineCurvePoints(const FRichCurve& InRichCurve, double TimeThreshold, float ValueThreshold, TArray<TTuple<double, double>>& InOutPoints) const;

	virtual void GetKeys(double MinTime, double MaxTime, double MinValue,
	                     double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const override {}
	
	virtual void AddKeys(TArrayView<const FKeyPosition> InPositions, TArrayView<const FKeyAttributes> InAttributes,
		TArrayView<TOptional<FKeyHandle>>* OutKeyHandles) override;

	virtual void RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime) override {}

	virtual void GetKeyPositions(TArrayView<const FKeyHandle> InKeys,
		TArrayView<FKeyPosition> OutKeyPositions) const override;

	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions,
	                             EPropertyChangeType::Type ChangeType) override {}
	
	virtual void GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle,
		FKeyDrawInfo& OutDrawInfo) const override {}
	
	virtual void GetTimeRange(double& MinTime, double& MaxTime) const override;

	virtual void GetValueRange(double& MinValue, double& MaxValue) const override;

	virtual int32 GetNumKeys() const override
	{
		return RichCurve.Keys.Num();
	}
	
	virtual void GetNeighboringKeys(const FKeyHandle InKeyHandle, TOptional<FKeyHandle>& OutPreviousKeyHandle,
		TOptional<FKeyHandle>& OutNextKeyHandle) const override;

	virtual bool Evaluate(double InTime, double& OutValue) const override
	{
		return true;
	}

	virtual void GetKeyAttributes(TArrayView<const FKeyHandle> InKeys,
		TArrayView<FKeyAttributes> OutAttributes) const override {}
	
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes,
		EPropertyChangeType::Type ChangeType) override {}
	
};



/** Defines an element representing a keyframe of the mutable curve. Used by a ListView */
struct FMutableCurveElement
{
	FMutableCurveElement(const int32 InIndex,const  FRichCurveKey& InCurveKeyFrame)
		: KeyFrameIndex(InIndex),
		CurveKeyFrame(InCurveKeyFrame)
	{
		
	}
	
	const int32 KeyFrameIndex;
	const FRichCurveKey& CurveKeyFrame;
};


/**
 * Object designed to be used as a way to inspect the data present on a Mutable Curve (UE::Mutable::Private::Curve)
 */
class SMutableCurveViewer final : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMutableCurveViewer) {}
		SLATE_ARGUMENT(FRichCurve, MutableCurve) ;
	SLATE_END_ARGS()
	
public:
	/**
	 * Builds this slate with the assistance of the provided arguments
	 * @param InArgs - Input arguments used to provide this slate with the data required to operate
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Sets the curve object to be used as source for the data to be displayed in this slate
	 * @param InMutableCurve - Reference to a mutable curve object to be set as origin for the data displayed
	 * on this slate object
	 */
	void SetCurve(const FRichCurve& InMutableCurve);
	
private:
	
	/** Mutable curve to read the data from. It is the origin for all the data displayed on this slate object */
	FRichCurve MutableCurve;

	/*
	 * Curve graph view
	 */

	/** Object designed to contain the curve to be displayed onto the SCurveEditorPanel. It serves as proxy to the curve being
	 * worked on and allows interaction with it.
	 */
	TSharedPtr<FCurveEditor> CurveEditor = nullptr;

	/** Slate object with the visual representation of the curve held by FCurveEditor. */
	TSharedPtr<SCurveEditorPanel> CurveEditorPanel = nullptr;

	/** Loads the data found on the mutable curve onto our FMutablePreviewerCurveModel to be displayed */
	void SetupMutableCurveGraph() const;
	
	/*
	 *  Curve List View 
	 */
	
	/** List View showing all the data found on the mutable curve */
	TSharedPtr<SListView<TSharedPtr<FMutableCurveElement>>> CurveListView;

	/** Array with all the curve keyframes as Curve elements for the Table to be able to display them */
	TArray<TSharedPtr<FMutableCurveElement>> CurveElements;
	
	/** Store all the keyframe data found on the mutable curve onto the array of Curve Elements so the list view
	 * can properly display it.
	 */
	void SetupMutableCurveListView();
	
	/**
	 * Callback method invoked for each element on CurveElements. It generates for each of them a new slate object to
	 * represent each MutableCurveElement
	 * @param InElement - Element to process
	 * @param OwnerTable - The table object where it will be loaded onto (not explicitly in this method)
	 * @return The slate object representing InElement.
	 */
	TSharedRef<ITableRow> OnGenerateCurveTableRow(TSharedPtr<FMutableCurveElement> InElement, const TSharedRef<STableViewBase>& OwnerTable) const;
};




