// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableGraphViewer.h"

#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDesktopPlatform.h"
#include "MuCOE/CustomizableObjectCompileRunnable.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/SMutableCodeViewer.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SNumericDropDown.h"
#include "Widgets/Views/STreeView.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeColorParameter.h"
#include "MuT/NodeColorSampleImage.h"
#include "MuT/NodeColorSwitch.h"
#include "MuT/NodeColorTable.h"
#include "MuT/NodeModifierSkeletalMeshMerge.h"
#include "MuT/NodeComponentSwitch.h"
#include "MuT/NodeImageConvert.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeImageInterpolate.h"
#include "MuT/NodeImageInvert.h"
#include "MuT/NodeImageLayer.h"
#include "MuT/NodeImageLayerColor.h"
#include "MuT/NodeImageMipmap.h"
#include "MuT/NodeImageMultiLayer.h"
#include "MuT/NodeImagePlainColor.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeImageResize.h"
#include "MuT/NodeImageSwitch.h"
#include "MuT/NodeImageSwizzle.h"
#include "MuT/NodeImageTable.h"
#include "MuT/NodeObjectGroup.h"
#include "MuT/NodeObjectNew.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeSurfaceSwitch.h"
#include "MuT/NodeSurfaceVariation.h"
#include "MuT/NodeLOD.h"
#include "MuT/NodeMaterialModify.h"
#include "MuT/NodeMaterialSkeletalMeshBreak.h"
#include "MuT/NodeMatrixConstant.h"
#include "MuT/NodeMatrixParameter.h"
#include "MuT/NodeMeshClipWithMesh.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeMeshFormat.h"
#include "MuT/NodeMeshFragment.h"
#include "MuT/NodeMeshMakeMorph.h"
#include "MuT/NodeMeshMorph.h"
#include "MuT/NodeMeshRemoveMesh.h"
#include "MuT/NodeMeshReshape.h"
#include "MuT/NodeMeshSkeletalMeshObjectBreak.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeMeshTransform.h"
#include "MuT/NodeMeshTransformInMesh.h"
#include "MuT/NodeMeshTransformWithBone.h"
#include "MuT/NodeSurfaceModifierMeshClipDeform.h"
#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"
#include "MuT/NodeSurfaceModifierMeshClipWithUVMask.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/NodeScalarCurve.h"
#include "MuT/NodeScalarSwitch.h"
#include "MuT/NodeScalarTable.h"
#include "MuT/NodeSurfaceModifierMeshTransformInMesh.h"
#include "MuT/NodeSurfaceModifierMeshTransformWithBone.h"
#include "MuT/NodeScalarArithmeticOperation.h"
#include "MuT/NodeSkeletalMeshConvert.h"
#include "MuT/NodeSkeletalMeshMerge.h"
#include "MuT/NodeSkeletalMeshModify.h"
#include "MuT/NodeSkeletalMeshMorph.h"
#include "MuT/NodeSkeletalMeshNew.h"
#include "MuT/NodeSkeletalMeshObjectConvert.h"
#include "MuT/NodeSkeletalMeshObjectParameter.h"
#include "MuT/NodeSkeletalMeshTransform.h"
#include "MuT/NodeSurfaceModifierSurfaceEdit.h"
#include "Widgets/MutableExpanderArrow.h"

class FExtender;
class FReferenceCollector;
class FUICommandList;
class ITableRow;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SMutableDebugger"

// \todo: multi-column tree
namespace MutableGraphTreeViewColumns
{
	static const FName Name("Name");
};


static const char* MutableNodeNames[] =
{
	"None",

	"Node",

	"Mesh",
	"MeshConstant",
	"MeshTable",
	"MeshFormat",
	"MeshTangents",
	"MeshMorph",
	"MeshMakeMorph",
	"MeshSwitch",
	"MeshFragment",
	"MeshTransform",
	"MeshClipMorphPlane",
	"MeshClipWithMesh",
	"MeshRemoveMesh",
	"MeshTransformWithBone",
	"MeshTransformInMesh",
	"MeshApplyPose",
	"MeshVariation",
	"MeshReshape",
	"MeshClipDeform",
	"MeshSkeletalMeshObjectBreak",
	"MeshExternal",

	"Image",
	"ImageConstant",
	"ImageInterpolate",
	"ImageSaturate",
	"ImageTable",
	"ImageSwizzle",
	"ImageColorMap",
	"ImageGradient",
	"ImageBinarise",
	"ImageLuminance",
	"ImageLayer",
	"ImageLayerColor",
	"ImageResize",
	"ImagePlainColor",
	"ImageProject",
	"ImageMipmap",
	"ImageSwitch",
	"ImageConditional",
	"ImageFormat",
	"ImageConform",
	"ImageMultiLayer",
	"ImageInvert",
	"ImageVariation",
	"ImageNormalComposite",
	"ImageTransform",
	"ImageFromMaterialParameter",
	"ImageExternal",
	"ImageConvert",

	"ImageObject",
	"ImageObjectParameter",
	
	"Bool",
	"BoolConstant",
	"BoolParameter",
	"BoolNot",
	"BoolAnd",

	"Color",
	"ColorConstant",
	"ColorParameter",
	"ColorSampleImage",
	"ColorTable",
	"ColorImageSize",
	"ColorFromScalars",
	"ColorArithmeticOperation",
	"ColorSwitch",
	"ColorVariation",
	"ColorToSRGB",
	"ColorExternal",

	"Scalar",
	"ScalarConstant",
	"ScalarParameter",
	"ScalarEnumParameter",
	"ScalarCurve",
	"ScalarSwitch",
	"ScalarArithmeticOperation",
	"ScalarVariation",
	"ScalarTable",
	"ScalarExternal",

	"String",
	"StringConstant",
	"StringParameter",

	"Projector",
	"ProjectorConstant",
	"ProjectorParameter",

	"Range",
	"RangeFromScalar",

	"Layout",

	"PatchImage",
	"PatchMesh",

	"Surface",
	"SurfaceNew",
	"SurfaceSwitch",
	"SurfaceVariation",

	"LOD",

	"SkeletalMesh",
	"SkeletalMeshNew",
	"SkeletalMeshMerge",
	"SkeletalMeshConvert",
	"SkeletalMeshModify",
	"SkeletalMeshMorph",
	"SkeletalMeshReshape",
	"SkeletalMeshSwitch",
	"SkeletalMeshVariation",
	"SkeletalMeshTransform",
	"SkeletalMeshTransformWithBone",
	"SkeletalMeshClipWithSkeletalMesh",

	"SkeletalMeshObject",
	"SkeletalMeshObjectConvert",
	"SkeletalMeshObjectParameter",
	"SkeletalMeshObjectSwitch",

	"Component",
	"ComponentNew",
	"ComponentSwitch",
	"ComponentVariation",

	"Object",
	"ObjectNew",
	"ObjectGroup",

	"Modifier",
	"ModifierSkeletalMeshMerge",

	"SurfaceModifier",
	"SurfaceModifierMeshClipMorphPlane",
	"SurfaceModifierMeshClipWithMesh",
	"SurfaceModifierMeshClipDeform",
	"SurfaceModifierMeshClipWithUVMask",
	"SurfaceModifierSurfaceEdit",
	"SurfaceModifierTransformInMesh",
	"SurfaceModifierTransformWithBone",

	"External",
	"ExternalOperation",
	"ExternalParameter",
	"ExternalSwitch",
	
	"ExtensionData",
	"ExtensionDataConstant",
	"ExtensionDataSwitch",
	"ExtensionDataVariation",

	"Matrix",
	"MatrixConstant",
	"MatrixParameter",

	"Material",
	"MaterialConstant",
	"MaterialTable",
	"MaterialSwitch",
	"MaterialVariation",
	"MaterialParameter",
	"MaterialExternal",
	"MaterialSkeletalMeshObjectBreak",
	"MaterialSkeletalMeshBreak",
	"MaterialModify",

	"ImageMaterialBreak",
	"ScalarMaterialBreak",
	"ColorMaterialBreak",
};

static_assert(UE_ARRAY_COUNT(MutableNodeNames) == SIZE_T(UE::Mutable::Private::Node::EType::Count));


class SMutableGraphTreeRow : public STableRow<TSharedPtr<FMutableGraphTreeElement>>
{
public:

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMutableGraphTreeElement>& InRowItem)
	{
		RowItem = InRowItem;

		FText MainLabel = FText::GetEmpty();
		if (RowItem->MutableNode)
		{
			UE::Mutable::Private::Node::EType MutableType = RowItem->MutableNode->GetType()->Type;
			
			FString NodeName = MutableNodeNames[int32(MutableType)];

			const FString LabelString = RowItem->Prefix.IsEmpty() 
				? FString::Printf( TEXT("%s"), *NodeName)
				: FString::Printf( TEXT("%s : %s"), *RowItem->Prefix, *NodeName);

			MainLabel = FText::FromString(LabelString);
			if (RowItem->DuplicatedOf)
			{
				MainLabel = FText::FromString( FString::Printf(TEXT("%s (Duplicated)"), *NodeName));
			}
		}
		else
		{
			MainLabel = FText::FromString( *RowItem->Prefix);
		}


		this->ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SMutableExpanderArrow, SharedThis(this))
			]

			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(MainLabel)
			]
		];

		STableRow< TSharedPtr<FMutableGraphTreeElement> >::ConstructInternal(
			STableRow::FArguments()
			//.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(true)
			, InOwnerTableView
		);

	}


private:

	TSharedPtr<FMutableGraphTreeElement> RowItem;
};


void SMutableGraphViewer::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Add UObjects here if we own any at some point
	//Collector.AddReferencedObject(CustomizableObject);
}


FString SMutableGraphViewer::GetReferencerName() const
{
	return TEXT("SMutableGraphViewer");
}


void SMutableGraphViewer::Construct(const FArguments& InArgs, const UE::Mutable::Private::NodePtr& InRootNode)
{
	ReferencedRuntimeTextures = InArgs._ReferencedRuntimeTextures;
	ReferencedCompileTextures = InArgs._ReferencedCompileTextures;
	RootNode = InRootNode;
	  
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(EOrientation::Orient_Horizontal)
		+ SSplitter::Slot()
		.Value(0.25f)
		[
			SNew(SBorder)
			.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
			.Padding(FMargin(4.0f, 4.0f))
			[
				SAssignNew(TreeView, STreeView<TSharedPtr<FMutableGraphTreeElement>>)
				.TreeItemsSource(&RootNodes)
				.OnGenerateRow(this,&SMutableGraphViewer::GenerateRowForNodeTree)
				.OnGetChildren(this, &SMutableGraphViewer::GetChildrenForInfo)
				.OnSetExpansionRecursive(this, &SMutableGraphViewer::TreeExpandRecursive)
				.OnContextMenuOpening(this, &SMutableGraphViewer::OnTreeContextMenuOpening)
				.SelectionMode(ESelectionMode::Single)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(MutableGraphTreeViewColumns::Name)
					.FillWidth(25.f)
					.DefaultLabel(LOCTEXT("Node Name", "Node Name"))
				)
			]
		]
		+ SSplitter::Slot()
		.Value(0.75f)
		[
			SNew(SBorder)
			.BorderImage(UE_MUTABLE_GET_BRUSH("ToolPanel.GroupBorder"))
			.Padding(FMargin(4.0f, 4.0f))
		]
	];
	
	RebuildTree();
}


void SMutableGraphViewer::RebuildTree()
{
	RootNodes.Reset();
	ItemCache.Reset();
	MainItemPerNode.Reset();

	RootNodes.Add(MakeShareable(new FMutableGraphTreeElement(RootNode)));
	TreeView->RequestTreeRefresh();
	TreeExpandUnique();
}


TSharedRef<ITableRow> SMutableGraphViewer::GenerateRowForNodeTree(TSharedPtr<FMutableGraphTreeElement> InTreeNode, const TSharedRef<STableViewBase>& InOwnerTable)
{
	TSharedRef<SMutableGraphTreeRow> Row = SNew(SMutableGraphTreeRow, InOwnerTable, InTreeNode);
	return Row;
}

void SMutableGraphViewer::GetChildrenForInfo(TSharedPtr<FMutableGraphTreeElement> InInfo, TArray<TSharedPtr<FMutableGraphTreeElement>>& OutChildren)
{
// This is necessary because of problems with rtti information in other platforms. In any case, this part of the debugger is only useful in the standard editor.
#if PLATFORM_WINDOWS
	if (!InInfo->MutableNode)
	{
		return;
	}

	// If this is a duplicated of another row, don't provide its children.
	if (InInfo->DuplicatedOf)
	{
		return;
	}

	UE::Mutable::Private::Node* ParentNode = InInfo->MutableNode.get();
	uint32 InputIndex = 0;

	auto AddChildFunc = [this, ParentNode, &InputIndex, &OutChildren](UE::Mutable::Private::Node* ChildNode, const FString& Prefix)
	{
		if (ChildNode)
		{
			FItemCacheKey Key = { ParentNode, ChildNode, InputIndex };
			TSharedPtr<FMutableGraphTreeElement>* CachedItem = ItemCache.Find(Key);

			if (CachedItem)
			{
				OutChildren.Add(*CachedItem);
			}
			else
			{
				TSharedPtr<FMutableGraphTreeElement>* MainItemPtr = MainItemPerNode.Find(ChildNode);
				TSharedPtr<FMutableGraphTreeElement> Item = MakeShareable(new FMutableGraphTreeElement(ChildNode, MainItemPtr, Prefix));
				OutChildren.Add(Item);
				ItemCache.Add(Key, Item);

				if (!MainItemPtr)
				{
					MainItemPerNode.Add(ChildNode, Item);
				}
			}
		}
		else
		{
			// No mutable node has been provided so create a dummy tree element
			TSharedPtr<FMutableGraphTreeElement> Item = MakeShareable(new FMutableGraphTreeElement(nullptr, nullptr , Prefix));
			OutChildren.Add(Item);
		}
		++InputIndex;
	};

	const UE::Mutable::Private::FNodeType* ParentNodeType = ParentNode->GetType();
	
	if (ParentNodeType == UE::Mutable::Private::NodeObjectNew::GetStaticType())
	{
		UE::Mutable::Private::NodeObjectNew* ObjectNew = StaticCast<UE::Mutable::Private::NodeObjectNew*>(ParentNode);
		for (int32 l = 0; l < ObjectNew->Components.Num(); ++l)
		{
			AddChildFunc(ObjectNew->Components[l].get(), TEXT("COMP") );
		}

		for (int32 Modifier = 0; Modifier < ObjectNew->Modifiers.Num(); Modifier++)
		{
			AddChildFunc(ObjectNew->Modifiers[Modifier].get(), FString::Printf(TEXT("MOD [%d]"), Modifier));
		}

		for (int32 l = 0; l < ObjectNew->Children.Num(); ++l)
		{
			AddChildFunc(ObjectNew->Children[l].get(), TEXT("CHILD"));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeObjectGroup::GetStaticType())
	{
		UE::Mutable::Private::NodeObjectGroup* ObjectGroup = StaticCast<UE::Mutable::Private::NodeObjectGroup*>(ParentNode);
		for (int32 l = 0; l < ObjectGroup->Children.Num(); ++l)
		{
			AddChildFunc(ObjectGroup->Children[l].get(), TEXT("CHILD"));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceNew::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceNew* SurfaceNew = StaticCast<UE::Mutable::Private::NodeSurfaceNew*>(ParentNode);
		AddChildFunc(SurfaceNew->Mesh.get(), TEXT("MESH"));

		AddChildFunc(SurfaceNew->Material.get(), TEXT("MATERIAL"));

		for (int32 l = 0; l < SurfaceNew->Strings.Num(); ++l)
		{
			AddChildFunc(SurfaceNew->Strings[l].String.get(), FString::Printf(TEXT("STRING [%s]"), *SurfaceNew->Strings[l].Name));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierSurfaceEdit::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceModifierSurfaceEdit* SurfaceEdit = StaticCast<UE::Mutable::Private::NodeSurfaceModifierSurfaceEdit*>(ParentNode);

		AddChildFunc(SurfaceEdit->MorphFactor.get(), FString::Printf(TEXT("MORPH_FACTOR [%s]"), *SurfaceEdit->MeshMorph));

		for (int32 LODIndex = 0; LODIndex < SurfaceEdit->LODs.Num(); ++LODIndex)
		{
			AddChildFunc(SurfaceEdit->LODs[LODIndex].MeshAdd.get(), FString::Printf(TEXT("LOD%d MESH_ADD"), LODIndex));
			AddChildFunc(SurfaceEdit->LODs[LODIndex].MeshRemove.get(), FString::Printf(TEXT("LOD%d MESH_REMOVE"), LODIndex));

			for (int32 l = 0; l < SurfaceEdit->LODs[LODIndex].Textures.Num(); ++l)
			{
				AddChildFunc(SurfaceEdit->LODs[LODIndex].Textures[l].Extend.get(), FString::Printf(TEXT("LOD%d EXTEND [%d]"), LODIndex, l));
				AddChildFunc(SurfaceEdit->LODs[LODIndex].Textures[l].PatchImage.get(), FString::Printf(TEXT("LOD%d PATCH IMAGE [%d]"), LODIndex, l));
				AddChildFunc(SurfaceEdit->LODs[LODIndex].Textures[l].PatchMask.get(), FString::Printf(TEXT("LOD%d PATCH MASK [%d]"), LODIndex, l));

			}
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceSwitch::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceSwitch* SurfaceSwitch = StaticCast<UE::Mutable::Private::NodeSurfaceSwitch*>(ParentNode);
		AddChildFunc(SurfaceSwitch->Parameter.get(), TEXT("PARAM"));
		for (int32 l = 0; l < SurfaceSwitch->Options.Num(); ++l)
		{
			AddChildFunc(SurfaceSwitch->Options[l].get(), FString::Printf(TEXT("OPTION [%d]"), l));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceVariation::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceVariation* SurfaceVar = StaticCast<UE::Mutable::Private::NodeSurfaceVariation*>(ParentNode);
		for (int32 l = 0; l < SurfaceVar->DefaultSurfaces.Num(); ++l)
		{
			AddChildFunc(SurfaceVar->DefaultSurfaces[l].get(), FString::Printf(TEXT("DEF SURF [%d]"), l));
		}
		for (int32 l = 0; l < SurfaceVar->DefaultModifiers.Num(); ++l)
		{
			AddChildFunc(SurfaceVar->DefaultModifiers[l].get(), FString::Printf(TEXT("DEF MOD [%d]"), l));
		}

		for (int32 v = 0; v < SurfaceVar->Variations.Num(); ++v)
		{
			const UE::Mutable::Private::NodeSurfaceVariation::FVariation Var = SurfaceVar->Variations[v];
			for (int32 l = 0; l < Var.Surfaces.Num(); ++l)
			{
				AddChildFunc(Var.Surfaces[l].get(), FString::Printf(TEXT("VAR [%s] SURF [%d]"), *Var.Tag, l));
			}
			for (int32 l = 0; l < Var.Modifiers.Num(); ++l)
			{
				AddChildFunc(Var.Modifiers[l].get(), FString::Printf(TEXT("VAR [%s] MOD [%d]"), *Var.Tag, l));
			}
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeLOD::GetStaticType())
	{
		UE::Mutable::Private::NodeLOD* LodVar = StaticCast<UE::Mutable::Private::NodeLOD*>(ParentNode);

		for (int32 SurfaceIndex = 0; SurfaceIndex < LodVar->Surfaces.Num(); SurfaceIndex++)
		{
			AddChildFunc(LodVar->Surfaces[SurfaceIndex].get(), FString::Printf(TEXT("SURFACE [%d]"), SurfaceIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeComponentNew::GetStaticType())
	{
		UE::Mutable::Private::NodeComponentNew* ComponentVar = StaticCast<UE::Mutable::Private::NodeComponentNew*>(ParentNode);
		AddChildFunc(ComponentVar->SkeletalMeshObject.get(), TEXT("SKELETAL MESH OBJECT"));
		AddChildFunc(ComponentVar->OverlayMaterial.get(), TEXT("OVERLAY MATERIAL"));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeModifierSkeletalMeshMerge::GetStaticType())
	{
		UE::Mutable::Private::NodeModifierSkeletalMeshMerge* ComponentVar = StaticCast<UE::Mutable::Private::NodeModifierSkeletalMeshMerge*>(ParentNode);
		AddChildFunc(ComponentVar->ToAddSkeletalMesh.get(), TEXT("TO SKELETAL MESH"));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeComponentSwitch::GetStaticType())
	{
		UE::Mutable::Private::NodeComponentSwitch* ComponentSwitch = StaticCast<UE::Mutable::Private::NodeComponentSwitch*>(ParentNode);
		AddChildFunc(ComponentSwitch->Parameter.get(), TEXT("PARAM"));
		for (int32 OptionIndex = 0; OptionIndex < ComponentSwitch->Options.Num(); OptionIndex++)
		{
			AddChildFunc(ComponentSwitch->Options[OptionIndex].get(), FString::Printf(TEXT("OPTION [%d]"), OptionIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshConvert::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshConvert* Node = StaticCast<UE::Mutable::Private::NodeSkeletalMeshConvert*>(ParentNode);
		AddChildFunc(Node->SkeletalMesh.get(), TEXT("SKELETAL MESH OBJECT"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshObjectParameter::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshObjectParameter* SkeletalMeshVar = StaticCast<UE::Mutable::Private::NodeSkeletalMeshObjectParameter*>(ParentNode);
		for (int32 RangeIndex = 0; RangeIndex < SkeletalMeshVar->Ranges.Num(); RangeIndex++)
		{
			AddChildFunc(SkeletalMeshVar->Ranges[RangeIndex].get(), FString::Printf(TEXT("RANGE [%d]"), RangeIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshNew::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshNew* SkeletalMeshNew = StaticCast<UE::Mutable::Private::NodeSkeletalMeshNew*>(ParentNode);
		for (int32 LODIndex = 0; LODIndex < SkeletalMeshNew->LODs.Num(); LODIndex++)
		{
			AddChildFunc(SkeletalMeshNew->LODs[LODIndex].get(), FString::Printf(TEXT("LOD [%d]"), LODIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshMerge::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshMerge* NodeSkeletalMeshMerge = StaticCast<UE::Mutable::Private::NodeSkeletalMeshMerge*>(ParentNode);
		AddChildFunc(NodeSkeletalMeshMerge->BaseSkeletalMesh.get(), FString::Printf(TEXT("BASE ")));
		AddChildFunc(NodeSkeletalMeshMerge->ToAddSkeletalMesh.get(), FString::Printf(TEXT("TO ADD ")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshModify::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshModify* NodeSkeletalMeshModify = StaticCast<UE::Mutable::Private::NodeSkeletalMeshModify*>(ParentNode);
		
		AddChildFunc(NodeSkeletalMeshModify->SkeletalMesh.get(), FString::Printf(TEXT("BASE ")));
		
		for (TTuple<FName, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeMaterial>> SlotMaterial : NodeSkeletalMeshModify->SlotMaterials)
		{
			AddChildFunc(SlotMaterial.Value.get(), FString::Printf(TEXT("Slot [%s]"), *SlotMaterial.Key.ToString()));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshMorph::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshMorph* NodeSkeletalMeshMorph = StaticCast<UE::Mutable::Private::NodeSkeletalMeshMorph*>(ParentNode);
		AddChildFunc(NodeSkeletalMeshMorph->Base.get(), FString::Printf(TEXT("BASE ")));
		AddChildFunc(NodeSkeletalMeshMorph->Factor.get(), FString::Printf(TEXT("FACTOR ")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshObjectConvert::GetStaticType())
	{
		UE::Mutable::Private::NodeSkeletalMeshObjectConvert* NodeSkeletalMeshObjectConvert = StaticCast<UE::Mutable::Private::NodeSkeletalMeshObjectConvert*>(ParentNode);
		AddChildFunc(NodeSkeletalMeshObjectConvert->SkeletalMesh.get(), FString::Printf(TEXT("SKELETAL MESH ")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSkeletalMeshTransform::GetStaticType())
	{
		const UE::Mutable::Private::NodeSkeletalMeshTransform* NodeSkeletalMeshTransform = StaticCast<UE::Mutable::Private::NodeSkeletalMeshTransform*>(ParentNode);
		AddChildFunc(NodeSkeletalMeshTransform->Source.get(), TEXT("SOURCE"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshSkeletalMeshObjectBreak::GetStaticType())
	{
		const UE::Mutable::Private::NodeMeshSkeletalMeshObjectBreak* NodeSkeletalMeshObjectBreak = StaticCast<UE::Mutable::Private::NodeMeshSkeletalMeshObjectBreak*>(ParentNode);
		AddChildFunc(NodeSkeletalMeshObjectBreak->SkeletalMeshObject.get(), TEXT("SKELETAL MESSH OBJECT"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshConstant::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshConstant* MeshConstantVar = StaticCast<UE::Mutable::Private::NodeMeshConstant*>(ParentNode);
		for (int32 LayoutIndex = 0; LayoutIndex < MeshConstantVar->Layouts.Num(); LayoutIndex++)
		{
			AddChildFunc(MeshConstantVar->Layouts[LayoutIndex].get(), FString::Printf(TEXT("LAYOUT [%d]"), LayoutIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageFormat::GetStaticType())
	{
		UE::Mutable::Private::NodeImageFormat* ImageFormatVar = StaticCast<UE::Mutable::Private::NodeImageFormat*>(ParentNode);
		AddChildFunc(ImageFormatVar->Source.get(), FString::Printf(TEXT("SOURCE IMAGE")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMeshFormat::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshFormat* MeshFormatVar = StaticCast<UE::Mutable::Private::NodeMeshFormat*>(ParentNode);
		AddChildFunc(MeshFormatVar->Source.get(), FString::Printf(TEXT("SOURCE MESH")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierMeshClipMorphPlane::GetStaticType())
	{
		// Nothing to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierMeshClipWithMesh::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceModifierMeshClipWithMesh* ModifierMeshClipWithMeshVar = StaticCast<UE::Mutable::Private::NodeSurfaceModifierMeshClipWithMesh*>(ParentNode);
		AddChildFunc(ModifierMeshClipWithMeshVar->ClipMesh.get(), FString::Printf(TEXT("CLIP MESH")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshClipWithMesh::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshClipWithMesh* ClipMeshWithMeshVar = StaticCast<UE::Mutable::Private::NodeMeshClipWithMesh*>(ParentNode);
		AddChildFunc(ClipMeshWithMeshVar->Source.get(), FString::Printf(TEXT("SOURCE MESH")));
		AddChildFunc(ClipMeshWithMeshVar->ClipMesh.get(), FString::Printf(TEXT("CLIP MESH")));
	}

	else if (ParentNode->GetType() == UE::Mutable::Private::NodeMeshRemoveMesh::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshRemoveMesh* RemoveMeshVar = StaticCast<UE::Mutable::Private::NodeMeshRemoveMesh*>(ParentNode);
		AddChildFunc(RemoveMeshVar->Source.get(), FString::Printf(TEXT("SOURCE MESH")));
		AddChildFunc(RemoveMeshVar->RemoveMesh.get(), FString::Printf(TEXT("REMOVE MESH")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierMeshClipDeform::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceModifierMeshClipDeform* ModifierMeshClipDeformVar = StaticCast<UE::Mutable::Private::NodeSurfaceModifierMeshClipDeform*>(ParentNode);
		AddChildFunc(ModifierMeshClipDeformVar->ClipMesh.get(), FString::Printf(TEXT("CLIP MESH")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierMeshClipWithUVMask::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceModifierMeshClipWithUVMask* ModifierMeshClipWithUVMaskVar = StaticCast<UE::Mutable::Private::NodeSurfaceModifierMeshClipWithUVMask*>(ParentNode);
		AddChildFunc(ModifierMeshClipWithUVMaskVar->ClipMask.get(), FString::Printf(TEXT("CLIP MASK")));
		AddChildFunc(ModifierMeshClipWithUVMaskVar->ClipLayout.get(), FString::Printf(TEXT("CLIP LAYOUT")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierMeshTransformInMesh::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceModifierMeshTransformInMesh* ModifierMeshTransformInMeshVar = StaticCast<UE::Mutable::Private::NodeSurfaceModifierMeshTransformInMesh*>(ParentNode);
		AddChildFunc(ModifierMeshTransformInMeshVar->BoundingMesh.get(), FString::Printf(TEXT("BOUNDING MESH")));
		AddChildFunc(ModifierMeshTransformInMeshVar->MatrixNode.get(), FString::Printf(TEXT("MESH TRANSFORM")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeSurfaceModifierMeshTransformWithBone::GetStaticType())
	{
		UE::Mutable::Private::NodeSurfaceModifierMeshTransformWithBone* ModifierMeshTransformWithBoneVar = StaticCast<UE::Mutable::Private::NodeSurfaceModifierMeshTransformWithBone*>(ParentNode);
		AddChildFunc(ModifierMeshTransformWithBoneVar->MatrixNode.get(), FString::Printf(TEXT("MESH TRANSFORM")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageSwitch::GetStaticType())
	{
		UE::Mutable::Private::NodeImageSwitch* ImageSwitchVar = StaticCast<UE::Mutable::Private::NodeImageSwitch*>(ParentNode);
		AddChildFunc(ImageSwitchVar->Parameter.get(), FString::Printf(TEXT("PARAM")));
		for (int32 OptionIndex = 0; OptionIndex < ImageSwitchVar->Options.Num(); OptionIndex++)
		{
			AddChildFunc(ImageSwitchVar->Options[OptionIndex].get(), FString::Printf(TEXT("OPTION [%d]"), OptionIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageMipmap::GetStaticType())
	{
		UE::Mutable::Private::NodeImageMipmap* ImageMipMapVar = StaticCast<UE::Mutable::Private::NodeImageMipmap*>(ParentNode);
		AddChildFunc(ImageMipMapVar->Source.get(), FString::Printf(TEXT("SOURCE")));
		AddChildFunc(ImageMipMapVar->Factor.get(), FString::Printf(TEXT("FACTOR")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageLayer::GetStaticType())
	{
		UE::Mutable::Private::NodeImageLayer* ImageLayerVar = StaticCast<UE::Mutable::Private::NodeImageLayer*>(ParentNode);
		AddChildFunc(ImageLayerVar->Base.get(), FString::Printf(TEXT("BASE")));
		AddChildFunc(ImageLayerVar->Mask.get(), FString::Printf(TEXT("MASK")));
		AddChildFunc(ImageLayerVar->Blended.get(), FString::Printf(TEXT("BLEND")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeImageLayerColor::GetStaticType())
	{
		UE::Mutable::Private::NodeImageLayerColor* ImageLayerColorVar = StaticCast<UE::Mutable::Private::NodeImageLayerColor*>(ParentNode);
		AddChildFunc(ImageLayerColorVar->Base.get(), FString::Printf(TEXT("BASE")));
		AddChildFunc(ImageLayerColorVar->Mask.get(), FString::Printf(TEXT("MASK")));
		AddChildFunc(ImageLayerColorVar->Color.get(), FString::Printf(TEXT("COLOR")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageResize::GetStaticType())
	{
		UE::Mutable::Private::NodeImageResize* ImageResizeVar = StaticCast<UE::Mutable::Private::NodeImageResize*>(ParentNode);
		AddChildFunc(ImageResizeVar->Base.get(), FString::Printf(TEXT("BASE")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMeshMorph::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshMorph* MeshMorphVar = StaticCast<UE::Mutable::Private::NodeMeshMorph*>(ParentNode);
		AddChildFunc(MeshMorphVar->Base.get(), FString::Printf(TEXT("BASE")));
		AddChildFunc(MeshMorphVar->Factor.get(), FString::Printf(TEXT("FACTOR")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageProject::GetStaticType())
	{
		UE::Mutable::Private::NodeImageProject* ImageProjectVar = StaticCast<UE::Mutable::Private::NodeImageProject*>(ParentNode);
		AddChildFunc(ImageProjectVar->Projector.get(), FString::Printf(TEXT("PROJECTOR")));
		AddChildFunc(ImageProjectVar->Mesh.get(), FString::Printf(TEXT("MESH")));
		AddChildFunc(ImageProjectVar->Image.get(), FString::Printf(TEXT("IMAGE")));
		AddChildFunc(ImageProjectVar->Mask.get(), FString::Printf(TEXT("MASK")));
		AddChildFunc(ImageProjectVar->AngleFadeStart.get(), FString::Printf(TEXT("FADE START ANGLE")));
		AddChildFunc(ImageProjectVar->AngleFadeEnd.get(), FString::Printf(TEXT("FADE END ANGLE")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImagePlainColor::GetStaticType())
	{
		UE::Mutable::Private::NodeImagePlainColor* ImagePlainColorVar = StaticCast<UE::Mutable::Private::NodeImagePlainColor*>(ParentNode);
		AddChildFunc(ImagePlainColorVar->Color.get(), FString::Printf(TEXT("COLOR")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeLayout::GetStaticType())
	{
		// Nothing to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeScalarEnumParameter::GetStaticType())
	{
		UE::Mutable::Private::NodeScalarEnumParameter* ScalarEnumParameterVar = StaticCast<UE::Mutable::Private::NodeScalarEnumParameter*>(ParentNode);
		for (int32 RangeIndex = 0; RangeIndex < ScalarEnumParameterVar->Ranges.Num(); RangeIndex++)
		{
			AddChildFunc(ScalarEnumParameterVar->Ranges[RangeIndex].get(), FString::Printf(TEXT("RANGE [%d]"), RangeIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMeshFragment::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshFragment* MeshFragmentVar = StaticCast<UE::Mutable::Private::NodeMeshFragment*>(ParentNode);
		AddChildFunc(MeshFragmentVar->SourceMesh.get(), FString::Printf(TEXT("MESH")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeColorSampleImage::GetStaticType())
	{
		UE::Mutable::Private::NodeColorSampleImage* ColorSampleImageVar = StaticCast<UE::Mutable::Private::NodeColorSampleImage*>(ParentNode);
		AddChildFunc(ColorSampleImageVar->Image.get(), FString::Printf(TEXT("IMAGE")));
		AddChildFunc(ColorSampleImageVar->X.get(), FString::Printf(TEXT("X")));
		AddChildFunc(ColorSampleImageVar->Y.get(), FString::Printf(TEXT("Y")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageInterpolate::GetStaticType())
	{
		UE::Mutable::Private::NodeImageInterpolate* ImageInterpolateVar = StaticCast<UE::Mutable::Private::NodeImageInterpolate*>(ParentNode);
		AddChildFunc(ImageInterpolateVar->Factor.get(), FString::Printf(TEXT("FACTOR")));
		for (int32 TargetIndex = 0; TargetIndex < ImageInterpolateVar->Targets.Num(); TargetIndex++)
		{
			AddChildFunc(ImageInterpolateVar->Targets[TargetIndex].get(), FString::Printf(TEXT("TARGET [%d]"), TargetIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeScalarConstant::GetStaticType())
	{
		// Nothing to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeScalarParameter::GetStaticType())
	{
		UE::Mutable::Private::NodeScalarParameter* ScalarParameterVar = StaticCast<UE::Mutable::Private::NodeScalarParameter*>(ParentNode);
		for (int32 RangeIndex = 0; RangeIndex < ScalarParameterVar->Ranges.Num(); RangeIndex++)
		{
			AddChildFunc(ScalarParameterVar->Ranges[RangeIndex].get(), FString::Printf(TEXT("RANGE [%d]"), RangeIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeColorParameter::GetStaticType())
	{
		UE::Mutable::Private::NodeColorParameter* ColorParameterVar = StaticCast<UE::Mutable::Private::NodeColorParameter*>(ParentNode);
		for (int32 RangeIndex = 0; RangeIndex < ColorParameterVar->Ranges.Num(); RangeIndex++)
		{
			AddChildFunc(ColorParameterVar->Ranges[RangeIndex].get(), FString::Printf(TEXT("RANGE [%d]"), RangeIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeColorConstant::GetStaticType())
	{
		// Nothing to show
	}


	else if (ParentNodeType == UE::Mutable::Private::NodeImageConstant::GetStaticType())
	{
		// Nothing to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMatrixParameter::GetStaticType())
	{
		const UE::Mutable::Private::NodeMatrixParameter* NodeMatrix = StaticCast<UE::Mutable::Private::NodeMatrixParameter*>(ParentNode);
			
		for (int32 RangeIndex = 0; RangeIndex < NodeMatrix->Ranges.Num(); RangeIndex++)
		{
			AddChildFunc(NodeMatrix->Ranges[RangeIndex].get(), FString::Printf(TEXT("RANGE [%d]"), RangeIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMatrixConstant::GetStaticType())
	{
		// Nothing to show
	}
	
	
	else if (ParentNodeType == UE::Mutable::Private::NodeScalarCurve::GetStaticType())
	{
		UE::Mutable::Private::NodeScalarCurve* ScalarCurveVar = StaticCast<UE::Mutable::Private::NodeScalarCurve*>(ParentNode);
		AddChildFunc(ScalarCurveVar->CurveSampleValue.get(), FString::Printf(TEXT("INPUT")));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMeshMakeMorph::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshMakeMorph* MeshMakeMorphVar = StaticCast<UE::Mutable::Private::NodeMeshMakeMorph*>(ParentNode);
		AddChildFunc(MeshMakeMorphVar->Base.get(), FString::Printf(TEXT("BASE")));
		AddChildFunc(MeshMakeMorphVar->Target.get(), FString::Printf(TEXT("TARGET")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeProjectorParameter::GetStaticType())
	{
		UE::Mutable::Private::NodeProjectorParameter* ProjectorParameterVar = StaticCast<UE::Mutable::Private::NodeProjectorParameter*>(ParentNode);
		for (int32 RangeIndex = 0; RangeIndex < ProjectorParameterVar->Ranges.Num(); RangeIndex++)
		{
			AddChildFunc(ProjectorParameterVar->Ranges[RangeIndex].get(), FString::Printf(TEXT("RANGE [%d]"), RangeIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeProjectorConstant::GetStaticType())
	{
		// Nothing to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeColorSwitch::GetStaticType())
	{
		UE::Mutable::Private::NodeColorSwitch* ColorSwitchVar = StaticCast<UE::Mutable::Private::NodeColorSwitch*>(ParentNode);
		AddChildFunc(ColorSwitchVar->Parameter.get(), TEXT("PARAM"));
		for (int32 OptionIndex = 0; OptionIndex < ColorSwitchVar->Options.Num(); ++OptionIndex)
		{
			AddChildFunc(ColorSwitchVar->Options[OptionIndex].get(), FString::Printf(TEXT("OPTION [%d]"), OptionIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageSwizzle::GetStaticType())
	{
		UE::Mutable::Private::NodeImageSwizzle* ImageSwizzleVar = StaticCast<UE::Mutable::Private::NodeImageSwizzle*>(ParentNode);
		for (int32 SourceIndex = 0; SourceIndex < ImageSwizzleVar->Sources.Num(); ++SourceIndex)
		{
			AddChildFunc(ImageSwizzleVar->Sources[SourceIndex].get(), FString::Printf(TEXT("SOURCE [%d]"), SourceIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeImageInvert::GetStaticType())
	{
		UE::Mutable::Private::NodeImageInvert* ImageInvertVar = StaticCast<UE::Mutable::Private::NodeImageInvert*>(ParentNode);
		AddChildFunc(ImageInvertVar->Base.get(), TEXT("BASE"));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeImageMultiLayer::GetStaticType())
	{
		UE::Mutable::Private::NodeImageMultiLayer* ImageMultilayerVar = StaticCast<UE::Mutable::Private::NodeImageMultiLayer*>(ParentNode);
		AddChildFunc(ImageMultilayerVar->Base.get(), FString::Printf(TEXT("BASE")));
		AddChildFunc(ImageMultilayerVar->Mask.get(), FString::Printf(TEXT("MASK")));
		AddChildFunc(ImageMultilayerVar->Blended.get(), FString::Printf(TEXT("BLEND")));
		AddChildFunc(ImageMultilayerVar->Range.get(), FString::Printf(TEXT("RANGE")));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeImageTable::GetStaticType())
	{
		// No nodes to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMeshTable::GetStaticType())
	{
		UE::Mutable::Private::NodeMeshTable* MeshTableVar = StaticCast<UE::Mutable::Private::NodeMeshTable*>(ParentNode);
		for (int32 LayoutIndex = 0; LayoutIndex < MeshTableVar->Layouts.Num(); ++LayoutIndex)
		{
			AddChildFunc(MeshTableVar->Layouts[LayoutIndex].get(), FString::Printf(TEXT("LAYOUT [%d]"), LayoutIndex));
		}
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeScalarTable::GetStaticType())
	{
		// Nothing to show
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeScalarSwitch::GetStaticType())
	{
		UE::Mutable::Private::NodeScalarSwitch* ScalarSwitchVar = StaticCast<UE::Mutable::Private::NodeScalarSwitch*>(ParentNode);
		AddChildFunc(ScalarSwitchVar->Parameter.get(), TEXT("PARAM"));
		for (int32 OptionIndex = 0; OptionIndex < ScalarSwitchVar->Options.Num(); ++OptionIndex)
		{
			AddChildFunc(ScalarSwitchVar->Options[OptionIndex].get(), FString::Printf(TEXT("OPTION [%d]"), OptionIndex));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeColorFromScalars::GetStaticType())
	{
		UE::Mutable::Private::NodeColorFromScalars* ScalarTableVar = StaticCast<UE::Mutable::Private::NodeColorFromScalars*>(ParentNode);
		AddChildFunc(ScalarTableVar->X.get(), TEXT("X"));
		AddChildFunc(ScalarTableVar->Y.get(), TEXT("Y"));
		AddChildFunc(ScalarTableVar->Z.get(), TEXT("Z"));
		AddChildFunc(ScalarTableVar->W.get(), TEXT("W"));
	}

	else if (ParentNodeType == UE::Mutable::Private::NodeMaterialModify::GetStaticType())
	{
		UE::Mutable::Private::NodeMaterialModify* MaterialModify = StaticCast<UE::Mutable::Private::NodeMaterialModify*>(ParentNode);

		AddChildFunc(MaterialModify->MaterialSource.get(), TEXT("MATERIAL"));
		
		for (const TPair<UE::Mutable::Private::FParameterKey, UE::Mutable::Private::FImageParameterData>& ImageData : MaterialModify->ImageParameters)
		{
			AddChildFunc(ImageData.Value.ImageNode.get(), FString::Printf(TEXT("IMAGE [%s]"), *ImageData.Key.ParameterName.ToString()));
		}

		for (const TPair<UE::Mutable::Private::FParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeColor>>& ColorData : MaterialModify->ColorParameters)
		{
			AddChildFunc(ColorData.Value.get(), FString::Printf(TEXT("VECTOR [%s]"), *ColorData.Key.ParameterName.ToString()));
		}

		for (const TPair<UE::Mutable::Private::FParameterKey, UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeScalar>>& ScalarData : MaterialModify->ScalarParameters)
		{
			AddChildFunc(ScalarData.Value.get(), FString::Printf(TEXT("SCALAR [%s]"), *ScalarData.Key.ParameterName.ToString()));
		}
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMaterialSkeletalMeshBreak::GetStaticType())
	{
		const UE::Mutable::Private::NodeMaterialSkeletalMeshBreak* Node = StaticCast<UE::Mutable::Private::NodeMaterialSkeletalMeshBreak*>(ParentNode);
		AddChildFunc(Node->SkeletalMesh.get(), TEXT("SKELETAL MESH"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshTransformWithBone::GetStaticType())
	{
		const UE::Mutable::Private::NodeMeshTransformWithBone* NodeMeshTransformWithBone = StaticCast<UE::Mutable::Private::NodeMeshTransformWithBone*>(ParentNode);
		AddChildFunc(NodeMeshTransformWithBone->Source.get(), TEXT("SOURCE"));
		AddChildFunc( NodeMeshTransformWithBone->MatrixNode.get(), TEXT("TRANSFORM"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshTransform::GetStaticType())
	{
		const UE::Mutable::Private::NodeMeshTransform* NodeMeshTransform = StaticCast<UE::Mutable::Private::NodeMeshTransform*>(ParentNode);
		AddChildFunc(NodeMeshTransform->Source.get(), TEXT("SOURCE"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshTransformInMesh::GetStaticType())
	{
		const UE::Mutable::Private::NodeMeshTransformInMesh* NodeMeshTransformInMesh = StaticCast<UE::Mutable::Private::NodeMeshTransformInMesh*>(ParentNode);
		AddChildFunc(NodeMeshTransformInMesh->SourceMesh.get(), TEXT("SOURCE"));
		AddChildFunc(NodeMeshTransformInMesh->BoundingMesh.get(), TEXT("BOUNDS"));
		AddChildFunc(NodeMeshTransformInMesh->MatrixNode.get(), TEXT("TRANSFORM"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeColorTable::GetStaticType())
	{
		// Nothing to show
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeMeshReshape::GetStaticType())
	{
		const UE::Mutable::Private::NodeMeshReshape* NodeMeshReshape = StaticCast<UE::Mutable::Private::NodeMeshReshape*>(ParentNode);
		AddChildFunc(NodeMeshReshape->BaseMesh.get(), TEXT("BASE MESH"));
		AddChildFunc(NodeMeshReshape->BaseShape.get(), TEXT("BASE SHAPE"));
		AddChildFunc(NodeMeshReshape->TargetShape.get(), TEXT("TGT SHAPE"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeScalarArithmeticOperation::GetStaticType())
	{
		const UE::Mutable::Private::NodeScalarArithmeticOperation* NodeScalarArithmeticOperation = StaticCast<UE::Mutable::Private::NodeScalarArithmeticOperation*>(ParentNode);
		AddChildFunc(NodeScalarArithmeticOperation->A.get(), TEXT("A"));
		AddChildFunc(NodeScalarArithmeticOperation->B.get(), TEXT("B"));
	}
	
	else if (ParentNodeType == UE::Mutable::Private::NodeImageConvert::GetStaticType())
	{
		const UE::Mutable::Private::NodeImageConvert* NodeImageConvert = StaticCast<UE::Mutable::Private::NodeImageConvert*>(ParentNode);
		AddChildFunc(NodeImageConvert->ImageParameter.get(), TEXT("IMAGE PARAM"));
	}
	
	else
	{
		UE_LOGF(LogMutable,Error, "The node of type %d has not been implemented, so its children won't be added to the tree.", int32(ParentNodeType->Type));

		// Add a placeholder to the tree
		const FString Prefix =  FString::Printf(TEXT("[%d] NODE TYPE NOT IMPLEMENTED"), int32(ParentNodeType->Type));
		AddChildFunc(nullptr, Prefix);
	}
#endif
}


TSharedPtr<SWidget> SMutableGraphViewer::OnTreeContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Graph_Expand_Instance", "Expand Instance-Level Operations"),
		LOCTEXT("Graph_Expand_Instance_Tooltip", "Expands all the operations in the tree that are instance operations (not images, meshes, booleans, etc.)."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SMutableGraphViewer::TreeExpandUnique)
			//, FCanExecuteAction::CreateSP(this, &SMutableCodeViewer::HasAnyItemInPalette)
		)
	);

	return MenuBuilder.MakeWidget();
}


void SMutableGraphViewer::TreeExpandRecursive(TSharedPtr<FMutableGraphTreeElement> InInfo, bool bExpand)
{
	if (bExpand)
	{
		TreeExpandUnique();
	}
}


void SMutableGraphViewer::TreeExpandUnique()
{
	TArray<TSharedPtr<FMutableGraphTreeElement>> Pending = RootNodes;

	TSet<TSharedPtr<FMutableGraphTreeElement>> Processed;

	TArray<TSharedPtr<FMutableGraphTreeElement>> Children;

	while (!Pending.IsEmpty())
	{
		TSharedPtr<FMutableGraphTreeElement> Item = Pending.Pop();
		TreeView->SetItemExpansion(Item, true);

		Children.SetNum(0);
		GetChildrenForInfo(Item, Children);
		Pending.Append(Children);
	}
}


#undef LOCTEXT_NAMESPACE 


