// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/CustomizableObjectCompiler.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SNumericDropDown.h"

class STableViewBase;
namespace ESelectInfo { enum Type : int; }
template <typename ItemType> class STreeView;

class FDragDropEvent;
class FReferenceCollector;
class FMutableGraphTreeElement;
class FTabManager;
class ITableRow;
class STextComboBox;
class SWidget;
struct FGeometry;


/** This widget shows the internal Mutable Graph for debugging purposes. 
 * This is not the Unreal source graph in the UCustomizableObject, but an intermediate step of the compilation process.
 */
class SMutableGraphViewer :
	public SCompoundWidget,
	public FGCObject
{
public:

	// SWidget interface
	SLATE_BEGIN_ARGS(SMutableGraphViewer) {}
		
	SLATE_ARGUMENT(TArray<TSoftObjectPtr<const UTexture>>, ReferencedRuntimeTextures)
	SLATE_ARGUMENT(TArray<FMutableSourceTextureData>, ReferencedCompileTextures)
	SLATE_ARGUMENT(TArray<TSoftObjectPtr<const UStreamableRenderAsset>>, ReferencedRuntimeMeshes)
	SLATE_ARGUMENT(TArray<FMutableSourceMeshData>, ReferencedCompileMeshes)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const UE::Mutable::Private::NodePtr& InRootNode);

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:
	/** The Mutable Graph to show, represented by its root. */
	UE::Mutable::Private::NodePtr RootNode;

	/** Array of external referenced textures in MutableModel, indexed by id. */
	TArray<TSoftObjectPtr<const UTexture>> ReferencedRuntimeTextures;
	TArray<FMutableSourceTextureData> ReferencedCompileTextures;

	/** Object compiler. */
	TSharedRef<FCustomizableObjectCompiler> Compiler = MakeShared<FCustomizableObjectCompiler>();

	/** Root nodes of the tree widget. */
	TArray<TSharedPtr<FMutableGraphTreeElement>> RootNodes;

	/** Tree showing the graph. */
	TSharedPtr<STreeView<TSharedPtr<FMutableGraphTreeElement>>> TreeView;

	/** Cache of tree elements matching the graph nodes that have been generated so far. 
	* We store both the parent and the node in the key, because a single node may appear multiple times if it has different parents.
	*/
	struct FItemCacheKey
	{
		const UE::Mutable::Private::Node* Parent = nullptr;
		const UE::Mutable::Private::Node* Child = nullptr;
		uint32 ChildIndexInParent = 0;

		friend FORCEINLINE bool operator == (const FItemCacheKey& A, const FItemCacheKey& B)
		{
			return A.Parent == B.Parent && A.Child == B.Child && A.ChildIndexInParent == B.ChildIndexInParent;
		}

		friend FORCEINLINE uint32 GetTypeHash(const FItemCacheKey& Key)
		{
			return HashCombine( GetTypeHash(Key.Parent), HashCombine(GetTypeHash(Key.Child), Key.ChildIndexInParent));
		}
	};
	TMap< FItemCacheKey, TSharedPtr<FMutableGraphTreeElement>> ItemCache;

	/** Main tree item for each node. A graph node can be representes with multiple tree ndoes if it is reachable from different paths. */
	TMap< const UE::Mutable::Private::Node*, TSharedPtr<FMutableGraphTreeElement>> MainItemPerNode;

	/** */
	void RebuildTree();
	
	/** Callbacks from the tree widget. */
 	TSharedRef<ITableRow> GenerateRowForNodeTree(TSharedPtr<FMutableGraphTreeElement> InTreeNode, const TSharedRef<STableViewBase>& InOwnerTable);
	void GetChildrenForInfo(TSharedPtr<FMutableGraphTreeElement> InInfo, TArray< TSharedPtr<FMutableGraphTreeElement> >& OutChildren);
	TSharedPtr<SWidget> OnTreeContextMenuOpening();
	void TreeExpandRecursive(TSharedPtr<FMutableGraphTreeElement> Info, bool bExpand);
	void TreeExpandUnique();

};


/** An row of the code tree in the SMutableGraphViewer. */
class FMutableGraphTreeElement : public TSharedFromThis<FMutableGraphTreeElement>
{
public:
	FMutableGraphTreeElement(const UE::Mutable::Private::NodePtr& InNode, TSharedPtr<FMutableGraphTreeElement>* InDuplicatedOf=nullptr, const FString& InPrefix=FString() )
	{
		MutableNode = InNode;
		Prefix = InPrefix;
		if (InDuplicatedOf)
		{
			DuplicatedOf = *InDuplicatedOf;
		}
	}

public:

	/** Mutable Graph Node represented in this tree row. */
	UE::Mutable::Private::NodePtr MutableNode;

	/** If this tree element is a duplicated of another node, this is the node. */
	TSharedPtr<FMutableGraphTreeElement> DuplicatedOf;

	/** Optional label prefix */
	FString Prefix;
};
