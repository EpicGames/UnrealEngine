// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableImageViewer.h"

#include "Brushes/SlateImageBrush.h"
#include "MuT/TypeInfo.h"
#include "MuT/UnrealPixelFormatOverride.h"
#include "TextureResource.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"

class FPaintArgs;
class FSlateRect;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SSimpleTextureViewer::Construct( const FArguments& InArgs )
{
	GridSize = InArgs._GridSize;

	SetTexture(InArgs._Texture);
}


void SSimpleTextureViewer::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (Texture)
	{
		Collector.AddReferencedObject(Texture);
	}
}


FString SSimpleTextureViewer::GetReferencerName() const
{
	return TEXT("SSimpleTextureViewer");
}


void SSimpleTextureViewer::SetTexture( UTexture* InTexture)
{
	Texture = InTexture;

	if (!Texture)
	{
		TextureBrush = MakeShared<FSlateBrush>();
		return;
	}

	FTextureResource* Resource = Texture->GetResource();
	if (!Resource)
	{
		TextureBrush = MakeShared<FSlateBrush>();
		return;
	}

	FVector2D Size(Resource->GetSizeX(), Resource->GetSizeY());
	TextureBrush = MakeShared<FSlateImageBrush>(Texture, Size,
		FSlateColor(FLinearColor(1, 1, 1, 1)), ESlateBrushTileType::NoTile,
		ESlateBrushImageType::Linear
);

}


int32 SSimpleTextureViewer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 RetLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyClippingRect, OutDrawElements, LayerId,InWidgetStyle, bParentEnabled );

	bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const ESlateDrawEffect DrawEffects = bEnabled ?
		(ImageChannels == EMutableImageChannels::RGBA ? ESlateDrawEffect::None : ESlateDrawEffect::IgnoreTextureAlpha)
		: ESlateDrawEffect::DisabledEffect;

	// Paint inside the border only. 
	const FVector2D BorderPadding = FVector2D(2,2);
	FPaintGeometry ForegroundPaintGeometry = AllottedGeometry.ToInflatedPaintGeometry( -BorderPadding );
	
	const FIntPoint GridSizePoint = GridSize.Get();
	const float OffsetX = BorderPadding.X;
	const FVector2D AreaSize =  AllottedGeometry.Size - 2.0f * BorderPadding;
	const float GridRatio = float(GridSizePoint.X) / float(GridSizePoint.Y);
	FVector2D Size;
	if ( AreaSize.X/GridRatio > AreaSize.Y )
	{
		Size.Y = AreaSize.Y;
		Size.X = AreaSize.Y*GridRatio;
	}
	else
	{
		Size.X =  AreaSize.X;
		Size.Y =  AreaSize.X/GridRatio;
	}

	float AuxCellSize = Size.X / GridSizePoint.X;
	FVector2D ImageOffset = (AreaSize-Size)/2.0f;
	FVector2D ImageOrigin = BorderPadding + ImageOffset;

	// Draw background image
	{
		FGeometry ImageGeometry = AllottedGeometry.MakeChild(Size, FSlateLayoutTransform(ImageOrigin));

		const FSlateBrush* ThisBackgroundImage = TextureBrush.Get();
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			ImageGeometry.ToPaintGeometry(),
			ThisBackgroundImage,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * ThisBackgroundImage->TintColor.GetColor(InWidgetStyle)
		);
	}
	
	// Create line points
	TArray< FVector2D > LinePoints;
	LinePoints.SetNum(2);
	
	for( int32 LineIndex = 0; LineIndex < GridSizePoint.X + 1; LineIndex++ )
	{
		LinePoints[0] = FVector2D(ImageOrigin.X + LineIndex * AuxCellSize, ImageOrigin.Y );
		LinePoints[1] = FVector2D(ImageOrigin.X + LineIndex * AuxCellSize, ImageOrigin.Y + Size.Y );

		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			RetLayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FColor(150, 150, 150, 64),
			false,
			2.0
			);
	}

	for( int32 LineIndex = 0; LineIndex < GridSizePoint.Y + 1; LineIndex++ )
	{
		LinePoints[0] = FVector2D(ImageOrigin.X, ImageOrigin.Y + LineIndex * AuxCellSize );
		LinePoints[1] = FVector2D(ImageOrigin.X + Size.X, ImageOrigin.Y + LineIndex * AuxCellSize );

		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			RetLayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			FColor(150, 150, 150, 64),
			false,
			2.0
			);
	}

	RetLayerId++;

	return RetLayerId - 1;
}


FVector2D SSimpleTextureViewer::ComputeDesiredSize(float NotUsed) const
{
	return FVector2D(200.0f, 200.0f);
}


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void SMutableImageViewer::Construct(const FArguments& InArgs)
{
	GridSize = InArgs._GridSize;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(4.0f)
		.AutoHeight()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(this, &SMutableImageViewer::GetImageDescriptionLabel)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0, 4.0f, 0.0f, 4.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowLOD","Show LOD "))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 4.0f, 4.0f, 4.0f))
				[
					SNew(SNumericEntryBox<int32>)
					.Visibility(this, &SMutableImageViewer::IsLODSelectionVisible)
					.AllowSpin(true)
					.MinValue(0)
					.MaxValue(this, &SMutableImageViewer::GetImageLODMaxValue)
					.MinSliderValue(0)
					.MaxSliderValue(this, &SMutableImageViewer::GetImageLODMaxValue)
					.Value(this, &SMutableImageViewer::GetCurrentImageLOD)
					.OnValueChanged(this, &SMutableImageViewer::OnCurrentLODChanged)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 4.0f, 4.0f, 4.0f))
				[
					SNew(SSegmentedControl<EMutableImageChannels>)
					.Value(this, &SMutableImageViewer::GetImageChannels)
					.OnValueChanged(this, &SMutableImageViewer::SetImageChannels)
					+ SSegmentedControl<EMutableImageChannels>::Slot(EMutableImageChannels::RGBA)
					.Text(FText::FromString(TEXT("RGBA")))
					+ SSegmentedControl<EMutableImageChannels>::Slot(EMutableImageChannels::RGB)
					.Text(FText::FromString(TEXT("RGB")))
					+ SSegmentedControl<EMutableImageChannels>::Slot(EMutableImageChannels::R)
					.Text(FText::FromString(TEXT("R")))
					+ SSegmentedControl<EMutableImageChannels>::Slot(EMutableImageChannels::G)
					.Text(FText::FromString(TEXT("G")))
					+ SSegmentedControl<EMutableImageChannels>::Slot(EMutableImageChannels::B)
					.Text(FText::FromString(TEXT("B")))
					+ SSegmentedControl<EMutableImageChannels>::Slot(EMutableImageChannels::A)
					.Text(FText::FromString(TEXT("A")))
				]
			]
		]

		+ SVerticalBox::Slot()
		[
			SAssignNew(TextureViewer, SSimpleTextureViewer)
			.GridSize(GridSize)
		]
	];

	if (InArgs._Image)
	{
		SetImage(InArgs._Image,0);
	}
}


void SMutableImageViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bIsPendingUpdate)
	{
		bIsPendingUpdate = false;

		// Convert from Mutable To Unreal
		UTexture2D* UnrealImage = NewObject<UTexture2D>(UTexture2D::StaticClass());

		if (MutableImage && MutableImage->GetSizeX()>0 && MutableImage->GetSizeY()>0)
		{
			// If the image format is not supported in the editor platform (for instance, it is a mobile-specific format)
			// convert the mutable image to a generic format so that it can be previewed.
			UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> ImageToConvert = MutableImage;

			// \TODO: This test is a bit weak
			if (MutableImage->GetFormat() >= UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR
				&&
				MutableImage->GetFormat() <= UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR)
			{
				UE::Mutable::Private::FImageOperator ImOp = UE::Mutable::Private::FImageOperator::GetDefault(UE::Mutable::Private::FImageOperator::FImagePixelFormatFunc());
				int32 Quality = 4;
				ImageToConvert = ImOp.ImagePixelFormat(Quality, MutableImage.Get(), UE::Mutable::Private::GetUncompressedFormat(MutableImage->GetFormat()) );
			}

			// Should we extract a single channel?
			int32 ExtractChannel = -1;
			switch (GetImageChannels())
			{
			case EMutableImageChannels::R: ExtractChannel = 0; break;
			case EMutableImageChannels::G: ExtractChannel = 1; break;
			case EMutableImageChannels::B: ExtractChannel = 2; break;
			case EMutableImageChannels::A: ExtractChannel = 3; break;
			default:
				break;
			}

			FMutableModelImageProperties Props;
			Props.Filter = TF_Nearest;
			Props.SRGB = true;
			Props.LODBias = 0;
			ConvertImage(UnrealImage, ImageToConvert, Props, CurrentVisibleLOD, ExtractChannel);
			UnrealImage->NeverStream = true;
			UnrealImage->UpdateResource();
		}

		TextureViewer->SetTexture(UnrealImage);
	}
}


void SMutableImageViewer::SetImage(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage>& InMutableImage, int32 CurrentLOD)
{
	MutableImage = InMutableImage;
	CurrentVisibleLOD = CurrentLOD;
	bIsPendingUpdate = true;

	constexpr bool bDebugDump = false;

	if (bDebugDump && InMutableImage)
	{
		static int32 SImageCount = 0;
		FString ImagePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectSavedDir(), FString::Printf( TEXT("MutableDebug_%d.astc"), SImageCount)));
		++SImageCount;

		DebugImageDump( InMutableImage.Get(), ImagePath);
	}
}


FText SMutableImageViewer::GetImageDescriptionLabel() const
{
	if (!MutableImage)
	{
		return FText::FromString(TEXT("No image."));
	}

	UE::Mutable::Private::EImageFormat Format = MutableImage->GetFormat();
	FString Label = FString::Printf(TEXT("%s - %d x %d - %d LODs"), 
		StringCast<TCHAR>(UE::Mutable::Private::TypeInfo::s_imageFormatName[(size_t)Format]).Get(),
		MutableImage->GetSizeX(), 
		MutableImage->GetSizeY(),
		MutableImage->GetLODCount() );
	return FText::FromString( Label );
}


TOptional<int32> SMutableImageViewer::GetCurrentImageLOD() const
{
	return CurrentVisibleLOD;
}


TOptional<int32> SMutableImageViewer::GetImageLODMaxValue() const
{
	if (MutableImage)
	{
		return MutableImage->GetLODCount() - 1;
	}

	return 0;
}


EVisibility SMutableImageViewer::IsLODSelectionVisible() const
{
	if (MutableImage)
	{
		return EVisibility::Visible;
	}
	
	return EVisibility::Hidden;
}


void SMutableImageViewer::OnCurrentLODChanged(int32 NewValue)
{
	CurrentVisibleLOD = NewValue;
	SetImage( MutableImage, CurrentVisibleLOD );
}


EMutableImageChannels SMutableImageViewer::GetImageChannels() const
{
	if (TextureViewer)
	{
		return TextureViewer->ImageChannels;
	}

	return EMutableImageChannels::RGBA;
}


void SMutableImageViewer::SetImageChannels(EMutableImageChannels InChannels)
{
	if (TextureViewer)
	{
		TextureViewer->ImageChannels = InChannels;
		bIsPendingUpdate = true;
	}
}


#undef LOCTEXT_NAMESPACE

