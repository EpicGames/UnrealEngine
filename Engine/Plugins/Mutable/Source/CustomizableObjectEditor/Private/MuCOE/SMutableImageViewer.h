// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FReferenceCollector;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class UTexture;
struct FGeometry;
struct FSlateBrush;


/** Describes different modes to show an image. */
enum class EMutableImageChannels : uint8
{
	RGBA,
	RGB,
	R,
	G,
	B,
	A
};


/** Simple widget that shows a UTexture. */
class SSimpleTextureViewer : public SCompoundWidget, public FGCObject
{
public:

	SLATE_BEGIN_ARGS(SSimpleTextureViewer){}
		SLATE_ATTRIBUTE(FIntPoint, GridSize)
		SLATE_ARGUMENT_DEFAULT(UTexture*, Texture) { nullptr };
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

	// SWidgetInterface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	// Own interface

	/** Set the image to show in the widget. */
	void SetTexture(UTexture* InTexture);

	/** */
	EMutableImageChannels ImageChannels = EMutableImageChannels::RGBA;

private:

	/** Reference to the texture. */
	TObjectPtr<UTexture> Texture;

	/** Brush used to render the image. */
	TSharedPtr<FSlateBrush> TextureBrush;

	/** Size of the underlying rendered grid. */
	TAttribute<FIntPoint> GridSize;

};


/** Widget showing a Mutable image, with its properties. */
class SMutableImageViewer : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SMutableImageViewer) {}
		SLATE_ATTRIBUTE(FIntPoint, GridSize)
		SLATE_ARGUMENT(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage>, Image)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// Own interface

	/** Set the image to show in the widget. */
	void SetImage(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage>& Image, int32 VisibleLOD );

private:

	/** Brush used to render the image. */
	TSharedPtr<SSimpleTextureViewer> TextureViewer;

	/** Size of the underlying rendered grid. */
	TAttribute<FIntPoint> GridSize;

	/** */
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> MutableImage;

	/** Mipmap shown in the widget. 0 is the biggest one. */
	int32 CurrentVisibleLOD=0;

	/** Is true, the image or the visible LOD have changed and we need to update. */
	bool bIsPendingUpdate = false;

	/** User interface callbacks */
	FText GetImageDescriptionLabel() const;
	TOptional<int32> GetCurrentImageLOD() const;
	TOptional<int32> GetImageLODMaxValue() const;
	EVisibility IsLODSelectionVisible() const;
	void OnCurrentLODChanged(int32 NewValue);
	EMutableImageChannels GetImageChannels() const;
	void SetImageChannels(EMutableImageChannels);
};
