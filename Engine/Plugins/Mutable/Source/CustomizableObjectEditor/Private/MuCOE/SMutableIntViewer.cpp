// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableIntViewer.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void SMutableIntViewer::Construct(const FArguments& InArgs)
{
	const FText IntValueTitle = LOCTEXT("IntValueTitle", "Int Value : ");

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Title
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(IntValueTitle)
		]

		// Value
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(this, &SMutableIntViewer::GetValue)
		]
	];
}


void SMutableIntViewer::SetInt(const int32& InInt)
{
	this->IntValue = InInt;
}

FText SMutableIntViewer::GetValue() const
{
	return FText::AsNumber(this->IntValue);
}

#undef LOCTEXT_NAMESPACE
