// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

/*
* Mutable viewer to show a representation of an int variable held by mutable.
*/
class SMutableIntViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableIntViewer) {}
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Bool to be used for this widget */
	void SetInt(const int32& InInt);

private:

	/** The value that is being displayed by the UI*/
	int32 IntValue;

	/*
	* Callback for udpating the widget UI
	* @return The value stored on IntValue as a FText
	*/
	FText GetValue() const;
};
