// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableLayoutViewer.h"

#include "MuCOE/SCustomizableObjectLayoutGrid.h"
#include "MuR/Mesh.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"



void SMutableLayoutViewer::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(4.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 4.0f, 4.0f, 4.0f))
			[
				SNew(STextBlock)
				.Text(this, &SMutableLayoutViewer::GetLayoutDescriptionLabel)
			]
			//+ SHorizontalBox::Slot()
			//.AutoWidth()
			//.VAlign(VAlign_Center)
			//.Padding(FMargin(4.0f, 4.0f, 0.0f, 4.0f))
			//[
			//	SNew(STextBlock)
			//	.Text(LOCTEXT("ShowLOD","Show LOD: "))
			//]
			//+ SHorizontalBox::Slot()
			//.FillWidth(1.0f)
			//.VAlign(VAlign_Center)
			//.Padding(FMargin(0.0f, 4.0f, 4.0f, 4.0f))
			//[
			//	SNew(SNumericEntryBox<int32>)
			//	.Visibility(this, &SMutableLayoutViewer::IsLODSelectionVisible)
			//	.AllowSpin(true)
			//	.MinValue(0)
			//	.MaxValue(this, &SMutableLayoutViewer::GetLayoutLODMaxValue)
			//	.MinSliderValue(0)
			//	.MaxSliderValue(this, &SMutableLayoutViewer::GetLayoutLODMaxValue)
			//	.Value(this, &SMutableLayoutViewer::GetCurrentLayoutLOD)
			//	.OnValueChanged(this, &SMutableLayoutViewer::OnCurrentLODChanged)
			//]
			//+ SHorizontalBox::Slot()
			//.AutoWidth()
			//.Padding(FMargin(4.0f, 4.0f, 4.0f, 4.0f))
			//[
			//	SNew(SSegmentedControl<EMutableLayoutChannels>)
			//	.Value(this, &SMutableLayoutViewer::GetLayoutChannels)
			//	.OnValueChanged(this, &SMutableLayoutViewer::SetLayoutChannels)
			//	+ SSegmentedControl<EMutableLayoutChannels>::Slot(EMutableLayoutChannels::RGBA)
			//	.Text(FText::FromString(TEXT("RGBA")))
			//	+ SSegmentedControl<EMutableLayoutChannels>::Slot(EMutableLayoutChannels::RGB)
			//	.Text(FText::FromString(TEXT("RGB")))
			//	+ SSegmentedControl<EMutableLayoutChannels>::Slot(EMutableLayoutChannels::A)
			//	.Text(FText::FromString(TEXT("A")))
			//]
		]

		+ SVerticalBox::Slot()
		[
			SAssignNew(LayoutViewer, SCustomizableObjectLayoutGrid)
			.Mode(ELayoutGridMode::ELGM_Show)
		]
	];

	if (InArgs._Layout)
	{
		SetLayout(InArgs._Layout);
	}
}


void SMutableLayoutViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bIsPendingUpdate)
	{
		bIsPendingUpdate = false;

		if (!MutableLayout)
		{
			LayoutViewer->SetBlocks(FIntPoint(1, 1), {});
		}
		else
		{
			int32 BlockCount = MutableLayout->GetBlockCount();
			TArray<FCustomizableObjectLayoutBlock> Blocks;
			Blocks.SetNum( BlockCount );
			for (int32 BlockIndex = 0; BlockIndex < BlockCount; ++BlockIndex)
			{
				FIntVector2 Min = MutableLayout->Blocks[BlockIndex].Min;
				FIntVector2 Size = MutableLayout->Blocks[BlockIndex].Size;

				Blocks[BlockIndex].Min[0] = Min[0];
				Blocks[BlockIndex].Min[1] = Min[1];
				Blocks[BlockIndex].Max[0] = Min[0] + Size[0];
				Blocks[BlockIndex].Max[1] = Min[1] + Size[1];
				Blocks[BlockIndex].Id = FGuid(0,0,0,BlockIndex);
			}

			LayoutViewer->SetBlocks(MutableLayout->GetGridSize(), Blocks);
		}
	}
}


void SMutableLayoutViewer::SetLayout(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout>& InMutableLayout)
{
	MutableLayout = InMutableLayout;
	bIsPendingUpdate = true;
}


FText SMutableLayoutViewer::GetLayoutDescriptionLabel() const
{
	if (!MutableLayout)
	{
		return FText::FromString(TEXT("No Layout."));
	}

	FString Label = FString::Printf(TEXT("%d x %d - %d blocks"), 		
		MutableLayout->GetGridSize()[0],
		MutableLayout->GetGridSize()[1],
		MutableLayout->GetBlockCount() );
	return FText::FromString( Label );
}


#undef LOCTEXT_NAMESPACE

