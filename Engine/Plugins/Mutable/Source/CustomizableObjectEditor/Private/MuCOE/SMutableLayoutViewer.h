// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Layout.h"
#include "Widgets/SCompoundWidget.h"

class SCustomizableObjectLayoutGrid;
struct FGeometry;


/** Widget showing a Mutable layout, with its properties. */
class SMutableLayoutViewer : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SMutableLayoutViewer) {}
		SLATE_ATTRIBUTE(FIntPoint, GridSize)
		SLATE_ARGUMENT(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout>, Layout)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget interface
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// Own interface

	/** Set the image to show in the widget. */
	void SetLayout(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout>& Image );

private:

	/** Brush used to render the layout. */
	TSharedPtr<SCustomizableObjectLayoutGrid> LayoutViewer;

	/** Mutable Layout object being previewed */
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FLayout> MutableLayout;

	/** Is true, the image or the visible LOD have changed and we need to update. */
	bool bIsPendingUpdate = false;

	/** User interface callbacks */
	FText GetLayoutDescriptionLabel() const;
};
