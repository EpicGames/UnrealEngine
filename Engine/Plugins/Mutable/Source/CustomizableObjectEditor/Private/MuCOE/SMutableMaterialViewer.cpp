// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableMaterialViewer.h"

#include "Editor.h"
#include "PropertyCustomizationHelpers.h"
#include "SMutableColorViewer.h"
#include "SMutableImageViewer.h"
#include "Components/HorizontalBox.h"
#include "MuT/NodeMaterialModify.h"
#include "MuT/TypeInfo.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SRichTextHyperlink.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"


class ITableRow;
class STableViewBase;
class SWidget;
class USkeleton;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

namespace MutableMaterialParameterColumns
{
	static const FName ParameterIndexID("Index");
	static const FName ParameterNameID("Parameter");
	static const FName ParameterLayerID("Layer");

	// Image Params
	static const FName ParameterImageAddressID("Image OP");
	static const FName ParameterImageID("Image");
	
	// Color Params
	static const FName ParameterColorID("Color");
	
	// Scalar Params
	static const FName ParameterScalarID("Scalar");
}

class SMutableMaterialImageParameterRow : public SMultiColumnTableRow<TSharedPtr<FMutableMaterialImageParameterElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView,
				   const TSharedPtr<FMutableMaterialImageParameterElement>& InRowItem, TWeakPtr<SMutableCodeViewer> InHostCodeViewer)
	{
		MutableCodeViewer = InHostCodeViewer;
		RowItem = InRowItem;
		
		SMultiColumnTableRow<TSharedPtr<FMutableMaterialImageParameterElement>>::Construct(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (!RowItem)
		{
			return SNullWidget::NullWidget;
		}
		
		// Index of the parameter within the array of material parameters of the same type
		if (InColumnName == MutableMaterialParameterColumns::ParameterIndexID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::AsNumber(RowItem->Index));
		}

		// Material Parameter Layer index
		if (InColumnName == MutableMaterialParameterColumns::ParameterLayerID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::AsNumber(RowItem->LayerIndex));
		}
		
		// Material Parameter Name
		if (InColumnName == MutableMaterialParameterColumns::ParameterNameID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::FromName(RowItem->ParameterName));
		}
		
		// OP ADDRESS
		if (InColumnName == MutableMaterialParameterColumns::ParameterImageAddressID)
		{
			if (const UE::Mutable::Private::FOperation::ADDRESS* ImageAddress = RowItem->Value.ImageParameter.TryGet<UE::Mutable::Private::FOperation::ADDRESS>())
			{
				return 
					SNew(STextBlock)
					.Text(FText::AsNumber((int32)*ImageAddress));
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		}
		
		
		// Try showing the evaluated Image. This will currently fail and produce a blank image
		if (InColumnName == MutableMaterialParameterColumns::ParameterImageID)
		{
			if (TSharedPtr<SMutableCodeViewer> PinnedCodeViewer = MutableCodeViewer.Pin())
			{
				if (const UE::Mutable::Private::FOperation::ADDRESS* ImageAddress = RowItem->Value.ImageParameter.TryGet<UE::Mutable::Private::FOperation::ADDRESS>())
				{
					UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FImage> GeneratedImage = PinnedCodeViewer->BuildImage(*ImageAddress);
					return 
						SNew(SMutableImageViewer)
						.Image(GeneratedImage);
				}
			}
			
			return SNullWidget::NullWidget;
		}

		// Invalid column name so no widget will be produced 
		checkNoEntry();
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FMutableMaterialImageParameterElement> RowItem;
	
	TWeakPtr<SMutableCodeViewer> MutableCodeViewer;
};


class SMutableMaterialColorParameterRow : public SMultiColumnTableRow<TSharedPtr<FMutableMaterialColorParameterElement>>
{
	public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView,
				   const TSharedPtr<FMutableMaterialColorParameterElement>& InRowItem)
	{
		RowItem = InRowItem;
		
		SMultiColumnTableRow<TSharedPtr<FMutableMaterialColorParameterElement>>::Construct(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (!RowItem)
		{
			return SNullWidget::NullWidget;
		}
	
		// Index of the parameter within the array of material parameters of the same type
		if (InColumnName == MutableMaterialParameterColumns::ParameterIndexID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::AsNumber(RowItem->Index));
		}
		
		// Material Parameter Layer index
		if (InColumnName == MutableMaterialParameterColumns::ParameterLayerID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::AsNumber(RowItem->LayerIndex));
		}
		
		// Material Parameter Name
		if (InColumnName == MutableMaterialParameterColumns::ParameterNameID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::FromName(RowItem->ParameterName));
		}

		// Generate the sub table here
		if (InColumnName == MutableMaterialParameterColumns::ParameterColorID)
		{
			TSharedRef<SMutableColorViewer> Viewer = SNew(SMutableColorViewer);
			Viewer->SetColor(RowItem->Value);
			return Viewer;
		}

		// Invalid column name so no widget will be produced 
		checkNoEntry();
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FMutableMaterialColorParameterElement> RowItem;
};


class SMutableMaterialScalarParameterRow : public SMultiColumnTableRow<TSharedPtr<FMutableMaterialScalarParameterElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView,
				   const TSharedPtr<FMutableMaterialScalarParameterElement>& InRowItem)
	{
		RowItem = InRowItem;
		
		SMultiColumnTableRow<TSharedPtr<FMutableMaterialScalarParameterElement>>::Construct(
	STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (!RowItem)
		{
			return SNullWidget::NullWidget;
		}
	
		// Index of the parameter within the array of material parameters of the same type
		if (InColumnName == MutableMaterialParameterColumns::ParameterIndexID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::AsNumber(RowItem->Index));
		}
		
		// Material Parameter Layer index
		if (InColumnName == MutableMaterialParameterColumns::ParameterLayerID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::AsNumber(RowItem->LayerIndex));
		}
		
		// Material Parameter Name
		if (InColumnName == MutableMaterialParameterColumns::ParameterNameID)
		{
			return 
				SNew(STextBlock)
				.Text(FText::FromName(RowItem->ParameterName));
		}

		// Generate the sub table here
		if (InColumnName == MutableMaterialParameterColumns::ParameterScalarID)
		{
			return SNew(STextBlock).Text(FText::AsNumber(RowItem->Value));
		}

		// Invalid column name so no widget will be produced 
		checkNoEntry();
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FMutableMaterialScalarParameterElement> RowItem;
};


void SMutableMaterialViewer::Construct(const FArguments& InArgs)
{
	MutableCodeViewer = InArgs._HostCodeViewer;
	
	const FText ParameterIndexNameLabel = FText::FromString(TEXT("Index"));
	const FText ParameterLayerNameLabel = FText::FromString(TEXT("Layer"));
	const FText ParameterNameLabel = FText::FromString(TEXT("Parameter"));
	
	ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew (SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(5)
				[
					// Container for the passthrough material preview
					SAssignNew(PreviewContainer, SBox)
				]

				+ SHorizontalBox::Slot()
				.Padding(5)
				[
					SNew(SVerticalBox)
					
					// Image count
					+ SVerticalBox::Slot()
					[
						SNew(STextBlock).
						Text(this, &SMutableMaterialViewer::GetImageParametersCountAsText)
					]
					// Color Count
					+ SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(this, &SMutableMaterialViewer::GetColorParametersCountAsText)
					]
					// Scalar Count
					+ SVerticalBox::Slot()
					[
						SNew(STextBlock)
						.Text(this, &SMutableMaterialViewer::GetScalarParametersCountAsText)
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.Padding(2,5)
		[
			SNew(SVerticalBox)

			// Image Parameters
			+ SVerticalBox::Slot()
			.Padding(5)
			[
				SAssignNew(ImageParameterListView, SListView<TSharedPtr<FMutableMaterialImageParameterElement>>)
				.ListItemsSource(&ImageParameters)
				.SelectionMode(ESelectionMode::None)
				.OnGenerateRow(this, &SMutableMaterialViewer::OnGenerateImageParameterRow)
				.HeaderRow
				(
					SNew(SHeaderRow)
		
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterIndexID)
						.DefaultLabel(ParameterIndexNameLabel)
						.FillWidth(0.2f)
						
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterLayerID)
						.DefaultLabel(ParameterLayerNameLabel)
						.FillWidth(0.2f)
					
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterNameID)
						.DefaultLabel(ParameterNameLabel)
						.FillWidth(0.2f)
				
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterImageAddressID)
						.DefaultLabel(FText::FromString(TEXT("OP_ADDRESS")))
						.FillWidth(0.15f)

					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterImageID)
						.DefaultLabel(FText::FromString(TEXT("FImage")))
						.FillWidth(0.45f)
				)
			]

			+ SVerticalBox::Slot()
			.Padding(5)
			[
				SAssignNew(ColorParameterListView, SListView<TSharedPtr<FMutableMaterialColorParameterElement>>)
				.ListItemsSource(&ColorParameters)
				.SelectionMode(ESelectionMode::None)
				.OnGenerateRow(this, &SMutableMaterialViewer::OnGenerateColorParameterRow)
				.HeaderRow
				(
					SNew(SHeaderRow)
		
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterIndexID)
						.DefaultLabel(ParameterIndexNameLabel)
						.FillWidth(0.2f)
	
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterLayerID)
						.DefaultLabel(ParameterLayerNameLabel)
						.FillWidth(0.2f)
						
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterNameID)
						.DefaultLabel(ParameterNameLabel)
						.FillWidth(0.2f)
				
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterColorID)
						.DefaultLabel(FText::FromString(TEXT("Color")))
						.FillWidth(0.6f)
				)
			]
			
			// Scalar Parameters
			+ SVerticalBox::Slot()
			.Padding(5)
			[
				SAssignNew(ScalarParameterListView, SListView<TSharedPtr<FMutableMaterialScalarParameterElement>>)
				.ListItemsSource(&ScalarParameters)
				.SelectionMode(ESelectionMode::None)
				.OnGenerateRow(this, &SMutableMaterialViewer::OnGenerateScalarParameterRow)
				.HeaderRow
				(
					SNew(SHeaderRow)
						
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterIndexID)
						.DefaultLabel(ParameterIndexNameLabel)
						.FillWidth(0.2f)
					
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterLayerID)
						.DefaultLabel(ParameterLayerNameLabel)
						.FillWidth(0.2f)
						
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterNameID)
						.DefaultLabel(ParameterNameLabel)
						.FillWidth(0.2f)
								
					+ SHeaderRow::Column(MutableMaterialParameterColumns::ParameterScalarID)
						.DefaultLabel(FText::FromString(TEXT("Scalar")))
						.FillWidth(0.6f)
				)
			]
		]
	];
	
	SetMaterial(InArgs._Material);
}


void SMutableMaterialViewer::SetMaterial(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>& InMaterial)
{
	if (InMaterial != HostedMaterial)
	{
		HostedMaterial = InMaterial;

		if (HostedMaterial)
		{
			// Image Parameters
			ImageParameters.Reset(HostedMaterial->ImageParameters.Num());
			int32 ImageParameterIndex = 0;
			for (const TTuple<UE::Mutable::Private::FParameterKey, UE::Mutable::Private::FMaterial::FImageParameterData>& ImageParameterTuple : HostedMaterial->ImageParameters)
			{
				TSharedPtr<FMutableMaterialImageParameterElement> ImageParameter = MakeShared<FMutableMaterialImageParameterElement>();
				ImageParameter->Index = ImageParameterIndex++;
				ImageParameter->ParameterName = ImageParameterTuple.Key.ParameterName;
				ImageParameter->LayerIndex = (int32)ImageParameterTuple.Key.LayerIndex;
				ImageParameter->Value = ImageParameterTuple.Value;
				ImageParameters.Add(ImageParameter);
			}
			
			// Color Parameters
			int32 ColorParameterIndex = 0;
			ColorParameters.Reset(HostedMaterial->ColorParameters.Num());
			for (const TTuple<UE::Mutable::Private::FParameterKey, FVector4f>& ColorParameterTuple : HostedMaterial->ColorParameters)
			{
				TSharedPtr<FMutableMaterialColorParameterElement> ColorParameter = MakeShared<FMutableMaterialColorParameterElement>();
				ColorParameter->Index = ColorParameterIndex++;
				ColorParameter->ParameterName = ColorParameterTuple.Key.ParameterName;
				ColorParameter->LayerIndex = (int32)ColorParameterTuple.Key.LayerIndex;
				ColorParameter->Value = ColorParameterTuple.Value;
				ColorParameters.Add(ColorParameter);
			}
			
			// Scalar Parameters
			ScalarParameters.Reset(HostedMaterial->ScalarParameters.Num());
			int32 ScalarParameterIndex = 0;
			for (const TTuple<UE::Mutable::Private::FParameterKey, float>& ScalarParameterTuple : HostedMaterial->ScalarParameters)
			{
				TSharedPtr<FMutableMaterialScalarParameterElement> ScalarParameter = MakeShared<FMutableMaterialScalarParameterElement>();
				ScalarParameter->Index = ScalarParameterIndex++;
				ScalarParameter->ParameterName = ScalarParameterTuple.Key.ParameterName;
				ScalarParameter->LayerIndex = (int32)ScalarParameterTuple.Key.LayerIndex;
				ScalarParameter->Value = ScalarParameterTuple.Value;
				ScalarParameters.Add(ScalarParameter);
			}
		}
		else
		{
			ImageParameters.Reset();
			ColorParameters.Reset();
			ScalarParameters.Reset();
		}
		
		// Now that the Material has changed refresh the preview of it
		check(PreviewContainer)
		PreviewContainer->SetContent(GeneratePassthroughMaterialWidget());
		
		// Request the refreshing of the list views so they show up-to-date data
		ImageParameterListView->RequestListRefresh();
		ColorParameterListView->RequestListRefresh();
		ScalarParameterListView->RequestListRefresh();
	}
}


TSharedRef<SWidget> SMutableMaterialViewer::GeneratePassthroughMaterialWidget()
{
	if (HostedMaterial)
	{
		// Try to get the actual object and if null then display the ID as fallback
		if (HostedMaterial->PassthroughObject.IsResolved())
		{
			if (const UMaterialInterface* Material = HostedMaterial->PassthroughObject.Get())
			{
				return SNew(SHyperlink)
					.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
					.Text(FText::FromString(Material->GetName()))
					.OnNavigate(this, &SMutableMaterialViewer::OnMaterialNavigation);
			}
		}
		else
		{
			FText MaterialIDText;
				
			UE::Mutable::Private::PASSTHROUGH_ID PassthroughMaterialID = HostedMaterial->PassthroughObject.GetId();
			if (PassthroughMaterialID != UE::Mutable::Private::PASSTHROUGH_ID_INVALID)
			{
				MaterialIDText = FText::AsNumber(PassthroughMaterialID);
			}
			else
			{
				MaterialIDText = FText::FromString(TEXT("INVALID_ID"));
			}
				
			return SNew(STextBlock).Text(MaterialIDText);
		}
	}

	return SNew(STextBlock).Text(FText::FromString("No FMaterial has been provided!"));
}


void SMutableMaterialViewer::OnMaterialNavigation() const
{
	if (HostedMaterial)
	{
		if (HostedMaterial && HostedMaterial->PassthroughObject.IsResolved())
		{
			const UMaterialInterface* Material = HostedMaterial->PassthroughObject.Get();
			check(Material);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Material);
		}
	}
}


FText SMutableMaterialViewer::GetImageParametersCountAsText() const
{
	const FString ImageParametersCountTitleLabel {TEXT("Image Parameter Count : ")};
	const FString ImageParametersAsNumber = FString::FormatAsNumber(ImageParameters.Num());
	return FText::FromString(ImageParametersCountTitleLabel + ImageParametersAsNumber);
}


FText SMutableMaterialViewer::GetColorParametersCountAsText() const
{
	const FString ColorParametersCountTitleLabel {TEXT("Color Parameter Count : ")};
	const FString ColorParametersAsNumber = FString::FormatAsNumber(ColorParameters.Num());
	return FText::FromString(ColorParametersCountTitleLabel + ColorParametersAsNumber);
}


FText SMutableMaterialViewer::GetScalarParametersCountAsText() const
{
	const FString ScalarParametersCountTitleLabel {TEXT("Scalar Parameter Count : ")};
	const FString ScalarParametersAsNumber = FString::FormatAsNumber(ScalarParameters.Num());
	return FText::FromString(ScalarParametersCountTitleLabel + ScalarParametersAsNumber);
}


TSharedRef<ITableRow> SMutableMaterialViewer::OnGenerateImageParameterRow(TSharedPtr<FMutableMaterialImageParameterElement> MutableMaterialImageParameterElement,
	const TSharedRef<STableViewBase>& Shared) const
{
	TSharedRef<SMutableMaterialImageParameterRow> Row = SNew(SMutableMaterialImageParameterRow, Shared, MutableMaterialImageParameterElement, MutableCodeViewer);
	return Row;
}


TSharedRef<ITableRow> SMutableMaterialViewer::OnGenerateColorParameterRow(TSharedPtr<FMutableMaterialColorParameterElement> MutableMaterialColorParameterElement,
	const TSharedRef<STableViewBase>& Shared) const
{
	TSharedRef<SMutableMaterialColorParameterRow> Row = SNew(SMutableMaterialColorParameterRow, Shared, MutableMaterialColorParameterElement);
	return Row;
}


TSharedRef<ITableRow> SMutableMaterialViewer::OnGenerateScalarParameterRow(TSharedPtr<FMutableMaterialScalarParameterElement> MutableMaterialScalarParameterElement,
	const TSharedRef<STableViewBase>& Shared) const
{
	TSharedRef<SMutableMaterialScalarParameterRow> Row = SNew(SMutableMaterialScalarParameterRow, Shared, MutableMaterialScalarParameterElement);
	return Row;
}

#undef LOCTEXT_NAMESPACE
