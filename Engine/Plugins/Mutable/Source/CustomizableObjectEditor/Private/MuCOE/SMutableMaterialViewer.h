// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SMutableCodeViewer.h"
#include "Widgets/SCompoundWidget.h"
#include "MuR/Material.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

// Base for all parameter types
struct FMutableMaterialParameterListElement
{
	FName ParameterName;
	int32 Index = -1;
	int32 LayerIndex = -1;
};


struct FMutableMaterialImageParameterElement : public FMutableMaterialParameterListElement
{
	UE::Mutable::Private::FMaterial::FImageParameterData Value;		// !
};


struct FMutableMaterialColorParameterElement : public FMutableMaterialParameterListElement
{
	FVector4f Value;
};


struct FMutableMaterialScalarParameterElement : public FMutableMaterialParameterListElement
{
	float Value;
};




/** Widget designed to show the statistical data from a Mutable FMaterial*/
class SMutableMaterialViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableMaterialViewer) {}
		SLATE_ARGUMENT_DEFAULT(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>, Material){nullptr};
		SLATE_ARGUMENT_DEFAULT(TWeakPtr<SMutableCodeViewer>, HostCodeViewer) {nullptr};
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Material to be used for this widget */
	void SetMaterial(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial>& InMaterial);

private:
	
	TSharedRef<SWidget> GeneratePassthroughMaterialWidget();
	
	// callbacks

	/**
	 * Callback method called when the material link is pressed. Will open the editor for the provided material
	 */
	void OnMaterialNavigation() const;
	
	FText GetImageParametersCountAsText() const;
	FText GetColorParametersCountAsText() const;
	FText GetScalarParametersCountAsText() const;
	
	TSharedRef<ITableRow> OnGenerateImageParameterRow(TSharedPtr<FMutableMaterialImageParameterElement> MutableMaterialImageParameterElement, const TSharedRef<STableViewBase>& Shared) const;
	TSharedRef<ITableRow> OnGenerateColorParameterRow(TSharedPtr<FMutableMaterialColorParameterElement> MutableMaterialColorParameterElement, const TSharedRef<STableViewBase>& Shared) const;
	TSharedRef<ITableRow> OnGenerateScalarParameterRow(TSharedPtr<FMutableMaterialScalarParameterElement> MutableMaterialScalarParameterElement, const TSharedRef<STableViewBase>& Shared) const;
	
	// The material we are showing some data of
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMaterial> HostedMaterial;
	
	TWeakPtr<SMutableCodeViewer> MutableCodeViewer;
	
	TSharedPtr<SBox> PreviewContainer;
	
	// Backend of the parameter list views. Extracted from the provided Hosted Material
	TArray<TSharedPtr<FMutableMaterialImageParameterElement>> ImageParameters;
	TArray<TSharedPtr<FMutableMaterialColorParameterElement>> ColorParameters;
	TArray<TSharedPtr<FMutableMaterialScalarParameterElement>> ScalarParameters;
	
	// Lists
	TSharedPtr<SListView<TSharedPtr<FMutableMaterialImageParameterElement>>> ImageParameterListView;
	TSharedPtr<SListView<TSharedPtr<FMutableMaterialColorParameterElement>>> ColorParameterListView;
	TSharedPtr<SListView<TSharedPtr<FMutableMaterialScalarParameterElement>>> ScalarParameterListView;
};
