// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableMeshViewer.h"

#include "Engine/SkeletalMesh.h"
#include "Framework/Views/TableViewMetadata.h"
#include "MuCOE/SMutableMeshViewport.h"
#include "MuCOE/SMutableSkeletonViewer.h"
#include "MuCO/CustomizableObject.h"
#include "MuT/TypeInfo.h"
#include "MuR/MeshPrivate.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class SWidget;
class USkeleton;

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


/** Namespace containing the IDs for the header on the buffer's channels list */
namespace MutableBufferChannelsListColumns
{
	static const FName ChannelSemanticIndexColumnID("Channel Semantic Index");
	static const FName ChannelSemanticColumnID("Channel Semantic");
	static const FName ChannelFormatColumnID("Format");
	static const FName ChannelComponentCountID("Components");
};

/* Row element generated on the buffers list. It represents the UI side of the Buffers data**/
class SMutableMeshBufferChannelListRow : public SMultiColumnTableRow<TSharedPtr<FBufferChannelElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView,
	               const TSharedPtr<FBufferChannelElement>& InRowItem)
	{
		RowItem = InRowItem;
		
		SMultiColumnTableRow<TSharedPtr<FBufferChannelElement>>::Construct(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}


	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		// Column with the index for the buffer .
		// Useful for knowing the channels on what buffer reside
		if (InColumnName == MutableBufferChannelsListColumns::ChannelSemanticIndexColumnID)
		{
			return SNew(SHorizontalBox) +
				SHorizontalBox::Slot()
				.Padding(4,0)
			[
				SNew(STextBlock).
				Text(RowItem->SemanticIndex)
			];
		}

		// Column with the name for the buffer (Semantic of the buffer)
		if (InColumnName == MutableBufferChannelsListColumns::ChannelSemanticColumnID)
		{
			return SNew(SHorizontalBox) +
				SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(RowItem->BufferSemantic)
			];
		}

		// Column with the format of the buffer
		if (InColumnName == MutableBufferChannelsListColumns::ChannelFormatColumnID)
		{
			return SNew(SHorizontalBox) + SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(RowItem->BufferFormat)
			];
		}

		// Column with Buffer component count 
		if (InColumnName == MutableBufferChannelsListColumns::ChannelComponentCountID)
		{
			return SNew(SHorizontalBox) + SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(RowItem->BufferComponentCount)
			];
		}

		// Invalid column name so no widget will be produced 
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FBufferChannelElement> RowItem;
};


/** Namespace containing the IDs for the header on the buffers list */
namespace MutableMeshBuffersListColumns
{
	static const FName BufferIndexColumnID("Buffer Index");
	static const FName BufferChannelsColumnID("Channels");
}


class SMutableMeshBufferListRow : public SMultiColumnTableRow<TSharedPtr<FBufferElement>>
{
public:
	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView,
	               const TSharedPtr<FBufferElement>& InRowItem, TSharedPtr<SMutableMeshViewer> InHost)
	{
		HostMutableMeshViewer = InHost;
		RowItem = InRowItem;
		
		SMultiColumnTableRow<TSharedPtr<FBufferElement>>::Construct(
	STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		// Column with the index for the buffer .
		// Useful for knowing the channels on what buffer reside
		if (InColumnName == MutableMeshBuffersListColumns::BufferIndexColumnID)
		{
			return SNew(SBorder)
			.Content()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock).
					Text(RowItem->BufferIndex)
				]
			];
		}

		// Generate the sub table here
		if (InColumnName == MutableMeshBuffersListColumns::BufferChannelsColumnID)
		{
			const TSharedRef<SWidget> GeneratedChannelList =
				HostMutableMeshViewer->GenerateBufferChannelsListView(RowItem->BufferChannels);
			
			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				GeneratedChannelList
			];
		}

		// Invalid column name so no widget will be produced 
		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FBufferElement> RowItem;
	TSharedPtr<SMutableMeshViewer> HostMutableMeshViewer;
};


void SMutableMeshViewer::Construct(const FArguments& InArgs)
{
	// Splitter values
	constexpr float TablesSplitterValue = 0.5f;
	constexpr float ViewportSplitterValue = 0.5f;

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+ SSplitter::Slot()
		.Value(TablesSplitterValue)
		[
			GenerateDataTableSlates()
		]

		+ SSplitter::Slot()
		.Value(ViewportSplitterValue)
		[
			GenerateViewportSlates()
		]
	];

	// If a mesh has been provided then do set the mesh for this object
	if (InArgs._Mesh)
	{
		SetMesh(InArgs._Mesh);
	}
}


void SMutableMeshViewer::SetMesh(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& InMesh)
{
	if (InMesh != MutableMesh)
	{
		MutableMesh = InMesh;
		
		if (MutableMesh)
		{
			OnMeshChanged();

			// Skeleton slate viewer update
			if (UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> MutableSkeleton = MutableMesh->GetSkeleton())
			{
				MutableSkeletonViewer->SetSkeleton(MutableSkeleton);

				MutableSkeletonViewer->SetVisibility(EVisibility::Visible);
			}
			else
			{
				MutableSkeletonViewer->SetVisibility(EVisibility::Hidden);
			}

			FString DebugLog;
			MutableMesh->Log(DebugLog, 8);
			UE_LOGF(LogMutable, Warning, "[%ls]", *DebugLog);
		}

		MeshViewport->SetMesh(MutableMesh);
	}
}

TSharedRef<SWidget> SMutableMeshViewer::GenerateViewportSlates()
{
	TSharedRef<SWidget> Container = SNew(SVerticalBox)

		// User warning messages 
		// + SVerticalBox::Slot()
		// .AutoHeight()
		// [
		// 	// TODO: Add a message to tell the user why no mesh is being displayed
		// 	SNew(SWarningOrErrorBox)
		// 	.MessageStyle(EMessageStyle::Warning)
		// 	.Message(FText::FromString(FString("This is just a simulation")))
		// ]

		// Mesh Drawing space
		+ SVerticalBox::Slot()
		[
			SAssignNew(MeshViewport, SMutableMeshViewport)
			.Mesh(MutableMesh)
		];


	return Container;
}


TSharedRef<SWidget> SMutableMeshViewer::GenerateDataTableSlates()
{
	// Formatting
	constexpr int32 IndentationSpace = 16;
	constexpr int32 SimpleSpacing = 1;

	constexpr int32 AfterTitleSpacing = 4;
	constexpr int32 EndOfSectionSpacing = 12;

	// Naming
	const FText GeneralDataTitle = LOCTEXT("GeneralDataTitle", "General Data");
	const FText VerticesCountTitle = LOCTEXT("VerticesCountTitle", "Vertex count : ");
	const FText FacesCountTitle = LOCTEXT("FacesCountTitle", "Face count : ");
	const FText BonesCountTitle = LOCTEXT("BonesCountTitle", "Bone count : ");
	const FText MeshIdPrefixTitle = LOCTEXT("MeshIdPrefixTitle", "Mesh ID prefix : ");
	const FText MeshFlagsTitle = LOCTEXT("MeshFlagsTitle", "Mesh flags : ");
	const FText BuffersTitle = LOCTEXT("BuffersTitle", "Buffers");

	return SNew(SSplitter)
	+ SSplitter::Slot()
	[
		 SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)

			// General data ----------------------------------------------------------------
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock).
				Text(GeneralDataTitle)
			]

			+ SVerticalBox::Slot().
			  Padding(IndentationSpace, AfterTitleSpacing).
			  AutoHeight()
			[
				SNew(SVerticalBox)

				// Vertices
				+ SVerticalBox::Slot().
				  Padding(0, SimpleSpacing).
				  AutoHeight()
				[
					SNew(SHorizontalBox)

					// Vertices title
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(VerticesCountTitle)
					]

					// Vertices value
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableMeshViewer::GetVertexCount)
					]

				]

				// Faces
				+ SVerticalBox::Slot().
				  Padding(0, SimpleSpacing).
				  AutoHeight()
				[
					SNew(SHorizontalBox)

					// Faces title
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(FacesCountTitle)
					]

					// Faces Value
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableMeshViewer::GetFaceCount)
					]
				]

				// Bones
				+ SVerticalBox::Slot().
				  Padding(0, SimpleSpacing).
				  AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(BonesCountTitle)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock).
						Text(this, &SMutableMeshViewer::GetBoneCount)
					]
				]

				// Vertex ID
				+ SVerticalBox::Slot().
				Padding(0, SimpleSpacing).
				AutoHeight()
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock).
								Text(MeshIdPrefixTitle)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock).
								Text(this, &SMutableMeshViewer::GetMeshIdPrefix)
						]
				]

				// Flags
				+ SVerticalBox::Slot().
				Padding(0, SimpleSpacing).
				AutoHeight()
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock).
								Text(MeshFlagsTitle)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock).
								Text(this, &SMutableMeshViewer::GetMeshFlags)
						]
				]
			]

			// Buffers Data --------------------------------------------------------------
			+ SVerticalBox::Slot()
			  .Padding(0, EndOfSectionSpacing)
			  .AutoHeight()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot().
				AutoHeight()
				[
					// Buffers data Title
					SNew(STextBlock).
					Text(BuffersTitle)
				]

				+ SVerticalBox::Slot()
				  .Padding(IndentationSpace, AfterTitleSpacing)
				  .AutoHeight()
				[
					SNew(SVerticalBox)

					// List of vertex buffers ----------
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						GenerateBuffersListView(
							VertexBuffersSlateView,
							VertexBuffers,
							FText(LOCTEXT("VertexBufferType", "Vertex")))
					]
					// ---------------------------------

				]

				+ SVerticalBox::Slot()
				  .Padding(IndentationSpace, 6)
				  .AutoHeight()
				[
					SNew(SVerticalBox)
					
					// List of Index buffers ----------
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						GenerateBuffersListView(
							IndexBuffersSlateView,
							IndexBuffers,
							FText(LOCTEXT("IndexBufferType", "Index")))
					]
					// ---------------------------------
				]
				
			]

			// Bones data ----------------------------------------------------------------
			+ SVerticalBox::Slot()
			  .Padding(0, EndOfSectionSpacing)
			  .AutoHeight()
			[
				SAssignNew(MutableSkeletonViewer, SMutableSkeletonViewer)
			]
			// ---------------------------------
		]
	]
	
	+ SSplitter::Slot()
	[
		SNew(SScrollBox)
	   + SScrollBox::Slot()
		[
			SNew(SVerticalBox)

			// UVs
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock).
					Text(LOCTEXT("UVs", "UVs"))
			]

			+ SVerticalBox::Slot()
			.Padding(IndentationSpace, AfterTitleSpacing)
			.AutoHeight()
			.MinHeight(300.0f)
			[
				SAssignNew(LayoutGridWidget, SCustomizableObjectLayoutGrid)
					.Mode(ELGM_Show)
					.GridSize(this, &SMutableMeshViewer::GetGridSize)
					.Blocks(this, &SMutableMeshViewer::GetBlocks)
					.UVLayout(this, &SMutableMeshViewer::GetUVs)
			]
			// ---------------------------------
		]
	];
}


FIntPoint SMutableMeshViewer::GetGridSize() const
{
	// TODO
	return FIntPoint(8,8);
}


TArray<FCustomizableObjectLayoutBlock> SMutableMeshViewer::GetBlocks() const
{
	TArray<FCustomizableObjectLayoutBlock> Blocks;

	// TODO
	return Blocks;
}


TArray<FVector2f> SMutableMeshViewer::GetUVs() const
{
	TArray<FVector2f> Lines;

	if (MutableMesh && !MutableMesh->VertexBuffers.IsDescriptor())
	{
		int32 NumFaces = MutableMesh->GetFaceCount();
		Lines.Reserve(NumFaces*6);

		// TODO
		int32 UVIndex = 0;

		UE::Mutable::Private::UntypedMeshBufferIteratorConst ItIndices(MutableMesh->IndexBuffers, UE::Mutable::Private::EMeshBufferSemantic::VertexIndex, 0);
		UE::Mutable::Private::UntypedMeshBufferIteratorConst It(MutableMesh->VertexBuffers, UE::Mutable::Private::EMeshBufferSemantic::TexCoords, UVIndex);
		for (int32 FaceIndex=0; FaceIndex<NumFaces; ++FaceIndex)
		{
			uint32 V0 = (ItIndices + FaceIndex * 3 + 0).GetAsUINT32();
			uint32 V1 = (ItIndices + FaceIndex * 3 + 1).GetAsUINT32();
			uint32 V2 = (ItIndices + FaceIndex * 3 + 2).GetAsUINT32();

			Lines.Add((It + V0).GetAsVec2f());
			Lines.Add((It + V1).GetAsVec2f());

			Lines.Add((It + V1).GetAsVec2f());
			Lines.Add((It + V2).GetAsVec2f());

			Lines.Add((It + V2).GetAsVec2f());
			Lines.Add((It + V0).GetAsVec2f());
		}
	}

	return Lines;
}


TSharedRef<ITableRow> SMutableMeshViewer::GenerateTagRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FName>>, OwnerTable)
		[
			SNew(STextBlock).Text(FText::FromString(*InItem))
		];
}


TSharedRef<SWidget> SMutableMeshViewer::GenerateBuffersListView(
	TSharedPtr<SListView<TSharedPtr<FBufferElement>>>& OutHostListView,
	const TArray<TSharedPtr<FBufferElement>>& InBufferElements,
	const FText& InBufferSetTypeName)
{
	// Headers
	const FText BufferIndexTitle = FText(LOCTEXT("BufferIndexTitle", "Buffer"));
	const FText BufferChannelsTitle = FText::Format(
		LOCTEXT("NumberOfBufferChannels", "{0} Buffer Channels"), InBufferSetTypeName);

	// Tooltips
	const FText BufferIndexTooltip = FText(LOCTEXT("BufferIndexTooltip",
	                                               "Represents the index where the mutable buffer is found inside the buffer set"));
	const FText BufferChannelsTooltip = FText(LOCTEXT("BufferChannelsTooltip",
												   "The channels contained inside each mutable buffer."));
	
	return SAssignNew(OutHostListView, SListView<TSharedPtr<FBufferElement>>)
		.ListItemsSource(&InBufferElements)
		.OnGenerateRow(this, &SMutableMeshViewer::OnGenerateBufferRow)
		.SelectionMode(ESelectionMode::None)
		.HeaderRow
		(
			SNew(SHeaderRow)
		
			+ SHeaderRow::Column(MutableMeshBuffersListColumns::BufferIndexColumnID)
				.DefaultTooltip(BufferIndexTooltip)
				.DefaultLabel(BufferIndexTitle)
				.FillWidth(0.1f)
		
			+ SHeaderRow::Column(MutableMeshBuffersListColumns::BufferChannelsColumnID)
				.DefaultTooltip(BufferChannelsTooltip)
				.DefaultLabel(BufferChannelsTitle)
			  	.FillWidth(0.9f)
		);
}


TSharedRef<SWidget> SMutableMeshViewer::GenerateBufferChannelsListView(
	const TSharedPtr<TArray<TSharedPtr<FBufferChannelElement>>>& InBufferChannelElements)
{
	// Headers
	const FText ChannelIndex = FText(LOCTEXT("ChannelIndexTitle", "Index"));
	const FText ChannelSemanticTitle = FText(LOCTEXT("SemanticLabelTitle", "Semantic"));
	const FText ChannelFormatTitle = FText(LOCTEXT("FormatLabelTitle", "Format"));
	const FText ComponentCountTitle = FText(LOCTEXT("ComponentCountLabelTitle", "Components"));

	// Tooltips
	const FText ChannelIndexTooltip = FText(LOCTEXT("ChannelIndexTooltip","Represents the SemanticIndex of the mutable channel inside the whole buffer set. Usefull when more than one channel does share the same type."));
	const FText ChannelSemanticTooltip = FText(LOCTEXT("ChannelSemanticTooltip","The semantic that identifies this channel."));
	const FText ChannelFormatTooltip = FText(LOCTEXT("ChannelFormatTooltip","The format of the data being held."));
	const FText ComponentCountTooltip = FText(LOCTEXT("ChannelComponentTooltip","The amount of components each unit of data has."));
	
	return SNew(SListView<TSharedPtr<FBufferChannelElement>>)
		.ListItemsSource(InBufferChannelElements.Get())
		.OnGenerateRow(this, &SMutableMeshViewer::OnGenerateBufferChannelRow)
		.SelectionMode(ESelectionMode::None)
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column(MutableBufferChannelsListColumns::ChannelSemanticIndexColumnID)
				.DefaultTooltip(ChannelIndexTooltip)
				.DefaultLabel(ChannelIndex)
				.FillWidth(0.14f)
			
			+ SHeaderRow::Column(MutableBufferChannelsListColumns::ChannelSemanticColumnID)
				.DefaultTooltip(ChannelSemanticTooltip)
				.DefaultLabel(ChannelSemanticTitle)
				.FillWidth(0.35f)
		
			+ SHeaderRow::Column(MutableBufferChannelsListColumns::ChannelFormatColumnID)
				.DefaultTooltip(ChannelFormatTooltip)
			  	.DefaultLabel(ChannelFormatTitle)
			  	.FillWidth(0.65f)
		
			+ SHeaderRow::Column(MutableBufferChannelsListColumns::ChannelComponentCountID)
				.DefaultTooltip(ComponentCountTooltip)
			  	.DefaultLabel(ComponentCountTitle)
			  	.FillWidth(0.3f)
		);
}


void SMutableMeshViewer::OnMeshChanged()
{
	// Cache the data accessible from the UE::Mutable::Private::FMesh to be later used by the UI
	FillTargetBufferSetDataArray(MutableMesh->GetVertexBuffers(), VertexBuffers, VertexBuffersSlateView);
	FillTargetBufferSetDataArray(MutableMesh->GetIndexBuffers(), IndexBuffers, IndexBuffersSlateView);

	// Restore the widths of the columns each time the mesh gets changed.
	VertexBuffersSlateView->GetHeaderRow()->ResetColumnWidths();
	IndexBuffersSlateView->GetHeaderRow()->ResetColumnWidths();
}


void SMutableMeshViewer::FillTargetBufferSetDataArray(const UE::Mutable::Private::FMeshBufferSet& InMuMeshBufferSet,
                                                      TArray<TSharedPtr<FBufferElement>>& OutBuffersDataArray,
                                                      TSharedPtr<SListView<TSharedPtr<FBufferElement>>>& InHostListView)
{
	// Make sure no data is left from previous runs
	OutBuffersDataArray.Empty();

	// Iterate over the buffers and get the type and semantic
	const int32 BuffersCount = InMuMeshBufferSet.GetBufferCount();
	for (int32 BufferIndexOnBufferSet = 0; BufferIndexOnBufferSet < BuffersCount; BufferIndexOnBufferSet++)
	{
		// Object tasked with the displaying of all the channels it is made with
		TSharedPtr<FBufferElement> NewBufferDefinition = MakeShareable(new FBufferElement());

		// Array with the data from all channels found inside the current buffer object
		TSharedPtr<TArray<TSharedPtr<FBufferChannelElement>>> ChannelsArray =
			MakeShareable(new TArray<TSharedPtr<FBufferChannelElement>>);
		
		// Get the channels the buffer has 
		const int32 ChannelsOnBuffer = InMuMeshBufferSet.GetBufferChannelCount(BufferIndexOnBufferSet);
		if (ChannelsOnBuffer == 0)
		{
			// Add a new row telling the user no channels are set on the buffer
			const TSharedPtr<FBufferChannelElement> NewChannelDefinition = MakeShareable(new FBufferChannelElement());
			NewChannelDefinition->BufferSemantic = FText(INVTEXT("No Channels found..."));
			ChannelsArray->Add(NewChannelDefinition);

			NewBufferDefinition->BufferChannels = ChannelsArray;
			
			continue;
		}

		// Load all channels found onto our array of channels to be later displayed by the UI
		for (int32 ChannelIndexOnBuffer = 0; ChannelIndexOnBuffer < ChannelsOnBuffer; ChannelIndexOnBuffer++)
		{
			TSharedPtr<FBufferChannelElement> NewChannelDefinition = MakeShareable(new FBufferChannelElement());

			UE::Mutable::Private::EMeshBufferSemantic BufferChannelSemantic = UE::Mutable::Private::EMeshBufferSemantic::None;
			UE::Mutable::Private::EMeshBufferFormat BufferFormat = UE::Mutable::Private::EMeshBufferFormat::None;
			int32 BufferComponentCount = 0;
			int32 SemanticIndex = -1;

			// Get the data from mutable that we require from the selected buffer set buffer
			InMuMeshBufferSet.GetChannel
			(
				BufferIndexOnBufferSet,
				ChannelIndexOnBuffer,
				&(BufferChannelSemantic),
				&(SemanticIndex),
				&(BufferFormat),
				&(BufferComponentCount),
				nullptr
			);

			// In order to get the indexes for the UI display, cast the value of the enumeration to int
			const int32 SemanticTypeIndex = static_cast<int32>(BufferChannelSemantic);
			const int32 BufferFormatIndex = static_cast<int32>(BufferFormat);

			// Using UE::Mutable::Private::TypeInfo find what is the name of the buffer semantic and the buffer format
			NewChannelDefinition->SemanticIndex = FText::AsNumber(SemanticIndex);
			NewChannelDefinition->BufferSemantic = FText::FromString(
				*FString(UE::Mutable::Private::TypeInfo::s_meshBufferSemanticName[SemanticTypeIndex]));
			NewChannelDefinition->BufferFormat = FText::FromString(
				*FString(UE::Mutable::Private::TypeInfo::s_meshBufferFormatName[BufferFormatIndex]));
			NewChannelDefinition->BufferComponentCount = FText::FromString(*FString::FromInt(BufferComponentCount));

			ChannelsArray->Add(NewChannelDefinition);
		}

		NewBufferDefinition->BufferIndex = FText::AsNumber( BufferIndexOnBufferSet);
		NewBufferDefinition->BufferChannels = ChannelsArray;
		OutBuffersDataArray.Add(NewBufferDefinition);
	}

	// If no data has been found ad an element to show it
	if (OutBuffersDataArray.IsEmpty())
	{
		const TSharedPtr<FBufferElement> NewBufferDefinition = MakeShareable(new FBufferElement());
		NewBufferDefinition->BufferIndex = FText(INVTEXT("N/A"));

		const TSharedPtr<FBufferChannelElement> NewChannelDefinition = MakeShareable(new FBufferChannelElement());
		NewChannelDefinition->BufferSemantic = FText(INVTEXT("No buffers found..."));
		
		TSharedPtr<TArray<TSharedPtr<FBufferChannelElement>>> ChannelsArray =
			MakeShareable(new TArray<TSharedPtr<FBufferChannelElement>>);
		ChannelsArray->Add(NewChannelDefinition);
		
		NewBufferDefinition->BufferChannels = ChannelsArray;
		OutBuffersDataArray.Add(NewBufferDefinition);
	}

	// Make sure the list gets refreshed with the new contents
	InHostListView->RequestListRefresh();
}



FText SMutableMeshViewer::GetVertexCount() const
{
	return FText::AsNumber(MutableMesh ? MutableMesh->GetVertexCount() : 0);
}

FText SMutableMeshViewer::GetFaceCount() const
{
	return FText::AsNumber(MutableMesh ? MutableMesh->GetFaceCount() : 0);
}

FText SMutableMeshViewer::GetBoneCount() const
{
	return FText::AsNumber(MutableMesh && MutableMesh->GetSkeleton() ? MutableMesh->GetSkeleton()->GetNumBones() : 0);
}

FText SMutableMeshViewer::GetMeshIdPrefix() const
{
	return FText::AsNumber(MutableMesh ? MutableMesh->MeshIDPrefix : 0);
}

FText SMutableMeshViewer::GetMeshFlags() const
{
	return FText::AsNumber(MutableMesh ? uint32(MutableMesh->Flags) : uint32(0));
}


TSharedRef<ITableRow> SMutableMeshViewer::OnGenerateBufferRow(TSharedPtr<FBufferElement, ESPMode::ThreadSafe> InBuffer,
                                                              const TSharedRef<STableViewBase, ESPMode::ThreadSafe>& OwnerTable)
{
	TSharedRef<SMutableMeshBufferListRow> Row = SNew(SMutableMeshBufferListRow, OwnerTable, InBuffer, SharedThis(this) );
	return Row;
}

TSharedRef<ITableRow> SMutableMeshViewer::OnGenerateBufferChannelRow(TSharedPtr<FBufferChannelElement> InBufferChannel,
                                                                     const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SMutableMeshBufferChannelListRow> Row = SNew(SMutableMeshBufferChannelListRow, OwnerTable, InBufferChannel);
	return Row;
}

#undef LOCTEXT_NAMESPACE
