// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Mesh.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "MuCOE/SCustomizableObjectLayoutGrid.h"

template <typename ItemType> class SListView;

class ITableRow;
class SBorder;
class SMutableMeshViewport;
class SMutableSkeletonViewer;
class SScrollBox;
class SSplitter;
class STableViewBase;
class SWidget;
class USkeletalMesh;
namespace UE::Mutable::Private { class FMeshBufferSet; }
struct FAssetData;


/** Container designed to hold the buffer channel data of a mutable mesh buffer to be later used by the UI */
struct FBufferChannelElement
{
	FText SemanticIndex;
	FText BufferSemantic;
	FText BufferFormat;
	FText BufferComponentCount;
};


/** Element representing a mutable buffer. It contains an array with all the elements representing the channels the mutable buffer is made of.*/
struct FBufferElement
{
	/** The index of the buffer on the origin mutable buffer set */
	FText BufferIndex;

	/** An array of BufferChannels that represent the relative mutable channels on the mutable buffer*/
	TSharedPtr<TArray<TSharedPtr<FBufferChannelElement>>> BufferChannels;
};

/** Widget designed to show the statistical data from a Mutable FMesh*/
class SMutableMeshViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableMeshViewer) {}
		SLATE_ARGUMENT_DEFAULT(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>, Mesh){nullptr};
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Mesh to be used for this widget */
	void SetMesh(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& InMesh);

	
	/** Generate a new SListView for all the ChannelElements provided.
	 * @param InBufferChannelElements - List with all the elements that will be ued to fill InHostListView
	 * @return A SListView containing all the channel elements as UI elements. 
	 */
	TSharedRef<SWidget> GenerateBufferChannelsListView(
		const TSharedPtr<TArray<TSharedPtr<FBufferChannelElement>>>& InBufferChannelElements);

private:
	
	/** Slate whose task is to display the skeleton found on this mesh as a slate tree view */
	TSharedPtr<SMutableSkeletonViewer> MutableSkeletonViewer;

	TSharedRef<ITableRow> GenerateTagRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Data backend for the widget. It represents the mesh that is being "displayed" */
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> MutableMesh = nullptr;

	/** Splitter used to separate the two sides of the slate (tables and viewport) */
	TSharedPtr<SSplitter> SpaceSplitter;

	/** Slate object containing all the buffer tables alongside with the bone tree */
	TSharedPtr<SScrollBox> DataSpaceSlate;

	/** Viewport object to preview the current mesh inside an actual Unreal scene */
	TSharedPtr<SMutableMeshViewport> MeshViewport;
	
	/*
	 * Slate views for the main types of mesh buffers (vertex , index and face)
	 * Each buffer element also contains the channels it uses
	 */
	
	TSharedPtr<SListView<TSharedPtr<FBufferElement>>> VertexBuffersSlateView;
	TSharedPtr<SListView<TSharedPtr<FBufferElement>>> IndexBuffersSlateView;

	/** Widget to show the UVs */
	TSharedPtr<SCustomizableObjectLayoutGrid> LayoutGridWidget;

private:

	/** Generates all slate objects related with the Mesh Viewport Slate */
	TSharedRef<SWidget> GenerateViewportSlates();

	/** Generates the tables showing the buffer data on the mesh alongside with the bone tree found on the mutable mesh */
	TSharedRef<SWidget> GenerateDataTableSlates();
	
	/** Method called each time the mesh selected changes so the UI gets updated reliably */
	void OnMeshChanged();

	/** UV widget callbacks. */
	FIntPoint GetGridSize() const;
	TArray<FCustomizableObjectLayoutBlock> GetBlocks() const;
	TArray<FVector2f> GetUVs() const;

private:
	
	/* Generic UI callbacks used by the Widget */
	FText GetVertexCount() const;
	FText GetFaceCount() const;
	FText GetBoneCount() const;
	FText GetMeshIdPrefix() const;
	FText GetMeshFlags() const;

	/*
	 * Elements used to feed the buffers list (index and buffer channels as an internal list)
	 */
	
	TArray<TSharedPtr<FBufferElement>> VertexBuffers;
	TArray<TSharedPtr<FBufferElement>> IndexBuffers;
	
	/** 
	* Fills the provided TArray with the buffer definitions generated from the mutable buffers found on the provided mutable buffer set
	* @param InMuMeshBufferSetPtr - Mutable buffer set pointer pointing to the buffer set witch buffer data is wanted to be saved on the provided TArray
	* @param OutBuffersDataArray - TArray witch will contain the parsed data from the provided mutable buffer set converted onto FBufferElements
	* @param InHostListView - The slate list using the previously referenced array as data backend. 
	*/
	void FillTargetBufferSetDataArray(const UE::Mutable::Private::FMeshBufferSet& InMuMeshBufferSetPtr,
	                                  TArray<TSharedPtr<FBufferElement>>& OutBuffersDataArray,
	                                  TSharedPtr<SListView<TSharedPtr<FBufferElement>>>& InHostListView);

	
	/**
	 * Provided an array of buffer elements and the type of buffer it generates a new SListView for said buffer elements
	 * @param InBufferElements - The FBufferElement objects to be used as backend for the SListView
	 * @param InBufferSetTypeName - The name of the buffer set you are generating. Some examples are "Vertex", "index" or "Face"
	 * @param OutHostListView - THe SListView generated.
	 * @return - A shared reference to the SListView generated. Required by this slate operation.
	 */
	TSharedRef<SWidget> GenerateBuffersListView(TSharedPtr<SListView<TSharedPtr<FBufferElement>>>& OutHostListView,
	                                          const TArray<TSharedPtr<FBufferElement>>& InBufferElements,
	                                          const FText& InBufferSetTypeName);
	
	/** Callback method invoked each time a new row of the SListView containing the buffer elements needs to be built
	 * @param InBuffer - The buffer element being used to build the new row.
	 * @param OwnerTable - The table that is going to get the new row added into it.
	 * @return The new row object as the base interface used for all the table rows
	 */
	TSharedRef<ITableRow> OnGenerateBufferRow(TSharedPtr<FBufferElement, ESPMode::ThreadSafe> InBuffer,
											  const TSharedRef<STableViewBase, ESPMode::ThreadSafe>& OwnerTable);
	
	/** 
	* Callback Method responsible of generating each row of the buffer channel lists based on the channel definition provided.
	* @param InBufferChannel - The mesh Buffer channel definition that is being used as blueprint for the actual UI row
	* @param OwnerTable - The parent UI element that is going to be expanded with the new row object
	* @return The new row object as the base interface used for all the table rows
	*/
	TSharedRef<ITableRow> OnGenerateBufferChannelRow(TSharedPtr<FBufferChannelElement> InBufferChannel,
	                                                 const TSharedRef<STableViewBase>& OwnerTable);
	
};
