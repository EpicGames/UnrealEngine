// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableMeshViewport.h"

#include "AdvancedPreviewScene.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor/UnrealEdTypes.h"
#include "Engine/SkeletalMesh.h"
#include "MuCOE/FMutableViewportClient.h"
#include "MuCOE/MutableMeshPreviewUtils.h"

class FEditorViewportClient;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SMutableMeshViewerViewport"


void SMutableMeshViewport::Construct(const FArguments& InArgs)
{
	// Construct the viewport object
	SEditorViewport::Construct(SEditorViewport::FArguments());

	// Set the mesh used by default if a mesh has been provided
	if (InArgs._Mesh)
	{
		SetMesh(InArgs._Mesh);
	}
	
}

void SMutableMeshViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SEditorViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SMutableMeshViewport::SetMesh(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& InMesh)
{
	if (InMesh == MutableMesh)
	{
		return;
	}

	// Update the viewport with the new mesh
	MutableMesh = InMesh;
	
	// Update viewport 
	RefreshViewportContents();
}

void SMutableMeshViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SkeletalMeshComponent);
}

TSharedRef<FEditorViewportClient> SMutableMeshViewport::MakeEditorViewportClient()
{
	// Initialize the preview scene being used by this viewport
	PreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));

	// Initialize the viewport client and define it's properties
	ViewportClient = MakeShareable(new FMutableMeshViewportClient(PreviewScene.ToSharedRef()));
	ViewportClient->ViewportType = LVT_Perspective;
	ViewportClient->bSetListenerPosition = false;
	ViewportClient->SetRealtime(true);		
	
	const FVector StaringLocation = FVector( 0.0f, 128.0f, 128.0f );
	ViewportClient->SetViewLocation(StaringLocation);

	const FRotator StartingRotation = FRotator( 0.0f, -90.0f, 0 );
	ViewportClient->SetViewRotation(StartingRotation);
	
	ViewportClient->SetCameraSpeedSettings(FEditorViewportCameraSpeedSettings(0.25f));
	
	return ViewportClient.ToSharedRef();
}

void SMutableMeshViewport::RefreshViewportContents()
{
	// Clear previous contents
	ClearViewport();
	// Generate all the viewport contents if a mutable mesh has been provided
	if(MutableMesh)
	{
		SendMeshToViewport();
	}
}

void SMutableMeshViewport::ClearViewport() const
{
	if (SkeletalMeshComponent)
	{
		// Only allow it for meshes with skeleton 
		PreviewScene->RemoveComponent(SkeletalMeshComponent);

		SkeletalMeshComponent->DestroyComponent();
	}
}

void SMutableMeshViewport::SendMeshToViewport()
{
	// Get the SkeletalMesh from the MutableMesh and if successful add the component to the viewport
	if (GenerateUnrealMesh())
	{
		// Provide the previewer with the mesh object
		PreviewScene->AddComponent(SkeletalMeshComponent,FTransform::Identity);
	}

}

bool SMutableMeshViewport::GenerateUnrealMesh()
{
	// Convert the provided mutable mesh to unreal with heavy aid of the Reference Skeletal Mesh
	USkeletalMesh* GeneratedSkeletalMesh = MutableMeshPreviewUtils::GenerateSkeletalMeshFromMutableMesh(MutableMesh.Get());
	
	if (!GeneratedSkeletalMesh)
	{
		return false;
	}
	
	// Create the new SkeletalMeshComponent and set the mesh to be the one we just created.
	SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	SkeletalMeshComponent->SetSkeletalMesh(GeneratedSkeletalMesh);
	
	return true;
} 

#undef LOCTEXT_NAMESPACE
