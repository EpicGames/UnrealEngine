// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Mesh.h"
#include "SEditorViewport.h"
#include "UObject/GCObject.h"

class FAdvancedPreviewScene;
class FEditorViewportClient;
class FMutableMeshViewportClient;
class FReferenceCollector;
class USkeletalMesh;
class USkeletalMeshComponent;
struct FGeometry;


/**
* Object with the objective of showing a preview of a selected mutable mesh. It is designed to have few dependencies
* as possible
*/
class SMutableMeshViewport : public SEditorViewport , public FGCObject
{
	SLATE_BEGIN_ARGS(SMutableMeshViewport) {}
    	SLATE_ARGUMENT_DEFAULT(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>, Mesh) { nullptr };
    SLATE_END_ARGS()
    
public:
    /** Builds the widget */
    void Construct(const FArguments& InArgs);
	
    /** Set the Mutable Mesh to be used for this widget
     * @param InMesh - The mutable mesh to be displayed
     */
    void SetMesh(const UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh>& InMesh);

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SMutableMeshViewport");
	}
	
    // SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	// SEditorViewport interface (called by SEditorViewport::Construct)
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;

private:

	/** The mutable mesh being displayed */
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FMesh> MutableMesh = nullptr;

	/*
	 *	UI  
	 */
	
	/** The preview scene that we are viewing */
	TSharedPtr<FAdvancedPreviewScene> PreviewScene = nullptr;

	/** Editor viewport client */
	TSharedPtr<FMutableMeshViewportClient> ViewportClient = nullptr;

	/** Restores the default state for the viewport */
	void ClearViewport() const;

	/** Adds the required content to the viewport*/
	void SendMeshToViewport();

	/** Clears or displays a mesh depending on the presence of a mutable mesh on MutableMesh**/
	void RefreshViewportContents();

	/*
	 * Utility
	 */

	/** Mutable mesh converted into a Unreal mesh object */
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = nullptr;
	
	/** Generates a new USkeletalMesh from the MutableMesh set on this object
	 * @return - If the operation was able to be processed or not
	 */
	bool GenerateUnrealMesh();
};


