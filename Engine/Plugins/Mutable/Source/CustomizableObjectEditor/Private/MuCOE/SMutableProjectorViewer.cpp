// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableProjectorViewer.h"

#include "MuT/TypeInfo.h"
#include "Widgets/Input/SVectorInputBox.h"

#define LOCTEXT_NAMESPACE "MutableProjectorViewer"

void SMutableProjectorViewer::Construct(const FArguments& InArgs)
{
	// Formatting constants
	constexpr float ValuesHorizontalPadding = 20;
	constexpr float ValueVerticalPadding = 0;
	constexpr float InBetweenSectionsVerticalPadding = 4;
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
	
		// Projector position 
		+ SVerticalBox::Slot()
		.Padding(0,InBetweenSectionsVerticalPadding)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectorPosition","Position : "))
			]

			+ SHorizontalBox::Slot()
			.Padding(ValuesHorizontalPadding,ValueVerticalPadding)
			[
				SNew(SVectorInputBox)
				.IsEnabled(false)
				.bColorAxisLabels(true)
				.X(this, &SMutableProjectorViewer::GetProjectorPositionComponent,0)
				.Y(this, &SMutableProjectorViewer::GetProjectorPositionComponent,1)
				.Z(this, &SMutableProjectorViewer::GetProjectorPositionComponent,2)
			]
		]

		// Projector direction 
		+ SVerticalBox::Slot()
		.Padding(0,InBetweenSectionsVerticalPadding)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectorDirection","Direction : "))
			]

			+ SHorizontalBox::Slot()
			.Padding(ValuesHorizontalPadding,ValueVerticalPadding)
			[
				SNew(SVectorInputBox)
				.IsEnabled(false)
				.bColorAxisLabels(true)
				.X(this, &SMutableProjectorViewer::GetProjectorDirectionComponent,0)
				.Y(this, &SMutableProjectorViewer::GetProjectorDirectionComponent,1)
				.Z(this, &SMutableProjectorViewer::GetProjectorDirectionComponent,2)
			]
		]

		// Projector scale 
		+ SVerticalBox::Slot()
		.Padding(0,InBetweenSectionsVerticalPadding)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectorScale","Scale : "))
			]

			+ SHorizontalBox::Slot()
			.Padding(ValuesHorizontalPadding,ValueVerticalPadding)
			[
				SNew(SVectorInputBox)
				.IsEnabled(false)
				.bColorAxisLabels(true)
				.X(this, &SMutableProjectorViewer::GetProjectorScaleComponent,0)
				.Y(this, &SMutableProjectorViewer::GetProjectorScaleComponent,1)
				.Z(this, &SMutableProjectorViewer::GetProjectorScaleComponent,2)
			]
		]

		// Projector up 
		+ SVerticalBox::Slot()
		.Padding(0,InBetweenSectionsVerticalPadding)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectorUpVector","Up Vector : "))
			]

			+ SHorizontalBox::Slot()
			.Padding(ValuesHorizontalPadding,ValueVerticalPadding)
			[
				SNew(SVectorInputBox)
				.IsEnabled(false)
				.bColorAxisLabels(true)
				.X(this, &SMutableProjectorViewer::GetProjectorUpComponent,0)
				.Y(this, &SMutableProjectorViewer::GetProjectorUpComponent,1)
				.Z(this, &SMutableProjectorViewer::GetProjectorUpComponent,2)
			]
		]

		// Projector Angle
		+ SVerticalBox::Slot()
		.Padding(0,InBetweenSectionsVerticalPadding)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectorAngle","Angle : "))
			]

			+ SHorizontalBox::Slot()
			.Padding(ValuesHorizontalPadding,ValueVerticalPadding)
			[
				SNew(STextBlock)
				.Text(this, &SMutableProjectorViewer::GetProjectorAngleAsText)
			]
		]

		// Projector Type
		+ SVerticalBox::Slot()
		.Padding(0,InBetweenSectionsVerticalPadding)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProjectorType","Type : "))
			]

			+ SHorizontalBox::Slot()
			.Padding(ValuesHorizontalPadding,ValueVerticalPadding)
			[
				SNew(STextBlock)
				.Text(this, &SMutableProjectorViewer::GetProjectorTypeAsText)
			]
		]

	];
	
	SetProjector(InArgs._MutableProjector);
}


void SMutableProjectorViewer::SetProjector(const UE::Mutable::Private::FProjector& InMutableProjector)
{
	MutableProjector = InMutableProjector;
}


TOptional<float> SMutableProjectorViewer::GetProjectorPositionComponent(int32 VectorComponentIndex) const
{
	checkf(VectorComponentIndex <= 2 ,TEXT("Unable to select component. Value out of range"));

	TOptional<float> Value = MutableProjector.position[VectorComponentIndex];
	return Value;
}


TOptional<float> SMutableProjectorViewer::GetProjectorDirectionComponent(int32 VectorComponentIndex) const
{
	checkf(VectorComponentIndex <= 2 ,TEXT("Unable to select component. Value out of range"));

	TOptional<float> Value = MutableProjector.direction[VectorComponentIndex];
	return Value;
}

TOptional<float> SMutableProjectorViewer::GetProjectorScaleComponent(int32 VectorComponentIndex) const
{
	checkf(VectorComponentIndex <= 2 ,TEXT("Unable to select component. Value out of range"));
	
	TOptional<float> Value = MutableProjector.scale[VectorComponentIndex];
	return Value;
}

TOptional<float> SMutableProjectorViewer::GetProjectorUpComponent(int32 VectorComponentIndex) const
{
	checkf(VectorComponentIndex <= 2 ,TEXT("Unable to select component. Value out of range"));

	TOptional<float> Value = MutableProjector.up[VectorComponentIndex];
	return Value;
}


float SMutableProjectorViewer::GetProjectorAngle() const
{
	return MutableProjector.projectionAngle;
}


FText SMutableProjectorViewer::GetProjectorAngleAsText() const
{
	FString ValueAsString = FString::SanitizeFloat(GetProjectorAngle());

	const FString AngleSign = FString(TEXT(" º"));
	ValueAsString.Append(AngleSign);
	
	return FText::FromString(ValueAsString);
}


UE::Mutable::Private::EProjectorType SMutableProjectorViewer::GetProjectorType() const
{
	const UE::Mutable::Private::EProjectorType ProjectorType = MutableProjector.type;
	return ProjectorType;
}


FText SMutableProjectorViewer::GetProjectorTypeAsText() const
{
	const UE::Mutable::Private::EProjectorType ProjectorType =  GetProjectorType();
	const uint32 TypeIndex = static_cast<uint32> (ProjectorType);

	return FText::FromString(* FString(UE::Mutable::Private::TypeInfo::s_projectorTypeName[TypeIndex]));
}

#undef LOCTEXT_NAMESPACE
