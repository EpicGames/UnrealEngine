// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Parameters.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Mutable::Private { enum class EProjectorType : uint32; }


class SMutableProjectorViewer final : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMutableProjectorViewer) {}
		SLATE_ARGUMENT(UE::Mutable::Private::FProjector,MutableProjector);
	SLATE_END_ARGS()

public:

	/**
	 * Builds this slate with the assistance of the provided arguments
	 * @param InArgs - Input arguments used to provide this slate with the data required to operate
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Sets the projector object whose data we want to inspect
	 * @param InMutableProjector - Projector we desire to inspect. It is the origin of the data shown on the UI
	 */
	void SetProjector(const UE::Mutable::Private::FProjector& InMutableProjector);

private:

	/** The mutable projector witch data is being exposed on the UI */
	UE::Mutable::Private::FProjector MutableProjector;
	
	/** Get the float corresponding to the position of the projector on the targeted component index
	 * @param VectorComponentIndex - Determines if you get the X, Y or Z value from the vector (0,1 and 2)
	 * @return The value corresponding to the index provided. 
	 */
	TOptional<float> GetProjectorPositionComponent(int32 VectorComponentIndex) const;
	
	/** Get the float corresponding to the direction vector of the projector on the targeted component index
	 * @param VectorComponentIndex - Determines if you get the X, Y or Z value from the vector (0,1 and 2)
	 * @return The value corresponding to the index provided. 
	 */
	TOptional<float> GetProjectorDirectionComponent(int32 VectorComponentIndex) const;

	/** Get the float corresponding to the scale vector of the projector on the targeted component index
	 * @param VectorComponentIndex - Determines if you get the X, Y or Z value from the vector (0,1 and 2)
	 * @return The value corresponding to the index provided. 
	 */
	TOptional<float> GetProjectorScaleComponent(int32 VectorComponentIndex) const;

	/** Get the float corresponding to the up vector of the projector on the targeted component index
	 * @param VectorComponentIndex - Determines if you get the X, Y or Z value from the vector (0,1 and 2)
	 * @return The value corresponding to the index provided. 
	*/		
	TOptional<float> GetProjectorUpComponent(int32 VectorComponentIndex) const;

	/** Get the angle of the projector object
	 * @return The angle of the projector. 0 if no projector is set.
	 */
	float GetProjectorAngle() const;
	
	/**
	 * Gets the angle of the projector so a STextBlock is able to display it
	 * @return The angle as an FText
	 */
	FText GetProjectorAngleAsText() const;

	/**
	 * It accesses the set projector and returns the type of it.
	 * @return - The projector type.
	 */
	UE::Mutable::Private::EProjectorType GetProjectorType() const;
	
	/**
	 * Provides the UI with the mutable type in a format it can display on an STextBlock
	 * @return - The projector type as an FText
	 */
	FText GetProjectorTypeAsText() const;
};

