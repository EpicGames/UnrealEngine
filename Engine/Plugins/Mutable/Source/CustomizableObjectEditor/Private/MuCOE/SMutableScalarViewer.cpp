// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableScalarViewer.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void SMutableScalarViewer::Construct(const FArguments& InArgs)
{
	const FText ScalarValueTitle = LOCTEXT("ScalarValueTitle", "Scalar Value : ");

	ChildSlot
	[
		SNew(SHorizontalBox)

		// title
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(ScalarValueTitle)
		]

		// value
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(this, &SMutableScalarViewer::GetValue)
		]
	];
}


void SMutableScalarViewer::SetScalar(const float& InFloat)
{
	this->ScalarValue = InFloat;
}

FText SMutableScalarViewer::GetValue() const
{
	return FText::AsNumber(this->ScalarValue);
}

#undef LOCTEXT_NAMESPACE
