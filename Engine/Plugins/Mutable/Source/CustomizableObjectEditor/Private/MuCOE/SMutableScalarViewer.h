// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"


/*
* Mutable viewer to show a representation of an scalar variable held by mutable.
*/
class SMutableScalarViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableScalarViewer) {}
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Bool to be used for this widget */
	void SetScalar(const float& InFloat);

private:

	/** The value being displayed by the widget's UI */
	float ScalarValue;

	/*
	* Callback used by the widget to get the value to display on the UI
	* @return The variable ScalarValue as an FText
	*/
	FText GetValue() const;
};

