// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableScrubPanel.h"

#include "Widgets/SBoxPanel.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimBlueprint.h"
#include "AnimPreviewInstance.h"
#include "SScrubControlPanel.h"
#include "ScopedTransaction.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "MuCOE/CustomizableObjectEditorViewportClient.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SMutableScrubPanel::Construct(const FArguments& InArgs, const TSharedRef<FCustomizableObjectEditorViewportClient>& InPreviewScene)
{
	bSliderBeingDragged = false;
	OnSetInputViewRange = InArgs._OnSetInputViewRange;

	PreviewScenePtr = InPreviewScene;

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill) 
		.VAlign(VAlign_Center)
		.FillWidth(1)
		.Padding(0.0f)
		[
			SAssignNew(ScrubControlPanel, SScrubControlPanel)
			.IsEnabled(true)
			.Value(this, &SMutableScrubPanel::GetScrubValue)
			.NumOfKeys(this, &SMutableScrubPanel::GetNumberOfKeys)
			.SequenceLength(this, &SMutableScrubPanel::GetSequenceLength)
			.DisplayDrag(this, &SMutableScrubPanel::GetDisplayDrag)
			.OnValueChanged(this, &SMutableScrubPanel::OnValueChanged)
			.OnBeginSliderMovement(this, &SMutableScrubPanel::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &SMutableScrubPanel::OnEndSliderMovement)
			.OnClickedForwardPlay(this, &SMutableScrubPanel::OnClick_Forward)
			.OnClickedForwardStep(this, &SMutableScrubPanel::OnClick_Forward_Step)
			.OnClickedForwardEnd(this, &SMutableScrubPanel::OnClick_Forward_End)
			.OnClickedBackwardPlay(this, &SMutableScrubPanel::OnClick_Backward)
			.OnClickedBackwardStep(this, &SMutableScrubPanel::OnClick_Backward_Step)
			.OnClickedBackwardEnd(this, &SMutableScrubPanel::OnClick_Backward_End)
			.OnClickedToggleLoop(this, &SMutableScrubPanel::OnClick_ToggleLoop)
			.OnGetLooping(this, &SMutableScrubPanel::IsLoopStatusOn)
			.OnGetPlaybackMode(this, &SMutableScrubPanel::GetPlaybackMode)
			.ViewInputMin(this, &SMutableScrubPanel::GetViewMinInput)
			.ViewInputMax(this, &SMutableScrubPanel::GetViewMaxInput)
			.bDisplayAnimScrubBarEditing(InArgs._bDisplayAnimScrubBarEditing)
			.OnSetInputViewRange(InArgs._OnSetInputViewRange)
			.OnReZeroAnimSequence( this, &SMutableScrubPanel::OnReZeroAnimSequence )
			.bAllowZoom(true)
			.IsRealtimeStreamingMode(this, &SMutableScrubPanel::IsRealtimeStreamingMode)
		]
	];
}


float SMutableScrubPanel::GetViewMinInput() const
{
	for (TPair<FName,TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		if (PreviewMeshComponent->PreviewInstance != NULL)
		{
			return 0.0f;
		}
		if (PreviewMeshComponent->GetAnimInstance() != NULL)
		{
			return FMath::Max<float>((float)(PreviewMeshComponent->GetAnimInstance()->LifeTimer - 30.0), 0.0f);
		}
	}
	
	return 0.f; 
}


float SMutableScrubPanel::GetViewMaxInput() const
{ 
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}
		
		if (PreviewMeshComponent->PreviewInstance != NULL)
		{
			return PreviewMeshComponent->PreviewInstance->GetLength();
		}
		else if (PreviewMeshComponent->GetAnimInstance() != NULL)
		{
			return static_cast<float>(PreviewMeshComponent->GetAnimInstance()->LifeTimer);
		}
	}

	return 0.f;
}


FReply SMutableScrubPanel::OnClick_Forward_Step()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}
		
		if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent))
		{
			bool bShouldStepCloth = FMath::Abs(PreviewInstance->GetLength() - PreviewInstance->GetCurrentTime()) > SMALL_NUMBER;

			PreviewInstance->SetPlaying(false);
			PreviewInstance->StepForward();

			if(PreviewMeshComponent && bShouldStepCloth)
			{
				PreviewMeshComponent->bPerformSingleClothingTick = true;
			}
		}
		else if (PreviewMeshComponent)
		{
			// BlendSpaces and Animation Blueprints combine animations so there's no such thing as a frame. However, 1/30 is a sensible/common rate.
			const float FixedFrameRate = 30.0f;

			// Advance a single frame, leaving it paused afterwards
			PreviewMeshComponent->GlobalAnimRateScale = 1.0f;
			GetPreviewScene()->Tick(1.0f / FixedFrameRate); 
			PreviewMeshComponent->GlobalAnimRateScale = 0.0f;
		}
	}

	return FReply::Handled();
}


FReply SMutableScrubPanel::OnClick_Forward_End()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
        {
        	continue;
        }
		
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
        if (PreviewInstance)
        {
        	PreviewInstance->SetPlaying(false);
        	PreviewInstance->SetPosition(PreviewInstance->GetLength(), false);
        }
	}

	return FReply::Handled();
}


FReply SMutableScrubPanel::OnClick_Backward_Step()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (PreviewInstance)
		{
			bool bShouldStepCloth = PreviewInstance->GetCurrentTime() > SMALL_NUMBER;

			PreviewInstance->SetPlaying(false);
			PreviewInstance->StepBackward();

			if(PreviewMeshComponent && bShouldStepCloth)
			{
				PreviewMeshComponent->bPerformSingleClothingTick = true;
			}
		}
	}
	
	return FReply::Handled();
}


FReply SMutableScrubPanel::OnClick_Backward_End()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (PreviewInstance)
		{
			PreviewInstance->SetPlaying(false);
			PreviewInstance->SetPosition(0.f, false);
		}
	}

	return FReply::Handled();
}


FReply SMutableScrubPanel::OnClick_Forward()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);

		if (PreviewInstance)
		{
			bool bIsReverse = PreviewInstance->IsReverse();
			bool bIsPlaying = PreviewInstance->IsPlaying();
			// if current bIsReverse and bIsPlaying, we'd like to just turn off reverse
			if (bIsReverse && bIsPlaying)
			{
				PreviewInstance->SetReverse(false);
			}
			// already playing, simply pause
			else if (bIsPlaying) 
			{
				PreviewInstance->SetPlaying(false);
			
				if(PreviewMeshComponent && PreviewMeshComponent->bPauseClothingSimulationWithAnim)
				{
					PreviewMeshComponent->SuspendClothingSimulation();
				}
			}
			// if not playing, play forward
			else 
			{
				//if we're at the end of the animation, jump back to the beginning before playing
				if ( GetScrubValue() >= GetSequenceLength() )
				{
					PreviewInstance->SetPosition(0.0f, false);
				}

				PreviewInstance->SetReverse(false);
				PreviewInstance->SetPlaying(true);

				if(PreviewMeshComponent && PreviewMeshComponent->bPauseClothingSimulationWithAnim)
				{
					PreviewMeshComponent->ResumeClothingSimulation();
				}
			}
		}
		else if(PreviewMeshComponent)
		{
			PreviewMeshComponent->GlobalAnimRateScale = (PreviewMeshComponent->GlobalAnimRateScale > 0.0f) ? 0.0f : 1.0f;
		}
	}
	
	return FReply::Handled();
}


FReply SMutableScrubPanel::OnClick_Backward()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (PreviewInstance)
		{
			bool bIsReverse = PreviewInstance->IsReverse();
			bool bIsPlaying = PreviewInstance->IsPlaying();
			// if currently playing forward, just simply turn on reverse
			if (!bIsReverse && bIsPlaying)
			{
				PreviewInstance->SetReverse(true);
			}
			else if (bIsPlaying)
			{
				PreviewInstance->SetPlaying(false);
			}
			else
			{
				//if we're at the beginning of the animation, jump back to the end before playing
				if ( GetScrubValue() <= 0.0f )
				{
					PreviewInstance->SetPosition(GetSequenceLength(), false);
				}

				PreviewInstance->SetPlaying(true);
				PreviewInstance->SetReverse(true);
			}
		}
	}
	
	return FReply::Handled();
}


FReply SMutableScrubPanel::OnClick_ToggleLoop()
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (PreviewInstance)
		{
			bool bIsLooping = PreviewInstance->IsLooping();
			PreviewInstance->SetLooping(!bIsLooping);
		}
	}
	
	return FReply::Handled();
}


bool SMutableScrubPanel::IsLoopStatusOn() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent))
		{
			return PreviewInstance->IsLooping();
		}
	}
	
	return false;
}


EPlaybackMode::Type SMutableScrubPanel::GetPlaybackMode() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent))
		{
			if (PreviewInstance->IsPlaying())
			{
				return PreviewInstance->IsReverse() ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
			}
			return EPlaybackMode::Stopped;
		}
		else if (PreviewMeshComponent)
		{
			return PreviewMeshComponent->GlobalAnimRateScale > 0.0f ? EPlaybackMode::PlayingForward : EPlaybackMode::Stopped;
		}
	}
	
	return EPlaybackMode::Stopped;
}


bool SMutableScrubPanel::IsRealtimeStreamingMode() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent))
		{
			return !PreviewInstance->GetCurrentAsset();
		}
	}
	
	return true;
}


void SMutableScrubPanel::OnValueChanged(float NewValue)
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent))
		{
			PreviewInstance->SetPosition(NewValue);
		}
		else
		{
			UAnimInstance* Instance;
			FAnimBlueprintDebugData* DebugData;
			if (GetAnimBlueprintDebugData(/*out*/ Instance, /*out*/ DebugData))
			{
				DebugData->SetSnapshotIndexByTime(Instance, NewValue);
			}
		}
	}
}


void SMutableScrubPanel::OnBeginSliderMovement()
{
	bSliderBeingDragged = true;

	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent))
		{
			PreviewInstance->SetPlaying(false);
		}
	}	
}


void SMutableScrubPanel::OnEndSliderMovement(float NewValue)
{
	bSliderBeingDragged = false;
}


uint32 SMutableScrubPanel::GetNumberOfKeys() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}

		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (!PreviewInstance)
		{
			continue;
		}
			
		float Length = PreviewInstance->GetLength();
		// if anim sequence, use correct num frames
		int32 NumKeys = (int32) (Length/0.0333f); 
		if (PreviewInstance->GetCurrentAsset())
		{
			if (PreviewInstance->GetCurrentAsset()->IsA(UAnimSequenceBase::StaticClass()))
			{
				NumKeys = CastChecked<UAnimSequenceBase>(PreviewInstance->GetCurrentAsset())->GetNumberOfSampledKeys();
			}
			else if(PreviewInstance->GetCurrentAsset()->IsA(UBlendSpace::StaticClass()))
			{
				// Blendspaces dont display frame notches, so just return 0 here
				NumKeys = 0;
			}
		}
		return NumKeys;
	}

	return 1;
}


float SMutableScrubPanel::GetSequenceLength() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}
			
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (!PreviewInstance)
		{
			continue;
		}
			
		return PreviewInstance->GetLength();
	}

	return 0.f;
}


void SMutableScrubPanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bSliderBeingDragged)
	{
		GetPreviewScene()->Invalidate();
	}
}


UAnimSingleNodeInstance* SMutableScrubPanel::GetPreviewInstance(UDebugSkelMeshComponent* PreviewMeshComponent) const
{
	return PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn() ? PreviewMeshComponent->PreviewInstance : nullptr;
}


float SMutableScrubPanel::GetScrubValue() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}
			
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (!PreviewInstance)
		{
			continue;
		}
			
		return PreviewInstance->GetCurrentTime(); 
	}
	
	return 0.f;
}


bool SMutableScrubPanel::GetAnimBlueprintDebugData(UAnimInstance*& Instance, FAnimBlueprintDebugData*& DebugInfo) const
{
	Instance = nullptr;

	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}
		
		UAnimInstance* AnimInstance = PreviewMeshComponent->GetAnimInstance();
		if (AnimInstance && AnimInstance->GetClass()->ClassGeneratedBy)
		{
			Instance = AnimInstance;
			break;
		}
	}

	if (Instance)
	{
		// Avoid updating the instance if we're replaying the past
		if (UAnimBlueprintGeneratedClass* AnimBlueprintClass = Cast<UAnimBlueprintGeneratedClass>(Instance->GetClass()))
		{
			if (UAnimBlueprint* Blueprint = Cast<UAnimBlueprint>(AnimBlueprintClass->ClassGeneratedBy))
			{
				if (Blueprint->GetObjectBeingDebugged() == Instance)
				{
					DebugInfo = &(AnimBlueprintClass->GetAnimBlueprintDebugData());
					return true;
				}
			}
		}
	}

	return false;
}


TSharedRef<FCustomizableObjectEditorViewportClient> SMutableScrubPanel::GetPreviewScene() const
{
	return PreviewScenePtr.Pin().ToSharedRef();
}


void SMutableScrubPanel::OnReZeroAnimSequence(int32 FrameIndex)
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewSkelComp = Entry.Value.Get();
		if (!PreviewSkelComp)
		{
			continue;
		}
		
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewSkelComp);
		if (!PreviewInstance)
		{
			continue;
		}

		if (PreviewInstance->GetCurrentAsset())
		{
			if (UAnimSequence* AnimSequence = Cast<UAnimSequence>( PreviewInstance->GetCurrentAsset()))
			{
				if (const USkeleton* Skeleton = AnimSequence->GetSkeleton())
				{
					const FName RootBoneName = Skeleton->GetReferenceSkeleton().GetBoneName(0);

					if (AnimSequence->GetDataModel()->IsValidBoneTrackName(RootBoneName))
					{
						TArray<FVector3f> PosKeys;
						TArray<FQuat4f> RotKeys;
						TArray<FVector3f> ScaleKeys;

						TArray<FTransform> BoneTransforms;
						AnimSequence->GetDataModel()->GetBoneTrackTransforms(RootBoneName, BoneTransforms);

						PosKeys.SetNum(BoneTransforms.Num());
						RotKeys.SetNum(BoneTransforms.Num());
						ScaleKeys.SetNum(BoneTransforms.Num());

						// Find vector that would translate current root bone location onto origin.
						FVector FrameTransform = FVector::ZeroVector;
						if (FrameIndex == INDEX_NONE)
						{
							// Use current transform
							FrameTransform = PreviewSkelComp->GetComponentSpaceTransforms()[0].GetLocation();
						}
						else if(BoneTransforms.IsValidIndex(FrameIndex))
						{
							// Use transform at frame
							FrameTransform = BoneTransforms[FrameIndex].GetLocation();
						}

						FVector ApplyTranslation = -1.f * FrameTransform;

						// Convert into world space
						const FVector WorldApplyTranslation = PreviewSkelComp->GetComponentTransform().TransformVector(ApplyTranslation);
						ApplyTranslation = PreviewSkelComp->GetComponentTransform().InverseTransformVector(WorldApplyTranslation);

						for(int32 KeyIndex = 0; KeyIndex < BoneTransforms.Num(); KeyIndex++)
						{
							PosKeys[KeyIndex] = FVector3f(BoneTransforms[KeyIndex].GetLocation() + ApplyTranslation);
							RotKeys[KeyIndex] = FQuat4f(BoneTransforms[KeyIndex].GetRotation());
							ScaleKeys[KeyIndex] = FVector3f(BoneTransforms[KeyIndex].GetScale3D());
						}

						IAnimationDataController& Controller = AnimSequence->GetController();
						Controller.SetBoneTrackKeys(RootBoneName, PosKeys, RotKeys, ScaleKeys);
					}
				}
			}
		}
	}
}


bool SMutableScrubPanel::GetDisplayDrag() const
{
	for (TPair<FName, TWeakObjectPtr<UDebugSkelMeshComponent>>& Entry : GetPreviewScene()->GetPreviewMeshComponents())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = Entry.Value.Get();
		if (!PreviewMeshComponent)
		{
			continue;
		}
		
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance(PreviewMeshComponent);
		if (!PreviewInstance)
		{
			continue;
		}

		if (PreviewInstance->GetCurrentAsset())
		{
			return true;
		}
	}

	return false;
}


#undef LOCTEXT_NAMESPACE
