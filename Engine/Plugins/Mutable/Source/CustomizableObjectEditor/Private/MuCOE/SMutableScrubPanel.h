// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPersonaPreviewScene.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "SScrubWidget.h"
#include "ITransportControl.h"

class FCustomizableObjectEditorViewportClient;
class SScrubControlPanel;
class UAnimationAsset;
class UAnimInstance;
class UAnimSequenceBase;
class UDebugSkelMeshComponent;
class UAnimSingleNodeInstance;
struct FAnimBlueprintDebugData;


class SMutableScrubPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableScrubPanel) : _LockedSequence() {}
		SLATE_ARGUMENT(UAnimSequenceBase*, LockedSequence)
		SLATE_ARGUMENT(bool, bDisplayAnimScrubBarEditing)
		SLATE_EVENT(FOnSetInputViewRange, OnSetInputViewRange)
		SLATE_ARGUMENT(bool, bAllowZoom)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FCustomizableObjectEditorViewportClient>& InPreviewScene);

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	float GetViewMinInput() const;

	float GetViewMaxInput() const;
	
	FReply OnClick_Forward_Step();

	FReply OnClick_Forward_End();

	FReply OnClick_Backward_Step();
	
	FReply OnClick_Backward_End();

	FReply OnClick_Forward();

	FReply OnClick_Backward();

	FReply OnClick_ToggleLoop();

	void OnValueChanged(float NewValue);

	void OnReZeroAnimSequence(int32 FrameIndex);

	void OnBeginSliderMovement();
	
	void OnEndSliderMovement(float NewValue);

	EPlaybackMode::Type GetPlaybackMode() const;

	bool IsLoopStatusOn() const;
	
	bool IsRealtimeStreamingMode() const;
	
	float GetScrubValue() const;

	UAnimSingleNodeInstance* GetPreviewInstance(UDebugSkelMeshComponent* PreviewMeshComponent) const;

	TSharedPtr<SScrubControlPanel> ScrubControlPanel;

	uint32 GetNumberOfKeys() const;

	float GetSequenceLength() const;
	
	bool GetAnimBlueprintDebugData(UAnimInstance*& Instance, FAnimBlueprintDebugData*& DebugInfo) const;

	TSharedRef<FCustomizableObjectEditorViewportClient> GetPreviewScene() const;

	bool GetDisplayDrag() const;
	
	TWeakPtr<FCustomizableObjectEditorViewportClient> PreviewScenePtr;

	FOnSetInputViewRange OnSetInputViewRange;

	bool bSliderBeingDragged = false;
};
