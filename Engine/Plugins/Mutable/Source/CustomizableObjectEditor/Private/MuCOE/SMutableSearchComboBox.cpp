// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableSearchComboBox.h"

#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/Attribute.h"
#include "DetailLayoutBuilder.h"	// For font: TODO: move to argument


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


void SMutableSearchComboBox::Construct(const FArguments& InArgs)
{
	check(InArgs._ComboBoxStyle);

	ItemStyle = InArgs._ItemStyle;
	MenuRowPadding = InArgs._ComboBoxStyle->MenuRowPadding;

	bAllowAddNewOptions = InArgs._AllowAddNewOptions;

	// Work out which values we should use based on whether we were given an override, or should use the style's version
	OurComboButtonStyle = InArgs._ComboBoxStyle->ComboButtonStyle;
	if (InArgs._MenuButtonBrush)
	{
		OurComboButtonStyle.DownArrowImage = *InArgs._MenuButtonBrush;
	}
	const FButtonStyle* const OurButtonStyle = InArgs._ButtonStyle ? InArgs._ButtonStyle : &OurComboButtonStyle.ButtonStyle;

	this->OnSelectionChanged = InArgs._OnSelectionChanged;

	OptionsSource = InArgs._OptionsSource;

	TAttribute<EVisibility> SearchVisibility = InArgs._SearchVisibility;
	const EVisibility CurrentSearchVisibility = SearchVisibility.Get();

	TSharedRef<SWidget> ComboBoxMenuContent =
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxListHeight)
		[
			SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(this->SearchField, SEditableTextBox)
						.HintText(bAllowAddNewOptions ? LOCTEXT("SearchOrAdd", "Search or add...") : LOCTEXT("Search", "Search..."))
						.OnTextChanged(this, &SMutableSearchComboBox::OnSearchTextChanged)
						.OnTextCommitted(this, &SMutableSearchComboBox::OnSearchTextCommitted)
						.Visibility(SearchVisibility)
				]

				+ SVerticalBox::Slot()
				[
					SAssignNew(this->ComboTreeView, SComboTreeType)
						.TreeItemsSource(&FilteredRootOptionsSource)
						.OnGenerateRow(this, &SMutableSearchComboBox::GenerateMenuItemRow)
						.OnGetChildren(this, &SMutableSearchComboBox::OnGetChildren)
						.OnSelectionChanged(this, &SMutableSearchComboBox::OnSelectionChanged_Internal)
						.OnKeyDownHandler(this, &SMutableSearchComboBox::OnKeyDownHandler)
						.SelectionMode(ESelectionMode::Single)
				]
		];

	// Set up content
	TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
	if (InArgs._Content.Widget == SNullWidget::NullWidget)
	{
		SAssignNew(ButtonContent, STextBlock);
	}


	SComboButton::Construct(SComboButton::FArguments()
		.ComboButtonStyle(&OurComboButtonStyle)
		.ButtonStyle(OurButtonStyle)
		.Method(InArgs._Method)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
		.MenuContent()
		[
			ComboBoxMenuContent
		]
		.ContentPadding(InArgs._ContentPadding)
		.ForegroundColor(InArgs._ForegroundColor)
		.OnMenuOpenChanged(this, &SMutableSearchComboBox::OnMenuOpenChanged)
		.IsFocusable(true)
	);

	if (CurrentSearchVisibility == EVisibility::Visible)
	{
		SetMenuContentWidgetToFocus(SearchField);
	}
	else
	{
		SetMenuContentWidgetToFocus(ComboTreeView);
	}
}


void SMutableSearchComboBox::RefreshOptions()
{
	// Need to refresh filtered list whenever options change
	FilteredOptionsSource.Reset();

	if (SearchText.IsEmpty())
	{
		FilteredOptionsSource = *OptionsSource;
	}
	else
	{
		TArray<FString> SearchTokens;
		SearchText.ToString().ParseIntoArrayWS(SearchTokens);

		for (const TSharedRef<FFilteredOption>& Option : *OptionsSource)
		{
			bool bAllTokensMatch = true;
			for (const FString& SearchToken : SearchTokens)
			{
				if (Option->DisplayOption.Find(SearchToken, ESearchCase::Type::IgnoreCase) == INDEX_NONE)
				{
					bAllTokensMatch = false;
					break;
				}
			}

			if (bAllTokensMatch)
			{
				FilteredOptionsSource.Add(Option);
			}
		}

		bool bFullMatch = false;
		FString SearchString = SearchText.ToString();
		for (const TSharedRef<FFilteredOption>& Option : *OptionsSource)
		{
			if (Option->DisplayOption == SearchString && !Option->ActualOption.IsEmpty())
			{
				bFullMatch = true; 
				break;
			}
		}

		if ( bAllowAddNewOptions && !bFullMatch && !SearchText.IsEmpty() )
		{
			FFilteredOption Data;
			Data.ActualOption = SearchString;
			Data.DisplayOption = FString::Printf(TEXT("Add new tag (%s)"), *SearchString);
			FilteredOptionsSource.Insert(MakeShared<FFilteredOption>(Data),0);
		}

		// Ensure filtered options parents are added
		for (int32 OptionIndex = 0; OptionIndex < FilteredOptionsSource.Num(); ++OptionIndex)
		{
			TSharedPtr<FFilteredOption> Parent = FilteredOptionsSource[OptionIndex]->Parent;
			if (Parent)
			{
				FilteredOptionsSource.AddUnique(Parent.ToSharedRef());

				while (Parent)
				{
					ComboTreeView->SetItemExpansion(Parent.ToSharedRef(), true);
					Parent = Parent->Parent;
				}
			}
		}
	}

	FilteredRootOptionsSource.SetNum(0, EAllowShrinking::No);
	FilteredRootOptionsSource.Reserve(FilteredOptionsSource.Num());
	for (const TSharedRef<FFilteredOption>& Option : FilteredOptionsSource)
	{
		if (!Option->Parent.IsValid())
		{
			FilteredRootOptionsSource.Add(Option);
		}
	}

	if (ComboTreeView)
	{
		ComboTreeView->RequestTreeRefresh();
	}
}


TSharedRef<ITableRow> SMutableSearchComboBox::GenerateMenuItemRow(TSharedRef<FFilteredOption> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FSlateColor LabelColor = InItem->ActualOption.IsEmpty() 
		? FSlateColor::UseSubduedForeground() 
		: FSlateColor::UseForeground();

	return SNew(SComboRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock)
				.Text(FText::FromString(InItem->DisplayOption))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(LabelColor)
		];
}


void SMutableSearchComboBox::OnGetChildren(TSharedRef<FFilteredOption> InItem, TArray<TSharedRef<FFilteredOption>>& OutChildren)
{
	// TODO: Optimize if necessary
	for (const TSharedRef<FFilteredOption>& Option : FilteredOptionsSource)
	{
		if (Option->Parent==InItem)
		{
			OutChildren.Add(Option);
		}
	}

}


void SMutableSearchComboBox::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen)
	{
		RefreshOptions();
	}
	else
	{
		// Set focus back to ComboBox for users focusing the ListView that just closed
		FSlateApplication::Get().ForEachUser([this](FSlateUser& User)
			{
				TSharedRef<SWidget> ThisRef = this->AsShared();
				if (User.IsWidgetInFocusPath(this->ComboTreeView))
				{
					User.SetFocus(ThisRef);
				}
			});

	}
}


void SMutableSearchComboBox::OnSelectionChanged_Internal(TSharedPtr<FFilteredOption> ProposedSelection, ESelectInfo::Type SelectInfo)
{
	if (!ProposedSelection)
	{
		return;
	}

	// If the proposed selection is not a valid element (it is a hierarchy label)
	if (ProposedSelection->ActualOption.IsEmpty())
	{
		return;
	}

	// Close combo as long as the selection wasn't from navigation
	if (SelectInfo != ESelectInfo::OnNavigation)
	{		
		OnSelectionChanged.ExecuteIfBound(FText::FromString(ProposedSelection->ActualOption));
		this->SetIsOpen(false);
	}
}


void SMutableSearchComboBox::OnSearchTextChanged(const FText& ChangedText)
{
	SearchText = ChangedText;

	RefreshOptions();
}


void SMutableSearchComboBox::OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if ((InCommitType == ETextCommit::Type::OnEnter) && !FilteredOptionsSource.IsEmpty())
	{
		TSharedPtr<FFilteredOption> Selected;

		for (const TSharedRef<FFilteredOption>& Option : FilteredOptionsSource)
		{
			if (Option->ActualOption == InText.ToString())
			{
				Selected = Option;
				break;
			}
		}

		if (!Selected)
		{
			for (const TSharedRef<FFilteredOption>& Option : FilteredOptionsSource)
			{
				if (Option->DisplayOption == InText.ToString())
				{
					Selected = Option;
					break;
				}
			}
		}

		if (!Selected)
		{
			Selected = FilteredOptionsSource[0];
		}

		ComboTreeView->SetSelection(Selected.ToSharedRef(), ESelectInfo::OnKeyPress);
	}
}


FReply SMutableSearchComboBox::OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Select the first selected item on hitting enter
		TArray<TSharedRef<FFilteredOption>> SelectedItems = ComboTreeView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::OnKeyPress);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}


#undef LOCTEXT_NAMESPACE
