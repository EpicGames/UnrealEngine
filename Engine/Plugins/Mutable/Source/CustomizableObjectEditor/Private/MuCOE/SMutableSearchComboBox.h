// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/ITableRow.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UCustomizableObjectNodeObject;

/** */
class SMutableSearchComboBox : public SComboButton
{
public:
	struct FFilteredOption
	{
		FString ActualOption;
		FString DisplayOption;

		TSharedPtr<FFilteredOption> Parent;
	};

	/** Type of list used for showing menu options. */
	typedef STreeView< TSharedRef<FFilteredOption> > SComboTreeType;

	/** Delegate type used to generate widgets that represent Options */
	typedef typename TSlateDelegates< TSharedRef<FString> >::FOnGenerateWidget FOnGenerateWidget;

	SLATE_BEGIN_ARGS(SMutableSearchComboBox)
		: _Content()
		, _ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox"))
		, _ButtonStyle(nullptr)
		, _ItemStyle(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
		, _ContentPadding(_ComboBoxStyle->ContentPadding)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _OptionsSource()
		, _OnSelectionChanged()
		, _MenuButtonBrush(nullptr)
		, _Method()
		, _AllowAddNewOptions(false)
		, _MaxListHeight(450.0f)
		, _SearchVisibility()
		{}

		/** Slot for this button's content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_STYLE_ARGUMENT(FComboBoxStyle, ComboBoxStyle)

		/** The visual style of the button part of the combo box (overrides ComboBoxStyle) */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemStyle)

		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)

		SLATE_ARGUMENT(const TArray< TSharedRef<FFilteredOption> >*, OptionsSource)
		SLATE_EVENT(FOnTextChanged, OnSelectionChanged)

		SLATE_ARGUMENT(const FSlateBrush*, MenuButtonBrush)

		SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)

		SLATE_ARGUMENT(bool, AllowAddNewOptions)

		/** The max height of the combo box menu */
		SLATE_ARGUMENT(float, MaxListHeight)

		/** Allow setting the visibility of the search box dynamically */
		SLATE_ATTRIBUTE(EVisibility, SearchVisibility)

	SLATE_END_ARGS()

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the combo box
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Requests a list refresh after updating options
	 */
	void RefreshOptions();

private:

	/** Generate a row for the InItem in the combo box's list (passed in as OwnerTable). Do this by calling the user-specified OnGenerateWidget */
	TSharedRef<ITableRow> GenerateMenuItemRow(TSharedRef<FFilteredOption> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**  */
	void OnGetChildren(TSharedRef<FFilteredOption> InItem, TArray<TSharedRef<FFilteredOption>>& OutChildren);

	/** Called if the menu is closed */
	void OnMenuOpenChanged(bool bOpen);

	/** Invoked when the selection in the list changes */
	void OnSelectionChanged_Internal(TSharedPtr<FFilteredOption> ProposedSelection, ESelectInfo::Type SelectInfo);

	/** Invoked when the search text changes */
	void OnSearchTextChanged(const FText& ChangedText);

	/** Sets the current selection to the first valid match when user presses enter in the filter box */
	void OnSearchTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** The item style to use. */
	const FTableRowStyle* ItemStyle;

	/** The padding around each menu row */
	FMargin MenuRowPadding;

private:
	/** Delegate that is invoked when the selected item in the combo box changes */
	FOnTextChanged OnSelectionChanged;
	/** The search field used for the combox box's contents */
	TSharedPtr<SEditableTextBox> SearchField;
	/** The ListView that we pop up; visualized the available options. */
	TSharedPtr< SComboTreeType > ComboTreeView;
	/** The Scrollbar used in the ListView. */
	TSharedPtr< SScrollBar > CustomScrollbar;

	/** Updated whenever search text is changed */
	FText SearchText;

	/** Source data for this combo box */
	const TArray< TSharedRef<FFilteredOption> >* OptionsSource;

	/** Filtered list that is actually displayed */
	TArray< TSharedRef<FFilteredOption> > FilteredOptionsSource;
	TArray< TSharedRef<FFilteredOption> > FilteredRootOptionsSource;

	/** Copied to replace the image. */
	FComboButtonStyle OurComboButtonStyle;

	bool bAllowAddNewOptions = false;
};

