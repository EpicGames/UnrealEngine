// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableSkeletonViewer.h"

#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STreeView.h"

class ITableRow;

#define LOCTEXT_NAMESPACE "MutableSkeletonViewer"

/** Row on the tree representing the bone structure of the UE::Mutable::Private::FMesh. It represents the UI side of the BoneDefinition */
class SMutableMeshBoneTreeRow : public STableRow<TSharedPtr<FBoneDefinition>>
{
public:

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FBoneDefinition>& InRowItem)
	{
		RowItem = InRowItem;

		this->ChildSlot
		[
			SNew(SHorizontalBox)

			// Expander Arrow
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]

			// Bone Name
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock).
				Text(FText::FromName(InRowItem->BoneName))
			]
		];


		STableRow< TSharedPtr<FBoneDefinition> >::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(true)
			, InOwnerTableView );
	}

private:

	TSharedPtr<FBoneDefinition> RowItem;
};


void SMutableSkeletonViewer::Construct(const FArguments& InArgs)
{
	const FText BonesDataTitle = LOCTEXT("BoneTreeTitle", "Bone Tree : ");

	constexpr int32 IndentationSpace = 16;
	constexpr int32 AfterTitleSpacing = 4;

	// Tree of bones -----------------
	ChildSlot
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock).
			Text(BonesDataTitle)
		]

		+ SVerticalBox::Slot()
		.Padding(IndentationSpace, AfterTitleSpacing)
		[
			SNew(SScrollBox)

			+ SScrollBox::Slot()
			[
				SAssignNew(BoneTree,STreeView<TSharedPtr<FBoneDefinition>>)
				.TreeItemsSource(&RootBones)
				.OnGenerateRow(this, &SMutableSkeletonViewer::OnGenerateBoneTreeRow)
				.OnGetChildren(this, &SMutableSkeletonViewer::OnGetChildrenFromBoneDefinition)
				.SelectionMode(ESelectionMode::None)
			]
		]
	];
	// ---------------------------------
	
	SetSkeleton(InArgs._Skeleton);
}

void SMutableSkeletonViewer::SetSkeleton(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> InSkeleton)
{
	if (InSkeleton != Skeleton)
	{
		Skeleton = InSkeleton;

		FillBoneDefinitionArrays();
	}

	ExpandFullBoneTree();
}


void SMutableSkeletonViewer::FillBoneDefinitionArrays()
{
	ChildBones.Empty();
	RootBones.Empty();
	
	if (!Skeleton)
	{
		BoneTree->RequestTreeRefresh();
		return;
	}

	const int32 BoneCount = Skeleton->GetNumBones();
	for (int32 BoneIndex = 0; BoneIndex < BoneCount; BoneIndex++)
	{
		// Generate a new definition object
		TSharedPtr<FBoneDefinition> NewBoneDefinition = MakeShareable(new FBoneDefinition());

		// Save the bone data
		NewBoneDefinition->BoneIndex = BoneIndex;
		NewBoneDefinition->BoneName = Skeleton->BoneNames[BoneIndex];

		// Get the parent node if any
		const int32 ParentBoneIndex = Skeleton->BoneParents[BoneIndex];
		if (ParentBoneIndex != INDEX_NONE)
		{
			NewBoneDefinition->ParentBoneIndex = ParentBoneIndex;
			NewBoneDefinition->ParentBoneName = Skeleton->BoneNames[ParentBoneIndex];

			// Store child bones
			ChildBones.Add(NewBoneDefinition);
		}
		else
		{
			// Store the root bones on their own data structure separated from the child bones
			RootBones.Add(NewBoneDefinition);
		}
	}

	// Add default definition to tell the user no bone has been found
	if (RootBones.Num() == 0)
	{
		// Save the bone data
		const TSharedPtr<FBoneDefinition> NewBoneDefinition = MakeShareable(new FBoneDefinition());
		NewBoneDefinition->BoneName = FName("No bones found...");
		RootBones.Add(NewBoneDefinition);
	}

	// Make sure the tree gets refreshed
	BoneTree->RequestTreeRefresh();

}

TSharedRef<ITableRow> SMutableSkeletonViewer::OnGenerateBoneTreeRow(TSharedPtr<FBoneDefinition> InBone, const TSharedRef<STableViewBase>& OwnerTable) const
{
	TSharedRef<SMutableMeshBoneTreeRow> Row = SNew(SMutableMeshBoneTreeRow, OwnerTable, InBone);
	return Row;
}

void SMutableSkeletonViewer::OnGetChildrenFromBoneDefinition(TSharedPtr<FBoneDefinition> InBone, TArray<TSharedPtr<FBoneDefinition>>& OutChildrenBones)
{
	const int32 TargetBoneIndex = InBone->BoneIndex;
	// Iterate over the bone definitions and save the ones that do have as parent the bone we have selected
	for (TSharedPtr< FBoneDefinition>& PossibleChildBoneDefinition : ChildBones)
	{
		if (PossibleChildBoneDefinition->ParentBoneIndex == TargetBoneIndex)
		{
			OutChildrenBones.Add(PossibleChildBoneDefinition);
		}
	}
}

void SMutableSkeletonViewer::ExpandFullBoneTree()
{
	for (const TSharedPtr< FBoneDefinition>& RootBoneDefinition : RootBones)
	{
		BoneTree->SetItemExpansion(RootBoneDefinition, true);
	}

	for (const TSharedPtr< FBoneDefinition>& ChildBoneDefinition : ChildBones)
	{
		BoneTree->SetItemExpansion(ChildBoneDefinition, true);
	}
}

#undef LOCTEXT_NAMESPACE
