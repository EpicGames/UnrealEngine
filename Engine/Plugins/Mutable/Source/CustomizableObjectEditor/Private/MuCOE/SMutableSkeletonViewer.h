// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Mesh.h"
#include "Widgets/SCompoundWidget.h"

class STableViewBase;
template <typename ItemType> class STreeView;

class ITableRow;

/** Container with the information required to generate the tree of bones */
struct FBoneDefinition
{
	FName ParentBoneName = FName("");
	int32 ParentBoneIndex = INDEX_NONE;

	FName BoneName = FName("");
	int32 BoneIndex = INDEX_NONE;
};

/** Object responsible of showing with a tree view slate the bones present on the provided mutable skeleton */
class SMutableSkeletonViewer : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SMutableSkeletonViewer) {}
		SLATE_ARGUMENT_DEFAULT(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton>, Skeleton) { nullptr };
	SLATE_END_ARGS()

public:
	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Sets the mutable skeleton object to be inspected */
	void SetSkeleton(UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> InSkeleton);
	
private:


	/** The mutable skeleton whose data is being inspected */
	UE::Mutable::Private::TManagedPtr<const UE::Mutable::Private::FSkeleton> Skeleton;
	
	/** Bones data: Bone definitions used to generate the bone tree structure */
	TArray<TSharedPtr<FBoneDefinition>> RootBones;
	TArray<TSharedPtr< FBoneDefinition>> ChildBones;

	/** UI Tree showing the bone data */
	TSharedPtr<STreeView<TSharedPtr<FBoneDefinition>>> BoneTree;

	/** Fills the bone TArrays that hold the root and child bone definitions */
	void FillBoneDefinitionArrays();
	
	/** 
	* Method responsible of generating each row for the tree of bones 
	* @param InBone - The mesh bone definition that is being used as blueprint for the actual UI row
	* @param OwnerTable - The parent UI element that is going to be expanded with the new row object
	* @return The new row object as the base interface used for all the table rows
	*/
	TSharedRef<ITableRow> OnGenerateBoneTreeRow(TSharedPtr<FBoneDefinition> InBone, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/** 
	* Fills the parameter OutChildrenBones with all the child bones from a selected Bone
	* @param InBone - The mesh bone definition that is being used as blueprint for the actual UI row
	* @param OutChildrenBones - An array that end up holding all the bone definitions that are children of the InBone. It can end up
	* being empty if no children bones are found for the InBone provided.
	*/
	void OnGetChildrenFromBoneDefinition(TSharedPtr<FBoneDefinition> InBone, TArray<TSharedPtr<FBoneDefinition>>& OutChildrenBones);
	
	/** Sets all rows of the tree as expanded */
	void ExpandFullBoneTree();
	
};
