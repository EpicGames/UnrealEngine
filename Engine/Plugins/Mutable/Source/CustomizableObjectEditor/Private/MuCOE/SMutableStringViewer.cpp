// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableStringViewer.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

void SMutableStringViewer::Construct(const FArguments& InArgs)
{
	// Set a default value for the displayed text
	if (InArgs._DefaultText.IsSet())
	{
		StringValue = InArgs._DefaultText.Get();
	}
	
	const FText StringValueTitle = LOCTEXT("StringValueTitle", "String Value : ");

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Title
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(StringValueTitle)
		]

		// Value
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).
			Text(this, &SMutableStringViewer::GetValue)
		]
	];
}


void SMutableStringViewer::SetString(const FText& InString)
{
	this->StringValue = InString;
}

FText SMutableStringViewer::GetValue() const
{
	return this->StringValue;
}

#undef LOCTEXT_NAMESPACE
