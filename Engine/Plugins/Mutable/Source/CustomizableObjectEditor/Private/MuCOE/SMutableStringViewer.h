// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

/**
* Widget designed to display the value of a string.
*/
class SMutableStringViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutableStringViewer) {}
		SLATE_ATTRIBUTE(FText, DefaultText)
	SLATE_END_ARGS()

	/** Builds the widget */
	void Construct(const FArguments& InArgs);

	/** Set the Mutable Bool to be used for this widget */
	void SetString(const FText& InString);

private:

	/** The value to be shown by the widget's UI */
	FText StringValue;

	/**
	* Callback used by the widget to get the value to display on the UI
	* @return The variable StringValue as an FText
	*/
	FText GetValue() const;
};
