// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SMutableTagListWidget.h"

#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectEditorStyle.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObject.h"
#include "MuCOE/Nodes/CustomizableObjectNodeObjectGroup.h"
#include "MuCOE/Nodes/CONodeSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeModifierBase.h"
#include "MuCOE/Nodes/CONodeModifierExtendSkeletalMeshSection.h"
#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTextureVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeFloatVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeColorVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeMeshVariation.h"
#include "MuCOE/Nodes/CustomizableObjectNodeComponentVariation.h"
#include "MuCOE/UnrealEditorPortabilityHelpers.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/Attribute.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


void SMutableTagComboBox::Construct(const FArguments& InArgs)
{
	Node = InArgs._Node;
	bAllowInternalTags = InArgs._AllowInternalTags;

	RefreshOptions();

	SMutableSearchComboBox::Construct(
		SMutableSearchComboBox::FArguments()
		.OptionsSource(&TagComboOptionsSource)
		.AllowAddNewOptions(true)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.MenuButtonBrush(InArgs._MenuButtonBrush)
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		.ForegroundColor(FSlateColor(FLinearColor(.35f, .35f, .35f)))
		.Content()
		[
			InArgs._Content.Widget
		]
	);
}


TSharedPtr<SMutableSearchComboBox::FFilteredOption> SMutableTagComboBox::AddNodeHierarchyOptions(UEdGraphNode* InNode, TMap<UEdGraphNode*, TSharedPtr<SMutableSearchComboBox::FFilteredOption>>& AddedOptions)
{
	TSharedPtr<SMutableSearchComboBox::FFilteredOption> Option;
	if (TSharedPtr<SMutableSearchComboBox::FFilteredOption>* FoundCached = AddedOptions.Find(InNode))
	{
		Option = *FoundCached;
	}

	if (InNode && !Option)
	{
		// Add parents
		TSharedPtr<SMutableSearchComboBox::FFilteredOption> ParentOption;
		{
			// Add this node as placeholder in the cache to prevent infinite loops because of graph loops.
			AddedOptions.Add(InNode, Option);

			// Pin traversal
			for (const UEdGraphPin* Pin : InNode->Pins)
			{
				if (Pin && Pin->Direction == EEdGraphPinDirection::EGPD_Output
					&& !Pin->LinkedTo.IsEmpty() && Pin->LinkedTo[0])
				{
					UEdGraphNode* ParentNode = Pin->LinkedTo[0]->GetOwningNode();

					ParentOption = AddNodeHierarchyOptions(ParentNode, AddedOptions);

					// We are ok with just one parent
					if (ParentOption)
					{
						break;
					}
				}
			}

			// Node internal references
			if (!ParentOption)
			{
				// Is it an object referencing an external group?
				if (UCustomizableObjectNodeObject* ObjectNode = Cast<UCustomizableObjectNodeObject>(InNode))
				{
					if (ObjectNode->ParentObject)
					{
						UEdGraphNode* ExternalParentNode = GetCustomizableObjectExternalNode<UEdGraphNode>(ObjectNode->ParentObject, ObjectNode->ParentObjectGroupId);
						ParentOption = AddNodeHierarchyOptions(ExternalParentNode, AddedOptions);
					}
				}
			}

			// TODO: Support import/export nodes
		}

		// Is it a relevant type that we want to show in the hierarchy?
		if (UCONodeSkeletalMeshSection* MeshSectionNode = Cast<UCONodeSkeletalMeshSection>(InNode))
		{
			Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
			Option->Parent = ParentOption;
			Option->DisplayOption = MeshSectionNode->GetInternalTagDisplayName();
			if (bAllowInternalTags)
			{
				Option->ActualOption = MeshSectionNode->GetInternalTag();
			}
			TagComboOptionsSource.Add(Option.ToSharedRef());
		}

		else if (UCONodeModifierExtendSkeletalMeshSection* ExtendMeshSectionNode = Cast<UCONodeModifierExtendSkeletalMeshSection>(InNode))
		{
			Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
			Option->Parent = ParentOption;
			UMaterialInterface* Material = ExtendMeshSectionNode->ReferenceMaterial;
			Option->DisplayOption = ExtendMeshSectionNode->GetInternalTagDisplayName();
			if (bAllowInternalTags)
			{
				Option->ActualOption = ExtendMeshSectionNode->GetInternalTag();
			}
			TagComboOptionsSource.Add(Option.ToSharedRef());
		}

		else if (UCustomizableObjectNodeObject* ObjectNode = Cast<UCustomizableObjectNodeObject>(InNode))
		{
			Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
			Option->Parent = ParentOption;
			Option->DisplayOption = ObjectNode->GetObjectName();
			if (Option->DisplayOption.IsEmpty())
			{
				Option->DisplayOption = "Unnamed Object";
			}
			TagComboOptionsSource.Add(Option.ToSharedRef());
		}

		else if (UCustomizableObjectNodeObjectGroup* GroupNode = Cast<UCustomizableObjectNodeObjectGroup>(InNode))
		{
			Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
			Option->Parent = ParentOption;
			Option->DisplayOption = GroupNode->GetGroupName();
			if (Option->DisplayOption.IsEmpty())
			{
				Option->DisplayOption = "Unnamed Group";
			}
			TagComboOptionsSource.Add(Option.ToSharedRef());
		}

		else if (UCustomizableObjectNodeModifierBase* ModifierNode = Cast<UCustomizableObjectNodeModifierBase>(InNode))
		{
			Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
			Option->Parent = ParentOption;
			Option->DisplayOption = ModifierNode->GetNodeTitle(ENodeTitleType::ListView).ToString();
			TagComboOptionsSource.Add(Option.ToSharedRef());
		}

		// Overwrite in to cache, to prevent loops
		AddedOptions.Add(InNode, Option);

		// If this node wasn't of interest, maybe its parent was.
		if (!Option)
		{
			Option = ParentOption;
		}
	}

	return Option;
}


void SMutableTagComboBox::RefreshOptions()
{
	TagComboOptionsSource.SetNum(0, EAllowShrinking::No);

	// Scan all potential receivers
	UCustomizableObject* ThisNodeObject = GraphTraversal::GetObject(*Node);
	UCustomizableObject* RootObject = GraphTraversal::GetRootObject(ThisNodeObject);

	TSet<UCustomizableObject*> AllCustomizableObject;
	GraphTraversal::GetAllObjectsInGraph(RootObject, AllCustomizableObject);

	TMap<UEdGraphNode*, TSharedPtr<SMutableSearchComboBox::FFilteredOption>> AddedOptions;

	for (const UCustomizableObject* CustObject : AllCustomizableObject)
	{
		if (!CustObject)
		{
			continue;
		}

		for (const TObjectPtr<UEdGraphNode>& CandidateNode : CustObject->GetPrivate()->GetSource()->Nodes)
		{
			UCustomizableObjectNode* Typed = Cast<UCustomizableObjectNode>(CandidateNode);
			if (!Typed)
			{
				continue;
			}

			const TArray<FString>* EnableTags = Typed->GetEnableTagsArray();
			if (EnableTags)
			{
				TSharedPtr<SMutableSearchComboBox::FFilteredOption> NodeOption = AddNodeHierarchyOptions(Typed, AddedOptions);

				for (const FString& OneTag : *EnableTags)
				{
					if (OneTag.IsEmpty())
					{
						continue;
					}

					if (!bAllowInternalTags && UCustomizableObjectNode::IsInternalTag(OneTag))
					{
						continue;
					}

					TSharedRef Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
					Option->ActualOption = OneTag;
					Option->DisplayOption = OneTag;
					Option->Parent = NodeOption;
					TagComboOptionsSource.Add(Option);
				}
			}

			UCustomizableObjectNodeModifierBase* TypedModifier = Cast<UCustomizableObjectNodeModifierBase>(CandidateNode);
			if (TypedModifier)
			{
				for (const FString& OneTag : TypedModifier->RequiredTags)
				{
					TSharedPtr<SMutableSearchComboBox::FFilteredOption> NodeOption = AddNodeHierarchyOptions(Typed, AddedOptions);

					if (OneTag.IsEmpty())
					{
						continue;
					}

					if (!bAllowInternalTags && UCustomizableObjectNode::IsInternalTag(OneTag))
					{
						continue;
					}

					TSharedRef Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
					Option->ActualOption = OneTag;
					Option->DisplayOption = OneTag;
					Option->Parent = NodeOption;
					TagComboOptionsSource.Add(Option);
				}
			}

			// Generic variation nodes
			{
				UCustomizableObjectNodeVariation* VariationNode = Cast<UCustomizableObjectNodeVariation>(CandidateNode);
				if (VariationNode)
				{
					for (const FCustomizableObjectVariation& Var : VariationNode->VariationsData)
					{
						TSharedPtr<SMutableSearchComboBox::FFilteredOption> NodeOption = AddNodeHierarchyOptions(VariationNode, AddedOptions);

						if (Var.Tag.IsEmpty())
						{
							continue;
						}

						if (!bAllowInternalTags && UCustomizableObjectNode::IsInternalTag(Var.Tag))
						{
							continue;
						}

						TSharedRef Option = MakeShared<SMutableSearchComboBox::FFilteredOption>();
						Option->ActualOption = Var.Tag;
						Option->DisplayOption = Var.Tag;
						Option->Parent = NodeOption;
						TagComboOptionsSource.Add(Option);
					}
				}
			}
		}
	}
}


void SMutableTagListWidget::Construct(const FArguments& InArgs)
{
	Node = InArgs._Node;
	TagArray = InArgs._TagArray;
	EmptyListText = InArgs._EmptyListText;

	OnTagListChangedDelegate = InArgs._OnTagListChanged;

	RefreshOptions();

	ChildSlot
	[
		SNew(SVerticalBox)

		// Header with the "add tag" UI
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(10.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SAssignNew(this->TagCombo, SMutableTagComboBox)
						.Node(Node)
						.MenuButtonBrush( FAppStyle::GetBrush(TEXT("Icons.PlusCircle")) )
						.AllowInternalTags( InArgs._AllowInternalTags )
						.OnSelectionChanged(this, &SMutableTagListWidget::OnTagComboBoxSelectionChanged)
				]
		]

		// List of tags
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(this->TagListWidget, SListView<TSharedPtr<FTagUIData>>)
				.ListItemsSource(&CurrentTagsSource)
				.OnGenerateRow(this, &SMutableTagListWidget::GenerateTagListItemRow)
				.SelectionMode(ESelectionMode::None)
		]

		// Shown only if there are no tags defined
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(STextBlock)
				.Text(EmptyListText)
				.AutoWrapText(true)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Visibility_Lambda([&]()
					{
						return (!TagArray || TagArray->IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
					})
		]
	];
}


void SMutableTagListWidget::RefreshOptions()
{
	if (TagCombo)
	{
		TagCombo->RefreshOptions();
	}

	// Current Tags
	if (TagArray)
	{
		CurrentTagsSource.SetNum(0, EAllowShrinking::No);

		for (const FString& OneTag : *TagArray)
		{
			TSharedPtr<FTagUIData> Data = MakeShared<FTagUIData>();

			UCustomizableObjectNode* InternalTagNode = nullptr;
			UCustomizableObject* InternalTagObject = nullptr;
			bool bIsInternal = Node->FindNodeForInternalTag( OneTag, InternalTagNode, InternalTagObject );
			if (bIsInternal && InternalTagNode)
			{
				Data->DisplayName = InternalTagNode->GetInternalTagDisplayName();
			}
			else
			{
				Data->DisplayName = OneTag;
			}

			Data->Tag = OneTag;
			CurrentTagsSource.Add(Data);
		}
	}

	if (TagListWidget)
	{
		TagListWidget->RequestListRefresh();
	}
}


void SMutableTagListWidget::OnTagComboBoxSelectionChanged(const FText& NewText)
{
	FString OneTag = NewText.ToString();
	if (TagArray && !TagArray->Contains(OneTag))
	{
		TagArray->Add(OneTag);
		OnTagListChangedDelegate.ExecuteIfBound();
	}
}


TSharedRef<ITableRow> SMutableTagListWidget::GenerateTagListItemRow(TSharedPtr<FTagUIData> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (!InItem)
	{
		return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
			[
				SNew(STextBlock)
					.Text(FText::FromString("No item."))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			];
	}

	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		.Padding(0)
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(4)
				[
					SNew(STextBlock)
						.Text(FText::FromString(InItem->DisplayName))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0)
				[
					SNew(SButton)
						.ContentPadding(2)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ToolTipText(LOCTEXT("RemoveModifierTag", "Remove this tag from the modifier."))
						.OnClicked_Lambda([this, InItem]()
							{
								if (Node && InItem)
								{
									if (TagArray->Contains(InItem->Tag))
									{
										TagArray->Remove(InItem->Tag);
										OnTagListChangedDelegate.ExecuteIfBound();
									}
								}
								return FReply::Handled();
							})
						[
							SNew(SImage)
								.Image(FAppStyle::GetBrush(TEXT("Icons.MinusCircle")))
								.ColorAndOpacity(FSlateColor(FLinearColor(.35f, .35f, .35f)))
						]
				]
		];
}



#undef LOCTEXT_NAMESPACE
