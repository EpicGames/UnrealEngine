// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "MuCOE/SMutableSearchComboBox.h"
#include "IDetailCustomization.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/ITableRow.h"

namespace ESelectInfo { enum Type : int; }

class FString;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UCustomizableObjectNode;
class UCustomizableObjectNodeObject;
class UEdGraphNode;


/** */
class SMutableTagComboBox : public SMutableSearchComboBox
{
public:

	SLATE_BEGIN_ARGS(SMutableTagComboBox)
		: _Node(nullptr)
		, _AllowInternalTags(true)
		{}
		/** Slot for this button's content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(UCustomizableObjectNode*, Node)
		SLATE_ARGUMENT(const FSlateBrush*, MenuButtonBrush)
		SLATE_ARGUMENT(bool, AllowInternalTags)
		SLATE_EVENT(FOnTextChanged, OnSelectionChanged)

	SLATE_END_ARGS()

	/** */
	void Construct(const FArguments& InArgs);

	/** */
	void RefreshOptions();

private:

	class UCustomizableObjectNode* Node = nullptr;

	TArray< TSharedRef<SMutableSearchComboBox::FFilteredOption> > TagComboOptionsSource;

	bool bAllowInternalTags = true;

	/** */
	TSharedPtr<SMutableSearchComboBox::FFilteredOption> AddNodeHierarchyOptions(UEdGraphNode* Node, TMap<UEdGraphNode*, TSharedPtr<SMutableSearchComboBox::FFilteredOption>>& AddedOptions);

};


/** */
class SMutableTagListWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMutableTagListWidget)
		: _Node(nullptr)
		, _TagArray(nullptr)
		, _AllowInternalTags(true)
		{}

		SLATE_ARGUMENT(UCustomizableObjectNode*, Node)
		SLATE_ARGUMENT(TArray<FString>*, TagArray)
		SLATE_ARGUMENT(FText, EmptyListText)
		SLATE_ARGUMENT(bool, AllowInternalTags)

		SLATE_EVENT(FSimpleDelegate, OnTagListChanged)

	SLATE_END_ARGS()

	/** */
	void Construct(const FArguments& InArgs);

	/** */
	FSimpleDelegate OnTagListChangedDelegate;

	/** */
	void RefreshOptions();

private:

	class UCustomizableObjectNode* Node = nullptr;
	TArray<FString>* TagArray = nullptr;
	FText EmptyListText;

	TSharedPtr<SMutableTagComboBox> TagCombo;
	void OnTagComboBoxSelectionChanged(const FText& NewText);

	struct FTagUIData
	{
		FString Tag;
		FString DisplayName;
	};
	TSharedPtr<SListView<TSharedPtr<FTagUIData>>> TagListWidget;
	TArray< TSharedPtr<FTagUIData> > CurrentTagsSource;

	/** */
	TSharedRef<ITableRow> GenerateTagListItemRow(TSharedPtr<FTagUIData> InItem, const TSharedRef<STableViewBase>& OwnerTable);

};

