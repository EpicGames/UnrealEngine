// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SComboBox.h"

/** Typed SComboBox with a translator to FText. */
template<typename OptionType>
class SMutableTextComboBox : public SComboBox<OptionType>
{
public:
	typedef typename SComboBox<OptionType>::FOnSelectionChanged FOnSelectionChanged;
	typedef typename SComboBox<OptionType>::NullableOptionType NullableOptionType;
	
	DECLARE_DELEGATE_RetVal_OneParam(FText, FTranslateDelegate, OptionType);

	SLATE_BEGIN_ARGS(SMutableTextComboBox<OptionType>) {}
		// Super arguments.
		SLATE_ARGUMENT(const TArray<OptionType>*, Options)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_ARGUMENT(NullableOptionType, InitiallySelectedItem)
		// Own arguments.
		/** Translates OptionType to FText. */
		SLATE_EVENT(FTranslateDelegate, Translate)
	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs)
	{
		Translate = InArgs._Translate;
		
		SComboBox<OptionType>::Construct(
			typename SComboBox<OptionType>::FArguments()
			.OptionsSource(InArgs._Options)
			.OnSelectionChanged(InArgs._OnSelectionChanged)
			.InitiallySelectedItem(InArgs._InitiallySelectedItem)
			.OnGenerateWidget_Lambda([this](OptionType InOption)
			{
				return SNew(STextBlock).Text(this, &SMutableTextComboBox<OptionType>::GetText, InOption);
			})
			[
				SNew(STextBlock).Text(this, &SMutableTextComboBox<OptionType>::GetSelectedItemText) 
			]
		);
	}

private:
	/** Translate delegate. */
	FTranslateDelegate Translate;

	/** If the translate delegate is bounded, returns call it and get the FText. */
	FText GetText(OptionType Option) const
	{
		if (Translate.IsBound())
		{
			return Translate.Execute(Option);
		}
		else
		{
			return FText();
		}
	}

	/** Get the the text if there is a selected item. */
	FText GetSelectedItemText() const
	{
		SMutableTextComboBox<OptionType>* NonConstThis = const_cast<SMutableTextComboBox<OptionType>*>(this); // GetSelectedItem is non constant
		if (NullableOptionType NullableSelected =  NonConstThis->GetSelectedItem(); TListTypeTraits<OptionType>::IsPtrValid(NullableSelected))
		{
			OptionType ActuallySelected = TListTypeTraits<OptionType>::NullableItemTypeConvertToItemType(NullableSelected);
			return GetText(ActuallySelected);
		}
		else
		{
			return FText();
		}
	}
};