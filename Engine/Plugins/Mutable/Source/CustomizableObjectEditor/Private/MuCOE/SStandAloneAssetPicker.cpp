// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/SStandAloneAssetPicker.h"

#include "Rendering/SlateRenderer.h"
#include "Styling/SlateTypes.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Engine/Texture2D.h"
#include "Widgets/Layout/SBorder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "MuCO/LoadUtils.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SStandAloneAssetPicker::Construct(const FArguments& InArgs)
{
	OnAssetSelected = InArgs._OnAssetSelected;
	OnGetAllowedClasses = InArgs._OnGetAllowedClasses;
	CurrentAsset = InArgs._InitialAsset;

	ChildSlot
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.0f,3.0f,5.0f,0.0f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.Padding(FMargin(0.0f, 0.0f, 4.0f, 4.0f))
					.BorderImage(FAppStyle::Get().GetBrush("PropertyEditor.AssetTileItem.DropShadow"))
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						.Padding(1.0f)
						[
							SNew(SBorder)
							.Padding(0)
							.BorderImage(FStyleDefaults::GetNoBrush())
							[
								SAssignNew(ThumbnailContainer, SBox)
							]
						]
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Right)
						[
							SAssignNew(AssetPickerAnchor, SMenuAnchor)
								.Placement(MenuPlacement_AboveAnchor)
								.OnGetMenuContent(this, &SStandAloneAssetPicker::OnGenerateAssetPicker)
						]
						+ SVerticalBox::Slot()
						.FillHeight(1)
						[
							SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
								.OnClicked(this, &SStandAloneAssetPicker::OnClicked)
								.ToolTipText(LOCTEXT("PickButtonLabel", "Pick Asset"))
								.ContentPadding(0.0f)
								.ForegroundColor(FSlateColor::UseForeground())
								.IsFocusable(false)
								[
									SNew(SImage)
										.Image(FAppStyle::GetBrush("PropertyWindow.Button_PickAsset"))
										.ColorAndOpacity(FSlateColor::UseForeground())
								]
						]
				]
		];

	RefreshThumbnail();
}

void SStandAloneAssetPicker::RefreshThumbnail()
{
	FIntPoint ThumbnailSize(128, 128);
	TSharedPtr<FAssetThumbnailPool> Pool = UThumbnailManager::Get().GetSharedThumbnailPool();
	AssetThumbnail = MakeShareable(new FAssetThumbnail(CurrentAsset, ThumbnailSize.X, ThumbnailSize.Y, Pool));

	FAssetThumbnailConfig AssetThumbnailConfig;

	ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig);

	ThumbnailContainer->SetContent(ThumbnailWidget.ToSharedRef());
}

FReply SStandAloneAssetPicker::OnClicked()
{
	AssetPickerAnchor->SetIsOpen(true);
	return FReply::Handled();
}

TSharedRef<SWidget> SStandAloneAssetPicker::OnGenerateAssetPicker()
{
	TArray<const UClass*> AllowedClasses;
	OnGetAllowedClasses.ExecuteIfBound(AllowedClasses);

	if (AllowedClasses.Num() == 0)
	{
		// Assume all classes are allowed
		AllowedClasses.Add(UObject::StaticClass());
	}
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	for (auto ClassIt = AllowedClasses.CreateConstIterator(); ClassIt; ++ClassIt)
	{
		const UClass* Class = (*ClassIt);
		AssetPickerConfig.Filter.ClassPaths.Add(Class->GetClassPathName());
	}
	// Allow child classes
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	// Set a delegate for setting the asset from the picker
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SStandAloneAssetPicker::OnAssetSelectedFromPicker);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SStandAloneAssetPicker::OnAssetEnterPressedFromPicker);
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bAllowNullSelection = true;
	// Use the list view by default
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	AssetPickerConfig.InitialAssetSelection = FAssetData(CurrentAsset);

	TSharedRef<SWidget> MenuContent =
		SNew(SBox)
		.HeightOverride(300.0f)
		.WidthOverride(300.0f)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				]
		];

	return MenuContent;
}

void SStandAloneAssetPicker::OnAssetSelectedFromPicker(const struct FAssetData& AssetData)
{
	AssetPickerAnchor->SetIsOpen(false);

	CurrentAsset = UE::Mutable::Private::LoadObject(AssetData);

	OnAssetSelected.ExecuteIfBound(CurrentAsset.Get());

	RefreshThumbnail();
}

void SStandAloneAssetPicker::OnAssetEnterPressedFromPicker(const TArray<struct FAssetData>& AssetData)
{
	AssetPickerAnchor->SetIsOpen(false);

	CurrentAsset = nullptr;
	if (AssetData.Num() > 0)
	{
		CurrentAsset = UE::Mutable::Private::LoadObject(AssetData[0]);
	}

	OnAssetSelected.ExecuteIfBound(CurrentAsset.Get());
	RefreshThumbnail();
}


#undef LOCTEXT_NAMESPACE
