// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "PropertyCustomizationHelpers.h"
#include "AssetThumbnail.h"


class SStandAloneAssetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStandAloneAssetPicker) {}
		SLATE_EVENT(FOnGetAllowedClasses, OnGetAllowedClasses)
		SLATE_EVENT(FOnAssetSelected, OnAssetSelected)
		SLATE_ARGUMENT(TObjectPtr<UObject>, InitialAsset)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
private:

	FReply OnClicked();

	TSharedRef<SWidget> OnGenerateAssetPicker();

	void OnAssetSelectedFromPicker(const struct FAssetData& AssetData);
	void OnAssetEnterPressedFromPicker(const TArray<struct FAssetData>& AssetData);
	
	void RefreshThumbnail();

private:

	/** Menu anchor for opening and closing the asset picker */
	TSharedPtr<SMenuAnchor> AssetPickerAnchor;

	FOnGetAllowedClasses OnGetAllowedClasses;
	FOnAssetSelected OnAssetSelected;
	TObjectPtr<UObject> CurrentAsset;

	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	TSharedPtr<SWidget> ThumbnailWidget;
	TSharedPtr<SBox> ThumbnailContainer;
};

