// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/UnrealBakeHelpers.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectMipDataProvider.h"
#include "MuT/UnrealPixelFormatOverride.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"

#include "TextureResource.h"



UObject* FUnrealBakeHelpers::BakeHelper_DuplicateAsset(UObject* Object, TMap<UObject*, UObject*>* ReplacementMap, const bool bGenerateConstantMaterialInstances,
											  			FResourceBakingData& InOutPackageData)
{
	const FString FinalPackageFullName = InOutPackageData.GetResourceComposedFullPath();
	const FString FinalPackageShortName = InOutPackageData.GetResourceComposedName();
	
	// Create or LOAD the package (if said package exists)
	UPackage* Package = CreatePackage(*FinalPackageFullName);
	check(Package);
	Package->FullyLoad();
	
	UMaterialInstance* MatInstance = Cast<UMaterialInstance>(Object);

	// Duplicated object we will return to the caller
	UObject* DupObject = nullptr;
	
	// Only generate UMaterialInstanceConstant constant material instances if the original material is actually an instance, so check it here. Otherwise just duplicate
	if (bGenerateConstantMaterialInstances && MatInstance)
	{
		UMaterialInstanceConstantFactoryNew* MaterialFactory = NewObject<UMaterialInstanceConstantFactoryNew>();
		MaterialFactory->InitialParent = MatInstance->Parent;

		UMaterialInstanceConstant* MatInstanceConst = CastChecked<UMaterialInstanceConstant>(MaterialFactory->FactoryCreateNew(UMaterialInstanceConstant::StaticClass(), Package, FName(FinalPackageShortName), RF_NoFlags, nullptr, GWarn));

		TMap<int32, UTexture*> EmptyTextureReplacementMap;
		CopyAllMaterialParameters<UMaterialInstanceConstant>(*MatInstanceConst, *MatInstance, EmptyTextureReplacementMap);
		
		DupObject = MatInstanceConst;
	}
	else
	{
		FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(Object, Package, *FinalPackageShortName, RF_AllFlags, nullptr, EDuplicateMode::Normal);
		DupObject = StaticDuplicateObjectEx(Params);
	}

	if (DupObject)
	{
		DupObject->SetFlags(RF_Public | RF_Standalone);
		DupObject->ClearFlags(RF_Transient);

		// The garbage collector is called in the middle of the bake process, and this can destroy this temporary objects. 
		// We add them to the garbage root to prevent this. This will avoid them being unloaded while the editor is running, but this
		// action is not used often.
		DupObject->AddToRoot();
		DupObject->MarkPackageDirty();

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(DupObject);

		if (ReplacementMap)
		{
			// Replace all references
			ReplacementMap->Add(Object, DupObject);

			constexpr EArchiveReplaceObjectFlags ReplaceFlags = (EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
			FArchiveReplaceObjectRef<UObject> ReplaceAr(DupObject, *ReplacementMap, ReplaceFlags);
		}
	}

	return DupObject;
}


namespace
{

	void CopyTextureProperties(UTexture2D* Texture, const UTexture2D* SourceTexture)
	{
		MUTABLE_CPUPROFILER_SCOPE(CopyTextureProperties)

		Texture->NeverStream = SourceTexture->NeverStream;

		Texture->SRGB = SourceTexture->SRGB;
		Texture->Filter = SourceTexture->Filter;
		Texture->LODBias = SourceTexture->LODBias;

		Texture->MipGenSettings = SourceTexture->MipGenSettings;
		Texture->CompressionNone = SourceTexture->CompressionNone;

		Texture->LODGroup = SourceTexture->LODGroup;
		Texture->AddressX = SourceTexture->AddressX;
		Texture->AddressY = SourceTexture->AddressY;
	}
}


UTexture2D* FUnrealBakeHelpers::BakeHelper_CreateAssetTexture(UTexture2D* SourceTexture, const UTexture* OrgTex,
																TMap<UObject*, UObject*>* ReplacementMap, FResourceBakingData& InOutPackageData)
{
	const bool bIsMutableTexture = !SourceTexture->Source.IsValid();
	if (!bIsMutableTexture)
	{
		return Cast<UTexture2D>(BakeHelper_DuplicateAsset(SourceTexture, ReplacementMap,false, InOutPackageData));
	}

	int32 SourceTextureSizeX = SourceTexture->GetPlatformData()->SizeX;
	int32 SourceTextureSizeY = SourceTexture->GetPlatformData()->SizeY;
	EPixelFormat SourceTexturePixelFormat = SourceTexture->GetPlatformData()->PixelFormat;
	ETextureSourceFormat PixelFormat = (SourceTexturePixelFormat == PF_BC4 || SourceTexturePixelFormat == PF_G8) ? TSF_G8 : TSF_BGRA8;

	// Begin duplicate texture
	const FString FinalPackageFullName = InOutPackageData.GetResourceComposedFullPath();
	const FString FinalPackageShortName = InOutPackageData.GetResourceComposedName();

	UPackage* Package = CreatePackage(*FinalPackageFullName);
	check(Package);
	Package->FullyLoad();

	UTexture2D* DupTex = NewObject<UTexture2D>(Package, *FinalPackageShortName, RF_Public | RF_Standalone);

	// The garbage collector is called in the middle of the bake process, and this can destroy this temporary object. 
	// We add them to the garbage root to prevent this. This will avoid them being unloaded while the editor is running, but this
	// action is not used often.
	DupTex->AddToRoot();
	DupTex->MarkPackageDirty();
	DupTex->PreEditChange(nullptr);

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(DupTex);

	// Replace all references
	if (ReplacementMap)
	{
		ReplacementMap->Add(SourceTexture, DupTex);
		
		constexpr EArchiveReplaceObjectFlags ReplaceFlags = (EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
		FArchiveReplaceObjectRef<UObject> ReplaceAr(DupTex, *ReplacementMap, ReplaceFlags);
	}

	// End duplicate texture

	CopyTextureProperties(DupTex, SourceTexture);

	DupTex->RemoveUserDataOfClass(UMutableTextureMipDataProviderFactory::StaticClass());
	if (OrgTex)
	{
		DupTex->CompressionSettings = OrgTex->CompressionSettings;
	}
	
	// Mutable textures only have platform data. We need to build the source data for them to be assets.
	int32 MipCount = SourceTexture->GetPlatformData()->Mips.Num();
	if (!MipCount)
	{
		UE_LOGF(LogMutable, Warning, "Bake Instances: Empty texture found [%ls].", *SourceTexture->GetName());
		
		DupTex->PostEditChange();
		return DupTex;
	}
	
	DupTex->Source.Init(SourceTextureSizeX, SourceTextureSizeY, 1, 1, PixelFormat);

	// Create a mutable image from the platform data.
	UE::Mutable::Private::EImageFormat MutableFormat = UnrealToMutablePixelFormat(SourceTexture->GetPlatformData()->PixelFormat, SourceTexture->HasAlphaChannel());
	TSharedPtr<UE::Mutable::Private::FImage> PlatformImage = MakeShared<UE::Mutable::Private::FImage>(SourceTextureSizeX, SourceTextureSizeY, 1, MutableFormat, UE::Mutable::Private::EInitializationType::NotInitialized);

	constexpr int32 MipIndex = 0;
	const uint8* SourceData = reinterpret_cast<const uint8*>(SourceTexture->GetPlatformData()->Mips[MipIndex].BulkData.LockReadOnly());
	check(SourceData); // A mutable-generated texture should always contain platform data
	int32 PlatformDataSize = SourceTexture->GetPlatformData()->Mips[MipIndex].BulkData.GetBulkDataSize();
	check(PlatformImage->GetDataSize()== PlatformDataSize);
	FMemory::Memcpy(PlatformImage->GetLODData(0), SourceData, PlatformDataSize);
	SourceData = nullptr;
	SourceTexture->GetPlatformData()->Mips[MipIndex].BulkData.Unlock();

	// Reformat the mutable image
	UE::Mutable::Private::EImageFormat UncompressedMutableFormat = UE::Mutable::Private::EImageFormat::RGBA_UByte;
	switch (SourceTexture->GetPlatformData()->PixelFormat)
	{
	case PF_G8:
	case PF_L8:
	case PF_A8:
	case PF_BC4:
		UncompressedMutableFormat = UE::Mutable::Private::EImageFormat::L_UByte;
		break;

	default:
		break;
	}

	UE::Mutable::Private::FImageOperator ImOp = UE::Mutable::Private::FImageOperator::GetDefault(UE::Mutable::Private::FImageOperator::FImagePixelFormatFunc());
	int32 Quality = 4; // Doesn't matter for decompression.
	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> UncompressedImage = ImOp.ImagePixelFormat(Quality, PlatformImage.Get(), UncompressedMutableFormat);

	// Copy the decompressed data to the texture source data
	int32 SourceDataSize = DupTex->Source.CalcMipSize(MipIndex);

	TArrayView<uint8> UncompressedView = UncompressedImage->DataStorage.GetLOD(0);
	
	// If this doesn't match, more cases have to be added to the switch above.
	check(UncompressedView.Num() == SourceDataSize);

	uint8* Dest = DupTex->Source.LockMip(MipIndex);
	check(Dest);
	FMemory::Memcpy(Dest, UncompressedView.GetData(), SourceDataSize);

	// Probably can be integrated in the pixel format
	const bool bNeedsRBSwizzle = PixelFormat == TSF_BGRA8;
	if (bNeedsRBSwizzle)
	{
		for (int32 x = 0; x < SourceTextureSizeX * SourceTextureSizeY; ++x)
		{
			uint8 temp = Dest[0];
			Dest[0] = Dest[2];
			Dest[2] = temp;
			Dest += 4;
		}
	}

	Dest = nullptr;
	DupTex->Source.UnlockMip(MipIndex);

	bool bNeeds_TC_Grayscale = PixelFormat == TSF_G8 || PixelFormat == TSF_G16;
	bool bDoNotCompress = SourceTexturePixelFormat == PF_R8G8B8A8;
	bool bIsNormalMap = SourceTexturePixelFormat == PF_BC5;

	if (bNeeds_TC_Grayscale || bDoNotCompress || bIsNormalMap)
	{
		FTextureFormatSettings Settings;

		Settings.SRGB = SourceTexture->SRGB;

		if (bNeeds_TC_Grayscale)
		{
			// If compression settings are not set to TC_Grayscale the texture will get a DXT format
			// instead of G8 or G16.
			Settings.CompressionSettings = TC_Grayscale;
			DupTex->CompressionSettings = TC_Grayscale;
		}

		if (bDoNotCompress)
		{
			// In this case keep the RGBA format instead of compressing to DXT
			Settings.CompressionNone = true;
			DupTex->CompressionNone = true;
		}

		if (bIsNormalMap)
		{
			Settings.CompressionSettings = TC_Normalmap;
			DupTex->CompressionSettings = TC_Normalmap;
		}
		
		DupTex->SetLayerFormatSettings(0, Settings);
	}

	DupTex->PostEditChange();

	return DupTex;
}

