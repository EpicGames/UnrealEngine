// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialParameters.h"
#include "HAL/Platform.h"
#include "Containers/Map.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionTextureBase.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInterface.h"
#include "MuCOE/CustomizableObjectInstanceBakingUtils.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

enum class EPackageSaveResolutionType : uint8;
class UObject;
class USkeletalMesh;
class UTexture2D;
class UTexture;
class UMaterialInterface;
class UMaterial;


class FUnrealBakeHelpers
{
public:

	static UE_API UObject* BakeHelper_DuplicateAsset(UObject* Object,
											  TMap<UObject*, UObject*>* ReplacementMap, const bool bGenerateConstantMaterialInstances, FResourceBakingData& InOutPackageData);

	/**
	 * Duplicates a texture asset. Duplicates Mutable and non Mutable textures.
	 *
	 * @param OrgTex Original source texture from which a Mutable texture has been generated. Only required when SrcTex is a Mutable texture.
	 */
	static UE_API UTexture2D* BakeHelper_CreateAssetTexture(UTexture2D* SourceTexture, const UTexture* OrgTex,
													 TMap<UObject*, UObject*>* ReplacementMap, FResourceBakingData& InOutPackageData);

	template<typename MaterialType>
	static void CopyAllMaterialParameters(MaterialType& DestMaterial, UMaterialInterface& OriginMaterial, const TMap<int32, UTexture*>& TextureReplacementMap)
	{
		// Copy scalar parameters
		{
			TArray<FMaterialParameterInfo> ScalarParameterInfoArray;
			TArray<FGuid> GuidArray;
			OriginMaterial.GetAllScalarParameterInfo(ScalarParameterInfoArray, GuidArray);
			for (const FMaterialParameterInfo& Param : ScalarParameterInfoArray)
			{
				float Value = 0.f;
				if (OriginMaterial.GetScalarParameterValue(Param, Value, true))
				{
					DestMaterial.SetScalarParameterValueEditorOnly(Param.Name, Value);
				}
			}
			
		}

		// Copy vector parameters
		{
			TArray<FMaterialParameterInfo> VectorParameterInfoArray;
			TArray<FGuid> GuidArray;
			OriginMaterial.GetAllVectorParameterInfo(VectorParameterInfoArray, GuidArray);
			for (const FMaterialParameterInfo& Param : VectorParameterInfoArray)
			{
				FLinearColor Value;
				if (OriginMaterial.GetVectorParameterValue(Param, Value, true))
				{
					DestMaterial.SetVectorParameterValueEditorOnly(Param.Name, Value);
				}
			}
		}

		// Copy switch parameters								
		{
			TArray<FMaterialParameterInfo> StaticSwitchParameterInfoArray;
			TArray<FGuid> GuidArray;
			OriginMaterial.GetAllStaticSwitchParameterInfo(StaticSwitchParameterInfoArray, GuidArray);
			for (int32 ParametersArrayIndex = 0; ParametersArrayIndex < StaticSwitchParameterInfoArray.Num(); ++ParametersArrayIndex)
			{
				bool Value = false;
				FGuid ExpressionsGuid;
				if (OriginMaterial.GetStaticSwitchParameterValue(StaticSwitchParameterInfoArray[ParametersArrayIndex].Name, Value, ExpressionsGuid, true))
				{
					// For some reason UMaterialInstance::SetStaticSwitchParameterValueEditorOnly signature is different than UMaterial::SetStaticSwitchParameterValueEditorOnly
					if constexpr (std::is_same_v<MaterialType, UMaterial>)
					{
						DestMaterial.SetStaticSwitchParameterValueEditorOnly(StaticSwitchParameterInfoArray[ParametersArrayIndex].Name, Value, ExpressionsGuid);
					}
					else if (std::is_same_v<MaterialType, UMaterialInstanceConstant>)
					{
						DestMaterial.SetStaticSwitchParameterValueEditorOnly(StaticSwitchParameterInfoArray[ParametersArrayIndex].Name, Value);
					}
					else
					{
						static_assert(
							std::is_same_v<MaterialType, UMaterial> ||
							std::is_same_v<MaterialType, UMaterialInstanceConstant>);
					}
				}
			}
		}

		// Replace Textures
		{
			TArray<FMaterialParameterInfo> OutParameterInfo;
			TArray<FGuid> Guids;
			OriginMaterial.GetAllTextureParameterInfo(OutParameterInfo, Guids);
			for (const TPair<int32, UTexture*>& It : TextureReplacementMap)
			{
				if (OutParameterInfo.IsValidIndex(It.Key))
				{
					DestMaterial.SetTextureParameterValueEditorOnly(OutParameterInfo[It.Key].Name, It.Value);
				}
			}			
		}

		// Fix potential errors compiling materials due to Sampler Types
		for (const TObjectPtr<UMaterialExpression>& Expression : DestMaterial.GetMaterial()->GetExpressions())
		{
			if (UMaterialExpressionTextureBase* MatExpressionTexBase = Cast<UMaterialExpressionTextureBase>(Expression))
			{
				MatExpressionTexBase->AutoSetSampleType();
			}
		}

		DestMaterial.PreEditChange(NULL);
		DestMaterial.PostEditChange();
	}
};

#undef UE_API
