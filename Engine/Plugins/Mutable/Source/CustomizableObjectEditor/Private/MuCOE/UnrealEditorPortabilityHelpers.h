// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphPin.h"
#include "MuCO/UnrealPortabilityHelpers.h"

#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "Materials/Material.h"
#include "Editor.h"
#include "Engine/StaticMesh.h"
#include "Engine/DataTable.h"
#include "Engine/Texture.h"
#include "Runtime/Launch/Resources/Version.h"


//---------------------------------------------------------------
// Helpers to ease portability across unreal engine versions
//---------------------------------------------------------------

#if ENGINE_MAJOR_VERSION==5 && ENGINE_MINOR_VERSION>=1

#define UE_MUTABLE_GET_BRUSH			FAppStyle::GetBrush
#define UE_MUTABLE_GET_FLOAT			FAppStyle::GetFloat
#define UE_MUTABLE_GET_MARGIN			FAppStyle::GetMargin
#define UE_MUTABLE_GET_FONTSTYLE		FAppStyle::GetFontStyle
#define UE_MUTABLE_GET_WIDGETSTYLE		FAppStyle::GetWidgetStyle
#define UE_MUTABLE_GET_COLOR			FAppStyle::GetColor
#define UE_MUTABLE_GET_SLATECOLOR		FAppStyle::GetSlateColor

#elif ENGINE_MAJOR_VERSION==5 && ENGINE_MINOR_VERSION==0

#define UE_MUTABLE_GET_BRUSH			FAppStyle::Get().GetBrush
#define UE_MUTABLE_GET_FLOAT			FAppStyle::Get().GetFloat
#define UE_MUTABLE_GET_MARGIN			FAppStyle::Get().GetMargin
#define UE_MUTABLE_GET_FONTSTYLE		FAppStyle::Get().GetFontStyle
#define UE_MUTABLE_GET_WIDGETSTYLE		FAppStyle::Get().GetWidgetStyle
#define UE_MUTABLE_GET_COLOR			FAppStyle::Get().GetColor
#define UE_MUTABLE_GET_SLATECOLOR		FAppStyle::Get().GetSlateColor

#endif


inline FString Helper_GetPinName(const UEdGraphPin* Pin)
{
	return Pin ? Pin->PinName.ToString() : FString();
}
