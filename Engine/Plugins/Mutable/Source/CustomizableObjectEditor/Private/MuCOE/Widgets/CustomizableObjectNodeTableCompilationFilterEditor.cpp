// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Widgets/CustomizableObjectNodeTableCompilationFilterEditor.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IDetailsView.h"
#include "MuCOE/Nodes/CustomizableObjectNodeTable.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IPropertyTypeCustomization> FCustomizableObjectNodeTableCompilationFilterEditor::MakeInstance()
{
	return MakeShared<FCustomizableObjectNodeTableCompilationFilterEditor>();
}


void FCustomizableObjectNodeTableCompilationFilterEditor::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}


void FCustomizableObjectNodeTableCompilationFilterEditor::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);

	// Get table node
	if (OuterObjects.Num())
	{
		Node = Cast<UCustomizableObjectNodeTable>(OuterObjects[0]);

		if (!Node.IsValid())
		{
			return;
		}
	}

	ColumnPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTableNodeCompilationFilter, FilterColumn));
	TSharedPtr<IPropertyHandle> CompilationFilterPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTableNodeCompilationFilter, Filters));
	TSharedPtr<IPropertyHandle> OperationTypePropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTableNodeCompilationFilter, OperationType));

	if (ColumnPropertyHandle && CompilationFilterPropertyHandle && OperationTypePropertyHandle)
	{
		TSharedPtr<FString> CurrentCompilationFilterColumn = GenerateCompilationFilterColumnComboBoxOptions();

		ChildBuilder.AddProperty(ColumnPropertyHandle.ToSharedRef()).CustomWidget()
			.NameContent()
			[
				ColumnPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SAssignNew(CompilationFilterColumnComboBox, STextComboBox)
					.InitiallySelectedItem(CurrentCompilationFilterColumn)
					.OptionsSource(&CompilationFilterColumnOptionNames)
					.OnComboBoxOpening(this, &FCustomizableObjectNodeTableCompilationFilterEditor::OnOpenCompilationFilterColumnComboBox)
					.OnSelectionChanged(this, &FCustomizableObjectNodeTableCompilationFilterEditor::OnCompilationFilterColumnComboBoxSelectionChanged)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(this, &FCustomizableObjectNodeTableCompilationFilterEditor::GetCompilationFilterColumnComboBoxTextColor)
			];

		ChildBuilder.AddProperty(CompilationFilterPropertyHandle.ToSharedRef());
		ChildBuilder.AddProperty(OperationTypePropertyHandle.ToSharedRef());
	}
}


TSharedPtr<FString> FCustomizableObjectNodeTableCompilationFilterEditor::GenerateCompilationFilterColumnComboBoxOptions()
{
	TSharedPtr<FString> CurrentSelection;
	FName CurrentOption;
	ColumnPropertyHandle->GetValue(CurrentOption);

	if (!Node.IsValid())
	{
		return CurrentSelection;
	}

	const UScriptStruct* TableStruct = Node->GetTableNodeStruct();
	
	CompilationFilterColumnOptionNames.Reset();

	FText NothingSelectedText = LOCTEXT("NothingSelectedText", "- Nothing Selected -");
	CompilationFilterColumnOptionNames.Add(MakeShareable(new FString(NothingSelectedText.ToString())));

	if (!TableStruct)
	{
		return CurrentSelection;
	}

	// Iterating struct Options
	for (TFieldIterator<FProperty> It(TableStruct); It; ++It)
	{
		FProperty* ColumnProperty = *It;

		if (!ColumnProperty)
		{
			continue;
		}

		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ColumnProperty))
		{
			ColumnProperty = ArrayProperty->Inner;
		}

		if (UCustomizableObjectNodeTable::SupportedFilterTypes.Contains(ColumnProperty->GetClass()))
		{
			TSharedRef<FString> Option = MakeShareable(new FString(ColumnProperty->GetDisplayNameText().ToString()));
			CompilationFilterColumnOptionNames.Add(Option);

			if (*Option == CurrentOption.ToString())
			{
				CurrentSelection = CompilationFilterColumnOptionNames.Last();
			}
		}
	}

	if (!CurrentSelection)
	{
		if (!CurrentOption.IsNone())
		{
			CompilationFilterColumnOptionNames.Add(MakeShareable(new FString(CurrentOption.ToString())));
			CurrentSelection = CompilationFilterColumnOptionNames.Last();
		}
		else
		{
			CurrentSelection = CompilationFilterColumnOptionNames[0];
		}
	}

	return CurrentSelection;
}


void FCustomizableObjectNodeTableCompilationFilterEditor::OnOpenCompilationFilterColumnComboBox()
{
	TSharedPtr<FString> CurrentSelection = GenerateCompilationFilterColumnComboBoxOptions();

	if (CompilationFilterColumnComboBox.IsValid())
	{
		CompilationFilterColumnComboBox->ClearSelection();
		CompilationFilterColumnComboBox->RefreshOptions();
		CompilationFilterColumnComboBox->SetSelectedItem(CurrentSelection);
	}
}


void FCustomizableObjectNodeTableCompilationFilterEditor::OnCompilationFilterColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	FName CurrentOption;
	ColumnPropertyHandle->GetValue(CurrentOption);

	if (Selection && CurrentOption != FName(*Selection)
		&& (SelectInfo == ESelectInfo::OnKeyPress || SelectInfo == ESelectInfo::OnMouseClick))
	{
		if (Selection == CompilationFilterColumnOptionNames[0])
		{
			ColumnPropertyHandle->SetValue(NAME_None);
		}
		else
		{
			ColumnPropertyHandle->SetValue(FName(*Selection));
		}
	}
}


FSlateColor FCustomizableObjectNodeTableCompilationFilterEditor::GetCompilationFilterColumnComboBoxTextColor() const
{
	FName CurrentOption;
	ColumnPropertyHandle->GetValue(CurrentOption);

	if ((Node.IsValid() && Node->FindColumnProperty(CurrentOption)) || CurrentOption.IsNone())
	{
		return FSlateColor::UseForeground();
	}

	// Table Struct null or does not contain the selected property anymore
	return FSlateColor(FLinearColor(0.9f, 0.05f, 0.05f, 1.0f));
}


void FCustomizableObjectNodeTableCompilationFilterEditor::OnCompilationFilterColumnComboBoxSelectionReset()
{
	if (Node.IsValid())
	{
		ColumnPropertyHandle->SetValue(NAME_None);

		if (CompilationFilterColumnComboBox.IsValid())
		{
			GenerateCompilationFilterColumnComboBoxOptions();
			CompilationFilterColumnComboBox->SetSelectedItem(CompilationFilterColumnOptionNames[0]);
			CompilationFilterColumnComboBox->RefreshOptions();
		}
	}
}


#undef LOCTEXT_NAMESPACE
