// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class STextComboBox;
class UCustomizableObjectNodeTable;

class FCustomizableObjectNodeTableCompilationFilterEditor : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	// Pointer to the node represented in this details
	TWeakObjectPtr<UCustomizableObjectNodeTable> Node;

	TSharedPtr<IPropertyHandle> ColumnPropertyHandle;

	// Pinter to the structure instance, useful to get the array index of the property
	TSharedPtr<IPropertyHandle> StructPropertyHandlePtr;
	
	// Array with the name of the Version columns
	TArray<TSharedPtr<FString>> CompilationFilterColumnOptionNames;

	// ComboBox widget to select a VersionColumn from the NodeTable
	TSharedPtr<STextComboBox> CompilationFilterColumnComboBox;

	// Generates MutableMetadata columns combobox options
	// Returns the current selected option or a null pointer
	TSharedPtr<FString> GenerateCompilationFilterColumnComboBoxOptions();

	// Callback to regenerate the combobox options
	void OnOpenCompilationFilterColumnComboBox();

	// OnComboBoxSelectionChanged Callback for Layout ComboBox
	void OnCompilationFilterColumnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	// Sets the combo box selection color
	FSlateColor GetCompilationFilterColumnComboBoxTextColor() const;

	// OnComboBoxSelectionChanged Callback for Layout ComboBox
	void OnCompilationFilterColumnComboBoxSelectionReset();
};
