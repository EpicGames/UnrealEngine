// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Widgets/CustomizableObjectVariationCustomization.h"

#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOE/CustomizableObjectEditorUtilities.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "MuCOE/GraphTraversal.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeVariation.h"
#include "MuCOE/SMutableTagListWidget.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectDetails"


TSharedRef<IPropertyTypeCustomization> FCustomizableObjectVariationCustomization::MakeInstance()
{
	return MakeShared<FCustomizableObjectVariationCustomization>();
}


void FCustomizableObjectVariationCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);
	PropertyHandle = StructPropertyHandle;

	constexpr bool bRecurse = false;
	TagPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCustomizableObjectVariation, Tag), bRecurse);
	check(TagPropertyHandle);

	if (OuterObjects.Num())
	{
		BaseObjectNode = Cast<UCustomizableObjectNode>(OuterObjects[0]);
		
		if (!BaseObjectNode.IsValid())
		{
			return;
		}
	}

	InHeaderRow
		.NameContent().HAlign(EHorizontalAlignment::HAlign_Fill).VAlign(EVerticalAlignment::VAlign_Center)
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent().MinDesiredWidth(300.0f)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor::Transparent)
			[
				SNew(SMutableTagComboBox)				
					.Node(BaseObjectNode.Get())
					.MenuButtonBrush(FAppStyle::GetBrush(TEXT("Icons.Search")))
					.OnSelectionChanged_Lambda([&](const FText& NewText)
						{
							TagPropertyHandle->SetValue(NewText.ToString());
						})
					[
						SNew(SEditableTextBox)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text_Lambda([&]() 
								{ 
									FString Value;
									if (TagPropertyHandle)
									{
										TagPropertyHandle->GetValue(Value);
									}

									Value = BaseObjectNode->GetTagDisplayName(Value);

									return FText::FromString(Value); 
								})
							.OnTextCommitted_Lambda([&](const FText& NewText, ETextCommit::Type)
								{
									TagPropertyHandle->SetValue(NewText.ToString());
								})
					]
			]
		]
		.OverrideResetToDefault(FResetToDefaultOverride::Create(FSimpleDelegate::CreateSP(this, &FCustomizableObjectVariationCustomization::ResetSelectedParameterButtonClicked)));
}


void FCustomizableObjectVariationCustomization::ResetSelectedParameterButtonClicked()
{
	TagPropertyHandle->SetValue(FString());
}


#undef LOCTEXT_NAMESPACE
