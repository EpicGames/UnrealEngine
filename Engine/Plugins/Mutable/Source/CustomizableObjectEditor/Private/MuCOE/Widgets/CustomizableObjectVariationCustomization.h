// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyHandleArray;
class SBoneSelectionWidget;
class UCustomizableObjectNode;

struct FReferenceSkeleton;

enum class ECheckBoxState : uint8;

class FCustomizableObjectVariationCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {};

private:

	/** Weak pointer to the Customizable Object node that contains this property */
	TWeakObjectPtr<UCustomizableObjectNode> BaseObjectNode;

	/** Runtime Parameter Name property of a State */
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> TagPropertyHandle;

	/** Callback of the function OverrideResetToDefault */
	void ResetSelectedParameterButtonClicked();

};
