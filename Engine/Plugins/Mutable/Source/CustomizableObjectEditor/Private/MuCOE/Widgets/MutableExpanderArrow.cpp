// Copyright Epic Games, Inc. All Rights Reserved.

#include "MutableExpanderArrow.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/ITableRow.h"


void SMutableExpanderArrow::Construct(const FArguments& InArgs, const TSharedPtr<ITableRow>& TableRow)
{
	SExpanderArrow::Construct(SExpanderArrow::FArguments().ShouldDrawWires(true),TableRow);
}


int32 SMutableExpanderArrow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	static constexpr float WireThickness = 2.0f;
	static constexpr float HalfWireThickness = WireThickness / 2.0f;
	
	static const FName NAME_VerticalBarBrush = TEXT("WhiteBrush");
	const float Indent = GetIndentAmount();
	const FSlateBrush* VerticalBarBrush = (StyleSet == nullptr) ? nullptr : StyleSet->GetBrush(NAME_VerticalBarBrush);

	if (ShouldDrawWires.Get() == true && VerticalBarBrush != nullptr)
	{
		const TSharedPtr<ITableRow> OwnerRow = OwnerRowPtr.Pin();
		
		// Draw vertical wires to indicate paths to parent nodes.
		const TBitArray<>& NeedsWireByLevel = OwnerRow->GetWiresNeededByDepth();
		const int32 NumLevels = NeedsWireByLevel.Num();
		for (int32 Level = 0; Level < NumLevels; ++Level)
		{
			if (NeedsWireByLevel[Level])
			{
				const float CurrentIndent = Indent * Level;
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, 0))),
					VerticalBarBrush,
					ESlateDrawEffect::None,
					GetLevelColor(Level)
				);
			}
		}

		const float HalfCellHeight = 0.5f * AllottedGeometry.Size.Y;
		
		// For items that are the last expanded child in a list, we need to draw a special angle connector wire.
		if (const bool bIsLastChild = OwnerRow->IsLastChild())
		{
			const float CurrentIndent = Indent * (NumLevels-1);
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, HalfCellHeight + HalfWireThickness), FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, 0))),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				GetLevelColor(NumLevels -1)
			);
		}
		
		// If this item is expanded, we need to draw a 1/2-height the line down to its first child cell.
		if ( const bool bItemAppearsExpanded = OwnerRow->IsItemExpanded() && OwnerRow->DoesItemHaveChildren())
		{
			const float CurrentIndent = Indent * NumLevels;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, HalfCellHeight+ HalfWireThickness), FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, HalfCellHeight- HalfWireThickness))),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				GetLevelColor(NumLevels)
			);
		}

		
		// Draw horizontal connector from parent wire to child.
		if (NumLevels > 1)
		{
			float LeafDepth = OwnerRow->DoesItemHaveChildren() ? 10.f : 0.0f;
			const float HorizontalWireStart = (NumLevels - 1)*Indent;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(
					FVector2D(AllottedGeometry.Size.X - HorizontalWireStart - WireThickness - LeafDepth, WireThickness),
					FSlateLayoutTransform(FVector2D(HorizontalWireStart + WireThickness - 3.f, 0.5f*(AllottedGeometry.Size.Y - WireThickness)))
				),
				VerticalBarBrush,
				ESlateDrawEffect::None,
				GetLevelColor(NumLevels - 1)
			);
		}	
	}

	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	return LayerId;
}


FLinearColor SMutableExpanderArrow::GetLevelColor(const int32 Level) const
{
	constexpr float Alpha = 0.6f;
	constexpr int32 ArraySize = 4;
	constexpr FLinearColor AvailableColors[ArraySize] =
	{
		FLinearColor(0.40784313725490196f,0.5098039215686274f,0.08627450980392157f,Alpha),
		FLinearColor(0.6549019607843137f,0.3176470588235294f,0.0784313725490196f,Alpha),
		FLinearColor(0.6666666666666666f,0.10196078431372549f,0.3803921568627451f,Alpha),
		FLinearColor(0.2f,0.25098039215686274f,0.6392156862745098f, Alpha)
	};
	
	const int ColorIndex = Level % ArraySize;
	return AvailableColors[ColorIndex];
}