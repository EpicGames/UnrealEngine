// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/Views/SExpanderArrow.h"

/**
 * Custom expander arrow object where the lines drawn will be colored using a circular coloring pattern based
 * on the depth of the element to be drawn.
 */
class SMutableExpanderArrow final : public SExpanderArrow
{
public:
	SLATE_BEGIN_ARGS( SMutableExpanderArrow ){}
	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow );
	
protected:
	
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:

	/**
	 * Provided a level value this method will return the color to be used to paint it.
	 * @param Level The level whose draw color we want to know
	 * @return The color to use during the drawing of the provided level index
	 */
	FLinearColor GetLevelColor(const int32 Level) const;
};
