// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

/**
 * Slate containing a list view designed to contain high amounts of elements. Will separate the elements in pages so the parent
 * UI element drawing them can get the listview to fit without triggering any hard limit in the size of the containing slate object.
 * This class was created out of the necessity of having lists with hundreds of elements (mutable constant images for example).
 * 
 * @tparam ItemType The type of item the list view is holding
 */
template <typename ItemType>
class SMutableMultiPageListView final : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SMutableMultiPageListView<ItemType>)
			: _HostedListView(nullptr)
			, _ElementsToSeparateInPages(nullptr) {}

	    /** The list view we want to manipulate */
    	SLATE_ARGUMENT(TSharedPtr<SListView<ItemType>>, HostedListView)

	    /** The elements that got fed to the list view we are hosting */ 
	    SLATE_ARGUMENT(TSharedPtr<const TArray<ItemType>>, ElementsToSeparateInPages)
	    
    SLATE_END_ARGS()
	
		
	void Construct(const FArguments& InArgs);

private:
	
	/**
	 * Get the visibility state set for the navigation buttons box. If there is no need for the box existence it will then be collapsed
	 * @return The visibility state to use in the navigation panel
	 */
	EVisibility GetPageNavigationBoxVisibility() const;

	/**
	 * Changes the page to the previous one
	 * @return the reply status to know if the operation was handled or not.
	 */
	FReply OnBackButtonClicked();

	/**
	 * Changes the page to the next one
	 * @return the reply status to know if the operation was handled or not.
	 */
	FReply OnForwardButtonClicked();

	/**
	 * Sets the page to the first one
	 * @return the reply status to know if the operation was handled or not.
	 */
	FReply OnFullBackButtonClicked();

	/**
	 * Sets the page to the last one
	 * @return the reply status to know if the operation was handled or not.
	 */
	FReply OnFullForwardButtonClicked();

	/**
	 * Controls if the back button should or should not be enabled
	 * @return True if it should, false otherwise.
	 */
	bool ShouldBackButtonBeEnabled() const;

	/**
	 * Controls if the forward button should or should not be enabled
	 * @return True if it should, false otherwise.
	 */
	bool ShouldNextButtonBeEnabled() const;

	/**
	 * Handles the drawing of the current / total pages UI text.
	 * @return The text representing the currently displayed page
	 */
	FText OnDrawCurrentPageText() const;

public:

	/** Compute the elements to display for the current page */
	void RegeneratePage();

	/** Clear the selected item from the handled list view */
	void ClearSelection();
	
private:
	
	/**
	 * Amount of elements an array segment can hold. The lowest the more stable the UI drawing gets
	 * (avoids crash due to oversize slate Y size)
	 */
	const uint32 ElementsPerPage = 24;

	/** The index of the current page/segment. Will increase and decrease using the UI */
	uint32 CurrentSegmentIndex = 0;

	/**
	 * The computed total amount of segments. It is based in the amount of elements provided in the AllElements array and the
	 * value set in ElementsPerPage
	 */
	uint32 TotalAmountOfPages = 0;

	/** Elements being displayed in the current page */
	TArray<ItemType> CurrentSegmentElements;

	/** List view handled by this slate. */
	TSharedPtr<SListView<ItemType>> HostedListView;

	/** Array with all the elements to manage from the provided ListView*/
	TSharedPtr<const TArray<ItemType>> AllElements;
};



template <typename ItemType>
void SMutableMultiPageListView<ItemType>::Construct(const FArguments& InArgs)
{
	HostedListView = InArgs._HostedListView;
	AllElements = InArgs._ElementsToSeparateInPages;

	check(HostedListView);
	check(AllElements);

	// Compute the max amount of pages in case it needs to be updated
	TotalAmountOfPages = AllElements->Num() / ElementsPerPage;
	// Add one more segment if there are elements not filling a full page
	TotalAmountOfPages = AllElements->Num() % ElementsPerPage > 0 ? TotalAmountOfPages + 1 : TotalAmountOfPages;
	
	this->ChildSlot
	[
		SNew(SVerticalBox)

		// Buttons being used to change the segment to be displayed
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2,0)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SMutableMultiPageListView::GetPageNavigationBoxVisibility)

			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(FText(INVTEXT("|<")))
				.OnClicked(this,&SMutableMultiPageListView::OnFullBackButtonClicked)
				.IsEnabled(this,&SMutableMultiPageListView::ShouldBackButtonBeEnabled)
			]

			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(FText(INVTEXT("<")))
				.OnClicked(this,&SMutableMultiPageListView::OnBackButtonClicked)
				.IsEnabled(this,&SMutableMultiPageListView::ShouldBackButtonBeEnabled)
			]
			
			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text(this,&SMutableMultiPageListView::OnDrawCurrentPageText)
			]

			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(FText(INVTEXT(">")))
				.OnClicked(this,&SMutableMultiPageListView::OnForwardButtonClicked)
				.IsEnabled(this,&SMutableMultiPageListView::ShouldNextButtonBeEnabled)
			]

			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(FText(INVTEXT(">|")))
				.OnClicked(this,&SMutableMultiPageListView::OnFullForwardButtonClicked)
				.IsEnabled(this,&SMutableMultiPageListView::ShouldNextButtonBeEnabled)
			]
		]
		
		// List view showing the selected segment of elements
		+ SVerticalBox::Slot()
		.Padding(2,5)
		[
			HostedListView.ToSharedRef()
		]
	];
	
	// Generate the array to be used as first displaying elements. 
	RegeneratePage();
	
	// Tell the list to use our set of elements
	//  note: If this code throws a runtime error may be because no children slot has been setup in the ListView. That could happen if
	//  the list view provided did not get fully setup. For example, if no ItemsSource is defined no Child slot will be created and, therefore,
	//  the SetItemsSource method will fail to run.
	HostedListView->SetItemsSource(&CurrentSegmentElements);
}


template <typename ItemType>
EVisibility SMutableMultiPageListView<ItemType>::GetPageNavigationBoxVisibility() const
{
	return TotalAmountOfPages > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


template <typename ItemType>
FReply SMutableMultiPageListView<ItemType>::OnBackButtonClicked()
{
	CurrentSegmentIndex--;
	
	// Regenerate the proxy list with the elements on the currently selected list segment
	RegeneratePage();

	return FReply::Handled();
}


template <typename ItemType>
FReply SMutableMultiPageListView<ItemType>::OnForwardButtonClicked()
{
	CurrentSegmentIndex++;
	
	// Regenerate the proxy list with the elements on the currently selected list segment
	RegeneratePage();

	return FReply::Handled();
}


template <typename ItemType>
FReply SMutableMultiPageListView<ItemType>::OnFullBackButtonClicked()
{ 
	CurrentSegmentIndex = 0;
	
	// Regenerate the proxy list with the elements on the currently selected list segment
	RegeneratePage();

	return FReply::Handled();
}


template <typename ItemType>
FReply SMutableMultiPageListView<ItemType>::OnFullForwardButtonClicked()
{
	CurrentSegmentIndex = TotalAmountOfPages - 1;
	
	// Regenerate the proxy list with the elements on the currently selected list segment
	RegeneratePage();

	return FReply::Handled();
}


template <typename ItemType>
bool SMutableMultiPageListView<ItemType>::ShouldBackButtonBeEnabled() const
{
	return  CurrentSegmentIndex > 0;
}


template <typename ItemType>
bool SMutableMultiPageListView<ItemType>::ShouldNextButtonBeEnabled() const
{
	return TotalAmountOfPages >= 1 && CurrentSegmentIndex < TotalAmountOfPages - 1;
}


template <typename ItemType>
FText SMutableMultiPageListView<ItemType>::OnDrawCurrentPageText() const
{
	FString BuiltText = FString::FromInt(  CurrentSegmentIndex + 1);
	BuiltText += FString(" / ");
	BuiltText += FString::FromInt(TotalAmountOfPages);
	
	return FText::Format(LOCTEXT("PageNumber","Page : {0}"), FText::FromString(BuiltText));
}


template <typename ItemType>
void SMutableMultiPageListView<ItemType>::RegeneratePage()
{
	// Prepare the segment of elements to be filled up
	CurrentSegmentElements.SetNum(0);
	CurrentSegmentElements.Reserve(ElementsPerPage);
	
	const uint32 TotalElements = AllElements->Num();
	const uint32 StartingIndex = CurrentSegmentIndex * ElementsPerPage;
	const uint32 FinishIndex = StartingIndex + FMath::Min(ElementsPerPage, (TotalElements - StartingIndex));
	// Iterate over the main array from the point where the segment starts to the point where it ends. Stop before if needed
	for	(uint32 MainArrayIndex = StartingIndex ; MainArrayIndex < FinishIndex; ++ MainArrayIndex)
	{
		CurrentSegmentElements.Add(AllElements.Get()->GetData()[MainArrayIndex]);
	}
	
	if (HostedListView)
	{
		HostedListView->RequestListRefresh();
	}
}


template <typename ItemType>
void SMutableMultiPageListView<ItemType>::ClearSelection()
{
	if (HostedListView)
	{
		HostedListView->ClearSelection();
	}
}

#undef LOCTEXT_NAMESPACE
