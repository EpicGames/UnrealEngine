// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Widgets/SCustomizableObjectMacroLibraryList.h"

#include "Misc/MessageDialog.h"
#include "MuCOE/CustomizableObjectGraph.h"
#include "PropertyCustomizationHelpers.h"
#include "SPositiveActionButton.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/STreeView.h"


#define LOCTEXT_NAMESPACE "SCustomizableObjectMacroLibraryList"

/** Row on the tree representing the bone structure of the UE::Mutable::Private::FMesh. It represents the UI side of the BoneDefinition */
class SMacroLibraryTreeRow : public STableRow<TSharedPtr<FMacroTreeEntry>>
{
public:
	DECLARE_DELEGATE_OneParam(FOnRemoveMacroButtonClickedDelegate, UCustomizableObjectMacro*)

	SLATE_BEGIN_ARGS(SMacroLibraryTreeRow) {}
		SLATE_ARGUMENT(FOnRemoveMacroButtonClickedDelegate, OnRemoveMacro)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<FMacroTreeEntry>& InRowItem)
	{
		RowItem = InRowItem;
		OnRemoveMacro = Args._OnRemoveMacro;

		this->ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f, 5.0f, 5.0f)
				.AutoWidth()
				[
					SNew(SImage).Image(FAppStyle::GetBrush(TEXT("GraphEditor.Macro_16x")))
				]

				+ SHorizontalBox::Slot()
				.Padding(5.0f, 5.0f, 0.0f, 5.0f)
				.AutoWidth()
				[
					SNew(STextBlock).Text(this, &SMacroLibraryTreeRow::GetMacroName)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.HAlign(EHorizontalAlignment::HAlign_Right)
				.AutoWidth()
				[
					PropertyCustomizationHelpers::MakeClearButton(
						FSimpleDelegate::CreateSP(this, &SMacroLibraryTreeRow::RemoveMacro), LOCTEXT("RemoveMacroTooltip", "Remove Macro."))
				]
			];

		STableRow< TSharedPtr<FMacroTreeEntry> >::ConstructInternal(
			STableRow::FArguments()
			.Style(FAppStyle::Get(), "TableView.Row")
			.ShowSelection(true)
			, InOwnerTableView);
	}


	FText GetMacroName() const
	{
		FText ErrorMsg = LOCTEXT("NullMacroRowError", "#Error# - Null Macro");
		return RowItem->Macro.IsValid() ? FText::FromName(RowItem->Macro->Name) : ErrorMsg;
	}


	void RemoveMacro()
	{
		if (RowItem && RowItem->Macro.IsValid())
		{
			FText Msg = LOCTEXT("RemoveMacroTextWindow", "Are you sure you want to remove the Macro?");

			if (FMessageDialog::Open(EAppMsgType::OkCancel, Msg) == EAppReturnType::Ok)
			{
				OnRemoveMacro.ExecuteIfBound(RowItem->Macro.Get());
			}
		}
	}

private:

	TSharedPtr<FMacroTreeEntry> RowItem;
	FOnRemoveMacroButtonClickedDelegate OnRemoveMacro;
};

void SCustomizableObjectMacroLibraryList::Construct(const FArguments& InArgs)
{
	MacroLibrary = InArgs._MacroLibrary;
	OnAddMacroButtonClicked = InArgs._OnAddMacroButtonClicked;
	OnSelectMacro = InArgs._OnSelectMacro;
	OnRemoveMacro = InArgs._OnRemoveMacro;

	GenerateRowView();

	SAssignNew(ListView, SListView<TSharedPtr<FMacroTreeEntry>>)
	.SelectionMode(ESelectionMode::Single)
	.ListItemsSource(&ListViewSource)
	.OnGenerateRow_Lambda([this](TSharedPtr<FMacroTreeEntry> InItem, const TSharedRef<STableViewBase>& InOwnerTable) -> TSharedRef<ITableRow>
	{
		TSharedRef<SMacroLibraryTreeRow> Row = SNew(SMacroLibraryTreeRow, InOwnerTable, InItem)
		.OnRemoveMacro(SMacroLibraryTreeRow::FOnRemoveMacroButtonClickedDelegate::CreateSP(this, &SCustomizableObjectMacroLibraryList::OnRemoveCurrentMacro));

		return Row;
	})
	.OnSelectionChanged_Lambda([this](TSharedPtr<FMacroTreeEntry> InItem, ESelectInfo::Type SelectInfo)
	{
		if (InItem.IsValid() && InItem->Macro.IsValid())
		{
			OnSelectMacro.ExecuteIfBound(InItem->Macro.Get());
		}
	});

	// Select the firs macro on create the list view
	if (ListView.IsValid() && ListViewSource.Num())
	{
		ListView->SetSelection(ListViewSource[0]);
	}

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SPositiveActionButton)
			.OnClicked_Lambda([this]() 
			{
				if (OnAddMacroButtonClicked.IsBound())
				{
					OnAddMacroButtonClicked.Execute();
					GenerateRowView();
					
					ListView->SetSelection(ListViewSource.Last());
					ListView->RequestListRefresh();
					
					return FReply::Handled();
				}

				return FReply::Unhandled();
			})
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ListView.ToSharedRef()
		]
	];
}


void SCustomizableObjectMacroLibraryList::GenerateRowView()
{
	ListViewSource.Empty();

	for (UCustomizableObjectMacro* Macro : MacroLibrary->Macros)
	{
		TSharedPtr<FMacroTreeEntry> NewGraphDefinition = MakeShareable(new FMacroTreeEntry());
		NewGraphDefinition->Macro = Macro;

		ListViewSource.Add(NewGraphDefinition);
	}
	
	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

void SCustomizableObjectMacroLibraryList::OnRemoveCurrentMacro(UCustomizableObjectMacro* MacroToDelete)
{
	if (MacroToDelete)
	{
		OnRemoveMacro.ExecuteIfBound(MacroToDelete);
		GenerateRowView();
	}
}

void SCustomizableObjectMacroLibraryList::SetSelectedMacro(const UCustomizableObjectMacro& MacroToSelect)
{
	if (ListView.IsValid())
	{
		for (TSharedPtr<FMacroTreeEntry> MacroElement : ListViewSource)
		{
			if (MacroElement->Macro == &MacroToSelect)
			{
				ListView->SetSelection(MacroElement);
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
