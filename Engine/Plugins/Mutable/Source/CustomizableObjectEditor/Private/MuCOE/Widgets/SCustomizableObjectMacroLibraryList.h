// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/CustomizableObjectMacroLibrary/CustomizableObjectMacroLibrary.h"
#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class SListView;

struct FMacroTreeEntry
{
	TWeakObjectPtr<UCustomizableObjectMacro> Macro;
};

class SCustomizableObjectMacroLibraryList : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnAddMacroButtonClickedDelegate)
	DECLARE_DELEGATE_OneParam(FOnSelectMacroDelegate, UCustomizableObjectMacro*)
	DECLARE_DELEGATE_OneParam(FOnRemoveMacroDelegate, UCustomizableObjectMacro*)

	SLATE_BEGIN_ARGS(SCustomizableObjectMacroLibraryList) {}
		SLATE_ARGUMENT(TObjectPtr<UCustomizableObjectMacroLibrary>, MacroLibrary)
		SLATE_ARGUMENT(FOnAddMacroButtonClickedDelegate, OnAddMacroButtonClicked)
		SLATE_ARGUMENT(FOnSelectMacroDelegate, OnSelectMacro)
		SLATE_ARGUMENT(FOnRemoveMacroDelegate, OnRemoveMacro)
	SLATE_END_ARGS()

	// SWidget interface
	void Construct(const FArguments& InArgs);

	/** Generates and Updates the List View widget and its source. */
	void GenerateRowView();

	/** Callback to regenerate the List View when a Macro has been removed. It is also used to comunicate the Macro Library editor. */
	void OnRemoveCurrentMacro(UCustomizableObjectMacro* MacroToDelete);

	/** Set the Selected Macro */
	void SetSelectedMacro(const UCustomizableObjectMacro& MacroToSelect);

private:

	/** Pointer to the Macro Library that contains the macros. */
	TObjectPtr<UCustomizableObjectMacroLibrary> MacroLibrary;

	/** Pointer to the List View widget. Needed to refresh it when something changes. */
	TSharedPtr<SListView<TSharedPtr<FMacroTreeEntry>>> ListView;

	/** Array that contains all the macros of the Library in a format for the List View*/
	TArray<TSharedPtr<FMacroTreeEntry>> ListViewSource;

	// List view action callbacks
	FOnAddMacroButtonClickedDelegate OnAddMacroButtonClicked;
	FOnSelectMacroDelegate OnSelectMacro;
	FOnRemoveMacroDelegate OnRemoveMacro;
};

