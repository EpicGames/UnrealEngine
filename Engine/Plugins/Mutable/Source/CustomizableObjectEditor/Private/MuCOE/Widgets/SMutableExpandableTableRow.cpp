// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Widgets/SMutableExpandableTableRow.h"



FSlateColor GetRowBackgroundColor(const int32 IndentLevel, const bool IsHovered)
{
	int32 ColorIndex = 0;
	int32 Increment = 1;

	for (int i = 0; i < IndentLevel; ++i)
	{
		ColorIndex += Increment;

		if (ColorIndex == 0 || ColorIndex == 3)
		{
			Increment = -Increment;
		}
	}

	static const uint8 ColorOffsets[] =
	{
		0, 4, (4 + 2), (6 + 4), (10 + 6)
	};

	const FSlateColor BaseSlateColor = IsHovered ? 
	   FAppStyle::Get().GetSlateColor("Colors.Header") : 
	   FAppStyle::Get().GetSlateColor("Colors.Panel");

	const FColor BaseColor = BaseSlateColor.GetSpecifiedColor().ToFColor(true);

	const FColor ColorWithOffset(
		BaseColor.R + ColorOffsets[ColorIndex], 
		BaseColor.G + ColorOffsets[ColorIndex], 
		BaseColor.B + ColorOffsets[ColorIndex]);

	return FSlateColor(FLinearColor::FromSRGBColor(ColorWithOffset));
}
