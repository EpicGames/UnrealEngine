// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/SMutableConstrainedBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableRow.h"

struct FSlateBrush;


/** Get the background color of the Pin Viewer elements.
 * @param IndentLevel Element indent level. Returns different colors to distinguish between them.
 *
 * Copy of PropertyEditorConstants.cpp */
FSlateColor GetRowBackgroundColor(const int32 IndentLevel, const bool IsHovered);

/** Pin Viewer custom row. Allows a row to have expandable custom details.
 *
 *The following code is copy of SMultiColumnTableRow with slight modifications. */
template<typename ItemType>
class SMutableExpandableTableRow : public STableRow<ItemType>
{
public:

	/**
	 * Users of SMultiColumnTableRow would usually some piece of data associated with it.
	 * The type of this data is ItemType; it's the stuff that your TableView (i.e. List or Tree) is visualizing.
	 * The ColumnName tells you which column of the TableView we need to make a widget for.
	 * Make a widget and return it.
	 *
	 * @param ColumnName    A unique ID for a column in this TableView; see SHeaderRow::FColumn for more info.
	 * @return a widget to represent the contents of a cell in this row of a TableView. 
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& InColumnName ) = 0;

	/** Add pin custom pin custom details panel. Override if required. */
	virtual TSharedPtr<SWidget> GenerateAdditionalWidgetForRow()
	{
		return nullptr;
	}

	/** Returns the default visibility of the additional widget. Override if required. */
	virtual EVisibility GetAdditionalWidgetDefaultVisibility() const
	{
		return EVisibility::Collapsed;
	}

	/** Set the visibility to the additional widget. */
	virtual void SetAdditionalWidgetVisibility(const EVisibility InVisibility)
	{
		if (AdditionalWidget)
		{
			AdditionalWidget->SetVisibility(InVisibility);
		}
	}
	
	/** Use this to construct the superclass; e.g. FSuperRowType::Construct( FTableRowArgs(), OwnerTableView ) */
	typedef SMultiColumnTableRow< ItemType > FSuperRowType;

	/** Use this to construct the superclass; e.g. FSuperRowType::Construct( FTableRowArgs(), OwnerTableView ) */
	typedef typename STableRow<ItemType>::FArguments FTableRowArgs;

protected:
	void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		TSharedPtr<SVerticalBox> VerticalBox;
		
		STableRow<ItemType>::Construct(
			FTableRowArgs()
			.Style(InArgs._Style)
			.ExpanderStyleSet(InArgs._ExpanderStyleSet)
			.Padding(InArgs._Padding)
			.ShowSelection(InArgs._ShowSelection)
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(InArgs._OnDragDetected)
			.OnDragEnter(InArgs._OnDragEnter)
			.OnDragLeave(InArgs._OnDragLeave)
			.OnDrop(InArgs._OnDrop)
			.Content()
			[
				SAssignNew(VerticalBox, SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
					.Padding(FMargin(0, 0, 0, 1))
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
						.BorderBackgroundColor(this, &SMutableExpandableTableRow::GetBackgroundColor)
						.Padding(0)
						.Clipping(EWidgetClipping::ClipToBounds)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SMutableConstrainedBox)
								.MinWidth(12)
								.Visibility(this, &SMutableExpandableTableRow::GetExpanderVisibility)
								[
									SAssignNew(ExpanderArrow, SButton)
									.ButtonStyle(FCoreStyle::Get(), "NoBorder")
									.VAlign(VAlign_Center)
									.HAlign(HAlign_Center)
									.ClickMethod(EButtonClickMethod::MouseDown)
									.OnClicked(this, &SMutableExpandableTableRow::OnExpanderClicked)
									.ContentPadding(0)
									.IsFocusable(false)
									[
										SNew(SImage)
										.Image(this, &SMutableExpandableTableRow::GetExpanderImage)
										.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									]
								]
							]
							+SHorizontalBox::Slot()
							[
								SAssignNew(Box, SHorizontalBox)
							]
						]
					]
				]
			]
			, OwnerTableView );

		AdditionalWidget = GenerateAdditionalWidgetForRow();
		if (AdditionalWidget)
		{
			VerticalBox->AddSlot()
			[
				AdditionalWidget.ToSharedRef()
			];
			
			AdditionalWidget->SetVisibility(GetAdditionalWidgetDefaultVisibility());
		}

		// Sign up for notifications about changes to the HeaderRow
		TSharedPtr< SHeaderRow > HeaderRow = OwnerTableView->GetHeaderRow();
		check( HeaderRow.IsValid() );
		HeaderRow->OnColumnsChanged()->AddSP( this, &SMutableExpandableTableRow<ItemType>::GenerateColumns );

		// Populate the row with user-generated content
		this->GenerateColumns( HeaderRow.ToSharedRef() );
	}

	virtual void ConstructChildren( ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent ) override
	{
		STableRow<ItemType>::Content = InContent;

		// MultiColumnRows let the user decide which column should contain the expander/indenter item.
		this->ChildSlot
		.Padding( InPadding )
		[
			InContent
		];
	}

	void GenerateColumns( const TSharedRef<SHeaderRow>& InColumnHeaders )
	{
		Box->ClearChildren();
		const TIndirectArray<SHeaderRow::FColumn>& Columns = InColumnHeaders->GetColumns();
		const int32 NumColumns = Columns.Num();
		TMap< FName, TSharedRef< SWidget > > NewColumnIdToSlotContents;

		for( int32 ColumnIndex = 0; ColumnIndex < NumColumns; ++ColumnIndex )
		{
			const SHeaderRow::FColumn& Column = Columns[ColumnIndex];
			if ( InColumnHeaders->ShouldGeneratedColumn(Column.ColumnId) )
			{
				TSharedRef< SWidget >* ExistingWidget = ColumnIdToSlotContents.Find(Column.ColumnId);
				TSharedRef< SWidget > CellContents = SNullWidget::NullWidget;
				if (ExistingWidget != nullptr)
				{
					CellContents = *ExistingWidget;
				}
				else
				{
					CellContents = GenerateWidgetForColumn(Column.ColumnId);
				}

				if ( CellContents != SNullWidget::NullWidget )
				{
					CellContents->SetClipping(EWidgetClipping::OnDemand);
				}

				switch (Column.SizeRule)
				{
				case EColumnSizeMode::Fill:
				{
					TAttribute<float> WidthBinding;
					WidthBinding.BindRaw(&Column, &SHeaderRow::FColumn::GetWidth);

					Box->AddSlot()
					.HAlign(Column.CellHAlignment)
					.VAlign(Column.CellVAlignment)
					.FillWidth(WidthBinding)
					[
						CellContents
					];
				}
				break;

				case EColumnSizeMode::Fixed:
				{
					Box->AddSlot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(Column.Width.Get())
						.HAlign(Column.CellHAlignment)
						.VAlign(Column.CellVAlignment)
						.Clipping(EWidgetClipping::OnDemand)
						[
							CellContents
						]
					];
				}
				break;

				case EColumnSizeMode::Manual:
				case EColumnSizeMode::FillSized:
				{
					auto GetColumnWidthAsOptionalSize = [&Column]() -> FOptionalSize
					{
						const float DesiredWidth = Column.GetWidth();
						return FOptionalSize(DesiredWidth);
					};

					TAttribute<FOptionalSize> WidthBinding;
					WidthBinding.Bind(TAttribute<FOptionalSize>::FGetter::CreateLambda(GetColumnWidthAsOptionalSize));

					Box->AddSlot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(WidthBinding)
						.HAlign(Column.CellHAlignment)
						.VAlign(Column.CellVAlignment)
						.Clipping(EWidgetClipping::OnDemand)
						[
							CellContents
						]
					];
				}
				break;

				default:
					ensure(false);
					break;
				}

				NewColumnIdToSlotContents.Add(Column.ColumnId, CellContents);
			}
		}

		ColumnIdToSlotContents = NewColumnIdToSlotContents;
	}

	void ClearCellCache()
	{
		ColumnIdToSlotContents.Empty();
	}

	const TSharedRef<SWidget>* GetWidgetFromColumnId(const FName& ColumnId) const
	{
		return ColumnIdToSlotContents.Find(ColumnId);
	}

	/** Returns custom background color. */
	FSlateColor GetBackgroundColor() const
	{
		return GetRowBackgroundColor(0, this->IsHovered());
	}

	/** Returns visible if there is an additional widget. */
	EVisibility GetExpanderVisibility() const 
	{
		return AdditionalWidget ? EVisibility::Visible : EVisibility::Hidden;
	}

	/** Collapse or show the additional details. */
	FReply OnExpanderClicked()
	{
		const EVisibility NewVisibility = AdditionalWidget->GetVisibility() == EVisibility::Visible ?
			EVisibility::Collapsed :
			EVisibility::Visible;

		SetAdditionalWidgetVisibility(NewVisibility);
		
		return FReply::Handled();
	}

	/** Returns the expander image depending on the additional details visibility state. */
	const FSlateBrush* GetExpanderImage() const
	{
		FName ResourceName;
		if (AdditionalWidget && AdditionalWidget->GetVisibility() == EVisibility::Visible)
		{
			if (ExpanderArrow->IsHovered())
			{
				static const FName ExpandedHoveredName = "TreeArrow_Expanded_Hovered";
				ResourceName = ExpandedHoveredName;
			}
			else
			{
				static const FName ExpandedName = "TreeArrow_Expanded";
				ResourceName = ExpandedName;
			}
		}
		else
		{
			if (ExpanderArrow->IsHovered())
			{
				static const FName CollapsedHoveredName = "TreeArrow_Collapsed_Hovered";
				ResourceName = CollapsedHoveredName;
			}
			else
			{
				static const FName CollapsedName = "TreeArrow_Collapsed";
				ResourceName = CollapsedName;
			}
		}

		return FAppStyle::Get().GetBrush(ResourceName);
	}

private:
	TSharedPtr<SHorizontalBox> Box;
	TMap< FName, TSharedRef< SWidget > > ColumnIdToSlotContents;
	
	/** Pin Viewer row additional widget. */
	TSharedPtr<SWidget> AdditionalWidget; 

	/** Expand button. */
	TSharedPtr<SButton> ExpanderArrow;
};

