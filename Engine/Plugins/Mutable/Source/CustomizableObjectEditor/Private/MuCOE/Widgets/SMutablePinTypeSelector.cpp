// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOE/Widgets/SMutablePinTypeSelector.h"

#include "DetailLayoutBuilder.h"
#include "MuCOE/Nodes/CustomizableObjectNode.h"
#include "MuCOE/EdGraphSchema_CustomizableObject.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"


void SMutablePinTypeSelector::Construct(const FArguments& InArgs)
{
	PinReference = InArgs._PinReference;

	const UEdGraphPin* Pin = PinReference.Get();
	check(Pin);
	
	if (UCustomizableObjectNode* OwningNode = Cast<UCustomizableObjectNode>(Pin->GetOwningNode()))
	{
		AllowedPinTypes = OwningNode->GetPinAllowedTypes(*Pin);
		PinOwner = MakeWeakObjectPtr(OwningNode);
	}
	else
	{
		checkNoEntry();
	}
	
	if (AllowedPinTypes.Num() > 1)
	{
		ChildSlot
		[
			SNew(SComboBox<FName>)
			.OptionsSource(&AllowedPinTypes)
			.InitiallySelectedItem(GetInitiallySelectedPinType())
			.IsEnabled(this, &SMutablePinTypeSelector::ShouldVariableTypePropertyBeEnabled)
			.ToolTipText(this, &SMutablePinTypeSelector::GetToolTipText)
			.OnGenerateWidget(this, &SMutablePinTypeSelector::OnGenerateVariableTypeRow)
			.OnSelectionChanged(this, &SMutablePinTypeSelector::OnTypeDropdownSelectionChange)
			.Content()
			[
				GenerateCurrentSelectedTypeWidget()
			]
		];
	}
	else
	{
		ChildSlot
		[
			SNew(STextBlock)
			.Text(UEdGraphSchema_CustomizableObject::GetPinCategoryFriendlyName(GetInitiallySelectedPinType()))
		];
	}
	

}


FName SMutablePinTypeSelector::GetInitiallySelectedPinType() const
{
	if (UEdGraphPin* PinPointer = PinReference.Get())
	{
		return PinPointer->PinType.PinCategory;
	}
	else 
	{
		return NAME_Name;
	}
}


bool SMutablePinTypeSelector::ShouldVariableTypePropertyBeEnabled() const
{
	if (PinReference.Get())
	{
		return PinReference.Get()->LinkedTo.Num() == 0;
	}
	
	return false;
}


FText SMutablePinTypeSelector::GetToolTipText() const
{
	if (ShouldVariableTypePropertyBeEnabled())
	{
		return LOCTEXT("SMutablePinTypeSelector_SelectorTooltip", "Select the type of pin you want for this pin to be.");
	}
	else
	{
		return LOCTEXT("SMutablePinTypeSelector_SelectorTooltip_Disabled", "Unable to change the pin type while connected to another pin.");
	}
}


TSharedRef<SWidget> SMutablePinTypeSelector::OnGenerateVariableTypeRow(FName Type)
{
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));

	TSharedPtr<SHorizontalBox> RowWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SImage)
				.Image(IconBrush)
				.ColorAndOpacity(UEdGraphSchema_CustomizableObject::GetPinTypeColor(Type))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(7.5f, 0.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
				.Text(Type.IsValid() ? GetDefault<UEdGraphSchema_CustomizableObject>()->GetPinCategoryFriendlyName(Type) : FText::FromString("Invalid"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	return RowWidget.ToSharedRef();
}


void SMutablePinTypeSelector::OnTypeDropdownSelectionChange(FName NewTypeName, ESelectInfo::Type Arg) const
{
	if (UEdGraphPin* Pin = PinReference.Get())
	{
		Pin->PinType.PinCategory = NewTypeName;
		
		if (TStrongObjectPtr<UCustomizableObjectNode> Node = PinOwner.Pin())
		{
			Node->ReconstructNode();
		}
	}
}


TSharedRef<SWidget> SMutablePinTypeSelector::GenerateCurrentSelectedTypeWidget()
{
	const FName InitiallySelectedPinType = GetInitiallySelectedPinType();
	check(AllowedPinTypes.Contains(InitiallySelectedPinType))
	
	return OnGenerateVariableTypeRow(InitiallySelectedPinType);
}



void SMutablePinSubTypeSelector::Construct(const FArguments& InArgs)
{
	PinReference = InArgs._PinReference;

	const UEdGraphPin* Pin = PinReference.Get();
	check(Pin);
	
	if (UCustomizableObjectNode* OwningNode = Cast<UCustomizableObjectNode>(Pin->GetOwningNode()))
	{
		AllowedPinSubtypes = OwningNode->GetPinAllowedSubTypes(*Pin);
		PinOwner = MakeWeakObjectPtr(OwningNode);
	}
	else
	{
		checkNoEntry();
	}
	
	if (AllowedPinSubtypes.Num() > 1)
	{
		ChildSlot
		[
			SNew(SComboBox<FName>)
			.OptionsSource(&AllowedPinSubtypes)
			.InitiallySelectedItem(GetInitiallySelectedPinSubType())
			.IsEnabled(true)
			.OnGenerateWidget(this, &SMutablePinSubTypeSelector::OnGenerateVariableSubtypeRow)
			.OnSelectionChanged(this, &SMutablePinSubTypeSelector::OnTypeDropdownSelectionChange)
			.Content()
			[
				GenerateCurrentSelectedTypeWidget()
			]
		];
	}
	else
	{
		ChildSlot
		[
			SNew(STextBlock)
			.Text(UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(GetInitiallySelectedPinSubType()))
		];
	}
	
}


FName SMutablePinSubTypeSelector::GetInitiallySelectedPinSubType() const
{
	if (UEdGraphPin* PinPointer = PinReference.Get())
	{
		return PinPointer->PinType.PinSubCategory;
	}
	else 
	{
		return NAME_Name;
	}
}


TSharedRef<SWidget> SMutablePinSubTypeSelector::OnGenerateVariableSubtypeRow(FName Subtype)
{
	TSharedRef<STextBlock> RowWidget = SNew(STextBlock)
				.Text(Subtype.IsValid() ? UEdGraphSchema_CustomizableObject::GetPinSubCategoryFriendlyName(Subtype) : FText::FromString("Invalid"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ToolTipText(UEdGraphSchema_CustomizableObject::GetSubCategoryTooltipText(Subtype));

	return RowWidget;
}


void SMutablePinSubTypeSelector::OnTypeDropdownSelectionChange(FName NewSubTypeName, ESelectInfo::Type Arg) const
{
	if (UEdGraphPin* Pin = PinReference.Get())
	{
		Pin->PinType.PinSubCategory = NewSubTypeName;
		
		if (TStrongObjectPtr<UCustomizableObjectNode> Node = PinOwner.Pin())
		{
			Node->ReconstructNode();
		}
	}
}


TSharedRef<SWidget> SMutablePinSubTypeSelector::GenerateCurrentSelectedTypeWidget()
{
	const FName InitiallySelectedPinSubType = GetInitiallySelectedPinSubType();
	check(AllowedPinSubtypes.Contains(InitiallySelectedPinSubType))
	
	return OnGenerateVariableSubtypeRow(InitiallySelectedPinSubType);
}


#undef LOCTEXT_NAMESPACE
