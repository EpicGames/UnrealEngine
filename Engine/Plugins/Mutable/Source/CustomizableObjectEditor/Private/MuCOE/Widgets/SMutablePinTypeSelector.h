// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditAction.h"
#include "EdGraph/EdGraphPin.h"
#include "Widgets/Input/SComboBox.h"

template <typename ItemType> class SListView;

class UCustomizableObjectNode;
struct FEdGraphPinReference;
struct FGuid;

/**
 * Slate that exposes a dropdown that takes care of exposing the pin category and allowing the end user to change it.
 * It will call the refresh of the node to apply that change.
 */
class SMutablePinTypeSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutablePinTypeSelector) {}
	SLATE_ARGUMENT(FEdGraphPinReference, PinReference)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:
	
	FName GetInitiallySelectedPinType() const;
	bool ShouldVariableTypePropertyBeEnabled() const;
	FText GetToolTipText() const;
	TSharedRef<SWidget> OnGenerateVariableTypeRow(FName Type);
	void OnTypeDropdownSelectionChange(FName NewTypeName, ESelectInfo::Type Arg) const;
	TSharedRef<SWidget> GenerateCurrentSelectedTypeWidget();
	
	FEdGraphPinReference PinReference;

	TArray<FName> AllowedPinTypes;
	
	/** Weak pointer to the Customizable Object node that contains the pin */
	TWeakObjectPtr<UCustomizableObjectNode> PinOwner;
};


/**
 * Slate that exposes a dropdown that takes care of exposing the pin subcategory and allowing the end user to change it.
 * It will call the refresh of the node to apply that change.
 */
class SMutablePinSubTypeSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMutablePinSubTypeSelector) {}
		SLATE_ARGUMENT(FEdGraphPinReference, PinReference)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:
	
	FName GetInitiallySelectedPinSubType() const;
	TSharedRef<SWidget> OnGenerateVariableSubtypeRow(FName Subtype);
	void OnTypeDropdownSelectionChange(FName NewSubTypeName, ESelectInfo::Type Arg) const;
	TSharedRef<SWidget> GenerateCurrentSelectedTypeWidget();
	
	FEdGraphPinReference PinReference;

	TArray<FName> AllowedPinSubtypes;
	
	/** Weak pointer to the Customizable Object node that contains the pin */
	TWeakObjectPtr<UCustomizableObjectNode> PinOwner;
};

