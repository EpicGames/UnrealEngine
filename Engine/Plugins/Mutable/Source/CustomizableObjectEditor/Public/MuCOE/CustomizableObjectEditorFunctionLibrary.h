// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MuCO/CustomizableObject.h"

#include "CustomizableObjectEditorFunctionLibrary.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

// This mirrors the logic in CustomizableObjectEditor.cpp

UENUM()
enum class ECustomizableObjectCompilationState : uint8
{
	None,
	InProgress,
	Completed,
	Failed
};


USTRUCT(BlueprintType)
struct FNewCustomizableObjectParameters
{
	GENERATED_BODY()

	/** Must not end with slash. For example "/Game" */
	UPROPERTY(BlueprintReadWrite, Category = NewCustomizableObjectParameters)
	FString PackagePath;

	/** For example "SampleAssetName" */
	UPROPERTY(BlueprintReadWrite, Category = NewCustomizableObjectParameters)
	FString AssetName;

	/** Parent to attach the child Customizable Object to. */
	UPROPERTY(BlueprintReadWrite, Category = NewCustomizableObjectParameters)
	TObjectPtr<UCustomizableObject> ParentObject;

	/** Group to attach the child Customizable Object to. Only used if ParentObject is provided. */
	UPROPERTY(BlueprintReadWrite, Category = NewCustomizableObjectParameters)
	FString ParentGroupNode;
};


/**
 * Functions we want to be able to call on CustomizableObjects at edit time - could
 * be exposed to cook as well.
 */
UCLASS(MinimalAPI)
class UCustomizableObjectEditorFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** DEPRECATED. Use UCustomizableObject::Compile instead.
	 * 
	 *	Synchronously compiles the provided CustomizableObject, LogMutable will contain intermittent updates on
	 *	progress.
	 * 
	 * @param CustomizableObject	The CustomizableObject to compile
	 * 
	 * @return	The final ECustomizableObjectCompilationState - typically Completed or Failed
	 */
	UFUNCTION(BlueprintCallable, Category = "CustomizableObject")
	static UE_API ECustomizableObjectCompilationState CompileCustomizableObjectSynchronously(
		UCustomizableObject* CustomizableObject, 
		ECustomizableObjectOptimizationLevel OptimizationLevel = ECustomizableObjectOptimizationLevel::None, 
		ECustomizableObjectTextureCompression TextureCompression = ECustomizableObjectTextureCompression::Fast,
		bool bGatherReferences = false);

	/** Create a new Customizable Object inside a package. */
	UFUNCTION(BlueprintCallable, Category = "CustomizableObject")
	static UE_API UCustomizableObject* NewCustomizableObject(const FNewCustomizableObjectParameters& Parameters);
};

#undef UE_API
