// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CustomizableObjectFactory.generated.h"

enum class ECheckBoxState : uint8;

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class FFeedbackContext;
class FReply;
class STextComboBox;
class SWindow;
class UClass;
class UCustomizableObject;
class UObject;
class USkeletalMesh;

struct EVisibility;


UCLASS(MinimalAPI)
class UCustomizableObjectFactory : public UFactory
{
	GENERATED_BODY()

	UCustomizableObjectFactory();

public:
	// Begin UFactory Interface
	virtual bool DoesSupportClass(UClass * Class) override;
	virtual UClass* ResolveSupportedClass() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent,FName Name,EObjectFlags Flags,UObject* Context,FFeedbackContext* Warn) override;
};
