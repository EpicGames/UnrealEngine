// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "CustomizableObjectInstanceFactory.generated.h"

class AActor;
class FText;
class UObject;
class USkeletalMesh;
struct FAssetData;

UCLASS(MinimalAPI, config=Editor)
class UCustomizableObjectInstanceFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

protected:
	// UActorFactory interface
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual FQuat AlignObjectToSurfaceNormal(const FVector& InSurfaceNormal, const FQuat& ActorRotation) const override;
};
