// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "StructUtils/InstancedStruct.h"
#include "CustomizableObjectVersionBridge.generated.h"

UINTERFACE(MinimalAPI, Blueprintable)
class UCustomizableObjectVersionBridgeInterface : public UInterface
{
	GENERATED_BODY()
};

class ICustomizableObjectVersionBridgeInterface
{
	GENERATED_BODY()

public:
	/** Interface function declarations */

	/** Checks whether the version encoded in VersionStruct is older or equal to the CurrentRelease, meaning that all
	  *	Mutable child Customizable Objects marked with this version that return true should be compiled and packaged.
	  * The VersionStruct will encode the game-specific version struct, and should be accessed from the game-specific
	  * implementation of this interface in the the VersionBridge property in the CustomizableObject with the
	  * VersionStruct.GetPtr<GameSpecificVersionType>() method. The VersionStruct in child CustomizableObjects
	  * must be of the GameSpecificVersionType type.
	  * @param VersionStruct Struct encoding the version of the Mutable table row
	  * @return True if the table row is included in the current release
	*/
	virtual bool IsVersionStructIncludedInCurrentRelease(const FInstancedStruct& VersionStruct) const { return false; };

	/** Checks whether the version encoded in VersionProperty/CellData is older or equal to the CurrentRelease, meaning that all
	  *	Mutable table rows marked with this version that return true should be compiled and packaged.
	  * The VersionProperty/CellData will encode the game-specific version struct, and should be accessed from the game-specific
	  * implementation of this interface in the the VersionBridge property in the CustomizableObject by casting CellData
	  * to the GameSpecificVersionType. The Version column in Mutable data table must be of the
	  * GameSpecificVersionType type.
	  * @param VersionProperty Property of the column encoding the version of the Mutable table row. Useful for type checking.
	  * @param CellData pointer to the GameSpecificVersionType struct encoding the version of the Mutable table row
	  * @return True if the table row is included in the current release
	*/
	virtual bool IsVersionPropertyIncludedInCurrentRelease(const FProperty& VersionProperty, const uint8* CellData) const { return false; };

	/** Get the version as string to include it when building the derived data cache key.
	  * @return current version as string.
	  */
	virtual FString GetCurrentVersionAsString() const { return {}; };
};
