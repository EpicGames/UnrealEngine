// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

#include "EdGraphSchema_CustomizableObject.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

enum class EMaterialParameterType : uint8;

class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UObject;
struct FAssetData;
struct FEdGraphPinType;
class IAssetReferenceFilter;
class ICustomizableObjectEditor;


UCLASS(MinimalAPI)
class UEdGraphSchema_CustomizableObject : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	// Allowable PinType.PinCategory values
	static UE_API const FName PC_Object;
	static UE_API const FName PC_Component;
	static UE_API const FName PC_MeshSection;
	static UE_API const FName PC_Modifier;
	static UE_API const FName PC_Mesh;
	static UE_API const FName PC_SkeletalMesh;
	static UE_API const FName PC_SkeletalMesh_Passthrough;
	static UE_API const FName PC_SkeletalMesh_Object;
	static UE_API const FName PC_Texture;
	static UE_API const FName PC_Texture_Passthrough;
	static UE_API const FName PC_Texture_Object;
	static UE_API const FName PC_Projector;
	static UE_API const FName PC_GroupProjector;
	static UE_API const FName PC_Color;
	static UE_API const FName PC_Float;
	static UE_API const FName PC_Bool;
	static UE_API const FName PC_Enum;
	static UE_API const FName PC_Stack;
	static UE_API const FName PC_Material;
	static UE_API const FName PC_Wildcard;
	static UE_API const FName PC_PoseAsset;
	static UE_API const FName PC_Transform;
	static UE_API const FName PC_String;
	static UE_API const FName PC_StaticMesh;

	static UE_API const FName PSC_Material_Override;
	static UE_API const FName PSC_Material_Overlay;

	static UE_API const TArray<FName> SupportedMacroPinTypes;

	// Node categories
	static UE_API const FText NC_Experimental;
	static UE_API const FText NC_Material;

	// EdGraphSchema interface
	UE_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	UE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	UE_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	UE_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	UE_API virtual void GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	UE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	UE_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	UE_API virtual bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	UE_API virtual FPinConnectionResponse MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const override;
	UE_API virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj) const override;

	// Own interface
	static UE_API FLinearColor GetPinTypeColor(const FName& PinType);
	static UE_API FName GetPinCategoryName(const FName& PinCategory);
	static UE_API FName GetPinCategoryName(const UScriptStruct* Type);
	static UE_API FName GetPinSubCategoryName(const FName& PinSubCategory);
	static UE_API FText GetPinFriendlyName(const FName& PinCategory);
	static UE_API FText GetPinCategoryFriendlyName(const FName& PinCategory);
	static UE_API FText GetPinSubCategoryFriendlyName(const FName& PinSubcategory);
	static UE_API bool IsPassthrough(const FName& PinCategory);
	static UE_API bool IsTypeNative(const UScriptStruct* Type);
	static UE_API FText GetSubCategoryTooltipText(FName PinSubType);
	
private:
	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

#undef UE_API
