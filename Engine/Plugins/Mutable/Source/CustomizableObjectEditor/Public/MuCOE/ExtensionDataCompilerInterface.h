// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/PassthroughObject.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class FText;
class UCustomizableObject;
class UCustomizableObjectNode;
struct FInstancedStruct;
struct FMutableGraphGenerationContext;

/**
 * An object that gets passed around during Customizable Object compilation to help set up
 * Extension Data.
 */
class FExtensionDataCompilerInterface
{
public:
	UE_API FExtensionDataCompilerInterface(FMutableGraphGenerationContext& InGenerationContext);

	/** Register a new Extension Data. 
	 * @param bDuplicate if true, object will be duplicated during cook. */
	UE_API UE::Mutable::Private::PASSTHROUGH_ID MakeExtensionData(UObject& ExtensionData, bool bDuplicate);

	/** Adds a node to the Generation Context list of generated nodes. This function is meant to be called
	* from classes that implement ICustomizableObjectExtensionNode::GenerateMutableNode for any generated nodes
	* so they are registered against the Mutable compiler */
	UE_API void AddGeneratedNode(const UCustomizableObjectNode* InNode);

	/** Adds a compiler log message to be displayed at the end of the compilation process */
	UE_API void CompilerLog(const FText& InLogText, const UCustomizableObjectNode* InNode);

	FMutableGraphGenerationContext& GenerationContext;
};

#undef UE_API
