// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeExtensionData.h"
#include "UObject/Interface.h"

#include "ICustomizableObjectExtensionNode.generated.h"

class FExtensionDataCompilerInterface;

/** Customizable Object graph nodes that output FExtensionData should implement this interface */
UINTERFACE(MinimalAPI)
class UCustomizableObjectExtensionNode : public UInterface
{
	GENERATED_BODY()
};

class ICustomizableObjectExtensionNode
{
	GENERATED_BODY()

public:
	/** Generate a UE::Mutable::Private::Node that produces FExtensionData */
	virtual UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> GenerateMutableNode(FExtensionDataCompilerInterface& CompilerInterface) const = 0;
};
