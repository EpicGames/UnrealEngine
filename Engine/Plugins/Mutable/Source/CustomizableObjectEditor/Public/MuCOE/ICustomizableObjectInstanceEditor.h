// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class UPointLightComponent;
class UCustomizableObjectNodeProjectorParameter;
class UCustomizableObjectNodeProjectorConstant;
class SCustomizableObjectEditorViewportTabBody;
class UCustomizableObject;
class UCustomizableObjectInstance;
class UProjectorParameter;
class UCustomSettings;
class UCustomizableObjectNode;
class UCustomizableObjectNodeModifierClipMorph;
class UEdGraphPin;
class ULightComponent;
class UPoseAsset;
class UCustomizableObjectEditorProperties;
class SCustomizableObjectEditorAdvancedPreviewSettings;

/**
 * Public interface to Customizable Object Instance Editor
 */
class ICustomizableObjectInstanceEditor
{
public:
	virtual UCustomizableObjectInstance* GetPreviewInstance() = 0;

	/** Refreshes the Customizable Object Instance Editor's viewport. */
	virtual TSharedPtr<SCustomizableObjectEditorViewportTabBody> GetViewport() = 0;

	/** Refreshes everything in the Customizable Object Instance Editor. */
	virtual void RefreshTool() = 0;
	
	/** Return the selected projector in the viewport/editor. */
	virtual UProjectorParameter* GetProjectorParameter() = 0;

	virtual UCustomSettings* GetCustomSettings() = 0;

	/** Hide the currently selected gizmo. Notice that only a single gizmo can be shown at the same time.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void HideGizmo() = 0;

	/** Show the default projector value gizmo of the given NodeProjectorConstant.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void ShowGizmoProjectorNodeProjectorConstant(UCustomizableObjectNodeProjectorConstant& Node) {}

	/** Hide the default projector value gizmo of the given NodeProjectorConstant.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void HideGizmoProjectorNodeProjectorConstant() {}
	
	/** Show the default projector value gizmo of the given NodeProjectorParameter.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void ShowGizmoProjectorNodeProjectorParameter(UCustomizableObjectNodeProjectorParameter& Node) {}

	/** Hide the default projector value gizmo of the given NodeProjectorParameter.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void HideGizmoProjectorNodeProjectorParameter() {}
	
	/** Show the projector value gizmo of the given parameter.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void ShowGizmoProjectorParameter(const FString& ParamName, int32 RangeIndex = -1) = 0;

	/** Hide the projector value gizmo of the given parameter.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void HideGizmoProjectorParameter() = 0;
	
	/** Show the clip morph plane gizmo of the NodeClipMorph.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void ShowGizmoClipMorph(UCustomizableObjectNodeModifierClipMorph& ClipPlainNode) {}

	/** Hide the clip morph plane gizmo of the NodeClipMorph.
	 * @param bClearGraphSelectionSet If set to true will deselect all nodes in the graph.
	 */
	virtual void HideGizmoClipMorph(bool bClearGraphSelectionSet = true) {}
	
	/** Show the clipping mesh gizmo from a node that uses a transformable bounding mesh. */
	virtual void ShowGizmoClipMesh(UCustomizableObjectNode& Node, FTransform* Transform, const UEdGraphPin& MeshPin) {}

	/** Hide the clipping mesh gizmo from the NodeMeshClipWithMesh.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void HideGizmoClipMesh() {}
	
	/** Show the light gizmo of the light.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void ShowGizmoLight(ULightComponent& SelectedLight) {}

	/** Hide the light gizmo of the light.
	 * Synchronizes all editor widgets so that the selection and the widgets is always consistent. */
	virtual void HideGizmoLight() {}

	/** @return Editor properties. */
	virtual UCustomizableObjectEditorProperties* GetEditorProperties() = 0;
	
	virtual TSharedPtr<SCustomizableObjectEditorAdvancedPreviewSettings> GetAdvancedPreviewSettings() = 0;

	virtual bool ShowLightingSettings() = 0;

	virtual bool ShowProfileManagementOptions() = 0;

	virtual const UObject* GetObjectBeingEdited() = 0;

	/** Returns the editor's ModeTools for use by viewport clients.
	 * Editors that inherit from FAssetEditorToolkit should return their EditorModeManager.
	 * Returns nullptr by default, which causes viewport clients to create their own. */
	virtual FEditorModeTools* GetEditorModeTools() { return nullptr; }
};
