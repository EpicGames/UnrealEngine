// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class UCustomizableObject;

/**
 * The public interface of the CustomizableObject module
 */
class ICustomizableObjectPopulationModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to ICustomizableObjectPopulationModule
	 *
	 * @return Returns CustomizableObjectModule singleton instance, loading the module on demand if needed
	 */
	static inline ICustomizableObjectPopulationModule& Get()
	{
		return FModuleManager::LoadModuleChecked< ICustomizableObjectPopulationModule >("CustomizableObjectPopulation");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("CustomizableObjectPopulation");
	}

	// Return a string representing the plugin version.
	virtual FString GetPluginVersion() const = 0;

	// Recompiles all the populations referenced in the Customizabled Object
	virtual void RecompilePopulations(UCustomizableObject* Object) {}
};
