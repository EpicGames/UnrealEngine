// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h" // IWYU pragma: keep
#include "EdGraph/EdGraphPin.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"

#include "CustomizableObjectNode.generated.h"

#define UE_API CUSTOMIZABLEOBJECTEDITOR_API

class FArchive;
class FCustomizableObjectGraphEditorToolkit;
class SWidget;
class UCustomizableObjectNodeMacroInstance;
class UCustomizableObjectNodeObject;
class UCustomizableObjectNodeRemapPins;
class UCustomizableObjectNodeRemapPinsByName;
class UCustomizableObjectNodeRemapPinsByPosition;
class IDetailsView;
class ICustomizableObjectEditor;

struct FPropertyChangedEvent;

/**
 * Determines where the "Add Pin" button should be added within the node.
 */
enum class EAddPinNodeButtonLocation : uint8
{
	/** No "Add Pin" option will be added to the node UI. */
	NONE = 0,

	/** An option to add pins on the INPUT side of the node will be added. */
	INPUT,

	/** An option to add pins on the OUTPUT side of the node will be added. */
	OUTPUT
};


/**
 * Determines the visibility behaviour for the editable pin name text boxes
 */
enum class EEditablePinNameBoxVisibilityPolicy : uint8
{
	/** Default behaviour : Any linked pin will have it's editable textbox hidden. */
	HIDE_IF_LINKED = 0,

	/** Always show the text box. */
	ALWAYS_VISIBLE
};



UENUM()
enum class ESizeOptions : uint8
{
	Value_2		= 2		UMETA(DisplayName="2"),
	Value_4		= 4		UMETA(DisplayName="4"),
	Value_8		= 8		UMETA(DisplayName="8"),
	Value_16	= 16	UMETA(DisplayName="16"),
	Value_32	= 32	UMETA(DisplayName="32"),
	Value_64	= 64	UMETA(DisplayName="64"),
	Value_128	= 128	UMETA(DisplayName="128")
};


USTRUCT()
struct FLayoutSettings 
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category= NoCategory)
	ESizeOptions MaxGridSizeX = ESizeOptions::Value_4;
	
	UPROPERTY(EditAnywhere, Category= NoCategory)
	ESizeOptions MaxGridSizeY = ESizeOptions::Value_4;
};


/** Abstract base class for all pin data. */
UCLASS(Abstract)
class UCustomizableObjectNodePinData : public UObject
{
	GENERATED_BODY()

public:
	UCustomizableObjectNodePinData();

	/** False if inherited types are different. */
	bool operator==(const UCustomizableObjectNodePinData& Other) const;
	
	virtual bool operator!=(const UCustomizableObjectNodePinData& Other) const;

	/** Virtual function used to copy pin data when remapping pins. */
	virtual void Copy(const UCustomizableObjectNodePinData& Other) {}
	
	/** Add backwards compatible code here.
	 * When called, it is guaranteed that all nodes in this graph will have executed the PostLoad function. */
	virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion) {}

protected:
	/** Virtual function used to perform comparision between different UCustomizableObjectNodePinData inherited types. */
	virtual bool Equals(const UCustomizableObjectNodePinData& Other) const;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FPostEditChangePropertyDelegate, UObject*, FPropertyChangedEvent&);

DECLARE_MULTICAST_DELEGATE(FPostReconstructNodeDelegate);

DECLARE_MULTICAST_DELEGATE(FNodeConnectionListChangedDelegate);

DECLARE_MULTICAST_DELEGATE_OneParam(FPinConnectionListChangedDelegate, UEdGraphPin*);

DECLARE_MULTICAST_DELEGATE_TwoParams(FPostEditablePinNameChangedDelegate, const UEdGraphPin&, const FText NewContent);

DECLARE_MULTICAST_DELEGATE(FDestroyNodeDelegate);

using FRemapPinsDelegateParameter = TMap<UEdGraphPin*, UEdGraphPin*>; // Required for the delegate macro.
DECLARE_MULTICAST_DELEGATE_OneParam(FRemapPinsDelegate, const FRemapPinsDelegateParameter&);

/** Base class of all Customizable Object nodes.
 * 
 * The Customizable Object node system is build on top of the following premises. To avoid breaking this system, these premise must be hold!
 * 
 * PREMISES:
 * 1. A node can only be modified when a node is reconstructed (i.e., inside the ReconstructNode, AllocateDefaultPins functions). Exceptionally, a pin can also be created inside the BeginConstruct.
 *    We consider that a node has been modified if one of the following things happen:
 *        - Creation, destruction and modification of pins.
 *        - Modification of any data that would modify any pin.
 */
UCLASS(MinimalAPI, abstract)
class UCustomizableObjectNode : public UEdGraphNode
{
public:
	GENERATED_BODY()

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override; // Do work at PostBackwardsCompatibleFixup.

	// UEdGraphNode interface
	UE_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	UE_API virtual void NodeConnectionListChanged() override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	UE_API virtual bool GetCanRenameNode() const override;

	/** Allocates the default pins using the empty remap pins action. Usually called from CreateNode. */
	UE_API virtual void AllocateDefaultPins() override final; // Final. Override AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) instead

	/** Reconstructs the node using its default remap pins action. */
	UE_API virtual void ReconstructNode() override final; // Final. Override ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPins) instead
	UE_API virtual void PostInitProperties() override;

	// Own interface
	/** Called at the beginning of the node lifecycle.*/
	virtual void BeginConstruct() {};

	/** Add backwards compatible code here.
	 * When called, it is guaranteed that all nodes in this graph will have executed the PostLoad function. */
	UE_API virtual void BackwardsCompatibleFixup(int32 CustomizableObjectCustomVersion);

	/** Reconstructs the node pins using the provided AllocateDefaultPins function.
	 * Safe to call in BackwardsCompatibleFixup.
	 * Notice that this function will not call the node AllocateDefaultPins function nor the ReconstructNode overrides.
	 *
	 * Since this function is used BackwardsCompatibleFixup, it should not have major changes (or they will break existing fixups).
	 * If changes are needed, duplicate this function so old fixups remain unchanged. */
	UE_API void FixupReconstructPins(UCustomizableObjectNodeRemapPins* RemapPinsAction, TFunction<void(UCustomizableObjectNode*, UCustomizableObjectNodeRemapPins*)> AllocateDefaultPins);

	/** Add post load work here.
	 * When called, it is guaranteed that all nodes in this graph will have executed the BackwardsCompatibleFixup function.
	 * Notice that no compatibility code should go here. Use BackwardsCompatibleFixup instead. */
	virtual void PostBackwardsCompatibleFixup() {}

	/** Virtual implementation of RemovePin. Allows to do work before removing a pin.
	 * Use this function instead of RemovePin. RemovePin does not remove possible attached PinData. */
	UE_API virtual bool CustomRemovePin(UEdGraphPin& Pin);

	/**
	 * Subclasses should override this to return true and set OutCategory if this node should be
	 * auto-added to the right-click context menu in the graph editor.
	 *
	 * Some nodes are added manually in the graph editor code and don't need to do this.
	 */
	UE_API virtual bool ShouldAddToContextMenu(FText& OutCategory) const;
	
	virtual bool IsNodeOutDatedAndNeedsRefresh() { return false; }
	virtual FString GetRefreshMessage() const { return "Refresh Node."; }
	UE_API void SetRefreshNodeWarning();
	UE_API void RemoveWarnings();

	// Used to replace Ids of referenced nodes by their new ids after duplicating the CustomizableObject.
	virtual void UpdateReferencedNodeId(const FGuid& NewGuid) {};

	UE_API TSharedPtr<FCustomizableObjectGraphEditorToolkit> GetGraphEditor() const;

	virtual bool ProvidesCustomPinRelevancyTest() const { return false; } // Override and return true if IsPinRelevant is overridden.
	virtual bool IsPinRelevant(const UEdGraphPin*) const { return true; } // Override if default pins allocated with AllocateDefaultPins do not provide enough information.

	struct FAttachedErrorDataView
	{
		TArrayView<const float> UnassignedUVs;
	};

	/**
	 * Check if two pins can be connected.
	 *
	 * @param	InOwnedInputPin	Input pin which belongs to this node.
	 * @param	InOutputPin		Output pin which belongs to another node. If the node connects to iself, it could belong to this node.
	 * @param	bOutIsOtherNodeBlocklisted		Is the other node of a type we are not allowed to connect?
	 * @param	bOutArePinsCompatible		Does InOutputPin pin share the same type as the InOwnedInputPin?
	 * @return	True if the pin types match and if the other node is not one of the Blocklisted types.
	 */
	UE_API virtual bool CanConnect(const UEdGraphPin* InOwnedInputPin, const UEdGraphPin* InOutputPin, bool& bOutIsOtherNodeBlocklisted, bool& bOutArePinsCompatible) const;

	// Used during compilation process to cache the node for all LODs, or generate it specifically for each of them.
	UE_API virtual bool IsAffectedByLOD() const;

	/** Return an array of tags that this node will enable and apply to its data. This tags can come from the node details or string nodes linked to the "Enable tags" pin. */
	UE_API virtual TArray<FString> GetEnableTags(TArray<const UCustomizableObjectNodeMacroInstance*>* MacroContext = nullptr);

	/** Return an array with all tags defined in the node's details panel. Null if none, or of it doesn't apply. */
	UE_API virtual TArray<FString>* GetEnableTagsArray();

	/** Return the unique internal tag that can identify this node across all objects.
	* Internal tags are automatically added to some nodes that can be then referred to from other nodes.
	* They are usually shown differently in the UI, to represent them as tags that will be defined in a single node.
	*/
	UE_API FString GetInternalTag() const;

	/** Return a valid GUID if a tag represents an internal tag. */
	static UE_API FGuid GetInternalTagNodeId(const FString& Tag);
	static UE_API bool IsInternalTag(const FString& Tag);

	/** Find the object and node that a given tag refers to, if it belongs to the same CO hierarchy that this node. */
	UE_API bool FindNodeForInternalTag(const FString& Tag, UCustomizableObjectNode*& OutNode, UCustomizableObject*& OutObject);

	/** Return a non-unique, non-persistent UI-ready text to represent the interna tag of this node. It includes some node information for the user to identify the tag. */
	UE_API virtual FString GetInternalTagDisplayName();

	/** Return a UI-ready text to represent the given tag. For most cases this is just the tag, but if it is an internal tag
	* of any node related to this one (in the same CO hierarchy) a more descriptive (non-persitent) name is build with node information.
	*/
	UE_API FString GetTagDisplayName(const FString& Tag);

	// Get the CustomizableObject graph that owns this node
	UE_API class UCustomizableObjectGraph* GetCustomizableObjectGraph() const;

	/** If returns true, only a single link can go out of every output pin. False by default. */
	UE_API virtual bool IsSingleOutputNode() const;

	/** Wrapper of the pin creation method. Will use the PinName and PinFriendlyName returned by the "UEdGraphSchema_CustomizableObject" */
	UE_API UEdGraphPin* CustomCreatePin( EEdGraphPinDirection Direction, const FName& Category, UCustomizableObjectNodePinData* PinData = nullptr);
	
	// Wrapper of the pin creation method, to cope with multiple Unreal Engine versions
	UE_API UEdGraphPin* CustomCreatePin(EEdGraphPinDirection Direction, const FName& Category, const FName& Name, const FText& FriendlyName, UCustomizableObjectNodePinData* PinData = nullptr);
	
	virtual void AddAttachedErrorData(const FAttachedErrorDataView& ) {};
	virtual void ResetAttachedErrorData() {};

	/** 
	 * When creating a new connection, break all previous connections. This method can be overridden.
	 * 
	 * @return false by default.
	 */
	UE_API virtual bool ShouldBreakExistingConnections(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Custom post duplicate function. Called at the beginning of duplication, before the nodes have their Guid updated. */
	virtual void BeginPostDuplicate(bool bDuplicateForPIE) {};
	
	/** Gets all non orphan pins this node owns. */
	UE_API TArray<UEdGraphPin*> GetAllNonOrphanPins() const;

	/** Gets all orphan pins this node owns. */
	UE_API TArray<UEdGraphPin*> GetAllOrphanPins() const;

	/**
	 * Specialization of ReconstructNode UEdGraphNode function.
	 *
	 * @param RemapPinsMode pointer to a remap pins action. Remap pins action which will be used once the node has been reconstructed. Can be null.
	 */
	UE_API virtual void ReconstructNode(UCustomizableObjectNodeRemapPins* RemapPinsMode);
	
	/** Set a pin to be hidden or not. */
	UE_API void SetPinHidden(UEdGraphPin& InPin, bool bHidden);

	/** Set an array of pins to be hidden or not. 
	 *
	 * @param Pins Input pins. All pins must be non null.
	 */
	UE_API void SetPinHidden(const TArray<UEdGraphPin*>& Pins, bool bHidden);

	/** Return true if a pin can be hidden. */
	UE_API virtual bool CanPinBeHidden(const UEdGraphPin& Pin) const;

	/** Determines if a pin can be renamed or not which determines if the pin will spawn with a TextBox slate at the side of the pin. */
	UE_API virtual bool CanRenamePin(const UEdGraphPin& Pin) const;

	/** Determines the policy to use when drawing the pin slate. */
	UE_API virtual EEditablePinNameBoxVisibilityPolicy GetEditablePinNameVisibilityPolicy (const UEdGraphPin& Pin) const;

	/** Determines if the Pin Viewer is allowed to include the Pin Editable Name as part of the PinName string shown. */
	UE_API virtual bool ShouldPinViewerShowPinEditableName(const UEdGraphPin& Pin) const;

	UE_API virtual FText GetPinEditableName(const UEdGraphPin& Pin) const;

	UE_API virtual void SetPinEditableName(const UEdGraphPin& Pin, const FText& Value);
	
	UE_API virtual FLinearColor GetPinColor(const UEdGraphPin &Pin) const;
	
	/** @return true if the details should spawn a Pin Viewer. */
	UE_API virtual bool HasPinViewer() const;

	/** Determines to which side the "Add Pin" button should be added to*/
	UE_API virtual EAddPinNodeButtonLocation GetAddPinButtonNodeSide() const;

	/**
	 * Will request the generation of a new pin and, if successful, will notify the graph of the change.
	 * @note Method called when the Add Pins button is pressed.
	 */
	UE_API virtual void AddPinFromUI();

	/**
	 * Determines if a pin of this node can or can not be removed.
	 * Override this method to handle the removal of your node pins.
	 * @return True if the pin can be removed, false otherwise.
	 */
	UE_API virtual bool CanPinBeRemoved(const UEdGraphPin& Pin) const;
	
	/** Returns pin custom details. Override if required. */
	UE_API virtual TSharedPtr<IDetailsView> CustomizePinDetails(const UEdGraphPin& Pin) const;

	/** Given the pin data, returns its associated pin. */
	UE_API UEdGraphPin* GetPin(const UCustomizableObjectNodePinData& PinData);
	
	/** See GetPinData(const FEdGraphPinReference&). */
	UE_API UCustomizableObjectNodePinData* GetPinData(const UEdGraphPin &Pin) const;
	
	/** Given a pin which has attached data, get the typed pin data.
	 * @param Pin Pin which has attached data.
	 * @return Pin data already typed. */
	template<typename T>
	T& GetPinData(const UEdGraphPin& Pin) const
	{
		return *CastChecked<T>(PinsDataId[Pin.PinId]);
	}

	/** Returns true if this node can be added to a COMacro Graph */
	UE_API virtual bool IsNodeSupportedInMacros() const;

	/** Returns true if this node is inside a Macro Graph */
	UE_API bool IsInMacro() const;

	// Returns an array with all the PC_Types that can be created from the pin viewer
	UE_API virtual TArray<FName> GetAllowedPinViewerCreationTypes() const;

	/**
	 * Get the types the provided pin can changed to. 
	 * @param Pin The pin you whant to know the types it can be changed to. 
	 * @return A list of pin category names.
	 */
	UE_API virtual TArray<FName> GetPinAllowedTypes(const UEdGraphPin& Pin) const;

	UE_API virtual TArray<FName> GetPinAllowedSubTypes(const UEdGraphPin& Pin) const;

	UE_API virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	
	FPostEditChangePropertyDelegate PostEditChangePropertyDelegate;

	FPostReconstructNodeDelegate PostReconstructNodeDelegate;

	FNodeConnectionListChangedDelegate NodeConnectionListChangedDelegate;

	FPinConnectionListChangedDelegate PinConnectionListChangedDelegate;

	FPostEditablePinNameChangedDelegate PinEditableNameChangedDelegate;
	
	FDestroyNodeDelegate DestroyNodeDelegate;

	FRemapPinsDelegate RemapPinsDelegate;

protected:
	/**
	 * Specialization of AllocateDefaultPins UEdGraphNode function. Override.
	 * 
	 * Given the node context (this), create all the required pins.
	 *
	 * @param RemapPins remap pins action. Use this object to save any information required to later perform the remap pins action. Can be null.
	 */
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) {}

	/**
	 * Creates and returns the node default node remap pins action.
	 * 
	 * By default all nodes remap by name. Override if a node requires a different default remap pins action.
	 */
	UE_API virtual UCustomizableObjectNodeRemapPins* CreateRemapPinsDefault() const;

	/**
	 * Creates and returns a remap pins by name action. 
	 * 
	 * Override a if a node requires an specialized remap pins by name.
	 */
	UE_API virtual UCustomizableObjectNodeRemapPinsByName* CreateRemapPinsByName() const;

	/**
	 * Creates and returns a remap pins by position action.
	 *
	 * Override a if a node requires an specialized remap pins by position.
	 */
	UE_API virtual UCustomizableObjectNodeRemapPinsByPosition* CreateRemapPinsByPosition() const;
	
	/** Allows to perform work when remapping a pin.
	  * Copies pin data from old pin to new pin. Keeps the id of the new pin. */
	UE_API virtual void RemapPin(UEdGraphPin& NewPin, const UEdGraphPin& OldPin);

	/** Allows to perform work when remapping a pin. */
	UE_API virtual void RemapPins(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap);

	/*** Allows to perform work when remapping the pin data. */
	UE_API virtual void RemapPinsData(const TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap);
	
	/** Attach data to a given pin. */
	UE_API void AddPinData(const UEdGraphPin& Pin, UCustomizableObjectNodePinData& PinData);

public:
	/** Return true to mark this node as experimental. */
	UE_API virtual bool IsExperimental() const;

	/** Return true to mark this node as deprecated. Deprecated nodes emit a compile-time warning. */
	UE_API virtual bool IsDeprecated() const override;

	UE_API virtual bool IsLoaded() const;

private:
	/** Data attached to a given pin. Not all pins contain data.
	 * Data must be eliminated once the pins is removed. This is done automatically on CustomRemovePin.
	 * Do not use RemovePin since it will not remove the data. */
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UCustomizableObjectNodePinData>> PinsDataId;

	// Deprecated properties in favour of PinsDataId
	UPROPERTY()
	TMap<FEdGraphPinReference, TObjectPtr<UCustomizableObjectNodePinData>> PinsData_DEPRECATED;
};


/** Return the specified node located in another Customizable Object graph.
 *
 * @param Customizable Object object to look for the node. If nulltpr, it returns nullptr.
 * @param NodeGuid node guid. If invalid, it returns nullptr.
 * 
 * Return nullptr if not found.
 */
template<class FNodeType>
FNodeType* GetCustomizableObjectExternalNode(UCustomizableObject* Object, const FGuid& NodeGuid)
{
	FNodeType* Result = nullptr;

	if (Object && Object->GetPrivate()->GetSource() && NodeGuid.IsValid())
	{
		TArray<FNodeType*> GroupNodes;
		Object->GetPrivate()->GetSource()->GetNodesOfClass<FNodeType>(GroupNodes);

		for (FNodeType* GroupNode : GroupNodes)
		{
			if (NodeGuid == GroupNode->NodeGuid)
			{
				Result = GroupNode;
				break;
			}
		}
	}

	return Result;
}

#undef UE_API
