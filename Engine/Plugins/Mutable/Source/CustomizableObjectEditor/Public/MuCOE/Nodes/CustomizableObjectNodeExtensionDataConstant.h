// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOE/Nodes/CustomizableObjectNode.h"

#include "CustomizableObjectNodeExtensionDataConstant.generated.h"

/**
 * A base class for nodes that output a constant FExtensionData.
 *
 * This is typically used to import an Unreal asset into the Customizable Object graph.
 */
UCLASS(MinimalAPI, abstract)
class UCustomizableObjectNodeExtensionDataConstant : public UCustomizableObjectNode
{
	GENERATED_BODY()

public:
	virtual bool IsAffectedByLOD() const override { return false; }
};

