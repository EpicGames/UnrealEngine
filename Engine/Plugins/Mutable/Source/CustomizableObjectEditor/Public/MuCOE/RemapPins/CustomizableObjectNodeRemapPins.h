// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CustomizableObjectNodeRemapPins.generated.h"

class UEdGraphPin;
class UCustomizableObjectNode;


/**
 * Remap pins action interface.
 * 
 * Do not implement any functions or members directly, use inheritance. As an example, see UCustomizableObjectNodeRemapPinsByName class.
 * If data needs to be passed between ReconstructNode and AllocateDefaultPins, use the derived class context to save it.
 */
UCLASS()
class UCustomizableObjectNodeRemapPins : public UObject
{
public:
	GENERATED_BODY()

	/**
	 * Remap existing connections from old pins to new pins.
	 * 
	 * Use MovePersistentDataFromOldPin from this file to copy the links from the old node to the new node.
	 * 
	 * @param OldPins Old pins which are no longer present in the node.
	 * @param NewPins Current node pins.
	 * @param PinsToRemap Out parameter. Remapping pairs. Key is the old pin while value is the new pin.
	 * From pin has to be a pin from the NewPins, while FromPin has to be a pin from OldPins.
	 * @param PinsToOrphan Out Parameter. List of pins which will be marked as orphaned. Only pins from OldPins must be added.
	 */
	virtual void RemapPins(const UCustomizableObjectNode& Node, const TArray<UEdGraphPin*>& OldPins, const TArray<UEdGraphPin*>& NewPins, TMap<UEdGraphPin*, UEdGraphPin*>& PinsToRemap, TArray<UEdGraphPin*>& PinsToOrphan) { check(false); }
};

