// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/IntVector.h"
#include "Templates/Function.h"
// Configure the compression library with these defines:

// Support for BC1 to BC7, also known as DXTC1 to 5
#ifndef MIRO_INCLUDE_BC
#define MIRO_INCLUDE_BC                     1
#endif

// Support for ASTC
#ifndef MIRO_INCLUDE_ASTC
#define MIRO_INCLUDE_ASTC                   1
#endif

// Exclude high quality compression code. The index corresponds to the quality levels.
#ifndef MIRO_EXCLUDE_QUALITY_ABOVE
#define MIRO_EXCLUDE_QUALITY_ABOVE          4
#endif


namespace miro
{
    // MIRO is a runtime texture compression library for hardware-accelerated formats.

    // Compression quality levels (q):
    // 0 - Fastest for runtime
    // 1 - Best for runtime
    // 2 - Fast for tools
    // 3 - Best for tools
    // 4 - Maximum, with no time limits.

    using FImageSize = UE::Math::TIntVector2<uint16>;

    extern void initialize();
    extern void finalize();

#if MIRO_INCLUDE_BC

    //! BC1 support
    extern void RGB_to_BC1( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGBA_to_BC1( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void BC1_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC1_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );

    //! BC2 support
    extern void RGB_to_BC2( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGBA_to_BC2( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void BC2_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC2_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );

    //! BC3 support
    extern void RGB_to_BC3( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGBA_to_BC3( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void BC3_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC3_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC1_to_BC3( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );

    //! BC4 support
    extern void L_to_BC4( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void BC4_to_L( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC4_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC4_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );

    //! BC5 support
    extern void RGBA_to_BC5( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGB_to_BC5( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void BC5_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void BC5_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );

#endif


#if MIRO_INCLUDE_ASTC

    //! ASTC 4x4 support
	MUTABLERUNTIME_API extern void RGBA_to_ASTC4x4RGBAL( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGB_to_ASTC4x4RGBAL( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void ASTC4x4RGBAL_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void ASTC4x4RGBAL_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );

    extern void RGB_to_ASTC4x4RGBL( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGBA_to_ASTC4x4RGBL( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void ASTC4x4RGBL_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void ASTC4x4RGBL_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );
	extern void L_to_ASTC4x4RGBL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);

    extern void RGB_to_ASTC4x4RGL( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void RGBA_to_ASTC4x4RGL( uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality );
    extern void ASTC4x4RGL_to_RGB( uint32 sx, uint32 sy, const uint8* From, uint8* To );
    extern void ASTC4x4RGL_to_RGBA( uint32 sx, uint32 sy, const uint8* From, uint8* To );

	extern void ASTC4x4RGBL_to_ASTC4x4RGBAL(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC4x4RGBAL_to_ASTC4x4RGBL(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	//! ASTC 6x6 support
	extern void RGBA_to_ASTC6x6RGBAL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void RGB_to_ASTC6x6RGBAL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void ASTC6x6RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC6x6RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	
	extern void RGB_to_ASTC6x6RGBL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void RGBA_to_ASTC6x6RGBL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void ASTC6x6RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC6x6RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	extern void RGB_to_ASTC6x6RGL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void RGBA_to_ASTC6x6RGL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void ASTC6x6RGL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC6x6RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	//! ASTC 8x8 support
	MUTABLERUNTIME_API extern void RGBA_to_ASTC8x8RGBAL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void RGB_to_ASTC8x8RGBAL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void ASTC8x8RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC8x8RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	extern void RGB_to_ASTC8x8RGBL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void RGBA_to_ASTC8x8RGBL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void ASTC8x8RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC8x8RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	extern void RGB_to_ASTC8x8RGL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void RGBA_to_ASTC8x8RGL(uint32 sx, uint32 sy, const uint8* From, uint8* To, int32 Quality);
	extern void ASTC8x8RGL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC8x8RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	//! ASTC 10x10 support
	extern void ASTC10x10RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC10x10RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC10x10RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC10x10RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC10x10RGL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC10x10RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);

	//! ASTC 12x12 support
	extern void ASTC12x12RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC12x12RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC12x12RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC12x12RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC12x12RGL_to_RGB(uint32 sx, uint32 sy, const uint8* From, uint8* To);
	extern void ASTC12x12RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* From, uint8* To);

#endif

}

namespace miro::SubImageCompression
{
    using FuncRefType = TFunctionRef<void(miro::FImageSize, miro::FImageSize, miro::FImageSize, const uint8*, uint8*, int32)>;
	
#if MIRO_INCLUDE_BC
    void RGB_To_BC3SubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGBA_To_BC3SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

    void RGB_To_BC1SubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGBA_To_BC1SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

    void RGB_To_BC2SubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGBA_To_BC2SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

    void L_To_BC4SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

    void RGB_To_BC5SubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGBA_To_BC5SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

#endif

#if MIRO_INCLUDE_ASTC

    /** ASTC 4x4 support */
    void RGBA_To_ASTC4x4RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGB_To_ASTC4x4RGBALSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

    void RGB_To_ASTC4x4RGBLSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGBA_To_ASTC4x4RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void L_To_ASTC4x4RGBLSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGB_To_ASTC4x4RGLSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
    void RGBA_To_ASTC4x4RGLSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

	/** ASTC 6x6 support */
	void RGBA_To_ASTC6x6RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGB_To_ASTC6x6RGBALSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGB_To_ASTC6x6RGBLSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGBA_To_ASTC6x6RGBLSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGB_To_ASTC6x6RGLSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGBA_To_ASTC6x6RGLSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

	/** ASTC 8x8 support */
	void RGBA_To_ASTC8x8RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGB_To_ASTC8x8RGBALSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGB_To_ASTC8x8RGBLSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGBA_To_ASTC8x8RGBLSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGB_To_ASTC8x8RGLSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);
	void RGBA_To_ASTC8x8RGLSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality);

#endif

}


namespace miro::SubImageDecompression
{
    using FuncRefType = TFunctionRef<void(miro::FImageSize, miro::FImageSize, miro::FImageSize, const uint8*, uint8*)>;

#if MIRO_INCLUDE_ASTC
	 void ASTC4x4RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC4x4RGBAL_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC4x4RGBL_To_RGBSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC4x4RGBL_To_RGBASubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC4x4RGL_To_RGBSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC4x4RGL_To_RGBASubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);

	 void ASTC6x6RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC6x6RGBAL_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC6x6RGBL_To_RGBSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC6x6RGBL_To_RGBASubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC6x6RGL_To_RGBSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC6x6RGL_To_RGBASubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);

	 void ASTC8x8RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC8x8RGBAL_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC8x8RGBL_To_RGBSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC8x8RGBL_To_RGBASubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC8x8RGL_To_RGBSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC8x8RGL_To_RGBASubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);

	 void ASTC10x10RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC10x10RGBAL_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC10x10RGBL_To_RGBSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC10x10RGBL_To_RGBASubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC10x10RGL_To_RGBSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC10x10RGL_To_RGBASubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);

	 void ASTC12x12RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC12x12RGBAL_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC12x12RGBL_To_RGBSubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC12x12RGBL_To_RGBASubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC12x12RGL_To_RGBSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void ASTC12x12RGL_To_RGBASubImage  (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);

#endif // MIRO_INCLUDE_ASTC
	// BC Formats.

#if MIRO_INCLUDE_BC
	 void BC1_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC1_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC2_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC2_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC3_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC3_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC4_To_LSubImage   (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC4_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC4_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC5_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
	 void BC5_To_RGBSubImage (FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To);
#endif
}

