// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/RefCounted.h"
#include "PassthroughObject.h"
#include "ClothingAsset.h"
#include "MemoryTrackingAllocationPolicy.h"
#include "MeshBufferSet.h"
#include "Rendering/SkeletalMeshVertexClothBuffer.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
	class FCloth : public FResource
	{
	public:
		FCloth() = default;

		FCloth(const FCloth& Other) = default;
		
		FCloth& operator=(const FCloth& Other) = default;

		UE_API bool operator==(const FCloth& Other) const;
		
		UE_API virtual int32 GetDataSize() const override;
		
		UE_API void Serialise(FOutputArchive& Ar) const;
		UE_API void Unserialise(FInputArchive& Ar);
		
		UE_API bool IsValid() const;
	
		int32 AssetLODIndex = -1;
		TArray<FMeshToMeshVertData, FDefaultMemoryTrackingAllocator<MemoryCounters::FMeshMemoryCounter>> Data;
		
		TPassthroughObjectPtr<UClothingAssetBase> ClothingAsset;
	};
}


#undef UE_API
