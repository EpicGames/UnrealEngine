// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/System.h"
#include "MuR/System.h"

namespace UE::Mutable::Private
{
    /** Code visitor that:
    * - is top-down
    * - cannot change the visited instructions.
    * - will not visit twice the same instruction with the same state.
    * - Its iterative
	*/
    template<typename STATE=int32>
    class UniqueConstCodeVisitorIterative
    {
    public:

        UniqueConstCodeVisitorIterative()
        {
            // Default state
            States.Add(STATE());
            CurrentState = 0;
        }

        //! Ensure virtual destruction
        virtual ~UniqueConstCodeVisitorIterative() = default;

    protected:

        //!
        void SetDefaultState(const STATE& State)
        {
            States[0] = State;
        }

        //!
        const STATE& GetDefaultState() const
        {
            return States[0];
        }

        //! Use this from visit to access the state at the time of processing the current
        //! instruction.
        STATE GetCurrentState() const
        {
            return States[CurrentState];
        }

        //! For manual recursion that changes the state for a specific path.
        void RecurseWithState(FOperation::ADDRESS Address, const STATE& NewState)
        {
			int32 StateIndex = States.Find(NewState);
            if (StateIndex==INDEX_NONE)
            {
				StateIndex = States.Add(NewState);
            }
            Pending.Emplace( Address, StateIndex );
        }

        //! For manual recursion that doesn't change the state for a specific path.
        void RecurseWithCurrentState(FOperation::ADDRESS Address)
        {
            Pending.Emplace( Address, CurrentState );
        }

        //! Can be called from visit to set the state to visit all children ops
        void SetCurrentState(const STATE& NewState)
        {
			CurrentState = States.Find(NewState);
            if (CurrentState ==INDEX_NONE)
            {
				CurrentState = States.Add(NewState);
            }
        }


        void Traverse(FOperation::ADDRESS Root, const FProgram& Program )
        {
            Pending.Reserve(Program.NumOps);
            // Visit the given root
            Pending.Emplace(Root, 0);
            Recurse(Program);
         }

        void FullTraverse(const FProgram& Program)
        {
            // Visit all the state roots
            for (int32 StateIndex = 0; StateIndex<Program.States.Num(); ++StateIndex)
            {
                Pending.Add(FPending(Program.States[StateIndex].Root, 0));
                Recurse(Program);
            }
        }


    private:

        /** Do the actual work by overriding this in the derived classes.
         * Return true if the traverse has to continue with the children of the given address.
		 */
        virtual bool Visit(FOperation::ADDRESS, const FProgram&) = 0;

        //! Operations to be processed
        struct FPending
        {
			FPending()
			{
				Address = 0;
				StateIndex = 0;
			}

			FPending(FOperation::ADDRESS InAddress, int32 InStateIndex)
			{
				Address = InAddress;
				StateIndex = InStateIndex;
			}
			
			FOperation::ADDRESS Address=0;
            int32 StateIndex=0;
        };
		TArray<FPending> Pending;

        //! States found so far
		TArray<STATE> States;

        //! Index of the current state, from the States array.
        int32 CurrentState;
    	
        //! Array of states visited for each operation.
        //! Empty array means operation not visited at all.
		TMap<FOperation::ADDRESS, TArray<int32>> Visited;

        //! Process all the pending operations and visit all children if necessary
        void Recurse(const FProgram& Program)
        {
			Visited.Empty();
			Visited.Reserve(Program.NumOps);

            while ( Pending.Num() )
            {
                FOperation::ADDRESS Address = Pending.Last().Address;
                CurrentState = Pending.Last().StateIndex;
                Pending.Pop();

                bool bRecurse = false;

                bool bVisitedInThisState = Visited.FindOrAdd(Address).Contains(CurrentState);
                if (!bVisitedInThisState)
                {
                    Visited[Address].Add(CurrentState);

                    // Visit may change current state
                    bRecurse = Visit(Address, Program);
                }

                if (bRecurse)
                {
                    ForEachReference( Program, Address, [&](FOperation::ADDRESS Ref)
                    {
                        if (Ref)
                        {
                            Pending.Emplace(Ref, CurrentState);
                        }
                    });
                }
            }
        }

    };


    //---------------------------------------------------------------------------------------------
    //! Code visitor template for visitors that:
    //! - only traverses the operations that are relevant for a given set of parameter values. It
    //! only considers the discrete parameters like integers and booleans. In the case of forks
    //! caused by continuous parameters like float weights for interpolation, all the branches are
    //! traversed.
    //! - cannot change the instructinos
    //---------------------------------------------------------------------------------------------
    struct COVERED_CODE_VISITOR_STATE
    {
    };

    template<typename PARENT,typename STATE>
    class DiscreteCoveredCodeVisitorBase : public PARENT
    {
    public:

        DiscreteCoveredCodeVisitorBase(FSystem* InSystem, const TSharedRef<FLiveInstance>& InLiveInstance) : LiveInstance(InLiveInstance)
        {
            System = InSystem;
            Model = InLiveInstance->Model;

            // Visiting state
            PARENT::SetDefaultState( STATE() );
        }

        void Run( FOperation::ADDRESS at  )
        {
            PARENT::SetDefaultState( STATE() );

            PARENT::Traverse( at, Model->GetProgram() );
        }

    protected:

        virtual bool Visit(FOperation::ADDRESS Address, const FProgram& Program)
        {
            bool bRecurse = true;

            EOpType Type = Program.GetOpType(Address);

            switch ( Type )
            {
            case EOpType::NU_CONDITIONAL:
            case EOpType::SC_CONDITIONAL:
            case EOpType::CO_CONDITIONAL:
            case EOpType::IM_CONDITIONAL:
            case EOpType::ME_CONDITIONAL:
            case EOpType::LA_CONDITIONAL:
            case EOpType::IN_CONDITIONAL:
            case EOpType::ED_CONDITIONAL:
            case EOpType::MI_CONDITIONAL:
            case EOpType::SK_CONDITIONAL:
            case EOpType::LD_CONDITIONAL:
            case EOpType::IS_CONDITIONAL:
            	{
            		FOperation::ConditionalArgs Args = Program.GetOpArgs<FOperation::ConditionalArgs>(Address);

            		bRecurse = false;

            		PARENT::RecurseWithCurrentState( Args.condition );

            		// If there is no expression, we'll assume true.
            		bool bValue = true;

            		if (Args.condition)
            		{
            			bValue = System->BuildBool(LiveInstance, Args.condition);
            		}

            		if (bValue)
            		{
            			PARENT::RecurseWithCurrentState( Args.yes );
            		}
            		else
            		{
            			PARENT::RecurseWithCurrentState( Args.no );
            		}
            		break;
            	}

            case EOpType::NU_SWITCH:
            case EOpType::SC_SWITCH:
            case EOpType::CO_SWITCH:
            case EOpType::IM_SWITCH:
            case EOpType::ME_SWITCH:
            case EOpType::LA_SWITCH:
            case EOpType::IN_SWITCH:
            case EOpType::ED_SWITCH:
            case EOpType::SK_SWITCH:
            case EOpType::LD_SWITCH:
            case EOpType::MI_SWITCH:
            	{
            		bRecurse = false;

            		const uint8* Data = Program.GetOpArgsPointer(Address);
				
            		FOperation::ADDRESS VarAddress;
            		FMemory::Memcpy(&VarAddress, Data, sizeof(FOperation::ADDRESS));
            		Data += sizeof(FOperation::ADDRESS);

            		if (VarAddress)
            		{
            			FOperation::ADDRESS DefAddress;
            			FMemory::Memcpy(&DefAddress, Data, sizeof(FOperation::ADDRESS));
            			Data += sizeof(FOperation::ADDRESS);

            			FOperation::FSwitchCaseDescriptor CaseDesc;
            			FMemory::Memcpy(&CaseDesc, Data, sizeof(FOperation::FSwitchCaseDescriptor));
            			Data += sizeof(FOperation::FSwitchCaseDescriptor);

            			PARENT::RecurseWithCurrentState( VarAddress );

            			int32 Var = System->BuildInt(LiveInstance, VarAddress );

            			FOperation::ADDRESS ValueAt = DefAddress;

            			if (!CaseDesc.bUseRanges)
            			{
            				for (uint32 CaseIndex = 0; CaseIndex < CaseDesc.Count; ++CaseIndex)
            				{
            					int32 Condition;
            					FMemory::Memcpy(&Condition, Data, sizeof(int32));
            					Data += sizeof(int32);

            					FOperation::ADDRESS CaseAt;
            					FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
            					Data += sizeof(FOperation::ADDRESS);

            					if (CaseAt && Var == (int32)Condition)
            					{
            						ValueAt = CaseAt;
            						break;
            					}
            				}
            			}
            			else
            			{
            				for (uint32 C = 0; C < CaseDesc.Count; ++C)
            				{
            					int32 ConditionStart;
            					FMemory::Memcpy(&ConditionStart, Data, sizeof(int32));
            					Data += sizeof(int32);

            					uint32 RangeSize;
            					FMemory::Memcpy(&RangeSize, Data, sizeof(uint32));
            					Data += sizeof(uint32);

            					FOperation::ADDRESS CaseAt;
            					FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
            					Data += sizeof(FOperation::ADDRESS);

            					if (CaseAt && Var >= ConditionStart && Var < int32(ConditionStart + RangeSize))
            					{
            						ValueAt = CaseAt;
            						break;
            					}
            				}
            			}

            			PARENT::RecurseWithCurrentState(ValueAt);
            		}

            		break;
            	}
            }

            return bRecurse;
        }
    
        FSystem* System = nullptr;
		TSharedPtr<const FModel> Model;
    	TSharedRef<FLiveInstance> LiveInstance;
    };


    /** Code visitor that :
    * - only traverses the operations that are relevant for a given set of parameter values. It
    * only considers the discrete parameters like integers and booleans. In the case of forks
    * caused by continuous parameters like float weights for interpolation, all the branches are
    * traversed.
    * - cannot change the instructions
    * - will not repeat visits to instructions with the same state
    * - the state has to be a compatible with COVERED_CODE_VISITOR_STATE
    */
    template<typename COVERED_STATE = COVERED_CODE_VISITOR_STATE>
    class UniqueDiscreteCoveredCodeVisitor : public DiscreteCoveredCodeVisitorBase<UniqueConstCodeVisitorIterative<COVERED_STATE>, COVERED_STATE>
    {
        using PARENT=DiscreteCoveredCodeVisitorBase<UniqueConstCodeVisitorIterative<COVERED_STATE>, COVERED_STATE>;

    public:

        UniqueDiscreteCoveredCodeVisitor
            (
                FSystem* InSystem,
				const TSharedRef<FLiveInstance>& InLiveInstance
            )
            : PARENT(InSystem, InLiveInstance)
        {
        }

    };

}

