// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


namespace UE::Mutable::Private
{
	using FComponentId = int32;
}
