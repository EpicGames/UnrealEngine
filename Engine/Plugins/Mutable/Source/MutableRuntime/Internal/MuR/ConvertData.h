// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Mesh.h"
#include "MuR/Platform.h"
#include "MuR/MutableMath.h"
#include "Math/Float16.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	//! Convert one channel element
	//---------------------------------------------------------------------------------------------
	inline void ConvertData
		(
			int channel,
			void* pResult, EMeshBufferFormat resultFormat,
			const void* pSource, EMeshBufferFormat sourceFormat
		)
	{
		switch ( resultFormat )
		{
		case EMeshBufferFormat::Float64:
		{
			double* pTypedResult = reinterpret_cast<double*>(pResult);
			uint8* pByteResult = reinterpret_cast<uint8*>(pResult);
			const uint8* pByteSource = reinterpret_cast<const uint8*>(pSource);

			switch (sourceFormat)
			{
			case EMeshBufferFormat::Float64:
			{
				// Just dereferencing is not safe in all architectures. some like ARM require the
				// floats to be 4-byte aligned and it may not be the case. We need memcpy.
				memcpy(pByteResult + 8 * channel, pByteSource + 8* channel, 8);
				break;
			}

			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int32:
			{
				const int32* pTypedSource = reinterpret_cast<const int32*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
				const uint32* pTypedSource = reinterpret_cast<const uint32*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int16:
			{
				const int16* pTypedSource = reinterpret_cast<const int16*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
				const uint16* pTypedSource = reinterpret_cast<const uint16*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
				const int8* pTypedSource = reinterpret_cast<const int8*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
				const uint8* pTypedSource = reinterpret_cast<const uint8*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::NInt32:
			{
				const int32* pTypedSource = reinterpret_cast<const int32*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 65536.0f * 65536.0f / 2.0f;
				break;
			}

			case EMeshBufferFormat::NUInt32:
			{
				const uint32* pTypedSource = reinterpret_cast<const uint32*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 65536.0f * 65536.0f - 1.0f;
				break;
			}

			case EMeshBufferFormat::NInt16:
			{
				const int16* pTypedSource = reinterpret_cast<const int16*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 32768.0f;
				break;
			}

			case EMeshBufferFormat::NUInt16:
			{
				const uint16* pTypedSource = reinterpret_cast<const uint16*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 65535.0f;
				break;
			}

			case EMeshBufferFormat::NInt8:
			{
				const int8* pTypedSource = reinterpret_cast<const int8*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 128.0f;
				break;
			}

			case EMeshBufferFormat::NUInt8:
			{
				const uint8* pTypedSource = reinterpret_cast<const uint8*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 255.0f;
				break;
			}

			case EMeshBufferFormat::PackedDir8:
			case EMeshBufferFormat::PackedDir8_W_TangentSign:
			{
				const uint8* pTypedSource = reinterpret_cast<const uint8*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 127.5f;
				pTypedResult[channel] -= 1.0f;
				break;
			}

			case EMeshBufferFormat::PackedDirS8:
			case EMeshBufferFormat::PackedDirS8_W_TangentSign:
			{
				const int8* pTypedSource = reinterpret_cast<const int8*>(pSource);
				pTypedResult[channel] = (double)(pTypedSource[channel]);
				pTypedResult[channel] /= 127.5f;
				break;
			}

			default:
				checkf(false, TEXT("Conversion not implemented."));
				break;
			}
			break;
		}

		case EMeshBufferFormat::Float32:
		{
			float* pTypedResult = reinterpret_cast<float*>( pResult );
            uint8* pByteResult = reinterpret_cast<uint8*>( pResult );
            const uint8* pByteSource = reinterpret_cast<const uint8*>( pSource );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float64:
			{
				const double* pTypedSource = reinterpret_cast<const double*>(pSource);
				pTypedResult[channel] = float(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Float32:
			{
				// Just dereferencing is not safe in all architectures. some like ARM require the
				// floats to be 4-byte aligned and it may not be the case. We need memcpy.
				memcpy(pByteResult + 4 * channel, pByteSource + 4 * channel, 4);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int32:
			{
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::NInt32:
			{
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				pTypedResult[channel] /= 65536.0f*65536.0f/2.0f;
				break;
			}

			case EMeshBufferFormat::NUInt32:
			{
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				pTypedResult[channel] /= 65536.0f*65536.0f-1.0f;
				break;
			}

			case EMeshBufferFormat::NInt16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				pTypedResult[channel] /= 32768.0f;
				break;
			}

			case EMeshBufferFormat::NUInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				pTypedResult[channel] /= 65535.0f;
				break;
			}

			case EMeshBufferFormat::NInt8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				pTypedResult[channel] /= 128.0f;
				break;
			}

			case EMeshBufferFormat::NUInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				pTypedResult[channel] /= 255.0f;
				break;
			}

            case EMeshBufferFormat::PackedDir8:
            case EMeshBufferFormat::PackedDir8_W_TangentSign:
            {
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
                pTypedResult[channel] = (float)(pTypedSource[channel]);
                pTypedResult[channel] /= 127.5f;
                pTypedResult[channel] -= 1.0f;
                break;
            }

            case EMeshBufferFormat::PackedDirS8:
            case EMeshBufferFormat::PackedDirS8_W_TangentSign:
            {
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                pTypedResult[channel] = (float)(pTypedSource[channel]);
                pTypedResult[channel] /= 127.5f;
                break;
            }

			default:
				checkf( false, TEXT("Conversion not implemented.") );
				break;
			}
			break;
		}

		//-----------------------------------------------------------------------------------------
		case EMeshBufferFormat::Float16:
		{
			FFloat16* pTypedResult = reinterpret_cast<FFloat16*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int32:
			{
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
				pTypedResult[channel] = (float)(pTypedSource[channel]);
				break;
			}

			default:
				checkf( false, TEXT("Conversion not implemented.") );
				break;
			}
			break;
		}

		//-----------------------------------------------------------------------------------------
		case EMeshBufferFormat::UInt8:
		{
            uint8* pTypedResult = reinterpret_cast<uint8*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (uint8)
                        FMath::Min<uint32>(
							0xFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (uint8)
                        FMath::Min<uint32>(
							0xFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)float(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                pTypedResult[channel] =  (uint8)FMath::Max<int8>( 0, pTypedSource[channel] );
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
                pTypedResult[channel] =  (uint8)
                        FMath::Min<int16>(
							0xFF,
                            FMath::Max<int16>(
								0,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
				// Clamp
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
                pTypedResult[channel] = (uint8)
                        FMath::Min<uint16>(
							0xFF,
                            FMath::Max<uint16>(
								0,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::Int32:
			{
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
                pTypedResult[channel] =  (uint8)
                        FMath::Min<int32>(
							0xFF,
                            FMath::Max<int32>(
								0,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
				// Clamp
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
                pTypedResult[channel] = (uint8)
                        FMath::Min<uint32>(
							0xFF,
                            FMath::Max<uint32>(
								0,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::NInt32:
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::UInt16:
		{
            uint16* pTypedResult = reinterpret_cast<uint16*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (uint16)
                        FMath::Min<uint32>(
							0xFFFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (uint16)
                        FMath::Min<uint32>(
							0xFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)float(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
				pTypedResult[channel] =  pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                pTypedResult[channel] =  (uint16)FMath::Max<int8>( 0, pTypedSource[channel] );
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
				pTypedResult[channel] =  pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
                pTypedResult[channel] =  (uint16)FMath::Max<int16>( 0, pTypedSource[channel] );
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
				// Clamp
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
                pTypedResult[channel] = (uint16)
                        FMath::Min<uint32>(
							0xFFFF,
                            FMath::Max<uint32>(
								0,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::Int32:
			{
				// Clamp
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
                pTypedResult[channel] = (uint16)
                        FMath::Min<int32>(
							0xFFFF,
                            FMath::Max<int32>(
								0,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::NInt32:
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::UInt32:
		{
            uint32* pTypedResult = reinterpret_cast<uint32*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
				pTypedResult[channel] =
                        FMath::Min<uint32>(
							0xFFFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
				pTypedResult[channel] =
                        FMath::Min<uint32>(
							0xFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)float(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
				pTypedResult[channel] =  pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                pTypedResult[channel] =  (uint16)FMath::Max<int8>( 0, pTypedSource[channel] );
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
				pTypedResult[channel] =  pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
                pTypedResult[channel] =  (uint16)FMath::Max<int16>( 0, pTypedSource[channel] );
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int32:
			{
				// Clamp
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
                pTypedResult[channel] = (uint32)FMath::Max<int32>( 0, pTypedSource[channel] );
				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::NInt32:
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::UInt64:
		{
			uint64* pTypedResult = reinterpret_cast<uint64*>(pResult);

			switch (sourceFormat)
			{
			case EMeshBufferFormat::UInt8:
			{
				const uint8* pTypedSource = reinterpret_cast<const uint8*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
				const uint16* pTypedSource = reinterpret_cast<const uint16*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
				const uint32* pTypedSource = reinterpret_cast<const uint32*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt64:
			{
				const uint64* pTypedSource = reinterpret_cast<const uint64*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			default:
				checkf(false, TEXT("Conversion not implemented."));
				break;
			}
			break;
		}

		case EMeshBufferFormat::Int8:
		{
            int8* pTypedResult = reinterpret_cast<int8*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (int8)
                        FMath::Min<int32>(
							127,
                            FMath::Max<int32>(
								-128,
                                (int32)(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (int8)
                        FMath::Min<int32>(
							127,
                            FMath::Max<int32>(
								-128,
                                (int32)float(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::NInt32:
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::Int16:
		{
            int16* pTypedResult = reinterpret_cast<int16*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
							32767,
                            FMath::Max<int32>(
								-32768,
                                (int32)(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
							32767,
                            FMath::Max<int32>(
								-32768,
                                (int32)float(pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                pTypedResult[channel] = (int16)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
                pTypedResult[channel] = (int16)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
                            32767, (int32)pTypedSource[channel]
							);
				break;
			}

			case EMeshBufferFormat::Int32:
			{
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
							32767,
                            FMath::Max<int32>(
								-32768,
								pTypedSource[channel]
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
							32767, pTypedSource[channel]
							);
				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::NInt32:
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::Int32:
		{
            int32* pTypedResult = reinterpret_cast<int32*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (int32)(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (int32)float(pTypedSource[channel]);
				break;
			}

			case EMeshBufferFormat::Int8:
			{
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                pTypedResult[channel] = (int32)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt8:
			{
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
                pTypedResult[channel] = (int32)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int16:
			{
                const int16* pTypedSource = reinterpret_cast<const int16*>( pSource );
                pTypedResult[channel] = (int32)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt16:
			{
                const uint16* pTypedSource = reinterpret_cast<const uint16*>( pSource );
                pTypedResult[channel] = (int32)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::UInt32:
			{
                const uint32* pTypedSource = reinterpret_cast<const uint32*>( pSource );
                pTypedResult[channel] = (int32)pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::Int32:
			{
                const int32* pTypedSource = reinterpret_cast<const int32*>( pSource );
                pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::NInt32:
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		//-----------------------------------------------------------------------------------------
		case EMeshBufferFormat::NUInt8:
		{
            uint8* pTypedResult = reinterpret_cast<uint8*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::NUInt8:
			{
				const uint8* pTypedSource = reinterpret_cast<const uint8*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::NUInt16:
			{
				const uint16* pTypedSource = reinterpret_cast<const uint16*>(pSource);
				pTypedResult[channel] = pTypedSource[channel] / (65535 / 255);
				break;
			}

			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>(pSource);
				pTypedResult[channel] = (uint8)
					FMath::Min<uint32>(
						0xFF,
						FMath::Max<uint32>(
							0,
							(uint32)(((float)0xFF)*pTypedSource[channel] + 0.5f)
							)
						);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (uint8)
                        FMath::Min<uint32>(
							0xFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(((float)0xFF)*float(pTypedSource[channel])+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int8:
			case EMeshBufferFormat::Int16:
			case EMeshBufferFormat::Int32:
				// TODO: 1 or 0
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::NUInt16:
		{
            uint16* pTypedResult = reinterpret_cast<uint16*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::NUInt16:
			{
				const uint16* pTypedSource = reinterpret_cast<const uint16*>(pSource);
				pTypedResult[channel] = pTypedSource[channel];
				break;
			}

			case EMeshBufferFormat::NUInt8:
			{
				const uint8* pTypedSource = reinterpret_cast<const uint8*>(pSource);
				pTypedResult[channel] = pTypedSource[channel] * (65535 / 255);
				break;
			}

			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (uint16)
                        FMath::Min<uint32>(
							0xFFFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(((float)0xFFFF)*pTypedSource[channel]+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (uint16)
                        FMath::Min<uint32>(
							0xFFFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(((float)0xFFFF)*float(pTypedSource[channel])+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int8:
			case EMeshBufferFormat::Int16:
			case EMeshBufferFormat::Int32:
				// TODO: 1 or 0
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::NUInt32:
		{
            uint32* pTypedResult = reinterpret_cast<uint32*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (uint32)
                        FMath::Min<uint32>(
							0xFFFFFFFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(((float)0xFFFFFFFF)*pTypedSource[channel])
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (uint32)
                        FMath::Min<uint32>(
							0xFFFFFFFF,
                            FMath::Max<uint32>(
								0,
                                (uint32)(((float)0xFFFFFFFF)*float(pTypedSource[channel])+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int8:
			case EMeshBufferFormat::Int16:
			case EMeshBufferFormat::Int32:
				// TODO: 1 or 0
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::NInt8:
		{
            int8* pTypedResult = reinterpret_cast<int8*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (int8)
                        FMath::Min<int32>(
							127,
                            FMath::Max<int32>(
								-128,
                                (int32)(128.0f*pTypedSource[channel]+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (int8)
                        FMath::Min<int32>(
							127,
                            FMath::Max<int32>(
								-128,
                                (int32)(128.0f*float(pTypedSource[channel])+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int8:
			case EMeshBufferFormat::Int16:
			case EMeshBufferFormat::Int32:
				// TODO: -1, 1 or 0
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

		case EMeshBufferFormat::NInt16:
		{
            int16* pTypedResult = reinterpret_cast<int16*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
							32767,
                            FMath::Max<int32>(
								-32768,
                                (int32)(32768.0f*pTypedSource[channel]+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (int16)
                        FMath::Min<int32>(
							32767,
                            FMath::Max<int32>(
								-32768,
                                (int32)(32768.0f*float(pTypedSource[channel])+0.5f)
								)
							);
				break;
			}

			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int8:
			case EMeshBufferFormat::Int16:
			case EMeshBufferFormat::Int32:
				// TODO: -1, 1 or 0
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}


		case EMeshBufferFormat::NInt32:
		{
            int32* pTypedResult = reinterpret_cast<int32*>( pResult );

			switch ( sourceFormat )
			{
			case EMeshBufferFormat::Float32:
			{
				const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                pTypedResult[channel] = (int32)(2147483648.0f*pTypedSource[channel]+0.5f);
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				const FFloat16* pTypedSource = reinterpret_cast<const FFloat16*>( pSource );
                pTypedResult[channel] = (int32)(2147483648.0f*float(pTypedSource[channel])+0.5f);
				break;
			}

			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int8:
			case EMeshBufferFormat::Int16:
			case EMeshBufferFormat::Int32:
				// TODO: -1, 1 or 0
				pTypedResult = 0;
				break;

			default:
				checkf( false, TEXT("Conversion not implemented." ) );
				break;
			}
			break;
		}

        case EMeshBufferFormat::PackedDir8:
        case EMeshBufferFormat::PackedDir8_W_TangentSign:
        {
            uint8* pTypedResult = reinterpret_cast<uint8*>( pResult );

            switch ( sourceFormat )
            {
            case EMeshBufferFormat::PackedDir8:
            case EMeshBufferFormat::PackedDir8_W_TangentSign:
            {
                const uint8* pTypedSource = reinterpret_cast<const uint8*>( pSource );
                uint8 source = pTypedSource[channel];
                pTypedResult[channel] = source;
                break;
            }

            case EMeshBufferFormat::Float32:
            {
                const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                float source = pTypedSource[channel];
                source = (source*0.5f+0.5f)*255.0f;
                pTypedResult[channel] =
                        (uint8)FMath::Min<float>( 255.0f, FMath::Max<float>( 0.0f, source ) );
                break;
            }

            default:
                checkf( false, TEXT("Conversion not implemented." ) );
                break;
            }
            break;
        }

        case EMeshBufferFormat::PackedDirS8:
        case EMeshBufferFormat::PackedDirS8_W_TangentSign:
        {
            int8* pTypedResult = reinterpret_cast<int8*>( pResult );

            switch ( sourceFormat )
            {
            case EMeshBufferFormat::PackedDirS8:
            case EMeshBufferFormat::PackedDirS8_W_TangentSign:
            {
                const int8* pTypedSource = reinterpret_cast<const int8*>( pSource );
                int8 source = pTypedSource[channel];
                pTypedResult[channel] = source;
                break;
            }

            case EMeshBufferFormat::Float32:
            {
                const float* pTypedSource = reinterpret_cast<const float*>( pSource );
                float source = pTypedSource[channel];
                source = source*0.5f*255.0f;
                pTypedResult[channel] =
                        (int8)FMath::Min<float>( 127.0f, FMath::Max<float>( -128.0f, source ) );
                break;
            }

            default:
                checkf( false, TEXT("Conversion not implemented." ) );
                break;
            }
            break;
        }

		default:
			checkf( false, TEXT("Conversion not implemented." ) );
			break;
		}

	}

}
