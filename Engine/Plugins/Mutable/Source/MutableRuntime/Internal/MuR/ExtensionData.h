// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Types.h"
#include "MuR/PassthroughObject.h"
#include "UObject/Object.h"

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private
{
	/** ExtensionData represents data types that Mutable doesn't support natively.
	* Extensions can provide data, and functionality to operate on that data, without Mutable
	* needing to know what the data refers to.
	*/
	class FExtensionData
	{
	public:
		TPassthroughObjectPtr<UObject> PassthroughObject;
	};
}

#undef UE_API
