// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Operations.h"
#include "StructUtils/InstancedStruct.h"


namespace UE::Mutable::Private
{
	class IExternalOperationProvider
	{
	public:
		virtual ~IExternalOperationProvider() = default;
		
		virtual const FInstancedStruct& Get(FOperation::ADDRESS Address) const = 0;
	};
}

