// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/PassthroughObject.h"
#include "MuR/ImageTypes.h"

#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "Containers/Array.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Engine/Texture.h"

#include "MuR/ImageDataStorage.h"

#include "MuR/ManagedPointer.h"

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private
{
    /** 2D image resource with mipmaps.
	*/
    class FImage : public FResource
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------

		UE_API FImage();

		//! Constructor from the size and format.
		//! It will allocate the memory for the image, but its content will be undefined.
		//! \param sizeX Width of the image.
        //! \param sizeY Height of the image.
        //! \param lods Number of levels of detail (mipmaps) to include in the image, inclduing the
        //!         base level. It must be a number between 1 and the maximum possible levels, which
        //!         depends on the image size.
        //! \param format Pixel format.
		UE_API FImage(uint32 SizeX, uint32 SizeY, uint32 Lods, EImageFormat Format, EInitializationType InitType);

		/** Create a new empty image that repreents an external resource image. */
		static UE_API TManagedRef<FImage> CreateAsReference(const TPassthroughObjectPtr<UTexture>& ExternalObject, const FImageDesc& Desc, bool bForceLoad);

		//! Serialisation
		static UE_API void Serialise( const FImage*, FOutputArchive& );
		static UE_API TManagedRef<FImage> StaticUnserialise( FInputArchive& );

		// Resource interface
		UE_API int32 GetDataSize() const override;

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		/** */
		UE_API void Init(uint32 SizeX, uint32 SizeY, uint32 Lods, EImageFormat Format, EInitializationType InitType);

		/** Clear the image to black color. */
		UE_API void InitToBlack();

		/** Return true if this is a reference to an engine image. */
		UE_API bool IsReference() const;

		/** If true, this is a reference that must be resolved at compile time. */
		UE_API bool IsForceLoad() const;
    	
		//! Return the width of the image.
        UE_API uint16 GetSizeX() const;

		//! Return the height of the image.
        UE_API uint16 GetSizeY() const;

		UE_API const FImageSize& GetSize() const;

		//! Return the pixel format of the image.
		UE_API EImageFormat GetFormat() const;

		//! Return the number of levels of detail (mipmaps) in the texture. The base lavel is also
		//! counted, so the minimum is 1.
		UE_API int32 GetLODCount() const;

		//! Return a pointer to a instance-owned buffer where the image pixels are.
        UE_API const uint8* GetLODData(int32 LODIndex) const;
        UE_API uint8* GetLODData(int32 LODIndex);

        //! Return the size in bytes of a specific LOD of the image.
        UE_API int32 GetLODDataSize(int32 LODIndex) const;
		
	public:

		// This used to be the data in the private implementation of the image interface
		//-----------------------------------------------------------------------------------------
		/** These set of flags are used to cache information for images at runtime. */
		typedef enum
		{
			// Set if the next flag has been calculated and its value is valid.
			IF_IS_PLAIN_COLOR_VALID = 1 << 0,

			// If the previous flag is set and this one too, the image is single color.
			IF_IS_PLAIN_COLOR = 1 << 1,

			// If this is set, the image shouldn't be scaled: it's contents is resolution-dependent.
			IF_CANNOT_BE_SCALED = 1 << 2,

			// If this is set, the image has an updated relevancy map. This flag is not persistent.
			IF_HAS_RELEVANCY_MAP = 1 << 3,
			
			/** For reference images, this indicates that they should be loaded into full images as soon as they are generated. */
			IF_IS_FORCELOAD = 1 << 5
		} EImageFlags;

		/** Persistent flags with some image properties. The meaning will depend of every context. */
		mutable uint8 Flags = 0;

		/** Non-persistent relevancy map. */
		uint16 RelevancyMinY = 0;
		uint16 RelevancyMaxY = 0;

		/** Passthrough Id or TStrongObjectPtr. */
    	TPassthroughObjectPtr<UTexture> PassthroughObject;

    	
		/** Pixel data for all lods. */
		FImageDataStorage DataStorage;

		// This used to be the methods in the private implementation of the image interface
		//-----------------------------------------------------------------------------------------

		//! Deep clone.
		TManagedPtr<FImage> Clone() const
		{
			LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
			MUTABLE_CPUPROFILER_SCOPE(ImageClone)

			TManagedPtr<FImage> Result = MakeManaged<FImage>();

			Result->Flags = Flags;
			Result->RelevancyMinY = RelevancyMinY;
			Result->RelevancyMaxY = RelevancyMaxY;
			Result->DataStorage = DataStorage;
			Result->PassthroughObject = PassthroughObject;

			return Result;
		}

		//! Copy another image.
		void Copy(const FImage* Other)
		{
			LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
			MUTABLE_CPUPROFILER_SCOPE(Copy);

			if (Other == this)
			{
				return;
			}

			Flags = Other->Flags;
			RelevancyMinY = Other->RelevancyMinY;
			RelevancyMaxY = Other->RelevancyMaxY;
			DataStorage = Other->DataStorage;
			PassthroughObject = Other->PassthroughObject;
		}

		void CopyMove(FImage* Other)
		{
			LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
			MUTABLE_CPUPROFILER_SCOPE(CopyMove);

			if (Other == this)
			{
				return;
			}

			Flags = Other->Flags;
			RelevancyMinY = Other->RelevancyMinY;
			RelevancyMaxY = Other->RelevancyMaxY;
			PassthroughObject = Other->PassthroughObject;

			DataStorage = MoveTemp(Other->DataStorage);
		}


		//!
		UE_API void Serialise(FOutputArchive&) const;

		//!
		UE_API void Unserialise(FInputArchive&);

		//!
		inline bool operator==(const FImage& Other) const
		{
			return 
				DataStorage == Other.DataStorage &&
				PassthroughObject == Other.PassthroughObject;
		}

		//-----------------------------------------------------------------------------------------

		//! Sample the image and return an RGB color.
		UE_API FVector4f Sample(FVector2f Coords) const;

		//! Calculate the size of the image data in bytes, regardless of what is allocated in
		//! m_data, only using the image descriptions. For non-block-compressed images, it returns
		//! 0.
		static UE_API int32 CalculateDataSize(int32 SizeX, int32 SizeY, int32 LodCount, EImageFormat Format);

		//! Calculate the size in pixels of a particular mipmap of this image. The size doesn't
		//! include pixels necessary for completing blocks in block-compressed formats.
		UE_API FIntVector2 CalculateMipSize(int32 LOD) const;

		//! Return a pointer to the beginning of the data for a particular mip.
		UE_API uint8* GetMipData(int32 Mip);
		UE_API const uint8* GetMipData(int32 Mip) const;
		
		//! Return the size of the resident mips. for block-compressed images the result is the same as in CalculateDataSize()
		//! for non-block-compressed images, the sizes encoded in the image data is used to compute the final size.
		UE_API int32 GetMipsDataSize() const;

		//! See if all the pixels in the image are equal and return the color.
		UE_API bool IsPlainColor(FVector4f& Color) const;

		//! See if all the pixels in the alpha channel are the max value of the pixel format
		//! (white).
		UE_API bool IsFullAlpha() const;

		//! Reduce the LODs of the image to the given amount. Don't do anything if there are already 
		//! less LODs than specified.
		UE_API void ReduceLODsTo(int32 NewLODCount);
		UE_API void ReduceLODs(int32 LODsToSkip);

		//! Calculate the number of mipmaps for a particular image size.
		static UE_API int32 GetMipmapCount(int32 SizeX, int32 SizeY);

		//! Get the rect inside the image bounding the non-black content of the image.
		UE_API void GetNonBlackRect(FImageRect& OutRect) const;
		UE_API void GetNonBlackRect_Reference(FImageRect& OutRect) const;
	};

	/** This struct contains image operations that may need to allocate and free temporary images to work.
	* It's purpose is to override the creation, release and clone functions depending on the context.
	*/
	struct FImageOperator
	{
		/** Common callback used for functions that can create temporary images. */
		typedef TFunction<TManagedPtr<FImage>(int32, int32, int32, EImageFormat, EInitializationType)> FImageCreateFunc;
		typedef TFunction<TManagedPtr<FImage>(const FImage*)> FImageCloneFunc;

		FImageCreateFunc CreateImage;
		FImageCloneFunc CloneImage;

		/** Interface to override the internal mutable image pixel format conversion functions.
		* Arguments match the FImageOperator::ImagePixelFormat function.
		*/
		typedef TFunction<void(bool&, int32, FImage*, const FImage*, int32)> FImagePixelFormatFunc;
		FImagePixelFormatFunc FormatImageOverride;

		FImageOperator(FImageCreateFunc Create, FImageCloneFunc Clone, FImagePixelFormatFunc FormatOverride) 
			: CreateImage(Create)
			, CloneImage(Clone)
			, FormatImageOverride(FormatOverride) 
		{
		};

		/** Create an default version for untracked resources. */
		static FImageOperator GetDefault( const FImagePixelFormatFunc& InFormatOverride )
		{
			return FImageOperator(
				[](int32 SizeX, int32 SizeY, int32 NumLODs, EImageFormat Format, EInitializationType InitType) { return MakeManaged<FImage>(SizeX, SizeY, NumLODs, Format, InitType); },
				[](const FImage* In) { return In->Clone(); },
				InFormatOverride
			);
		}

		/** Convert an image to another pixel format.
		* Allocates the destination image.
		* \warning Not all format conversions are implemented.
		* \param onlyLOD If different than -1, only the specified lod level will be converted in the returned image.
		* \return nullptr if the conversion failed, usually because not enough memory was allocated in the result. This is only checked for RLE compression.
		*/
		MUTABLERUNTIME_API TManagedPtr<FImage> ImagePixelFormat(int32 Quality, const FImage* Base, EImageFormat TargetFormat, int32 OnlyLOD = -1);

		/** Convert an image to another pixel format.
		* \warning Not all format conversions are implemented.
		* \param onlyLOD If different than -1, only the specified lod level will be converted in the returned image.
		* \return false if the conversion failed, usually because not enough memory was allocated in the result. This is only checked for RLE compression.
		*/
		MUTABLERUNTIME_API void ImagePixelFormat(bool& bOutSuccess, int32 Quality, FImage* Result, const FImage* Base, int32 OnlyLOD = -1);
		MUTABLERUNTIME_API void ImagePixelFormat(bool& bOutSuccess, int32 Quality, FImage* Result, const FImage* Base, int32 BeginResultLOD, int32 BeginBaseLOD, int32 NumLODs);

		MUTABLERUNTIME_API TManagedPtr<FImage> ImageSwizzle(EImageFormat Format, const TManagedPtr<const FImage> Sources[], const uint8 Channels[]);

		/** */
		MUTABLERUNTIME_API TManagedPtr<FImage> ExtractMip(const FImage* From, int32 Mip);

		/** Bilinear filter image resize. */
		MUTABLERUNTIME_API void ImageResizeLinear(FImage* Dest, int32 ImageCompressionQuality, const FImage* Base);

		/** Fill the image with a plain color. */
		void FillColor(FImage* Image, FVector4f Color);

		/** Support struct to keep preallocated data required for some mipmap operations. */
		struct FScratchImageMipmap
		{
			TManagedPtr<FImage> Uncompressed;
			TManagedPtr<FImage> UncompressedMips;
			TManagedPtr<FImage> CompressedMips;
		};

		/** Generate the mipmaps for images.
		* if bGenerateOnlyTail is true, generates the mips missing from Base to LevelCount and sets
		* them in Dest (the full chain is spit in two images). Otherwise generate the mips missing
		* from Base up to LevelCount and append them in Dest to the already generated Base's mips.
		*/
		void ImageMipmap(int32 CompressionQuality, FImage* Dest, const FImage* Base,
			int32 StartLevel, int32 LevelCount,
			const FMipmapGenerationSettings&, bool bGenerateOnlyTail = false);

		/** Mipmap separating the worst case treatment in 3 steps to manage allocations of temp data. */
		void ImageMipmap_PrepareScratch(const FImage* DestImage, int32 StartLevel, int32 LevelCount, FScratchImageMipmap&);
		void ImageMipmap(FImageOperator::FScratchImageMipmap&, int32 CompressionQuality, FImage* Dest, const FImage* Base,
			int32 LevelStart, int32 LevelCount,
			const FMipmapGenerationSettings&, bool bGenerateOnlyTail = false);

		/** */
		MUTABLERUNTIME_API bool ImageCrop(TManagedPtr<FImage>& OutCropped, int32 CompressionQuality, const FImage* Base, const box<FIntVector2>& Rect);

		/** */
		void ImageCompose(FImage* Base, const FImage* Block, const box<FIntVector2>& Rect);

	};

	/** Update all the mipmaps in the image from the data in the base one.
	* Only the mipmaps already existing in the image are updated.
	*/
	MUTABLERUNTIME_API void ImageMipmapInPlace(int32 CompressionQuality, FImage* Base, const FMipmapGenerationSettings&);

	/** */
	MUTABLERUNTIME_API void ImageSwizzle(FImage* Result, const TManagedPtr<const FImage> Sources[], const uint8 Channels[]);

	MUTABLERUNTIME_API inline EImageFormat GetUncompressedFormat(EImageFormat Format)
	{
		check(Format < EImageFormat::Count);

		EImageFormat Result = Format;

		switch (Result)
		{
		case EImageFormat::L_UBitRLE: Result = EImageFormat::L_UByte; break;
		case EImageFormat::L_UByteRLE: Result = EImageFormat::L_UByte; break;
		case EImageFormat::RGB_UByteRLE: Result = EImageFormat::RGB_UByte; break;
        case EImageFormat::RGBA_UByteRLE: Result = EImageFormat::RGBA_UByte; break;
        case EImageFormat::BC1: Result = EImageFormat::RGBA_UByte; break;
        case EImageFormat::BC2: Result = EImageFormat::RGBA_UByte; break;
        case EImageFormat::BC3: Result = EImageFormat::RGBA_UByte; break;
        case EImageFormat::BC4: Result = EImageFormat::L_UByte; break;
        case EImageFormat::BC5: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_4x4_RGB_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_4x4_RGBA_LDR: Result = EImageFormat::RGBA_UByte; break;
		case EImageFormat::ASTC_4x4_RG_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_6x6_RGB_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_6x6_RGBA_LDR: Result = EImageFormat::RGBA_UByte; break;
		case EImageFormat::ASTC_6x6_RG_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_8x8_RGB_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_8x8_RGBA_LDR: Result = EImageFormat::RGBA_UByte; break;
		case EImageFormat::ASTC_8x8_RG_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_10x10_RGB_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_10x10_RGBA_LDR: Result = EImageFormat::RGBA_UByte; break;
		case EImageFormat::ASTC_10x10_RG_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_12x12_RGB_LDR: Result = EImageFormat::RGB_UByte; break;
		case EImageFormat::ASTC_12x12_RGBA_LDR: Result = EImageFormat::RGBA_UByte; break;
		case EImageFormat::ASTC_12x12_RG_LDR: Result = EImageFormat::RGB_UByte; break;
		default: break;
		}

		return Result;
	}

	/** */
	template<class T>
	TManagedPtr<T> CloneOrTakeOver(TManagedPtr<T>&& Source)
	{
		TManagedPtr<T> Result;
		if (Source.IsUniqueReference())
		{
			Result = ConstCastManagedPtr<T>(Source);
		}
		else
		{
			if (Source)
			{
				Result = Source->Clone();
			}
		}

		Source = nullptr;
		return Result;
	}

	/** */
	template<class T>
	TManagedPtr<T> CloneOrTakeOver(TManagedPtr<const T>&& Source)
	{
		TManagedPtr<T> Result;
		if (Source.IsUniqueReference())
		{
			Result = ConstCastManagedPtr<T>(Source);
		}
		else
		{
			if (Source)
			{
				Result = Source->Clone();
			}
		}

		Source = nullptr;
		return Result;
	}

}

#undef UE_API
