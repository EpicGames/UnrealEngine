// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImageTypes.h"
#include "MuR/MemoryTrackingAllocationPolicy.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"

#include <atomic>

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private::MemoryCounters
{
	struct FImageMemoryCounter
	{
		static UE_API std::atomic<SSIZE_T>& Get();
	};
}


namespace UE::Mutable::Private
{
	using FImageArray = TArray<uint8, FDefaultMemoryTrackingAllocator<MemoryCounters::FImageMemoryCounter>>;
}

namespace UE::Mutable::Private::FImageDataStorageInternal
{
	inline int32 ComputeNumLODsForSize(FImageSize Size)
	{
		return static_cast<int32>(FMath::CeilLogTwo(FMath::Max(Size.X, Size.Y))) + 1;
	}
}

namespace UE::Mutable::Private
{
class FImageDataStorage
{
public:
	TArray<FImageArray, TInlineAllocator<1>> Buffers;
	
	FImageSize ImageSize = FImageSize(0, 0);
	EImageFormat ImageFormat = EImageFormat::None;
	uint8 NumLODs = 0;

	static constexpr int32 NumLODsInCompactedTail = 7;
	// This is needed for images its size cannot be known from dimensions and format, e.g., RLE compressed formats.
	// It stores the offset to the end of the LOD so that CompactedTailOffsets[LOD] - CompactedTailOffsets[LOD - 1]
	// is the size of LOD.
	TStaticArray<uint16, NumLODsInCompactedTail> CompactedTailOffsets = MakeUniformStaticArray<uint16, NumLODsInCompactedTail>(0);

public:

	MUTABLERUNTIME_API FImageDataStorage();
	MUTABLERUNTIME_API FImageDataStorage(const FImageDesc& Desc);

	MUTABLERUNTIME_API FImageDataStorage(const FImageDataStorage& Other);
	MUTABLERUNTIME_API FImageDataStorage& operator=(const FImageDataStorage& Other);
	
	FImageDataStorage(FImageDataStorage&& Other) = default;
	FImageDataStorage& operator=(FImageDataStorage&& Other) = default;

	MUTABLERUNTIME_API bool operator==(const FImageDataStorage& Other) const;

	inline FImageDesc MakeImageDesc() const
	{
		return FImageDesc(ImageSize, ImageFormat, ImageFormat, NumLODs);
	}

	inline void InitVoid(const FImageDesc& Desc) 
	{
		Buffers.Empty();	
		CompactedTailOffsets[0] = TNumericLimits<uint16>::Max();

		ImageSize = Desc.m_size;
		ImageFormat = Desc.m_format;
		NumLODs = Desc.m_lods;
	}

	inline bool IsVoid() const
	{
		return Buffers.IsEmpty() && CompactedTailOffsets[0] == TNumericLimits<uint16>::Max();
	}

	MUTABLERUNTIME_API void InitInternalArray(int32 Index, int32 Size, EInitializationType InitType);
	MUTABLERUNTIME_API void Init(const FImageDesc& ImageDesc, EInitializationType InitType);

	MUTABLERUNTIME_API void CopyFrom(const FImageDataStorage& Other);

	inline int32 GetNumLODs() const
	{
		return NumLODs;
	}

	/**
	* Get a const view to the data containing the LODIndex.
	*/
	MUTABLERUNTIME_API TArrayView<const uint8> GetLOD(int32 LODIndex) const;

	/**
	* Get a view to the data containing the LODIndex.
	*/
	MUTABLERUNTIME_API TArrayView<uint8> GetLOD(int32 LODIndex);

	/**
	 * Change the number of lods in the image. If InNumLODs is smaller than the current lod count, 
	 * LODs are dropped from the tail up.
	 */
	MUTABLERUNTIME_API void SetNumLODs(int32 InNumLODs, EInitializationType InitType = EInitializationType::NotInitialized);

	/**
	 * Remove NumLODsToDrop starting form LOD 0. The resulting image will be smaller. 
	 */
	MUTABLERUNTIME_API void DropLODs(int32 NumLODsToDrop);

	/**
	 * Resizes LODIndex Data to NewSizeBytes
	 */
	MUTABLERUNTIME_API void ResizeLOD(int32 LODIndex, int32 NewSizeBytes, EAllowShrinking AllowShrinking = EAllowShrinking::Default);

	inline const FImageArray& GetInternalArray(int32 LODIndex) const
	{	
		check(!IsVoid());
		check(LODIndex < NumLODs);
		return Buffers[FMath::Min(LODIndex, ComputeFirstCompactedTailLOD())];
	}

	inline FImageArray& GetInternalArray(int32 LODIndex)
	{	
		check(!IsVoid());
		check(LODIndex < NumLODs);
		return Buffers[FMath::Min(LODIndex, ComputeFirstCompactedTailLOD())];
	}

	inline int32 ComputeFirstCompactedTailLOD() const
	{
		using namespace FImageDataStorageInternal;
		return FMath::Max(0, ComputeNumLODsForSize(ImageSize) - NumLODsInCompactedTail); 
	}

	/**
	 * Stateless iteration in batches.
	 */
	MUTABLERUNTIME_API int32 GetNumBatches(int32 BatchSizeInElems, int32 BatchElemSizeInBytes) const;
	MUTABLERUNTIME_API TArrayView<const uint8> GetBatch(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes) const;
	MUTABLERUNTIME_API TArrayView<uint8> GetBatch(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes);

	/**
	 * Returns the number of batches GetBatchFirtsLODOffet will admit and that will cover
	 * the first LOD buffer with OffsetInBytes removed from the front. 
	 * 
	 * A batch cannot be larger than BatchElemsSize. The last batch of a buffer will be smaller if 
	 * BatchSizeInElems*BatchElemSizeInBytes is not multiple of the buffer size.
	 *
	 **/
	MUTABLERUNTIME_API int32 GetNumBatchesFirstLODOffset(int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 OffsetInBytes) const;

	/**
	 * Returns a non modifiable view to the portion of the first LOD buffer with OffsetInBytes for the BatchId 
	 *
	 * A batch cannot be larger than BatchElemsSize. The last batch of a buffer will be smaller if 
	 * BatchSizeInElems*BatchElemSizeInBytes is not multiple of the buffer size.
	 *
	 */
	MUTABLERUNTIME_API TArrayView<const uint8> GetBatchFirstLODOffset(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 OffsetInBytes) const;

	/**
	 * Non const version of GetBatchFirstLODOffset(). 
	 */
	MUTABLERUNTIME_API TArrayView<uint8> GetBatchFirstLODOffset(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 OffsetInBytes = 0);

	/**
	 * Returns the number of batches GetBatchLODRange will admit and that will cover
	 * the [LODBegin, LODEnd) range.
	 *
	 **/
	MUTABLERUNTIME_API int32 GetNumBatchesLODRange(int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 LODBegin, int32 LODEnd) const;

	/**
	 * Returns a non modifiable view to the portion of the [LODBegin, LODEnd) for the BatchId. 
	 *
	 * A batch cannot be larger than BatchElemsSize. The last batch of a buffer will be smaller if 
	 * BatchSizeInElems*BatchElemSizeInBytes is not multiple of the buffer size.
	 *
	 */
	MUTABLERUNTIME_API TArrayView<const uint8> GetBatchLODRange(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 LODBegin, int32 LODEnd) const;
	
	/**
	 * Non const version of GetBatchLODRange().
	 */
	MUTABLERUNTIME_API TArrayView<uint8> GetBatchLODRange(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 LODBegin, int32 LODEnd);

	MUTABLERUNTIME_API int32 GetAllocatedSize() const;
	MUTABLERUNTIME_API int32 GetDataSize() const;

	/**
	 * Returns true if all buffers are empty. This can be true after initialization for image formats that report 
	 * BytesPerBlock 0, e.g., RLE formats. If not initialized will also return true.
	 */
	MUTABLERUNTIME_API bool IsEmpty() const;

	MUTABLERUNTIME_API void Serialise(FOutputArchive& Arch) const;
	MUTABLERUNTIME_API void Unserialise(FInputArchive& Arch);
};

}

#undef UE_API
