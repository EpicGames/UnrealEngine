// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/ImageTypes.h"

namespace UE::Mutable::Private
{
	inline EImageFormat GetMostGenericFormat(EImageFormat FormatA, EImageFormat FormatB)
    {
		if (FormatA == FormatB)
		{
			return FormatA;
		}

		if (FormatA == EImageFormat::None)
		{
			return FormatA;
		}

		if (FormatB == EImageFormat::None)
		{
			return FormatB;
		}

        if (GetImageFormatData(FormatA).Channels > GetImageFormatData(FormatB).Channels)
		{	
			return FormatA;
		}

        if (GetImageFormatData(FormatB).Channels > GetImageFormatData(FormatA).Channels)
		{
			return FormatB;
		}

        if (FormatA == EImageFormat::BC2 || FormatA == EImageFormat::BC3 
			|| FormatA == EImageFormat::ASTC_4x4_RGBA_LDR || FormatA == EImageFormat::ASTC_6x6_RGBA_LDR || FormatA == EImageFormat::ASTC_8x8_RGBA_LDR || FormatA == EImageFormat::ASTC_10x10_RGBA_LDR)
		{	
			return FormatA;
		}

        if (FormatB == EImageFormat::BC2 || FormatB == EImageFormat::BC3
			|| FormatB == EImageFormat::ASTC_4x4_RGBA_LDR || FormatB == EImageFormat::ASTC_6x6_RGBA_LDR || FormatB == EImageFormat::ASTC_8x8_RGBA_LDR || FormatB == EImageFormat::ASTC_10x10_RGBA_LDR)

		{
			return FormatB;
		}

        return FormatA;
    }

	inline EImageFormat GetRGBOrRGBAFormat(EImageFormat InFormat)
    {
		InFormat = GetUncompressedFormat(InFormat);

		if (InFormat == EImageFormat::None)
		{
			return InFormat;
		}

		switch (InFormat)
		{
		case EImageFormat::L_UByte: 
		{
			return EImageFormat::RGB_UByte;
		}
		case EImageFormat::RGB_UByte:
		case EImageFormat::RGBA_UByte:
		case EImageFormat::BGRA_UByte:
		{
			return InFormat;
		}
		default:
		{
			unimplemented();
		}
		}

		return EImageFormat::None;
    }

    //---------------------------------------------------------------------------------------------
    //!
    //---------------------------------------------------------------------------------------------
    inline bool IsCompressedFormat(EImageFormat Format)
    {
        return Format != GetUncompressedFormat(Format);
    }


	inline bool IsBlockCompressedFormat(EImageFormat Format)
	{
		return GetImageFormatData(Format).PixelsPerBlockX > 1 && Format != GetUncompressedFormat(Format);
	}
}

