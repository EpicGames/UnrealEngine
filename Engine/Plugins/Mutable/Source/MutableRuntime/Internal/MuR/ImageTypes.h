// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MutableMath.h"
#include "MuR/Serialisation.h"

#include <initializer_list>

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private
{
	//!
	using FImageSize = UE::Math::TIntVector2<uint16>;
	using FImageRect = box<FImageSize>;

	//! Pixel formats supported by the images.
	//! \ingroup runtime
	enum class EImageFormat : uint8
	{
		None,
		RGB_UByte,
		RGBA_UByte,
		L_UByte,

        //! Deprecated formats
        _DEPRECATED_1,
        _DEPRECATED_2,
        _DEPRECATED_3,
        _DEPRECATED_4,

		L_UByteRLE,
		RGB_UByteRLE,
		RGBA_UByteRLE,
		L_UBitRLE,

        //! Common S3TC formats
        BC1,
        BC2,
        BC3,
        BC4,
        BC5,

        //! Not really supported yet
        BC6,
        BC7,

        //! Swizzled versions, engineers be damned.
        BGRA_UByte,

        //! The new standard
        ASTC_4x4_RGB_LDR,
        ASTC_4x4_RGBA_LDR,
        ASTC_4x4_RG_LDR,

		ASTC_8x8_RGB_LDR,
		ASTC_8x8_RGBA_LDR,
		ASTC_8x8_RG_LDR,
		ASTC_12x12_RGB_LDR,
		ASTC_12x12_RGBA_LDR,
		ASTC_12x12_RG_LDR,
		ASTC_6x6_RGB_LDR,
		ASTC_6x6_RGBA_LDR,
		ASTC_6x6_RG_LDR,
		ASTC_10x10_RGB_LDR,
		ASTC_10x10_RGBA_LDR,
		ASTC_10x10_RG_LDR,

        Count
	};
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EImageFormat);

	inline uint32 GetTypeHash(const EImageFormat& Format)
	{
		return static_cast<uint32>(Format);
	}

	struct FImageDesc
	{
		FImageDesc()
		{
		}

		FImageDesc(const FImageSize& InSize, EImageFormat InFormat, EImageFormat InFormatIfAlpha, uint8 InLods)
			: m_size(InSize)
            , m_format(InFormat) 
			, FormatIfAlpha(InFormatIfAlpha) 
			, m_lods(InLods)
		{
		}

		/** */
		FImageSize m_size = FImageSize(0, 0);

		/** */
		EImageFormat m_format = EImageFormat::None;

		/** */
		EImageFormat FormatIfAlpha = EImageFormat::None;

		/** Levels of detail (mipmaps) */
		uint8 m_lods = 0;

		/** */
		inline bool operator==(const FImageDesc& Other) const
		{
			return
				m_size == Other.m_size && 
				m_format == Other.m_format && 
				FormatIfAlpha == Other.FormatIfAlpha &&
				m_lods == Other.m_lods;
		}

		friend inline uint32 GetTypeHash(const FImageDesc& Desc)
		{
			uint32 Result = GetTypeHash(Desc.m_size);
			Result = HashCombineFast(Result, GetTypeHash(Desc.m_format));
			Result = HashCombineFast(Result, GetTypeHash(Desc.FormatIfAlpha));
			Result = HashCombineFast(Result, GetTypeHash(Desc.m_lods));

			return Result;
		}
	};

	/** An image descriptor with information about missing data LODs and which images are needed to load to generate the image */
	struct FExtendedImageDesc : public FImageDesc
	{
		/** 
		 * Number of LODs that cannot be generated because the data is not available.
		 * It is assumed that non available LODs are consecutive starting from the mip with higher resolution.
		 **/
		uint8 FirstLODAvailable = 0;
		TArray<int32> ConstantImagesNeededToGenerate;
	};

	/** List of supported modes in generic image layering operations. */
	enum class EBlendType
	{
		BT_NONE = 0,
		BT_SOFTLIGHT,
		BT_HARDLIGHT,
		BT_BURN,
		BT_DODGE,
		BT_SCREEN,
		BT_OVERLAY,
		BT_BLEND,
		BT_MULTIPLY,
		BT_LIGHTEN,				// Increase the channel value by a given proportion of what is missing from white 
		BT_NORMAL_COMBINE,
		_BT_COUNT
	};	
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EBlendType);

	enum class EAddressMode
	{
		None,
		Wrap,
		ClampToEdge,
		ClampToBlack,
	};
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EAddressMode);
	
	inline uint32 GetTypeHash(const EAddressMode& Mode)
	{
		return static_cast<uint32>(Mode);
	}
	
	enum class EMipmapFilterType
	{
		Unfiltered,
		SimpleAverage,
	};
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EMipmapFilterType);

	enum class ECompositeImageMode
	{
		CIM_Disabled,
		CIM_NormalRoughnessToRed,
		CIM_NormalRoughnessToGreen,
		CIM_NormalRoughnessToBlue,
		CIM_NormalRoughnessToAlpha,
		_CIM_COUNT
	};	
	MUTABLE_DEFINE_ENUM_SERIALISABLE(ECompositeImageMode);

	enum class ESamplingMethod : uint8
	{
		Point = 0,
		BiLinear,
		MaxValue
	};
	static_assert(uint32(ESamplingMethod::MaxValue) <= (1 << 3), "ESampligMethod enum cannot hold more than 8 values");
	MUTABLE_DEFINE_ENUM_SERIALISABLE(ESamplingMethod);

	enum class EInitializationType
	{
		NotInitialized,
		Black
	};

	enum class EMinFilterMethod : uint8
	{
		None = 0,
		TotalAreaHeuristic,
		MaxValue
	};
	static_assert(uint32(EMinFilterMethod::MaxValue) <= (1 << 3), "EMinFilterMethod enum cannot hold more than 8 values");
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EMinFilterMethod);

	/** */
	struct FImageFormatData
	{
		static constexpr SIZE_T MAX_BYTES_PER_BLOCK = 16;

		FImageFormatData
		(
			uint8 InPixelsPerBlockX = 0,
			uint8 InPixelsPerBlockY = 0,
			uint16 InBytesPerBlock = 0,
			uint16 InChannels = 0
		)
		{
			PixelsPerBlockX = InPixelsPerBlockX;
			PixelsPerBlockY = InPixelsPerBlockY;
			BytesPerBlock = InBytesPerBlock;
			Channels = InChannels;
		}

		FImageFormatData
		(
			uint8 InPixelsPerBlockX,
			uint8 InPixelsPerBlockY,
			uint16 InBytesPerBlock,
			uint16 InChannels,
			std::initializer_list<uint8> BlackBlockInit
		)
			: FImageFormatData(InPixelsPerBlockX, InPixelsPerBlockY, InBytesPerBlock, InChannels)
		{
			check(MAX_BYTES_PER_BLOCK >= BlackBlockInit.size());

			const SIZE_T SanitizedBlockSize = FMath::Min<SIZE_T>(MAX_BYTES_PER_BLOCK, BlackBlockInit.size());
			FMemory::Memcpy(BlackBlock, BlackBlockInit.begin(), SanitizedBlockSize);
		}

		/** For block based formats, size of the block size.For uncompressed formats it will always be 1,1. For non-block-based compressed formats, it will be 0,0. */
		uint8 PixelsPerBlockX, PixelsPerBlockY;

		/** Number of bytes used by every pixel block, if uncompressed or block-compressed format.
		 * For non-block-compressed formats, it returns 0.
		 */
		uint16 BytesPerBlock;

		/** Channels in every pixel of the image. */
		uint16 Channels;

		/** Representation of a black block of the image. */
		uint8 BlackBlock[MAX_BYTES_PER_BLOCK] = { 0 };
	};

	MUTABLERUNTIME_API const FImageFormatData& GetImageFormatData(EImageFormat format);

	struct FMipmapGenerationSettings
	{
		EMipmapFilterType FilterType = EMipmapFilterType::SimpleAverage;
		EAddressMode AddressMode = EAddressMode::None;

		UE_API void Serialise(FOutputArchive& Arch) const;
		UE_API void Unserialise(FInputArchive& Arch);
	};
}

#undef UE_API
