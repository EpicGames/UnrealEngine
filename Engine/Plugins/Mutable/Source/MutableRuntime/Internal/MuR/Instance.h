// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ManagedPointer.h"
#include "MuR/ResourceID.h"
#include "HAL/PlatformMath.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/Material.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
	class FExtensionData;

	struct FOverrideMaterial
	{
		FName SlotName;
		FMaterialId MaterialId;
	};
	
	struct FOverlayMaterial
	{
		FName SlotName;
		FMaterialId MaterialId;
	};

	struct FInstanceComponent
	{
		FComponentId Id = INDEX_NONE;

		FMaterialId OverlayMaterialId;

		TArray<FOverrideMaterial> OverrideMaterials;

		TArray<FOverlayMaterial> OverlayMaterials;
		
		FSkeletalMeshId SkeletalMeshId;
	};
	
	
    /** A customised object created from a model and a set of parameter values.
    * It corresponds to an "engine object" but the contents of its data depends on the Model, and
    * it may contain any number of LODs, components, surfaces, meshes and images, even none.
	*/
	class FInstance : public FResource
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------
		
        //! Clone this instance
		UE_API UE::Mutable::Private::TManagedPtr<FInstance> Clone() const;

		// Resource interface
		UE_API int32 GetDataSize() const override;

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------
		
		//! Get the number of components of this instance.
		UE_API int32 GetComponentCount() const;

		//! Get the Id of a component
		//! \param ComponentIndex Index of the component, from 0 to GetComponentCount()-1
		UE_API FComponentId GetComponentId(int32 ComponentIndex) const;
		
		UE_API FSkeletalMeshId GetSkeletalMeshId(int32 ComponentIndex) const;
		
		//! Get the index of the overlay material.
		//! \param ComponentIndex Index of the component, from 0 to GetComponentCount()-1
		UE_API FMaterialId GetOverlayMaterialId(int32 ComponentIndex) const;

		UE_API int32 GetOverrideMaterialCount(int32 ComponentIndex) const;

		UE_API FName GetOverrideMaterialSlotSlotName(int32 ComponentIndex, int32 MaterialIndex) const;

		UE_API FMaterialId GetOverrideMaterialId(int32 ComponentIndex, int32 MaterialIndex) const;
		
		UE_API int32 GetOverlayMaterialCount(int32 ComponentIndex) const;

		UE_API FName GetOverlayMaterialSlotSlotName(int32 ComponentIndex, int32 MaterialIndex) const;

		UE_API FMaterialId GetOverlayMaterialId(int32 ComponentIndex, int32 MaterialIndex) const;
		
		//! Get the number of ExtensionData values in a component
		UE_API int32 GetExtensionDataCount() const;
		
		int32 AddComponent();

		void SetSkeletalMeshId(int32 ComponentIndex, const FSkeletalMeshId& SkeletalMeshId);

		void SetOverlayMaterialId(int32 ComponentIndex, const FMaterialId& MaterialId);
		
		void AddOverrideMaterial(int32 ComponentIndex, const FName& SlotName, const FMaterialId& MaterialId);

		void AddOverlayMaterial(int32 ComponentIndex, const FName& SlotName, const FMaterialId& MaterialId);
		
		TArray<FInstanceComponent, TInlineAllocator<1>> Components;

		// Every entry must have a valid ExtensionData and name
		TArray<TManagedPtr<const FExtensionData>> ExtensionData;
	};


}

#undef UE_API
