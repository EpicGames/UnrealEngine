// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RefCounted.h"
#include "ManagedPointer.h"


namespace UE::Mutable::Private
{
	class FMesh;

	class FLOD : public FResource
	{
	public:
		// FResource interface
		virtual int32 GetDataSize() const override { return 0; }; 
		
		TManagedPtr<FLOD> Clone() const
		{
			TManagedPtr<FLOD> New = MakeManaged<FLOD>();
			New->Meshes = Meshes;

			return New;
		}

		TArray<TManagedPtr<const FMesh>> Meshes;
	};
}

