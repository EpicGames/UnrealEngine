// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "Containers/Array.h"
#include "Misc/AssertionMacros.h"
#include "Math/IntVector.h"
#include "Math/NumericLimits.h"

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private
{
	//! Types of automatic block strategies 
	enum class EAutoBlocksStrategy : uint32
	{
		Rectangles,
		UVIslands,
		Ignore,
	};

	//! Types of layout packing strategies 
	enum class EPackStrategy : uint32
	{
		Resizeable,
		Fixed,
		Overlay
	};

	//! Types of layout reduction methods 
	enum class EReductionMethod : uint32
	{
		Halve,	// Divide axis by 2
		Unitary	// Reduces 1 block the axis 
	};

	/** */
	struct FLayoutBlock
	{
		static constexpr uint64 InvalidBlockId = TNumericLimits<uint64>::Max();
		static constexpr uint64 InvalidMaskIndex = TNumericLimits<uint16>::Max();

		FIntVector2 Min = { 0, 0 };
		FIntVector2 Size = { 0, 0 };

		//! Absolute id used to control merging of various layouts
		uint64 Id;

		//! Priority value to control the shrink texture layout strategy
		int32 Priority;

		//! Value to control the method to reduce the block
		uint16 bReduceBothAxes : 1;

		//! Value to control if a block has to be reduced by two in an unitary reduction strategy
		uint16 bReduceByTwo : 1;

		/** Explicit padding to prevent uninitialized memory in this POD. */
		uint16 UnusedPadding : 14;

		/** Index of the block mask used to classify the vertices and decide if they belong to the block.
		* The image is stored in the Masks array of the FLayout object. 
		* It is interpreted as covering the space of the entire layout grid.
		* If the mask is null, all the vertices inside the block rect belong to the block.
		* The image may be shared among several blocks and also among several layouts.
		*/
		uint16 MaskIndex = InvalidMaskIndex;

		/** */
		FLayoutBlock(FIntVector2 InMin = {}, FIntVector2 InSize = {})
		{
			Min = InMin;
			Size = InSize;
			Id = InvalidBlockId;
			Priority = 0;
			bReduceBothAxes = false;
			bReduceByTwo = false;
			UnusedPadding = 0;
		}

		//!
		inline bool operator==(const FLayoutBlock& o) const
		{
			return (Id == o.Id) && IsSimilar(o);
		}

		inline bool IsSimilar(const FLayoutBlock& o) const
		{
			// All but ids
			return (Min == o.Min) &&
				(Size == o.Size) &&
				(Priority == o.Priority) &&
				(bReduceBothAxes == o.bReduceBothAxes) &&
				(bReduceByTwo == o.bReduceByTwo) &&
				(MaskIndex==o.MaskIndex);
		}
	};


    /** Image block layout class.
    * It contains the information about what blocks are defined in a texture layout (texture
    * coordinates set from a mesh).
    * It is usually not necessary to use this objects, except for some advanced cases.
	*/
    class FLayout : public FResource
	{
	public:

		//!
		FIntVector2 Size = FIntVector2(0, 0);

		/** Maximum size in layout blocks that this layout can grow to. From there on, blocks will shrink to fit. 
		* If 0,0 then no maximum size applies.
		*/
		FIntVector2 MaxSize = FIntVector2(0, 0);

		/** */
		EPackStrategy Strategy = EPackStrategy::Resizeable;

		/** */
		EReductionMethod ReductionMethod = EReductionMethod::Halve;

		//!
		TArray<FLayoutBlock> Blocks;

		/** List of masks that may be used by the blocks for finer-grade selection of vertices. 
		* The masks may be shared by multiple blocks and also across layouts. 
		*/
		TArray<TManagedPtr<const FImage>> Masks;

	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------

		//! Deep clone this layout.
		UE_API TManagedPtr<FLayout> Clone() const;

		//! Serialisation
		static UE_API void Serialise( const FLayout* p, FOutputArchive& arch );
		static UE_API TManagedRef<FLayout> StaticUnserialise( FInputArchive& arch );

		//! Full compare
		UE_API bool operator==( const FLayout& other ) const;

		// Resource interface
		UE_API int32 GetDataSize() const override;

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		//! Get the resolution of the grid where the blocks are defined.
		UE_API FIntPoint GetGridSize() const;

		//! Get the resolution of the grid where the blocks are defined. It must be bigger than 0
		//! on each axis.
		//! \param sizeX width of the grid.
		//! \param sizeY height of the grid.
		UE_API void SetGridSize( int32 SizeX, int32 SizeY );

		//! Get the maximum resolution of the grid where the blocks are defined.
		//! \param[out] pSizeX The integer pointed by this will be set to the width of the grid.
		//! \param[out] pSizeY The integer pointed by this will be set to the height of the grid.
		UE_API void GetMaxGridSize(int32* SizeX, int32* SizeY) const;

		//! Get the maximum resolution of the grid where the blocks are defined. It must be bigger than 0
		//! on each axis.
		//! \param sizeX width of the grid.
		//! \param sizeY height of the grid.
		UE_API void SetMaxGridSize(int32 SizeX, int32 SizeY);

		//! Return the number of blocks in this layout.
		UE_API int32 GetBlockCount() const;

		//! Set the number of blocks in this layout.
		//! The existing blocks will be kept as much as possible. The new blocks will be undefined.
		UE_API void SetBlockCount( int32 );

		//! Set the texture layout packing strategy
		//! By default the texture layout packing strategy is set to resizable layout
		UE_API void SetLayoutPackingStrategy(EPackStrategy);

		//! Set the texture layout packing strategy
		UE_API EPackStrategy GetLayoutPackingStrategy() const;


	public:


		//!
		UE_API void Serialise(FOutputArchive& arch) const;

		//!
		UE_API void Unserialise(FInputArchive& arch);

		//!
		UE_API bool IsSimilar(const FLayout& o) const;

		/** Find a block by id. This converts the "absolute" id to a relative index to the layout blocks. Return -1 if not found. */
		UE_API int32 FindBlock(uint64 Id) const;

		//! Return true if the layout is a single block filling all area.
		UE_API bool IsSingleBlockAndFull() const;
	};

	MUTABLE_DEFINE_POD_SERIALISABLE(FLayoutBlock);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FLayoutBlock);

}

#undef UE_API
