// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/MemoryOps.h"
#include "Misc/UEOps.h"
#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"


namespace UE::Mutable::Private
{


/**
 * Mutable managed pointer, implementation is closely based on the engine TSharedPtr.
 * See SharedPointer.h and SharedPointerInteranls.h for details.
 *
 * The motivation for using our own smart pointer is to provide custom behavior that 
 * simplifies Mutable memory management.
 *
 * One of the main difference is how weak references are managed. In this case weak 
 * references are counted for automatic memory release, e.i, objects with only weak 
 * references will not be automatically destroyed, but an external system could try 
 * to delete the object through a strong reference. This operation will succeed only 
 * if the object is referenced by the strong reference trying the destruction ignoring
 * any weak reference. 
 * 
 * If all references, strong and weak, are removed, the referenced object will be deleted 
 * automatically.
 *
 * Similarly to SharedPointer weak references, our weak references can point to an already 
 * destroyed object and so pinning them to a strong reference is needed to access the object.
 */

namespace ManagedPointerInternal
{

	struct FStaticCastTag {};
	struct FConstCastTag {};
	struct FNullTag {};
	struct FNullWithControllerTag {};

	enum class EReferenceControllerFlags : uint8
	{
		None    = 0,
		Deleted = 1 << 0,
		//Locked  = 1 << 1,
		Null    = 1 << 2,
	};

	ENUM_CLASS_FLAGS(EReferenceControllerFlags);

	class FReferenceControllerBase
	{
		// For now a FMutex is used to guarantee thread safety.
		// All operations should lock for a very small amount of cycles an so FMutex is very unlikely 
		// to yield the thread. If this becomes problematic, an atomic state could be used. 
		mutable UE::FMutex Lock;
		
		EReferenceControllerFlags Flags = EReferenceControllerFlags::None;

		// Always start with one strong reference, this is the reference added at Object creation.
		int32 StrongRefCount {1};
		int32 WeakRefCount   {0};
		
		public:
		
		virtual ~FReferenceControllerBase()
		{
		}

		virtual void DeleteObject() = 0;
		
		inline bool IsUniqueReference() const
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			return StrongRefCount + WeakRefCount == 1;
		}

		inline bool IsUniqueStrong() const
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			return StrongRefCount == 1;
		}

		inline bool IsUniqueWeak() const
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			return WeakRefCount == 1 && StrongRefCount == 0;
		}

		inline void AddStrongReference()
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			++StrongRefCount;
		}

		bool ConditionallyAddStrongReference()
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);
			
			if (EnumHasAnyFlags(Flags, EReferenceControllerFlags::Deleted))
			{
				return false;
			}
		
			++StrongRefCount;
			return true;
		}

		int32 GetStrongReferenceCount()
		{
			int32 Result = 0;
			
			{
				UE::TScopeLock<UE::FMutex> LockGuard(Lock);
				Result = StrongRefCount;
			}

			return Result;
		}

		inline void ReleaseStrongReference()
		{
			bool bDeleteObject = false;
			bool bDeleteThisController = false;
			{
				UE::TScopeLock<UE::FMutex> LockGuard(Lock);
				--StrongRefCount;

				if (StrongRefCount + WeakRefCount == 0)
				{
					bDeleteThisController = true;
					bDeleteObject = !EnumHasAnyFlags(Flags, EReferenceControllerFlags::Deleted);
					EnumAddFlags(Flags, EReferenceControllerFlags::Deleted);
				}
			}

			if (bDeleteObject)
			{
				DeleteObject();
			}

			if (bDeleteThisController)
			{
				delete this;
			}
		}

		inline void AddWeakReference()
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			++WeakRefCount;
		}

		inline void ReleaseWeakReference()
		{
			bool bDeleteObject = false;
			bool bDeleteThisController = false;
			{
				UE::TScopeLock<UE::FMutex> LockGuard(Lock);
				--WeakRefCount;

				if (StrongRefCount + WeakRefCount == 0)
				{
					bDeleteObject = !EnumHasAnyFlags(Flags, EReferenceControllerFlags::Deleted);
					bDeleteThisController = true;
					EnumAddFlags(Flags, EReferenceControllerFlags::Deleted);
				}
			}

			if (bDeleteObject)
			{
				DeleteObject();
			}

			if (bDeleteThisController)
			{
				delete this;
			}
		}

		inline bool TryDeleteObject()
		{
			bool bDeleteObject = false;
			{
				UE::TScopeLock<UE::FMutex> LockGuard(Lock);

				if (StrongRefCount == 1)
				{
					checkSlow(!EnumHasAnyFlags(Flags, EReferenceControllerFlags::Deleted));
				
					bDeleteObject = true;
					EnumAddFlags(Flags, EReferenceControllerFlags::Deleted);
				}
			}

			if (bDeleteObject)
			{
				DeleteObject();
			}

			return bDeleteObject;
		}

		inline bool IsDeleted() const
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			return EnumHasAnyFlags(Flags, EReferenceControllerFlags::Deleted);
		}

		inline void SetNull()
		{

			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			return EnumAddFlags(Flags, EReferenceControllerFlags::Null);
		}

		inline bool IsNull() const
		{
			UE::TScopeLock<UE::FMutex> LockGuard(Lock);

			return EnumHasAnyFlags(Flags, EReferenceControllerFlags::Null);
		}
	};

	template<typename ObjectType>
	class TIntrusiveReferenceController : public FReferenceControllerBase
	{
		mutable TTypeCompatibleBytes<ObjectType> ObjectStorage;

	public:

		template<typename... ArgTypes>
		explicit TIntrusiveReferenceController(ArgTypes&&... Args)
		{	
			::new ((void*)&ObjectStorage) ObjectType(Forward<ArgTypes>(Args)...);
		}
		
		virtual void DeleteObject() override
		{
			DestructItem((ObjectType*)&ObjectStorage);
		}

		ObjectType* GetObjectPtr() const
		{
			return (ObjectType*)&ObjectStorage;
		}

		// Non-copyable
		TIntrusiveReferenceController(const TIntrusiveReferenceController&) = delete;
		TIntrusiveReferenceController& operator=(const TIntrusiveReferenceController&) = delete;
	};

	template<typename ObjectType>
	class TNullReferenceController :  public FReferenceControllerBase
	{
	public:

		explicit TNullReferenceController()
		{
			SetNull();
		}
		
		virtual void DeleteObject() override
		{
		}

		ObjectType* GetObjectPtr() const
		{
			return nullptr;
		}
	};

	template<typename ObjectType, typename... ArgsTypes>
	TIntrusiveReferenceController<ObjectType>* NewIntrusiveReferenceController(ArgsTypes&&... Args)
	{
		return new TIntrusiveReferenceController<ObjectType>(Forward<ArgsTypes>(Args)...);
	}

	template<typename ObjectType>
	TNullReferenceController<ObjectType>* NewNullReferenceController()
	{
		return new TNullReferenceController<ObjectType>();
	}


	class FWeakReferencer; 

	/**
	 * FStrongReferencer is a wrapper around a pointer to a reference controller that is used by either a
	 * TManagedRef or a TManagedPtr to keep track of a referenced object's lifetime
	 */

	class FStrongReferencer 
	{
	public:

		/** Constructor for an empty strong referencer object */
		inline FStrongReferencer();

		/** Constructor that counts a single reference to the specified object */
		inline explicit FStrongReferencer(FReferenceControllerBase* InReferenceController);
		
		/** Copy constructor creates a new reference to the existing object */
		inline FStrongReferencer(const FStrongReferencer& Other);

		/** Move constructor creates no new references */
		inline FStrongReferencer(FStrongReferencer&& Other);

		/** 
         * Creates a strong referencer object from a weak referencer object.  This will only result
		 * in a valid object reference if the object already has at least one other strong referencer. 
         */
		inline FStrongReferencer(const FWeakReferencer& OtherWeak);

		/** 
         * Creates a strong referencer object from a weak referencer object.  This will only result
         * in a valid object reference if the object already has at least one other strong referencer.
         */
		inline FStrongReferencer(FWeakReferencer&& OtherWeak);	

		/** Destructor. */
		inline ~FStrongReferencer();

		/** Assignment operator adds a reference to the assigned object.  If this counter was previously
		    referencing an object, that reference will be released. */
		inline FStrongReferencer& operator=(const FStrongReferencer& Other);
		

		/** Move assignment operator adds no references to the assigned object.  If this counter was previously
		    referencing an object, that reference will be released. */
		inline FStrongReferencer& operator=(FStrongReferencer&& Other);

		/**
		 * Tests to see whether or not this strong counter contains a valid reference
		 *
		 * @return  True if reference is valid
		 */
		inline bool IsValid() const;

		inline bool IsNull() const;

		/**
		 * Returns the number of strong references to this object (including this reference.)
		 *
		 * @return  Number of strong references to the object (including this reference.)
		 */
		inline int32 GetStrongReferenceCount() const;

		/**
		 * Returns true if this is the only reference to this object, weak or strong.
		 *
		 * @return  True if there is only one reference to the object.
		 */
		inline bool IsUniqueReference() const;

		/**
		 * Returns true if this the only strong reference to this object.
		 *
		 * @return True if there is only one strong reference to the object.
		 */
		inline bool IsUniqueStrong() const;

		inline bool TryDeleteObject();

	private:

 		// Expose access to ReferenceController to FWeakReferencer
		friend class FWeakReferencer;

	private:

		/** Pointer to the reference controller for the object a strong reference/pointer is referencing */
		FReferenceControllerBase* ReferenceController;
	};

	/**
	 * FWeakReferencer is a wrapper around a pointer to a reference controller that is used
	 * by a TManagedWeakPtr to keep track of a referenced object's lifetime.
	 */
	class FWeakReferencer
	{
	public:

		/** Default constructor with empty counter */
		inline FWeakReferencer();

		/** Construct a weak referencer object from another weak referencer */
		inline FWeakReferencer(const FWeakReferencer& Other);

		/** Construct a weak referencer object from an rvalue weak referencer */
		inline FWeakReferencer(FWeakReferencer&& Other);

		/** Construct a weak referencer object from a strong referencer object */
		inline FWeakReferencer(const FStrongReferencer& OtherStrongRef);

		/** Destructor. */
		inline ~FWeakReferencer();
		
		/** Assignment operator from a weak referencer object.  If this counter was previously referencing an
		    object, that reference will be released. */
		inline FWeakReferencer& operator=(const FWeakReferencer& Other);

		/** Assignment operator from an rvalue weak referencer object.  If this counter was previously referencing an
		    object, that reference will be released. */
		inline FWeakReferencer& operator=(FWeakReferencer&& Other);
		
		/** Assignment operator from a strong reference counter.  If this counter was previously referencing an
		   object, that reference will be released. */
		inline FWeakReferencer& operator=(const FStrongReferencer& OtherStrongRef);

		/**
		 * Tests to see whether or not this weak counter contains a valid reference
		 *
		 * @return  True if reference is valid
		 */
		inline bool IsValid() const;

	private:

		/** Assigns a new reference controller to this counter object, first adding a reference to it, then
		    releasing the previous object. */
		inline void AssignReferenceController(FReferenceControllerBase* NewReferenceController);
	private:

 		/** Expose access to ReferenceController to FStrongReferencer. */
		friend class FStrongReferencer;

	private:

		/** Pointer to the reference controller for the object a TManagedWeakPtr is referencing */
		FReferenceControllerBase* ReferenceController;
	};


	inline FStrongReferencer::FStrongReferencer()
		: ReferenceController(nullptr)
	{
	}

	inline FStrongReferencer::FStrongReferencer(FReferenceControllerBase* InReferenceController)
		: ReferenceController(InReferenceController)
	{
	}
	
	inline FStrongReferencer::FStrongReferencer(const FStrongReferencer& Other)
		: ReferenceController(Other.ReferenceController)
	{
		// If the incoming reference had an object associated with it, then go ahead and increment the
		// strong reference count
		if (ReferenceController != nullptr)
		{
			ReferenceController->AddStrongReference();
		}
	}
	
	inline FStrongReferencer::FStrongReferencer(FStrongReferencer&& Other)
		: ReferenceController(Other.ReferenceController)
	{
		Other.ReferenceController = nullptr;
	}

	inline FStrongReferencer::FStrongReferencer(const FWeakReferencer& OtherWeak)
		: ReferenceController(OtherWeak.ReferenceController)
	{
		// If the incoming reference had an object associated with it, then go ahead and increment the
		// strong reference count
		if (ReferenceController != nullptr)
		{
			// Attempt to elevate a weak reference to a strong one.  For this to work, the object this
			// weak counter is associated with must already have at least one strong reference.  We'll
			// never revive a pointer that has already expired!
			if (!ReferenceController->ConditionallyAddStrongReference())
			{
				ReferenceController = nullptr;
			}
		}
	}

	inline FStrongReferencer::FStrongReferencer(FWeakReferencer&& OtherWeak)
		: ReferenceController(OtherWeak.ReferenceController)
	{
		// If the incoming reference had an object associated with it, then go ahead and increment the
		// strong reference count
		if (ReferenceController != nullptr)
		{
			// Attempt to elevate a weak reference to a strong one.  For this to work, the object this
			// weak counter is associated with must already have at least one strong reference.  We'll
			// never revive a pointer that has already expired!
			if (!ReferenceController->ConditionallyAddStrongReference())
			{
				ReferenceController = nullptr;
			}

			// Tell the reference counter object that we're no longer referencing the object with
			// this weak pointer
			OtherWeak.ReferenceController->ReleaseWeakReference();
			OtherWeak.ReferenceController = nullptr;
		}
	}

	inline FStrongReferencer::~FStrongReferencer()
	{
		if (ReferenceController != nullptr)
		{
			ReferenceController->ReleaseStrongReference();
		}
	}

	inline FStrongReferencer& FStrongReferencer::operator=(const FStrongReferencer& Other)
	{
		// Make sure we're not be reassigned to ourself!
		FReferenceControllerBase* NewReferenceController = Other.ReferenceController;
		if (NewReferenceController != ReferenceController)
		{
			// First, add a strong reference to the new object
			if (NewReferenceController != nullptr)
			{
				NewReferenceController->AddStrongReference();
			}

			// Release strong reference to the old object
			if (ReferenceController != nullptr)
			{
				ReferenceController->ReleaseStrongReference();
			}

			// Assume ownership of the assigned reference counter
			ReferenceController = NewReferenceController;
		}

		return *this;
	}

	inline FStrongReferencer& FStrongReferencer::operator=(FStrongReferencer&& Other)
	{
		// Make sure we're not be reassigned to ourself!
		FReferenceControllerBase* NewReferenceController = Other.ReferenceController;
		FReferenceControllerBase* OldReferenceController = ReferenceController;

		if (NewReferenceController != OldReferenceController)
		{
			// Assume ownership of the assigned reference counter
			Other.ReferenceController = nullptr;
			ReferenceController       = NewReferenceController;

			// Release strong reference to the old object
			if (OldReferenceController != nullptr)
			{
				OldReferenceController->ReleaseStrongReference();
			}
		}

		return *this;
	}

	inline bool FStrongReferencer::TryDeleteObject()
	{
		if (ReferenceController)
		{
			return ReferenceController->TryDeleteObject();
		}

		return false;
	}

	inline bool FStrongReferencer::IsValid() const
	{
		return ReferenceController != nullptr;
	}

	inline bool FStrongReferencer::IsNull() const
	{
		if (ReferenceController)
		{
			return ReferenceController->IsNull();
		}
		
		return false;
	}	

	inline int32 FStrongReferencer::GetStrongReferenceCount() const
	{
		return ReferenceController != nullptr ? ReferenceController->GetStrongReferenceCount() : 0;
	}

	inline bool FStrongReferencer::IsUniqueReference() const
	{
		return ReferenceController != nullptr && ReferenceController->IsUniqueReference();
	}

	inline bool FStrongReferencer::IsUniqueStrong() const
	{
		return ReferenceController != nullptr && ReferenceController->IsUniqueStrong();
	}


	inline FWeakReferencer::FWeakReferencer()
		: ReferenceController(nullptr)
	{
	}

	inline FWeakReferencer::FWeakReferencer(const FWeakReferencer& Other)
		: ReferenceController(Other.ReferenceController)
	{
		// If the weak referencer has a valid controller, then go ahead and add a weak reference to it.
		if (ReferenceController != nullptr)
		{
			ReferenceController->AddWeakReference();
		}
	}

	inline FWeakReferencer::FWeakReferencer(FWeakReferencer&& Other)
		: ReferenceController(Other.ReferenceController)
	{
		Other.ReferenceController = nullptr;
	}

	inline FWeakReferencer::FWeakReferencer(const FStrongReferencer& OtherStrongRef)
		: ReferenceController(OtherStrongRef.ReferenceController)
	{
		// If the strong referencer had a valid controller, then go ahead and add a weak reference to it.
		if (ReferenceController != nullptr)
		{
			ReferenceController->AddWeakReference();
		}
	}

	inline FWeakReferencer::~FWeakReferencer()
	{
		if (ReferenceController != nullptr)
		{
			// Tell the reference counter object that we're no longer referencing the object with
			// this weak pointer
			ReferenceController->ReleaseWeakReference();
		}
	}
	
	inline FWeakReferencer& FWeakReferencer::operator=(const FWeakReferencer& Other)
	{
		AssignReferenceController(Other.ReferenceController);

		return *this;
	}

	inline FWeakReferencer& FWeakReferencer::operator=(FWeakReferencer&& Other)
	{
		FReferenceControllerBase* Temp = ReferenceController;
		ReferenceController = Other.ReferenceController;
		Other.ReferenceController = nullptr;

		if (Temp != nullptr)
		{
			Temp->ReleaseWeakReference();
		}

		return *this;
	}

	inline FWeakReferencer& FWeakReferencer::operator=(const FStrongReferencer& OtherStrongRef)
	{
		AssignReferenceController(OtherStrongRef.ReferenceController);

		return *this;
	}

	inline bool FWeakReferencer::IsValid() const
	{
		return ReferenceController != nullptr && !ReferenceController->IsDeleted();
	}

	inline void FWeakReferencer::AssignReferenceController(FReferenceControllerBase* NewReferenceController)
	{
		// Only proceed if the new reference counter is different than our current
		if (NewReferenceController != ReferenceController)
		{
			// First, add a weak reference to the new object
			if (NewReferenceController != nullptr)
			{
				NewReferenceController->AddWeakReference();
			}

			// Release weak reference to the old object
			if (ReferenceController != nullptr)
			{
				ReferenceController->ReleaseWeakReference();
			}

			// Assume ownership of the assigned reference counter
			ReferenceController = NewReferenceController;
		}
	}

} // namespace ManagedPointerInternal

template<class> class TManagedRef;
template<class> class TManagedPtr;
template<class> class TManagedWeakPtr;


namespace ManagedPointerInternal
{
	template<typename ObjectType>
	TManagedRef<ObjectType> MakeManagedStrongRef(ObjectType* Object, FReferenceControllerBase* Controller)
	{
		return TManagedRef<ObjectType>(Object, Controller);
	}

	template<typename ObjectType>
	TManagedPtr<ObjectType> MakeManagedNullPtr(FReferenceControllerBase* Controller)
	{
		return TManagedPtr<ObjectType>(Controller, FNullWithControllerTag{});
	}
}

template<class ObjectType>
class TManagedRef
{
public:
	using ElementType = ObjectType;


	/**
	 * Constructs a strong reference as a reference to an existing strong reference's object.
	 * This constructor is needed so that we can implicitly upcast to base classes.
	 *
	 * @param  Other  The strong reference whose object we should create an additional reference to
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedRef(const TManagedRef<OtherType>& Other)
		: Object(Other.Object)
		, StrongReferenceState(Other.StrongReferenceState)
	{
	}


	inline TManagedRef(const TManagedRef& Other)
		: Object(Other.Object)
		, StrongReferenceState(Other.StrongReferenceState)
	{
    }

	inline TManagedRef(TManagedRef&& Other)
		: Object(Other.Object)
		, StrongReferenceState(Other.StrongReferenceState)
	{
		// We're intentionally not moving here, because we don't want to leave Other in a
		// null state, because that breaks the class invariant.  But we provide a move constructor
		// anyway in case the compiler complains that we have a move assign but no move construct.
	}

	/**
	 * Assignment operator replaces this strong reference with the specified strong reference.  The object
	 * currently referenced by this strong reference will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  Other  Strong reference to replace with
	 */
	inline TManagedRef& operator=(const TManagedRef& Other)
	{
		TManagedRef Temp = Other;
		::Swap(Temp, *this);
		return *this;
	}

	inline TManagedRef& operator=(TManagedRef&& Other)
	{
		::Swap(*this, Other);
		return *this;
	}

	/**
	 * Converts a strong reference to a strong pointer.
	 *
	 * @return  Managed pointer to the object
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TManagedPtr<ObjectType> ToStrongPtr() const
	{
		return TManagedPtr<ObjectType>(*this);
	}

	/**
	 * Converts a strong reference to a weak ptr.
	 *
	 * @return  Weak pointer to the object
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TManagedWeakPtr<ObjectType> ToWeakPtr() const
	{
		return TManagedWeakPtr<ObjectType>(*this);
	}

	/**
	 * Returns a C++ reference to the object this strong reference is referencing
	 *
	 * @return  The object owned by this strong reference
	 */
	[[nodiscard]] inline ObjectType& Get() const
	{
		// Should never be nullptr as TManagedRef is never nullable
		checkSlow(IsValid());
		return *Object;
	}

	/**
	 * Dereference operator returns a reference to the object this strong pointer points to
	 *
	 * @return  Reference to the object
	 */
	[[nodiscard]] inline ObjectType& operator*() const
	{
		// Should never be nullptr as TManagedRef is never nullable
		checkSlow(IsValid());
		return *Object;
	}

	/**
	 * Arrow operator returns a pointer to this strong reference's object
	 *
	 * @return  Returns a pointer to the object referenced by this strong reference
	 */
	[[nodiscard]] inline ObjectType* operator->() const
	{
		// Should never be nullptr as TManagedRef is never nullable
		checkSlow(IsValid());
		return Object;
	}

	/**
	 * Returns true if this is the only reference to this object, weak or strong.
	 *
	 * @return  True if there is only one reference to the object.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsUniqueReference() const
	{
		return StrongReferenceState.IsUniqueReference();
	}

	template <typename OtherType>
	[[nodiscard]] bool UEOpEquals(const TManagedRef<OtherType>& Rhs) const
	{
		return this->Object == &Rhs.Get();
	}

	template <typename OtherType>
	[[nodiscard]] bool UEOpEquals(const TManagedPtr<OtherType>& Rhs) const
	{
		// This comparison against null is maintained as existing behavior, but isn't consistent with TManagedWeakPtr comparison.
		OtherType* RhsPtr = Rhs.Get();
		return RhsPtr && RhsPtr == this->Object;
	}

	template <typename OtherType>
	[[nodiscard]] bool UEOpEquals(const TManagedWeakPtr<OtherType>& Rhs) const
	{
		return this->Object == Rhs.Pin().Get();
	}

	template <typename OtherType>
	[[nodiscard]] bool UEOpLessThan(const TManagedRef<OtherType>& Rhs) const
	{
		return this->Object < &Rhs.Get();
	}

private:

	/**
	 * Converts a strong pointer to a strong reference.  The pointer *must* be valid or an assertion will trigger.
	 * NOTE: This explicit conversion constructor is intentionally private.  Use 'ToStrongRef()' instead.
	 *
	 * @return  Reference to the object
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline explicit TManagedRef(const TManagedPtr<OtherType>& Other)
		: Object(Other.Object)
		, StrongReferenceState(Other.StrongReferenceState)
	{
		// If this assert goes off, it means a strong reference was created from a strong pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TManagedPtr instead.
		check(IsValid());
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline explicit TManagedRef(TManagedPtr<OtherType>&& Other)
		: Object(Other.Object)
		, StrongReferenceState(MoveTemp(Other.StrongReferenceState))
	{
		Other.Object = nullptr;

		// If this assert goes off, it means a strong reference was created from a strong pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TManagedPtr instead.
		check(IsValid());
	}

	/**
	 * Checks to see if this strong reference is actually pointing to an object. 
	 * NOTE: This validity test is intentionally private because strong references must always be valid.
	 *
	 * @return  True if the strong reference is valid and can be dereferenced
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Object != nullptr;
	}

	friend TManagedRef ManagedPointerInternal::MakeManagedStrongRef<ObjectType>(ObjectType* Object, ManagedPointerInternal::FReferenceControllerBase* Controller);

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
    template<class OtherType> friend class TManagedRef;

	// Declare other smart pointer types as friends as needed
    template<class OtherType> friend class TManagedPtr;
    template<class OtherType> friend class TManagedWeakPtr;

private:

	/** The object we're holding a reference to.  Can be nullptr. */
	ObjectType* Object;

	/** Interface to the reference counter for this object.  Note that the actual reference
		controller object is strong by all strong and weak pointers that refer to the object */
	ManagedPointerInternal::FStrongReferencer StrongReferenceState;
	
	inline explicit TManagedRef(ObjectType* InObject, ManagedPointerInternal::FReferenceControllerBase* InController)
		: Object(InObject)
		, StrongReferenceState(InController)
	{
	}
};


template<typename ObjectType>
class TManagedPtr
{
public:
	using ElementType = ObjectType;

	/**
	 * Constructs an empty strong pointer
	 */
	inline TManagedPtr(ManagedPointerInternal::FNullTag* = nullptr)
		: Object(nullptr)
		, StrongReferenceState()
	{
	}

	/**
	 * Constructs a strong pointer as a strong reference to an existing strong pointer's object.
	 * This constructor is needed so that we can implicitly upcast to base classes.
	 *
	 * @param  Other  The strong pointer whose object we should create an additional reference to
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedPtr(const TManagedPtr<OtherType>& Other)
		: Object(Other.Object)
		, StrongReferenceState(Other.StrongReferenceState)
	{
	}

	inline TManagedPtr(const TManagedPtr& Other)
		: Object(Other.Object)
		, StrongReferenceState(Other.StrongReferenceState)
	{
	}

	inline TManagedPtr(TManagedPtr&& Other)
		: Object(Other.Object)
		, StrongReferenceState(MoveTemp(Other.StrongReferenceState))
	{
		Other.Object = nullptr;
	}

	/**
	 * Implicitly converts a strong reference to a strong pointer, adding a reference to the object.
	 * NOTE: We allow an implicit conversion from TManagedRef to TManagedPtr because it's always a safe conversion.
	 *
	 * @param  OtherRef  The strong reference that will be converted to a strong pointer
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedPtr(const TManagedRef<OtherType>& OtherRef)
		: Object(OtherRef.Object)
		, StrongReferenceState(OtherRef.StrongReferenceState)
	{
		// There is no rvalue overload of this constructor, because 'stealing' the pointer from a
		// TSharedRef would leave it as null, which would invalidate its invariant.
	}

	/**
	 * Special constructor used internally to statically cast one strong pointer type to another.  You
	 * should never call this constructor directly.  Instead, use the StaticCastManagedPtr() function.
	 * This constructor creates a strong pointer as a strong reference to an existing strong pointer after
	 * statically casting that pointer's object.  This constructor is needed for static casts.
	 *
	 * @param  InSharedPtr  The strong pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	inline TManagedPtr(const TManagedPtr<OtherType>& Other, ManagedPointerInternal::FStaticCastTag)
		: Object(static_cast<ObjectType*>(Other.Object))
		, StrongReferenceState(Other.StrongReferenceState)
	{
	}
	
	/**
	 * Special constructor used internally to cast a 'const' strong pointer a 'mutable' pointer.  You
	 * should never call this constructor directly.  Instead, use the ConstCastManagedPtr() function.
	 * This constructor creates a strong pointer as a strong reference to an existing strong pointer after
	 * const casting that pointer's object.  This constructor is needed for const casts.
	 *
	 * @param  Other  The strong pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	inline TManagedPtr(const TManagedPtr<OtherType>& Other, ManagedPointerInternal::FConstCastTag)
		: Object(const_cast<ObjectType*>(Other.Object))
		, StrongReferenceState(Other.StrongReferenceState)
	{
	}

	/**
	 * Assignment operator replaces this strong pointer with the specified strong pointer.  The object
	 * currently referenced by this strong pointer will no longer be referenced and will be deleted if
	 * there are no other referencers.
	 *
	 * @param  Other  Shared pointer to replace with
	 */
	inline TManagedPtr& operator=(const TManagedPtr& Other)
	{
		TManagedPtr Temp = Other;
		::Swap(Temp, *this);
		return *this;
	}

	// PGO_LINK_DISABLE_WARNINGS was found on the original implementation, for now 
	// leave it commented out, enable if it becomes an issue.

	// Disable false positive buffer overrun warning during pgo linking step.
	//PGO_LINK_DISABLE_WARNINGS
	inline TManagedPtr& operator=(TManagedPtr&& Other)
	{
		if (this != &Other)
		{
			Object = Other.Object;
			Other.Object = nullptr;
			StrongReferenceState = MoveTemp(Other.StrongReferenceState);
		}

		return *this;
	}
	//PGO_LINK_ENABLE_WARNINGS

	/**
	 * Converts a strong pointer to a strong reference.  The pointer *must* be valid or an assertion will trigger.
	 *
	 * @return  Reference to the object
	 */
	[[nodiscard]] inline TManagedRef<ObjectType> ToStrongRef() const&
	{
		// If this assert goes off, it means a strong reference was created from a strong pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TManagedPtr instead.
		check(IsValid());
		return TManagedRef<ObjectType>(*this);
	}

	/**
	 * Converts a strong pointer to a strong reference.  The pointer *must* be valid or an assertion will trigger.
	 *
	 * @return  Reference to the object
	 */
	[[nodiscard]] inline TManagedRef<ObjectType> ToStrongRef() &&
	{
		// If this assert goes off, it means a strong reference was created from a strong pointer that was nullptr.
		// Shared references are never allowed to be null.  Consider using TManagedPtr instead.
		check(IsValid());
		return TManagedRef<ObjectType>(MoveTemp(*this));
	}

	/**
	 * Converts a strong pointer to a weak ptr.
	 *
	 * @return  Weak pointer to the object
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TManagedWeakPtr<ObjectType> ToWeakPtr() const
	{
		return TManagedWeakPtr<ObjectType>(*this);
	}

	/**
	 * Returns the object referenced by this pointer, or nullptr if no object is reference
	 *
	 * @return  The object owned by this strong pointer, or nullptr
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT ObjectType* Get() const
	{
		return Object;
	}

	/**
	 * Checks to see if this strong pointer is actually pointing to an object
	 *
	 * @return  True if the strong pointer is valid and can be dereferenced
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return Object != nullptr;
	}

	/**
	 * Checks to see if this strong pointer is actually pointing to an object
	 *
	 * @return  True if the strong pointer is valid and can be dereferenced
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const bool IsValid() const
	{
		return Object != nullptr;
	}

	/**
	 * Check to see if this trong pointer has been set to an explicit value using the Make* functions.
	 * NOTE: The pointer may have been set with the MakeNull function, and not be valid.
	 *
	 * @return True if the strong pointer has beens set an explicit value.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT const bool IsSet() const
	{
		return Object != nullptr || StrongReferenceState.IsNull();
	}
	

	/**
	 * Dereference operator returns a reference to the object this strong pointer points to
	 *
	 * @return  Reference to the object
	 */
	template <
		typename U = ObjectType
		UE_REQUIRES(UE_REQUIRES_EXPR(*(U*)nullptr)) // this construct means that operator* is only considered for overload resolution if T is dereferenceable
	>
	[[nodiscard]] inline U& operator*() const
	{
		check(IsValid());
		return *Object;
	}

	/**
	 * Arrow operator returns a pointer to the object this strong pointer references
	 *
	 * @return  Returns a pointer to the object referenced by this strong pointer
	 */
	[[nodiscard]] inline ObjectType* operator->() const
	{
		check(IsValid());
		return Object;
	}

	/**
	 * Resets this strong pointer, removing a reference to the object.  If there are no other strong
	 * references to the object then it will be destroyed.
	 */
	UE_FORCEINLINE_HINT void Reset()
	{
 		*this = TManagedPtr<ObjectType>();
	}

	/**
	 * Returns true if this is the only reference, strong or weak, to this object.
	 *
	 * @return  True if there is only one reference to the object.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsUniqueReference() const
	{
		return StrongReferenceState.IsUniqueReference();
	}

	template <typename OtherType>
	[[nodiscard]] inline bool UEOpEquals(const TManagedPtr<OtherType>& Rhs) const
	{
		return this->Object == Rhs.Get();
	}

	template <typename OtherType>
	[[nodiscard]] UE_REWRITE bool UEOpEquals(const TManagedWeakPtr<OtherType>& Rhs) const
	{
		return this->Object == Rhs.Pin().Get();
	}

	[[nodiscard]] UE_REWRITE bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !this->Object;
	}

	template <typename OtherType>
	[[nodiscard]] inline bool UEOpLessThan(const TManagedPtr<OtherType>& Rhs) const
	{
		return this->Object < Rhs.Get();
	}

	/** Deletes the object if this is the only strong reference. */
	inline bool TryDeleteObject()
	{
		const bool bSuccess = StrongReferenceState.TryDeleteObject();

		if (bSuccess)
		{
			Object = nullptr;
		}

		return bSuccess;
	}

private:
	/**
	 * Constructs a strong pointer from a weak pointer, allowing you to access the object (if it
	 * hasn't expired yet.)  Remember, if the object did not have strong references and have been 
	 * deleted the strong pointer will not be valid.  You should always check to make sure this strong
	 * pointer is valid before trying to dereference the strong pointer.
	 *
	 * NOTE: This constructor is private to force users to be explicit when converting a weak
	 *       pointer to a strong pointer.  Use the weak pointer's Pin() method instead!
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline explicit TManagedPtr(const TManagedWeakPtr<OtherType>& OtherWeak)
		: Object(nullptr)
		, StrongReferenceState(OtherWeak.WeakReferenceState)
	{
		// Check that the strong reference was created from the weak reference successfully.  We'll only
		// cache a pointer to the object if we have a valid strong reference.
		if (StrongReferenceState.IsValid())
		{
			Object = OtherWeak.Object;
		}
	}

	/**
	 * Constructs a strong pointer from a weak pointer, allowing you to access the object (if it
	 * hasn't expired yet.)  Remember, if the object did not have strong references and have been 
	 * deleted the strong pointer will not be valid.  You should always check to make sure this strong
	 * pointer is valid before trying to dereference the strong pointer.
	 *
	 * NOTE: This constructor is private to force users to be explicit when converting a weak
	 *       pointer to a strong pointer.  Use the weak pointer's Pin() method instead.
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline explicit TManagedPtr(TManagedWeakPtr<OtherType>&& OtherWeak)
		: Object(nullptr)
		, StrongReferenceState(MoveTemp(OtherWeak.WeakReferenceState))
	{
		// Check that the strong reference was created from the weak reference successfully.  We'll only
		// cache a pointer to the object if we have a valid strong reference.
		if (StrongReferenceState.IsValid())
		{
			Object = OtherWeak.Object;
			OtherWeak.Object = nullptr;
		}
	}

	friend TManagedPtr ManagedPointerInternal::MakeManagedNullPtr<ObjectType>(ManagedPointerInternal::FReferenceControllerBase* Controller);

	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
	template<class OtherType> friend class TManagedPtr;

	// Declare other smart pointer types as friends as needed
	template<class OtherType> friend class TManagedRef;
	template<class OtherType> friend class TManagedWeakPtr;

private:

	/** The object we're holding a reference to.  Can be nullptr. */
	ObjectType* Object;

	/** Interface to the reference counter for this object.  Note that the actual reference
		controller object is strong by all strong and weak pointers that refer to the object */
	ManagedPointerInternal::FStrongReferencer StrongReferenceState;

	/** Special constructor only used when we want to create a null refrence with a null controller */
	inline explicit TManagedPtr(ManagedPointerInternal::FReferenceControllerBase* InController, ManagedPointerInternal::FNullWithControllerTag)
		: Object(nullptr)
		, StrongReferenceState(InController)
	{
	}

};


/**
 * TManagedWeakPtr is a non-intrusive reference-counted weak object pointer.
 */
template<class ObjectType>
class TManagedWeakPtr
{
public:
	using ElementType = ObjectType;

	/** Constructs an empty TWeakPtr */
	inline TManagedWeakPtr(ManagedPointerInternal::FNullTag* = nullptr)
		: Object(nullptr)
		, WeakReferenceState()
	{
	}

	/**
	 * Constructs a weak pointer from a strong reference
	 *
	 * @param  OtherStrongRef  The strong reference to create a weak pointer from
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr(const TManagedRef<OtherType>& OtherStrongRef)
		: Object(OtherStrongRef.Object)
		, WeakReferenceState(OtherStrongRef.StrongReferenceState)
	{
	}

	/**
	 * Constructs a weak pointer from a strong pointer
	 *
	 * @param  OtherStrong  The strong pointer to create a weak pointer from
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr(const TManagedPtr<OtherType>& OtherStrong)
		: Object(OtherStrong.Object )
		, WeakReferenceState(OtherStrong.StrongReferenceState)
	{
	}

	/**
	 * Special constructor used internally to statically cast one weak pointer type to another.  You
	 * should never call this constructor directly.  Instead, use the StaticCastManagedWeakPtr() function.
	 * This constructor creates a weak pointer as a weak reference to an existing weak pointer after
	 * statically casting that pointer's object.  This constructor is needed for static casts.
	 *
	 * @param  Other  The weak pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	inline TManagedWeakPtr(const TManagedWeakPtr<OtherType>& Other, ManagedPointerInternal::FStaticCastTag)
		: Object(static_cast<ObjectType*>(Other.Object))
		, WeakReferenceState(Other.WeakReferenceState)
	{
	}

	/**
	 * Special constructor used internally to cast a 'const' weak pointer a 'mutable' pointer.  You
	 * should never call this constructor directly.  Instead, use the ConstCastManagedWeakPtr() function.
	 * This constructor creates a weak pointer as a weak reference to an existing weak pointer after
	 * const casting that pointer's object.  This constructor is needed for const casts.
	 *
	 * @param  Other  The weak pointer whose object we should create an additional reference to
	 */
	template <typename OtherType>
	inline TManagedWeakPtr(TManagedWeakPtr<OtherType> const& Other, ManagedPointerInternal::FConstCastTag)
		: Object(const_cast<ObjectType*>(Other.Object))
		, WeakReferenceState(Other.WeakReferenceState)
	{
	}

	/**
	 * Constructs a weak pointer from a weak pointer of another type.
	 * This constructor is intended to allow derived-to-base conversions.
	 *
	 * @param  Other  The weak pointer to create a weak pointer from
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr(const TManagedWeakPtr<OtherType>& Other)
		: Object(Other.Object)
		, WeakReferenceState(Other.WeakReferenceState)
	{
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr(TManagedWeakPtr<OtherType>&& Other)
		: Object(Other.Object)
		, WeakReferenceState(MoveTemp(Other.WeakReferenceState))
	{
		Other.Object = nullptr;
	}

	inline TManagedWeakPtr(const TManagedWeakPtr& Other)
		: Object(Other.Object)
		, WeakReferenceState(Other.WeakReferenceState)
	{
	}

	inline TManagedWeakPtr(TManagedWeakPtr&& Other)
		: Object(Other.Object)
		, WeakReferenceState(MoveTemp(Other.WeakReferenceState))
	{
		Other.Object = nullptr;
	}

	/**
	 * Assignment to a nullptr pointer.  Clears this weak pointer's reference.
	 */
	inline TManagedWeakPtr& operator=(ManagedPointerInternal::FNullTag*)
	{
		Reset();

		return *this;
	}

	/**
	 * Assignment operator adds a weak reference to the object referenced by the specified weak pointer
	 *
	 * @param  Other  The weak pointer for the object to assign
	 */
	inline TManagedWeakPtr& operator=(const TManagedWeakPtr& Other)
	{
		TManagedWeakPtr Temp = Other;
		::Swap(Temp, *this);

		return *this;
	}

	inline TManagedWeakPtr& operator=(TManagedWeakPtr&& OtherPtr)
	{
		if (this != &OtherPtr)
		{
			Object             = OtherPtr.Object;
			OtherPtr.Object    = nullptr;
			WeakReferenceState = MoveTemp(OtherPtr.WeakReferenceState);
		}

		return *this;
	}

	/**
	 * Assignment operator adds a weak reference to the object referenced by the specified weak pointer.
	 * This assignment operator is intended to allow derived-to-base conversions.
	 *
	 * @param  Other  The weak pointer for the object to assign
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr& operator=(const TManagedWeakPtr<OtherType>& Other)
	{
		Object = Other.Pin().Get();
		WeakReferenceState = Other.WeakReferenceState;

		return *this;
	}

	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr& operator=(TManagedWeakPtr<OtherType>&& OtherPtr)
	{
		Object 				= OtherPtr.Object;
		OtherPtr.Object		= nullptr;
		WeakReferenceState	= MoveTemp(OtherPtr.WeakReferenceState);

		return *this;
	}

	/**
	 * Assignment operator sets this weak pointer from a strong reference
	 *
	 * @param  OtherStrongRef  The strong reference used to assign to this weak pointer
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr& operator=(const TManagedRef<OtherType>& OtherStrongRef)
	{
		Object = OtherStrongRef.Object;
		WeakReferenceState = OtherStrongRef.StrongReferenceState;
		return *this;
	}

	/**
	 * Assignment operator sets this weak pointer from a strong pointer
	 *
	 * @param  OtherStrongPtr  The strong pointer used to assign to this weak pointer
	 */
	template <
		typename OtherType
		UE_REQUIRES(std::is_convertible_v<OtherType*, ObjectType*>)
	>
	inline TManagedWeakPtr& operator=(const TManagedPtr<OtherType>& OtherStrongPtr)
	{
		Object = OtherStrongPtr.Object;
		WeakReferenceState = OtherStrongPtr.StrongReferenceState;

		return *this;
	}

	/**
	 * Converts this weak pointer to a strong pointer that you can use to access the object (if it
	 * hasn't expired yet.)  Remember, if there are no more strong references to the object, 
	 * the object may have been deleted and the retuned value not be valid.  You should always 
	 * check to make sure the returned pointer is valid before trying to dereference the strong
	 * pointer.
	 *
	 * @return  Strong pointer for this object (will only be valid if not deleted)
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TManagedPtr<ObjectType> Pin() const&
	{
		return TManagedPtr<ObjectType>(*this);
	}

	/**
	 * Converts this weak pointer to a strong pointer that you can use to access the object (if it
	 * hasn't expired yet.)  Remember, if there are no more strong references to the object, 
	 * the object may have been deleted and the retuned value not be valid.  You should always 
	 * check to make sure the returned pointer is valid before trying to dereference the strong
	 * pointer.
	 *
	 * @return  Strong pointer for this object (will only be valid if not deleted)
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT TManagedPtr<ObjectType> Pin() &&
	{
		return TManagedPtr<ObjectType>(MoveTemp(*this));
	}

	/**
	 * Checks to see if this weak pointer actually has a valid reference to an object
	 *
	 * @return  True if the weak pointer is valid and a pin operator would have succeeded
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Object != nullptr && WeakReferenceState.IsValid();
	}

	/**
	 * Resets this weak pointer, removing a weak reference to the object.  If there are no other strong
	 * or weak references to the object, then the tracking object will be destroyed.
	 */
	UE_FORCEINLINE_HINT void Reset()
	{
		*this = TManagedWeakPtr<ObjectType>();
	}

	/**
	 * Returns true if the object this weak pointer points to is the same as the specified object pointer.
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool HasSameObject( const void* InOtherPtr ) const
	{
		return Pin().Get() == InOtherPtr;
	}

	UE_FORCEINLINE_HINT uint32 GetWeakPtrTypeHash() const
	{
		return ::PointerHash( Object );
	}

	template <typename OtherType>
	[[nodiscard]] inline bool UEOpEquals(const TManagedWeakPtr<OtherType>& Rhs) const
	{
		return this->Pin().Get() == Rhs.Pin().Get();
	}

	[[nodiscard]] UE_REWRITE bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !this->IsValid();
	}

	template <typename OtherType>
	[[nodiscard]] inline bool UEOpLessThan(const TManagedWeakPtr<OtherType>& Rhs) const
	{
		return this->Pin().Get() < Rhs.Pin().Get();
	}

private:
	// We declare ourselves as a friend (templated using OtherType) so we can access members as needed
    template<class OtherType> friend class TManagedWeakPtr;

	// Declare ourselves as a friend of TManagedPtr so we can access members as needed
    template<class OtherType> friend class TManagedPtr;

private:
	/** The object we have a weak reference to.  Can be nullptr.  Also, it's important to note that because
	    this is a weak reference, the object this pointer points to may have already been destroyed. */
	ObjectType* Object;

	/** Interface to the reference counter for this object.  Note that the actual reference
		controller object is strong by all strong and weak pointers that refer to the object */
	ManagedPointerInternal::FWeakReferencer WeakReferenceState;
};


template <typename ObjectType, typename... ArgTypes>
[[nodiscard]] inline TManagedRef<ObjectType> MakeManaged(ArgTypes&&... Args)
{
	ManagedPointerInternal::TIntrusiveReferenceController<ObjectType>* Controller = 
		ManagedPointerInternal::NewIntrusiveReferenceController<ObjectType>(Forward<ArgTypes>(Args)...);
	
	return ManagedPointerInternal::MakeManagedStrongRef<ObjectType>(Controller->GetObjectPtr(), (ManagedPointerInternal::FReferenceControllerBase*)Controller);
}


template <typename ObjectType>
[[nodiscard]] inline TManagedPtr<ObjectType> MakeManagedNull()
{
	ManagedPointerInternal::TNullReferenceController<ObjectType>* Controller = 
		ManagedPointerInternal::NewNullReferenceController<ObjectType>();

	return ManagedPointerInternal::MakeManagedNullPtr<ObjectType>((ManagedPointerInternal::FReferenceControllerBase*)Controller);
}

/**
 * Casts a strong pointer of one type to another type. (static_cast)  Useful for down-casting.
 *
 * @param  Other  The strong pointer to cast
 */
template<class CastToType, class CastFromType>
[[nodiscard]] UE_FORCEINLINE_HINT TManagedPtr<CastToType> StaticCastManagedPtr(const TManagedPtr<CastFromType>& Other)
{
	return TManagedPtr<CastToType>(Other, ManagedPointerInternal::FStaticCastTag());
}


/**
 * Casts a weak pointer of one type to another type. (static_cast)  Useful for down-casting.
 *
 * @param  Other  The weak pointer to cast
 */
template <class CastToType, class CastFromType>
[[nodiscard]] UE_FORCEINLINE_HINT TManagedWeakPtr<CastToType> StaticCastManagedWeakPtr(const TManagedWeakPtr<CastFromType>& Other)
{
	return TManagedWeakPtr<CastToType>(Other, ManagedPointerInternal::FStaticCastTag());
}


/**
 * Casts a 'const' strong reference to 'mutable' strong reference. (const_cast)
 *
 * @param  Other  The strong reference to cast
 */
template <class CastToType, class CastFromType>
[[nodiscard]] UE_FORCEINLINE_HINT TManagedRef<CastToType> ConstCastManagedStrongRef(const TManagedRef<CastFromType>& Other)
{
	return TManagedRef<CastToType>(Other, ManagedPointerInternal::FConstCastTag());
}


/**
 * Casts a 'const' strong pointer to 'mutable' strong pointer. (const_cast)
 *
 * @param  Other  The strong pointer to cast
 */
template <class CastToType, class CastFromType>
[[nodiscard]] UE_FORCEINLINE_HINT TManagedPtr<CastToType> ConstCastManagedPtr(const TManagedPtr<CastFromType>& Other)
{
	return TManagedPtr<CastToType>(Other, ManagedPointerInternal::FConstCastTag());
}


/**
 * Casts a 'const' weak pointer to 'mutable' weak pointer. (const_cast)
 *
 * @param  InWeakPtr  The weak pointer to cast
 */
template <class CastToType, class CastFromType>
[[nodiscard]] UE_FORCEINLINE_HINT TManagedWeakPtr<CastToType> ConstCastManagedWeakPtr(const TManagedWeakPtr<CastFromType>& Other)
{
	return TManagedWeakPtr<CastToType>(Other, ManagedPointerInternal::FConstCastTag());
}


/**
* Computes a hash code for this object
*
* @param  InStrongRef  Strong pointer to compute hash code for
*
* @return  Hash code value
*/
template <typename ObjectType>
[[nodiscard]] uint32 GetTypeHash(const TManagedRef<ObjectType>& InStrongRef)
{
	return ::PointerHash(&InStrongRef.Get());
}

/**
* Computes a hash code for this object
*
* @param  InStrongPtr  Shared pointer to compute hash code for
*
* @return  Hash code value
*/
template <typename ObjectType>
[[nodiscard]] uint32 GetTypeHash(const TManagedPtr<ObjectType>& InStrongPtr)
{
	return ::PointerHash(InStrongPtr.Get());
}

/**
* Computes a hash code for this object
*
* @param  InWeakPtr  Weak pointer to compute hash code for
*
* @return  Hash code value
*/
template <typename ObjectType>
[[nodiscard]] uint32 GetTypeHash(const TManagedWeakPtr<ObjectType>& InWeakPtr)
{
	return InWeakPtr.GetWeakPtrTypeHash();
}

UE_OPS_NAMESPACE_VISIBLE(int32) // Type is irrelevant here. It does a simple type validity check. Only adds the compare operators to this namespace.

} // namespace UE::Mutable::Private
