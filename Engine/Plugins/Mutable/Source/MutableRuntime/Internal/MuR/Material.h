// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/ResourceID.h"
#include "MuR/Types.h"
#include "MuR/Operations.h"
#include "MuR/TVariant.h"

#include "UObject/StrongObjectPtr.h"
#include "Materials/MaterialInterface.h"
#include "MuR/PassthroughObject.h"
#include "Templates/TypeHash.h"

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private
{
	class FImage;

	struct FParameterKey
	{
		FName ParameterName;
		int8 LayerIndex = -1;

		inline void Serialise(FOutputArchive& arch) const;
		inline void Unserialise(FInputArchive& arch);

		bool operator==(const FParameterKey& Other) const = default;
	};


	inline uint32 GetTypeHash(const FParameterKey& Key)
	{
		uint32 GuidHash = ::GetTypeHash(Key.LayerIndex);
		GuidHash = HashCombineFast(GuidHash, GetTypeHash(Key.ParameterName));

		return GuidHash;
	}

    /** Material type resource */
	class FMaterial : public FResource
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------

		//! Serialisation
		static UE_API void Serialise(const FMaterial* MaterialPtr, FOutputArchive& Arch);
		static UE_API TManagedPtr<FMaterial> StaticUnserialise(FInputArchive& Arch);

        //! Clone this material
		UE_API TManagedPtr<FMaterial> Clone() const;

		// Resource interface
		virtual int32 GetDataSize() const override { return 0; };

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------
		UE_API bool operator==(const FMaterial& Other) const;
		UE_API void Serialise(FOutputArchive&) const;
		UE_API void Unserialise(FInputArchive&);

		/** Passthrough Id or TStrongObjectPtr. */
		TPassthroughObjectPtr<UMaterialInterface> PassthroughObject;

		struct FImageParameterData 
		{
			TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>> ImageParameter;
			int32 ImagePropertyIndex;

			inline void Serialise(FOutputArchive& arch) const;
			inline void Unserialise(FInputArchive& arch);

			bool operator==(const FImageParameterData& Other) const = default;
		};

		TMap<FParameterKey, FImageParameterData> ImageParameters;
		TMap<FParameterKey, FVector4f> ColorParameters;
		TMap<FParameterKey, float> ScalarParameters;
	};
}

#undef UE_API
