// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MemoryTrackingAllocationPolicy.h"


namespace UE::Mutable::Private::MemoryCounters
{
	struct FInternalMemoryCounter
	{
		static MUTABLERUNTIME_API std::atomic<SSIZE_T>& Get();
	};
}


namespace UE::Mutable::Private
{
	using AllocType = FDefaultMemoryTrackingAllocator<MemoryCounters::FInternalMemoryCounter>;

	template<class Type, class Alloc = AllocType >
	using TMemoryTrackedArray = TArray<Type, Alloc>;
}