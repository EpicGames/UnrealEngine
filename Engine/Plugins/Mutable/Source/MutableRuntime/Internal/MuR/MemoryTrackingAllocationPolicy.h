// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "MuR/MemoryTrackingUtils.h"

#include <atomic>
#include <type_traits>

namespace UE::Mutable::Private
{
	
	
	// CounterType is expected to be of the following form. 
	//struct FCounterTypeName
	//{
	//	static std::atomic<SSIZE_T>& Get()
	//	{
	//		static std::atomic<SSIZE_T> Counter{0};
	//		return Counter;
	//	}
	//};


	template<typename BaseAlloc, typename CounterType>
	class TMemoryTrackingAllocatorWrapper
	{
		static_assert(std::is_same_v<decltype(CounterType::Get().load()), SSIZE_T>, "CounterType must be signed.");

	public:
		using SizeType = typename BaseAlloc::SizeType;

		enum { NeedsElementType = BaseAlloc::NeedsElementType };
		enum { RequireRangeCheck = BaseAlloc::RequireRangeCheck };

		/** 
		 * ForAnyElementType is privately inherited from the wrapped allocator ForAnyElementType so 
		 * the base class members must be explicitly defined to compile. This way if any new method 
		 * is added to the allocator interface, it forces its addition here. This is useful in case 
		 * the new method needs to do some memory tracking, otherwise a simple using declaration may 
		 * suffice.
		 */
		class ForAnyElementType : private BaseAlloc::ForAnyElementType 
		{
		public:
			ForAnyElementType() 
			{
			}

			/** Destructor. */
			inline ~ForAnyElementType()
			{
				CounterType::Get().fetch_sub(AllocSize, std::memory_order_relaxed);

#if UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK
				FGlobalMemoryCounter::Update(-AllocSize);
#endif

				AllocSize = 0;
			}

			ForAnyElementType(const ForAnyElementType&) = delete;
			ForAnyElementType& operator=(const ForAnyElementType&) = delete;

			inline void MoveToEmpty(ForAnyElementType& Other)
			{
				BaseAlloc::ForAnyElementType::MoveToEmpty(Other);

				CounterType::Get().fetch_sub(AllocSize, std::memory_order_relaxed);

#if UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK
				FGlobalMemoryCounter::Update(-AllocSize);
#endif

				AllocSize = Other.AllocSize;
				Other.AllocSize = 0;
			}

			inline void ResizeAllocation(
				SizeType CurrentNum,
				SizeType NewMax,
				SIZE_T NumBytesPerElement
			)
			{
				BaseAlloc::ForAnyElementType::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement);

				const SSIZE_T AllocatedSize = (SSIZE_T)BaseAlloc::ForAnyElementType::GetAllocatedSize(NewMax, NumBytesPerElement); 
				const SSIZE_T Differential = AllocatedSize - AllocSize;
				const SSIZE_T PrevCounterValue = CounterType::Get().fetch_add(Differential, std::memory_order_relaxed);
				check(PrevCounterValue >= AllocSize);

#if UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK
				FGlobalMemoryCounter::Update(Differential);
#endif

				AllocSize = AllocatedSize;
			}

			template <
				class T = BaseAlloc
				UE_REQUIRES(TAllocatorTraits<T>::SupportsElementAlignment)
			>
			inline void ResizeAllocation(
				SizeType CurrentNum,
				SizeType NewMax,
				SIZE_T NumBytesPerElement,
				uint32 AlignmentOfElement
			)
			{
				BaseAlloc::ForAnyElementType::ResizeAllocation(CurrentNum, NewMax, NumBytesPerElement, AlignmentOfElement);

				const SSIZE_T AllocatedSize = (SSIZE_T)BaseAlloc::ForAnyElementType::GetAllocatedSize(NewMax, NumBytesPerElement); 
				const SSIZE_T Differential = AllocatedSize - AllocSize;
				const SSIZE_T PrevCounterValue = CounterType::Get().fetch_add(Differential, std::memory_order_relaxed);
				check(PrevCounterValue >= AllocSize);

#if UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK
				FGlobalMemoryCounter::Update(Differential);
#endif

				AllocSize = AllocatedSize;
			}

			// Explicitly incorporate pass-through base allocator member functions.
			using BaseAlloc::ForAnyElementType::GetAllocation;
			using BaseAlloc::ForAnyElementType::CalculateSlackReserve;
			using BaseAlloc::ForAnyElementType::CalculateSlackShrink;
			using BaseAlloc::ForAnyElementType::CalculateSlackGrow;
			using BaseAlloc::ForAnyElementType::GetAllocatedSize;
			using BaseAlloc::ForAnyElementType::HasAllocation;
			using BaseAlloc::ForAnyElementType::GetInitialCapacity;

#if UE_ENABLE_ARRAY_SLACK_TRACKING
			using BaseAlloc::ForAnyElementType::SlackTrackerLogNum;
#endif

		private:
			SSIZE_T AllocSize = 0;
		};

		template<typename ElementType>
		class ForElementType : public ForAnyElementType
		{
		public:
			ForElementType()
			{
			}

			inline ElementType* GetAllocation() const
			{
				return (ElementType*)ForAnyElementType::GetAllocation();
			}
		};

	};


	/** Default memory tracking allocators needed for TArray and TMap. */

	template<typename CounterType>
	using FDefaultMemoryTrackingAllocator = TMemoryTrackingAllocatorWrapper<FDefaultAllocator, CounterType>;

	template<typename CounterType>
	using FDefaultMemoryTrackingBitArrayAllocator = TInlineAllocator<4, FDefaultMemoryTrackingAllocator<CounterType>>;

	template<typename CounterType>
	using FDefaultMemoryTrackingSparceArrayAllocator = TSparseArrayAllocator<
		FDefaultMemoryTrackingAllocator<CounterType>,
		FDefaultMemoryTrackingBitArrayAllocator<CounterType>>;
		
	template<typename CounterType>
	using FDefaultMemoryTrackingSetAllocator = TSetAllocator<
		FDefaultMemoryTrackingSparceArrayAllocator<CounterType>,
		TInlineAllocator<1, FDefaultMemoryTrackingAllocator<CounterType>>,
		DEFAULT_NUMBER_OF_ELEMENTS_PER_HASH_BUCKET,
		DEFAULT_BASE_NUMBER_OF_HASH_BUCKETS,
		DEFAULT_MIN_NUMBER_OF_HASHED_ELEMENTS>;
}

template<typename BaseAlloc, typename Counter>
struct TAllocatorTraits<UE::Mutable::Private::TMemoryTrackingAllocatorWrapper<BaseAlloc, Counter>> : public TAllocatorTraits<BaseAlloc>
{
	enum { SupportsMoveFromOtherAllocator = false };
};

