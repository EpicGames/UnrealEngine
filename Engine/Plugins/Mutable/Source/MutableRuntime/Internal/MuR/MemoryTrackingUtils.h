// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "HAL/Platform.h"
#include "HAL/CriticalSection.h"

#define UE_API MUTABLERUNTIME_API

#ifndef UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK
	#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT || UE_BUILD_TEST
		#define UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK 1
	#else
		#define UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK 0
	#endif
#endif

namespace UE::Mutable::Private
{
#if UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK 
	struct FGlobalMemoryCounter
	{
	private:
		 static inline SSIZE_T AbsoluteCounter   { 0 };
		 static inline SSIZE_T AbsolutePeakValue { 0 };
		 static inline SSIZE_T Counter           { 0 };
		 static inline SSIZE_T PeakValue         { 0 };
		
		 static inline FCriticalSection Mutex {};

	public:
		 static UE_API void Update(SSIZE_T Differential);
		 static UE_API void Zero();
		 static UE_API void Restore();
		 static UE_API SSIZE_T GetPeak();
		 static UE_API SSIZE_T GetCounter();
		 static UE_API SSIZE_T GetAbsolutePeak();
		 static UE_API SSIZE_T GetAbsoluteCounter();
	};
#else
	struct FGlobalMemoryCounter
	{
		 //static void Update(SSIZE_T Differential);
		 static UE_API void Zero();
		 static UE_API void Restore();
		 static UE_API SSIZE_T GetPeak();
		 static UE_API SSIZE_T GetCounter();
		 static UE_API SSIZE_T GetAbsolutePeak();
		 static UE_API SSIZE_T GetAbsoluteCounter();
	};
#endif
}

#undef UE_API
