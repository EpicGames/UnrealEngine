// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "Math/Transform.h"
#include "Math/VectorRegister.h"
#include "Math/Rotator.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "MuR/Layout.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/PhysicsBody.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Skeleton.h"
#include "MuR/PassthroughObject.h"
#include "MuR/ManagedPointer.h"
#include "MuR/Cloth.h"
#include "Templates/Tuple.h"
#include "UObject/StrongObjectPtr.h"

#include <type_traits>


#define UE_API MUTABLERUNTIME_API

class UAnimInstance;
class FString;
class USkeletalMesh;
class UPhysicsAsset;
class USkeleton;

namespace UE::Mutable::Private
{

	// Forward references
	class FMesh;
	class FLayout;
	class FSkeleton;
    class FPhysicsBody;

	struct FSurfaceSubMesh
	{		
		int32 VertexBegin = 0;
		int32 VertexEnd = 0;
		int32 IndexBegin = 0;
		int32 IndexEnd = 0;
		
		friend bool operator==(const FSurfaceSubMesh& Lhs, const FSurfaceSubMesh& Rhs)
		{
			return FMemory::Memcmp(&Lhs, &Rhs, sizeof(FSurfaceSubMesh)) == 0;
		}
	};
	static_assert(std::has_unique_object_representations_v<FSurfaceSubMesh>);
	
	MUTABLE_DEFINE_POD_SERIALISABLE(FSurfaceSubMesh);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FSurfaceSubMesh);

	enum class EMorphUsageFlags : uint8
	{
		None     = 0,

		Baked    = 1 << 0,
		RealTime = 1 << 1,
		External = 1 << 2,

		Merged   = RealTime | External,

		AllFlags = 0xFF,
	};

	ENUM_CLASS_FLAGS(EMorphUsageFlags);
	MUTABLE_DEFINE_POD_SERIALISABLE(EMorphUsageFlags);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(EMorphUsageFlags);

	struct FMeshSurface
	{
		TArray<FSurfaceSubMesh, TInlineAllocator<1>> SubMeshes; 

		uint32 BoneMapIndex = 0;
		uint32 BoneMapCount = 0;
		uint32 Id = 0;
		bool bCastShadow = false;
		bool bRecomputeTangent = false;


		friend bool operator==(const FMeshSurface& Lhs, const FMeshSurface& Rhs)
		{
			return
				Lhs.Id == Rhs.Id &&
				Lhs.BoneMapIndex == Rhs.BoneMapIndex &&
				Lhs.BoneMapCount == Rhs.BoneMapCount &&
				Lhs.SubMeshes == Rhs.SubMeshes &&
				Lhs.bCastShadow == Rhs.bCastShadow &&
				Lhs.bRecomputeTangent == Rhs.bRecomputeTangent;
		}

		inline void Serialise(FOutputArchive& Arch) const;
		inline void Unserialise(FInputArchive& Arch);
	};

	/** Helper structs for mesh utility methods below. */
	struct FTriangleInfo
	{
		/** Vertex indices in the original mesh. */
		uint32 Indices[3];

		/** Vertex indices in the collapsed vertex list of the mesh. */
		uint32 CollapsedIndices[3];

		/** Optional data with layout block indices. */
		uint16 BlockIndices[3];

		/** Optional data with a flag indicating the UVs have changed during layout for this trioangle. */
		bool bUVsFixed = false;
	};


	struct FMeshMorph 
	{
		TArray<FName> Names;
		
		TArray<FVector4f> MaximumValuePerMorph;
		TArray<FVector4f> MinimumValuePerMorph;
		TArray<uint32> BatchStartOffsetPerMorph;
		TArray<uint32> BatchesPerMorph;
		
		TArray<TArray<int32, TInlineAllocator<1>>> SurfacesInUsePerMorph;

		TArray<EMorphUsageFlags> UsageFlagsPerMorph;

		uint32 NumTotalBatches = 0;
		float PositionPrecision = 0.0f; // TODO MORPHS Due to merging, this must be per block
		float TangentZPrecision = 0.0f; // TODO MORPHS Due to merging, this must be per block

		bool operator==(const FMeshMorph&) const = default;

		void Serialise(FOutputArchive&) const;
		void Unserialise(FInputArchive&);
	};


	struct FSkinWeightProfile
	{
		FName Name;
		
		bool bDefaultProfile = false;
		int8 DefaultProfileFromLODIndex = 0;

		uint8 NumBoneInfluences   = 0;
		bool bUse16BitBoneIndex  = false;
		bool bUse16BitBoneWeight = false;

		TArray<uint8> BoneIDs;
		TArray<uint8> BoneWeights;

		struct FVertexInfo
		{
			uint32 VertexIndex;
			uint32 InfluenceOffset;

			bool operator==(const FVertexInfo&) const = default;
		};
		TArray<FVertexInfo> VertexIndexToInfluenceOffset;

		void CopyMetadata(const FSkinWeightProfile& From);

		bool operator==(const FSkinWeightProfile&) const = default;

		void Serialise(FOutputArchive&) const;
		void Unserialise(FInputArchive&);
	};
	static_assert(std::has_unique_object_representations_v<FSkinWeightProfile::FVertexInfo>);

	MUTABLE_DEFINE_POD_SERIALISABLE(FSkinWeightProfile::FVertexInfo);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FSkinWeightProfile::FVertexInfo);

	struct FMeshSocket
	{
		FName SocketName;
		FName BoneName;
		FVector RelativeLocation = FVector::ZeroVector;
		FRotator RelativeRotation = FRotator::ZeroRotator;
		FVector RelativeScale = FVector::ZeroVector;

		bool bForceAlwaysAnimated = false;

		int8 Priority = 0; // Not serialized. AddMetadata may modify it

		bool operator==(const FMeshSocket&) const = default;

		void Serialise(FOutputArchive&) const;
		void Unserialise(FInputArchive&);
	};
	
	/** Fill an array with the indices of all triangles belonging to the same UV island as InFirstTriangle.
	*/
	MUTABLERUNTIME_API void GetUVIsland(TArray<FTriangleInfo>& InTriangles,
		const uint32 InFirstTriangle,
		TArray<uint32>& OutTriangleIndices,
		const TArray<FVector2f>& InUVs,
		const TMultiMap<int32, uint32>& InVertexToTriangleMap);

	/** Create a map from vertices into vertices, collapsing vertices that have the same position. */
	MUTABLERUNTIME_API void MeshCreateCollapsedVertexMap(const FMesh* Mesh, TArray<int32>& CollapsedVertices);


	enum class EBoneUsageFlags : uint32
	{
		None		   = 0,
		Root		   = 1 << 1,
		Skinning	   = 1 << 2,
		SkinningParent = 1 << 3,
		Physics	       = 1 << 4,
		PhysicsParent  = 1 << 5,
		Deform         = 1 << 6,
		DeformParent   = 1 << 7,
		Reshaped       = 1 << 8	
	};

	ENUM_CLASS_FLAGS(EBoneUsageFlags);

	enum EBaseMeshBufferType : uint8
	{
		Position,
		TexCoords,
		Tangent,
		Color,
		Skinning,
		Indices,

		None,
		NumTypes = None
	};

	//!
	enum class EMeshBufferType
	{
		None,
		SkeletonDeformBinding,
		PhysicsBodyDeformBinding,
		PhysicsBodyDeformSelection,
		PhysicsBodyDeformOffsets,
		MeshLaplacianData,
		MeshLaplacianOffsets,
		UniqueVertexMap
	};

	//!
	enum class EShapeBindingMethod : uint32
	{
		ReshapeClosestProject = 0,
		ClipDeformClosestProject = 1,
		ClipDeformClosestToSurface = 2,
		ClipDeformNormalProject = 3	
	};

	enum class EVertexColorUsage : uint32
	{
		None = 0,
		ReshapeMaskWeight = 1,
		ReshapeClusterId = 2
	};
	
	inline uint32 GetTypeHash(const EVertexColorUsage& Usage)
	{
		return GetTypeHash(static_cast<uint32>(Usage));
	}

	enum class EMeshCopyFlags : uint32
	{
		None = 0,
		WithSkeletalMesh = 1 << 1,
		WithSurfaces = 1 << 2,
		WithSkeleton = 1 << 3,
		WithPhysicsBody = 1 << 4,
		WithFaceGroups = 1 << 5,
		WithGameplayTags = 1 << 6,
		WithVertexBuffers = 1 << 7,
		WithIndexBuffers = 1 << 8,
		WithSockets = 1 << 9,
		WithAdditionalBuffers = 1 << 10,
		WithLayouts = 1 << 11,
		WithPoses = 1 << 12,
		WithBoneMap = 1 << 13,
		WithSkeletonIDs = 1 << 14,
		WithAdditionalPhysics = 1 << 15,
		WithAssetUserData = 1 << 16,
		WithMorphData = 1 << 17,
		WithMorphMetaData = 1 << 18,
		WithClothData = 1 << 19,
		WithAnimSlots = 1 << 20,
		WithSkinWeightProfileData = 1 << 21,
		WithSkinWeightProfileMetaData = 1 << 22,

		AllFlags = 0xFFFFFFFF
	};
	
	ENUM_CLASS_FLAGS(EMeshCopyFlags);

	enum class EMeshContentFlags : uint8
	{
		None          = 0,
		GeometryData  = 1 << 0,
		PoseData      = 1 << 1,
		PhysicsData   = 1 << 2,
		MetaData      = 1 << 3,

		LastFlag      = MetaData,
		AllFlags      = (LastFlag << 1) - 1,
	};
	ENUM_CLASS_FLAGS(EMeshContentFlags);

	/** Optimised mesh formats that are identified in some operations to chose a faster version. */
	enum class EMeshFlags : uint32
	{
		None = 0,

		/** The mesh is formatted to be used for planar and cilyndrical projection */
		ProjectFormat = 1 << 0,

		/** The mesh is formatted to be used for wrapping projection */
		ProjectWrappingFormat = 1 << 1,
	};


    /** Mesh object containing any number of buffers with any number of channels.
	* The buffers can be per-index or per-vertex.
    * The mesh also includes layout information for every texture channel for internal usage, and it can be ignored.
    * The meshes are always assumed to be triangle list primitives.
    */
    class FMesh : public FResource
    {
	public:

		static constexpr uint64 InvalidVertexId = TNumericLimits<uint64>::Max();

	public:

        //-----------------------------------------------------------------------------------------
        // Life cycle
        //-----------------------------------------------------------------------------------------
				
		/** Create a new empty mesh that repreents an external resource mesh. */
		static UE_API TManagedRef<FMesh> CreateAsReference(const TPassthroughObjectPtr<USkeletalMesh>& PasstroughObjectPtr);

        /** Deep clone this mesh. */
        UE_API TManagedPtr<FMesh> Clone() const;
		
		/** Clone with flags allowing to not include some parts in the cloned mesh */
		UE_API TManagedPtr<FMesh> Clone(EMeshCopyFlags Flags) const;

		/** Copy form another mesh. */
		UE_API void CopyFrom(const FMesh& From, EMeshCopyFlags Flags = EMeshCopyFlags::AllFlags);

        /** Serialisation */
        static UE_API void Serialise(const FMesh* InMesh, FOutputArchive& Arch );
        static UE_API TManagedPtr<FMesh> StaticUnserialise(FInputArchive& Arch);

		// Resource interface
		UE_API int32 GetDataSize() const override;

        //-----------------------------------------------------------------------------------------
        // Own interface
        //-----------------------------------------------------------------------------------------

		/** Return true if this is a reference to an engine image. */
		UE_API bool IsReference() const;
    	
		/** Mesh references can actually reference a morphed mesh. In that case you can set and get the morph with these functions. */
		UE_API void SetReferencedMorph(const FString& MorphName);
		UE_API const FString& GetReferencedMorph() const;

        //! \name Buffers
        //! \{

        //!
        UE_API int32 GetIndexCount() const;

        /** Index buffers. They are owned by this mesh. */
        UE_API FMeshBufferSet& GetIndexBuffers();
        UE_API const FMeshBufferSet& GetIndexBuffers() const;

        //
        UE_API int32 GetVertexCount() const;

        /** Vertex buffers. They are owned by this mesh. */
        UE_API FMeshBufferSet& GetVertexBuffers();
        UE_API const FMeshBufferSet& GetVertexBuffers() const;

        UE_API int32 GetFaceCount() const;

        /**
		 * Get the number of surfaces defined in this mesh. Surfaces are buffer-contiguous mesh
         * fragments that share common properties (usually material)
		 */
        UE_API int32 GetSurfaceCount() const;
        UE_API void GetSurface(int32 SurfaceIndex,
                        int32& OutFirstVertex, int32& OutVertexCount,
                        int32& OutFirstIndex, int32& OutIndexCount,
						int32& OutFirstBone, int32& OutBoneCount) const;

        /**
		 * Return an internal id that can be used to match mesh surfaces and instance surfaces.
         * Only valid for meshes that are part of instances.
		 */
        UE_API uint32 GetSurfaceId(int32 SurfaceIndex) const;

        //! \}

		/** Return true if the mesh has unique vertex IDs and they stored in an implicit way. 
		* This is relevant for some mesh operations that will need to make them explicit so that the result is still correct.
		*/
		UE_API bool AreVertexIdsImplicit() const;
		UE_API bool AreVertexIdsExplicit() const;

		/** Create an explicit vertex buffer for vertex IDs if they are implicit. */
		UE_API void MakeVertexIdsRelative();

		/** Ensure the format of an empty mesh includes explicit IDs. The mesh cannot have any vertex data. */
		UE_API void MakeIdsExplicit();

        //! \name Texture layouts
        //! \{

        //!
        UE_API void AddLayout( TManagedPtr<const FLayout> );

        //!
        UE_API int32 GetLayoutCount() const;

        //!
		UE_API TManagedPtr<const FLayout> GetLayout( int32 Index) const;

        //!
        UE_API void SetLayout( int32 Index, TManagedPtr<const FLayout> );
        //! \}

        //! \name Skeleton information
        //! \{

        UE_API void SetSkeleton(const TManagedPtr<const FSkeleton>&);
		UE_API TManagedPtr<const FSkeleton> GetSkeleton() const;

		UE_API void ConvertBoneIdsToIndices();

        //! \}

        //! \name PhysicsBody information
        //! \{

        UE_API void SetPhysicsBody(const TManagedPtr<const FPhysicsBody>&);
		UE_API TManagedPtr<const FPhysicsBody> GetPhysicsBody() const;

		UE_API int32 AddAdditionalPhysicsBody(const TManagedPtr<const FPhysicsBody>&);
		UE_API TManagedPtr<const FPhysicsBody> GetAdditionalPhysicsBody(int32 I) const;
		//int32 GetAdditionalPhysicsBodyExternalId(int32 I) const;
    	
		//!
		UE_API int32 FindBonePoseByBoneIndex(int32 Index) const;
		
		//!
		UE_API void SetBonePoseCount(int32 Count);

		//!
		UE_API int32 GetBonePoseCount() const;

		//!
		UE_API void SetBonePose(int32 Index, FBoneIdOrIndex BoneName, FTransform3f Transform, EBoneUsageFlags BoneUsageFlags, int8 Priority = 0);

		//! @return - Bone identifier of the pose at 'Index'.
		UE_API FBoneIdOrIndex GetBonePoseId(int32 BoneIndex) const;

		//! Return a matrix stored per bone. It is a set of 16-float values.
		UE_API void GetBonePoseTransform(int32 BoneIndex, FTransform3f& Transform) const;

		//! 
		UE_API EBoneUsageFlags GetBoneUsageFlags(int32 BoneIndex) const;

		//! Set the bonemap of this mesh
		UE_API void SetBoneMap(const TArray<FBoneIdOrIndex>& InBoneMap);

		//! Return an array containing the bonemaps of all surfaces in the mesh.
		UE_API const TArray<FBoneIdOrIndex>& GetBoneMap() const;

        /** 
		 * Get an internal identifier used to reference this mesh in operations like deferred
         * mesh building, or instance updating.
		 */
        UE_API uint32 GetId() const;

    	UE_API bool HasMorphs() const;

    	UE_API bool HasClothing() const;
    	
		template<typename Type>
		using TMemoryTrackedArray = TArray<Type, FDefaultMemoryTrackingAllocator<MemoryCounters::FMeshMemoryCounter>>;

		/** Non-persistent internal id unique for a mesh generated for a specific state and parameter values. */
		mutable uint32 InternalId = 0;

		/** 
		 * This is bit - mask on the EMeshFlags enumeration, marking what static formats are compatible with this one and other properties. 
		 * It should be reset after any operation that modifies the format.
		 */
		mutable EMeshFlags Flags = EMeshFlags::None;

		/** Passthrough Id or TStrongObjectPtr. */
		TPassthroughObjectPtr<USkeletalMesh> PassthroughObject; // TODO GMT It should not be a TPassthroughObject but instead directly UObject.

		/** If the mesh is a reference the referenced morph name is stored here. Otherwise it is an empty string. */
		FString ReferencedMorph;

		/** 
		 * Prefix for the unique IDs related to this mesh (vertices and layout blocks). Useful if the mesh stores them in an implicit, or relative way. 
		 * See MeshVertexIdIterator for details.
		 */
		uint32 MeshIDPrefix = 0;

		FMeshBufferSet VertexBuffers;

		FMeshBufferSet IndexBuffers;
    	
    	TArray<uint32> MorphDataBuffer;

		/** Additional buffers used for temporary or custom data in different algorithms. */
		TArray<TPair<EMeshBufferType, FMeshBufferSet>> AdditionalBuffers;

		TArray<FMeshSurface> Surfaces;

    	FMeshMorph Morph;
    	
    	/** Only valid if the mesh is a LOD (has multiple sections). */
    	TArray<FCloth> ClothSections;
    	
		/** SkinWeight Profiles */
		TArray<FSkinWeightProfile> SkinWeightProfiles;

		/** Externally provided skeletons required by this mesh. */
		TArray<TPassthroughObjectPtr<USkeleton>> SkeletonObjects;

		/** 
		 * This skeleton and physicsbody are not owned and may be used by other meshes, so it cannot be modified
		 * once the mesh has been fully created.
		 */
		TManagedPtr<const FSkeleton> Skeleton;
		TManagedPtr<const FPhysicsBody> PhysicsBody;
    	
		/** Additional physics bodies referenced by the mesh that don't merge. */
		TArray<TManagedPtr<const FPhysicsBody>> AdditionalPhysicsBodies;

		/** Physics assets referenced by the mesh to be merged. */
		TArray<TPassthroughObjectPtr<UPhysicsAsset>> PhysicsAssets;

		/** Additional physics assets referenced by the mesh that don't merge. */
		TArray<TPassthroughObjectPtr<UPhysicsAsset>> AdditionalPhysicsAssets;

		/** 
		 * Texture Layout blocks attached to this mesh. They are const because they could be shared with
		 * other meshes, so they need to be cloned and replaced if a modification is needed.
		 */
		TArray<TManagedPtr<const FLayout>> Layouts;		

		TArray<FName> GameplayTags;

		TArray<TPassthroughObjectPtr<UAssetUserData>> AssetUserData;
		
    	TArray<TPair<FName, TPassthroughObjectPtr<UClass>>> AnimationSlots;

		struct FBonePose
		{
			// Identifier built from the bone FName.
			FBoneIdOrIndex BoneId;

			EBoneUsageFlags BoneUsageFlags = EBoneUsageFlags::None;
			
			// Not serialized. Add Metadata may overwrite it. 
			int8 BonePosePriority = 0;

			FTransform3f BoneTransform;

			inline void Serialise(FOutputArchive& arch) const;
			inline void Unserialise(FInputArchive& arch);

			inline bool operator==(const FBonePose& Other) const
			{
				return BoneUsageFlags == Other.BoneUsageFlags &&
				BoneId == Other.BoneId &&
				BonePosePriority == Other.BonePosePriority &&
				BoneTransform.Equals(Other.BoneTransform);
			}
		};

		/** 
		 * This is the pose used by this mesh fragment, used to update the transforms of the final skeleton
		 * taking into consideration the meshes being used.
		 */
		TMemoryTrackedArray<FBonePose> BonePoses;

		/** Array containing the bonemaps of all surfaces in the mesh. */
		TArray<FBoneIdOrIndex> BoneMap;
    	
		/** Array containing the sockets of this mesh. Not serialized. Added with Metadata */
		TArray<FMeshSocket> Sockets;

		inline void Serialise(FOutputArchive& arch) const;

		inline void Unserialise(FInputArchive& arch);


		inline bool operator==(const FMesh& Other) const
		{
			bool bEqual = true;

			if (bEqual) bEqual = (PassthroughObject == Other.PassthroughObject);
			if (bEqual) bEqual = (MeshIDPrefix == Other.MeshIDPrefix);
			if (bEqual) bEqual = (IndexBuffers == Other.IndexBuffers);
			if (bEqual) bEqual = (VertexBuffers == Other.VertexBuffers);
			if (bEqual) bEqual = (Layouts.Num() == Other.Layouts.Num());
			if (bEqual) bEqual = (BonePoses.Num() == Other.BonePoses.Num());
			if (bEqual) bEqual = (BoneMap.Num() == Other.BoneMap.Num());
			if (bEqual && Skeleton != Other.Skeleton)
			{
				if (Skeleton && Other.Skeleton)
				{
					bEqual = (*Skeleton == *Other.Skeleton);
				}
				else
				{
					bEqual = false;
				}
			}
			if (bEqual && PhysicsBody != Other.PhysicsBody)
			{
				if (PhysicsBody && Other.PhysicsBody)
				{
					bEqual = (*PhysicsBody == *Other.PhysicsBody);
				}
				else
				{
					bEqual = false;
				}
			}
			if (bEqual) bEqual = (AssetUserData == Other.AssetUserData);
			if (bEqual) bEqual = (AnimationSlots == Other.AnimationSlots);
			if (bEqual) bEqual = (Surfaces == Other.Surfaces);
			if (bEqual) bEqual = (GameplayTags == Other.GameplayTags);
			if (bEqual) bEqual = (SkeletonObjects == Other.SkeletonObjects);

			for (int32 i = 0; bEqual && i < Layouts.Num(); ++i)
			{
				bEqual &= (*Layouts[i]) == (*Other.Layouts[i]);
			}

			bEqual &= AdditionalBuffers.Num() == Other.AdditionalBuffers.Num();
			for (int32 i = 0; bEqual && i < AdditionalBuffers.Num(); ++i)
			{
				bEqual &= AdditionalBuffers[i] == Other.AdditionalBuffers[i];
			}

			bEqual &= BonePoses.Num() == Other.BonePoses.Num();
			for (int32 i = 0; bEqual && i < BonePoses.Num(); ++i)
			{
				bEqual &= BonePoses[i] == Other.BonePoses[i];
			}

			bEqual &= Sockets == Other.Sockets;

			if (bEqual) bEqual = BoneMap == Other.BoneMap;

			bEqual &= AdditionalPhysicsBodies.Num() == Other.AdditionalPhysicsBodies.Num();
			for (int32 i = 0; bEqual && i < AdditionalPhysicsBodies.Num(); ++i)
			{
				bEqual &= *AdditionalPhysicsBodies[i] == *Other.AdditionalPhysicsBodies[i];
			}

			bEqual = bEqual && ClothSections == Other.ClothSections;
			bEqual = bEqual && Morph == Other.Morph;
			bEqual = bEqual && MorphDataBuffer == Other.MorphDataBuffer;
			bEqual = bEqual && SkinWeightProfiles == Other.SkinWeightProfiles;

			return bEqual;
		}

		/** Compare the mesh with another one, but ignore internal data like generated vertex indices. */
		UE_API bool IsSimilar(const FMesh& Other) const;


		/**
		 * Make a map from the vertices in this mesh to thefirst matching vertex of the given
		 * mesh. If non is found, the index is set to -1.
		 */
		struct FVertexMatchMap
		{
			/** One for every vertex */
			TArray<int32> FirstMatch;

			/** The matches of every vertex in a sequence */
			TArray<int32> Matches;

			bool DoMatch(int32 Vertex, int32 OtherVertex) const;
		};

		UE_API void GetVertexMap(const FMesh& Other, FVertexMatchMap& VertexMap, float Tolerance = 1e-3f) const;

		/** Compare the vertex attributes to check if they match. */
		UE_API UE::Math::TIntVector3<uint32> GetFaceVertexIndices(int32 FaceIndex) const;

		/** 
		 * Return true if the given mesh has the same vertex and index formats, and in the same
		 * buffer structure.
		 */
		UE_API bool HasCompatibleFormat(const FMesh* Other) const;

		/** Update the flags identifying the mesh format as some of the optimised formats. */
		UE_API void ResetStaticFormatFlags() const;

		/** Create the surface data if not present. */
		UE_API void EnsureSurfaceData();

		/** Check mesh buffer data for possible inconsistencies */
		UE_API void CheckIntegrity() const;

		/** Return true if this mesh is closed. It is still closed if it has dangling vertices. */
		UE_API bool IsClosed() const;

		/** 
		 * Change the buffer descriptions so that all buffer indices start at 0 and are in the
		 * same order than memory.
		 */
		UE_API void ResetBufferIndices();

		/** Debug: get a text representation of the mesh */
		UE_API void Log(FString& Out, int32 VertexLimit) const;
    };


	MUTABLE_DEFINE_ENUM_SERIALISABLE(EBoneUsageFlags)
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EBaseMeshBufferType)
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EMeshBufferType)
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EShapeBindingMethod)
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EVertexColorUsage)
}

#undef UE_API
