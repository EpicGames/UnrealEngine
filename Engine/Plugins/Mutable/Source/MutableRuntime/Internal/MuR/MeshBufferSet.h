// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Serialisation.h"
#include "Containers/Array.h"
#include "HAL/PlatformMath.h"

#include "MuR/MemoryTrackingAllocationPolicy.h"

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private::MemoryCounters
{
	struct FMeshMemoryCounter
	{
		static MUTABLERUNTIME_API std::atomic<SSIZE_T>& Get();
	};
}

namespace UE::Mutable::Private
{
	/** Supported formats for the elements in mesh buffers. **/
	enum class EMeshBufferFormat : uint32
	{
		None,

		Float16,
		Float32,

		UInt8,
		UInt16,
		UInt32,
		Int8,
		Int16,
		Int32,

		/** Integers interpreted as being in the range 0.0f to 1.0f */
		NUInt8,
		NUInt16,
		NUInt32,

		/** Integers interpreted as being in the range -1.0f to 1.0f */
		NInt8,
		NInt16,
		NInt32,

        /** Packed 1 to -1 value using multiply+add (128 is almost zero). Use 8-bit unsigned ints. */
        PackedDir8,

        /** 
		 * Same as EMeshBufferFormat::PackedDir8, with the w component replaced with the sign of the determinant
         * of the vertex basis to define the orientation of the tangent space in UE4 format.
         * Use 8-bit unsigned ints.
		*/
        PackedDir8_W_TangentSign,

        /** Packed 1 to -1 value using multiply+add (128 is almost zero). Use 8-bit signed ints. */
        PackedDirS8,

        /** 
		 * Same as EMeshBufferFormat::PackedDirS8, with the w component replaced with the sign of the determinant
         * of the vertex basis to define the orientation of the tangent space in UE4 format.
         * Use 8-bit signed ints.
		 */
        PackedDirS8_W_TangentSign,

		Float64,
		UInt64,
		Int64,
		NUInt64,
		NInt64,

		Count,
	};

	/** */
	struct FMeshBufferFormatData
	{
		/** Size per component in bytes. */
		uint8 SizeInBytes;

		/** log 2 of the max value if integer. */
		uint8 MaxValueBits;
	};

	MUTABLERUNTIME_API const FMeshBufferFormatData& GetMeshFormatData(EMeshBufferFormat Format);


	/** Semantics of the mesh buffers */
	enum class EMeshBufferSemantic : uint32
	{
		None,

		/** For index buffers, and mesh morphs */
		VertexIndex,

		/** Standard vertex semantics */
		Position,
		Normal,
		Tangent,
		Binormal,
		TexCoords,
		Color,
		BoneWeights,
		BoneIndices,

		/**
		 * Internal semantic indicating what layout block each vertex belongs to.
		 * It can be safely ignored if present in meshes returned by the system.
		 * It will never be in the same buffer that other vertex semantics.
		 */
		LayoutBlock,

		_DEPRECATED,

		/** 
		 * To let users define channels with semantics unknown to the system.
		 * These channels will never be transformed, and the per-vertex or per-index data will be
		 * simply copied.
		 */
		Other,

        _DEPRECATED2,

		/** Semantics usefule for mesh binding. */
		TriangleIndex,
		BarycentricCoords,
		Distance,

		AltSkinWeight_DEPRECATED,

		/** Utility */
		Count,
	};


	/** */
	struct FMeshBufferChannel
	{
		EMeshBufferSemantic Semantic = EMeshBufferSemantic::None;

		EMeshBufferFormat Format = EMeshBufferFormat::None;

		/** Index of the semantic, in case there are more than one of this type. */
		int32 SemanticIndex = 0;

		/** Offset in bytes from the begining of a buffer element */
		uint16 Offset = 0;

		/** Number of components of the type in Format for every value in the channel */
		uint16 ComponentCount = 0;

		inline bool operator==(const FMeshBufferChannel& Other) const
		{
			return (Semantic == Other.Semantic) &&
				(Format == Other.Format) &&
				(SemanticIndex == Other.SemanticIndex) &&
				(Offset == Other.Offset) &&
				(ComponentCount == Other.ComponentCount);
		}

	};


	struct FMeshBuffer
	{
		template<typename Type>
		using TMemoryTrackedArray = TArray<Type, FDefaultMemoryTrackingAllocator<MemoryCounters::FMeshMemoryCounter>>;

		TArray<FMeshBufferChannel> Channels;
		TMemoryTrackedArray<uint8> Data;
		uint32 ElementSize = 0;

		void Serialise(UE::Mutable::Private::FOutputArchive& Arch) const;
		inline void Unserialise(UE::Mutable::Private::FInputArchive& Arch);

		inline bool operator==(const FMeshBuffer& Other) const
		{
			bool bEqual = (Channels == Other.Channels);
			bEqual = bEqual && (ElementSize == Other.ElementSize);
			bEqual = bEqual && (Data == Other.Data);

			return bEqual;
		}

		/** Return true if the buffer has any channel with the passed semantic. */
		inline bool HasSemantic(EMeshBufferSemantic Semantic) const
		{
			for (const FMeshBufferChannel& Channel : Channels)
			{
				if (Channel.Semantic == Semantic)
				{
					return true;
				}
			}
			return false;
		}

		inline bool HasSameFormat(const FMeshBuffer& Other) const
		{
			return (Channels == Other.Channels && ElementSize == Other.ElementSize);
		}

		inline bool HasPadding() const
		{
			uint32 ActualElementSize = 0;
			for (const FMeshBufferChannel& Channel : Channels)
			{
				ActualElementSize += Channel.ComponentCount * GetMeshFormatData(Channel.Format).SizeInBytes;
			}
			check(ActualElementSize <= ElementSize);
			return ActualElementSize < ElementSize;
		}
	};

	enum class EMeshBufferSetFlags : uint32
	{
		None = 0,
		IsDescriptor = 1 << 0
	};

	ENUM_CLASS_FLAGS(EMeshBufferSetFlags);

	/** Set of buffers storing mesh element data. Elements can be vertices, indices or faces. */
	class FMeshBufferSet
	{
	public:

		uint32 ElementCount = 0;
		EMeshBufferSetFlags Flags = EMeshBufferSetFlags::None; 
		TArray<FMeshBuffer> Buffers;

		UE_API void Serialise(FOutputArchive& Arch) const;
		UE_API void Unserialise(FInputArchive& Arch);

		inline bool operator==(const FMeshBufferSet& Other) const
		{
			return 
				ElementCount == Other.ElementCount && 
				Buffers == Other.Buffers && 
				Flags == Other.Flags;
		}

	public:

		/** Get the number of elements in the buffers */
		UE_API int32 GetElementCount() const;

		/** 
		* Set the number of vertices in the mesh. This will resize the vertex buffers keeping the
		* previous data when possible. New data content is defined by MemoryInitPolicy.
		*/
		UE_API void SetElementCount(int32 Count, EMemoryInitPolicy MemoryInitPolicy = EMemoryInitPolicy::Uninitialized);

		/**
		* Get the size in bytes of a buffer element.
		* @param buffer index of the buffer from 0 to GetBufferCount()-1
		*/
		UE_API int32 GetElementSize(int32 Buffer) const;

		/** Get the number of vertex buffers in the mesh */
		UE_API int32 GetBufferCount() const;

		/** Set the number of vertex buffers in the mesh. */
		UE_API void SetBufferCount(int32 Count);

		/**
		* Get the number of channels in a vertex buffer.
		* \param buffer index of the vertex buffer from 0 to GetBufferCount()-1
		*/
		UE_API int32 GetBufferChannelCount(int32 BufferIndex) const;

		/**
		 * Get a channel of a buffer by index
		 * \param buffer index of the vertex buffer from 0 to GetBufferCount()-1
		 * \param channel index of the channel from 0 to GetBufferChannelCount( buffer )-1
		 * \param[out] pSemantic semantic of the channel
		 * \param[out] pSemanticIndex index of the semantic in case of having more than one of the
		 *				same type.
		 * \param[out] pFormat data format of the channel
		 * \param[out] pComponentCount components of an element of the channel
		 * \param[out] pOffset offset in bytes from the beginning of an element of the buffer
		 */
		UE_API void GetChannel(
				int32 BufferIndex,
				int32 ChannelIndex,
				EMeshBufferSemantic* SemanticPtr,
				int32* SemanticIndexPtr,
				EMeshBufferFormat* FormatPtr,
				int32* ComponentCountPtr,
				int32* OffsetPtr
			) const;

		/** 
		 * Set all the channels of a buffer
		 * \param buffer index of the buffer from 0 to GetBufferCount()-1
         * \param elementSize sizei n bytes of a vertex element in this buffer
         * \param channelCount number of channels to set in the buffer
         * \param pSemantics buffer of channelCount semantics
         * \param pSemanticIndices buffer of indices for the semantic of every channel
         * \param pFormats buffer of channelCount formats
         * \param pComponentCounts buffer of channelCount component counts
         * \param pOffsets offsets in bytes of every particular channel inside the buffer element
		 */ 
        UE_API void SetBuffer(
				int32 BufferIndex,
				int32 ElementSize,
				int32 ChannelCount,
				const EMeshBufferSemantic* SemanticsPtr = nullptr,
				const int32* SemanticIndicesPtr = nullptr,
				const EMeshBufferFormat* FormatsPtr = nullptr,
				const int32* ComponentCountsPtr = nullptr,
				const int32* OffsetsPtr = nullptr,
				EMemoryInitPolicy MemoryInitPolicy = EMemoryInitPolicy::Uninitialized);

		/**
		 * Set one  channels of a buffer
		 * \param buffer index of the buffer from 0 to GetBufferCount()-1
         * \param elementSize sizei n bytes of a vertex element in this buffer
         * \param channelIndex number of channels to set in the buffer
		 */
        UE_API void SetBufferChannel(
				int32 BufferIndex,
				int32 ChannelIndex,
				EMeshBufferSemantic Semantic,
				int32 SemanticIndex,
				EMeshBufferFormat Format,
				int32 ComponentCount,
				int32 Offset
			);

		/** 
		 * Get a pointer to the object-owned data of a buffer.
		 * Channel data is interleaved for every element and packed in the order it was set
		 * without any padding.
		 * \param buffer index of the buffer from 0 to GetBufferCount()-1
		 * \todo Add padding support for better alignment of buffer elements.
		*/
        UE_API uint8* GetBufferData(int32 Buffer);
		UE_API const uint8* GetBufferData(int32 Buffer) const;
		UE_API uint32 GetBufferDataSize(int32 Buffer) const;

		/** Utility methods */

		/** 
		 * Find the index of a buffer channel by semantic and relative index inside the semantic.
         * \param semantic Semantic of the channel we are searching.
         * \param semanticIndex Index of the semantic of the channel we are searching. e.g. if we
         *         want the second set of texture coordinates, it should be 1.
         * \param[out] pBuffer -1 if the channel is not found, otherwise it will contain the index
		 * of the buffer where the channel was found.
		 * \param[out] pChannel -1 if the channel is not found, otherwise it will contain the
		 * channel index of the channel inside the buffer returned at [buffer]
		 */
		UE_API void FindChannel(EMeshBufferSemantic Semantic, int32 SemanticIndex, int32* BufferPtr, int32* ChannelPtr) const;

		/**
		 * Get the offset in bytes of the data of this channel inside an element data.
		 * \param buffer index of the buffer from 0 to GetBufferCount()-1
		 * \param channel index of the channel from 0 to GetBufferChannelCount( buffer )-1
		 */ 
		UE_API int32 GetChannelOffset(int32 Buffer, int32 Channel) const;

		/**
		 * Add a new buffer by cloning a buffer from another set.
		 * The buffers must have the same number of elements.
		 */ 
		UE_API void AddBuffer( const FMeshBufferSet& Other, int32 BufferIndex);

		/** Return true if the formats of the two vertex buffers set match. **/
		UE_API bool HasSameFormat(const FMeshBufferSet& Other) const;

		/** 
		 * Remove the buffer at the specified position. This "invalidates" any buffer index that
		 * was referencing buffers after the removed one.
		 */ 
		UE_API void RemoveBuffer(int32 BufferIndex);

	public:

		/**
		 * Copy an element from one position to another, overwriting the other element.
		 * Both positions must be valid, buffer size won't change.
		 */ 
		UE_API void CopyElement(uint32 FromIndex, uint32 ToIndex);

		/** Compare the format of the two buffers at index buffer and return true if they match. **/
		UE_API bool HasSameFormat(int32 ThisBufferIndex, const FMeshBufferSet& pOther, int32 OtherBufferIndex) const;

		/** Get the total memory size of the buffers and this struct */
		UE_API int32 GetDataSize() const;

		/** */
		UE_API int32 GetAllocatedSize() const;

		/** 
		 * Compare the mesh buffer with another one, but ignore internal data like generated
		 * vertex indices.
		 */ 
		UE_API bool IsSpecialBufferToIgnoreInSimilar(const FMeshBuffer& Buffer) const;

		/** 
		 * Compare the mesh buffer with another one, but ignore internal data like generated
		 * vertex indices. Be aware this method compares the data byte by byte without checking
		 * if the data belong to the buffer components and could give false negatives if unset 
		 * padding data is present.
		 */ 
		UE_API bool IsSimilar(const FMeshBufferSet& Other) const;

		/** 
		 * Compare the mesh buffer with another one, but ignore internal data like generated
		 * vertex indices. This version compares the data component-wise, skipping any memory
		 * not specified in the buffer description.
		 */
		UE_API bool IsSimilarRobust(const FMeshBufferSet& Other, bool bCompareUVs) const;

		/** 
		 * Change the buffer descriptions so that all buffer indices start at 0 and are in the
		 * same order than memory.
		 */
		UE_API void ResetBufferIndices();

		/** */
		UE_API void UpdateOffsets(int32 BufferIndex);
		
		/** Check that all channels of a specific semantic use the provided format. */
		UE_API bool HasAnySemanticWithDifferentFormat(EMeshBufferSemantic Semantic, EMeshBufferFormat ExpectedFormat) const;

		/** Check if this buffer set is only a format descriptor, e.i. reports num elements larger than 0 but does not hold any data*/ 
		UE_API bool IsDescriptor() const;
	};	

	
	MUTABLE_DEFINE_POD_SERIALISABLE(FMeshBufferChannel);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FMeshBufferChannel);
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EMeshBufferFormat);
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EMeshBufferSemantic);
	MUTABLE_DEFINE_ENUM_SERIALISABLE(EMeshBufferSetFlags);
}

#undef UE_API
