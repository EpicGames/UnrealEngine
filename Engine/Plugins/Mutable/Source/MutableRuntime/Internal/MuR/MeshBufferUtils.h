// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshBufferSet.h"

#define UE_API MUTABLERUNTIME_API 

/** Publicly accessible methods designed to prepare the Mutable Mesh buffers with the internal structure  required by Unreal and
 * our systems to be able to work with the mesh owning those buffers.
 */
namespace UE::Mutable::Private
{
	class FMeshBufferSet;

namespace  MeshBufferUtils
{
	// VERTEX BUFFERS

	/** Determines the max amount of channels that can inhabit a Texture Coordinate buffer. */
	constexpr int32 MaxTexCordChannelCount = 4;

	/**
	 * Sets up all the channels that ought to be part of the Vertex Position Buffer of the Mutable mesh owning the provided MeshBufferSet
	 * @param InCurrentVertexBuffer Index of the Vertex Buffer being set up inside the OutTargetVertexBuffers.
	 * @param OutTargetVertexBuffers Mesh buffer to be set up.
	 */
	UE_API void SetupVertexPositionsBuffer(const int32& InCurrentVertexBuffer, FMeshBufferSet& OutTargetVertexBuffers);

	/**
	 * Sets up all the channels that ought to be part of the Vertex Tangent Buffer of the Mutable mesh owning the provided MeshBufferSet
	 * @param InCurrentVertexBuffer Index of the Vertex Buffer being set up inside the OutTargetVertexBuffers.
	 * @param OutTargetVertexBuffers Mesh buffer to be set up.
	 */
	UE_API void SetupTangentBuffer(const int32& InCurrentVertexBuffer, FMeshBufferSet& OutTargetVertexBuffers);

	/**
	 * Sets up all the channels that ought to be part of the Vertex Texture Coordinates Buffer of the Mutable mesh owning the provided MeshBufferSet
	 * @param InCurrentVertexBuffer Index of the Vertex Buffer being set up inside the OutTargetVertexBuffers.
	 * @param InChannelCount Determines the amount of TexCord channels able to be added onto the buffer.
	 * @param OutTargetVertexBuffers Mesh buffer to be set up.
	 * @param InTextureSemanticIndicesOverride (Optional) If you are certain of the texture semantic indices that should be used populate this argument, if not, leave it as null
	 */
	UE_API void SetupTexCoordinatesBuffer(const int32& InCurrentVertexBuffer, const int32& InChannelCount, bool bHighPrecision, FMeshBufferSet& OutTargetVertexBuffers, const int32* InTextureSemanticIndicesOverride = nullptr);

	/**
	 * Sets up all the channels that ought to be part of the Vertex Skin Buffer of the Mutable mesh owning the provided MeshBufferSet
	 * @param InCurrentVertexBuffer Index of the Vertex Buffer being set up inside the OutTargetVertexBuffers.
	 * @param MaxBoneIndexTypeSizeBytes Maximum size of the vertex bone index type.
	 * @param MaxNumBonesPerVertex Amount of bone influences a vertex can have.
	 * @param OutTargetVertexBuffers Mesh buffer to be set up.
	 */
	UE_API void SetupSkinBuffer(const int32& InCurrentVertexBuffer, const int32& MaxBoneIndexTypeSizeBytes, const int32& MaxBoneWeightTypeSizeBytes,
		const int32& MaxNumBonesPerVertex, FMeshBufferSet& OutTargetVertexBuffers);

	/**
	 * Sets up all the channels that ought to be part of the Vertex Color Buffer of the Mutable mesh owning the provided MeshBufferSet
	 * @param InCurrentVertexBuffer Index of the Vertex Buffer being set up inside the OutTargetVertexBuffers.
	 * @param OutTargetVertexBuffers Mesh buffer to be set up.
	 */
	UE_API void SetupVertexColorBuffer(const int32& InCurrentVertexBuffer, FMeshBufferSet& OutTargetVertexBuffers);

	// INDEX BUFFERS

	/**
	 * Sets up all the channels that ought to be part of the Index Buffer of the Mutable mesh owning the provided MeshBufferSet
	 * @param OutTargetIndexBuffers Index buffer to be set up.
	 */
	UE_API void SetupIndexBuffer(FMeshBufferSet& OutTargetIndexBuffers, EMeshBufferFormat InFormat);
}
}

#undef UE_API