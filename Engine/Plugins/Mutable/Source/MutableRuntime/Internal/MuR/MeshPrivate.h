// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Mesh.h"

#include "MuR/Layout.h"
#include "MuR/Skeleton.h"
#include "MuR/Serialisation.h"
#include "MuR/ConvertData.h"

#include "CoreFwd.h"

namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	inline void GetMeshBuf
		(
			FMesh* Mesh,
			EMeshBufferSemantic Semantic,
			EMeshBufferFormat expectedFormat,
			int32 expectedComponents,
            uint8*& pBuf,
			int32& elemSize
		)
	{
		// Avoid unreferenced parameter warnings
		(void)expectedFormat;
		(void)expectedComponents;

		int32 BufferIndex = -1;
		int32 ChannelIndex = -1;
		Mesh->GetVertexBuffers().FindChannel( Semantic, 0, &BufferIndex, &ChannelIndex );
		check( BufferIndex>=0 && ChannelIndex>=0 );

		EMeshBufferSemantic realSemantic = EMeshBufferSemantic::None;
		int32 realSemanticIndex = 0;
		EMeshBufferFormat format = EMeshBufferFormat::None;
		int32 components = 0;
		int32 offset = 0;
		Mesh->GetVertexBuffers().GetChannel( BufferIndex, ChannelIndex, &realSemantic, &realSemanticIndex, &format, &components, &offset );
		check( realSemantic == Semantic );
		check( format == expectedFormat );
		check( components == expectedComponents );
		elemSize = Mesh->GetVertexBuffers().GetElementSize( BufferIndex );
		pBuf = Mesh->GetVertexBuffers().GetBufferData( BufferIndex );
		pBuf += offset;
	}


	//---------------------------------------------------------------------------------------------
	inline void GetMeshBuf
		(
			const FMesh* Mesh,
			EMeshBufferSemantic Semantic,
			EMeshBufferFormat expectedFormat,
			int32 expectedComponents,
            const uint8*& pBuf,
			int32& elemSize
		)
	{
		// Avoid unreferenced parameter warnings
		(void)expectedFormat;
		(void)expectedComponents;

		int32 BufferIndex = -1;
		int32 ChannelIndex = -1;
		Mesh->GetVertexBuffers().FindChannel( Semantic, 0, &BufferIndex, &ChannelIndex );
		check( BufferIndex>=0 && ChannelIndex>=0 );

		EMeshBufferSemantic realSemantic = EMeshBufferSemantic::None;
		int32 realSemanticIndex = 0;
		EMeshBufferFormat format = EMeshBufferFormat::None;
		int32 components = 0;
		int32 offset = 0;
		Mesh->GetVertexBuffers().GetChannel
				( BufferIndex, ChannelIndex, &realSemantic, &realSemanticIndex, &format, &components, &offset );
		check( realSemantic == Semantic );
		check( format == expectedFormat );
		check( components == expectedComponents );
		elemSize = Mesh->GetVertexBuffers().GetElementSize( BufferIndex );
		pBuf = Mesh->GetVertexBuffers().GetBufferData( BufferIndex );
		pBuf += offset;
	}


	//---------------------------------------------------------------------------------------------
	//! Class to iterate a specific BufferIndex ChannelIndex of unknown type
	//---------------------------------------------------------------------------------------------
	class UntypedMeshBufferIteratorConst;

	class UntypedMeshBufferIterator 
	{
	public:
		inline UntypedMeshBufferIterator()
		{
			Format = EMeshBufferFormat::None;
			Components = 0;
			ElementSize = 0;
			Buffer = nullptr;
		}


		inline UntypedMeshBufferIterator
			(
				FMeshBufferSet& BufferSet,
				EMeshBufferSemantic Semantic,
				int32 SemanticIndex = 0
			)
		{
			int32 BufferIndex = -1;
			int32 ChannelIndex = -1;
			BufferSet.FindChannel( Semantic, SemanticIndex, &BufferIndex, &ChannelIndex );

			if ( BufferIndex>=0 && ChannelIndex>=0 )
			{
				EMeshBufferSemantic realSemantic = EMeshBufferSemantic::None;
				int32 realSemanticIndex = 0;
				Format = EMeshBufferFormat::None;
				Components = 0;
				int32 offset = 0;
				BufferSet.GetChannel
						(
							BufferIndex, ChannelIndex,
							&realSemantic, &realSemanticIndex,
							&Format, &Components,
							&offset
						);
				check( realSemantic == Semantic );
				check( realSemanticIndex == SemanticIndex );

				ElementSize = BufferSet.GetElementSize( BufferIndex );

				Buffer = BufferSet.GetBufferData( BufferIndex );
				Buffer += offset;
			}
			else
			{
				Format = EMeshBufferFormat::None;
				Components = 0;
				ElementSize = 0;
				Buffer = nullptr;
			}
		}

        inline uint8* ptr() const
		{
			return Buffer;
		}

		inline void operator++()
		{
			Buffer += ElementSize;
		}

		inline void operator++(int32)
		{
			Buffer += ElementSize;
		}

		inline void operator+=( int32 c )
		{
			Buffer += c*ElementSize;
		}

		inline UntypedMeshBufferIterator operator+(int32 c) const
		{
			UntypedMeshBufferIterator res = *this;
			res += c;

			return res;
		}

		inline SIZE_T operator-( const UntypedMeshBufferIterator& Other ) const
		{
			// Special degenerate case.
			if (ElementSize == 0)
			{
				return 0;
			}
			check( Other.ElementSize == ElementSize );
			check( (ptr()-Other.ptr()) % ElementSize == 0 );
			return (ptr()-Other.ptr()) / ElementSize;
		}

		inline int32 GetElementSize() const
		{
			return ElementSize;
		}

		inline EMeshBufferFormat GetFormat() const
		{
			return Format;
		}

		inline int32 GetComponents() const
		{
			return Components;
		}

        FVector4f GetAsVec4f() const
        {
			FVector4f res;

            for (int32 c=0; c< FMath::Min(Components,4); ++c)
            {
                ConvertData( c, &res[0], EMeshBufferFormat::Float32, ptr(), Format );
            }

            return res;
        }

		FVector3f GetAsVec3f() const
		{
			FVector3f res = { 0,0,0 };
			for (int32 c = 0; c < FMath::Min(Components, 3); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float32, ptr(), Format);
			}
			return res;
		}

		FVector3d GetAsVec3d() const
		{
			FVector3d res = { 0,0,0 };
			for (int32 c = 0; c < FMath::Min(Components, 3); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float64, ptr(), Format);
			}
			return res;
		}
		
		FVector2f GetAsVec2f() const
		{
			FVector2f res = {0.0f, 0.0f};
			for (int32 c = 0; c < FMath::Min(Components, 2); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float32, ptr(), Format);
			}

			return res;
		}

		uint32 GetAsUINT32() const
		{
			uint32 res;
			ConvertData(0, &res, EMeshBufferFormat::UInt32, ptr(), Format);
			return res;
		}

		uint64 GetAsUINT64() const
		{
			uint64 res;
			ConvertData(0, &res, EMeshBufferFormat::UInt64, ptr(), Format);
			return res;
		}

		void SetFromUINT32(uint32 v)
		{
			ConvertData(0, ptr(), Format, &v, EMeshBufferFormat::UInt32);
		}

		void SetFromVec3f(const FVector3f& v)
		{
			for (int32 c = 0; c < FMath::Min(Components, 3); ++c)
			{
				ConvertData(c, ptr(), Format, &v, EMeshBufferFormat::Float32);
			}
		}

		void SetFromVec3d(const FVector3d& v)
		{
			for (int32 c = 0; c < FMath::Min(Components, 3); ++c)
			{
				ConvertData(c, ptr(), Format, &v, EMeshBufferFormat::Float64);
			}
		}

	protected:

		int32 ElementSize;
        uint8* Buffer;
		EMeshBufferFormat Format;
		int32 Components;
	};


	//---------------------------------------------------------------------------------------------
	//! Class to iterate a specific BufferIndex ChannelIndex with known type
	//---------------------------------------------------------------------------------------------
	template<EMeshBufferFormat FORMAT, class CTYPE, int32 COMPONENTS>
	class MeshBufferIteratorConst;

	template<EMeshBufferFormat FORMAT, class CTYPE, int32 COMPONENTS>
	class MeshBufferIterator : public UntypedMeshBufferIterator
	{
	public:
		inline MeshBufferIterator
			(
				FMeshBufferSet& BufferSet,
				EMeshBufferSemantic Semantic,
				int32 SemanticIndex = 0
			)
			: UntypedMeshBufferIterator( BufferSet, Semantic, SemanticIndex )
		{
			if (Buffer)
			{
				// Extra checks
				int32 BufferIndex = -1;
				int32 ChannelIndex = -1;
				BufferSet.FindChannel(Semantic, SemanticIndex, &BufferIndex, &ChannelIndex);
				
				if (BufferIndex >= 0 && ChannelIndex >= 0)
				{
					EMeshBufferSemantic realSemantic = EMeshBufferSemantic::None;
					int32 realSemanticIndex = 0;
					EMeshBufferFormat format = EMeshBufferFormat::None;
					int32 components = 0;
					int32 offset = 0;
					BufferSet.GetChannel
					(
						BufferIndex, ChannelIndex,
						&realSemantic, &realSemanticIndex,
						&format, &components,
						&offset
					);

					if (format!=FORMAT || components!=COMPONENTS)
					{
						// Invalidate
						Format = EMeshBufferFormat::None;
						Components = 0;
						ElementSize = 0;
						Buffer = nullptr;
					}
				}
				else
				{
					// Invalidate
					Format = EMeshBufferFormat::None;
					Components = 0;
					ElementSize = 0;
					Buffer = nullptr;
				}
			}
		}

		// \TODO: Replace this with safer sized-typed access.
		inline CTYPE* operator*()
		{
			return reinterpret_cast<CTYPE*>(Buffer);
		}

		inline MeshBufferIterator<FORMAT,CTYPE,COMPONENTS> operator+(int32 c) const
		{
			MeshBufferIterator<FORMAT,CTYPE,COMPONENTS> res = *this;
			res += c;
			return res;
		}

	};


	//---------------------------------------------------------------------------------------------
	//! Class to iterate a specific BufferIndex ChannelIndex of unknown type
	//---------------------------------------------------------------------------------------------
	class UntypedMeshBufferIteratorConst
	{
	public:

		UntypedMeshBufferIteratorConst() = default;
		UntypedMeshBufferIteratorConst(UntypedMeshBufferIteratorConst&&) = default;
		UntypedMeshBufferIteratorConst(const UntypedMeshBufferIteratorConst&) = default;
		UntypedMeshBufferIteratorConst& operator=(UntypedMeshBufferIteratorConst&&) = default;
		UntypedMeshBufferIteratorConst& operator=(const UntypedMeshBufferIteratorConst&) = default;

		inline UntypedMeshBufferIteratorConst
			(
				const FMeshBufferSet& BufferSet,
				EMeshBufferSemantic Semantic,
				int32 SemanticIndex = 0
			)
		{
			int32 BufferIndex = -1;
			int32 ChannelIndex = -1;
			BufferSet.FindChannel( Semantic, SemanticIndex, &BufferIndex, &ChannelIndex );

			if ( BufferIndex>=0 && ChannelIndex>=0 )
			{
				EMeshBufferSemantic RealSemantic = EMeshBufferSemantic::None;
				int32 RealSemanticIndex = 0;
				Format = EMeshBufferFormat::None;
				Components = 0;
				int32 offset = 0;
				BufferSet.GetChannel
						(
							BufferIndex, ChannelIndex,
							&RealSemantic, &RealSemanticIndex,
							&Format, &Components,
							&offset
						);
				check( RealSemantic == Semantic );
				check( RealSemanticIndex == SemanticIndex );

				ElementSize = BufferSet.GetElementSize( BufferIndex );

				Buffer = BufferSet.GetBufferData( BufferIndex );
				Buffer += offset;
			}
			else
			{
				Format = EMeshBufferFormat::None;
				Components = 0;
				ElementSize = 0;
				Buffer = nullptr;
			}
		}

        inline const uint8* ptr() const
		{
			return Buffer;
		}

		inline void operator++()
		{
			Buffer += ElementSize;
		}

		inline void operator++(int32)
		{
			Buffer += ElementSize;
		}

		inline void operator+=( int32 c )
		{
			Buffer += c*ElementSize;
		}

		inline SIZE_T operator-( const UntypedMeshBufferIterator& Other ) const
		{
			// Special degenerate case.
			if (ElementSize == 0)
			{
				return 0;
			}
			check( Other.GetElementSize() == ElementSize );
			check( (ptr()-Other.ptr()) % ElementSize == 0 );
			return (ptr()-Other.ptr()) / ElementSize;
		}

		inline SIZE_T operator-( const UntypedMeshBufferIteratorConst& Other ) const
		{
			// Special degenerate case.
			if (ElementSize == 0)
			{
				return 0;
			}
			check( Other.GetElementSize() == ElementSize );
			check( (ptr()-Other.ptr()) % ElementSize == 0 );
			return (ptr()-Other.ptr()) / ElementSize;
		}

		inline int32 GetElementSize() const
		{
			return ElementSize;
		}

        inline EMeshBufferFormat GetFormat() const
        {
            return Format;
        }

        inline int32 GetComponents() const
        {
            return Components;
        }

		FVector4f GetAsVec4f() const
		{
			FVector4f res(0.0f,0.0f,0.0f,0.0f);
			for (int32 c = 0; c < FMath::Min(Components, 4); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float32, ptr(), Format);
			}
			return res;
		}

		FVector3f GetAsVec3f() const
		{
			FVector3f res = { 0,0,0 };
			for (int32 c = 0; c < FMath::Min(Components, 3); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float32, ptr(), Format);
			}
			return res;
		}

		FVector3d GetAsVec3d() const
		{
			FVector3d res = { 0,0,0 };
			for (int32 c = 0; c < FMath::Min(Components, 3); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float64, ptr(), Format);
			}
			return res;
		}

		FVector2f GetAsVec2f() const
		{
			FVector2f res = {0.0f, 0.0f};
			for (int32 c = 0; c < FMath::Min(Components, 2); ++c)
			{
				ConvertData(c, &res[0], EMeshBufferFormat::Float32, ptr(), Format);
			}

			return res;
		}

		uint32 GetAsUINT32() const
		{
			uint32 res = 0;
			ConvertData(0, &res, EMeshBufferFormat::UInt32, ptr(), Format);
			return res;
		}

		uint64 GetAsUINT64() const
		{
			uint64 res;
			ConvertData(0, &res, EMeshBufferFormat::UInt64, ptr(), Format);
			return res;
		}

		void GetAsInt32Vec(int32* Data, int32 Count) const
		{
			for (int32 c = 0; c < FMath::Min(Components, Count); ++c)
			{
				ConvertData(c, Data, EMeshBufferFormat::Int32, ptr(), Format);
			}
		}

        inline UntypedMeshBufferIteratorConst operator+(int32 c) const
        {
            UntypedMeshBufferIteratorConst res = *this;
            res += c;
            return res;
        }

	protected:

		int32 ElementSize = 0;
        const uint8* Buffer = nullptr;
		EMeshBufferFormat Format = EMeshBufferFormat::None;
		int32 Components = 0;
	};




	//---------------------------------------------------------------------------------------------
	//! Class to iterate a specific BufferIndex ChannelIndex of a constant BufferIndex set
	//---------------------------------------------------------------------------------------------
	template<EMeshBufferFormat FORMAT, class CTYPE, int32 COMPONENTS>
	class MeshBufferIteratorConst : public UntypedMeshBufferIteratorConst
	{
	public:

		inline MeshBufferIteratorConst()
		{
		}


		inline MeshBufferIteratorConst
			(
				const FMeshBufferSet& BufferSet,
				EMeshBufferSemantic Semantic,
				int32 SemanticIndex = 0
			)
			: UntypedMeshBufferIteratorConst( BufferSet, Semantic, SemanticIndex )
		{
			if (Buffer)
			{
				// Extra checks
				int32 BufferIndex = -1;
				int32 ChannelIndex = -1;
				BufferSet.FindChannel(Semantic, SemanticIndex, &BufferIndex, &ChannelIndex);

				if (BufferIndex >= 0 && ChannelIndex >= 0)
				{
					EMeshBufferSemantic realSemantic = EMeshBufferSemantic::None;
					int32 realSemanticIndex = 0;
					EMeshBufferFormat format = EMeshBufferFormat::None;
					int32 components = 0;
					int32 offset = 0;
					BufferSet.GetChannel
					(
						BufferIndex, ChannelIndex,
						&realSemantic, &realSemanticIndex,
						&format, &components,
						&offset
					);
					
					if (format!=FORMAT || components!=COMPONENTS)
					{
						// Invalidate
						Format = EMeshBufferFormat::None;
						Components = 0;
						ElementSize = 0;
						Buffer = nullptr;
					}
				}
				else
				{
					// Invalidate
					Format = EMeshBufferFormat::None;
					Components = 0;
					ElementSize = 0;
					Buffer = nullptr;
				}
			}
		}

		inline const CTYPE* operator*() const
		{
			return reinterpret_cast<const CTYPE*>(Buffer);
		}

		inline MeshBufferIteratorConst<FORMAT,CTYPE,COMPONENTS> operator+(int32 c) const
		{
			MeshBufferIteratorConst<FORMAT,CTYPE,COMPONENTS> res = *this;
			res += c;
			return res;
		}

	};


	/** Iterator for vertex Ids.
	* The lifecycle of the iterator cannot exceed the mesh lifecycle. 
	*/
	class MeshVertexIdIteratorConst
	{
	private:

		/** Current Id that the iterator is pointing at. */
		int32 CurrentIdIndex = 0;
		int32 NumVertices = 0;

		uint64 MeshIDPrefix = 0;

		/** Buffer iterator in case there is an actual Id BufferIndex. */
		UntypedMeshBufferIteratorConst BufferIterator;

	public:

		inline MeshVertexIdIteratorConst()
		{
		}

		inline MeshVertexIdIteratorConst( const FMesh* InMesh )
		{
			if (!InMesh)
			{
				return;
			}

			NumVertices = InMesh->GetVertexCount();
			MeshIDPrefix = (uint64(InMesh->MeshIDPrefix) << 32);
			BufferIterator = UntypedMeshBufferIteratorConst(InMesh->VertexBuffers, EMeshBufferSemantic::VertexIndex, 0);
		}		

		inline void operator++()
		{
			CurrentIdIndex++;			

			if (BufferIterator.ptr())
			{
				BufferIterator++;
			}
		}

		inline void operator++(int32)
		{
			CurrentIdIndex++;

			if (BufferIterator.ptr())
			{
				BufferIterator++;
			}
		}

		inline void operator+=(int32 c)
		{
			CurrentIdIndex+=c;

			if (BufferIterator.ptr())
			{
				BufferIterator+=c;
			}
		}

		inline MeshVertexIdIteratorConst operator+(int32 c) const
		{
			MeshVertexIdIteratorConst res = *this;
			res += c;
			return res;
		}

		bool IsValid() const
		{
			return CurrentIdIndex < NumVertices;
		}

		uint64 Get() const
		{
			check(CurrentIdIndex < NumVertices);

			switch (BufferIterator.GetFormat())
			{
			case EMeshBufferFormat::UInt32: // relative
			{
				// There is a BufferIndex storing IDs without prefix because it is the same for all vertices.
				return MeshIDPrefix | uint64(*reinterpret_cast<const uint32*>(BufferIterator.ptr()));
			}
			case EMeshBufferFormat::UInt64: // explicit
			{
				return *reinterpret_cast<const uint64*>(BufferIterator.ptr());
			}
			case EMeshBufferFormat::None: // implicit
			{
				// The id is just prefix and index
				return MeshIDPrefix | uint64(CurrentIdIndex);
			}
			default:
			{
				unimplemented();
				return FMesh::InvalidVertexId;
			}
			}
		}

	};


}
