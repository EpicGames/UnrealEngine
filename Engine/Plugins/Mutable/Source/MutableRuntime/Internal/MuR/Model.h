// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingAsset.h"
#include "MuR/Parameters.h"
#include "MuR/Model.h"
#include "MuR/System.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Operations.h"
#include "MuR/ExtensionData.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/MutableRuntimeModule.h"

#include "Templates/SharedPointer.h"
#include "ExternalOperationProvider.h"
#include "RomManager.h"

class UMaterialInterface;


#define UE_API MUTABLERUNTIME_API

#define MUTABLE_MAX_RUNTIME_PARAMETERS_PER_STATE	65
#define MUTABLE_GROW_BORDER_VALUE					2


namespace UE::Mutable::Private
{
	class FInputArchive;
	class FModelWriter;
	class FOutputArchive;
    class FModelParametersGenerator;
	
	/** Used to debug and log. */
	constexpr bool DebugRom = false;
	constexpr bool DebugRomAll = false;
	constexpr int32 DebugRomIndex = 44;
	constexpr int32 DebugImageIndex = 9;

	struct FConstantResourceIndex
	{
		uint32 Index : 31;
		/** This may mean that the resource needs to be looked up on a different array. */
		uint32 Streamable : 1;
	};
	static_assert(sizeof(FConstantResourceIndex) == 4);
	MUTABLE_DEFINE_POD_SERIALISABLE(FConstantResourceIndex);
	//MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FConstantResourceIndex);

	/** This is encoded with minimal bits. Make sure to review all uses if extended. */
	enum class ERomDataType : uint32
	{
		Image = 0,
		Mesh  = 1
	};

    /** Data stored for a rom even if it is not loaded.
	* This struct is size-sensitive since there may be many roms and it is always loaded in memory when a CO is.
	*/
    struct FRomDataRuntime
    {
		/** Size of the rom */
		uint32 Size : 30;

		/** Index of the resource in its type-specific array. See ERomDataType. */
		uint32 ResourceType : 1;

		/** Properties of the rom data. */
		uint32 IsHighRes : 1;

		// TODO: Store the offset here and delete the FModelStreamableBlock map?
		/** Offset in file */
		// uint32 Offset = 0;
    };

	/** Not critical to keep this size, but it is memory-usage sensitive. */
	static_assert(sizeof(FRomDataRuntime) == 4);
	MUTABLE_DEFINE_POD_SERIALISABLE(FRomDataRuntime);

	struct FRomDataCompile
	{
		/** ID used to identify the origin of this data and used for grouping. */
		uint32 SourceId;
	};
	MUTABLE_DEFINE_POD_SERIALISABLE(FRomDataCompile);

    //!
    template<typename DATA>
    inline void AppendCode(TArray<uint8>& ByteCode, const DATA& Data)
    {
        int32 Pos = ByteCode.Num();
        ByteCode.SetNum(Pos + sizeof(DATA), EAllowShrinking::No);
		FMemory::Memcpy(&ByteCode[Pos], &Data, sizeof(DATA));
    }

	//!
	struct FImageLODRange
	{
		int32 FirstIndex    = 0;
		uint16 ImageSizeX   = 0;
		uint16 ImageSizeY   = 0;
		uint8 LODCount      = 0;
		uint8 NumLODsInTail = 0; 
		uint8 Flags         = 0;
		EImageFormat ImageFormat = EImageFormat::None;
	};
	MUTABLE_DEFINE_POD_SERIALISABLE(FImageLODRange);

	struct FMeshContentRange
	{
		static constexpr uint32 FirstIndexMaxBits   = 24;
		static constexpr uint32 ContentFlagsMaxBits = 32 - FirstIndexMaxBits;
		static constexpr uint32 FirstIndexBitMask   = (1 << FirstIndexMaxBits) - 1; 

		static_assert(FirstIndexMaxBits < 32);
		static_assert(ContentFlagsMaxBits >= sizeof(EMeshContentFlags)*8);

		// NOTE: Bitfields layout can be implementation defined, here we want consistency across all
		// compilers so that the struct can be POD serializable.
		uint32 FirstIndex_ContentFlags = 0; // Low bits are FirstIndex, high bits are ContentFlags.
		uint32 MeshIDPrefix = 0;

		FORCEINLINE EMeshContentFlags GetContentFlags() const
		{
			return static_cast<EMeshContentFlags>(
					(FirstIndex_ContentFlags >> FirstIndexMaxBits) &
					((1 << ContentFlagsMaxBits) - 1));
		}

		FORCEINLINE uint32 GetFirstIndex() const
		{
			return FirstIndex_ContentFlags & FirstIndexBitMask;
		}

		FORCEINLINE void SetContentFlags(EMeshContentFlags ContentFlags)
		{
			check(uint32(ContentFlags) < (1 << ContentFlagsMaxBits));

			FirstIndex_ContentFlags = 
					(FirstIndex_ContentFlags & FirstIndexBitMask) | 
					((uint32(ContentFlags) << FirstIndexMaxBits));
		}

		FORCEINLINE void SetFirstIndex(uint32 FirstIndex) 
		{
			check(FirstIndex < ((1 << FirstIndexMaxBits)));

			FirstIndex_ContentFlags = 
					(FirstIndex_ContentFlags & ~FirstIndexBitMask) | 
					(FirstIndex & FirstIndexBitMask);
		}
	};
	static_assert(sizeof(FMeshContentRange) == sizeof(uint32)*2);
	MUTABLE_DEFINE_POD_SERIALISABLE(FMeshContentRange);

    //!
    struct FProgram
    {
        UE_API FProgram();

        struct FState
        {
            /** Name of the state */
            FString Name;

            /** First instruction of the full build of an instance in this state */
            FOperation::ADDRESS Root = 0;

            /** 
			 * List of parameters index (to FProgram::Parameters) of the runtime parameters of
             * this state.
			 */
			TArray<int> m_runtimeParameters;

            /** List of instructions that need to be cached to efficiently update this state */
			TArray<FOperation::ADDRESS> m_updateCache;

            /** 
			 * List of root instructions for the dynamic resources that depend on the runtime
			 * parameters of this state, with a mask of relevant runtime parameters.
             * The mask has a bit on for every runtime parameter in the m_runtimeParameters array.
			 * The uint64 is linked to MUTABLE_MAX_RUNTIME_PARAMETERS_PER_STATE
			 */
			TArray< TPair<FOperation::ADDRESS,uint64> > m_dynamicResources;

            /** */
            void Serialise(FOutputArchive& Arch) const;

            /** */
            void Unserialise(FInputArchive& Arch);

            /** Returns the mask of parameters (from the runtime parameter list of this state) including the parameters that 
			* are relevant for the dynamic resource at the given address.
			*/
            uint64 IsDynamic(FOperation::ADDRESS At) const;

            /** */
            bool IsUpdateCache(FOperation::ADDRESS at) const;

            /** */ 
            void AddUpdateCache(FOperation::ADDRESS At);
        };

		int32 NumOps = 0;
		TArray<FOperation::ADDRESS> OperationDebugData;

        /** Byte-coded representation of the program, using variable-sized op data. */
		TArray<uint8> ByteCode;

        /** */
		TArray<FState> States;

		/** Data for every rom required in-game. */
		TArray<FRomDataRuntime> Roms;

		/** Data for every rom required at compile-time. It is empty in cooked data. */
		TArray<FRomDataCompile> RomsCompileData;
    	
		/** Constant image mip data is split in 2 sets: ConstantImageLODsPermanent constains data that is always loaded. 
		* Index with FConstantResourceIndex::Index, when Streamable is 0.
		*/
		TArray<TManagedPtr<const FImage>> ConstantImageLODsPermanent;

		/** Constant image mip chain indices: ranges in this array are defined in FImageLODRange and the indices here refer to ConstantImageLODs. */
		TArray<FConstantResourceIndex> ConstantImageLODIndices;

		/** Constant image data. */
		TArray<FImageLODRange> ConstantImages;

		/** Constant mesh content indices: ranges in this array are defined in FMeshContentRange and the indices here refer to ConstantMeshes. */
		TArray<FConstantResourceIndex> ConstantMeshContentIndices;
		
		/** Constant mesh data */
		TArray<FMeshContentRange> ConstantMeshes;

		/** Constant mesh data is split in 2 sets: ConstantMeshesPermanent constains data that is always loaded.
		* Index with FConstantResourceIndex::Index, when Streamable is 0.
		*/
		TArray<TManagedPtr<const FMesh>> ConstantMeshesPermanent;
    	
        /** Constant string data */
		TArray<FString> ConstantStrings;

		/** */
		TArray<TArray<uint32>> ConstantUInt32Lists;
    	
    	/** */
    	TArray<TArray<int32>> ConstantInt32Lists;

		/** */
		TArray<TArray<uint64>> ConstantUInt64Lists;

    	/** */
    	TArray<TArray<float>> ConstantFloatLists;

    	/** */
    	TArray<TArray<bool>> ConstantBoolLists;
    	
        /** Constant layout data */
		TArray<TManagedPtr<const FLayout>> ConstantLayouts;

    	/** Constant Instanced Struct */
    	TArray<TManagedPtr<const FInstancedStruct>> ConstantInstancedStructs;
    	
        /** Constant projectors */
		TArray<FProjector> ConstantProjectors;

        /** Constant matrices, usually used for transforms */
		TArray<FMatrix44f> ConstantMatrices;

		/** Constant shapes */
		TArray<FShape> ConstantShapes;

        /** Constant curves */
		TArray<FRichCurve> ConstantCurves;

        /** Constant skeletons */
		TArray<TManagedPtr<const FSkeleton>> ConstantSkeletons;

		/** Constant Physics Bodies */
		TArray<TManagedPtr<const FPhysicsBody>> ConstantPhysicsBodies;

        /** FParameters of the model. */
        /** The value stored here is the default value. */
		TArray<FParameterDesc> Parameters;

        /** Ranges for iteration of the model operations. */
		TArray<FRangeDesc> Ranges;

        /** 
		 * List of parameter lists. These are used in several places, like storing the
         * pregenerated list of parameters influencing a resource.
         * The parameter lists are sorted. 
		 */
		TArray<TArray<uint16>> ParameterLists;

    	/** Given an instruction, parameters that are in the subtree. */
	   	TMap<FOperation::ADDRESS, int32> RelevantParameterList;
    	
		/** Constant Material Data */
		TArray<TManagedPtr<const FMaterial>> ConstantMaterials;

		/** Constant FNames */
		TMap<uint32, FName> ConstantNames;

		/** Constant Mesh Sockets */
		TMap<uint32, FMeshSocket> ConstantSockets;


    	TSharedPtr<const IExternalOperationProvider> ExternalOperationProvider;
    	
#if WITH_EDITOR
		/** 
		 * State of the program. True unless the streamed resources were destroyed,
		 * which could happen in the editor after recompiling the CO. 
		 */
		bool bIsValid = true;
#endif
    	
        UE_API void Serialise(FOutputArchive& Arch) const;

        void Unserialise(FInputArchive& Arch);

		void PostLoad();

        /** Debug method that logs the top used instruction types. */
        void LogHistogram() const;

    	UE_API uint32 GetRomSize(int32 Index) const;
    	
        UE_API int32 GetRomCount() const;

#if WITH_EDITOR
    	UE_API uint32 GetRomSourceId(int32 Index) const;
#endif

    	UE_API ERomDataType GetRomDataType(int32 Index) const;

    	UE_API bool IsRomHighRes(int32 Index) const;
   
    	/** 
		 * Returns the ConstantImageIndex LODIndex rom id. In case ConstantImageIndex does not have LODIndex 
		 * returns -1.
		 */
    	UE_API int32 GetConstantImageRomId(int32 ConstantImageIndex, int32 LODIndex) const;

#if WITH_EDITOR    	
        UE_API FOperation::ADDRESS AddConstant(const TManagedPtr<const FLayout>& Layout);
    	
        UE_API FOperation::ADDRESS AddConstant(const TManagedPtr<const FSkeleton>& Skeleton);
    	
        UE_API FOperation::ADDRESS AddConstant(const TManagedPtr<const FPhysicsBody>& PhysicsBody);

        UE_API uint32 AddConstant(const FMeshSocket& Socket);

        UE_API FOperation::CONSTANT_LIST_UINT32 AddConstant(const TArray<FMeshSocket>& Sockets);
    	
        UE_API FOperation::CONSTANT_STRING AddConstant(const FString& Str);
    	
        UE_API FOperation::ADDRESS AddConstant(const TArray<uint32>& UInt32List);

    	UE_API FOperation::ADDRESS AddConstant(const TArray<int32>& Int32List);
    	
        UE_API FOperation::CONSTANT_LIST_UINT64 AddConstant(const TArray<uint64>& UInt64List);
    	
        UE_API FOperation::ADDRESS AddConstant(const TArray<FString>& StringList);

		UE_API FOperation::CONSTANT_NAME AddConstant(const FName& InName);

		UE_API FOperation::CONSTANT_LIST_UINT32 AddConstant(const TArray<FName>& NameList);

        UE_API FOperation::CONSTANT_MATRIX AddConstant(const FMatrix44f& Constant);
    	
        UE_API FOperation::CONSTANT_SHAPE AddConstant(const FShape& Constant);
    	
        UE_API FOperation::ADDRESS AddConstant(const FProjector& Constant);
    	
        UE_API FOperation::ADDRESS AddConstant(const FRichCurve& Constant);
    	
        UE_API FOperation::ADDRESS AddConstant(const TManagedPtr<const FMaterial>& Material);
    	
    	UE_API FOperation::ADDRESS AddConstant(const TArray<float>& FloatList);

    	UE_API FOperation::ADDRESS AddConstant(const TArray<bool>& BoolList);

#endif //WITH_EDITOR
    	
        UE_API EOpType GetOpType(FOperation::ADDRESS At) const;
    	
        template<typename ARGS>
		ARGS GetOpArgs(FOperation::ADDRESS Address) const
		{
			ARGS Result;
			
			uint32 ArgsByteCodeOffset = GetAddressByteCodeOffset(Address) + sizeof(EOpType);
			FMemory::Memcpy(&Result, &ByteCode[ArgsByteCodeOffset], sizeof(ARGS));
			
			return Result;
		}

		template<typename ARGS>
		void SetOpArgs(FOperation::ADDRESS Address, const ARGS& Args)
		{
			uint32 ArgsByteCodeOffset = GetAddressByteCodeOffset(Address) + sizeof(EOpType);
			FMemory::Memcpy(&ByteCode[ArgsByteCodeOffset], &Args, sizeof(ARGS));
		}

        UE_API const uint8* GetOpArgsPointer(FOperation::ADDRESS Address) const;
    	
        uint8* GetOpArgsPointer(FOperation::ADDRESS Address);
    };

	
    /** A Model represents a customisable object with any number of parameters.
    * When values are given to the parameters, specific Instances can be built, which hold the
    * built application-usable data.
	*/
    class FModel
	{
	public:
    	UE_API FModel();
    	
		static UE_API void Serialise(const FModel*, FOutputArchive&);
		static UE_API TSharedPtr<FModel> StaticUnserialise(FInputArchive&);
    	
    	void Serialise(FOutputArchive& Arch) const;

		void Unserialise(FInputArchive& Arch);

		/** Special serialise operation that serialises the data in separate "files". An object
         * with the ModelStreamer interface is responsible of storing this data and providing
		 * the "file" concept. 
		 * If bDropData is set to true, the rom data will be freed as it is serialized.
		 */
        static UE_API void Serialise(FModel* Model, FModelWriter& Streamer, bool bDropData = false);

		//! Return true if the model has external data in other files. This kind of models will
		//! require data streaming when used.
		UE_API bool HasExternalData() const;

#if WITH_EDITOR
		//! Return true unless the streamed resources were destroyed, which could happen in the
		//! editor after recompiling the CO.
		UE_API bool IsValid() const;

		//! Invalidate the Model. Compiling a compiled CO will invalidate the model kept by previously
		//! generated resources, like streamed textures.
		UE_API void Invalidate();
#endif

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		//! Create a set of new parameters of the model with the default values.
		//! If old parameters are provided, they will be reused when possible instead of the
		//! default values.
        static UE_API TSharedPtr<FParameters> NewParameters(TSharedPtr<const FModel> Model, const FParameters* OldParameters = nullptr);

		/** Return true if the parameter is multi-dimensional */
		UE_API bool IsParameterMultidimensional(int32 ParameterIndex) const;

		//! Get the number of states in the model.
		UE_API int32 GetStateCount() const;

		//! Get a state name by state index from 0 to GetStateCount-1
		UE_API const FString& GetStateName( int32 StateIndex ) const;

		//! Find a state index by state name
		UE_API int32 FindState( const FString& Name ) const;

		//! Get the number of parameters available in a particular state.
		UE_API int32 GetStateParameterCount( int32 StateIndex ) const;

		//! Get the index of one of the parameters in the given state. The index refers to the
		//! parameters in a FParameters object obtained from this model with NewParameters.
		UE_API int32 GetStateParameterIndex( int32 StateIndex, int32 ParamIndex ) const;
	
		//! Return the default value of a boolean parameter.
		//! \pre The parameter specified by index is a T_BOOL.
		//! \param Index Index of the parameter from 0 to GetCount()-1
    	UE_API bool GetBoolDefaultValue(int32 Index) const;

   		//! Return the default value of a integer parameter.
		//! \pre The parameter specified by index is a T_INT.
		//! \param Index Index of the parameter from 0 to GetCount()-1
    	UE_API int32 GetIntDefaultValue(int32 Index) const;

		//! Return the default value of a float parameter.
		//! \pre The parameter specified by index is a T_FLOAT.
		//! \param Index Index of the parameter from 0 to GetCount()-1
        UE_API float GetFloatDefaultValue(int32 Index) const;

		//! Return the default value of a color parameter.
		//! \pre The parameter specified by index is a T_FLOAT.
        //! \param Index Index of the parameter from 0 to GetCount()-1
        //! \param R,G,B Pointers to values where every resulting color channel will be stored
    	UE_API void GetColorDefaultValue(int32 Index, FVector4f& OutValue) const;

		//! Return the default value of a color parameter.
		//! \pre The parameter specified by index is a T_FLOAT.
        //! \param Index Index of the parameter from 0 to GetCount()-1
    	UE_API FMatrix44f GetMatrixDefaultValue(int32 Index) const;

    	//! Return the default value of a projector parameter, as a 4x4 matrix. The matrix is supposed to be
		//! a linear transform in column-major.
		//! \pre The parameter specified by index is a T_PROJECTOR.
        //! \param Index Index of the parameter from 0 to GetCount()-1
        //! \param OutPos Pointer to where the object-space position coordinates of the projector will be stored.
        //! \param OutDir Pointer to where the object-space direction vector of the projector will be stored.
        //! \param OutUp Pointer to where the object-space vertically up direction vector
        //!         of the projector will be stored. This controls the "roll" angle of the
        //!         projector.
        //! \param OutScale Pointer to the projector-space scaling of the projector.
    	UE_API void GetProjectorDefaultValue(int32 Index, EProjectorType* OutProjectionType, FVector3f* OutPos,
    	 	FVector3f* OutDir, FVector3f* OutUp, FVector3f* OutScale, float* OutProjectionAngle) const;
    	
    	 /** Get a constant image, assuming at least some mips are loaded. The image constant will be composed with lodaded mips if necessary.
    	  * @param RomsStreamed Given a rom index, return the  future of the streamed rom. */
		template <typename CreateImageFunc>
        void GetConstant(uint32 ConstantIndex, TManagedPtr<const FImage>& Result, int32 MipsToSkip, const TFunction<TManagedPtr<const FImage>(int32)>&& RomsStreamed, const CreateImageFunc&& CreateImage) const
        {
			const int32 NumLODs = Program.ConstantImages[ConstantIndex].LODCount;

			check(NumLODs > 0);
			const int32 ReallySkippedLODs = FMath::Min(NumLODs - 1, MipsToSkip);
			const int32 FirstLODIndexIndex = Program.ConstantImages[ConstantIndex].FirstIndex;
			const int32 FirstTailLOD = Program.ConstantImages[ConstantIndex].LODCount - Program.ConstantImages[ConstantIndex].NumLODsInTail;

			auto GetImageLOD = [&](FConstantResourceIndex Index) -> TManagedPtr<const FImage>
			{
				TManagedPtr<const FImage> Mip;
				if (!Index.Streamable)
				{
					if (Program.ConstantImageLODsPermanent.IsValidIndex(Index.Index))
					{
						Mip = Program.ConstantImageLODsPermanent[Index.Index];
					}
				}
				else
				{
					return RomsStreamed(Index.Index);
				}

				return Mip;
			};
			
			// Compose the result image
			{
				MUTABLE_CPUPROFILER_SCOPE(ComposeConstantImage);

				TManagedPtr<FImage> ResultImage = nullptr;

				int32 LOD = ReallySkippedLODs;

				// Find first availabe LOD.
				for (; LOD <= FirstTailLOD; ++LOD)
				{
					MUTABLE_CPUPROFILER_SCOPE(Search_FisrtValid);
					TManagedPtr<const FImage> FirstValidLODImage = GetImageLOD(Program.ConstantImageLODIndices[FirstLODIndexIndex + LOD]);
					
					if (FirstValidLODImage)
					{
						break;
					}
				}
			
				if (LOD < NumLODs)
				{						
					int32 SizeX = Program.ConstantImages[ConstantIndex].ImageSizeX; 
					int32 SizeY = Program.ConstantImages[ConstantIndex].ImageSizeY; 

					for (int32 I = 0; I < LOD; ++I)
					{
						SizeX = FMath::DivideAndRoundUp(SizeX, 2);
						SizeY = FMath::DivideAndRoundUp(SizeY, 2);
					}
					 
					EImageFormat Format = Program.ConstantImages[ConstantIndex].ImageFormat; 

					ResultImage = CreateImage(SizeX, SizeY, NumLODs - LOD, Format, EInitializationType::NotInitialized);
					ResultImage->Flags = Program.ConstantImages[ConstantIndex].Flags;
				}
	
				check(ResultImage);

				// Some non-block pixel formats require manual resize.
				const bool bFormatNeedsDataResize = ResultImage->DataStorage.IsEmpty();
				
				int32 DestLOD = 0;
				// Process hi-res separeted mips.
				for (; LOD < FirstTailLOD; ++LOD, ++DestLOD)
				{
					MUTABLE_CPUPROFILER_SCOPE(NonTailLODs);
					TManagedPtr<const FImage> Image = GetImageLOD(Program.ConstantImageLODIndices[FirstLODIndexIndex + LOD]);

					if (!ensureAlwaysMsgf(Image, TEXT("Missing hi-res image rom.")))
					{
						Result = nullptr;
						return;
					}
					
					if (bFormatNeedsDataResize)
					{
						ResultImage->DataStorage.ResizeLOD(DestLOD, Image->DataStorage.GetLOD(0).Num());
					}

					TArrayView<uint8> ResultLODView = ResultImage->DataStorage.GetLOD(DestLOD);
					TArrayView<const uint8> ImageLODView = Image->DataStorage.GetLOD(0);

					check(ResultLODView.Num() == ImageLODView.Num())
					FMemory::Memcpy(ResultLODView.GetData(), ImageLODView.GetData(), ResultLODView.Num());
				}

				// Process compacted tail.
				// NOTE: The rom tail data storage can have multiple image buffers.
				if (LOD < NumLODs)
				{
					MUTABLE_CPUPROFILER_SCOPE(TailLODs);
					TManagedPtr<const FImage> TailLODsImage = GetImageLOD(Program.ConstantImageLODIndices[FirstLODIndexIndex + FirstTailLOD]);

					if (!ensureAlwaysMsgf(TailLODsImage, TEXT("Missing tail image rom.")))
					{
						Result = nullptr;
						return;
					}

					check(TailLODsImage->DataStorage.GetNumLODs() >= NumLODs - LOD);		
	
					int32 FirstTailImageTailLOD = FMath::Min(
							TailLODsImage->DataStorage.ComputeFirstCompactedTailLOD(),
							TailLODsImage->DataStorage.GetNumLODs());

					int32 TailImageLOD = FMath::Max(0, ReallySkippedLODs - FirstTailLOD);
					for (; TailImageLOD < FirstTailImageTailLOD; ++TailImageLOD, ++LOD, ++DestLOD)
					{
						MUTABLE_CPUPROFILER_SCOPE(TailLODCopy);
						if (bFormatNeedsDataResize)
						{
							ResultImage->DataStorage.ResizeLOD(DestLOD, TailLODsImage->DataStorage.GetLOD(TailImageLOD).Num());
						}

						TArrayView<uint8> ResultLODView = ResultImage->DataStorage.GetLOD(DestLOD);
						TArrayView<const uint8> ImageLODView = TailLODsImage->DataStorage.GetLOD(TailImageLOD);

						check(ResultLODView.Num() == ImageLODView.Num());
						FMemory::Memcpy(ResultLODView.GetData(), ImageLODView.GetData(), ResultLODView.Num());
					}	

					if (LOD < NumLODs)
					{
						MUTABLE_CPUPROFILER_SCOPE(TailTailCopy);
						check(DestLOD >= ResultImage->DataStorage.ComputeFirstCompactedTailLOD());
						check(TailImageLOD >= TailLODsImage->DataStorage.ComputeFirstCompactedTailLOD());

						TArrayView<const uint8> FirstTailLODView = TailLODsImage->DataStorage.GetLOD(TailImageLOD);
						TArrayView<const uint8> LastTailLODView  = TailLODsImage->DataStorage.GetLOD(TailLODsImage->DataStorage.GetNumLODs() - 1);
					
						// The tail is stored compacted in memory.
						int64 NumBytesInTail = LastTailLODView.GetData() + LastTailLODView.Num() - FirstTailLODView.GetData();
						check(NumBytesInTail >= 0);
						
						if (bFormatNeedsDataResize)
						{
							const int32 NumLODsToResize = ResultImage->DataStorage.GetNumLODs() - DestLOD;
							check(NumLODsToResize == TailLODsImage->DataStorage.GetNumLODs() - TailImageLOD);

							ResultImage->DataStorage.GetInternalArray(DestLOD).Reserve(NumBytesInTail);

							for (int32 LODOffset = 0; LODOffset < NumLODsToResize; ++LODOffset)
							{
								int32 TailLODSizeInBytes = TailLODsImage->DataStorage.GetLOD(LODOffset + TailImageLOD).Num();
								ResultImage->DataStorage.ResizeLOD(LODOffset + DestLOD, TailLODSizeInBytes);
							}
						}

						TArrayView<uint8> ResultLODView = ResultImage->DataStorage.GetLOD(DestLOD);

						check((int64)ResultImage->DataStorage.GetInternalArray(DestLOD).Num() >= NumBytesInTail);
						FMemory::Memcpy(ResultLODView.GetData(), FirstTailLODView.GetData(), NumBytesInTail);
					}
				}
	
				Result = ResultImage;
			}
		}		

    	/* @param RomsStreamed Given a rom index, return the  future of the streamed rom. */
		template<class CreateMeshFunc>
        void GetConstant(
				uint32 MeshConstantIndex, 
				int32 SkeletonConstantIndex,
				const TPassthroughObjectPtr<UClothingAssetBase>& ClothAsset,
				TManagedPtr<const FMesh>& OutMesh, 
				EMeshContentFlags FilterContentFlags, 
				const TFunction<TManagedPtr<const FMesh>(int32)>&& RomsStreamed,
				CreateMeshFunc&& CreateMesh) const
        {
			MUTABLE_CPUPROFILER_SCOPE(GetConstant_Mesh)

			FMeshContentRange MeshContentRange = Program.ConstantMeshes[MeshConstantIndex];

			TManagedPtr<const FMesh> EmptyMesh = nullptr;
			auto GetMeshAtResourceIndex = [&](FConstantResourceIndex ResourceIndex) -> TManagedPtr<const FMesh>
			{
				MUTABLE_CPUPROFILER_SCOPE(GetConstant_Mesh_GetMesh)

				if (!ResourceIndex.Streamable)
				{
					if (Program.ConstantMeshesPermanent.IsValidIndex(ResourceIndex.Index))
					{
						return Program.ConstantMeshesPermanent[ResourceIndex.Index];
					}

					if (!EmptyMesh)
					{
						EmptyMesh = MakeManaged<FMesh>();
					}
					
					return EmptyMesh;
				}
				else
				{
					TManagedPtr<const FMesh> Found = RomsStreamed(ResourceIndex.Index);
					if (!Found)
					{
						Found = MakeManaged<FMesh>();
					}
					
					return Found;
				}
			};

			TManagedPtr<const FMesh> GeometryMesh = nullptr;
			TManagedPtr<const FMesh> PoseMesh     = nullptr;
			TManagedPtr<const FMesh> PhysicsMesh  = nullptr;
			TManagedPtr<const FMesh> MetaDataMesh = nullptr;

			int32 MeshRomCurrentIndex = MeshContentRange.GetFirstIndex();
			if (EnumHasAnyFlags(FilterContentFlags, EMeshContentFlags::GeometryData) &&
				EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::GeometryData))
			{
				const FConstantResourceIndex ResourceIndex = Program.ConstantMeshContentIndices[MeshRomCurrentIndex];
				GeometryMesh = GetMeshAtResourceIndex(ResourceIndex);
			}
			MeshRomCurrentIndex += (int32)EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::GeometryData);

			if (EnumHasAnyFlags(FilterContentFlags, EMeshContentFlags::PoseData) &&
				EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::PoseData))
			{
				const FConstantResourceIndex ResourceIndex = Program.ConstantMeshContentIndices[MeshRomCurrentIndex];
				PoseMesh = GetMeshAtResourceIndex(ResourceIndex);
			}
			MeshRomCurrentIndex += (int32)EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::PoseData);

			if (EnumHasAnyFlags(FilterContentFlags, EMeshContentFlags::PhysicsData) &&
				EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::PhysicsData))
			{
				const FConstantResourceIndex ResourceIndex = Program.ConstantMeshContentIndices[MeshRomCurrentIndex];
				PhysicsMesh = GetMeshAtResourceIndex(ResourceIndex);
			}
			MeshRomCurrentIndex += (int32)EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::PhysicsData);

			if (EnumHasAnyFlags(FilterContentFlags, EMeshContentFlags::MetaData) &&
				EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::MetaData))
			{
				const FConstantResourceIndex ResourceIndex = Program.ConstantMeshContentIndices[MeshRomCurrentIndex];
				MetaDataMesh = GetMeshAtResourceIndex(ResourceIndex);
			}
			MeshRomCurrentIndex += (int32)EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::MetaData);

			check(MeshRomCurrentIndex - MeshContentRange.GetFirstIndex() == FMath::CountBits((uint32)MeshContentRange.GetContentFlags()));

			uint32 MeshBudgetReserve = 0;
			MeshBudgetReserve += GeometryMesh ? GeometryMesh->GetDataSize() : 0;
			MeshBudgetReserve += PoseMesh     ? PoseMesh->GetDataSize()     : 0;
			MeshBudgetReserve += PhysicsMesh  ? PhysicsMesh->GetDataSize()  : 0;
			MeshBudgetReserve += MetaDataMesh ? MetaDataMesh->GetDataSize() : 0;

			TManagedPtr<FMesh> Result = nullptr;
			if (GeometryMesh)
			{
				MUTABLE_CPUPROFILER_SCOPE(GetConstant_Mesh_Geoemtry)

				Result = CreateMesh(MeshBudgetReserve);
				Result->CopyFrom(*GeometryMesh);
			}

			if (PoseMesh)
			{
				MUTABLE_CPUPROFILER_SCOPE(GetConstant_Mesh_Pose)

				if (!Result)
				{
					Result = CreateMesh(MeshBudgetReserve);
					Result->CopyFrom(*PoseMesh);
				}
				else
				{
					Result->BonePoses = PoseMesh->BonePoses; 
					Result->BoneMap   = PoseMesh->BoneMap;

					for (const TPair<EMeshBufferType, FMeshBufferSet>& AdditionalBuffer : PoseMesh->AdditionalBuffers)
					{
						const bool bIsPoseBufferType = 	
								AdditionalBuffer.Key == EMeshBufferType::SkeletonDeformBinding;

						if (bIsPoseBufferType)
						{
							Result->AdditionalBuffers.Emplace(AdditionalBuffer);
						}
					}	
				}
			}

			if (PhysicsMesh)
			{
				MUTABLE_CPUPROFILER_SCOPE(GetConstant_Mesh_Physics)
				
				if (!Result)
				{
					Result = CreateMesh(MeshBudgetReserve);
					Result->CopyFrom(*PhysicsMesh);
				}
				else
				{
					Result->PhysicsBody = PhysicsMesh->PhysicsBody;

					for (const TPair<EMeshBufferType, FMeshBufferSet>& AdditionalBuffer : PhysicsMesh->AdditionalBuffers)
					{
						const bool bIsPhysicsBufferType = 
								AdditionalBuffer.Key == EMeshBufferType::PhysicsBodyDeformBinding   ||
								AdditionalBuffer.Key == EMeshBufferType::PhysicsBodyDeformSelection ||
								AdditionalBuffer.Key == EMeshBufferType::PhysicsBodyDeformOffsets;

						if (bIsPhysicsBufferType)
						{
							Result->AdditionalBuffers.Emplace(AdditionalBuffer);
						}
					}
				}

				if (TManagedPtr<FPhysicsBody> PhysicsBody = ConstCastManagedPtr<FPhysicsBody>(Result->PhysicsBody))
				{
					PhysicsBody->ConvertBoneIdsToNames(Program.ConstantNames);
				}
			}

			if (MetaDataMesh)
			{
				MUTABLE_CPUPROFILER_SCOPE(GetConstant_Mesh_Metadata)

				if (!Result)
				{
					Result = CreateMesh(MeshBudgetReserve);
					Result->CopyFrom(*MetaDataMesh);
				}
				else
				{
					// Only in case the geometry has been filtered, add the meta data descriptors.
					if (EnumHasAnyFlags(MeshContentRange.GetContentFlags(), EMeshContentFlags::GeometryData) &&
						!EnumHasAnyFlags(FilterContentFlags, EMeshContentFlags::GeometryData))
					{
						check(MetaDataMesh->VertexBuffers.IsDescriptor());
						check(MetaDataMesh->IndexBuffers.IsDescriptor());

						Result->VertexBuffers = MetaDataMesh->VertexBuffers;
						Result->IndexBuffers = MetaDataMesh->IndexBuffers;
						Result->Surfaces = MetaDataMesh->Surfaces;
						Result->ClothSections = MetaDataMesh->ClothSections;
					}

					Result->GameplayTags = MetaDataMesh->GameplayTags; 
					Result->SkeletonObjects = MetaDataMesh->SkeletonObjects;
					Result->AssetUserData = MetaDataMesh->AssetUserData; 
					Result->AnimationSlots = MetaDataMesh->AnimationSlots; 
					Result->Morph = MetaDataMesh->Morph;
				}
			}

			Result->MeshIDPrefix = MeshContentRange.MeshIDPrefix;

			if (Program.ConstantSkeletons.IsValidIndex(SkeletonConstantIndex))
			{
				Result->Skeleton = Program.ConstantSkeletons[SkeletonConstantIndex];

				Result->ConvertBoneIdsToIndices();
			}

			if (Result->ClothSections.Num())
			{
				Result->ClothSections[0].ClothingAsset = ClothAsset;	
			}
			
			OutMesh = Result;
		}
    	
    	FProgram& GetProgram()
    	{
    		return Program;
    	}

    	const FProgram& GetProgram() const
    	{
    		return Program;
    	}
    
    private:
    	FProgram Program;
    	
    public:
    	FRomManager RomManager;
	};

}

#undef UE_API
