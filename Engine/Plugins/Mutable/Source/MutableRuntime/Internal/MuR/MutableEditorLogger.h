// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/TokenizedMessage.h"

/** Class used to log messages from CO compilations and COI updates to the open editor. */
class IMutableEditorLogger
{
public:

	virtual ~IMutableEditorLogger() = default;

	/** Logs the Message */
	virtual bool LogMessage(TSharedRef<FTokenizedMessage> Message) = 0;

	/** Clears all the visible Logs */
	virtual void ClearLogMessageList() = 0;
};

