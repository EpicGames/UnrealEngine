// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Types.h"
#include "MathUtil.h"
#include "Math/Matrix.h"
#include "Math/IntVector.h"


namespace UE::Mutable::Private
{

	template<class VECTOR>
	inline int32 GetDim()
	{
		return VECTOR::GetDim();
	}

	template<>
	inline int32 GetDim<FVector2f>()
	{
		return 2;
	}

	template<>
	inline int32 GetDim<FVector3f>()
	{
		return 3;
	}

	template<>
	inline int32 GetDim<FIntVector2>()
	{
		return 2;
	}

	template<>
	inline int32 GetDim<UE::Math::TIntVector2<uint16>>()
	{
		return 2;
	}

	template<class VECTOR>
	inline bool AllGreaterOrEqualThan(const VECTOR& v0, const VECTOR& v1)
	{
		return v0.AllGreaterOrEqualThan(v1);
	}

	template<>
	inline bool AllGreaterOrEqualThan<FIntVector2>(const FIntVector2& v0, const FIntVector2& v1)
	{
		return  (v0[0] >= v1[0]) && (v0[1] >= v1[1]);
	}

	template<class VECTOR>
	inline bool AllSmallerThan(const VECTOR& v0, const VECTOR& v1)
	{
		return v0.AllSmallerThan(v1);
	}

	template<>
	inline bool AllSmallerThan<FIntVector2>(const FIntVector2& v0, const FIntVector2& v1)
	{
		return  (v0[0] < v1[0]) && (v0[1] < v1[1]);
	}

	template<class VECTOR>
	inline bool AllSmallerOrEqualThan(const VECTOR& v0, const VECTOR& v1)
	{
		return v0.AllSmallerOrEqualThan(v1);
	}

	template<>
	inline bool AllSmallerOrEqualThan<FIntVector2>(const FIntVector2& v0, const FIntVector2& v1)
	{
		return  (v0[0] <= v1[0]) && (v0[1] <= v1[1]);
	}

	/** Apply a Linear to sRGB color transformation on a given color vector. */
	inline void ConvertLinearColorToSRGB(FVector4f& Color)
	{
		// Sourced from MPCDIUtils.ush
		auto LinearToSRGBFloat = [](const float Color) -> float
			{
				return (Color <= 0.0031308f) ? Color * 12.92f : 1.055f * FMathf::Pow(Color, 1.0f / 2.4f) - 0.055f;
			};

		Color.X = LinearToSRGBFloat(Color.X);
		Color.Y = LinearToSRGBFloat(Color.Y);
		Color.Z = LinearToSRGBFloat(Color.Z);
	}

	//!
    template <class VECTOR>
    class box
	{
	public:

		VECTOR min;
		VECTOR size;

	public:

		box() : min(0), size(0) {}

        //-----------------------------------------------------------------------------------------
        static box<VECTOR> FromMinSize(VECTOR m, VECTOR s)
        {
            box<VECTOR> result;
            result.min = m;
            result.size = s;
            return result;
        }

		//-----------------------------------------------------------------------------------------
		inline bool Contains(const VECTOR& v) const
		{
			return AllGreaterOrEqualThan(v, min) && AllSmallerThan(v, min + size);
		}

		//-----------------------------------------------------------------------------------------
		inline bool ContainsInclusive(const VECTOR& v) const
		{
			return AllGreaterOrEqualThan(v, min) && AllSmallerOrEqualThan(v, min + size);
		}

		//-----------------------------------------------------------------------------------------
		inline bool Contains(const box<VECTOR>& v) const
		{
			return Contains(v.min) && ContainsInclusive(v.min+v.size);
		}

        //-----------------------------------------------------------------------------------------
        //! It is inclusive: If the borders touch it is considered an intersection.
        //-----------------------------------------------------------------------------------------
        inline bool Intersects(const box<VECTOR>& other) const
        {
            bool result = true;
            for (int d = 0; d < GetDim<VECTOR>(); ++d)
            {
                result = result &&
                         FMath::Abs((min[d] * 2 + size[d]) - (other.min[d] * 2 + other.size[d])) <=
                             (size[d] + other.size[d]);
            }

            return result;
        }

        //-----------------------------------------------------------------------------------------
        //! It is exclusive: If the borders touch it is not considered an intersection.
        //-----------------------------------------------------------------------------------------
        inline bool IntersectsExclusive(const box<VECTOR>& other) const
        {
            bool result = true;
            for (int d = 0; d < GetDim<VECTOR>(); ++d)
            {
                result = result &&
                         FMath::Abs((min[d] * 2 + size[d]) - (other.min[d] * 2 + other.size[d])) <
                             (size[d] + other.size[d]);
            }

            return result;
        }

        //-----------------------------------------------------------------------------------------
        //! If the boxes don't intersect, at least one size of the returned box will be negative.
        //-----------------------------------------------------------------------------------------
        inline box<FIntVector2> Intersect2i(const box<FIntVector2>& other) const
        {
			constexpr int32 Dim = 2;
			box<VECTOR> result;
            for (int d = 0; d < Dim; ++d)
            {
                result.min[d] = FMath::Max(min[d], other.min[d]);
                result.size[d] =
                    FMath::Min(int32(min[d] + size[d] - result.min[d]),
                               int32(other.min[d] + other.size[d] - result.min[d]));
            }
            return result;
        }

        //-----------------------------------------------------------------------------------------
		VECTOR Homogenize( const VECTOR& v ) const
		{
			return (v-min)/size;
		}

		//-----------------------------------------------------------------------------------------
		inline void Bound(const VECTOR& v)
		{
			for (int32 d = 0; d < GetDim<VECTOR>(); ++d)
			{
				if (v[d] < min[d])
				{
					size[d] += min[d] - v[d];
					min[d] = v[d];
				}
				else if (v[d] - min[d] > size[d])
				{
					size[d] = v[d] - min[d];
				}
			}
		}

		//-----------------------------------------------------------------------------------------
		inline void Bound3(const VECTOR& v)
		{
			for (int d = 0; d < 3; ++d)
			{
				if (v[d] < min[d])
				{
					size[d] += min[d] - v[d];
					min[d] = v[d];
				}
				else if (v[d] - min[d] > size[d])
				{
					size[d] = v[d] - min[d];
				}
			}
		}


		//-----------------------------------------------------------------------------------------
		inline void Bound(const box<VECTOR>& other)
		{
			Bound(other.min);
			Bound(other.min + other.size);
		}

		//-----------------------------------------------------------------------------------------
		inline bool IsEmpty() const
		{
			return size[0] <= 0 || size[1] <= 0;
		}

		//-----------------------------------------------------------------------------------------
		//! Strict ordering of boxes.
		//-----------------------------------------------------------------------------------------
        inline bool operator<( const box<VECTOR>& other ) const
        {
            if( size < other.size )
            {
                return true;
            }
            else if( size == other.size )
            {
                return min < other.min;
            }

            return false;
        }

		//-----------------------------------------------------------------------------------------
		inline bool operator==(const box<VECTOR>& other) const
		{
			return (size == other.size)
				&&
				(min == other.min);
		}
	};


}

