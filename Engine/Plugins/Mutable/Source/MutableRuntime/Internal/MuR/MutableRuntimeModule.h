// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

#define UE_API MUTABLERUNTIME_API


MUTABLERUNTIME_API DECLARE_LOG_CATEGORY_EXTERN(LogMutableCore, Log, All);


class FMutableRuntimeModule : public IModuleInterface
{
public:

	// IModuleInterface 
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

};

#undef UE_API
