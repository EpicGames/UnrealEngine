// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssertionMacros.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Serialisation.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{

    /** Container for text string data. */
    class String : public FResource
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------

		UE_API String(const FString&);

		//! Deep clone this string.
		UE_API TSharedPtr<String> Clone() const;

		// Resource interface
		UE_API int32 GetDataSize() const override;

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		//! Get the string data.
		UE_API const FString& GetValue() const;

	public:

		//!
		FString Value;

		//!
		inline bool operator==(const String& o) const
		{
			return (Value == o.Value);
		}

	};

}

#undef UE_API
