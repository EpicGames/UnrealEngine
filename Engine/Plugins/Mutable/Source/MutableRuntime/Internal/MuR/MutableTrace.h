// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Trace.h"
#include "HAL/PlatformTime.h"


/** Custom Mutable profiler scope. */
#define MUTABLE_CPUPROFILER_SCOPE(Name) \
	TRACE_CPUPROFILER_EVENT_SCOPE(Mutable_##Name)

/** Custom Mutable profiler scope. */
#define MUTABLE_EDITOR_CPUPROFILER_SCOPE(Name) \
	TRACE_CPUPROFILER_EVENT_SCOPE(MutableEditor_##Name)


/** Simple class that saves the time the scope is alive in the given location. */
class FMutableScopeTimer
{
public:
	FMutableScopeTimer(double& InResult) : Result(InResult)
	{
		StartTime = FPlatformTime::Seconds();
	}

	~FMutableScopeTimer()
	{
		Result =  FPlatformTime::Seconds() - StartTime;
	}
	
private:
	double StartTime = 0.0;
	double& Result;
};
