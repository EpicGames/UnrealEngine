// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"
#include "MuR/ConvertData.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	//! Grow the image as if it was a bitmap
	//---------------------------------------------------------------------------------------------
	inline void ImageGrow( FImage* pImage )
	{
		check(pImage && pImage->GetFormat()== EImageFormat::L_UByte );

		MUTABLE_CPUPROFILER_SCOPE(ImageGrow1)

		int sizeX = pImage->GetSizeX();
		int sizeY = pImage->GetSizeY();
		check( sizeX>0 && sizeY>0 );

		TManagedPtr<const FImage> pSource = pImage->Clone();

		int rowSize = sizeX*GetImageFormatData( pImage->GetFormat() ).BytesPerBlock;

		for ( int y=0; y<sizeY; ++y )
		{
			int y0 = FMath::Max( 0, FMath::Min( sizeY-1, y-1 ) );
			int y1 = FMath::Max( 0, FMath::Min( sizeY-1, y   ) );
			int y2 = FMath::Max( 0, FMath::Min( sizeY-1, y+1 ) );

            const uint8_t* pS0 = pSource->GetLODData(0) + y0 * rowSize;
            const uint8_t* pS1 = pSource->GetLODData(0) + y1 * rowSize;
            const uint8_t* pS2 = pSource->GetLODData(0) + y2 * rowSize;

            uint8_t* pR = pImage->GetLODData(0) + y1 * rowSize;

			if ( pS0[0] || pS2[0] || pS1[1] )
			{
				pR[0] = 255;
			}

			if ( pS0[sizeX-1] || pS2[sizeX-1] || pS1[sizeX-2] )
			{
				pR[sizeX-1] = 255;
			}

			for ( int x=1; x<sizeX-1; ++x )
			{
				if ( pS0[x] || pS2[x] || pS1[x-1] || pS1[x+1] )
				{
					pR[x] = 255;
				}
			}

		}

	}


	//---------------------------------------------------------------------------------------------
	//! Grow the image copying border colors, by using an external mask as reference
	//---------------------------------------------------------------------------------------------
	inline void ImageGrow( FImage* pImage, const FImage* pMask )
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageGrow2)

		check( pMask->GetFormat()== EImageFormat::L_UByte );

		int sizeX = pImage->GetSizeX();
		int sizeY = pImage->GetSizeY();
		check( sizeX>0 && sizeY>0 );

        if (sizeX<2 || sizeY<2)
        {
            return;
        }

		int pixelSize = GetImageFormatData( pImage->GetFormat() ).BytesPerBlock;
		int rowSize = sizeX*pixelSize;

		for ( int y=0; y<sizeY; ++y )
		{
            const uint8_t* pS0 = y>0 ? pMask->GetLODData(0) + (y-1) * sizeX : nullptr;
            const uint8_t* pS1 = pMask->GetLODData(0) + y * sizeX;
            const uint8_t* pS2 = y<sizeY-1 ? pMask->GetLODData(0) + (y+1) * sizeX : nullptr;

            const uint8_t* pR0 = y>0 ? pImage->GetLODData(0) + (y-1) * rowSize : nullptr;
                  uint8_t* pR1 = pImage->GetLODData(0) + y * rowSize;
            const uint8_t* pR2 = y<sizeY-1 ? pImage->GetLODData(0) + (y+1) * rowSize : nullptr;

			for ( int x=0; x<sizeX; ++x )
			{
				if (pS1[x])
				{
					// Nothing to do.
				}
				else if ( y>0 && pS0[x] )
				{
					memcpy( &pR1[x*pixelSize], &pR0[x*pixelSize], pixelSize );
				}
				else if ( y<sizeY-1 && pS2[x] )
				{
					memcpy( &pR1[x*pixelSize], &pR2[x*pixelSize], pixelSize );
				}
				else if ( x<sizeX-1 && pS1[x+1] )
				{
					memcpy( &pR1[x*pixelSize], &pR1[(x+1)*pixelSize], pixelSize );
				}
				else if ( x>0 && pS1[x-1] )
				{
					memcpy( &pR1[x*pixelSize], &pR1[(x-1)*pixelSize], pixelSize );
				}
			}

		}

	}

}
