// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/IntVector.h"

namespace UE::Mutable::Private
{
	class FImage;
	class FMesh;
	struct FProjector;
	enum class ESamplingMethod : uint8;
	template <int NUM_INTERPOLATORS> class RasterVertex;

	struct FScratchImageProject
	{
		TArray<RasterVertex<4>> Vertices;
		TArray<uint8> CulledVertex;
	};

    extern void ImageRasterProjectedPlanar( const FMesh* pMesh, FImage* pTargetImage,
		const FImage* pSource, const FImage* pMask,
		bool bIsRGBFadingEnabled, bool bIsAlphaFadingEnabled,
		ESamplingMethod SamplingMethod,
		float FadeStart, float FadeEnd, float MipInterpolationFactor,
		int Layout, uint64 BlockId,
		UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
		FScratchImageProject* scratch, bool bUseVectorImplementation = false);

    extern void ImageRasterProjectedCylindrical( const FMesh* pMesh, FImage* pTargetImage,
		const FImage* pSource, const FImage* pMask,
		bool bIsRGBFadingEnabled, bool bIsAlphaFadingEnabled,
		ESamplingMethod SamplingMethod,
		float FadeStart, float FadeEnd, float MipInterpolationFactor,
		int32 Layout,
		float ProjectionAngle,
		UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
		FScratchImageProject* Scratch, bool bUseVectorImplementation = false);

    extern void ImageRasterProjectedWrapping( const FMesh* pMesh, FImage* pTargetImage,
		const FImage* pSource, const FImage* pMask,
		bool bIsRGBFadingEnabled, bool bIsAlphaFadingEnabled,
		ESamplingMethod SamplingMethod,
		float FadeStart, float FadeEnd, float MipInterpolationFactor,
		int32 Layout, uint64 BlockId,
		UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
		FScratchImageProject* Scratch, bool bUseVectorImplementation = false);

	extern float ComputeProjectedFootprintBestMip(
			const FMesh* pMesh, const FProjector& Projector, const FVector2f& TargetSize, const FVector2f& SourceSize);

    extern void MeshProject(FMesh* Result, const FMesh* pMesh, const FProjector& Projector, int32 LayoutIndex, bool& bOutSuccess);

	MUTABLERUNTIME_API extern void CreateMeshOptimisedForProjection(FMesh* Result, int32 LayoutIndex);
	MUTABLERUNTIME_API extern void CreateMeshOptimisedForWrappingProjection(FMesh* Result, int32 LayoutIndex);

}
