// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Platform.h"

namespace UE::Mutable::Private
{
	class FMesh;
	class FMeshBufferSet;
	struct FMeshBuffer;

	/** Convert a mesh format into another one.
	* Slow implementation, but it should never happen at run-time
	* \param keepSystemBuffers Will keep the internal system buffers even if they are not in the
	* original format. If they are, they will be duplicated, so be careful.
	*/
	MUTABLERUNTIME_API extern void MeshFormat
		(
			FMesh* Result,
			const FMesh* Source,
			const FMesh* Format,
			bool bKeepSystemBuffers,
			bool bFormatVertices,
			bool bFormatIndices,
			bool bIgnoreMissingChannels,
			bool& bOutSuccess
		);


    /** Fill a mesh buffer in Result (with an optional offset) with the current data in Source but keeping the format already in the Result. */
    void MeshFormatBuffer( const FMeshBufferSet& Source, FMeshBuffer& Result, int32 ResultElementOffset, bool bHasSpecialSemantics, uint32 IDPrefix);

	/** Try to reduce the mesh size by reducing the component count, and data type of some buffers. */
	MUTABLERUNTIME_API void MeshOptimizeBuffers( FMesh* );

}
