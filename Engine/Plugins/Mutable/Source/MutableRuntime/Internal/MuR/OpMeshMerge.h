// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ManagedPointer.h"

#define UE_API MUTABLERUNTIME_API 

namespace UE::Mutable::Private
{
	class FMesh;
	class FSkeleton;

	/** Merge two meshes into one new mesh */
	void MeshMerge(FMesh* Result, const TManagedPtr<const FMesh>& First, const TManagedPtr<const FMesh>& Second, bool bMergeSurfaces);


	/** Merge two skeletons, OutRemappedOtherBoneIndices containts the map from bone in Other to bone in Base */
	UE_API void MergeSkeletons(FSkeleton& Base, const FSkeleton& Other, TArray<int32>& OutRemappedOtherBoneIndices);

	/** Merge multiple meshes into one new mesh. */
	void MergeLODMeshesForConversion(FMesh* Result, TArray<TManagedPtr<const FMesh>>& Meshes, bool bIsInitialGeneration);
}

#undef UE_API
