// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Array.h"
#include "Math/Vector.h"

namespace UE::Mutable::Private
{
	class FMesh;
	class FLayout;

	/** Modify the mesh to use the given layout. 
	* It will be added to the mesh and additional mesh buffers created for the layout block ids. 
	* It may also modify the UVs to clamp them to the blocks if necessary.
	*/
	MUTABLERUNTIME_API void MeshPrepareLayout(
		FMesh& Mesh,
		const FLayout& InLayout,
		int32 LayoutChannel,
		bool bNormalizeUVs,
		bool bClampUVIslands,
		bool bEnsureAllVerticesHaveLayoutBlock,
		bool bUseAbsoluteBlockIds
	);


	/** Store the result of a layout vertex classify operation. */
	struct FMutableLayoutClassifyResult
	{
		/** The tex coords of every mesh vertex. They could be modified during the classify operation due to forced block clamping. */
		TArray<FVector2f> TexCoords;

		/** A block index for every mesh vertex. 
		* These are indices in the array of blocks of the given FLayout object to use to classify.
		* It could be NullBlockId if it didn't belong to a block. 
		*/
		TArray<uint16> LayoutData;

		/** Id of unassigned blocks. */
		static constexpr uint16 NullBlockId = TNumericLimits<uint16>::Max();
	};


	/** Assign a layout block to every vertex in the mesh if possible. 
	* If a vertex cannot be assigned, it will remain as NullBlockId.
	* Vertex UVs for the given layout will be copied and may be modified in the output struct if they needed to be clamped to fit in a block,
	* but the mesh won't be modified by this method.
	*/
	MUTABLERUNTIME_API bool ClassifyMeshVerticesInLayout(
		const FMesh& Mesh,
		const FLayout& InLayout,
		int32 LayoutChannel,
		bool bNormalizeUVs,
		bool bClampUVIslands,
		FMutableLayoutClassifyResult& OutResult
	);

}
