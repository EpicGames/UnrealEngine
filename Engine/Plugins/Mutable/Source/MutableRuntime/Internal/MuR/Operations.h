// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentId.h"
#include "PassthroughObject.h"
#include "MuR/ImageTypes.h"
#include "MuR/Types.h"
#include "MuR/Serialisation.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"

#define MUTABLE_OP_MAX_INTERPOLATE_COUNT	6
#define MUTABLE_OP_MAX_SWIZZLE_CHANNELS		4


namespace UE::Mutable::Private
{
	enum class EImageFormat : uint8;

    /** 
     * Operation type enum. 
     * All operations gerenrating a specific type **MUST** be contiguous and be between 
     * its TYPE_RANGE_BEGIN and TYPE_RANGE_END
     */
    enum class EOpType : uint8
    {
        /** No operation. */
        NONE,

        //-----------------------------------------------------------------------------------------
        // Boolean operations
        //-----------------------------------------------------------------------------------------
        
        /** Boolean constant */
        BO_CONSTANT,
        
        BO_RANGE_BEGIN = BO_CONSTANT,

        /** Boolean parameter */
        BO_PARAMETER,

        /** Compare an integerexpression with an integer constant */
        BO_EQUAL_INT_CONST,

        /** Logical and */
        BO_AND,

        /** Logical or */
        BO_OR,

        /** Logical not */
        BO_NOT,
        

        //-----------------------------------------------------------------------------------------
        // Number operations
        //-----------------------------------------------------------------------------------------
        
        /** Constant value */
        NU_CONSTANT,
        
        BO_RANGE_END   = NU_CONSTANT,
        NU_RANGE_BEGIN = BO_RANGE_END,
        
        /** User parameter */
        NU_PARAMETER,

        /** Select one value or the other depending on a boolean input */
        NU_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
        NU_SWITCH,
       

        //-----------------------------------------------------------------------------------------
        // Scalar operations
        //-----------------------------------------------------------------------------------------
         
        /** Constant value */
        SC_CONSTANT,
        
        NU_RANGE_END   = SC_CONSTANT,
        SC_RANGE_BEGIN = NU_RANGE_END,

        /** User parameter */
        SC_PARAMETER,
        
        /** Select one value or the other depending on a boolean input */
        SC_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
        SC_SWITCH,
        
        SC_MATERIAL_BREAK,
        
        /** Apply an arithmetic operation to two scalars */
        SC_ARITHMETIC,

        /** Get a scalar value from a curve */
        SC_CURVE,
		
    	/** External operation */
		SC_EXTERNAL,

        
        //-----------------------------------------------------------------------------------------
        // Color operations. Colors are sometimes used as generic vectors.
        //-----------------------------------------------------------------------------------------
        
        /** Constant value */
        CO_CONSTANT,

        SC_RANGE_END   = CO_CONSTANT,
        CO_RANGE_BEGIN = SC_RANGE_END,

        /** User parameter */
        CO_PARAMETER,
        
        /** Select one value or the other depending on a boolean input */
        CO_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
        CO_SWITCH,
        
        CO_MATERIAL_BREAK,
        
        /** Sample an image to get its color */
        CO_SAMPLEIMAGE,

        /** Make a color by shuffling channels from other color */
        CO_SWIZZLE,

        /** Compose a vector from 4 scalars */
        CO_FROMSCALARS,

        /** Apply component-wise arithmetic operations to two colors */
        CO_ARITHMETIC,

		/** Apply a Linear to sRGB color transformation on a given color vector */
		CO_LINEARTOSRGB,
		
    	/** External operation */
		CO_EXTERNAL,

        //-----------------------------------------------------------------------------------------
        // String operations
        //-----------------------------------------------------------------------------------------

        /** Constant value */
        ST_CONSTANT,
        
        CO_RANGE_END   = ST_CONSTANT,
        ST_RANGE_BEGIN = CO_RANGE_END,
        
		ST_PARAMETER,
      

        //-----------------------------------------------------------------------------------------
        // Projector operations
        //-----------------------------------------------------------------------------------------
        
        /** Constant value */
        PR_CONSTANT, 

        ST_RANGE_END   = PR_CONSTANT,
        PR_RANGE_BEGIN = ST_RANGE_END,
        
		PR_PARAMETER,


        //-----------------------------------------------------------------------------------------
        // Image operations
        //-----------------------------------------------------------------------------------------

        IM_CONSTANT,
        
        PR_RANGE_END   = IM_CONSTANT,
        IM_RANGE_BEGIN = PR_RANGE_END,
        
        /** User parameter */
		IM_PARAMETER,
        
		/** A referenced, but opaque engine image */
		IM_REFERENCE,
        
        /** Select one value or the other depending on a boolean input */
        IM_CONDITIONAL,

        IM_SWITCH,
        
        IM_MATERIAL_BREAK,

		/** Converts a texture of a material parameter into a texture parameter to process it at runtime. */
		IM_PARAMETER_FROM_MATERIAL,
        
        /** Combine an image on top of another one using a specific effect (Blend, SoftLight,
	    Hardlight, Burn...). And optionally a mask. */
        IM_LAYER,

        /** Apply a color on top of an image using a specific effect (Blend, SoftLight,
		Hardlight, Burn...), optionally using a mask. */
        IM_LAYERCOLOR,        

        /** Convert between pixel formats */
        IM_PIXELFORMAT,


		/** Conform the image to a given image descriptor */
		IM_CONFORM,

        /** Generate mipmaps up to a provided level */
        IM_MIPMAP,

        /** Resize the image to a constant size */
        IM_RESIZE,

        /** Resize the image to the size of another image */
        IM_RESIZELIKE,

        /** Resize the image by a relative factor */
        IM_RESIZEREL,

        /** Create an empty image to hold a particular layout. */
        IM_BLANKLAYOUT,

        /** Copy an image into a rect of another one. */
        IM_COMPOSE,

		/** Copy blocks from an image into a another one. */
		IM_MULTICOMPOSE,

        /** Interpolate between 2 images taken from a row of targets (2 consecutive targets). */
        IM_INTERPOLATE,

        /** Change the saturation of the image. */
        IM_SATURATE,

        /** Generate a one-channel image with the luminance of the source image. */
        IM_LUMINANCE,

        /** Recombine the channels of several images into one. */
        IM_SWIZZLE,

        /** Convert the source image colors using a "palette" image sampled with the source */
        /** grey-level. */
        IM_COLORMAP,

        /** Generate a black and white image from an image and a threshold. */
        IM_BINARISE,

        /** Generate a plain color image */
        IM_PLAINCOLOR,

        /** Cut a rect from an image */
        IM_CROP,

        /** Replace a subrect of an image with another one */
        IM_PATCH,

        /** Render a mesh texture layout into a mask */
        IM_RASTERMESH,

        /** Create an image displacement encoding the grow operation for a mask */
        IM_MAKEGROWMAP,

        /** Apply an image displacement on another image. */
        IM_DISPLACE,

        /** Repeately apply */
        IM_MULTILAYER,

        /** Inverts the colors of an image */
        IM_INVERT,

        /** Modifiy roughness channel of an image based on normal variance. */
        IM_NORMALCOMPOSITE,

		/** Apply linear transform to Image content. Resulting samples outside the original image are tiled. */
		IM_TRANSFORM,

        /** Convert an FIamge in Passthrough Parameter mode to an FImage with data. */
    	IM_PARAMETER_CONVERT,
			
        /** External operation */
		IM_EXTERNAL,
    
        //-----------------------------------------------------------------------------------------
        // Mesh operations
        //-----------------------------------------------------------------------------------------

        /** Constant value */
        ME_CONSTANT,
       
        IM_RANGE_END   = ME_CONSTANT,
        ME_RANGE_BEGIN = IM_RANGE_END, 

        /** User parameter */
        ME_PARAMETER,

		ME_REFERENCE,
        
        /** Select one value or the other depending on a boolean input */
        ME_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
        ME_SWITCH,
        
        /** Apply a layout to a mesh texture coordinates channel */
        ME_APPLYLAYOUT,

		/** */
		ME_PREPARELAYOUT,

        /** Compare two meshes and extract a morph from the first to the second
        The meshes must have the same topology, etc. */
        ME_DIFFERENCE,

        /** Apply a one morphs on a base.  */
        ME_MORPH,

        /** Merge a mesh to a mesh */
        ME_MERGE,

        /** Create a new mask mesh selecting all the faces of a source that are inside a given
        clip mesh. */
        ME_MASKCLIPMESH,

        /** Create a new mask mesh selecting the faces of a source that have UVs inside the 
        region marked in an image mask. */
		ME_MASKCLIPUVMASK,

        /** Create a new mask mesh selecting all the faces of a source that match another mesh. */
        ME_MASKDIFF,

        /** Remove all the geometry selected by a mask. */
        ME_REMOVEMASK,

        /** Change the mesh format to match the format of another one. */
        ME_FORMAT,

        /** Extract a fragment of a mesh containing specific layout blocks. */
        ME_EXTRACTLAYOUTBLOCK,

        /** Apply a transform in a 4x4 matrix to the geometry channels of the mesh */
        ME_TRANSFORM,

        /** Clip the mesh with a plane and morph it when it is near until it becomes an ellipse on the plane. */
        ME_CLIPMORPHPLANE,

        /** Clip the mesh with another mesh. */
        ME_CLIPWITHMESH,

        /** Project a mesh using a projector and clipping the irrelevant faces */
        ME_PROJECT,

        /** Deform a skinned mesh applying a skeletal pose */
        ME_APPLYPOSE,

		/** Calculate the binding of a mesh on a shape */
		ME_BINDSHAPE,

		/** Apply a shape on a (previously bound) mesh */
		ME_APPLYSHAPE,

		/** Clip Deform using bind data. */
		ME_CLIPDEFORM,

		/** Add a metadata to a mesh */
		ME_ADDMETADATA,

		/** Set the Material Slot Id to the mesh */
		ME_SETMATERIALSLOTID,

    	/** Transform with a 4x4 matrix the geometry channels of a mesh that are bounded by another mesh */
    	ME_TRANSFORMWITHMESH,

		/** Transform with a 4x4 matrix the geometry channels of a mesh that are skinned to a bone or hierarchy */
		ME_TRANSFORMWITHBONE,

    	/** External operation */
    	ME_EXTERNAL,

        /** Select a Mesh Section from a Skeletal Mesh. */
    	ME_SKELETALMESH_BREAK,

    	//-----------------------------------------------------------------------------------------
		// LOD operations
		//-----------------------------------------------------------------------------------------

    	LD_CONDITIONAL,
    	
    	ME_RANGE_END   = LD_CONDITIONAL,
		LD_RANGE_BEGIN = ME_RANGE_END,
    	
    	LD_SWITCH,
    	
    	/** Create a new LOD from Meshes. */
		LD_NEW,

    	//-----------------------------------------------------------------------------------------
		// Skeletal Mesh operations
		//-----------------------------------------------------------------------------------------

    	SK_CONDITIONAL,
    	
        LD_RANGE_END   = SK_CONDITIONAL,
        SK_RANGE_BEGIN = LD_RANGE_END,

    	SK_SWITCH,
    	
    	/** Create a new Skeletal Mesh from LODs. */
		SK_NEW,
	
    	/** User parameter */
		SK_PARAMETER,

		SK_MERGE,
    
        /** Convert a SkeletalMesh in Passthrough mode to an FSkeletalMesh with data. */
		SK_CONVERT,

		/** Apply a morph to a Skeletal Mesh */
		SK_MORPH,

		/** Transfer a mesh defromation from one SkletalMesh to another */
		SK_RESHAPE,

		/** SkeletalMesh material modification. */
		SK_MATERIALMODIFY,

		/** Clip parts of a skeletal mesh inside the volume defined by another SkeletalMesh */
		SK_CLIPMESHWITHMESH,
    	
		/** Apply a transform in a 4x4 matrix to the geometry channels of the Skeletal Mesh */
		SK_TRANSFORM,

		/** Transform with a 4x4 matrix the geometry channels of a Skeletal Mesh that are skinned to a bone or hierarchy */
		SK_TRANSFORMWITHBONE,

		/** Convert a FSkeletalMesh to USkeletalMesh. */
    	SKO_CONVERT,

        //-----------------------------------------------------------------------------------------
        // Instance operations
        //-----------------------------------------------------------------------------------------

        /** Select one value or the other depending on a boolean input */
        IN_CONDITIONAL,

        SK_RANGE_END   = IN_CONDITIONAL,
        IN_RANGE_BEGIN = SK_RANGE_END,
        
        /** Select one of several values depending on an int input */
        IN_SWITCH,
        
        /** Add a component to an instance LOD */
        IN_ADDCOMPONENT,

    	/** Add a Skeletal Mesh to an instance. */
		IN_ADDSKELETALMESH,

		/** Add extension data to an instance */
		IN_ADDEXTENSIONDATA,

		/** Add overlay material to an instance */
		IN_ADDOVERLAYMATERIAL,

		/** Add override material to an instance */
		IN_ADDOVERRIDEMATERIAL,

        //-----------------------------------------------------------------------------------------
        // Layout operations
        //-----------------------------------------------------------------------------------------

        /** Constant value */
        LA_CONSTANT,

        IN_RANGE_END   = LA_CONSTANT,
        LA_RANGE_BEGIN = IN_RANGE_END,
        
        /** Select one value or the other depending on a boolean input */
        LA_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
        LA_SWITCH,
        
        /** Pack all the layout blocks from the source in the grid without overlapping */
        LA_PACK,

        /** Merge two layouts */
        LA_MERGE,

        /** 
         * Remove all layout blocks not used by any vertex of the mesh. 
         * This operation is for the new way of managing layout blocks. 
         */
        LA_REMOVEBLOCKS,

		/** Extract a layout from a mesh */
		LA_FROMMESH,

    	//-----------------------------------------------------------------------------------------
	    // Material operations
	    //-----------------------------------------------------------------------------------------

        /** Constant value */
		MI_CONSTANT,

        LA_RANGE_END   = MI_CONSTANT,
        MI_RANGE_BEGIN = LA_RANGE_END,

        /** User parameter */
    	MI_PARAMETER,
        
        /** Select one value or the other depending on a boolean input */
		MI_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
		MI_SWITCH,
    	
        /** Get a material from a given skeletal mesh */
		MI_SKELETALMESHOBJECT_BREAK,

    	/** Get a material from a given skeletal mesh */
		MI_SKELETALMESH_BREAK,

        /** Get a material from a given skeletal mesh slot */
    	MI_FROM_SKELETALMESH_SLOT,

        /** Modifies the parameter values of a material */
		MI_MODIFY,

        /** External operation */
		MI_EXTERNAL,

    	
    	//-----------------------------------------------------------------------------------------
	    // Matrix operations
	    //-----------------------------------------------------------------------------------------

        /** User parameter */
    	MA_CONSTANT,

        MI_RANGE_END   = MA_CONSTANT,
        MA_RANGE_BEGIN = MI_RANGE_END,

        /** User parameter */
    	MA_PARAMETER,
   
    	//-----------------------------------------------------------------------------------------
	    // ExtensionData operations
	    //-----------------------------------------------------------------------------------------

		ED_CONSTANT,

	    MA_RANGE_END   = ED_CONSTANT,
        ED_RANGE_BEGIN = MA_RANGE_END,

        /** Select one value or the other depending on a boolean input */
		ED_CONDITIONAL,
        
        /** Select one of several values depending on an int input */
        ED_SWITCH,

        //-----------------------------------------------------------------------------------------
		// FInstancedStruct operations
		//-----------------------------------------------------------------------------------------

        /** User parameter */
    	IS_PARAMETER,

        ED_RANGE_END   = IS_PARAMETER,
        IS_RANGE_BEGIN = ED_RANGE_END,

        /** External operation */
		IS_EXTERNAL,

    	IS_CONDITIONAL,
    	
    	IS_SWITCH,
    
        IS_RANGE_END,

        //-----------------------------------------------------------------------------------------
        // Utility values
        //-----------------------------------------------------------------------------------------

        /** */
        COUNT = IS_RANGE_END, 
    };

	inline uint32 GetTypeHash(const EOpType& OpType)
	{
		return static_cast<uint32>(OpType);
	}

	/** Types of data handled by the Mutable runtime. */
	enum class EDataType : uint8
	{
		None = 0,
		Bool,
		Int,
		Scalar,
		Color,
		Image,
		Layout,
		Mesh,
		Instance,
		Projector,
		String,
		ExtensionData,
		Matrix,
		Material,
		InstancedStruct,
		SkeletalMesh,
		LOD,

		// Supporting data types : Never returned as an actual data type for any operation.
		Shape,
		Curve,
		Skeleton,
		PhysicsAsset,

		Count
	};

    /** Computes the data type of a given op type */
    FORCEINLINE EDataType GetOpDataType(EOpType OpType)
    { 
        uint8 OpBo = static_cast<uint8>(((OpType >= EOpType::BO_RANGE_BEGIN) & (OpType < EOpType::BO_RANGE_END)) 
                ? EDataType::Bool : EDataType::None); 
                                                                                                                            
        uint8 OpNu = static_cast<uint8>(((OpType >= EOpType::NU_RANGE_BEGIN) & (OpType < EOpType::NU_RANGE_END)) 
                ? EDataType::Int : EDataType::None); 
        
        uint8 OpSc = static_cast<uint8>(((OpType >= EOpType::SC_RANGE_BEGIN) & (OpType < EOpType::SC_RANGE_END)) 
                ? EDataType::Scalar : EDataType::None);
                                                                                                                            
        uint8 OpCo = static_cast<uint8>(((OpType >= EOpType::CO_RANGE_BEGIN) & (OpType < EOpType::CO_RANGE_END)) 
                ? EDataType::Color : EDataType::None);
                                                                                                                            
        uint8 OpLa = static_cast<uint8>(((OpType >= EOpType::LA_RANGE_BEGIN) & (OpType < EOpType::LA_RANGE_END)) 
                ? EDataType::Layout : EDataType::None);
        
        uint8 OpIm = static_cast<uint8>(((OpType >= EOpType::IM_RANGE_BEGIN) & (OpType < EOpType::IM_RANGE_END)) 
                ? EDataType::Image : EDataType::None);
                                                                                                                            
        uint8 OpMe = static_cast<uint8>(((OpType >= EOpType::ME_RANGE_BEGIN) & (OpType < EOpType::ME_RANGE_END)) 
                ? EDataType::Mesh : EDataType::None);

        uint8 OpIn = static_cast<uint8>(((OpType >= EOpType::IN_RANGE_BEGIN) & (OpType < EOpType::IN_RANGE_END)) 
                ? EDataType::Instance : EDataType::None);

        uint8 OpPr = static_cast<uint8>(((OpType >= EOpType::PR_RANGE_BEGIN) & (OpType < EOpType::PR_RANGE_END)) 
                ? EDataType::Projector : EDataType::None);

        uint8 OpSt = static_cast<uint8>(((OpType >= EOpType::ST_RANGE_BEGIN) & (OpType < EOpType::ST_RANGE_END)) 
                ? EDataType::String : EDataType::None);

        uint8 OpEd = static_cast<uint8>(((OpType >= EOpType::ED_RANGE_BEGIN) & (OpType < EOpType::ED_RANGE_END)) 
                ? EDataType::ExtensionData : EDataType::None);

        uint8 OpMa = static_cast<uint8>(((OpType >= EOpType::MA_RANGE_BEGIN) & (OpType < EOpType::MA_RANGE_END)) 
                ? EDataType::Matrix : EDataType::None);
        
        uint8 OpMi = static_cast<uint8>(((OpType >= EOpType::MI_RANGE_BEGIN) & (OpType < EOpType::MI_RANGE_END)) 
                ? EDataType::Material : EDataType::None);

        uint8 OpIs = static_cast<uint8>(((OpType >= EOpType::IS_RANGE_BEGIN) & (OpType < EOpType::IS_RANGE_END)) 
                ? EDataType::InstancedStruct : EDataType::None);

        uint8 OpSk = static_cast<uint8>(((OpType >= EOpType::SK_RANGE_BEGIN) & (OpType < EOpType::SK_RANGE_END)) 
                ? EDataType::SkeletalMesh : EDataType::None);
    	
    	uint8 OpLd = static_cast<uint8>(((OpType >= EOpType::LD_RANGE_BEGIN) & (OpType < EOpType::LD_RANGE_END)) 
			? EDataType::LOD : EDataType::None);
    	
        return static_cast<EDataType>(
                OpBo | OpNu | OpSc | OpCo | OpLa | OpIm | OpMe | OpIn | OpPr | OpSt | OpEd | OpMa | OpMi | OpIs | OpSk | OpLd);
    }

	enum class EMeshBindShapeFlags : uint32
	{
		None				   		= 0,
		ReshapeSkeleton		   		= 1 << 0,
		EnableRigidParts       		= 1 << 2,
		ReshapePhysicsVolumes  		= 1 << 4,
		ReshapeVertices		   		= 1 << 5,
		ApplyLaplacian		   		= 1 << 6,
		RecomputeNormals	   		= 1 << 7,
    	ReshapeSkeletonInvertSelection	= 1 << 8,
    	ReshapePhysicsVolumesInvertSelection = 1 << 9
	};
	ENUM_CLASS_FLAGS(EMeshBindShapeFlags);

	enum class EMeshBindColorChannelUsage : uint8
	{
		None       = 0,
		ClusterId  = 1,
		MaskWeight = 2,
	};

	struct FMeshBindColorChannelUsages
	{
		EMeshBindColorChannelUsage R;
		EMeshBindColorChannelUsage G;
		EMeshBindColorChannelUsage B;
		EMeshBindColorChannelUsage A;
	};

	static_assert(sizeof(FMeshBindColorChannelUsages) == sizeof(uint32));

    /** */
    struct FOperation
    {
        using ADDRESS_TYPE = uint32;

        typedef ADDRESS_TYPE ADDRESS;
        typedef ADDRESS_TYPE PARAMETER;
        typedef ADDRESS_TYPE CONSTANT_SHAPE;
    	typedef ADDRESS_TYPE CONSTANT_MATRIX;
    	typedef ADDRESS_TYPE CONSTANT_NAME;
    	typedef ADDRESS_TYPE CONSTANT_STRING;
		typedef ADDRESS_TYPE CONSTANT_LIST_ADDRESS;
		typedef ADDRESS_TYPE CONSTANT_LIST_UINT32;
		typedef ADDRESS_TYPE CONSTANT_LIST_UINT64;
		typedef ADDRESS_TYPE CONSTANT_LIST_INT32;
    	typedef ADDRESS_TYPE CONSTANT_LIST_PASSTHROUGH_ID;
    	typedef ADDRESS_TYPE CONSTANT_LIST_FLOAT;
    	typedef ADDRESS_TYPE CONSTANT_LIST_BOOL;

        static constexpr uint32 AddressByteCodeOffsetBits = 26;
        static constexpr uint32 MaxAddressByteCodeOffset = (1 << AddressByteCodeOffsetBits) - 1;

        /** Arguments for every operation type. */
        struct BoolConstantArgs
        {
            bool bValue;
        };

        struct IntConstantArgs
        {
            int32 Value;
        };

        struct ScalarConstantArgs
        {
            float Value;
        };

        struct ColorConstantArgs
        {
            FVector4f Value;
        };

    	struct MatrixConstantArgs
    	{
    		CONSTANT_MATRIX value;
    	};

        struct ResourceConstantArgs
        {
            ADDRESS value;
        };
    	
    	struct ExternalDataConstantArgs
    	{
    		PASSTHROUGH_ID ExternalObjectId;
    	};

        struct MeshConstantArgs
        {
            // Index of the mesh in the mesh constant array
            ADDRESS Value;

            // If not negative, index of the skeleton to set to the mesh from the skeleton
            // constant array.
            int32 Skeleton;
        	
        	PASSTHROUGH_ID ClothID;
        };

    	struct MaterialConstantArgs
    	{
    		ADDRESS Value;
    		PASSTHROUGH_ID ID;
    	};

    	
		struct ParameterArgs
		{
			PARAMETER variable;
		};

		struct MaterialParameterArgs
		{
			PARAMETER Variable;

			CONSTANT_LIST_UINT32 ImageParameterNames;
			CONSTANT_LIST_UINT32 ImageParameterAddress;
		};

    	struct MaterialSkeletalMeshObjectBreakArgs
    	{
    		CONSTANT_NAME SlotName;
    		ADDRESS SkeletalMeshObject;
    	};

    	struct MaterialSkeletalMeshBreakArgs
    	{
    		CONSTANT_NAME SlotName;
    		ADDRESS SkeletalMesh;
    	};
    	
    	struct MeshSkeletalMeshBreakArgs
    	{
    		PARAMETER SkeletalMeshParameter;
    		uint8 BaseLOD = 0;
			uint8 BaseSection = 0;
			uint8 LODBias = 0;
			uint8 Flags = 0;
			uint32 MeshID = 0;
		};

        struct ConditionalArgs
        {
            ADDRESS condition, yes, no;
        };

		struct ResourceReferenceArgs
		{
			FImageDesc ImageDesc;
			PASSTHROUGH_ID ID;
			int8 ForceLoad;
		};

		struct FSwitchCaseDescriptor
		{
			uint32 Count      : 31;
			uint32 bUseRanges : 1;
		};

        //-------------------------------------------------------------------------------------
        struct BoolEqualScalarConstArgs
        {
            ADDRESS Value;
            int32 Constant;
        };

        struct BoolBinaryArgs
        {
            ADDRESS A,B;
        };

        struct BoolNotArgs
        {
            ADDRESS A;
        };

        struct ScalarCurveArgs
        {
            ADDRESS time;   // Operation generating the time value to sample the curve
            ADDRESS curve;  // Constant curve (not an op)
        };

        struct ColorSampleImageArgs
        {
            ADDRESS Image;
            ADDRESS X,Y;
            uint8 Filter;
        };

        struct ColorSwizzleArgs
        {
            uint8 sourceChannels[ MUTABLE_OP_MAX_SWIZZLE_CHANNELS ];
            ADDRESS sources[ MUTABLE_OP_MAX_SWIZZLE_CHANNELS ];
        };

		struct ColorFromScalarsArgs
		{
			ADDRESS V[MUTABLE_OP_MAX_SWIZZLE_CHANNELS];
		};

        struct ArithmeticArgs
		{
			typedef enum
			{
				NONE,
				ADD,
				SUBTRACT,
				MULTIPLY,
				DIVIDE
			} OPERATION;
            uint8 Operation;

			ADDRESS A, B;
		};

		struct ColorArgs
		{
			ADDRESS Color;
		};


        //-------------------------------------------------------------------------------------
        struct ImageLayerArgs
        {
            ADDRESS base;
            ADDRESS mask;
            ADDRESS blended;
			uint8 blendType;		// One of EBlendType

			/** One of EBlendType. If this is different than NONE, it will be applied to the alpha with the channel BlendAlphaSourceChannel of the blended. */
			uint8 blendTypeAlpha;

			/** Channel to use from the source blended argument to apply blendTypeAlpha, if any. */
			uint8 BlendAlphaSourceChannel;

			uint8 flags;
			typedef enum
            {
                F_NONE           = 0,
                /** The mask is considered binary : 0 means 0% and any other value means 100% */
                F_BINARY_MASK    = 1 << 0,
				/** If the image has 4 channels, apply to the fourth channel as well. */
				F_APPLY_TO_ALPHA = 1 << 1,
				/** Use the alpha channel of the blended image as mask. Mask should be null.*/
				F_USE_MASK_FROM_BLENDED = 1 << 2,
				/** Use the alpha channel of the base image as its RGB.*/
				F_BASE_RGB_FROM_ALPHA = 1 << 3,
				/** Use the alpha channel of the blended image as its RGB.*/
				F_BLENDED_RGB_FROM_ALPHA = 1 << 4,
			} FLAGS;
        };

        struct ImageMultiLayerArgs
        {
            ADDRESS base;
            ADDRESS mask;
            ADDRESS blended;
            ADDRESS rangeSize;
            uint16 rangeId;
			uint8 blendType;		// One of EBlendType
			
			/** One of EBlendType. If this is different than NONE, it will be applied to the alpha with the channel BlendAlphaSourceChannel of the blended. */
			uint8 blendTypeAlpha;

			/** Channel to use from the source color argument to apply blendTypeAlpha, if any. */
			uint8 BlendAlphaSourceChannel;

			uint8 bUseMaskFromBlended;	
		};

        struct ImageLayerColorArgs
        {
            ADDRESS base;
            ADDRESS mask;
            ADDRESS color;
			uint8 blendType;		// One of EBlendType

			/** One of EBlendType. If this is different than NONE, it will be applied to the alpha but with the channel BlendAlphaSourceChannel of the color. */
			uint8 blendTypeAlpha;	

			/** Channel to use from the source color argument to apply blendTypeAlpha, if any. */
			uint8 BlendAlphaSourceChannel;

			/** Like in ImageLayerArgs. */
			uint8 flags;
		};

        struct ImagePixelFormatArgs
        {
            ADDRESS source;
			EImageFormat format;
			EImageFormat formatIfAlpha;
        };

		struct ImageConformArgs
		{
			ADDRESS Source;
			FImageDesc ConformDescriptor;
		};

        struct ImageMipmapArgs
        {
            ADDRESS source;

            //! Number of mipmaps to build. If zero, it means all.
            uint8 levels;

            //! Number of mipmaps that can be generated for a single layout block.
            uint8 blockLevels;

            //! This is true if this operation is supposed to build only the tail mipmaps.
            //! It is used during the code optimisation phase, and to validate the code.
            bool onlyTail;

            //! Mipmap generation settings. 
            EMipmapFilterType FilterType;
            EAddressMode AddressMode;
        };

        struct ImageResizeArgs
        {
            ADDRESS Source;
            uint16 Size[2];
        };

        struct ImageResizeLikeArgs
        {
            //! Image that will be resized
            ADDRESS Source;

            //! Image whose size will be used to resize the source.
            ADDRESS SizeSource;
        };

        struct ImageResizeVarArgs
        {
            //! Image that will be resized
            ADDRESS source;

            //! Size expression.
            ADDRESS size;
        };

        struct ImageResizeRelArgs
        {
            //! Image that will be resized
            ADDRESS Source;

            //! Factor for each axis.
            float Factor[2];
        };

        struct ImageBlankLayoutArgs
        {
            ADDRESS Layout;

            /** Size of a layout block in pixels. */
            uint16 BlockSize[2];
			EImageFormat Format;

            /** If true, generate mipmaps. */
            uint8 GenerateMipmaps;

            /** Mipmaps to generate if mipmaps are to be generated. 0 means all. */
            uint8 MipmapCount;
        };

        struct ImageComposeArgs
        {
            ADDRESS layout;
        	ADDRESS base;
        	ADDRESS blockImage;
            ADDRESS mask;
            uint64 BlockId;
        };

		struct ImageMultiComposeArgs
		{
			ADDRESS Layout;
			ADDRESS SourceLayout;
			ADDRESS Base;
			ADDRESS SourceImage;

			//ADDRESS Mask;

			/** Size of the image to create. If 0, reuse size from base.*/
			uint16 LayoutBlockSizeInPixelsX = 0;
			uint16 LayoutBlockSizeInPixelsY = 0;

			uint16 SourceSizeX = 0;
			uint16 SourceSizeY = 0;
		};

        struct ImageInterpolateArgs
        {
            ADDRESS Factor;
            ADDRESS Targets[ MUTABLE_OP_MAX_INTERPOLATE_COUNT ];
        };

        struct ImageSaturateArgs
        {
            /** Image to modify. */
            ADDRESS Base;

            /** Saturation factor : 0 desaturates, 1 leaves the same, >1 saturates */
            ADDRESS Factor;
        };

        struct ImageLuminanceArgs
        {
            //! Image to modify.
            ADDRESS Base;
        };

        struct ImageSwizzleArgs
        {
			EImageFormat format;
            uint8 sourceChannels[ MUTABLE_OP_MAX_SWIZZLE_CHANNELS ];
            ADDRESS sources[ MUTABLE_OP_MAX_SWIZZLE_CHANNELS ];
        };

        struct ImageColorMapArgs
        {
            ADDRESS Base;
            ADDRESS Mask;
            ADDRESS Map;
        };

        struct ImageBinariseArgs
        {
            ADDRESS Base;
            ADDRESS Threshold;
        };

        struct ImagePlainColorArgs
        {
            ADDRESS Color;
			EImageFormat Format;
            uint16 Size[2];

			/** Number of mipmaps to generate. 0 means all the chain. */
			uint8 LODs;
        };

        struct ImageCropArgs
        {
            ADDRESS source;
            uint16 minX, minY, sizeX, sizeY;
        };

        struct ImagePatchArgs
        {
            ADDRESS base;
            ADDRESS patch;
            uint16 minX, minY;
        };

        struct ImageRasterMeshArgs
        {
			uint64 BlockId;
			
			ADDRESS mesh;

			//! These are used in case of projected mesh raster.
			ADDRESS image;
			ADDRESS angleFadeProperties;

			//! Mask selecting the pixels in the destination image that may receive projection.
			ADDRESS mask;

			//! A projector may be needed for some kind of per-pixel raster operations
			//! like cylindrical projections.
			ADDRESS projector;
			
			uint16 sizeX, sizeY;
			uint16 SourceSizeX, SourceSizeY;
			uint16 CropMinX, CropMinY;
			uint16 UncroppedSizeX, UncroppedSizeY;
			uint8 bIsRGBFadingEnabled : 1;
			uint8 bIsAlphaFadingEnabled : 1;
			
			// Currently only 2 sampling methods are contemplated, but reserve 3 bits for future uses. 
			uint8 SamplingMethod : 3;
			// Currently only 2 min filter methods are contemplated, but reserve 3 bits for future uses. 
			uint8 MinFilterMethod : 3;

			uint8 LayoutIndex;
        };

        struct ImageMakeGrowMapArgs
        {
            ADDRESS mask;
            int32 border;
        };

        struct ImageDisplaceArgs
        {
            ADDRESS Source;
            ADDRESS DisplacementMap;
        };

		struct ImageInvertArgs
		{
			ADDRESS Base;
		};

        struct ImageNormalCompositeArgs
        {
            ADDRESS base;
            ADDRESS normal;

            float power;
            ECompositeImageMode mode;
        };

		struct ImageTransformArgs
		{
			ADDRESS Base = 0;
			ADDRESS OffsetX = 0;
			ADDRESS OffsetY = 0;
			ADDRESS ScaleX = 0;
			ADDRESS ScaleY = 0;
			ADDRESS Rotation = 0;

			uint32 AddressMode      : 31;
			uint32 bKeepAspectRatio : 1;

            /** Size of the image to create. If 0, reuse size from base.*/
            uint16 SizeX = 0;
            uint16 SizeY = 0;

			uint16 SourceSizeX = 0;
			uint16 SourceSizeY = 0;
		};

        //-------------------------------------------------------------------------------------
        struct MeshApplyLayoutArgs
        {
            ADDRESS Mesh;
            ADDRESS Layout;
            uint16 Channel;
        };

		struct MeshPrepareLayoutArgs
		{
			ADDRESS Mesh;
			ADDRESS Layout;
			uint8 LayoutChannel = 0;
			uint8 bUseAbsoluteBlockIds : 1;
			uint8 bNormalizeUVs : 1;
			uint8 bClampUVIslands : 1;
			uint8 bEnsureAllVerticesHaveLayoutBlock : 1;
		};
		
        struct MeshMergeArgs
        {
            ADDRESS Base;
            ADDRESS Added;

            // If 0, it merges the surfaces, otherwise, add a new surface for the added mesh.
            uint32 NewSurfaceID;
        };

		struct MeshMaskClipMeshArgs
		{
			ADDRESS source;
			ADDRESS clip;
		};

		struct MeshMaskClipUVMaskArgs
		{
			ADDRESS Source = 0;
			ADDRESS UVSource = 0;
			ADDRESS MaskImage = 0;
			ADDRESS MaskLayout = 0;
			uint8 LayoutIndex = 0;
		};

        struct MeshMaskDiffArgs
        {
            ADDRESS Source;
            ADDRESS Fragment;
        };

        struct MeshFormatArgs
        {
            ADDRESS source;
            ADDRESS format;

            enum
            {
                Vertex				= 1 << 0,
                Index				= 1 << 1,
                // deprecated Face  = 1 << 2,

                /** This flag will not add blank channels for the channels in the format mesh but not in the source mesh. */
				IgnoreMissing		= 1 << 4,

                /** This flag will force the reset of buffer indices to 0. */
                ResetBufferIndices	= 1 << 5,

				/** This flag will add a step to reduce some buffers size by removing components and changing the types if possible. */
				OptimizeBuffers		= 1 << 6
			} EFlags;

            //! EFlags combination, selecting the buffers to reformat and other options.
            uint8 Flags;

        };

		struct MeshTransformArgs
		{
			ADDRESS source;
			CONSTANT_MATRIX matrix;
		};

		struct MeshClipMorphPlaneArgs
		{
			ADDRESS Source;
			ADDRESS MorphShape;
			CONSTANT_SHAPE VertexSelectionShapeOrBone;

			float Dist, Factor, MaxBoneRadius;

			EClipVertexSelectionType VertexSelectionType;

			EFaceCullStrategy FaceCullStrategy;
		};

        struct MeshClipWithMeshArgs
        {
            ADDRESS Source;
            ADDRESS ClipMesh;
        	
        	EFaceCullStrategy FaceCullStrategy;
        };

        struct MeshProjectArgs
        {
            ADDRESS Mesh;
            ADDRESS Projector;
            uint8 LayoutIndex;
        };

        struct MeshApplyPoseArgs
        {
            ADDRESS base;
            ADDRESS pose;
        };

		struct MeshGeometryOperationArgs
		{
			ADDRESS meshA;
			ADDRESS meshB;
			ADDRESS scalarA;
			ADDRESS scalarB;
		};

		struct MeshMorphArgs
		{
			CONSTANT_STRING Name;
			ADDRESS Base;
			ADDRESS Factor;
		};

		struct MeshBindShapeArgs
		{
			ADDRESS mesh;
			ADDRESS shape;
			uint32 flags;
			uint32 bindingMethod;
			uint32 ColorUsage;
		};

		struct MeshApplyShapeArgs
		{
			ADDRESS mesh;
			ADDRESS shape;
			uint32 flags;
		};

		struct MeshClipDeformArgs
		{
			ADDRESS mesh;
			ADDRESS clipShape;

			float clipWeightThreshold = 0.9f;
			EFaceCullStrategy FaceCullStrategy;
		};

    	struct MeshTransformWithinMeshArgs
    	{
    		ADDRESS sourceMesh;
    		ADDRESS boundingMesh;
    		ADDRESS matrix;
    	};

		struct MeshTransformWithBoneArgs
		{
			ADDRESS SourceMesh;
			ADDRESS Matrix;
			uint32 BoneId;
			float ThresholdFactor;
		};

		struct MeshOptimizeSkinningArgs
		{
			ADDRESS source;
		};
         
        struct MeshAddMetadataArgs
        {
            enum class EnumFlags : uint16
            {
				None = 0,
				HasGameplayTags = 1 << 0,
				IsGameplayTagList = 1 << 1,
				HasAssetUserData = 1 << 2,
            	IsAssetUserDataList = 1 << 4,
            	HasAnimationSlots = 1 << 5,
            	IsAnimationSlotList = 1 << 6,
				HasAdditionalPhysicsAsset = 1 << 7,
				IsAdditionalPhysicsAssetList = 1 << 8,
				HasRealTimeMorphNames = 1 << 9,
				IsRealTimeMorphNamesList = 1 << 10,	
				HasSockets = 1 << 11,	
				IsSocketsList = 1 << 12,	
            };

        	union NameOrUInt32ListIndex
        	{
        		CONSTANT_NAME NameIndex;
        		CONSTANT_LIST_UINT32 ListIndex;
        	};
        	
            union StringAddressOrUInt32ListAddress
            {
                ADDRESS StringAddress;
				CONSTANT_LIST_UINT32 ListAddress;
            };

            union ResourceIdOrUInt64ListAddress
            {
                uint64 ResourceId;
                ADDRESS ListAddress;
            };

			union PassthroughIDOrUInt32ListAddress
			{
				PASSTHROUGH_ID PassthroughId;
				CONSTANT_LIST_UINT32 ListAddress;
			};

			union SocketIdOrUInt32ListAddress
			{
				uint32 SocketId;
				CONSTANT_LIST_UINT32 ListAddress;
			};

            EnumFlags Flags; 
			int8 BonePosePriority;
			int8 SocketPriority;
            ADDRESS Source;
            PASSTHROUGH_ID SkeletonId;
            NameOrUInt32ListIndex GameplayTags;
            PassthroughIDOrUInt32ListAddress AssetUserDataIds;
        	NameOrUInt32ListIndex AnimSlotNames;
        	PassthroughIDOrUInt32ListAddress AnimInstances;
			PASSTHROUGH_ID PhysicsAssetId; // Main Physics Asset of the mesh
			PassthroughIDOrUInt32ListAddress AdditionalPhysicsAssetsIds; // Physics Assets used by AnimBPs
            StringAddressOrUInt32ListAddress RealTimeMorphNames;
			SocketIdOrUInt32ListAddress Sockets;
        };
        static_assert(sizeof(MeshAddMetadataArgs) == 44, "MeshAddMetadataArgs has an unexpected size.");

		struct MeshSetMaterialSlotIdArgs
		{
			ADDRESS Mesh;
			uint32 MaterialSlotId;
		};

    	struct FLODNewArgs
    	{
    		CONSTANT_LIST_UINT32 Meshes;
    	};
    	
    	struct FSkeletalMeshNewArgs
    	{
    		CONSTANT_LIST_ADDRESS MaterialSlotMaterials;
    		CONSTANT_LIST_UINT32 MaterialSlotNames;
    		CONSTANT_LIST_UINT32 MaterialSlotIds;
    		CONSTANT_LIST_ADDRESS LODs;
    	};

		struct FSkeletalMeshMergeArgs
		{
			ADDRESS BaseMesh;
			ADDRESS AddedMesh;
		};
    
    	struct FSkeletalMeshMorphArgs
    	{
			CONSTANT_STRING MorphName;
			ADDRESS Base;
			ADDRESS Factor;
    	};

    	struct FSkeletalMeshReshapeArgs
    	{
			ADDRESS Base;
			ADDRESS BaseShape;
			ADDRESS TargetShape;
			uint32 Flags;
			uint32 BindingMethod;
			uint32 ColorUsage;

			int32 NumBones;
			int32 NumPhysics;
    	};

		struct FSkeletalMeshMaterialModifyArgs
		{
			CONSTANT_NAME MaterialSlotName;
			ADDRESS SkeletalMesh;
			ADDRESS NewMaterial;
		};

		struct FSkeletalMeshClipMeshWithMeshArgs
		{
			ADDRESS Source;
			ADDRESS Clip;

			EFaceCullStrategy FaceCullStrategy;
		};
    	
    	struct FSkeletalMeshTransformArgs
    	{
    		ADDRESS Source;
    		CONSTANT_MATRIX Matrix;
    	};

    	struct FSkeletalMeshTransformWithBoneArgs
    	{
    		ADDRESS SourceSkeletalMesh;
    		ADDRESS Matrix;
    		uint32 BoneId;
    		float ThresholdFactor;
    	};

    	struct FSkeletalMeshObjectConvertArgs
    	{
    		ADDRESS SkeletalMesh;
    		CONSTANT_NAME Name;
    		int32 NumLODs = 0;
    		int32 FirstLODAvailable = 0;
    		uint8 FirstLODResident = 0;
    		int32 MinLODsDefault = 0;
    		CONSTANT_LIST_UINT32 MinLODsKeys;
    		CONSTANT_LIST_INT32 MinLODsValues;
    		int32 MinQualityLevelLODsDefault = 0;
    		CONSTANT_LIST_INT32 MinQualityLevelLODsKeys;
    		CONSTANT_LIST_INT32 MinQualityLevelLODsValues;
    		CONSTANT_LIST_FLOAT ScreenSize;
			CONSTANT_LIST_FLOAT LODHysteresis;
			CONSTANT_LIST_BOOL bSupportUniformlyDistributedSampling;
			CONSTANT_LIST_BOOL bAllowCPUAccess;
    	};

    	struct FSkeletalMeshConvertArgs
    	{
    		ADDRESS SkeletalMeshObject;
			uint32 MeshID;
			uint8 ConversionFlags;
    	};

        struct InstanceAddArgs
        {
            ADDRESS instance;
            ADDRESS value;
        };
    	
    	struct InstanceAddComponentArgs
    	{
    		ADDRESS Instance;
    		ADDRESS Value;
    		FComponentId ExternalId;
    	};

		struct InstanceAddExtensionDataArgs
		{
			// This is a reference to an op that produces the Instance that the FExtensionData will
			// be added to.
			ADDRESS Instance;
			
			ADDRESS ExtensionData;
		};

		struct InstanceAddMaterialArgs
		{
			// This is a reference to an op that produces the Instance that the material will
			// be added to.
			ADDRESS Instance;
			ADDRESS Material;
		};

		struct InstanceAddOverrideMaterialArgs
		{
			ADDRESS Instance;
			ADDRESS Material;
			CONSTANT_STRING SlotName;
		};
    	
    	struct InstanceAddOverlayMaterialArgs
    	{
    		ADDRESS Instance;
    		ADDRESS Material;
    		
    		CONSTANT_STRING SlotName;
    	};

        struct LayoutPackArgs
        {
            ADDRESS Source;
        };

        struct LayoutMergeArgs
        {
            ADDRESS Base;
            ADDRESS Added;
        };

		struct LayoutRemoveBlocksArgs
		{
			/** Layout to be processed and modified. */
			ADDRESS Source;

			/** Source layout to scan for active blocks. */
			ADDRESS ReferenceLayout;
		};

		struct LayoutFromMeshArgs
		{
			/** Source mesh to retrieve the layout from. */
			ADDRESS Mesh;
			uint8 LayoutIndex;
		};

		struct MaterialBreakArgs
		{
			ADDRESS Material;
			CONSTANT_STRING ParameterName;
			int8 LayerIndex;
		};

		struct MaterialBreakImageParameterArgs
		{
			ADDRESS MaterialParameter;
			CONSTANT_STRING ParameterName;
			int8 LayerIndex;
		};

    	struct ExternalArgs
    	{
    		int32 NumOperants;
    	};

    	struct ImageParameterConvertArgs
    	{
    		ADDRESS ImageParameter;
    	};

    	struct MaterialModifyArgs
    	{
    		ADDRESS Material;
    		uint32 NumParameters;
    	};
    };

	void SkeletalMeshOptionsUnpack(uint16 ExecutionOptions, bool& bInitialGeneration, uint8& LODIndex, uint8& FirstLODResident, uint8& FirstLODAvailable, uint8& NumLODs);
    	
	uint16 SkeletalMeshOptionsPack(bool bInitialGeneration, uint8 LODIndex, uint8 FirstLODResident, uint8 FirstLODAvailable, uint8 NumLODs);
    
	void SkeletalMeshObjectOptionsUnpack(uint16 ExecutionOptions, bool& bInitialGeneration, bool& bStreamMeshLODs, uint8& LODIndex);
    	
	MUTABLERUNTIME_API uint16 SkeletalMeshObjectOptionsPack(bool bInitialGeneration, bool bStreamMeshLODs, uint8 LODIndex);
	
    ENUM_CLASS_FLAGS(FOperation::MeshAddMetadataArgs::EnumFlags);

	static_assert(sizeof(FOperation::FSwitchCaseDescriptor) == sizeof(uint32));

    FORCEINLINE FOperation::ADDRESS MakeProgramAddress(EOpType OpType, uint32 ByteCodeOffset)
    {
        check(ByteCodeOffset <= FOperation::MaxAddressByteCodeOffset);
        static_assert(1 << (32 - FOperation::AddressByteCodeOffsetBits) > (int32)EDataType::Count);

        return 
            (uint32(GetOpDataType(OpType)) << FOperation::AddressByteCodeOffsetBits) | 
            (ByteCodeOffset & FOperation::MaxAddressByteCodeOffset);
    } 

   // FORCEINLINE EOpType GetAddressOpType(FOperation::ADDRESS At)
   // {
   //     return static_cast<EOpType>(At >> FOperation::AddressByteCodeOffsetBits);
   // }

    FORCEINLINE EDataType GetAddressDataType(FOperation::ADDRESS At)
    {
        return static_cast<EDataType>(At >> FOperation::AddressByteCodeOffsetBits);
    }

    FORCEINLINE uint32 GetAddressByteCodeOffset(FOperation::ADDRESS At)
    {
        return At & FOperation::MaxAddressByteCodeOffset;
    }

    //! Utility function to apply a function to all operation references to other operations.
	MUTABLERUNTIME_API extern void ForEachReference( const struct FProgram& Program, FOperation::ADDRESS At, const TFunctionRef<void(FOperation::ADDRESS)> );

	inline EOpType GetSwitchForType( EDataType d )
	{
		switch (d)
		{
		case EDataType::Instance: return EOpType::IN_SWITCH;
		case EDataType::Mesh: return EOpType::ME_SWITCH;
		case EDataType::Image: return EOpType::IM_SWITCH;
		case EDataType::Layout: return EOpType::LA_SWITCH;
		case EDataType::Color: return EOpType::CO_SWITCH;
		case EDataType::Scalar: return EOpType::SC_SWITCH;
		case EDataType::Int: return EOpType::NU_SWITCH;
		case EDataType::ExtensionData: return EOpType::ED_SWITCH;
		case EDataType::InstancedStruct: return EOpType::IS_SWITCH;
		case EDataType::SkeletalMesh: return EOpType::SK_SWITCH;
		case EDataType::LOD: return EOpType::LD_SWITCH;
		case EDataType::Material: return EOpType::MI_SWITCH;

		default:
			check(false);
			break;
		}
		return EOpType::NONE;
	}
	
	inline EOpType GetConditionalForType(EDataType Type)
	{
		switch (Type)
		{
		case EDataType::Instance: return EOpType::IN_CONDITIONAL;
		case EDataType::Mesh: return EOpType::ME_CONDITIONAL;
		case EDataType::Image: return EOpType::IM_CONDITIONAL;
		case EDataType::Layout: return EOpType::LA_CONDITIONAL;
		case EDataType::Color: return EOpType::CO_CONDITIONAL;
		case EDataType::Scalar: return EOpType::SC_CONDITIONAL;
		case EDataType::Int: return EOpType::NU_CONDITIONAL;
		case EDataType::ExtensionData: return EOpType::ED_CONDITIONAL;
		case EDataType::InstancedStruct: return EOpType::IS_CONDITIONAL;
		case EDataType::SkeletalMesh: return EOpType::SK_CONDITIONAL;
		case EDataType::LOD: return EOpType::LD_CONDITIONAL;
		case EDataType::Material: return EOpType::MI_CONDITIONAL;

		default:
			check(false);
			break;
		}
		
		return EOpType::NONE;
	}
}

