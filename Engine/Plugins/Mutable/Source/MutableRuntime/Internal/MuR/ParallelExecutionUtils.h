// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

namespace UE::Mutable::Private::ParallelExecutionUtils
{
    void InvokeBatchParallelFor(int32 Num, TFunctionRef<void(int32)> Body);
    void InvokeBatchParallelForUnbalanced(int32 Num, TFunctionRef<void(int32)> Body);
    void InvokeBatchParallelForSingleThread(int32 Num, TFunctionRef<void(int32)> Body);
}

