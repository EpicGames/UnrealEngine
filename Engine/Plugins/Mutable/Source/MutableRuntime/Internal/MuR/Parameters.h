// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Serialisation.h"
#include "MuR/ManagedPointer.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/TVariant.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "VectorTypes.h"
#include "Misc/TVariant.h"
#include "Engine/Texture.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/MaterialInterface.h"

struct FInstancedStruct;


#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
	class FInputArchive;
	class FOutputArchive;
	class FParameters;
    class FRangeIndex;
	class FModel;


    /** Model parameter types. */
    enum class EParameterType : uint32
    {
        /** Undefined parameter type. */
        None,

        /** Boolean parameter type (true or false) */
        Bool,

        /** Integer parameter type. It usually has a limited range of possible values that can be queried in the FParameters object. */
        Int,

        /** Floating point value in the range of 0.0 to 1.0 */
        Float,

        /** Floating point RGBA color, with each channel ranging from 0.0 to 1.0 */
        Color,

        /** 3D Projector type, defining a position, scale and orientation.Basically used for projected decals. */
        Projector,

		/** An externally provided image. */
		Texture,

		/** An externally provided mesh. */
		SkeletalMesh,
		
		/** An externally provided material*/
    	Material,

        /** A text string. */
        String,

    	/** A 4x4 matrix. */
    	Matrix,

    	InstancedStruct,

        /** Utility enumeration value, not really a parameter type. */
        Count
    };
	
	
	inline uint32 GetTypeHash(const EParameterType& Type)
	{
		return static_cast<uint32>(Type);
	}


    /** Types of 3D projectors. */
    enum class EProjectorType : uint32
    {
        /** Standard projector that uses an affine transform. */
        Planar,

        /** Projector that wraps the projected image around a cylinder*/
        Cylindrical,

		/** Smart projector that tries to follow the projected surface geometry to minimize streching. */
        Wrapping,

        /** Utility enumeration value, not really a projector type. */
        Count
    };


    /** Type used to set or read multi-dimensional parameter values.
     * If parameters have multiple values because of ranges, FRangeIndex can be used to specify
     * what value is read or set in the methods of the Parameter class.
     * FRangeIndex objects can be reused in multiple calls for the same parameter in the Parameter
     * class interface even after changing the position values.
	 */
	class FRangeIndex
    {
    public:

		/** Instances of this class must be obtained from a FParameters instance. */
		FRangeIndex() = default;

        //-----------------------------------------------------------------------------------------
        // Own interface
        //-----------------------------------------------------------------------------------------

        //! Return the number of ranges (or dimensions) used by this index
        UE_API int32 GetRangeCount() const;

        //! Return the name of a range.
        //! \param index Index of the range from 0 to GetRangeCount()-1
        UE_API const FString& GetRangeName( int32 Index ) const;

        //! Return the Guid of the parameter, resistant to parameter name changes.
        //! \param index Index of the parameter from 0 to GetRangeCount()-1
        UE_API const FString& GetRangeUid( int32 Index) const;

        //! Set the position in one of the dimensions of the index.
        //! \param index Index of the parameter from 0 to GetRangeCount()-1
        UE_API void SetPosition( int32 Index, int32 Position );

        //! Return the position in one of the dimensions of the index.
        //! \param index Index of the parameter from 0 to GetRangeCount()-1
        UE_API int32 GetPosition( int32 Index) const;

        friend class FParameters;

		//! Run-time data
		TSharedPtr<const FParameters> Parameters;

		//! Index of the parameter for which we are a range index
		int32 Parameter = -1;

		//! Position in the several dimension of the range, as defined in Parameters
		TArray<int32> Values;
    };


	template<class T>
    class TIndirectObject
    {
        TUniquePtr<T> StoragePtr;

    public:
        template<typename... TArgs>
        TIndirectObject(TArgs&&... Args) : StoragePtr(MakeUnique<T>(Forward<TArgs>(Args)...)) 
        {
        }
        
        TIndirectObject(const TIndirectObject<T>& Other) : StoragePtr(MakeUnique<T>()) 
        { 
            Get() = Other; 
        }

		TIndirectObject(TIndirectObject<T>&& Other) : StoragePtr(MoveTemp(Other.StoragePtr)) 
        {
        }

        TIndirectObject(const T& Object) : StoragePtr(MakeUnique<T>()) 
        { 
            Get() = Object; 
        }
        
        TIndirectObject(T&& Object) : StoragePtr(MakeUnique<T>()) 
        { 
            Get() = MoveTemp(Object); 
        }

        TIndirectObject& operator=(TIndirectObject<T>&&) = default;

        TIndirectObject& operator=(const TIndirectObject<T>& Other) 
        { 
            Get() = Other; 
			return *this;
        }

        TIndirectObject& operator=(const T& Object) 
        { 
            Get() = Object; 
            return *this;
        }

        TIndirectObject& operator=(const T&& Object) 
        { 
            Get() = MoveTemp(Object); 
            return *this;
        }

        const T& Get() const 
        { 
            check(StoragePtr); 
            return *StoragePtr; 
        }

        T& Get() 
        { 
            check(StoragePtr); 
            return *StoragePtr; 
        }

        operator T&() 
        { 
            return Get(); 
        }

        operator const T&() const 
        { 
            return Get(); 
        }

        T* operator &() 
        { 
            return StoragePtr.Get(); 
        }

        const T* operator &() const 
        { 
            return StoragePtr.Get(); 
        }

        T& operator *() 
        { 
            return Get(); 
        }

        const T& operator *() const 
        { 
            return Get(); 
        }

        bool operator==(const TIndirectObject<T>& Other) const 
        { 
            return *StoragePtr == Other; 
        }

        bool operator==(const T& Object) const 
        { 
            return *StoragePtr == Object; 
        }

		//!
		void Serialise(FOutputArchive& Arch) const
		{
			Arch << Get();
		}

		//!
		void Unserialise(FInputArchive& Arch)
		{
			Arch >> Get();
		} 
    };


    MUTABLE_DEFINE_ENUM_SERIALISABLE(EParameterType)
    MUTABLE_DEFINE_ENUM_SERIALISABLE(EProjectorType)


    /** Description of a projector to project an image on a mesh. */
    struct FProjector
    {
        EProjectorType type = EProjectorType::Planar;
        FVector3f position = {0,0,0};
		FVector3f direction = {0,0,0};
		FVector3f up = {0,0,0};
		FVector3f scale = {0,0,0};
        float projectionAngle = 0.0f;

        //!
        inline void GetDirectionSideUp(FVector3f& OutDirection, FVector3f& OutSide, FVector3f& OutUp) const
        {
			OutDirection = direction;
            OutUp = up;
            OutSide = FVector3f::CrossProduct( up, direction );
			OutSide.Normalize();
        }


        //!
        void Serialise( FOutputArchive& arch ) const
        {
            arch << type;
            arch << position;
            arch << direction;
            arch << up;
            arch << scale;
            arch << projectionAngle;
        }

        //!
        void Unserialise( FInputArchive& arch )
        {
            arch >> type;
            arch >> position;
            arch >> direction;
            arch >> up;
            arch >> scale;
            arch >> projectionAngle;
        }

        bool operator==( const FProjector& o ) const
        {
            return     type==o.type
                    && position==o.position
                    && up==o.up
                    && direction==o.direction
                    && scale==o.scale
                    && projectionAngle==o.projectionAngle;
        }

    };

	//---------------------------------------------------------------------------------------------
	//! Information about a generic shape in space.
	//---------------------------------------------------------------------------------------------
	struct FShape
	{
		// Transform
		FVector3f position = FVector3f(0,0,0);
		FVector3f up = FVector3f(0, 0, 0);
		FVector3f side = FVector3f(0, 0, 0);
		
		FVector3f size = FVector3f(0, 0, 0);

		// 
		enum class Type : uint8
		{
			None = 0,
			Ellipse,
			AABox
		};
        uint8 type = 0;

		//!
		void Serialise(FOutputArchive& arch) const
		{
			arch << position;
			arch << up;
			arch << side;
			arch << size;
			arch << type;
		}

		//!
		void Unserialise(FInputArchive& arch)
		{
			arch >> position;
			arch >> up;
			arch >> side;
			arch >> size;
			arch >> type;
		}

        bool operator==( const FShape& o ) const
        {
            return type==o.type
                    && position==o.position
                    && up==o.up
                    && side==o.side
                    && size==o.size;
        }
	};


	using FParamBoolType = bool;
	using FParamIntType = int32;
	using FParamFloatType = float;
	using FParamColorType = FVector4f;
	using FParamProjectorType = TIndirectObject<FProjector>;
	using FParamTextureType = TStrongObjectPtr<UTexture>;
	using FParamSkeletalMeshType = TStrongObjectPtr<USkeletalMesh>;
	using FParamStringType = TIndirectObject<FString>;
	using FParamMatrixType = TIndirectObject<FMatrix44f>;
	using FParamMaterialType = TStrongObjectPtr<UMaterialInterface>;
	using FParamInstancedStructType = TManagedPtr<const FInstancedStruct>; // TSharedPtr always valid and FInstancedStruct always initialized to its type.
	
	using FParameterValue = TVariant<
		FParamBoolType,
		FParamIntType,
		FParamFloatType,
		FParamColorType,
		FParamProjectorType,
		FParamTextureType,
		FParamSkeletalMeshType,
		FParamStringType,
		FParamMatrixType,
		FParamMaterialType,
		FParamInstancedStructType>;

    // static_assert to track ParameterValue size changes. It is ok to change if needed.
    static_assert(sizeof(FParameterValue) == 8*4, "ParameterValue size has changed.");

    struct FParameterDesc
    {
        FString Name;

        //! Unique id (provided externally, so no actual guarantee that it is unique.)
		FGuid UID;

        EParameterType Type = EParameterType::None;

        FParameterValue DefaultValue;

        //! Ranges, if the parameter is multi-dimensional. The indices refer to the Model's program
        //! vector of range descriptors.
		TArray<uint32> Ranges;

        //! Possible values of the parameter in case of being an integer, and its names
        struct FIntValueDesc
        {
            int16 Value;
			FString Name;

            //!
            bool operator==( const FIntValueDesc& Other ) const
            {
                return Value== Other.Value &&
						Name== Other.Name;
            }

            //!
            void Serialise( FOutputArchive& Arch ) const
            {
				Arch << Value;
				Arch << Name;
            }

            //!
            void Unserialise( FInputArchive& Arch)
            {
				Arch >> Value;
				Arch >> Name;
            }
        };

        //! For integer parameters, this contains the description of the possible values.
        //! If empty, the integer may have any value.
		TArray<FIntValueDesc> PossibleValues;

        //!
        bool operator==( const FParameterDesc& other ) const
        {
            return Name == other.Name && UID == other.UID && Type == other.Type &&
                   DefaultValue == other.DefaultValue &&
                   Ranges == other.Ranges &&
                   PossibleValues == other.PossibleValues;
        }

        //!
        void Serialise( FOutputArchive& arch ) const
        {
			arch << Name;
			arch << UID;
            arch << Type;
            arch << DefaultValue;
            arch << Ranges;
            arch << PossibleValues;
        }

        //!
        void Unserialise( FInputArchive& arch )
        {
			arch >> Name;
			arch >> UID;
            arch >> Type;
			arch >> DefaultValue;
            arch >> Ranges;
			arch >> PossibleValues;
        }
    };


    struct FRangeDesc
    {
		FString Name;
		FString UID;

		/** Parameter that controls the size of this range, if any. */
		int32 DimensionParameter = -1;

        //!
        bool operator==( const FRangeDesc& other ) const
        {
            return Name==other.Name
				&&
				UID == other.UID 
				&&
				DimensionParameter == other.DimensionParameter;
        }

        //!
        void Serialise( FOutputArchive& Arch ) const
        {
			Arch << Name;
			Arch << UID;
			Arch << DimensionParameter;
        }

        //!
        void Unserialise( FInputArchive& Arch)
        {
			Arch >> Name;
			Arch >> UID;
			Arch >> DimensionParameter;
        }
    };
	
    /** This class represents the parameters of a model including description, type and
    * value, and additional resources associated to the parameter.
    *
    * \warning Every object of this class holds a reference to the model that it was created from.
    * This implies that while any instance of a FParameters object is alive, its model will not
    * be freed.
    */
	class FParameters : public TSharedFromThis<FParameters>
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------
		
        //! Serialisation
        static UE_API void Serialise( const FParameters*, FOutputArchive& );
        static UE_API TSharedPtr<FParameters> StaticUnserialise( FInputArchive& );

		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
		
		//! Deep clone this object.
		UE_API TSharedPtr<FParameters> Clone() const;
		
		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		//! Return the number of parameters
		UE_API int32 GetCount() const;

		//! Return the name of the parameter.
		//! \param index Index of the parameter from 0 to GetCount()-1
		UE_API const FString& GetName( int32 Index ) const;

		//! Return the Guid of the parameter, resistant to parameter name changes.
		//! \param index Index of the parameter from 0 to GetCount()-1
		UE_API const FGuid& GetUid( int32 Index) const;

		//! Find the parameter index by name.
		//! It returns -1 if the parameter doesn't exist.
		UE_API int32 Find( const FString& Name ) const;

		//! Return the type of the parameter.
		//! \param index Index of the parameter from 0 to GetCount()-1
		UE_API EParameterType GetType( int32 Index) const;

        //! Create a new FRangeIndex object to use to access a multi-dimensional parameter.
        //! It will return nullptr if the parameter is not multidimensional.
		UE_API TSharedPtr<FRangeIndex> NewRangeIndex( int32 ParamIndex) const;

        //! Return the number of values other than the default that have been set to a specific
        //! parameter.
        UE_API int32 GetValueCount( int32 ParamIndex ) const;

        //! Return the FRangeIndex of a value that has been set to a parameter.
        //! \param paramIndex Index of the parameter from 0 to GetCount()-1
        //! \param valueIndex Index of the value from 0 to GetValueCount()-1
		UE_API TSharedPtr<FRangeIndex> GetValueIndex( int32 ParamIndex, int32 ValueIndex ) const;

        //! Remove all the multidimensional values for a parameter. The current non-dimensional
        //! value is kept.
        UE_API void ClearAllValues( int32 ParamIndex);

		//! Return the value of a boolean parameter.
		//! \pre The parameter specified by index is a T_BOOL.
		//! \param index Index of the parameter from 0 to GetCount()-1
        //! \param pos Only for multidimensional parameters: relevant position to get in the ranges
        UE_API bool GetBoolValue( int32 Index, const FRangeIndex* Pos =nullptr ) const;

		//! Set the value of a parameter of type boolean.
		//! \pre The parameter specified by index is a T_BOOL.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param value new value of the parameter
        //! \param pos optional parameter to set a specific value for the given multidimensional
        //! position. If null, the value is set for all possible positions.
        UE_API void SetBoolValue( int32 Index, bool bValue, const FRangeIndex* Pos =nullptr );

		//! Get the number of possible values for this integer parameter.
		//! If the number is zero, it means any integer value is accepted.
		UE_API int32 GetIntPossibleValueCount( int32 ParamIndex) const;

		//! Get the value of one of the possible values for this integer.
		//! The paramIndex is in the range of 0 to GetIntPossibleValueCount()-1
		UE_API int32 GetIntPossibleValue( int32 ParamIndex, int32 ValueIndex) const;

        //! Get the name of one of the possible values for this integer.
        //! The paramIndex is in the range of 0 to GetIntPossibleValueCount()-1
        UE_API const FString& GetIntPossibleValueName( int32 ParamIndex, int32 ValueIndex ) const;

        //! Get the index of the value of one of the possible values for this integer.
        //! The paramIndex is in the range of 0 to GetIntPossibleValueCount()-1
		UE_API int32 GetIntValueIndex(int32 ParamIndex, const FString& ValueIndex) const;

		//! Get the index of the value of one of the possible values for this integer.
		//! The paramIndex is in the range of 0 to GetIntPossibleValueCount()-1
		UE_API int32 GetIntValueIndex(int32 ParamIndex, int32 Value) const;

		//! Return the value of a integer parameter.
		//! \pre The parameter specified by index is a T_INT.
		//! \param index Index of the parameter from 0 to GetCount()-1
        //! \param pos Only for multidimensional parameters: relevant position to get in the ranges
        UE_API int32 GetIntValue( int32 Index, const FRangeIndex* Pos=nullptr ) const;

		//! If the parameter is of the integer type, set its value.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param value new value of the parameter. It must be in the possible values for this
		//! parameter (see GetIntPossibleValue), or the method will leave the value unchanged.
        //! \param pos Only for multidimensional parameters: relevant position to set in the ranges
        UE_API void SetIntValue( int32 Index, int32 Value, const FRangeIndex* Pos=nullptr );

		//! Return the value of a float parameter.
		//! \pre The parameter specified by index is a T_FLOAT.
		//! \param index Index of the parameter from 0 to GetCount()-1
        //! \param pos Only for multidimensional parameters: relevant position to get in the ranges
        UE_API float GetFloatValue( int32 Index, const FRangeIndex* Pos=nullptr ) const;

		//! If the parameter is of the float type, set its value.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param value new value of the parameter
        //! \param pos Only for multidimensional parameters: relevant position to set in the ranges
        UE_API void SetFloatValue( int32 Index, float Value, const FRangeIndex* Pos=nullptr );

		//! Return the value of a color parameter.
		//! \pre The parameter specified by index is a T_FLOAT.
        //! \param index Index of the parameter from 0 to GetCount()-1
        //! \param OutColor Values where every resulting color channel will be stored
        //! \param pos Only for multidimensional parameters: relevant position to get in the ranges
        UE_API void GetColorValue( int32 Index, FVector4f& OutColor, const FRangeIndex* Pos=nullptr ) const;

		//! If the parameter is of the color type, set its value.
		//! \param index Index of the parameter from 0 to GetCount()-1
        //! \param r,g,b new value of the parameter
        //! \param pos Only for multidimensional parameters: relevant position to set in the ranges
        UE_API void SetColorValue( int32 Index, FVector4f Color, const FRangeIndex* Pos=nullptr );

		//! Return the value of a projector parameter, as a 4x4 matrix. The matrix is supposed to be
		//! a linear transform in column-major.
		//! \pre The parameter specified by index is a T_PROJECTOR.
        //! \param ParameterIndex Index of the parameter from 0 to GetCount()-1
        //! \param OutPos Pointer to where the object-space position coordinates of the projector will be stored.
        //! \param OutDir Pointer to where the object-space direction vector of the projector will be stored.
        //! \param OutUp Pointer to where the object-space vertically up direction vector
        //!         of the projector will be stored. This controls the "roll" angle of the
        //!         projector.
        //! \param OutScale Pointer to the projector-space scaling of the projector.
        //! \param RangePosition Only for multidimensional parameters: relevant position to get in the ranges
        UE_API void GetProjectorValue( int32 ParameterIndex,
                                EProjectorType* OutProjectionType,
								FVector3f* OutPos,
								FVector3f* OutDir,
								FVector3f* OutUp,
								FVector3f* OutScale,
                                float* OutProjectionAngle,
								const FRangeIndex* RangePosition=nullptr ) const;

		//! If the parameter is of the projector type, set its value.
		//! \param ParameterIndex Index of the parameter from 0 to GetCount()-1
        //! \param pos Object-space position coordinates of the projector.
        //! \param dir Object-space direction vector of the projector.
        //! \param up Object-space vertically up direction vector of the projector.
        //! \param scale Projector-space scaling of the projector.
        //! \param projectionAngle [only for Cylindrical projectors], the angle in radians of the
        //! projection area on the cylinder surface.
        //! \param RangePosition Only for multidimensional parameters: relevant position to set in the ranges
        UE_API void SetProjectorValue( int32 ParameterIndex,
								const FVector3f& Pos,
								const FVector3f& Dir,
								const FVector3f& Up,
								const FVector3f& Scale,
								float ProjectionAngle,
								const FRangeIndex*  RangePosition=nullptr );

        //! Return the value of a Texture parameter.
        //! \pre The parameter specified by index is a Texture.
        //! \param Index Index of the parameter from 0 to GetCount()-1
		//! \param Pos Only for multidimensional parameters: relevant position to set in the ranges
	    //! \return The UTexture specified when setting the image value (\see SetTextureValue)
		UE_API UTexture* GetTextureValue(int32 Index, const FRangeIndex* Pos = nullptr) const;

		/** */
		UE_API USkeletalMesh* GetSkeletalMeshValue(int32 Index, const FRangeIndex* Pos = nullptr) const;

        //! If the parameter is of the Texture type, set its value.
        //! \param Index Index of the parameter from 0 to GetCount()-1
        //! \param Pos Only for multidimensional parameters: relevant position to set in the ranges
		UE_API void SetTextureValue(int32 Index, UTexture* Value, const FRangeIndex* Pos = nullptr);

		/** */
		UE_API void SetSkeletalMeshValue(int32 Index, USkeletalMesh* Value, const FRangeIndex* = nullptr);

        //! Return the value of a float parameter.
        //! \pre The parameter specified by index is a T_FLOAT.
        //! \param index Index of the parameter from 0 to GetCount()-1
        //! \param pos Only for multidimensional parameters: relevant position to get in the ranges
        UE_API void GetStringValue( int32 Index, FString& OutValue, const FRangeIndex* = nullptr) const;

        //! If the parameter is of the float type, set its value.
        //! \param index Index of the parameter from 0 to GetCount()-1
        //! \param value new value of the parameter
        //! \param pos Only for multidimensional parameters: relevant position to set in the ranges
        UE_API void SetStringValue( int32 Index, const FString& Value, const FRangeIndex* = nullptr );

		//! Return the value of a matrix parameter.
		//! \pre The parameter specified by index is a T_MATRIX.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param pos Only for multidimensional parameters: relevant position to get in the ranges
		UE_API void GetMatrixValue( int32 Index, FMatrix44f& OutValue, const FRangeIndex* = nullptr) const;

		//! If the parameter is of the matrix type, set its value.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param value new value of the parameter
		//! \param pos Only for multidimensional parameters: relevant position to set in the ranges
		UE_API void SetMatrixValue( int32 Index, const FMatrix44f& Value, const FRangeIndex* Pos = nullptr );

		//! Return the value of a Material parameter.
		//! \pre The parameter specified by index is a T_MATERIAL.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param pos Only for multidimensional parameters: relevant position to set in the ranges
		//! \return The Material interface specified when setting the Material value (\see SetMaterialValue)
		UE_API UMaterialInterface* GetMaterialValue(int32 Index, const FRangeIndex* RangeIndex = nullptr) const;

		//! If the parameter is of the material type, set its value.
		//! \param index Index of the parameter from 0 to GetCount()-1
		//! \param Value Material interface to set as value.
		//! \param pos Only for multidimensional parameters: relevant position to set in the ranges
		UE_API void SetMaterialValue(int32 Index, UMaterialInterface* Value, const FRangeIndex* RangeIndex = nullptr);

		/** Return the value of a Instanced Struct parameter.
		* @pre The parameter specified by index is a T_INSTANCEDSTRUCT.
		* @param Index Index of the parameter from 0 to GetCount()-1
		* @param RangeIndex Only for multidimensional parameters: relevant position to set in the ranges
		* @return The Instanced Struct interface specified when setting the Instanced Struct value (\see SetInstancedStructValue) */
		UE_API TManagedPtr<const FInstancedStruct> GetInstancedStructValue(int32 Index, const FRangeIndex* RangeIndex = nullptr) const;

		//! If the parameter is of the Instanced Struct type, set its value.
		//! \param Index Index of the parameter from 0 to GetCount()-1
		//! \param Value Instanced Struct interface to set as value.
		//! \param RangeIndex Only for multidimensional parameters: relevant position to set in the ranges
		UE_API void SetInstancedStructValue(int32 Index, const TManagedPtr<const FInstancedStruct>& Value, const FRangeIndex* RangeIndex = nullptr);

        //! Utility method to compare the values of a specific parameter with the values of another
        //! FParameters object. It returns false if type or values are different.
        UE_API bool HasSameValue( int32 ThisParamIndex, const TSharedPtr<const FParameters>& Other, int32 OtherParamIndex ) const;
		
		UE_API FProjector GetProjectorValue( int32 index, const FRangeIndex* ) const;

		/** Return true if the parameter has any multi-dimensional values set. This is independent to if the model
		* accepts multi-dimensional parameters for this particular parameter. */
		UE_API bool HasMultipleValues(int32 ParamIndex) const;
		
		//! Run-time data
		TSharedPtr<const FModel> Model;
 
		//! Values for the parameters if they are not multidimensional.
		TArray<FParameterValue> Values;
		
		//! If the parameter is multidemensional, the values are stored here.
		//! The key of the map is the vector of values stored in a FRangeIndex
		TArray< TMap< TArray<int32>, FParameterValue > > MultiValues;

	};


}

#undef UE_API
