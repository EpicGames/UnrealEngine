// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/TVariant.h"
#include "MuR/Serialisation.h"


namespace UE::Mutable::Private
{
	class FPassthroughObjectLoader;
	
	using PASSTHROUGH_ID = uint32;
	static_assert(sizeof(PASSTHROUGH_ID) == 4); // Passthrough object Ids are stored in the program as uint32

	constexpr PASSTHROUGH_ID PASSTHROUGH_ID_INVALID = TNumericLimits<PASSTHROUGH_ID>::Max();
	
	MUTABLERUNTIME_API PASSTHROUGH_ID GetPassthroughId(const TSoftObjectPtr<UObject>& SoftObjectPtr);

	template<typename T>
	class TPassthroughObjectPtr
	{
		using ObjectType = T;
		using VariantType = TVariant<TStrongObjectPtr<UObject>, PASSTHROUGH_ID>;

		friend FPassthroughObjectLoader;
		template<class U> friend class TPassthroughObjectPtr;

	public:
		TPassthroughObjectPtr()
			: StableSnapshot(VariantType(TInPlaceType<PASSTHROUGH_ID>{}, PASSTHROUGH_ID_INVALID))
			, Resource(MakeShared<VariantType>(StableSnapshot))
		{
		}
		
		TPassthroughObjectPtr(PASSTHROUGH_ID Id)
			: StableSnapshot(VariantType(TInPlaceType<PASSTHROUGH_ID>{}, Id))
			, Resource(MakeShared<VariantType>(StableSnapshot))
		{
		}

		TPassthroughObjectPtr(T* Object) 
			: StableSnapshot(VariantType(TInPlaceType<TStrongObjectPtr<UObject>>{}, Object))
			, Resource(MakeShared<VariantType>(StableSnapshot))
		{
		}

		TPassthroughObjectPtr(const TPassthroughObjectPtr& Other) = default;

		template<typename OtherObjectType>
		TPassthroughObjectPtr(const TPassthroughObjectPtr<OtherObjectType>& Other)
			: StableSnapshot(Other.StableSnapshot)
			, Resource(Other.Resource)
		{
		}

		TPassthroughObjectPtr(TPassthroughObjectPtr&& Other) = default;
		TPassthroughObjectPtr& operator=(const TPassthroughObjectPtr& Other) = default;
		TPassthroughObjectPtr& operator=(TPassthroughObjectPtr&& Other) = default;

		bool operator==(const TPassthroughObjectPtr& Other) const
		{
			return StableSnapshot == Other.StableSnapshot;
		}

		friend uint32 GetTypeHash(const TPassthroughObjectPtr& Object)
		{
			uint32 Hash = GetTypeHash(Object.Resource);
			return Hash;
		}

		bool IsResolved() const
		{
			return Resource->IsType<TStrongObjectPtr<UObject>>();
		}
		
		T* Get() const
		{
			if (Resource->IsType<PASSTHROUGH_ID>())
			{
				return nullptr;
			}
			
			UObject* Object = Resource->Get<TStrongObjectPtr<UObject>>().Get();
			if (!Object)
			{
				return nullptr;
			}

			return CastChecked<T>(Object);
		}

		PASSTHROUGH_ID GetId() const
		{
			if (StableSnapshot.IsType<PASSTHROUGH_ID>())
			{
				return StableSnapshot.Get<PASSTHROUGH_ID>();
			}
			
			return PASSTHROUGH_ID_INVALID;
		}

		PASSTHROUGH_ID Reset()
		{
			PASSTHROUGH_ID Id = PASSTHROUGH_ID_INVALID; 
			
			if (StableSnapshot.IsType<PASSTHROUGH_ID>())
			{
				Id = StableSnapshot.Get<PASSTHROUGH_ID>();
			}

			StableSnapshot = VariantType(TInPlaceType<PASSTHROUGH_ID>{}, PASSTHROUGH_ID_INVALID);
			Resource = MakeShared<VariantType>(StableSnapshot);

			return Id; 
		}
		
		bool IsValid() const
		{
			return StableSnapshot.IsType<TStrongObjectPtr<UObject>>() || StableSnapshot.Get<PASSTHROUGH_ID>() != PASSTHROUGH_ID_INVALID;
		}
		
		explicit operator bool() const
		{
			return IsValid();
		}
	
	private:
		VariantType StableSnapshot;
		TSharedRef<VariantType> Resource;
	};	



	
	class FPassthroughObjectLoader
	{
	public:
		template<typename T>
		TPassthroughObjectPtr<T> FindChecked(PASSTHROUGH_ID Id)
		{
			if (Id == PASSTHROUGH_ID_INVALID)
			{
				return {};
			}
			else
			{
				return PassthroughObjects.FindChecked(Id);
			}
		}
		
		template<typename T>
		void Add(PASSTHROUGH_ID Id)
		{
			if (Id != PASSTHROUGH_ID_INVALID)
			{
				PassthroughObjects.Add(Id, TPassthroughObjectPtr<T>(Id));
			}
		}

		MUTABLERUNTIME_API void Load(const TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>>& ResolutionMap, TArray<FSoftObjectPath>& AssetsToStream) const;

		MUTABLERUNTIME_API void Resolve(TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>>& ResolutionMap);

	private:
		TMap<PASSTHROUGH_ID, TPassthroughObjectPtr<UObject>> PassthroughObjects;
	};
}
