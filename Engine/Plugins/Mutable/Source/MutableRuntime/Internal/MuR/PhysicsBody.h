// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <string>

#include "MuR/Serialisation.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Skeleton.h"
#include "Containers/Array.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
	struct FBodyShape
	{
		FString Name;
		uint32 Flags = 0;

		bool operator==(const FBodyShape& Other) const 
		{
			return Flags == Other.Flags && Name == Other.Name;
		}
	
		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};

	struct FSphereBody : FBodyShape
	{
		FVector3f Position = FVector3f::ZeroVector;
		float Radius = 0.0f;

		bool operator==( const FSphereBody& Other ) const
		{
			return FBodyShape::operator==(Other) && Position == Other.Position && Radius == Other.Radius;
		}
		 
		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};
	
	struct FBoxBody : FBodyShape
	{
		FVector3f Position = FVector3f::ZeroVector;
		FQuat4f Orientation = FQuat4f::Identity;
		FVector3f Size = FVector3f::ZeroVector;
		 
		inline bool operator==( const FBoxBody& Other ) const 
		{
			return FBodyShape::operator==(Other) && Position == Other.Position && Orientation == Other.Orientation && Size == Other.Size;
		}

		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};

	struct FSphylBody : FBodyShape
	{
		FVector3f Position = FVector3f::ZeroVector;
		FQuat4f Orientation = FQuat4f::Identity;
		float Radius = 0.0f;
		float Length = 0.0f;

		inline bool operator==( const FSphylBody& Other ) const 
		{
			return FBodyShape::operator==(Other) && 
				Position == Other.Position && 
				Orientation == Other.Orientation && 
				Radius == Other.Radius && 
				Length == Other.Length;
		}
		 
		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};
	
	struct FTaperedCapsuleBody : FBodyShape
	{
		FVector3f Position;
		FQuat4f Orientation;
		float Radius0 = 0.0f;
		float Radius1 = 0.0f;
		float Length = 0.0f;
		 
		inline bool operator==( const FTaperedCapsuleBody& Other ) const 
		{
			return FBodyShape::operator==(Other) && 
				Position == Other.Position && 
				Orientation == Other.Orientation && 
				Radius0 == Other.Radius0 && 
				Radius1 == Other.Radius1 && 
				Length == Other.Length;
		}
		 
		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};
	
	struct FConvexBody : FBodyShape
	{
		TArray<FVector3f> Vertices;
		TArray<int32> Indices;

		FTransform3f Transform = FTransform3f::Identity;
		
		inline bool operator==( const FConvexBody& Other ) const
		{
			return FBodyShape::operator==(Other) && 
				Vertices == Other.Vertices && 
				Indices == Other.Indices && 
				Transform.Equals(Other.Transform);
		}

		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};
	
	struct FPhysicsBodyAggregate
	{
		TArray<FSphereBody> Spheres;
		TArray<FBoxBody> Boxes;
		TArray<FConvexBody> Convex;
		TArray<FSphylBody> Sphyls;
		TArray<FTaperedCapsuleBody> TaperedCapsules;

		inline bool operator==( const FPhysicsBodyAggregate& Other ) const
		{
			return Spheres == Other.Spheres &&
					Boxes == Other.Boxes &&
					Sphyls == Other.Sphyls &&
					TaperedCapsules == Other.TaperedCapsules &&
					Convex == Other.Convex;
		}

		void Serialise(FOutputArchive& Arch) const;
		void Unserialise(FInputArchive& Arch);
	};

	class FPhysicsBody
	{
	public:
		
		UE_API TManagedPtr<FPhysicsBody> Clone() const;

		//! Serialisation
        static UE_API void Serialise( const FPhysicsBody*, FOutputArchive& );
		static UE_API TManagedPtr<FPhysicsBody> StaticUnserialise( FInputArchive& );

        //bool operator==( const PhysicsBody& Other ) const;

		UE_API void SetCustomId(int32 Id);
		UE_API int32 GetCustomId() const;

		UE_API void SetBodyCount(int32 B);
		UE_API int32 GetBodyCount() const;
	
		UE_API void SetBody(int32 BodyIndex, FName BoneName);

		UE_API FName GetBodyBoneName(int32 Index) const;

		UE_API int32 GetSphereCount(int32 B) const;
		UE_API int32 GetBoxCount(int32 B) const;
		UE_API int32 GetConvexCount(int32 B) const;
		UE_API int32 GetSphylCount(int32 B) const;
		UE_API int32 GetTaperedCapsuleCount(int32 B) const;
		
		UE_API void SetSphereCount(int32 B, int32 Count);
		UE_API void SetBoxCount(int32 B, int32 Count);
		UE_API void SetConvexCount(int32 B, int32 Count);
		UE_API void SetSphylCount(int32 B, int32 Count);
		UE_API void SetTaperedCapsuleCount(int32 B, int32 Count);
		
		UE_API void SetSphere( 
				int32 B, int32 I, 
				FVector3f Position, float Radius);

		UE_API void SetBox( 
				int32 B, int32 I, 
				FVector3f Position, FQuat4f Orientation, FVector3f Size);

		UE_API void SetConvexMesh( 
				int32 B, int32 I,
				TArrayView<const FVector3f> Vertices, TArrayView<const int32> Indices);

		UE_API void SetConvexTransform( 
				int32 B, int32 I, 
				const FTransform3f& Transform);

		UE_API void SetSphyl( 
				int32 B, int32 I, 
				FVector3f Position, FQuat4f Orientation, 
				float Radius, float Length);

		UE_API void SetTaperedCapsule( 
				int32 B, int32 I, 
				FVector3f Position, FQuat4f Orientation, 
				float Radius0, float Radius1, float Length);

		UE_API void SetSphereFlags(int32 B, int32 I, uint32 Flags);
		UE_API void SetBoxFlags(int32 B, int32 I, uint32 Flags);
		UE_API void SetConvexFlags(int32 B, int32 I, uint32 Flags);
		UE_API void SetSphylFlags(int32 B, int32 I, uint32 Flags);
		UE_API void SetTaperedCapsuleFlags(int32 B, int32 I, uint32 Flags);

		UE_API void SetSphereName(int32 B, int32 I, const char* Name);
		UE_API void SetBoxName(int32 B, int32 I, const char* Name);
		UE_API void SetConvexName(int32 B, int32 I, const char* Name);
		UE_API void SetSphylName(int32 B, int32 I, const char* Name);
		UE_API void SetTaperedCapsuleName(int32 B, int32 I, const char* Name);
	
		UE_API void GetSphere( 
				int32 B, int32 I, 
				FVector3f& OutPosition, float& OutRadius) const;

		UE_API void GetBox( 
				int32 B, int32 I, 
				FVector3f& OutPosition, FQuat4f& OutOrientation, FVector3f& OutSize) const;

		UE_API void GetConvex( 
				int32 B, int32 I, 
				TArrayView<const FVector3f>& OutVertices, TArrayView<const int32>& OutIndices, FTransform3f& OutTransform) const;

		UE_API void GetConvexMeshView(
				int32 B, int32 I, 
			    TArrayView<FVector3f>& OutVerticesView, TArrayView<int32>& OutIndicesView);

		UE_API void GetConvexTransform(
				int32 B, int32 I,
				FTransform3f& OutTransform) const;

		UE_API void GetSphyl( 
				int32 B, int32 I, 
				FVector3f& OutPosition, FQuat4f& OutOrientation, 
				float& OutRadius, float& OutLength) const;

		UE_API void GetTaperedCapsule( 
				int32 B, int32 I, 
				FVector3f& OutPosition, FQuat4f& OutOrientation, 
				float& OutRadius0, float& OutRadius1, float& OutLength) const;
		
		UE_API uint32 GetSphereFlags(int32 B, int32 I) const;
		UE_API uint32 GetBoxFlags(int32 B, int32 I) const;
		UE_API uint32 GetConvexFlags(int32 B, int32 I) const;
		UE_API uint32 GetSphylFlags(int32 B, int32 I) const;
		UE_API uint32 GetTaperedCapsuleFlags(int32 B, int32 I) const;

		UE_API const FString& GetSphereName(int32 B, int32 I) const;
		UE_API const FString& GetBoxName(int32 B, int32 I) const;
		UE_API const FString& GetConvexName(int32 B, int32 I) const;
		UE_API const FString& GetSphylName(int32 B, int32 I) const;
		UE_API const FString& GetTaperedCapsuleName(int32 B, int32 I) const;

		UE_API void ConvertBoneIdsToNames(const TMap<uint32, FName>& ConstantNames);

	public:
		int32 CustomId = -1;

		// Bone name the physics volume aggregate is bound to. 
		TArray<FName> BodiesBoneNames; // Runtime
		TArray<uint32> BoneIds; // Serialized
		TArray<FPhysicsBodyAggregate> Bodies;

		bool bBodiesModified = false;

        UE_API void Serialise(FOutputArchive& Arch) const;
        UE_API void Unserialise(FInputArchive& Arch);

		//!
		inline bool operator==(const FPhysicsBody& Other) const
        {
        	return CustomId        == Other.CustomId        &&
				   BoneIds		   == Other.BoneIds			&&
        		   Bodies          == Other.Bodies          &&
				   bBodiesModified == Other.bBodiesModified;
        }
	};
}

#undef UE_API
