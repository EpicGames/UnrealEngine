// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This file contains some hacks to solve differences between platforms

#include "HAL/PlatformTime.h"
#include "MuR/Types.h"
#include "HAL/UnrealMemory.h"



//! Unify debug defines
#if !defined(MUTABLE_DEBUG)
    #if !defined(NDEBUG) || defined(_DEBUG)
        #define MUTABLE_DEBUG
    #endif
#endif
