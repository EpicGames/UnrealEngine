// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/SystemTypes.h"
#include "MuR/ManagedPointer.h"
#include "MuR/Parameters.h"

#define UE_API MUTABLERUNTIME_API

namespace UE::Tasks
{
	class FTaskEvent;	
}

namespace UE::Mutable::Private
{
	class FLOD;
	class FMesh;
	class FSkeletalMesh;
	class FMaterial;
	class FLayout;
	class FExtensionData;
	class FInstance;
	class String;


	/** A cache address is the operation plus the context of execution(iteration indices, etc...). */
	struct FCacheAddress
	{
		/** The meaning of all these fields is the same than the FScheduledOp struct. */
		FOperation::ADDRESS At = 0;
		uint16 ExecutionIndex = 0;
		uint16 ExecutionOptions : 14 = 0;
		uint16 Type : 2 = static_cast<uint16>(FScheduledOp::EType::Full);

		FCacheAddress() 
		{
		}

		FCacheAddress(FOperation::ADDRESS InAt, uint16 InExecutionIndex, uint16 InExecutionOptions, FScheduledOp::EType InType = FScheduledOp::EType::Full)
			: At(InAt), ExecutionIndex(InExecutionIndex), ExecutionOptions(InExecutionOptions), Type(static_cast<uint16>(InType))
		{
		}

		FCacheAddress(FOperation::ADDRESS InAt, const FScheduledOp& Item)
			: At(InAt), ExecutionIndex(Item.ExecutionIndex), ExecutionOptions(Item.ExecutionOptions), Type(static_cast<uint16>(Item.Type))
		{
		}
		
		FCacheAddress(const FScheduledOp& Item)
			: At(Item.At), ExecutionIndex(Item.ExecutionIndex), ExecutionOptions(Item.ExecutionOptions), Type(static_cast<uint16>(Item.Type))
		{
		}
	};

	inline uint32 GetTypeHash(const FCacheAddress& Address)
	{
		uint32 Hash = Address.At;

		Hash = HashCombineFast(Hash, static_cast<uint32>(Address.ExecutionIndex));
		Hash = HashCombineFast(Hash, static_cast<uint32>(Address.ExecutionOptions));
		Hash = HashCombineFast(Hash, static_cast<uint32>(Address.Type));

		return Hash;
	}

	FORCEINLINE bool operator==(FCacheAddress A, FCacheAddress B)
	{
		return (A.At == B.At) & (A.ExecutionIndex == B.ExecutionIndex) & (A.ExecutionOptions == B.ExecutionOptions) & (A.Type == B.Type);
	}

	namespace ProgramCacheInternals
	{
		struct FProgramCacheInternals;
	}

	class FProgramCache
	{
		static constexpr uint32 InternalsAlign = 8;
		static constexpr uint32 InternalsSize  = 1088;

		struct alignas(InternalsAlign) FInternalsStorage
		{
			uint8 Data[InternalsSize];
		} InternalsStorage;

		ProgramCacheInternals::FProgramCacheInternals* GetInternals() const;

	public:

		enum class EClearFlags
		{
			None       = 0,
			
			FreeMemory = 1 << 0,

			Locked    = 1 << 1,
			Unlocked  = 1 << 2,
			Full      = Locked | Unlocked,

			ImageDesc = 1 << 3,
			Void      = 1 << 4,
			
			ForceClearMarkedToKeep = 1 << 5,
		};

		enum class EAcquireSetResult : uint8
		{
			Success = 0,
			Failure,
			Abort,
		};

		FProgramCache();

		FProgramCache(const FProgramCache&) = delete;
		FProgramCache(FProgramCache&&) = delete;
		
		~FProgramCache();
		
		FProgramCache& operator=(const FProgramCache&) = delete;
		FProgramCache& operator=(FProgramCache&&) = delete;

		/** Clear cache. */
		UE_API void Clear(EClearFlags Flags);
		
		bool IsSet(FCacheAddress Address);
		void UpdateHitCount(FCacheAddress Address, int8 Differential);
		void LockAddress(FCacheAddress Address);
		void MarkAddressToKeep(FCacheAddress Address);
		void SetAborted(FCacheAddress Address);
		bool IsAborted(FCacheAddress Address);

		[[nodiscard]] UE_API bool GetBoolIfSet(FCacheAddress Address, bool& bOutValue);
		[[nodiscard]] UE_API bool GetMaterialIfSet(FCacheAddress Address, TManagedPtr<const FMaterial>& bOutValue);

		[[nodiscard]] UE_API bool LoadBool(FCacheAddress Address);
		[[nodiscard]] UE_API float LoadScalar(FCacheAddress Address);
		[[nodiscard]] UE_API int32 LoadInt(FCacheAddress Address);
		[[nodiscard]] UE_API FVector4f LoadColor(FCacheAddress Address);
		[[nodiscard]] UE_API FMatrix44f LoadMatrix(FCacheAddress Address);
		[[nodiscard]] UE_API FProjector LoadProjector(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FInstance> LoadInstance(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FImage> LoadImage(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FMesh> LoadMesh(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FLOD> LoadLOD(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FSkeletalMesh> LoadSkeletalMesh(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FMaterial> LoadMaterial(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FLayout> LoadLayout(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const String> LoadString(FCacheAddress Address);
		[[nodiscard]] UE_API TManagedPtr<const FExtensionData> LoadExtensionData(FCacheAddress Address);	
		[[nodiscard]] UE_API TManagedPtr<const FInstancedStruct> LoadInstancedStruct(FCacheAddress Address);
		[[nodiscard]] UE_API FExtendedImageDesc LoadImageDesc(FCacheAddress Address);
		UE_API void LoadVoid(FCacheAddress Address);
	
		UE_API void StoreBool(FCacheAddress Address, bool Value);
		UE_API void StoreInt(FCacheAddress Address, int32 Value);
		UE_API void StoreScalar(FCacheAddress Address, float Value);
		UE_API void StoreColor(FCacheAddress Address, const FVector4f& Value);
		UE_API void StoreMatrix(FCacheAddress Address, const FMatrix44f& Value);
		UE_API void StoreMaterial(FCacheAddress Address, const TManagedPtr<const FMaterial>& Value);	
		UE_API void StoreProjector(FCacheAddress Address, const FProjector& Value);	
		UE_API void StoreInstancedStruct(FCacheAddress Address, const TManagedPtr<const FInstancedStruct>& Value);		
		UE_API void StoreImage(FCacheAddress Address, const TManagedPtr<const FImage>& Value);
		UE_API void StoreMesh(FCacheAddress Address, const TManagedPtr<const FMesh>& Value);
		UE_API void StoreLOD(FCacheAddress Address, const TManagedPtr<const FLOD>& Value);
		UE_API void StoreSkeletalMesh(FCacheAddress Address, const TManagedPtr<const FSkeletalMesh>& Value);
		UE_API void StoreLayout(FCacheAddress Address, const TManagedPtr<const FLayout>& Value);
		UE_API void StoreString(FCacheAddress Address, const TManagedPtr<const String>& Value);
		UE_API void StoreInstance(FCacheAddress Address, const TManagedPtr<const FInstance>& Value);
		UE_API void StoreExtensionData(FCacheAddress Address, const TManagedPtr<const FExtensionData>& Value);
		UE_API void StoreImageDesc(FCacheAddress Address, const FExtendedImageDesc& Value);
		UE_API void StoreVoid(FCacheAddress Address);

		SSIZE_T TryFreeWeakMemory(SSIZE_T NumBytesToFree);
		
		[[nodiscard]] EAcquireSetResult TryAcquireCachedResultSet(TConstArrayView<FCacheAddress> AddressSet);
		[[nodiscard]] bool TryAcquireCachedResult(FCacheAddress Address);
		
		/** 
		 * Tries to acquire any weak result. Returns if the operation needs to schedule.
		 *
		 * If the result is true, it is mandatory to to execute the operation at some point,
		 * otherwise other accesses to the same address will never complete.
		 */
		[[nodiscard]] bool AcquireCachedResultForScheduling(FCacheAddress Address);

		/**
		 * Check if all HitCount are zero. This is only for debug purposes.
		 */
		[[nodiscard]] bool CheckHitCountsCleared() const;

		/**
		 * Count number of bytes in the caches. This is only for debug purposes.
		 *
		 * Returns the number of internals and external bytes used by the caches.
		 */
		[[nodiscard]] SSIZE_T CountBytes() const;

		/** 
		 * Adds a copy the element at TemplateExecutionIndex modifing the RangeIndex to RangeValue. 
		 *
		 * Returns the Id of the new ExecutionIndex.
		 */
		int32 GetOrAddModifiedExecutionIndex(int32 TemplateExecutionIndex, uint16 RangeIndex, int32 RangeValue);

		/** 
		 * Get the value of the index from the range index in the model. 
		 */
		int32 GetExecutionIndexValue(int32 ExecutionIndex, uint16 RangeIndex) const;

		/**
		 * Register WaitEvent 
		 */
		Tasks::FTaskEvent& RegisterWaitEvent(const TCHAR* DebugName);

		/**
		 * Trigger registered WaitEvents.
		 */
		void TriggerWaitEvents();

	};
	
	ENUM_CLASS_FLAGS(FProgramCache::EClearFlags);
}

#undef UE_API
