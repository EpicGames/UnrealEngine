// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

// HEADER_UNIT_SKIP - Included through RefCounted

namespace UE::Mutable::Private
{
    //! \brief Intrusive smart pointer similar to the one in the boost libraries (intrusive_ptr<>)
    //!
    //! Some of the objects used by the runtime use reference counting to control their life-cycle.
    //! When an objects is returned in such a pointer, the user should keep this pointer valid while
    //! the data is in use, and clear it when done by assigning nullptr to it. The data returned
    //! in Ptr<T> pointers should never be directly deleted.
    //!
    //! The underlying lower-level pointer can be obtained with the get() method, but it should
    //! never be stored, since it will be valid only while there are valid Ptr<> to the same object.
    //!
	//! \ingroup runtime
	template<class T> class Ptr
	{
	public:

        Ptr( T* p = nullptr, bool add_ref = true )
		{
            m_ptr = p;
            if( m_ptr != nullptr && add_ref )
			{
				mutable_ptr_add_ref( m_ptr );
			}
		}

		template<class U> Ptr( Ptr<U> const& rhs )
			: m_ptr( rhs.get() )
		{
            if( m_ptr != nullptr )
			{
				mutable_ptr_add_ref( m_ptr );
			}
		}

        //! Assign constructor
        Ptr( Ptr const& rhs )
            : m_ptr( rhs.m_ptr )
        {
            if( m_ptr != nullptr )
            {
                mutable_ptr_add_ref( m_ptr );
            }
        }

        //! Move constructor
        Ptr( Ptr&& rhs )
        {
            m_ptr = rhs.m_ptr;
            rhs.m_ptr = nullptr;
        }

		~Ptr()
		{
            if( m_ptr != nullptr )
			{
				mutable_ptr_release( m_ptr );
			}
		}

		template<class U> Ptr& operator=( Ptr<U> const& rhs )
		{
			Ptr<T>(rhs).swap(*this);
			return *this;
		}

        //! Assign with copy
        Ptr& operator=( Ptr const& rhs)
        {
            Ptr<T>(rhs).swap(*this);
            return *this;
        }

        //! Assign with move
        Ptr& operator=( Ptr&& rhs)
        {
            if( m_ptr != nullptr )
            {
                mutable_ptr_release( m_ptr );
            }
            m_ptr = rhs.m_ptr;
            rhs.m_ptr = nullptr;
            return *this;
        }

		Ptr& operator=( T* rhs)
		{
			Ptr<T>(rhs).swap(*this);
			return *this;
		}

		void reset()
		{
			Ptr<T>().swap( *this );
		}

		void reset( T* rhs )
		{
			Ptr<T>( rhs ).swap( *this );
		}

		T* get() const
		{
			return m_ptr;
		}

		T& operator*() const
		{
			return *m_ptr;
		}

		T* operator->() const
		{
			return m_ptr;
		}


		inline explicit operator bool() const
		{
            return m_ptr != nullptr;
		}


		void swap( Ptr& rhs)
		{
			T * tmp = m_ptr;
			m_ptr = rhs.m_ptr;
			rhs.m_ptr = tmp;
		}

	private:

        T* m_ptr = nullptr;

	};


	template<class T, class U> inline bool operator==( Ptr<T> const& a, Ptr<U> const& b )
	{
		return a.get() == b.get();
	}

	template<class T, class U> inline bool operator!=( Ptr<T> const& a, Ptr<U> const& b )
	{
		return a.get() != b.get();
	}

	template<class T, class U> inline bool operator==( Ptr<T> const& a, U* b )
	{
		return a.get() == b;
	}

	template<class T, class U> inline bool operator!=( Ptr<T> const& a, U* b )
	{
		return a.get() != b;
	}

	template<class T, class U> inline bool operator==( T* a, Ptr<U> const& b )
	{
		return a == b.get();
	}

	template<class T, class U> inline bool operator!=( T* a, Ptr<U> const& b )
	{
		return a != b.get();
	}

	template<class T> inline bool operator<( Ptr<T> const& a, Ptr<T> const& b )
	{
		return ( a.get() < b.get() );
	}

	template<class T> void swap( Ptr<T>& lhs, Ptr<T>& rhs )
	{
		lhs.swap(rhs);
	}

	template<class T, class U> Ptr<T> static_pointer_cast( Ptr<U> const & p)
	{
		return static_cast<T *>(p.get());
	}

	template<class T, class U> Ptr<T> const_pointer_cast( Ptr<U> const & p)
	{
		return const_cast<T *>(p.get());
	}

}
