// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/Function.h"

namespace UE::Mutable::Private
{

	template<int NUM_INTERPOLATORS>
	class RasterVertex
	{
	public:
		RasterVertex() : x(0), y(0) {}
		RasterVertex( float aX, float aY ) : x(aX), y(aY) {}

		//! Raster coordinates of the vertex in image space.
		float x,y;

		//! Extra channels to interpolate accross de triangles.
		float interpolators[NUM_INTERPOLATORS];
	};


	//! Example rasterizer pixel processor.
	//! It only takes the color interpolated and assigns it to the pixel.
	class ColorPixelProcessor
	{
	public:
		inline void ProcessPixel(uint8* pBufferPos, float interpolatedValues[3]) const
		{
			pBufferPos[0] = (uint8)(interpolatedValues[0]*0xFF);
			pBufferPos[1] = (uint8)(interpolatedValues[1]*0xFF);
			pBufferPos[2] = (uint8)(interpolatedValues[2]*0xFF);
		}

		inline void operator()(uint8* BufferPos, float InterpolatedValues[3]) const
		{
			ProcessPixel(BufferPos, InterpolatedValues);
		}
	};

	template<int32 NumInterpolatedValues>
	using TTriangleRasterPixelProcRefType = TFunctionRef<void(uint8*, float[NumInterpolatedValues])>;

	//! Templatized rasterizer. It is not very efficient on the interpolating side, but the cost
    //! per pixel is supposed to be huge compared to the raster process (for lightmaps or
    //! normalmaps).
	//! TODO Optimize anyway.
	//! TODO Option for two sided? right now it culls back facing.
    template<int32 NUM_INTERPOLATORS, typename INT=int64>
	void Triangle( uint8* Buffer, int32 BufferSize, int Width, int Height, int PixelSize
				 , const RasterVertex<NUM_INTERPOLATORS> &av1
				 , const RasterVertex<NUM_INTERPOLATORS> &av2
				 , const RasterVertex<NUM_INTERPOLATORS> &av3
				 , TTriangleRasterPixelProcRefType<NUM_INTERPOLATORS> PixelProc
				 , bool cullBackface = true )
	{
		check(Width * Height * PixelSize <= BufferSize);

		float interpolatedValues[NUM_INTERPOLATORS];

		int Stride = Width*PixelSize;

		// Check backface culling
		// TODO better option than resorting vertices?
		float lCrossZ = (av2.x-av1.x) * (av3.y-av1.y) - (av2.y-av1.y) * (av3.x-av1.x);
		const RasterVertex<NUM_INTERPOLATORS>& v1 = av1;
		const RasterVertex<NUM_INTERPOLATORS>& v2 = cullBackface ? av2 : ( (lCrossZ>0) ? av3 : av2);
		const RasterVertex<NUM_INTERPOLATORS>& v3 = cullBackface ? av3 : ( (lCrossZ>0) ? av2 : av3);

	    // 28.4 fixed-point coordinates
        const INT Y1 = (INT)(16.0f * v1.y + 0.5f);
        const INT Y2 = (INT)(16.0f * v2.y + 0.5f);
        const INT Y3 = (INT)(16.0f * v3.y + 0.5f);

        const INT X1 = (INT)(16.0f * v1.x + 0.5f);
        const INT X2 = (INT)(16.0f * v2.x + 0.5f);
        const INT X3 = (INT)(16.0f * v3.x + 0.5f);

	    // Deltas
        const INT DX12 = X1 - X2;
        const INT DX23 = X2 - X3;
        const INT DX31 = X3 - X1;

        const INT DY12 = Y1 - Y2;
        const INT DY23 = Y2 - Y3;
        const INT DY31 = Y3 - Y1;

	    // Fixed-point deltas
        const INT FDX12 = DX12 * 16;
        const INT FDX23 = DX23 * 16;
        const INT FDX31 = DX31 * 16;

        const INT FDY12 = DY12 * 16;
        const INT FDY23 = DY23 * 16;
        const INT FDY31 = DY31 * 16;

		// Block size, standard 8x8 (must be power of two)
		constexpr INT q = 8;

		// Skip partial blocks
		INT ClampWidth = Width & ~(q - 1);
		INT ClampHeight = Height & ~(q - 1);

	    // Bounding rectangle
        INT minx = FMath::Max(INT(0),				(FMath::Min3(X1, X2, X3) + 0xF) / 16 );
        INT maxx = FMath::Min(INT(ClampWidth -1),	(FMath::Max3(X1, X2, X3) + 0xF) / 16 );
        INT miny = FMath::Max(INT(0),				(FMath::Min3(Y1, Y2, Y3) + 0xF) / 16 );
        INT maxy = FMath::Min(INT(ClampHeight -1),	(FMath::Max3(Y1, Y2, Y3) + 0xF) / 16 );

		// Start in corner of 8x8 block
		minx &= ~(q - 1);
		miny &= ~(q - 1);

		Buffer += miny * Stride;

	    // Half-edge constants
        INT C1 = DY12 * X1 - DX12 * Y1;
        INT C2 = DY23 * X2 - DX23 * Y2;
        INT C3 = DY31 * X3 - DX31 * Y3;

	    // Correct for fill convention
	    if(DY12 < 0 || (DY12 == 0 && DX12 > 0)) C1++;
	    if(DY23 < 0 || (DY23 == 0 && DX23 > 0)) C2++;
	    if(DY31 < 0 || (DY31 == 0 && DX31 > 0)) C3++;

		// Interpolation process constant stuff
        float ux = v2.x-v1.x;
        float uy = v2.y-v1.y;
        float vx = v3.x-v1.x;
        float vy = v3.y-v1.y;
        float det = ux*vy-vx*uy;
        ux /= det;
        uy /= det;
        vx /= det;
        vy /= det;

	    // Loop through blocks
        for(INT y = miny; y < maxy; y += q)
	    {
            for(INT x = minx; x < maxx; x += q)
	        {
	            // Corners of block
                INT x0 = x * 16;
                INT x1 = (x + q - 1) * 16;
                INT y0 = y * 16;
                INT y1 = (y + q - 1) * 16;

	            // Evaluate half-space functions
	            bool a00 = C1 + DX12 * y0 - DY12 * x0 > 0;
	            bool a10 = C1 + DX12 * y0 - DY12 * x1 > 0;
	            bool a01 = C1 + DX12 * y1 - DY12 * x0 > 0;
	            bool a11 = C1 + DX12 * y1 - DY12 * x1 > 0;
                INT a = (INT(a00) * 1) | (INT(a10) * 2) | (INT(a01) * 4) | (INT(a11) * 8);

	            bool b00 = C2 + DX23 * y0 - DY23 * x0 > 0;
	            bool b10 = C2 + DX23 * y0 - DY23 * x1 > 0;
	            bool b01 = C2 + DX23 * y1 - DY23 * x0 > 0;
	            bool b11 = C2 + DX23 * y1 - DY23 * x1 > 0;
                INT b = (INT(b00) * 1) | (INT(b10) * 2) | (INT(b01) * 4) | (INT(b11) * 8);

	            bool c00 = C3 + DX31 * y0 - DY31 * x0 > 0;
	            bool c10 = C3 + DX31 * y0 - DY31 * x1 > 0;
	            bool c01 = C3 + DX31 * y1 - DY31 * x0 > 0;
	            bool c11 = C3 + DX31 * y1 - DY31 * x1 > 0;
                INT c = (INT(c00) * 1) | (INT(c10) * 2) | (INT(c01) * 4) | (INT(c11) * 8);

	            // Skip block when outside an edge
				if (a == 0x0 || b == 0x0 || c == 0x0)
				{
					continue;
				}

				uint8 *CurrentBuffer = Buffer;

	            // Accept whole block when totally covered
	            if(a == 0xF && b == 0xF && c == 0xF)
	            {
                    for(INT iy = 0; iy < q; iy++)
	                {
                        for(INT ix = x; ix < x + q; ix++)
	                    {
	                        // Interpolate values
	                        float px = ix-v1.x;
	                        float py = y+iy-v1.y;
							float alpha = px*vy+py*-vx;
							float beta = px*-uy+py*ux;
							alpha = FMath::Max(0.0, FMath::Min(1.0f, alpha ));
							beta = FMath::Max(0.0, FMath::Min(1.0f, beta ));
							float gamma = FMath::Max(0.0, 1.0f-alpha-beta );

	                        for (int ii=0; ii<NUM_INTERPOLATORS; ii++)
							{
                                interpolatedValues[ii] = gamma * v1.interpolators[ii]
													   + alpha * v2.interpolators[ii]
													   + beta * v3.interpolators[ii];
							}

	                        // Process pixel
							PixelProc(&CurrentBuffer[ix*PixelSize], interpolatedValues);
	                    }

						CurrentBuffer += Stride;
	                }
	            }
	            else // Partially covered block
	            {
                    INT CY1 = C1 + DX12 * y0 - DY12 * x0;
                    INT CY2 = C2 + DX23 * y0 - DY23 * x0;
                    INT CY3 = C3 + DX31 * y0 - DY31 * x0;

                    for(INT iy = y; iy < y + q; iy++)
	                {
                        INT CX1 = CY1;
                        INT CX2 = CY2;
                        INT CX3 = CY3;

                        for(INT ix = x; ix < x + q; ix++)
	                    {
	                        if(CX1 > 0 && CX2 > 0 && CX3 > 0)
							{
		                        // Interpolate values
		                        float px = ix-v1.x;
		                        float py = iy-v1.y;
								float alpha = px*vy+py*-vx;
								float beta = px*-uy+py*ux;
								alpha = FMath::Max(0.0, FMath::Min(1.0f, alpha ));
								beta = FMath::Max(0.0, FMath::Min(1.0f, beta ));
								float gamma = FMath::Max(0.0, 1.0f-alpha-beta );

		                        for (int ii=0; ii<NUM_INTERPOLATORS; ii++)
								{
									interpolatedValues[ii] = gamma * v1.interpolators[ii]
															+ alpha * v2.interpolators[ii]
															+ beta * v3.interpolators[ii];
								}

								PixelProc(&CurrentBuffer[ix*PixelSize], interpolatedValues);
	                        }

	                        CX1 -= FDY12;
	                        CX2 -= FDY23;
	                        CX3 -= FDY31;
	                    }

	                    CY1 += FDX12;
	                    CY2 += FDX23;
	                    CY3 += FDX31;

						CurrentBuffer += Stride;
	                }
	            }
	        }

			Buffer += q * Stride;
	    }
	}
}
