// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - 

#include "MuR/Ptr.h"

#include "HAL/PlatformAtomics.h"
#include "HAL/LowLevelMemTracker.h"

#include <atomic>

namespace UE::Mutable::Private
{


	/** Base class for all reference counted objects.
	* Any subclass of this class can be managed using smart pointers through the Ptr<T> template.
	* \warning This base allow multi-threaded manipulation of smart pointers, since the count increments and decrements are atomic.
	*/
	class RefCounted
	{
	public:

		inline void IncRef() const
		{
			std::atomic_fetch_add_explicit( &m_refCount, 1, std::memory_order_relaxed );
		}

		inline void DecRef() const
		{
			LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

			if (m_refCount.fetch_sub(1, std::memory_order_release) == 1)
			{
				std::atomic_thread_fence(std::memory_order_acquire);
				delete this;
			}
		}

		inline bool IsUnique() const
		{
			return m_refCount.load(std::memory_order_relaxed)==1;
		}

		RefCounted(const RefCounted&) = delete;
		RefCounted(const RefCounted&&) = delete;
		RefCounted& operator=(const RefCounted&) = delete;
		RefCounted& operator=(const RefCounted&&) = delete;

	protected:

		RefCounted()
		{
			m_refCount = 0;
		}

		inline virtual ~RefCounted() = default;

	private:

		mutable std::atomic<int32> m_refCount;

	};


	inline void mutable_ptr_add_ref(const RefCounted* p)
	{
		if (p) p->IncRef();
	}


	inline void mutable_ptr_release(const RefCounted* p)
	{
		if (p)
		{
			p->DecRef();
		}
	}

	class FResource
	{
	public:

		/** Return the size in bytes of all the LODs of the image. */
		virtual int32 GetDataSize() const = 0;

		/** */
		virtual ~FResource() = default;

	};
}

