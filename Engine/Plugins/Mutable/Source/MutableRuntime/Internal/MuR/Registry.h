// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Misc/ScopeLock.h"


template <typename KeyType, typename ValueType>
class TRegistry : public TSharedFromThis<TRegistry<KeyType, ValueType>>
{
public:
	class FHandle
	{
		friend TRegistry;
	
	public:
		FHandle() = default;

	private:
		FHandle(uint32 InId, const TWeakPtr<TRegistry>& InWeakRegistry)
		{
			Id = InId;
			WeakRegistry = InWeakRegistry;
		}

	public:
		FHandle(const FHandle& Other)
		{
			*this = Other;
		}

		FHandle(FHandle&& Other)
		{
			*this = MoveTemp(Other);
		}

		operator bool() const
		{
			return Id != ID_NONE && WeakRegistry.IsValid();
		}
		
		FHandle& operator=(const FHandle& Other)
		{
			Id = Other.Id;
			WeakRegistry = Other.WeakRegistry;

			if (TSharedPtr<TRegistry> Registry = WeakRegistry.Pin())
			{
				Registry->Increment(Id);
			}
			
			return *this;
		}

		FHandle& operator=(FHandle&& Other)
		{
			Id = Other.Id;
			Other.Id = ID_NONE;

			WeakRegistry = Other.WeakRegistry;
			Other.WeakRegistry = nullptr;
			
			return *this;
		}
		
		bool operator==(const FHandle& Other) const
		{
			return Id == Other.Id && WeakRegistry == Other.WeakRegistry;
		}
		
		~FHandle()
		{
			if (TSharedPtr<TRegistry> Registry = WeakRegistry.Pin())
			{
				Registry->Decrement(Id);
			}
		}

		const KeyType* GetKey() const
		{
			if (TSharedPtr<TRegistry> Registry = WeakRegistry.Pin())
			{
				return &Registry->GetKey(Id);
			}
			else
			{
				return nullptr;
			}
		}

		const ValueType* GetValue() const
		{
			if (TSharedPtr<TRegistry> Registry = WeakRegistry.Pin())
			{
				return &Registry->GetValue(Id);
			}
			else
			{
				return nullptr;
			}
		}

		
		FString ToString() const
		{
			return FString::FromInt(Id);
		}

		friend uint32 GetTypeHash(const FHandle& Handle)
		{
			return HashCombine(Handle.Id, GetTypeHash(Handle.WeakRegistry));
		}

	private:
		uint32 Id = ID_NONE;

		TWeakPtr<TRegistry> WeakRegistry;
	};

private:
	struct FKeyContainer
	{
		TSharedRef<KeyType> Key;
		
		bool operator==(const FKeyContainer& Other) const
		{
			return Key.Get() == Other.Key.Get();
		}

		bool operator==(const KeyType& Other) const
		{
			return Key.Get() == Other;
		}
		
		friend uint32 GetTypeHash(const FKeyContainer& KeyContainer)
		{
			return GetTypeHash(KeyContainer.Key.Get());
		}
	};
	
	struct FEntry
	{
		uint32 RefCount = 0;

		FKeyContainer KeyContainer;

		ValueType Value;
	};
	
public:
	FHandle Add(const KeyType& Key, const ValueType& Value)
	{
		FScopeLock Lock(&CriticalSection);

		if (uint32* Result = KeyIndex.FindByHash(GetTypeHash(Key), Key))
		{
			const uint32 Id = *Result;
			
			++Entries[Id].RefCount;
			return FHandle(Id, TSharedFromThis<TRegistry>::AsShared());
		}

		uint32 Id = GlobalId++;

		FKeyContainer KeyContainer { MakeShared<KeyType>(Key) };
		
		Entries.Emplace(Id, FEntry(1, KeyContainer, Value));
		KeyIndex.Emplace(KeyContainer, Id);
		
		return FHandle(Id, TSharedFromThis<TRegistry>::AsShared());
	}

	ValueType* Find(const KeyType& Key)
	{
		if (uint32* Result = KeyIndex.FindByHash(GetTypeHash(Key), Key))
		{
			const uint32 Id = *Result;
			
			return &Entries[Id].Value;
		}
		else
		{
			return nullptr;
		}
	}
	
private:
	const KeyType& GetKey(int32 Id) const
	{
		return Entries[Id].KeyContainer.Key.Get();
	}

	const ValueType& GetValue(int32 Id) const
	{
		return Entries[Id].Value;
	}
	
	void Increment(uint32 Id)
	{
		FScopeLock Lock(&CriticalSection);

		FEntry& Entry = Entries[Id];
		++Entry.RefCount;
	}

	void Decrement(uint32 Id)
	{
		FScopeLock Lock(&CriticalSection);

		FEntry& Entry = Entries[Id];
		--Entry.RefCount;

		if (Entry.RefCount == 0)
		{
			KeyIndex.Remove(Entry.KeyContainer);
			Entries.Remove(Id);
		}
	}
	
	static constexpr uint32 ID_NONE = TNumericLimits<uint32>::Max();

	/** Global incremental Id counter. */
	uint32 GlobalId = 0;
	
	TMap<uint32, FEntry> Entries;
	TMap<FKeyContainer, uint32> KeyIndex;

	FCriticalSection CriticalSection;
};