// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MemoryCounters.h"
#include "MuR/Registry.h"
#include "MuR/Operations.h"


namespace UE::Mutable::Private
{
	// Image
	struct FGeneratedImageKey
	{
		FOperation::ADDRESS Address = 0;

		TMemoryTrackedArray<uint8> ParameterValuesBlob;

		bool operator==(const FGeneratedImageKey&) const = default;
	};
	
	struct FGeneratedImageData
	{
	};
	
	MUTABLERUNTIME_API uint32 GetTypeHash(const FGeneratedImageKey& Key);

	typedef TRegistry<FGeneratedImageKey, FGeneratedImageData>::FHandle FImageId;

	typedef TRegistry<FGeneratedImageKey, FGeneratedImageData> FImageIdRegistry;

	MUTABLERUNTIME_API uint32 GetTypeHashPersistent(const FImageId& Id);

	// Material
	struct FGeneratedMaterialKey
	{
		FOperation::ADDRESS Address = 0;

		TMemoryTrackedArray<uint8> ParameterValuesBlob;

		bool operator==(const FGeneratedMaterialKey&) const = default;
	};
	
	struct FGeneratedMaterialData
	{
	}; 
	
	MUTABLERUNTIME_API uint32 GetTypeHash(const FGeneratedMaterialKey& Key);
	
	typedef TRegistry<FGeneratedMaterialKey, FGeneratedMaterialData>::FHandle FMaterialId;

	typedef TRegistry<FGeneratedMaterialKey, FGeneratedMaterialData> FMaterialIdRegistry;

	MUTABLERUNTIME_API uint32 GetTypeHashPersistent(const FMaterialId& Id);


	// SkeletalMesh
	struct FGeneratedSkeletalMeshKey
	{
		FOperation::ADDRESS Address = 0;

		TMemoryTrackedArray<uint8> ParameterValuesBlob;

		bool operator==(const FGeneratedSkeletalMeshKey&) const = default;
	};

	struct FGeneratedSkeletalMeshData
	{
	};
	
	MUTABLERUNTIME_API uint32 GetTypeHash(const FGeneratedSkeletalMeshKey& Key);
	
	typedef TRegistry<FGeneratedSkeletalMeshKey, FGeneratedSkeletalMeshData>::FHandle FSkeletalMeshId;

	typedef TRegistry<FGeneratedSkeletalMeshKey, FGeneratedSkeletalMeshData> FSkeletalMeshIdRegistry;

	MUTABLERUNTIME_API uint32 GetTypeHashPersistent(const FSkeletalMeshId& Id);
}
