// Copyright Epic Games, Inc. All Rights Reserved.

#include "RomManager.h"

#include "Model.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
	template UE_API Tasks::TTask<TManagedPtr<const FImage>> FRomManager::GetRom<FImage>(uint32 RomIndex, TSharedRef<FModelReader> ModelReader);
	template UE_API Tasks::TTask<TManagedPtr<const FMesh>> FRomManager::GetRom<FMesh>(uint32 RomIndex, TSharedRef<FModelReader> ModelReader);

	template UE_API void FRomManager::SetRom<FImage>(uint32 RomIndex, const TManagedPtr<const FImage>& Task);
	template UE_API void FRomManager::SetRom<FMesh>(uint32 RomIndex, const TManagedPtr<const FMesh>& Task);


	std::atomic<SSIZE_T>& MemoryCounters::FStreamingMemoryCounter::Get()
	{
		static std::atomic<SSIZE_T> Counter{0};
		return Counter;
	}
	
	
	bool FRomManager::FRomWeight::operator<(const FRomWeight& Other) const
	{
		return RomTick < Other.RomTick;
	}


	int64 FRomManager::EnsureBudgetBelow(int64 BytesToFree)
	{
		MUTABLE_CPUPROFILER_SCOPE(FRomManager::EnsureBudgetBelow);
	
		TScopeLock<FMutex> Lock(CriticalSection);

		int64 FreedBytes = 0;
		
		while (true)
		{
			MUTABLE_CPUPROFILER_SCOPE(EnsureBudgetBelow_UnloadRom);

			if (FreedBytes >= BytesToFree || !RomWeights.Num())
			{
				break;
			}
			
			FRomWeight RomWeight;
			RomWeights.HeapPop(RomWeight, EAllowShrinking::No);

			switch (static_cast<ERomDataType>(Model.GetProgram().Roms[RomWeight.RomIndex].ResourceType))
			{
			case ERomDataType::Image:
			{
				Tasks::TTask<TManagedPtr<const FImage>>* Task = ConstantImageLODsStreamed.Find(RomWeight.RomIndex);
				check(Task);
					
				if (Task->IsCompleted())
				{
					const TManagedPtr<const FImage>& Data = Task->GetResult();
					if (Data.IsUniqueReference())
					{
						FreedBytes += Data->GetDataSize();
					}
				}
				
				// Released even when we do not know if it is really destroyed.
				ConstantImageLODsStreamed.Remove(RomWeight.RomIndex);
					
				break;
			}
			
			case ERomDataType::Mesh:
			{
				Tasks::TTask<TManagedPtr<const FMesh>>* Task = ConstantMeshesStreamed.Find(RomWeight.RomIndex);
				check(Task);

				if (Task->IsCompleted())
				{
					const TManagedPtr<const FMesh>& Data = Task->GetResult();
					if (Data.IsUniqueReference())
					{
						FreedBytes += Data->GetDataSize();
					}
				}

				ConstantMeshesStreamed.Remove(RomWeight.RomIndex);
					
				break;
			}
			
			default:
				check(false);
				break;
			}
		}
		
		return FreedBytes;
	}

	
	int64 FRomManager::GetRomBytes() const
	{
		TScopeLock<FMutex> Lock(CriticalSection);
		
		int64 Result = 0;
		
		// Count streamable and currently-loaded resources
		for (TPair<uint32, Tasks::TTask<TManagedPtr<const FImage>>>& Pair : ConstantImageLODsStreamed)
		{
			if (TManagedPtr<const FImage> Rom = Pair.Value.GetResult())
			{
				Result += Rom->GetDataSize();
			}
		}
		
		for (TPair<uint32, Tasks::TTask<TManagedPtr<const FMesh>>>& Pair : ConstantMeshesStreamed)
		{
			if (TManagedPtr<const FMesh> Rom = Pair.Value.GetResult())
			{
				Result += Rom->GetDataSize();
			}
		}
		
		return Result;
	}

	
	uint32 FRomManager::GetRomDiskSize(int32 RomIndex) const
	{
		return Model.GetProgram().Roms[RomIndex].Size;
	}


	FRomManager::FRomManager(FModel& InModel) :
		Model(InModel)
	{
	}

	
	void FRomManager::Serialise(FModelWriter& Streamer, bool bDropData)
	{
		FOutputMemoryStream MemStream(16 * 1024 * 1024);

		// Save images and unload from memory
		for (TPair<uint32, Tasks::TTask<TManagedPtr<const FImage>>>& Entry: ConstantImageLODsStreamed)
		{
			int32 RomIndex = Entry.Key;
			const FRomDataRuntime& RomData = Model.GetProgram().Roms[RomIndex];
			check(RomData.ResourceType == uint32(ERomDataType::Image));

			// Serialize to memory, to find out final size of this rom
			MemStream.Reset();
			FOutputArchive MemoryArch(&MemStream);
			FImage::Serialise(Entry.Value.GetResult().Get(), MemoryArch);
			check(RomData.Size == MemStream.GetBufferSize());

			Streamer.OpenWriteFile(RomIndex, true);
			Streamer.Write(MemStream.GetBuffer(), MemStream.GetBufferSize());
			Streamer.CloseWriteFile();

			// Do this progressively to avoid duplicating all data in memory.
			if (bDropData)
			{
				Entry = {};
			}
		}

		if (bDropData)
		{
			ConstantImageLODsStreamed.Empty(0);
		}

		// Save meshes and unload from memory
		for (TPair<uint32, Tasks::TTask<TManagedPtr<const FMesh>>>& Entry : ConstantMeshesStreamed)
		{
			int32 RomIndex = Entry.Key;
			const FRomDataRuntime& RomData = Model.GetProgram().Roms[RomIndex];
			check(RomData.ResourceType == uint32(ERomDataType::Mesh));

			// Serialize to memory, to find out final size of this rom
			MemStream.Reset();
			FOutputArchive MemoryArch(&MemStream);
			FMesh::Serialise(Entry.Value.GetResult().Get(), MemoryArch);
			check(RomData.Size == MemStream.GetBufferSize());

			Streamer.OpenWriteFile(RomIndex, true);
			Streamer.Write(MemStream.GetBuffer(), MemStream.GetBufferSize());
			Streamer.CloseWriteFile();

			// Do this progressively to avoid duplicating all data in memory.
			if (bDropData)
			{
				Entry = {};
			}
		}

		if (bDropData)
		{
			ConstantMeshesStreamed.Empty(0);
		}
	}

	
	void FRomManager::UnloadRoms()
	{
		TScopeLock<FMutex> Lock(CriticalSection);

		ConstantImageLODsStreamed.Empty();
		ConstantMeshesStreamed.Empty();
		RomWeights.Empty();
	}


	template <typename Type>
	void FRomManager::SetRom(uint32 RomIndex, const TManagedPtr<const Type>& Rom)
	{
		TScopeLock<FMutex> Lock(CriticalSection);

		if constexpr (std::is_same<Type, FImage>::value)
		{
			check(!ConstantImageLODsStreamed.Contains(RomIndex));
			ConstantImageLODsStreamed.Add(RomIndex, Tasks::MakeCompletedTask<TManagedPtr<const FImage>>(Rom));
		}
		else if constexpr (std::is_same<Type, FMesh>::value)
		{
			check(!ConstantMeshesStreamed.Contains(RomIndex));
			ConstantMeshesStreamed.Add(RomIndex, Tasks::MakeCompletedTask<TManagedPtr<const FMesh>>(Rom));
		}
		
		FRomWeight RomWeight;
		RomWeight.RomIndex = RomIndex;
		RomWeight.RomTick = FSystem::RomTick++;
		
		RomWeights.HeapPush(RomWeight);
	}

	
	template <typename Type>
	Tasks::TTask<TManagedPtr<const Type>> FRomManager::GetRom(uint32 RomIndex, TSharedRef<FModelReader> ModelReader)
	{
		TScopeLock<FMutex> Lock(CriticalSection);
				
		Tasks::TTask<TManagedPtr<const Type>>* Result = nullptr;
			
		if constexpr (std::is_same<Type, FImage>::value)
		{
			Result = ConstantImageLODsStreamed.Find(RomIndex);
		}
		else if constexpr (std::is_same<Type, FMesh>::value)
		{
			Result = ConstantMeshesStreamed.Find(RomIndex);
		}

		if (Result)
		{
			FRomWeight* RomWeight = RomWeights.FindByPredicate([&](const FRomWeight& Element)
			{
				return Element.RomIndex == RomIndex;
			});
				
			check(RomWeight);
			
			RomWeight->RomTick = FSystem::RomTick++;
				
			return *Result;
		}
			
		Tasks::FTaskEvent ReadCompletionEvent(TEXT("FLoadMeshRomsTaskRom"));

		struct FRomLoadOp
		{
			using StreamingDataContainerType = TArray<uint8, FDefaultMemoryTrackingAllocator<MemoryCounters::FStreamingMemoryCounter>>;

			FModelReader::FOperationID StreamID = -1;
			StreamingDataContainerType StreamBuffer;
		};
		
		TSharedRef<FRomLoadOp> RomLoadOp = MakeShared<FRomLoadOp>();
		const uint32 RomSize = GetRomDiskSize(RomIndex);
		check(RomSize > 0);
		
		RomLoadOp->StreamBuffer.SetNumUninitialized(RomSize);

		TFunction<void(bool)> Callback = [ReadCompletionEvent](bool bSuccess) mutable // Mutable due Trigger not being const
		{
			ReadCompletionEvent.Trigger();
		};
		
		EDataType DataType;
			
		if constexpr (std::is_same<Type, FImage>::value)
		{
			DataType = EDataType::Image;
		}
		else if constexpr (std::is_same<Type, FMesh>::value)
		{
			DataType = EDataType::Mesh;
		}
		
		RomLoadOp->StreamID = ModelReader->BeginReadBlock(&Model, RomIndex, RomLoadOp->StreamBuffer.GetData(), RomSize, DataType, &Callback);
		if (RomLoadOp->StreamID < 0)
		{
			return Tasks::MakeCompletedTask<TManagedPtr<const Type>>(nullptr);
		}
		
		Tasks::TTask<TManagedPtr<const Type>> Task = Tasks::Launch(UE_SOURCE_LOCATION, [Reader = ModelReader, RomLoadOp]() -> TManagedPtr<const Type>
		{
			const bool bSuccess = Reader->EndRead(RomLoadOp->StreamID);
			if (!bSuccess)
			{
				return nullptr;
			}

			FInputMemoryStream Stream(RomLoadOp->StreamBuffer.GetData(), RomLoadOp->StreamBuffer.Num());
			FInputArchive Arch(&Stream);
			
			return Type::StaticUnserialise(Arch);
		}, 
		ReadCompletionEvent);
			
		FRomWeight RomWeight;
		RomWeight.RomIndex = RomIndex;
		RomWeight.RomTick = FSystem::RomTick++;
				
		RomWeights.HeapPush(RomWeight);
			
		if constexpr  (std::is_same<Type, FImage>::value)
		{
			ConstantImageLODsStreamed.Add(RomIndex, Task);
		}
		else if constexpr (std::is_same<Type, FMesh>::value)
		{
			ConstantMeshesStreamed.Add(RomIndex, Task);
		}
			
		return Task;
	}
}


#undef UE_API
