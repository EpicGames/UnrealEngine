// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ManagedPointer.h"
#include "MemoryCounters.h"
#include "Mesh.h"
#include "Image.h"
#include "System.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
	namespace MemoryCounters
	{
		struct FStreamingMemoryCounter
		{
			static MUTABLERUNTIME_API std::atomic<SSIZE_T>& Get();
		};
	}
	
	struct FConstantResourceIndex;
	
	class FRomManager
	{
	public:
		FRomManager(FModel& InModel);

		void Serialise(FModelWriter& Streamer, bool bDropData);

		template<typename Type>
		void SetRom(uint32 RomIndex, const TManagedPtr<const Type>& Rom);

		template<typename Type>
		Tasks::TTask<TManagedPtr<const Type>> GetRom(uint32 RomIndex, TSharedRef<FModelReader> ModelReader);

		UE_API void UnloadRoms();

		UE_API int64 EnsureBudgetBelow(int64 BytesToFree);
		
		UE_API int64 GetRomBytes() const;
	
	private:
		uint32 GetRomDiskSize(int32 RomIndex) const;
		
		const FModel& Model;
		
		mutable FMutex CriticalSection;
		
		struct FRomWeight
		{
			uint32 RomIndex = INDEX_NONE;
			uint32 RomTick = 0;
			
			bool operator<(const FRomWeight& Other) const;
		};
		
		TMemoryTrackedArray<FRomWeight> RomWeights;
		
		/** Constant mesh data is split in 2 sets: ConstantMeshesStreamed constains data that is streamed in and out.
		* Index with FConstantResourceIndex::Index, when Streamable is 1.
		* This part is empty for an unused Model, and shouldn't be serialised. */
		mutable TMap<uint32, Tasks::TTask<TManagedPtr<const FMesh>>> ConstantMeshesStreamed;
		
		/** Constant image mip data is split in 2 sets: ConstantImageLODsStreamed constains data that is streamed in and out. 
		* Index with FConstantResourceIndex::Index, when Streamable is 1.
		* This part is empty for an unused Model, and shouldn't be serialised. */
		mutable TMap<uint32, Tasks::TTask<TManagedPtr<const FImage>>> ConstantImageLODsStreamed;
	};
}


#undef UE_API