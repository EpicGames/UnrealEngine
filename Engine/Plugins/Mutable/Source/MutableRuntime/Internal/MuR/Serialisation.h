// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ManagedPointer.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/MutableMath.h"
#include "MuR/Serialisation.h"

#include "Misc/TVariant.h"
#include "Math/Vector.h"
#include "Math/Quat.h"
#include "Math/IntVector.h"
#include "Math/Vector4.h"
#include "Math/Rotator.h"
#include "HAL/Platform.h"
#include "Curves/RichCurve.h"
#include "Containers/StaticArray.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/StrongObjectPtr.h"

struct FMeshToMeshVertData;
class UTexture;
class USkeletalMesh;

#define UE_API MUTABLERUNTIME_API

namespace UE::Mutable::Private
{    
    class FImage;
	class FModel;
	enum class EDataType : uint8;

#define MUTABLE_DEFINE_POD_SERIALISABLE(Type)									\
	void DLLEXPORT operator<<(UE::Mutable::Private::FOutputArchive& Arch, const Type& T);			\
	void DLLEXPORT operator>>(UE::Mutable::Private::FInputArchive& Arch, Type& T);

#define MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(Type)							\
	template<typename Alloc>													\
	void DLLEXPORT operator<<(UE::Mutable::Private::FOutputArchive& Arch, const TArray<Type, Alloc>& V);	\
	template<typename Alloc>													\
	void DLLEXPORT operator>>(UE::Mutable::Private::FInputArchive& Arch, TArray<Type, Alloc>& V);

#define MUTABLE_DEFINE_ENUM_SERIALISABLE(Type)									\
    void DLLEXPORT operator<<(UE::Mutable::Private::FOutputArchive& Arch, const Type& T);			\
    void DLLEXPORT operator>>(UE::Mutable::Private::FInputArchive& Arch, Type& T);
	
	
    /** */
	class FModelReader
    {
    public:

        //-----------------------------------------------------------------------------------------
        // Life cycle
        //-----------------------------------------------------------------------------------------

        //! Ensure virtual destruction.
        virtual ~FModelReader() = default;

        //-----------------------------------------------------------------------------------------
        // Reading interface
        //-----------------------------------------------------------------------------------------

        //! Identifier of reading data operations sent to this interface.
		//! Negative values indicate an error.
        typedef int32 FOperationID;

		virtual bool DoesBlockExist(const FModel*, uint32 BlockKey) = 0;

		//! \brief Start a data request operation.
		//! \param Model.
        //! \param BlockKey key identifying the model data fragment that is requested.
        //!         This key interpretation depends on the implementation of the ModelStreamer,
		//! \param Buffer is an already-allocated buffer big enough to receive the expected data.
		//! \param size is the size of the pBuffer buffer, which must match the size of the data
		//! requested with the key identifiers.
		//! \param CompletionCallback Optional callback. Copied inside the called function. Will always be called.
		//! \return a previously unused identifier, now used for this operation, that can be used in
		//! calls to the other methods of this interface. If the return value is negative it indicates
		//! an unrecoverable error.
		virtual FOperationID BeginReadBlock(const FModel*, uint32 BlockKey, void* Buffer, uint64 Size, EDataType DataType, TFunction<void(bool bSuccess)>* CompletionCallback = nullptr) = 0;

        //! Check if a data request operation has been completed.
        //! This is a weak check than *may* return true if the given operation has completed, but
        //! it is not mandatory. It is used as a hint by the System to optimise its opertaions.
        //! There is no guarantee that this method will ever be called, and it is safe to always
        //! return false.
        virtual bool IsReadCompleted(FOperationID) = 0;

        /** Complete a data request operation.This method has to block until a data request issued
        * with BeginReadBlock has been completed. After returning from this call, the ID cannot be used
        * any more to identify the same operation and becomes free.
		* \return true if the data was loaded successfully.
		*/
        virtual bool EndRead(FOperationID) = 0;
    };


	/** */
	class FModelWriter
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------

		//! Ensure virtual destruction.
		virtual ~FModelWriter() = default;

		//-----------------------------------------------------------------------------------------
		// Writing interface
		//-----------------------------------------------------------------------------------------

		/** */
		virtual void OpenWriteFile(uint32 BlockKey, bool bIsStreamable) = 0;

		/** */
		virtual void Write(const void* Buffer, uint64 Size) = 0;

		//! \brief Close the file open for writing in a previous call to OpenWriteFile in this
		//! object.
		virtual void CloseWriteFile() = 0;


	};


    /** Interface for any input stream to be use with InputArchives. */
    class FInputStream
    {
    public:

		/** Ensure virtual destruction. */
		virtual ~FInputStream() = default;

        /** Read a byte buffer
         * \param pData destination buffer, must have at least size bytes allocated.
         * \param size amount of bytes to read from the stream.
		 */
        virtual void Read( void* Data, uint64 Size ) = 0;
    };


    /** Interface for any output stream to be used with OutputArchives. */
    class FOutputStream
    {
    public:

        /** Ensure virtual destruction. */
        virtual ~FOutputStream() = default;

        /** Write a byte buffer
         * \param pData source buffer where data will be read from.
         * \param size amount of data to write to the stream.
		 */
        virtual void Write( const void* Data, uint64 Size ) = 0;

    };


    /** Archive containing data to be deserialised. */
    class FInputArchive
    {
    public:

        /** Construct form an input stream.The stream will not be owned by the archive and the
         * caller must make sure it is not modified or destroyed while serialisation is happening.
		 */
        UE_API FInputArchive( FInputStream* );

		/** Ensure virtual destruction. */
		virtual ~FInputArchive() = default;

		/** Not owned. */
		FInputStream* Stream = nullptr;

		/** Already read pointers. */
		TArray< TManagedPtr<void> > History;

    };


    /** Archive where data can be serialised to. */
    class FOutputArchive
    {
    public:

        /** Construct form an output stream.The stream will not be owned by the archive and the
         * caller must make sure it is not modified or destroyed while serialisation is happening.
		 */
        UE_API FOutputArchive( FOutputStream* );

		/** Not owned. */
		FOutputStream* Stream = nullptr;

		/** Already written pointers and their ids. */
		TMap< const void*, int32 > History;

    };


    //!
    class FOutputMemoryStream : public FOutputStream
    {
    public:

        /** Create the stream with an optional buffer size in bytes.
		* The internal buffer will be enlarged as much as necessary.
		*/
        UE_API FOutputMemoryStream( uint64 Reserve = 0 );

        // FOutputStream interface
        UE_API virtual void Write( const void* Data, uint64 Size ) override;

        // Own interface

        /** Get the serialised data buffer pointer. This pointer invalidates after a Write
         * operation has been done, and you need to get it again.
		 */
        UE_API const uint8* GetBuffer() const;

        /** Get the amount of data in the stream, in bytes. */
        UE_API int32 GetBufferSize() const;

		/** Clear the internal buffer. */
		UE_API void Reset();

    private:

		TArray64<uint8> Buffer;

    };


	/** This stream doesn't store any data: it just counts the amount of data serialised. */
	class FOutputSizeStream : public FOutputStream
	{
	public:

		// FOutputStream interface
		UE_API void Write(const void* Data, uint64 Size) override;

		// Own interface

		/** Get the amount of data serialised, in bytes. */
		UE_API uint64 GetBufferSize() const;

	private:

		uint64 WrittenBytes = 0;

	};


	/** This stream doesn't store any data: it just calculates of a hash of the data as it receives it. */
	class FOutputHashStream : public FOutputStream
	{
	public:
		// FOutputStream interface
		UE_API void Write(const void* Data, uint64 Size) override;

		// Own interface

		/** Return the hash of the data written so far. */
		UE_API uint64 GetHash() const;

	private:
		uint64 Hash = 0;
	};


	template<typename Type>
	void operator<<(FOutputArchive& Arch, const Type& Value)
	{
        Value.Serialise(Arch);
	}

	template<typename Type>
	void operator>>(FInputArchive& Arch, Type& Value)
	{
        Value.Unserialise(Arch);
	}

	MUTABLE_DEFINE_POD_SERIALISABLE(float);
	MUTABLE_DEFINE_POD_SERIALISABLE(double);

    MUTABLE_DEFINE_POD_SERIALISABLE(int8);
    MUTABLE_DEFINE_POD_SERIALISABLE(int16);
    MUTABLE_DEFINE_POD_SERIALISABLE(int32);
    MUTABLE_DEFINE_POD_SERIALISABLE(int64);

    MUTABLE_DEFINE_POD_SERIALISABLE(uint8);
    MUTABLE_DEFINE_POD_SERIALISABLE(uint16);
    MUTABLE_DEFINE_POD_SERIALISABLE(uint32);
	MUTABLE_DEFINE_POD_SERIALISABLE(uint64);

	MUTABLE_DEFINE_POD_SERIALISABLE(FUintVector2);
	MUTABLE_DEFINE_POD_SERIALISABLE(FIntVector2);
	MUTABLE_DEFINE_POD_SERIALISABLE(UE::Math::TIntVector2<uint16>);
	MUTABLE_DEFINE_POD_SERIALISABLE(UE::Math::TIntVector2<int16>);
	MUTABLE_DEFINE_POD_SERIALISABLE(FVector2f);
	MUTABLE_DEFINE_POD_SERIALISABLE(FVector4f);
	MUTABLE_DEFINE_POD_SERIALISABLE(FMatrix44f);
	MUTABLE_DEFINE_POD_SERIALISABLE(FRichCurveKey);

	MUTABLE_DEFINE_POD_SERIALISABLE(FGuid);

	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(float);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(double);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(uint8);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(uint16);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(uint32);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(uint64);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(int8);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(int16);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(int32);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(int64);

	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FVector2f);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FMatrix44f);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FIntVector2);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(TCHAR);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FRichCurveKey);

	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FUintVector2);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(UE::Math::TIntVector2<uint16>);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(UE::Math::TIntVector2<int16>);
	MUTABLE_DEFINE_POD_VECTOR_SERIALISABLE(FVector4f);
	
	//---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API void operator<<(FOutputArchive&, const FString&);
	MUTABLERUNTIME_API void operator>>(FInputArchive&, FString&);

	//---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API void operator<<(FOutputArchive&, const FRichCurve&);
	MUTABLERUNTIME_API void operator>>(FInputArchive&, FRichCurve&);


	//---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API void operator<<(FOutputArchive&, const FName&);
	MUTABLERUNTIME_API void operator>>(FInputArchive&, FName&);

	//---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API void operator<<(FOutputArchive&, const FMeshToMeshVertData&);
	MUTABLERUNTIME_API void operator>>(FInputArchive&, FMeshToMeshVertData&);

	//---------------------------------------------------------------------------------------------
	template<typename T, typename Alloc> 
	void operator<<(FOutputArchive& Arch, const TArray<T, Alloc>& V)
	{
		const uint32 Num = (uint32)V.Num();
		Arch << Num;
		
		for (SIZE_T Index = 0; Index < Num; ++Index)
		{
			Arch << V[Index];
		}
	}

	template<typename T, typename Alloc> 
	void operator>>(FInputArchive& Arch, TArray<T, Alloc>& V)
	{
		uint32 Num;
		Arch >> Num;
		V.SetNum(Num);

		for (SIZE_T Index = 0; Index < Num; ++Index)
		{
			Arch >> V[Index];
		}
	}


	// Bool size is not a standard
	MUTABLERUNTIME_API void operator<<(FOutputArchive&, const bool&);
	MUTABLERUNTIME_API void operator>>(FInputArchive&, bool&);

	
	template<typename T>
	void operator<<(FOutputArchive& Arch, const TStrongObjectPtr<T>& V)
	{
    	// Do Nothing. UObjects values are saved in the CO.	
	}

	template<typename T>
	void operator>>(FInputArchive& Arch, TStrongObjectPtr<T>& V)
	{
    	// Do Nothing. UObjects values are saved in the CO.	
	}

	MUTABLERUNTIME_API void operator<<(FOutputArchive& Arch, const TManagedPtr<const FInstancedStruct>& V);
	MUTABLERUNTIME_API void operator>>(FInputArchive& Arch, TManagedPtr<const FInstancedStruct>& V);
	
	
    //!
    class FInputMemoryStream : public FInputStream
    {
    public:

        //-----------------------------------------------------------------------------------------
        // Life cycle
        //-----------------------------------------------------------------------------------------

        //! Create the stream using an external buffer.
        //! The buffer will not be owned by this object, so it cannot be deallocated while this
        //! objects is in use.
        UE_API FInputMemoryStream( const void* pBuffer, uint64 size );


        //-----------------------------------------------------------------------------------------
        // FInputStream interface
        //-----------------------------------------------------------------------------------------
        UE_API void Read( void* pData, uint64 size ) override;


    private:

		const void* Buffer = nullptr;
		uint64 Size = 0;
		uint64 Pos = 0;

    };


#define MUTABLE_IMPLEMENT_POD_SERIALISABLE(Type)				     \
    void DLLEXPORT operator<<(FOutputArchive& Arch, const Type& T)   \
    {																 \
        Arch.Stream->Write(&T, sizeof(Type));						 \
    }																 \
                                                                     \
    void DLLEXPORT operator>>(FInputArchive& Arch, Type& T)		     \
    {																 \
        Arch.Stream->Read(&T, sizeof(Type));						 \
    }																 \
		

#define MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(Type)                        \
    template<class Alloc>                                                      \
	void DLLEXPORT operator<<(FOutputArchive& Arch, const TArray<Type, Alloc>& V)         \
	{                                                                          \
		uint32 Num = uint32(V.Num());                                          \
		Arch << Num;                                                           \
		if (Num)                                                               \
		{                                                                      \
			Arch.Stream->Write(V.GetData(), Num * sizeof(Type));			   \
		}                                                                      \
	}                                                                          \
                                                                               \
    template<class Alloc>                                                      \
	void DLLEXPORT operator>>(FInputArchive& Arch, TArray<Type, Alloc>& V)     \
	{                                                                          \
		uint32 Num;                                                            \
		Arch >> Num;                                                           \
		V.SetNum(Num);                                                         \
		if (Num)                                                               \
		{                                                                      \
			Arch.Stream->Read(V.GetData(), Num * sizeof(Type));					   \
		}                                                                      \
	}                                                                          \

	/** TVariant custom serialize. Based on the default serialization. */
	template <typename... Ts>
	void operator<<(FOutputArchive& Ar, const TVariant<Ts...>& Variant)
	{
		const uint8 Index = static_cast<uint8>(Variant.GetIndex());
		Ar << Index;
		
		Visit([&Ar](auto& StoredValue)
		{
			Ar << StoredValue;
		}, Variant);
	}


	template <typename T, typename VariantType>
	struct TVariantLoadFromInputArchiveCaller
	{
		/** Default construct the type and load it from the FArchive */
		static void Load(FInputArchive& Ar, VariantType& OutVariant)
		{
			OutVariant.template Emplace<T>();
			Ar >> OutVariant.template Get<T>();
		}
	};

	
	template <typename... Ts>
	struct TVariantLoadFromInputArchiveLookup
	{
		using VariantType = TVariant<Ts...>;
		static_assert((std::is_default_constructible<Ts>::value && ...), "Each type in TVariant template parameter pack must be default constructible in order to use FArchive serialization");

		/** Load the type at the specified index from the FArchive and emplace it into the TVariant */
		static void Load(SIZE_T TypeIndex, FInputArchive& Ar, VariantType& OutVariant)
		{
			static constexpr void(*Loaders[])(FInputArchive&, VariantType&) = { &TVariantLoadFromInputArchiveCaller<Ts, VariantType>::Load... };
			check(TypeIndex < UE_ARRAY_COUNT(Loaders));
			Loaders[TypeIndex](Ar, OutVariant);
		}
	};

	
	template <typename... Ts>
	void operator>>(FInputArchive& Ar, TVariant<Ts...>& Variant)
	{
		uint8 Index;
		Ar >> Index;
		check(Index < sizeof...(Ts));

		TVariantLoadFromInputArchiveLookup<Ts...>::Load(static_cast<SIZE_T>(Index), Ar, Variant);
	}

#define MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(Type)						\
        void DLLEXPORT operator<<(FOutputArchive& Arch, const Type& T)   \
		{																\
            uint32 V = (uint32)T;                                       \
            Arch.Stream->Write(&V, sizeof(uint32));					\
		}																\
																		\
        void DLLEXPORT operator>>(FInputArchive& Arch, Type& T)   		\
		{																\
            uint32 V;													\
            Arch.Stream->Read(&V, sizeof(uint32));					\
			T = (Type)V;												\
		}																\

    template<typename T0, typename T1>
    inline void operator<<(FOutputArchive& Arch, const std::pair<T0, T1>& V)
    {
        Arch << V.first;
        Arch << V.second;
    }

    template<typename T0, typename T1>
    inline void operator>>(FInputArchive& Arch, std::pair<T0, T1>& V)
    {
        Arch >> V.first;
        Arch >> V.second;
    }
	
	template<typename T, uint32 Size, uint32 Align> 
    void operator<<(FOutputArchive& Arch, const TStaticArray<T, Size, Align>& V)
	{
		for (int32 I = 0; I < Size; ++I)
		{
			Arch << V[I];
		}
	}

	template<typename T, uint32 Size, uint32 Align> 
    void operator>>(FInputArchive& Arch, TStaticArray<T, Size, Align>& V)
	{
		for (uint32 I = 0; I < Size; ++I)
		{
			Arch >> V[I];
		}
	}
	

	//---------------------------------------------------------------------------------------------
	template< typename K, typename T >
	inline void operator<< (FOutputArchive& Arch, const TMap<K, T>& V)
	{
		Arch << (uint32)V.Num();
		for (const TPair<K, T>& Element : V)
		{
			Arch << Element.Key;
			Arch << Element.Value;
		}
	}

	template< typename K, typename T >
	inline void operator>> (FInputArchive& Arch, TMap<K, T>& V)
	{
		uint32 Num;
		Arch >> Num;

		for (uint32 Index = 0; Index < Num; ++Index)
		{
			K Key;
			T Element;
			Arch >> Key;
			Arch >> Element;

			V.Emplace(MoveTemp(Key), MoveTemp(Element));
		}
	}

	// Unreal POD Serializables
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(float);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(double);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(uint8);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(uint16);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(uint32);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(uint64);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(int8);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(int16);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(int32);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(int64);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(TCHAR);

	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FIntVector2);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FUintVector2);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(UE::Math::TIntVector2<uint16>);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(UE::Math::TIntVector2<int16>);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FVector2f);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FVector4f);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FMatrix44f);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FRichCurveKey);


	template < 
		typename T 
	>
	inline void operator<< (FOutputArchive& Arch, const TManagedPtr<const T>& Ptr)
	{
		if (!Ptr)
		{
			Arch << (int32)-1;
		}
		else
		{
			int32* it = Arch.History.Find(Ptr.Get());

			if (!it)
			{
				int32 Id = Arch.History.Num();
				Arch.History.Add(Ptr.Get(), Id);
				Arch << Id;
				T::Serialise(Ptr.Get(), Arch);
			}
			else
			{
				Arch << *it;
			}
		}
	}

	template< 
		typename T
	>
	inline void operator>> (FInputArchive& Arch, TManagedPtr<const T>& Ptr)
	{
		int32 Id;
		Arch >> Id;

		if (Id == -1)
		{
			Ptr.Reset();
		}
		else
		{
			if (Id < Arch.History.Num())
			{
				Ptr = StaticCastManagedPtr<T>(Arch.History[Id]);

				// If the pointer was null it means the position in history is used, but not set yet
				// option 1: we have a smart pointer loop which is very bad.
				// option 2: the resource in this Ptr is also pointed by a Proxy that has absorbed it
				//			 and this reference should also be a proxy instead of a pointer.
				check(Ptr);
			}
			else
			{
				// Ids come in order, but they may have been absorbed outside in some serialisations
				// like proxies.
				//check( Id == Arch.>History.Num() );
				Arch.History.SetNum(Id + 1, EAllowShrinking::No);

				TManagedPtr<T> Temp = T::StaticUnserialise(Arch);
				Ptr = Temp;
				Arch.History[Id] = Temp;
			}
		}
	}

	//---------------------------------------------------------------------------------------------
	template< typename T >
	inline void operator<< ( FOutputArchive& arch, const Ptr<T>& p )
	{
		operator<<( arch, (const Ptr<const T>&) p );
	}

	template< typename T >
	inline void operator>> ( FInputArchive& arch, Ptr<T>& p )
	{
		operator>>( arch, (Ptr<const T>&) p );
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	template<typename T0, typename T1> inline void operator<<(FOutputArchive& arch, const TPair<T0,T1>& v)
	{
		arch << v.Key;
		arch << v.Value;
	}

	template<typename T0, typename T1> inline void operator>>(FInputArchive& arch, TPair<T0, T1>& v)
	{
		arch >> v.Key;
		arch >> v.Value;
	}


	//---------------------------------------------------------------------------------------------
	// TODO: As POD?
	template< typename T >
	inline void operator<< (FOutputArchive& arch, const UE::Math::TQuat<T>& v)
	{		
		arch << v.X;
		arch << v.Y;
		arch << v.Z;
		arch << v.W;
	}

	template< typename T >
	inline void operator>> (FInputArchive& arch, UE::Math::TQuat<T>& v)
	{
		arch >> v.X;
		arch >> v.Y;
		arch >> v.Z;
		arch >> v.W;
	}


	//---------------------------------------------------------------------------------------------
	// TODO: As POD?
	template< typename T >
	inline void operator<< (FOutputArchive& arch, const UE::Math::TVector<T>& v)
	{
		arch << v.X;
		arch << v.Y;
		arch << v.Z;
	}

	template< typename T >
	inline void operator>> (FInputArchive& arch, UE::Math::TVector<T>& v)
	{
		arch >> v.X;
		arch >> v.Y;
		arch >> v.Z;
	}


	//---------------------------------------------------------------------------------------------
	template< typename T >
	inline void operator<< (FOutputArchive& arch, const UE::Math::TRotator<T>& R)
	{
		arch << R.Pitch;
		arch << R.Yaw;
		arch << R.Roll;
	}

	template< typename T >
	inline void operator>> (FInputArchive& arch, UE::Math::TRotator<T>& R)
	{
		arch >> R.Pitch;
		arch >> R.Yaw;
		arch >> R.Roll;
	}


	//---------------------------------------------------------------------------------------------
	template< typename T >
	inline void operator<< (FOutputArchive& arch, const UE::Math::TTransform<T>& v)
	{
		arch << v.GetRotation();
		arch << v.GetTranslation();
		arch << v.GetScale3D();
	}

	template< typename T >
	inline void operator>> (FInputArchive& arch, UE::Math::TTransform<T>& v)
	{
		UE::Math::TQuat<T> Rot;
		UE::Math::TVector<T> Trans;
		UE::Math::TVector<T> Scale;

		arch >> Rot;
		arch >> Trans;
		arch >> Scale;

		v.SetComponents(Rot, Trans, Scale);
	}
}


#undef UE_API
