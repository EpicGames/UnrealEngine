// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{
    /** Settings class that can be used to create a customised System instance. */
    class FSettings
    {
    public:

        /** Record internal profiling data. Disabled by default. */
        UE_API void SetProfile( bool bEnabled );

        /** Limit the maximum memory in bytes used by the mutable core.A low value will force
        * more streaming and higher instance construction times, but will use less memory while
        * building objects.
        * Defaults to 0, which disables the limit.
		*/
        UE_API void SetWorkingMemoryBytes( uint64 Bytes );

        /** Set the quality for the image compression algorithms.
        * \param quality Quality level of the compression. Higher quality increases the instance
        * generation time. The value is in the range 0 to 4 with 0 being the fastest. It defaults
        * to 0, which is the fastes for runtime. Tools may want ot set it higher, and low
        * performance profiles to lower.
        * Genreal rules are:
        * 0 - Fastest for runtime
        * 1 - Best for runtime
        * 2 - Fast for tools
        * 3 - Best for tools
        * 4 - Maximum, with no time limits.
        */
        UE_API void SetImageCompressionQuality( int32 Quality );

    public:

		uint64 WorkingMemoryBytes = 0;
		int32 ImageCompressionQuality = 0;
		bool bProfile = false;

    };

}

#undef UE_API
