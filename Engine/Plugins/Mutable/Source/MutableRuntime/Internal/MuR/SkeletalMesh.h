// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RefCounted.h"

#include "PassthroughObject.h"
#include "Operations.h"

class USkeletalMesh;


namespace UE::Mutable::Private
{
	class FLOD;
	class FMaterial;

	class FSkeletalMesh : public FResource
	{
	public:
		// FResource interface
		virtual int32 GetDataSize() const override { return 0; };
		
		TManagedPtr<FSkeletalMesh> Clone() const
		{
			TManagedPtr<FSkeletalMesh> New = MakeManaged<FSkeletalMesh>();

			New->PassthroughObject = PassthroughObject;
			
			// Shallow copy, LODs and Materials are not necesarily owned by the FSkeletalMesh.
			New->LODs = LODs;	
			New->MaterialSlotMaterials = MaterialSlotMaterials;

			New->MaterialSlotNames = MaterialSlotNames;
			New->MaterialSlotIds = MaterialSlotIds;
			
			New->FirstLODAvailable = FirstLODAvailable;
			New->FirstLODResident = FirstLODResident;
		
			New->MinLODs = MinLODs;
			New->MinQualityLevelLODs = MinQualityLevelLODs;
		
			New->ScreenSize = ScreenSize;
			New->LODHysteresis = LODHysteresis;
			New->bSupportUniformlyDistributedSampling = bSupportUniformlyDistributedSampling;
			New->bAllowCPUAccess = bAllowCPUAccess;
			
			New->Name = Name;
			
			return New;
		}

    	TPassthroughObjectPtr<USkeletalMesh> PassthroughObject;
		
		TArray<TManagedPtr<const FLOD>> LODs;
		
		TArray<TVariant<FOperation::ADDRESS, TManagedPtr<const FMaterial>>> MaterialSlotMaterials;
		TArray<FName> MaterialSlotNames;
		TArray<uint32> MaterialSlotIds;
		
		uint8 FirstLODAvailable = MAX_uint8; // TODO SKMPIN SKO Only
		uint8 FirstLODResident = MAX_uint8; // TODO SKMPIN SKO Only
		
		FPerPlatformInt MinLODs;
		FPerQualityLevelInt MinQualityLevelLODs;
		
		TArray<float> ScreenSize;
		TArray<float> LODHysteresis;
		TArray<bool> bSupportUniformlyDistributedSampling;
		TArray<bool> bAllowCPUAccess;
		
		FName Name; // TODO SKMPIN SKO Only
	};
}

