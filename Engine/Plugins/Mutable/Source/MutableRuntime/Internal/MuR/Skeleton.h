// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Serialisation.h"
#include "Containers/Array.h"
#include "HAL/PlatformMath.h"
#include "Math/Transform.h"
#include "Math/VectorRegister.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"

#define UE_API MUTABLERUNTIME_API


namespace UE::Mutable::Private
{	
	union FBoneIdOrIndex
	{
		uint32 Id;
		int32 Index;

		inline void Serialise(FOutputArchive& Arch) const
		{
			Arch << Id;
		}

		inline void Unserialise(FInputArchive& Arch)
		{
			Arch >> Id;
		}

		inline bool operator==(const FBoneIdOrIndex& Other) const
		{
			return Id == Other.Id;
		}
	};

    /** Skeleton object. */
	class FSkeleton
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Life cycle
		//-----------------------------------------------------------------------------------------

        //! Deep clone this skeleton.
        UE_API TManagedPtr<FSkeleton> Clone() const;

		//! Serialisation
        static UE_API void Serialise( const FSkeleton*, FOutputArchive& );
        static UE_API TManagedPtr<FSkeleton> StaticUnserialise( FInputArchive& );


		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		UE_API int32 AddBone(const FName InBoneName, const int16 InParentIndex);
		UE_API int32 FindOrAddBone(const FName InBoneName, const int16 InParentIndex);

		UE_API int32 GetNumBones() const;

	public:

		//! FNames of the bones. Serialized in the program as ConstantNames.
		TArray<FName> BoneNames;

		//! FName Id in the ConstantNames map. 
		TArray<uint32> BoneIds;

		//! For each bone, index of the parent bone in the bone vectors. -1 means no parent.
		//! This array must have the same size than the m_bones array.
		TArray<int16> BoneParents;

		//!
		UE_API void Serialise(FOutputArchive&) const;

		//!
		UE_API void Unserialise(FInputArchive&);

		//!
		inline bool operator==(const FSkeleton& Other) const
		{
			return BoneNames == Other.BoneNames
				// && BoneIds == Other.BoneIds // Don't use boneIds to compare skeletons since they are added when creating the constants.
				&& BoneParents == Other.BoneParents;
		}
	};

}

#undef UE_API
