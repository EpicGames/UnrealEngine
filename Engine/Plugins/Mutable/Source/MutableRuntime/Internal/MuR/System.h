// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/Instance.h"
#include "MuR/RefCounted.h"
#include "MuR/Settings.h"
#include "MuR/Types.h"
#include "MuR/SystemTypes.h"
#include "MuR/MemoryCounters.h"
#include "MuR/Operations.h"
#include "MuR/MutableString.h"
#include "MuR/Material.h"
#include "MuR/Parameters.h"
#include "MuR/MutableTrace.h"
#include "MuR/ManagedPointer.h"
#include "MuR/MutableEditorLogger.h"
#include "MuR/ProgramCache.h"

#include "SkeletalMesh.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "HAL/Platform.h"

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "HAL/Thread.h"
#include "HAL/PlatformTLS.h"
#endif

#include "System.generated.h"

class UTexture;
class USkeletalMesh;

#define UE_API MUTABLERUNTIME_API


/** If set to 1, this enables some expensive Unreal Insights traces, but can lead to 5x slower mutable operation.
* Other cheaper traces are enabled at all times.
*/
#define UE_MUTABLE_ENABLE_SLOW_TRACES	0


/** Despite being an UEnum, this is not always version-serialized (in MutableTools).
 * Beware of changing the enum options or order.
 */
UENUM()
enum class ETextureCompressionStrategy : uint8
{
	/** Don't change the generated format. */
	None,

	/** If a texture depends on run-time parameters for an object state, don't compress. */
	DontCompressRuntime,

	/** Never compress the textures for this state. */
	NeverCompress
};

MUTABLE_DEFINE_ENUM_SERIALISABLE(ETextureCompressionStrategy);


namespace UE::Mutable::Private
{
	// Forward references
	class FModel;
	class FModelReader;
	class FParameters;
    class FMesh;
	class FExtensionDataStreamer;
	struct FLiveInstance;

	/** */
	enum class EExecutionStrategy : uint8
	{
		/** Undefined. */
		None = 0,

		/** Always try to run operations that reduce working memory first. */
		MinimizeMemory,

		/** Always try to run operations that unlock more operations first. */
		MaximizeConcurrency,

		/** Utility value with the number of error types. */
		Count
	};


    /** Interface to request external images used as parameters. */
    class FExternalResourceProvider
    {
    public:
        //! Ensure virtual destruction
        virtual ~FExternalResourceProvider() = default;

        /** Returns the completion event and a cleanup function that must be called once event is completed. */
		virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetImageAsync(UTexture* Texture, uint8 MipmapsToSkip, bool bLoadMipTail, TFunction<void(UE::Mutable::Private::TManagedPtr<FImage>)>& ResultCallback) = 0;
		virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetReferencedImageAsync(int32 Id, uint8 MipmapsToSkip, TFunction<void(UE::Mutable::Private::TManagedPtr<FImage>)>& ResultCallback) { check(false); return {}; }

		virtual FExtendedImageDesc GetImageDesc(UTexture* Texture) = 0;

		/** Returns the completion event and a cleanup function that must be called once event is completed. */
		virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetMeshAsync(USkeletalMesh* SkeleltalMesh, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 InConversionFlags, TFunction<void(UE::Mutable::Private::TManagedPtr<FMesh>)>& ResultCallback) = 0;
		virtual TTuple<UE::Tasks::FTask, TFunction<void()>> GetSkeletalMeshAsync(
				USkeletalMesh* SkeleltalMesh, int32 InLODBegin, int32 InLODEnd, int32 InGeometryLODBegin, int32 InGeometryLODEnd, uint8 InConversionFlags, TFunction<void(UE::Mutable::Private::TManagedPtr<FSkeletalMesh>)>& ResultCallback) = 0;
    };

	
		// Call the tick of the LLM system (we do this to simulate a frame since the LLM system is not entirelly designed to run over a program)
	inline void UpdateLLMStats()
	{
		// This code will only be compiled (and ran) if the global definition to enable LLM tracking is set to 1 for the host program
		// Ex : 			GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
#if ENABLE_LOW_LEVEL_MEM_TRACKER && IS_PROGRAM
		FLowLevelMemTracker& MemTracker = FLowLevelMemTracker::Get();
		if (MemTracker.IsEnabled())
		{
			MemTracker.UpdateStatsPerFrame();
		}
#endif
	}

	constexpr uint64 AllParametersMask = TNumericLimits<uint64>::Max();
 
	/** ExecutinIndex stores the location inside all ranges of the execution of a specific
	* operation. The first integer on each pair is the dimension/range index in the program
	* array of ranges, and the second integer is the value inside that range.
	* The vector order is undefined.
	*/

	/** Class to log update messages */
	class FLiveInstanceLogger
	{
	public:
		
		/** Set the logger class to print the messages to an editor widget */
		UE_API void SetLogger(TSharedPtr<IMutableEditorLogger> InInstanceLogger);

		/** Prints the message into the Logs console/file and also prints it into an editor widget.
			Use this function instead of UE_LOG to print important messages.
		*/
		void LogUpdateMessage(const FString& Message, ELogVerbosity::Type Verbosity = ELogVerbosity::Error);

		// TODO: Use UTF8CHAR or ANSICHAR to save program space.
		template<typename... Types>
		UE_REWRITE void LogInfo(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
		{
			LogUpdateMessage(FString::Printf(Fmt, Args...), ELogVerbosity::Verbose);
		}

		template<typename... Types>
		UE_REWRITE void LogWarn(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
		{
			LogUpdateMessage(FString::Printf(Fmt, Args...), ELogVerbosity::Warning);
		}

		template<typename... Types>
		UE_REWRITE void LogError(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
		{
			LogUpdateMessage(FString::Printf(Fmt, Args...), ELogVerbosity::Error);
		}

	private:

		/** Pointer to the editor log list to print all the messages related to the instance update. */
		TSharedPtr<IMutableEditorLogger> InstanceLogger;
	};

	/** Data for an instance that is currently being processed in the mutable system. This means it is
	* between a BeginUpdate and EndUpdate, or during an "atomic" operation (like generate a single resource).
	*/
	struct FLiveInstance
	{
		int32 State = 0;
		uint32 LODMask = 0;
		TManagedPtr<const FInstance> Instance;
		TSharedPtr<FModel> Model;

		TSharedPtr<const FParameters> Parameters;
		
		/** Cached data for the generation of this instance. */
		TSharedRef<FProgramCache> Cache = MakeShared<FProgramCache>();

		TSharedPtr<FImageIdRegistry> ImageIdRegistry;
		TSharedPtr<FMaterialIdRegistry> MaterialIdRegistry;
		TSharedPtr<FSkeletalMeshIdRegistry> SkeletalMeshIdRegistry;
		
		TSharedPtr<FExternalResourceProvider> ExternalResourceProvider;
		TSharedPtr<FPassthroughObjectLoader> PassthroughObjectLoader;
		
		TSharedPtr<FLiveInstanceLogger> UpdateLogger;
		
		FImageOperator::FImagePixelFormatFunc PixelFormatOverride;
	};


    /** Struct to manage all the memory allocated for resources used during mutable operation. */
    struct FWorkingMemoryManager
    {	
		using MemoryCounter = MemoryCounters::FInternalMemoryCounter;

		template<class Type>
		using TMemoryTrackedArray = TArray<Type, FDefaultMemoryTrackingAllocator<MemoryCounter>>;

		template<class KeyType, class ValueType>
		using TMemoryTrackedMap = TMap<KeyType, ValueType, FDefaultMemoryTrackingSetAllocator<MemoryCounter>>;

		/** Maximum working memory that mutable should be using. */
		int64 BudgetBytes = 0;

		/** Maximum excess memory reached during the current operation. */
		int64 BudgetExcessBytes = 0;
    	
		/** Data for each mutable instance that is being updated. */
		mutable FMutex LiveInstancesMutex;
		TMemoryTrackedArray<TWeakPtr<FLiveInstance>> LiveInstances;

        /** Make sure the working memory is below the internal budget, even counting with the passed additional memory. 
		* Return true if it succeeded, false otherwise.
		*/
		bool EnsureBudgetBelow(int64 AdditionalMemory, const TSharedPtr<FLiveInstance>& InCurrentLiveInstance);
		
		/** Return true if the memory budget is 90% full. */
		bool IsMemoryBudgetFull() const;

		/** Calculate the current usage of memory as used to calculate the budget. */
		int64 GetCurrentMemoryBytes() const;

		int32 GetRomBytes() const;

    	/** Calculate the amount of bytes in data cached in the level 0 and 1 cache in all live instances. */
    	int32 GetCacheBytes() const
    	{
    		int32 Result = 0;
    		TSet<FProgramCache*> VisitedCaches;

			TScopeLock<FMutex> LockGuard(LiveInstancesMutex);

    		for (const TWeakPtr<FLiveInstance>& WeakInstance : LiveInstances)
    		{
    			TSharedPtr<FLiveInstance> Instance = WeakInstance.Pin();
    			if (!Instance)
    			{
    				continue;
    			}

				TSharedPtr<FProgramCache> CurrentCache = Instance->Cache;

				if (VisitedCaches.Contains(CurrentCache.Get()))
				{
					continue;
				}
	
				VisitedCaches.Add(CurrentCache.Get());

				Result += CurrentCache->CountBytes();
    		}

			return Result;
    	}

		void LogWorkingMemory(const class CodeRunner* CurrentRunner) const;
	};
	
	
	struct FGetImageResult
	{
		TManagedPtr<const FImage> MutableImage;
		
		bool bWasOperationSuccessful = true;
	};
	
	struct FGetMaterialResult
	{
		TManagedPtr<const FMaterial> MutableMaterial;
		
		bool bWasOperationSuccessful = true;
	};
	
	struct FGetSkeletalMeshResult
	{
		TManagedPtr<const FSkeletalMesh> MutableSkeletalMesh;
		
		bool bWasOperationSuccessful = true;
	};
		
	
    /** Main system class to load models and build instances. */
	class FSystem
	{
    public:
        //! This constant can be used in place of the lodMask in methods like BeginUpdate
        static constexpr uint32 AllLODs = 0xffffffff;

		//! Constructor of a system object to build data.
        //! \param Settings Optional class with the settings to use in this system. The default
        //! value configures a production-ready system.
		UE_API FSystem( const FSettings& Settings = FSettings());

        //! Set a new provider for model data. 
        UE_API void SetStreamingInterface(const TSharedPtr<FModelReader>& );

        /** Set the working memory limit, overrding any set in the settings when the system was created.
         * Refer to Settings::SetWorkingMemoryBudget for more information.
		 */
        UE_API void SetWorkingMemoryBytes( uint64 Bytes );

        //! Create a new instance from the given model. The instance can then be configured through
        //! calls to BeginUpdate/EndUpdate.
        //! A call to NewInstance must be paired to a call to ReleasesInstance when the instance is
        //! no longer needed.
        UE_API TSharedRef<FLiveInstance> NewInstance(const TSharedPtr<FModel>& Model,
			const TSharedPtr<UE::Mutable::Private::FImageIdRegistry>& InImageIdRegistry,
			const TSharedPtr<UE::Mutable::Private::FMaterialIdRegistry>& InMaterialIdRegistry,
			const TSharedPtr<UE::Mutable::Private::FSkeletalMeshIdRegistry>& InSkeletalMeshIdRegistry,
        	const TSharedPtr<FExternalResourceProvider>& ExternalResourceProvider,
        	const TSharedPtr<FPassthroughObjectLoader>& PassthroughObjectLoader,
        	const FImageOperator::FImagePixelFormatFunc& PixelFormatOverride,
			const TSharedPtr<FLiveInstanceLogger>& LiveInstanceLogger);

		/** 
		 * Create a ready to use new FLiveInstance to be used during the mutable build operations. It will not be taken into consideration for any update operation
		 * as the life cycle of the returned object is expected to be short and contained during the compilation operation.
		 * It does not require further initialization as it is fully capable of operating as is.
		 */
		UE_API TSharedRef<FLiveInstance> NewBuildInstance(const TSharedPtr<FModel>& InModel, 
			const TSharedPtr<const FParameters>& InParameters,
			const TSharedPtr<FExternalResourceProvider>& ExternalResourceProvider);
				
        //! \brief Update an instance with a new parameter set and/or state.
        //!
        //! \warning a call to BeginUpdate must be paired with a call to EndUpdate once the returned
        //! data has been processed.
        //! \param LiveInstance The instance to update, as created by a NewInstance call.
        //! \param InParams The parameters that customize this instance.
        //! \param StateIndex The index of the state this instance will be set to. The states range
        //! from 0 to Model::GetStateCount-1
        //! \param LodMask Bitmask selecting the levels of detail to build (i-th bit selects i-th lod).
        //! \return the instance data with all the LOD, components, and ids to generate the meshes 
        //! and images. The returned Instance is only valid until the next call to EndUpdate with 
        //! the same instanceID parameter.
		//!
		//! Very fast. Can be called in the Game Thread.
		UE_API void BeginUpdate_GameThread(const TSharedRef<FLiveInstance>& LiveInstance, uint32 LodMask);

		/** Potentially expensive. Must be called in the Mutable Thread. */
		UE_API TManagedPtr<const FInstance> BeginUpdate_MutableThread(const TSharedRef<FLiveInstance>& LiveInstance, bool bClearCacheLayer1);
		
		//! Only valid between BeginUpdate and EndUpdate
		//! Calculate the description of an image, without generating it.
		UE_API UE::Tasks::TTask<FExtendedImageDesc> GetImageDesc(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId);

		//! Only valid between BeginUpdate and EndUpdate
		//! \param MipsToSkip Number of mips to skip compared from the full image.
		//! If 0, all mip levels will be generated. If more levels than possible to discard are specified, 
		//! the image will still contain a minimum number of mips specified at model compile time.
		UE_API UE::Tasks::TTask<FGetImageResult> GetImage(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId, int32 MipsToSkip = 0);

		UE_API UE::Tasks::TTask<FGetSkeletalMeshResult> GetSkeletalMesh(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FSkeletalMeshId& SkeletalMeshId, uint16 ExecutionOptions);

		//! Only valid between BeginUpdate and EndUpdate
		//! Calculate the description of an image, without generating it.
		UE_API FExtendedImageDesc GetImageDescInline(const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId);

		//! Only valid between BeginUpdate and EndUpdate
		//! \param MipsToSkip Number of mips to skip compared from the full image.
		//! If 0, all mip levels will be generated. If more levels than possible to discard are specified, 
		//! the image will still contain a minimum number of mips specified at model compile time.
		UE_API TManagedPtr<const FImage> GetImageInline(const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId, int32 MipsToSkip = 0);
		
		UE_API UE::Tasks::TTask<FGetMaterialResult> GetMaterial(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FMaterialId& MaterialId);
		
        //! Invalidate and free the last Instance data returned by a call to BeginUpdate with
        //! the same instance index. After a call to this method, that Instance cannot be used any
        //! more and its content is undefined.
        //! \param instance The index of the instance whose last data will be invalidated.
        UE_API void EndUpdate(const TSharedRef<FLiveInstance>& LiveInstance, bool bClearRoms, bool bFreeCache);
		
		//! Calculate the relevancy of every parameter. Some parameters may be unused depending on
		//! the values of other parameters. This is useful to hide irrelevant parameters in
		//! dynamic user interfaces.
        UE_API void GetParameterRelevancy(const TSharedRef<FLiveInstance>& LiveInstance, TArray<uint32>& OutRelevantParameters);
		
		UE_API static FImageId GetImageId(const TSharedPtr<FImageIdRegistry>& Registry, const FModel& Model, const FParameters& Params, FOperation::ADDRESS RootAt);

		UE_API static FMaterialId GetMaterialId(const TSharedPtr<FMaterialIdRegistry>& Registry, const FModel& Model, const FParameters& Params, FOperation::ADDRESS RootAt);

		UE_API static FSkeletalMeshId GetSkeletalMeshId(const TSharedPtr<FSkeletalMeshIdRegistry>& Registry, const FModel& Model, const FParameters& Params, FOperation::ADDRESS RootAt);
		
        // Prevent copy, move and assignment.
        FSystem( const FSystem& ) = delete;
		FSystem& operator=( const FSystem& ) = delete;
        FSystem( FSystem&& ) = delete;
        FSystem& operator=( FSystem&& ) = delete;
		
		MUTABLERUNTIME_API bool BuildBool(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		MUTABLERUNTIME_API int32 BuildInt(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		MUTABLERUNTIME_API float BuildScalar(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		MUTABLERUNTIME_API FVector4f BuildColor(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		MUTABLERUNTIME_API TManagedPtr<const String> BuildString(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		MUTABLERUNTIME_API TManagedPtr<const FImage> BuildImage(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress, int32 MipsToSkip);
		MUTABLERUNTIME_API TManagedPtr<const FMesh> BuildMesh(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InRootAddress, EMeshContentFlags MeshContentFilter);
		MUTABLERUNTIME_API TManagedPtr<const FInstance> BuildInstance(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress);
		MUTABLERUNTIME_API TManagedPtr<const FLayout> BuildLayout(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		MUTABLERUNTIME_API FProjector BuildProjector(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress) ;
		
		
        FSettings Settings;

        //! Data streaming interface, if any.
		TSharedPtr<FModelReader> StreamInterface;
		
		FWorkingMemoryManager WorkingMemoryManager;

		/** Increased each time a rom is loaded. Used to keep track of which roms to evict. */
		static std::atomic<uint32> RomTick;
		
		MUTABLERUNTIME_API static bool CheckForUpdatedParameters(const TSharedRef<FLiveInstance>& LiveInstance, const TSharedPtr<const FParameters>& InOldParameters);
		
		/** Runs the mutable code 
		 * @return true if the execution was successful and false otherwise
		 */
		bool RunCodeInline(
			const TSharedRef<FLiveInstance>& InLiveInstance,
			FOperation::ADDRESS,
			uint16 ExecutionOptions = 0);

		//! Update some mutable core unreal stats.
		void UpdateStats();
	};
}

#undef UE_API
