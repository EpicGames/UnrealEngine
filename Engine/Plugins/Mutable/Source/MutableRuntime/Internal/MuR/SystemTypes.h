// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Operations.h"

#ifndef MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK
	#define MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK 0	
#endif

namespace UE::Mutable::Private
{
	/** This structure stores the data about an ongoing mutable operation that needs to be executed. */
	struct FScheduledOp
	{
		static constexpr int32 NumStageBits = 6;
		static constexpr int32 MaxNumStages = (1 << NumStageBits) - 1;

		inline FScheduledOp() = default;

		inline FScheduledOp(FOperation::ADDRESS InAt, const FScheduledOp& InOpTemplate, uint8 InStage = 0, uint32 InCustomState = 0)
			: At(InAt)
			, CustomState(InCustomState)
			, ExecutionIndex(InOpTemplate.ExecutionIndex)
			, ExecutionOptions(InOpTemplate.ExecutionOptions)
			, Type(InOpTemplate.Type)
			, Stage(InStage)
		{
			check(InStage <= MaxNumStages);
		}

		static inline FScheduledOp FromOpAndOptions(FOperation::ADDRESS InAt, const FScheduledOp& InOpTemplate, uint16 InExecutionOptions)
		{
			FScheduledOp Result;

			Result.At = InAt;
			Result.ExecutionOptions = InExecutionOptions;
			Result.ExecutionIndex = InOpTemplate.ExecutionIndex;
			Result.Stage = 0;
			Result.CustomState = InOpTemplate.CustomState;
			Result.Type = InOpTemplate.Type;

			return Result;
		}

		//! Address of the operation
		FOperation::ADDRESS At = 0;

		//! Additional custom state data that the operation can store. This is usually used to pass information
		//! between execution stages of an operation.
		uint32 CustomState = 0;

		//! Index of the operation execution: This is used for iteration of different ranges.
		//! It is an index into the CodeRunner::GetMemory()::m_rangeIndex vector.
		//! executionIndex 0 is always used for empty ExecutionIndex, which is the most common
		//! one.
		uint16 ExecutionIndex = 0;

		//! Additional execution options. Set externally to this op, it usually alters the result.
		//! For example, this is used to keep track of the mipmaps to skip in image operations.
		uint16 ExecutionOptions : 14 = 0;
		
		//! Type of calculation we are requesting for this operation.
		enum class EType : uint8
		{
			//! Execute the operation to calculate the full result
			Full,

			//! Execute the operation to obtain the descriptor of an image or mesh.
			ImageDesc,
		};
		uint16 Type       : 2 = static_cast<uint16>(EType::Full);

		//! Internal stage of the operation.
		//! Stage 0 is usually scheduling of children, and 1 is execution. Some instructions
		//! may have more steges to schedule children that are optional for execution, etc.
		uint8 Stage      : NumStageBits = 0;

#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK
		static constexpr uint64 CallstackMaxDepth = 16;
		uint64 StackDepth = 0; 
		uint64 ScheduleCallstack[CallstackMaxDepth];
#endif
	};
	
	inline uint32 GetTypeHash(const FScheduledOp& Op)
	{
		return HashCombineFast(::GetTypeHash(Op.At), HashCombineFast((uint32)Op.Stage), (uint32)Op.ExecutionIndex);
	}
} // namespace UE::Mutable::Private
