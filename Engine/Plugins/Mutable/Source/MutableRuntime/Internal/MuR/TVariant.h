// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariant.h"
#include "Templates/Decay.h"


namespace UE::Mutable::Private
{
	// TVariant currently does not support this operator. Once supported remove it.
	template<typename ...T>
	bool operator==(const TVariant<T...>& ValueA, const TVariant<T...>& ValueB)
	{
		const int32 IndexA = ValueA.GetIndex();
		const int32 IndexB = ValueB.GetIndex();

		if (IndexA != IndexB)
		{
			return false;
		}

		return Visit([&ValueB](const auto& StoredValueA)
		{
			using Type = typename TDecay<decltype(StoredValueA)>::Type;
			return StoredValueA == ValueB.template Get<Type>();
		}, ValueA);
	}
}