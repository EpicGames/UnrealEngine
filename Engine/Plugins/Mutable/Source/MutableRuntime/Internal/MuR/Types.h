// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "StructUtils/InstancedStruct.h"
#include "Curves/RichCurve.h"
#include "UObject/PerPlatformProperties.h"
#include "PerQualityLevelProperties.h"

#include "Types.generated.h"

/** This type is used in some mesh clip operations. 
* Warning: this type is used compiled COs. Any relevant change to the option order requires and update in the CustomizableObjectPrivate::ECustomizableObjectVersions
*/
UENUM()
enum class EFaceCullStrategy : uint8
{
	AllVerticesCulled = 0 UMETA(DisplayName = "Remove face if all vertices removed"),
	OneVertexCulled = 1 UMETA(DisplayName = "Remove face if one vertex removed"),
};


/** This type is used in some mesh clip operations.
* Warning: this type is used compiled COs. Any relevant change to the option order requires and update in the CustomizableObjectPrivate::ECustomizableObjectVersions
*/
enum class EClipVertexSelectionType : uint8
{
	None = 0,
	Shape = 1,
	BoneHierarchy = 2,
};


namespace UE::Mutable::Private
{
	class FOutputArchive;
	class FInputArchive;

	
	enum class EMemoryInitPolicy
	{
		Uninitialized,
		Zeroed,
	};
	
	
	inline uint32 GetTypeHash(const FInstancedStruct& InstancedStruct)
	{
		if (!InstancedStruct.IsValid())
		{
			return 0;
		}

		const UScriptStruct* ScriptedStruct = InstancedStruct.GetScriptStruct();
		const uint32 ValueHash = ScriptedStruct->GetStructTypeHash(InstancedStruct.GetMemory());

		return HashCombine(ValueHash, PointerHash(ScriptedStruct));
	}
	
	
	inline uint32 GetTypeHash(const float& Key)
	{
		return static_cast<uint32>(Key);
	}
	
	
	inline uint32 GetTypeHash(const FRichCurveKey& Key)
	{
		uint32 Result = 0;
		
		Result = HashCombineFast(Result, GetTypeHash(Key.InterpMode));
		Result = HashCombineFast(Result, GetTypeHash(Key.TangentMode));
		Result = HashCombineFast(Result, GetTypeHash(Key.TangentWeightMode));
		Result = HashCombineFast(Result, GetTypeHash(Key.Time));
		Result = HashCombineFast(Result, GetTypeHash(Key.Value));
		Result = HashCombineFast(Result, GetTypeHash(Key.ArriveTangent));
		Result = HashCombineFast(Result, GetTypeHash(Key.ArriveTangentWeight));
		Result = HashCombineFast(Result, GetTypeHash(Key.LeaveTangent));
		Result = HashCombineFast(Result, GetTypeHash(Key.LeaveTangentWeight));
		 	
		return Result;
	}
	
	
	template<typename T>
	uint32 GetTypeHash(const TArray<T>& Key)
	{
		uint32 Hash = 0;
		
		for (const T& Element : Key)
		{
			Hash = HashCombine(Hash, GetTypeHash(Element));
		}
		
		return Hash;
	}
	
	
	template<typename K, typename V>
	bool operator==(const TMap<K,V>& A, const TMap<K,V>& B )
	{	
		return A.OrderIndependentCompareEqual(B);
	}
	
	
	inline uint32 GetTypeHash(const FRichCurve& Curve)
	{
		return GetTypeHash(Curve.Keys);
	}
	
	
	inline uint32 GetTypeHash(const FPerPlatformInt& Key)
	{
		uint32 Result = 0;
		
		Result = HashCombineFast(Result, GetTypeHash(Key.Default));
		//Result = HashCombineFast(Result, GetTypeHash(Key.PerPlatform));
		 	
		return Result;
	}
	
	
	inline uint32 GetTypeHash(const FPerQualityLevelInt& Key)
	{
		uint32 Result = 0;
		
		Result = HashCombineFast(Result, GetTypeHash(Key.Default));
		//Result = HashCombineFast(Result, GetTypeHash(Key.PerQuality));
		 	
		return Result;
	}


	inline bool operator==(const FPerPlatformInt& A, const FPerPlatformInt& B)
	{
		return A.Default == B.Default &&
#if WITH_EDITORONLY_DATA		
			A.PerPlatform.OrderIndependentCompareEqual(B.PerPlatform);
#else
			true;
#endif
	}
	
	
	inline bool operator==(const FPerQualityLevelInt& A, const FPerQualityLevelInt& B)
	{
		return A.Default == B.Default && A.PerQuality.OrderIndependentCompareEqual(B.PerQuality);
	}
}
