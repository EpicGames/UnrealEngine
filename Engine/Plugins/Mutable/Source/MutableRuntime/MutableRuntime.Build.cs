// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class MutableRuntime : ModuleRules
    {
        public MutableRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "MuR";

			DefaultBuildSettings = BuildSettingsVersion.Latest;
			IWYUSupport = IWYUSupport.KeepAsIsForNow;
			//bUseUnity = false;

			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange(
				new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"ImageCore",
				"ClothingSystemRuntimeCommon",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"GeometryCore",
				"TraceLog", 
				}
			);
		}
	}
}
