<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
	<!-- Copyright Epic Games, Inc. All Rights Reserved. -->

	<Type Name="UE::Mutable::Private::FProgramCache">
		<Expand>
			<Item Name="ProgramCacheInternals">(UE::Mutable::Private::ProgramCacheInternals::FProgramCacheInternals*)(InternalsStorage.Data)</Item>
		</Expand>
	</Type>
	<!--
	<Type Name="UE::Mutable::Private::FCacheAddress">
		<Expand>
			<ExpandedItem>this</ExpandedItem>
			<Item Name="OpType">(EOpType(At >> 24))</Item> 
			<Item Name="ByteCodeOffset">(At & 0xFFFFF)</Item> 
			<Item Name="MangledAddress">((At >> 2) & 0xFFFF))</Item> 
		</Expand>
	</Type>
	-->
</AutoVisualizer>
