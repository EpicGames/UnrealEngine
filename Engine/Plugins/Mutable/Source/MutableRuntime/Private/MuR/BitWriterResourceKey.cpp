// Copyright Epic Games, Inc. All Rights Reserved.

#include "BitWriterResourceKey.h"

#include "UObject/ObjectPtr.h"


FBitWriterResourceKey::FBitWriterResourceKey(int64 InMaxBits, bool AllowResize): FBitWriter(InMaxBits, AllowResize)
{
}


FArchive& FBitWriterResourceKey::operator<<(FObjectPtr& Value)
{
	Serialize(Value.Get(), sizeof(UObject*));
	return *this;
}
