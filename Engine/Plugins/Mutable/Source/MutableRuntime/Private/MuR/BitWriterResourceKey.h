// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/BitWriter.h"


class FBitWriterResourceKey : public FBitWriter
{
public:
	FBitWriterResourceKey(int64 InMaxBits, bool AllowResize = false);

	virtual FArchive& operator<<(FObjectPtr& Value) override;
};
