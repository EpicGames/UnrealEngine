// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Prepare Miro configuration definitions
#if defined(MUTABLE_EXCLUDE_BC)
#ifdef MIRO_INCLUDE_BC
#undef MIRO_INCLUDE_BC
#define MIRO_INCLUDE_BC 0
#endif
#endif

#if defined(MUTABLE_EXCLUDE_ASTC)
#ifdef MIRO_INCLUDE_ASTC
#undef MIRO_INCLUDE_ASTC
#define MIRO_INCLUDE_ASTC 0
#endif
#endif

#ifdef MUTABLE_EXCLUDE_IMAGE_COMPRESSION_QUALITY
#ifdef MIRO_EXCLUDE_QUALITY_ABOVE
#undef MIRO_EXCLUDE_QUALITY_ABOVE
#define MIRO_EXCLUDE_QUALITY_ABOVE MUTABLE_EXCLUDE_IMAGE_COMPRESSION_QUALITY
#endif
#endif

#include "MuR/BlockCompression/Miro/Miro.h"
