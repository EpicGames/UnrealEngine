// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/BlockCompression/Miro/Miro.h"
#include "MuR/Platform.h"

#include "Async/ParallelFor.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"

//#define UE_MIRO_DEBUG

#ifdef UE_MIRO_DEBUG
UE_DISABLE_OPTIMIZATION

static int32 s_DebugBlock = -1;
static int32 s_CurrentBlock = -1;

inline constexpr void miro_check(bool x)
{
	check(x);
}

#else

inline constexpr void miro_check(bool x)
{
	(void)x;
}

#endif //UE_MIRO_DEBUG


struct alignas(16) FMiroPixelBlock4x4_4Bytes
{
	uint8 Bytes[4 * 4 * 4];
};

struct alignas(16) FMiroPixelBlock4x4_1Bytes
{
	uint8 Bytes[4 * 4];
};

namespace impl
{

#if MIRO_INCLUDE_BC
	static uint8 s_initialised_dxtc_decompress = 0;
#endif

#if MIRO_INCLUDE_ASTC
	static uint8 s_initialised_astc_decompress = 0;
#endif


#if MIRO_INCLUDE_BC

	//-------------------------------------------------------------------------------------------------
	// Decide if a BC1 compressed block has alpha or not.
	//-------------------------------------------------------------------------------------------------
	inline bool BC1BlockHasAlpha(uint8 const* bytes)
	{
		int32 a = (int32(bytes[0]) << 0) + (int32(bytes[1]) << 8);
		int32 b = (int32(bytes[2]) << 0) + (int32(bytes[3]) << 8);
		return a <= b;
	}


	//-------------------------------------------------------------------------------------------------
	//! STB precomputed tables.
	//-------------------------------------------------------------------------------------------------
	static unsigned char stb__Expand5[32];
	static unsigned char stb__Expand6[64];
	static unsigned char stb__OMatch5[256][2];
	static unsigned char stb__OMatch6[256][2];
	static unsigned char stb__QuantRBTab[256 + 16];
	static unsigned char stb__QuantGTab[256 + 16];


	//-------------------------------------------------------------------------------------------------
	//! Unpack a 16 bit color and also return it as int
	//-------------------------------------------------------------------------------------------------
	inline int Unpack565(uint8 const* packed, uint8* color)
	{
		// build the packed value
		int value = ((int)packed[0] << 0) + ((int)packed[1] << 8);

		// get the components in the stored range
		uint8 red = (uint8)((value >> 11) & 0x1F);
		uint8 green = (uint8)((value >> 5) & 0x3F);
		uint8 blue = (uint8)(value & 0x1F);

		// scale up to 8 bits
		color[0] = stb__Expand5[red];
		color[1] = stb__Expand6[green];
		color[2] = stb__Expand5[blue];
		color[3] = 255;

		// return the value
		return value;
	}


	inline void Codebook3(uint8(&codes)[4 * 4])
	{
		// generate the midpoints
		for (int i = 0; i < 3; ++i)
		{
			const int c = codes[0 + i];
			const int d = codes[4 + i];

			codes[8 + i] = (uint8)((1 * c + 1 * d) >> 1);
			codes[12 + i] = 0;
		}

		// fill in alpha for the intermediate values
		codes[8 + 3] = 255;
		codes[12 + 3] = 0;
	}


	inline void Codebook4(uint8(&codes)[4 * 4])
	{
		// generate the midpoints
		for (int i = 0; i < 3; ++i)
		{
			const int c = codes[0 + i];
			const int d = codes[4 + i];

			codes[8 + i] = (uint8)((2 * c + 1 * d) / 3);
			codes[12 + i] = (uint8)((1 * c + 2 * d) / 3);
		}

		// fill in alpha for the intermediate values
		codes[8 + 3] = 255;
		codes[12 + 3] = 255;
	}


	inline void Codebook6(uint8(&codes)[8 * 1])
	{
		// generate the midpoints
		for (int i = 0; i < 1; ++i)
		{
			const int c = codes[0 + i];
			const int d = codes[1 + i];

			codes[2 + i] = (uint8)((4 * c + 1 * d) / 5);
			codes[3 + i] = (uint8)((3 * c + 2 * d) / 5);
			codes[4 + i] = (uint8)((2 * c + 3 * d) / 5);
			codes[5 + i] = (uint8)((1 * c + 4 * d) / 5);

			codes[6 + i] = (uint8)0;
			codes[7 + i] = (uint8)255;
		}
	}


	inline void Codebook8(uint8(&codes)[8 * 1])
	{
		// generate the midpoints
		for (int i = 0; i < 1; ++i)
		{
			const int c = codes[0 + i];
			const int d = codes[1 + i];

			codes[2 + i] = (uint8)((6 * c + 1 * d) / 7);
			codes[3 + i] = (uint8)((5 * c + 2 * d) / 7);
			codes[4 + i] = (uint8)((4 * c + 3 * d) / 7);
			codes[5 + i] = (uint8)((3 * c + 4 * d) / 7);
			codes[6 + i] = (uint8)((2 * c + 5 * d) / 7);
			codes[7 + i] = (uint8)((1 * c + 6 * d) / 7);
		}
	}


	inline void Codebook6(uint16(&codes)[8 * 1])
	{
		// generate the midpoints
		for (int i = 0; i < 1; ++i)
		{
			const int c = codes[0 + i];
			const int d = codes[1 + i];

			codes[2 + i] = (uint16)((4 * c + 1 * d) / 5);
			codes[3 + i] = (uint16)((3 * c + 2 * d) / 5);
			codes[4 + i] = (uint16)((2 * c + 3 * d) / 5);
			codes[5 + i] = (uint16)((1 * c + 4 * d) / 5);

			codes[6 + i] = (uint16)0;
			codes[7 + i] = (uint16)255;
		}
	}


	inline void Codebook8(uint16(&codes)[8 * 1])
	{
		// generate the midpoints
		for (int i = 0; i < 1; ++i)
		{
			const int c = codes[0 + i];
			const int d = codes[1 + i];

			codes[2 + i] = (uint16)((6 * c + 1 * d) / 7);
			codes[3 + i] = (uint16)((5 * c + 2 * d) / 7);
			codes[4 + i] = (uint16)((4 * c + 3 * d) / 7);
			codes[5 + i] = (uint16)((3 * c + 4 * d) / 7);
			codes[6 + i] = (uint16)((2 * c + 5 * d) / 7);
			codes[7 + i] = (uint16)((1 * c + 6 * d) / 7);
		}
	}


	inline void CompressAlphaBtc2u(uint8 const* rgba, void* block)
	{
		uint8* bytes = reinterpret_cast<uint8*>(block);

		// quantize and pack the alpha values pairwise
		for (int i = 0; i < 8; ++i)
		{
			// quantize down to 4 bits
			float alpha1 = (float)rgba[8 * i + 3] * (15.0f / 255.0f);
			float alpha2 = (float)rgba[8 * i + 7] * (15.0f / 255.0f);

			int quant1 = (int)(alpha1 + 0.5f);
			int quant2 = (int)(alpha2 + 0.5f);

			// pack into the byte
			bytes[i] = (uint8)((quant1 << 0) + (quant2 << 4));
		}
	}



	//-------------------------------------------------------------------------------------------------
	// Based on BCDec, under the MIT license
	// See: https://github.com/iOrange/bcdec
	//-------------------------------------------------------------------------------------------------
	namespace bcdec
	{
#define BCDEC_STATIC 1
#define BCDEC_IMPLEMENTATION

		/* if BCDEC_STATIC causes problems, try defining BCDECDEF to 'inline' or 'static inline' */
#ifndef BCDECDEF
#ifdef BCDEC_STATIC
#define BCDECDEF    static
#else
#ifdef __cplusplus
#define BCDECDEF    extern "C"
#else
#define BCDECDEF    extern
#endif
#endif
#endif

#define BCDEC_BC1_BLOCK_SIZE    8
#define BCDEC_BC2_BLOCK_SIZE    16
#define BCDEC_BC3_BLOCK_SIZE    16
#define BCDEC_BC4_BLOCK_SIZE    8
#define BCDEC_BC5_BLOCK_SIZE    16
#define BCDEC_BC6H_BLOCK_SIZE   16
#define BCDEC_BC7_BLOCK_SIZE    16

#define BCDEC_BC1_COMPRESSED_SIZE(w, h)     ((((w)>>2)*((h)>>2))*BCDEC_BC1_BLOCK_SIZE)
#define BCDEC_BC2_COMPRESSED_SIZE(w, h)     ((((w)>>2)*((h)>>2))*BCDEC_BC2_BLOCK_SIZE)
#define BCDEC_BC3_COMPRESSED_SIZE(w, h)     ((((w)>>2)*((h)>>2))*BCDEC_BC3_BLOCK_SIZE)
#define BCDEC_BC4_COMPRESSED_SIZE(w, h)     ((((w)>>2)*((h)>>2))*BCDEC_BC4_BLOCK_SIZE)
#define BCDEC_BC5_COMPRESSED_SIZE(w, h)     ((((w)>>2)*((h)>>2))*BCDEC_BC5_BLOCK_SIZE)
#define BCDEC_BC6H_COMPRESSED_SIZE(w, h)    ((((w)>>2)*((h)>>2))*BCDEC_BC6H_BLOCK_SIZE)
#define BCDEC_BC7_COMPRESSED_SIZE(w, h)     ((((w)>>2)*((h)>>2))*BCDEC_BC7_BLOCK_SIZE)

		BCDECDEF void bcdec_bc1(const void* compressedBlock, void* decompressedBlock, int destinationPitch);
		BCDECDEF void bcdec_bc2(const void* compressedBlock, void* decompressedBlock, int destinationPitch);
		BCDECDEF void bcdec_bc3(const void* compressedBlock, void* decompressedBlock, int destinationPitch);
		BCDECDEF void bcdec_bc4(const void* compressedBlock, void* decompressedBlock, int destinationPitch);
		BCDECDEF void bcdec_bc5(const void* compressedBlock, void* decompressedBlock, int destinationPitch);
		BCDECDEF void bcdec_bc6h_float(const void* compressedBlock, void* decompressedBlock, int destinationPitch, int isSigned);
		BCDECDEF void bcdec_bc6h_half(const void* compressedBlock, void* decompressedBlock, int destinationPitch, int isSigned);
		BCDECDEF void bcdec_bc7(const void* compressedBlock, void* decompressedBlock, int destinationPitch);


#ifdef BCDEC_IMPLEMENTATION

		static void bcdec__color_block(const void* compressedBlock, void* decompressedBlock, int destinationPitch, int onlyOpaqueMode) {
			unsigned short c0, c1;
			unsigned int refColors[4]; /* 0xAABBGGRR */
			unsigned char* dstColors;
			unsigned int colorIndices;
			int i, j, idx;
			unsigned int r0, g0, b0, r1, g1, b1, r, g, b;

			c0 = ((unsigned short*)compressedBlock)[0];
			c1 = ((unsigned short*)compressedBlock)[1];

			/* Expand 565 ref colors to 888 */
			r0 = (((c0 >> 11) & 0x1F) * 527 + 23) >> 6;
			g0 = (((c0 >> 5) & 0x3F) * 259 + 33) >> 6;
			b0 = ((c0 & 0x1F) * 527 + 23) >> 6;
			refColors[0] = 0xFF000000 | (b0 << 16) | (g0 << 8) | r0;

			r1 = (((c1 >> 11) & 0x1F) * 527 + 23) >> 6;
			g1 = (((c1 >> 5) & 0x3F) * 259 + 33) >> 6;
			b1 = ((c1 & 0x1F) * 527 + 23) >> 6;
			refColors[1] = 0xFF000000 | (b1 << 16) | (g1 << 8) | r1;

			if (c0 > c1 || onlyOpaqueMode) {    /* Standard BC1 mode (also BC3 color block uses ONLY this mode) */
				/* color_2 = 2/3*color_0 + 1/3*color_1
				   color_3 = 1/3*color_0 + 2/3*color_1 */
				r = (2 * r0 + r1 + 1) / 3;
				g = (2 * g0 + g1 + 1) / 3;
				b = (2 * b0 + b1 + 1) / 3;
				refColors[2] = 0xFF000000 | (b << 16) | (g << 8) | r;

				r = (r0 + 2 * r1 + 1) / 3;
				g = (g0 + 2 * g1 + 1) / 3;
				b = (b0 + 2 * b1 + 1) / 3;
				refColors[3] = 0xFF000000 | (b << 16) | (g << 8) | r;
			}
			else {                            /* Quite rare BC1A mode */
				/* color_2 = 1/2*color_0 + 1/2*color_1;
				   color_3 = 0;                         */
				r = (r0 + r1 + 1) >> 1;
				g = (g0 + g1 + 1) >> 1;
				b = (b0 + b1 + 1) >> 1;
				refColors[2] = 0xFF000000 | (b << 16) | (g << 8) | r;

				refColors[3] = 0x00000000;
			}

			colorIndices = ((unsigned int*)compressedBlock)[1];

			/* Fill out the decompressed color block */
			dstColors = (unsigned char*)decompressedBlock;
			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					idx = colorIndices & 0x03;
					((unsigned int*)dstColors)[j] = refColors[idx];
					colorIndices >>= 2;
				}

				dstColors += destinationPitch;
			}
		}

		static void bcdec__sharp_alpha_block(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			unsigned short* alpha;
			unsigned char* decompressed;
			int i, j;

			alpha = (unsigned short*)compressedBlock;
			decompressed = (unsigned char*)decompressedBlock;

			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					decompressed[j * 4] = ((alpha[i] >> (4 * j)) & 0x0F) * 17;
				}

				decompressed += destinationPitch;
			}
		}

		static void bcdec__smooth_alpha_block(const void* compressedBlock, void* decompressedBlock, int destinationPitch, int pixelSize) {
			unsigned char* decompressed;
			unsigned char alpha[8];
			int i, j;
			unsigned long long block, indices;

			block = *(unsigned long long*)compressedBlock;
			decompressed = (unsigned char*)decompressedBlock;

			alpha[0] = block & 0xFF;
			alpha[1] = (block >> 8) & 0xFF;

			if (alpha[0] > alpha[1]) {
				/* 6 interpolated alpha values. */
				alpha[2] = (6 * alpha[0] + alpha[1] + 1) / 7;   /* 6/7*alpha_0 + 1/7*alpha_1 */
				alpha[3] = (5 * alpha[0] + 2 * alpha[1] + 1) / 7;   /* 5/7*alpha_0 + 2/7*alpha_1 */
				alpha[4] = (4 * alpha[0] + 3 * alpha[1] + 1) / 7;   /* 4/7*alpha_0 + 3/7*alpha_1 */
				alpha[5] = (3 * alpha[0] + 4 * alpha[1] + 1) / 7;   /* 3/7*alpha_0 + 4/7*alpha_1 */
				alpha[6] = (2 * alpha[0] + 5 * alpha[1] + 1) / 7;   /* 2/7*alpha_0 + 5/7*alpha_1 */
				alpha[7] = (alpha[0] + 6 * alpha[1] + 1) / 7;   /* 1/7*alpha_0 + 6/7*alpha_1 */
			}
			else {
				/* 4 interpolated alpha values. */
				alpha[2] = (4 * alpha[0] + alpha[1] + 1) / 5;   /* 4/5*alpha_0 + 1/5*alpha_1 */
				alpha[3] = (3 * alpha[0] + 2 * alpha[1] + 1) / 5;   /* 3/5*alpha_0 + 2/5*alpha_1 */
				alpha[4] = (2 * alpha[0] + 3 * alpha[1] + 1) / 5;   /* 2/5*alpha_0 + 3/5*alpha_1 */
				alpha[5] = (alpha[0] + 4 * alpha[1] + 1) / 5;   /* 1/5*alpha_0 + 4/5*alpha_1 */
				alpha[6] = 0x00;
				alpha[7] = 0xFF;
			}

			indices = block >> 16;
			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					decompressed[j * pixelSize] = alpha[indices & 0x07];
					indices >>= 3;
				}

				decompressed += destinationPitch;
			}
		}

		typedef struct bcdec__bitstream {
			unsigned long long low;
			unsigned long long high;
		} bcdec__bitstream_t;

		static int bcdec__bitstream_read_bits(bcdec__bitstream_t* bstream, int numBits) {
			unsigned int mask = (1 << numBits) - 1;
			/* Read the low N bits */
			unsigned int bits = (bstream->low & mask);

			bstream->low >>= numBits;
			/* Put the low N bits of "high" into the high 64-N bits of "low". */
			bstream->low |= (bstream->high & mask) << (sizeof(bstream->high) * 8 - numBits);
			bstream->high >>= numBits;

			return bits;
		}

		static int bcdec__bitstream_read_bit(bcdec__bitstream_t* bstream) {
			return bcdec__bitstream_read_bits(bstream, 1);
		}

		/*  reversed bits pulling, used in BC6H decoding
			why ?? just why ??? */
		static int bcdec__bitstream_read_bits_r(bcdec__bitstream_t* bstream, int numBits) {
			int bits = bcdec__bitstream_read_bits(bstream, numBits);
			/* Reverse the bits. */
			int result = 0;
			while (numBits--) {
				result <<= 1;
				result |= (bits & 1);
				bits >>= 1;
			}
			return result;
		}



		BCDECDEF void bcdec_bc1(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			bcdec__color_block(compressedBlock, decompressedBlock, destinationPitch, 0);
		}

		BCDECDEF void bcdec_bc2(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			bcdec__color_block(((char*)compressedBlock) + 8, decompressedBlock, destinationPitch, 1);
			bcdec__sharp_alpha_block(compressedBlock, ((char*)decompressedBlock) + 3, destinationPitch);
		}

		BCDECDEF void bcdec_bc3(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			bcdec__color_block(((char*)compressedBlock) + 8, decompressedBlock, destinationPitch, 1);
			bcdec__smooth_alpha_block(compressedBlock, ((char*)decompressedBlock) + 3, destinationPitch, 4);
		}

		BCDECDEF void bcdec_bc4(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			bcdec__smooth_alpha_block(compressedBlock, decompressedBlock, destinationPitch, 1);
		}

		BCDECDEF void bcdec_bc5(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			bcdec__smooth_alpha_block(compressedBlock, decompressedBlock, destinationPitch, 2);
			bcdec__smooth_alpha_block(((char*)compressedBlock) + 8, ((char*)decompressedBlock) + 1, destinationPitch, 2);
		}

		/* http://graphics.stanford.edu/~seander/bithacks.html#VariableSignExtend */
		static int bcdec__extend_sign(int val, int bits) {
			return (val << (32 - bits)) >> (32 - bits);
		}

		static int bcdec__transform_inverse(int val, int a0, int bits, int isSigned) {
			/* If the precision of A0 is "p" bits, then the transform algorithm is:
			   B0 = (B0 + A0) & ((1 << p) - 1) */
			val = (val + a0) & ((1 << bits) - 1);
			if (isSigned) {
				val = bcdec__extend_sign(val, bits);
			}
			return val;
		}

		/* pretty much copy-paste from documentation */
		static int bcdec__unquantize(int val, int bits, int isSigned) {
			int unq, s = 0;

			if (!isSigned) {
				if (bits >= 15) {
					unq = val;
				}
				else if (!val) {
					unq = 0;
				}
				else if (val == ((1 << bits) - 1)) {
					unq = 0xFFFF;
				}
				else {
					unq = ((val << 16) + 0x8000) >> bits;
				}
			}
			else {
				if (bits >= 16) {
					unq = val;
				}
				else {
					if (val < 0) {
						s = 1;
						val = -val;
					}

					if (val == 0) {
						unq = 0;
					}
					else if (val >= ((1 << (bits - 1)) - 1)) {
						unq = 0x7FFF;
					}
					else {
						unq = ((val << 15) + 0x4000) >> (bits - 1);
					}

					if (s) {
						unq = -unq;
					}
				}
			}
			return unq;
		}

		static int bcdec__interpolate(int a, int b, int* weights, int index) {
			return (a * (64 - weights[index]) + b * weights[index] + 32) >> 6;
		}

		static unsigned short bcdec__finish_unquantize(int val, int isSigned) {
			int s;

			if (!isSigned) {
				return (unsigned short)((val * 31) >> 6);                   /* scale the magnitude by 31 / 64 */
			}
			else {
				val = (val < 0) ? -(((-val) * 31) >> 5) : (val * 31) >> 5;  /* scale the magnitude by 31 / 32 */
				s = 0;
				if (val < 0) {
					s = 0x8000;
					val = -val;
				}
				return (unsigned short)(s | val);
			}
		}

		/* modified half_to_float_fast4 from https://gist.github.com/rygorous/2144712 */
		static float bcdec__half_to_float_quick(unsigned short half) {
			typedef union {
				unsigned int u;
				float f;
			} FP32;

			static const FP32 magic = { 113 << 23 };
			static const unsigned int shifted_exp = 0x7c00 << 13;   /* exponent mask after shift */
			FP32 o;
			unsigned int exp;

			o.u = (half & 0x7fff) << 13;                            /* exponent/mantissa bits */
			exp = shifted_exp & o.u;                                /* just the exponent */
			o.u += (127 - 15) << 23;                                /* exponent adjust */

			/* handle exponent special cases */
			if (exp == shifted_exp) {                               /* Inf/NaN? */
				o.u += (128 - 16) << 23;                            /* extra exp adjust */
			}
			else if (exp == 0) {                                  /* Zero/Denormal? */
				o.u += 1 << 23;                                     /* extra exp adjust */
				o.f -= magic.f;                                     /* renormalize */
			}

			o.u |= (half & 0x8000) << 16;                           /* sign bit */
			return o.f;
		}

		BCDECDEF void bcdec_bc6h_half(const void* compressedBlock, void* decompressedBlock, int destinationPitch, int isSigned) {
			static char actual_bits_count[4][14] = {
				{ 10, 7, 11, 11, 11, 9, 8, 8, 8, 6, 10, 11, 12, 16 },   /*  W */
				{  5, 6,  5,  4,  4, 5, 6, 5, 5, 6, 10,  9,  8,  4 },   /* dR */
				{  5, 6,  4,  5,  4, 5, 5, 6, 5, 6, 10,  9,  8,  4 },   /* dG */
				{  5, 6,  4,  4,  5, 5, 5, 5, 6, 6, 10,  9,  8,  4 }    /* dB */
			};

			/* There are 32 possible partition sets for a two-region tile.
			   Each 4x4 block represents a single shape.
			   Here also every fix-up index has MSB bit set. */
			static unsigned char partition_sets[32][4][4] = {
				{ {128, 0,   1, 1}, {0, 0, 1, 1}, {  0, 0, 1, 1}, {0, 0, 1, 129} },   /*  0 */
				{ {128, 0,   0, 1}, {0, 0, 0, 1}, {  0, 0, 0, 1}, {0, 0, 0, 129} },   /*  1 */
				{ {128, 1,   1, 1}, {0, 1, 1, 1}, {  0, 1, 1, 1}, {0, 1, 1, 129} },   /*  2 */
				{ {128, 0,   0, 1}, {0, 0, 1, 1}, {  0, 0, 1, 1}, {0, 1, 1, 129} },   /*  3 */
				{ {128, 0,   0, 0}, {0, 0, 0, 1}, {  0, 0, 0, 1}, {0, 0, 1, 129} },   /*  4 */
				{ {128, 0,   1, 1}, {0, 1, 1, 1}, {  0, 1, 1, 1}, {1, 1, 1, 129} },   /*  5 */
				{ {128, 0,   0, 1}, {0, 0, 1, 1}, {  0, 1, 1, 1}, {1, 1, 1, 129} },   /*  6 */
				{ {128, 0,   0, 0}, {0, 0, 0, 1}, {  0, 0, 1, 1}, {0, 1, 1, 129} },   /*  7 */
				{ {128, 0,   0, 0}, {0, 0, 0, 0}, {  0, 0, 0, 1}, {0, 0, 1, 129} },   /*  8 */
				{ {128, 0,   1, 1}, {0, 1, 1, 1}, {  1, 1, 1, 1}, {1, 1, 1, 129} },   /*  9 */
				{ {128, 0,   0, 0}, {0, 0, 0, 1}, {  0, 1, 1, 1}, {1, 1, 1, 129} },   /* 10 */
				{ {128, 0,   0, 0}, {0, 0, 0, 0}, {  0, 0, 0, 1}, {0, 1, 1, 129} },   /* 11 */
				{ {128, 0,   0, 1}, {0, 1, 1, 1}, {  1, 1, 1, 1}, {1, 1, 1, 129} },   /* 12 */
				{ {128, 0,   0, 0}, {0, 0, 0, 0}, {  1, 1, 1, 1}, {1, 1, 1, 129} },   /* 13 */
				{ {128, 0,   0, 0}, {1, 1, 1, 1}, {  1, 1, 1, 1}, {1, 1, 1, 129} },   /* 14 */
				{ {128, 0,   0, 0}, {0, 0, 0, 0}, {  0, 0, 0, 0}, {1, 1, 1, 129} },   /* 15 */
				{ {128, 0,   0, 0}, {1, 0, 0, 0}, {  1, 1, 1, 0}, {1, 1, 1, 129} },   /* 16 */
				{ {128, 1, 129, 1}, {0, 0, 0, 1}, {  0, 0, 0, 0}, {0, 0, 0,   0} },   /* 17 */
				{ {128, 0,   0, 0}, {0, 0, 0, 0}, {129, 0, 0, 0}, {1, 1, 1,   0} },   /* 18 */
				{ {128, 1, 129, 1}, {0, 0, 1, 1}, {  0, 0, 0, 1}, {0, 0, 0,   0} },   /* 19 */
				{ {128, 0, 129, 1}, {0, 0, 0, 1}, {  0, 0, 0, 0}, {0, 0, 0,   0} },   /* 20 */
				{ {128, 0,   0, 0}, {1, 0, 0, 0}, {129, 1, 0, 0}, {1, 1, 1,   0} },   /* 21 */
				{ {128, 0,   0, 0}, {0, 0, 0, 0}, {129, 0, 0, 0}, {1, 1, 0,   0} },   /* 22 */
				{ {128, 1,   1, 1}, {0, 0, 1, 1}, {  0, 0, 1, 1}, {0, 0, 0, 129} },   /* 23 */
				{ {128, 0, 129, 1}, {0, 0, 0, 1}, {  0, 0, 0, 1}, {0, 0, 0,   0} },   /* 24 */
				{ {128, 0,   0, 0}, {1, 0, 0, 0}, {129, 0, 0, 0}, {1, 1, 0,   0} },   /* 25 */
				{ {128, 1, 129, 0}, {0, 1, 1, 0}, {  0, 1, 1, 0}, {0, 1, 1,   0} },   /* 26 */
				{ {128, 0, 129, 1}, {0, 1, 1, 0}, {  0, 1, 1, 0}, {1, 1, 0,   0} },   /* 27 */
				{ {128, 0,   0, 1}, {0, 1, 1, 1}, {129, 1, 1, 0}, {1, 0, 0,   0} },   /* 28 */
				{ {128, 0,   0, 0}, {1, 1, 1, 1}, {129, 1, 1, 1}, {0, 0, 0,   0} },   /* 29 */
				{ {128, 1, 129, 1}, {0, 0, 0, 1}, {  1, 0, 0, 0}, {1, 1, 1,   0} },   /* 30 */
				{ {128, 0, 129, 1}, {1, 0, 0, 1}, {  1, 0, 0, 1}, {1, 1, 0,   0} }    /* 31 */
			};

			static int aWeight3[8] = { 0, 9, 18, 27, 37, 46, 55, 64 };
			static int aWeight4[16] = { 0, 4, 9, 13, 17, 21, 26, 30, 34, 38, 43, 47, 51, 55, 60, 64 };

			bcdec__bitstream_t bstream;
			int mode, partition, numPartitions, i, j, partitionSet, indexBits, index;
			int r[4], g[4], b[4];       /* wxyz */
			int epR[2], epG[2], epB[2]; /* endpoints A and B */
			unsigned short* decompressed;

			decompressed = (unsigned short*)decompressedBlock;

			bstream.low = ((unsigned long long*)compressedBlock)[0];
			bstream.high = ((unsigned long long*)compressedBlock)[1];

			r[0] = r[1] = r[2] = r[3] = 0;
			g[0] = g[1] = g[2] = g[3] = 0;
			b[0] = b[1] = b[2] = b[3] = 0;

			mode = bcdec__bitstream_read_bits(&bstream, 2);
			if (mode > 1) {
				mode |= (bcdec__bitstream_read_bits(&bstream, 3) << 2);
			}

			switch (mode) {
				/* mode 1 */
			case 0b00: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 75 bits (10.555, 10.555, 10.555) */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rx[4:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* gx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* bx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 5);        /* ry[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rz[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 0;
			} break;

				/* mode 2 */
			case 0b01: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 75 bits (7666, 7666, 7666) */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* gy[5]   */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* gz[5]   */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 7);        /* rw[6:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 7);        /* gw[6:0] */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* by[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 7);        /* bw[6:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* bz[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rx[5:0] */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* gx[5:0] */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* bx[5:0] */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 6);        /* ry[5:0] */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rz[5:0] */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 1;
			} break;

				/* mode 3 */
			case 0b00010: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (11.555, 11.444, 11.444) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rx[4:0] */
				r[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* rw[10]  */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gx[3:0] */
				g[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* gw[10]  */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* bx[3:0] */
				b[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* bw[10]  */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 5);        /* ry[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rz[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 2;
			} break;

				/* mode 4 */
			case 0b00110: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (11.444, 11.555, 11.444) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* rx[3:0] */
				r[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* rw[10]  */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* gx[4:0] */
				g[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* gw[10]  */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* bx[3:0] */
				b[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* bw[10]  */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* ry[3:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* rz[3:0] */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 3;
			} break;

				/* mode 5 */
			case 0b01010: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (11.444, 11.444, 11.555) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* rx[3:0] */
				r[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* rw[10]  */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gx[3:0] */
				g[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* gw[10]  */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* bx[4:0] */
				b[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* bw[10]  */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* ry[3:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* rz[3:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 4;
			} break;

				/* mode 6 */
			case 0b01110: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (9555, 9555, 9555) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 9);        /* rw[8:0] */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 9);        /* gw[8:0] */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 9);        /* bw[8:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rx[4:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* gx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gx[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* bx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 5);        /* ry[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rz[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 5;
			} break;

				/* mode 7 */
			case 0b10010: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (8666, 8555, 8555) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* rw[7:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* gw[7:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* bw[7:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rx[5:0] */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* gx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* bx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 6);        /* ry[5:0] */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rz[5:0] */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 6;
			} break;

				/* mode 8 */
			case 0b10110: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (8555, 8666, 8555) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* rw[7:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* gw[7:0] */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* gy[5]   */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* bw[7:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* gz[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rx[4:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* gx[5:0] */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* zx[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* bx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 5);        /* ry[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rz[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 7;
			} break;

				/* mode 9 */
			case 0b11010: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (8555, 8555, 8666) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* rw[7:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* gw[7:0] */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* by[5]   */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 8);        /* bw[7:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* bz[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* bw[4:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 5);        /* gx[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* bx[5:0] */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 5);        /* ry[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 5);        /* rz[4:0] */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 8;
			} break;

				/* mode 10 */
			case 0b11110: {
				/* Partitition indices: 46 bits
				   Partition: 5 bits
				   Color Endpoints: 72 bits (6666, 6666, 6666) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rw[5:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gz[4]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream);            /* bz[0]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 1;       /* bz[1]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* by[4]   */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 6);        /* gw[5:0] */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* gy[5]   */
				b[2] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* by[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 2;       /* bz[2]   */
				g[2] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* gy[4]   */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 6);        /* bw[5:0] */
				g[3] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* gz[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 3;       /* bz[3]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 5;       /* bz[5]   */
				b[3] |= bcdec__bitstream_read_bit(&bstream) << 4;       /* bz[4]   */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rx[5:0] */
				g[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gy[3:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* gx[5:0] */
				g[3] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gz[3:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 6);        /* bx[5:0] */
				b[2] |= bcdec__bitstream_read_bits(&bstream, 4);        /* by[3:0] */
				r[2] |= bcdec__bitstream_read_bits(&bstream, 6);        /* ry[5:0] */
				r[3] |= bcdec__bitstream_read_bits(&bstream, 6);        /* rz[5:0] */
				partition = bcdec__bitstream_read_bits(&bstream, 5);    /* d[4:0]  */
				mode = 9;
			} break;

				/* mode 11 */
			case 0b00011: {
				/* Partitition indices: 63 bits
				   Partition: 0 bits
				   Color Endpoints: 60 bits (10.10, 10.10, 10.10) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rx[9:0] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gx[9:0] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bx[9:0] */
				mode = 10;
			} break;

				/* mode 12 */
			case 0b00111: {
				/* Partitition indices: 63 bits
				   Partition: 0 bits
				   Color Endpoints: 60 bits (11.9, 11.9, 11.9) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 9);        /* rx[8:0] */
				r[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* rw[10]  */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 9);        /* gx[8:0] */
				g[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* gw[10]  */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 9);        /* bx[8:0] */
				b[0] |= bcdec__bitstream_read_bit(&bstream) << 10;      /* bw[10]  */
				mode = 11;
			} break;

				/* mode 13 */
			case 0b01011: {
				/* Partitition indices: 63 bits
				   Partition: 0 bits
				   Color Endpoints: 60 bits (12.8, 12.8, 12.8) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 8);        /* rx[7:0] */
				r[0] |= bcdec__bitstream_read_bits_r(&bstream, 2) << 10;/* rx[10:11] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 8);        /* gx[7:0] */
				g[0] |= bcdec__bitstream_read_bits_r(&bstream, 2) << 10;/* gx[10:11] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 8);        /* bx[7:0] */
				b[0] |= bcdec__bitstream_read_bits_r(&bstream, 2) << 10;/* bx[10:11] */
				mode = 12;
			} break;

				/* mode 14 */
			case 0b01111: {
				/* Partitition indices: 63 bits
				   Partition: 0 bits
				   Color Endpoints: 60 bits (16.4, 16.4, 16.4) */
				r[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* rw[9:0] */
				g[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* gw[9:0] */
				b[0] |= bcdec__bitstream_read_bits(&bstream, 10);       /* bw[9:0] */
				r[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* rx[3:0] */
				r[0] |= bcdec__bitstream_read_bits_r(&bstream, 6) << 10;/* rw[10:15] */
				g[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* gx[3:0] */
				g[0] |= bcdec__bitstream_read_bits_r(&bstream, 6) << 10;/* gw[10:15] */
				b[1] |= bcdec__bitstream_read_bits(&bstream, 4);        /* bx[3:0] */
				b[0] |= bcdec__bitstream_read_bits_r(&bstream, 6) << 10;/* bw[10:15] */
				mode = 13;
			} break;

			default: {
				/* Modes 10011, 10111, 11011, and 11111 (not shown) are reserved.
				   Do not use these in your encoder. If the hardware is passed blocks
				   with one of these modes specified, the resulting decompressed block
				   must contain all zeroes in all channels except for the alpha channel. */
				for (i = 0; i < 4; ++i) {
					for (j = 0; j < 4; ++j) {
						decompressed[j * 3 + 0] = 0;
						decompressed[j * 3 + 1] = 0;
						decompressed[j * 3 + 2] = 0;
					}
					decompressed += destinationPitch;
				}

				return;
			}
			}

			if (mode >= 10) {
				partition = 0;
				numPartitions = 0;
			}
			else {
				numPartitions = 1;
			}

			if (isSigned) {
				r[0] = bcdec__extend_sign(r[0], actual_bits_count[0][mode]);
				g[0] = bcdec__extend_sign(g[0], actual_bits_count[0][mode]);
				b[0] = bcdec__extend_sign(b[0], actual_bits_count[0][mode]);
			}

			/* Mode 11 (like Mode 10) does not use delta compression,
			   and instead stores both color endpoints explicitly.  */
			if ((mode != 9 && mode != 10) || isSigned) {
				for (i = 1; i < (numPartitions + 1) * 2; ++i) {
					r[i] = bcdec__extend_sign(r[i], actual_bits_count[1][mode]);
					g[i] = bcdec__extend_sign(g[i], actual_bits_count[2][mode]);
					b[i] = bcdec__extend_sign(b[i], actual_bits_count[3][mode]);
				}
			}

			if (mode != 9 && mode != 10) {
				for (i = 1; i < (numPartitions + 1) * 2; ++i) {
					r[i] = bcdec__transform_inverse(r[i], r[0], actual_bits_count[0][mode], isSigned);
					g[i] = bcdec__transform_inverse(g[i], g[0], actual_bits_count[0][mode], isSigned);
					b[i] = bcdec__transform_inverse(b[i], b[0], actual_bits_count[0][mode], isSigned);
				}
			}

			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					partitionSet = (mode >= 10) ? ((i | j) ? 0 : 128) : partition_sets[partition][i][j];

					indexBits = (mode >= 10) ? 4 : 3;
					/* fix-up index is specified with one less bit */
					/* The fix-up index for subset 0 is always index 0 */
					if (partitionSet & 0x80) {
						indexBits--;
					}
					partitionSet &= 0x01;

					index = bcdec__bitstream_read_bits(&bstream, indexBits);

					epR[0] = bcdec__unquantize(r[partitionSet * 2 + 0], actual_bits_count[0][mode], isSigned);
					epG[0] = bcdec__unquantize(g[partitionSet * 2 + 0], actual_bits_count[0][mode], isSigned);
					epB[0] = bcdec__unquantize(b[partitionSet * 2 + 0], actual_bits_count[0][mode], isSigned);
					epR[1] = bcdec__unquantize(r[partitionSet * 2 + 1], actual_bits_count[0][mode], isSigned);
					epG[1] = bcdec__unquantize(g[partitionSet * 2 + 1], actual_bits_count[0][mode], isSigned);
					epB[1] = bcdec__unquantize(b[partitionSet * 2 + 1], actual_bits_count[0][mode], isSigned);

					decompressed[j * 3 + 0] = bcdec__finish_unquantize(
						bcdec__interpolate(epR[0], epR[1], (mode >= 10) ? aWeight4 : aWeight3, index), isSigned);
					decompressed[j * 3 + 1] = bcdec__finish_unquantize(
						bcdec__interpolate(epG[0], epG[1], (mode >= 10) ? aWeight4 : aWeight3, index), isSigned);
					decompressed[j * 3 + 2] = bcdec__finish_unquantize(
						bcdec__interpolate(epB[0], epB[1], (mode >= 10) ? aWeight4 : aWeight3, index), isSigned);
				}

				decompressed += destinationPitch;
			}
		}

		BCDECDEF void bcdec_bc6h_float(const void* compressedBlock, void* decompressedBlock, int destinationPitch, int isSigned) {
			unsigned short block[16 * 3];
			float* decompressed;
			const unsigned short* b;
			int i, j;

			bcdec_bc6h_half(compressedBlock, block, 4 * 3, isSigned);
			b = block;
			decompressed = (float*)decompressedBlock;
			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					decompressed[j * 3 + 0] = bcdec__half_to_float_quick(*b++);
					decompressed[j * 3 + 1] = bcdec__half_to_float_quick(*b++);
					decompressed[j * 3 + 2] = bcdec__half_to_float_quick(*b++);
				}
				decompressed += destinationPitch;
			}
		}

		static void bcdec__swap_values(int* a, int* b) {
			a[0] ^= b[0], b[0] ^= a[0], a[0] ^= b[0];
		}

		BCDECDEF void bcdec_bc7(const void* compressedBlock, void* decompressedBlock, int destinationPitch) {
			static char actual_bits_count[2][8] = {
				{ 4, 6, 5, 7, 5, 7, 7, 5 },     /* RGBA  */
				{ 0, 0, 0, 0, 6, 8, 7, 5 },     /* Alpha */
			};

			/* There are 64 possible partition sets for a two-region tile.
			   Each 4x4 block represents a single shape.
			   Here also every fix-up index has MSB bit set. */
			static unsigned char partition_sets[2][64][4][4] = {
				{   /* Partition table for 2-subset BPTC */
					{ {128, 0,   1, 1}, {0, 0,   1, 1}, {  0, 0, 1, 1}, {0, 0, 1, 129} }, /*  0 */
					{ {128, 0,   0, 1}, {0, 0,   0, 1}, {  0, 0, 0, 1}, {0, 0, 0, 129} }, /*  1 */
					{ {128, 1,   1, 1}, {0, 1,   1, 1}, {  0, 1, 1, 1}, {0, 1, 1, 129} }, /*  2 */
					{ {128, 0,   0, 1}, {0, 0,   1, 1}, {  0, 0, 1, 1}, {0, 1, 1, 129} }, /*  3 */
					{ {128, 0,   0, 0}, {0, 0,   0, 1}, {  0, 0, 0, 1}, {0, 0, 1, 129} }, /*  4 */
					{ {128, 0,   1, 1}, {0, 1,   1, 1}, {  0, 1, 1, 1}, {1, 1, 1, 129} }, /*  5 */
					{ {128, 0,   0, 1}, {0, 0,   1, 1}, {  0, 1, 1, 1}, {1, 1, 1, 129} }, /*  6 */
					{ {128, 0,   0, 0}, {0, 0,   0, 1}, {  0, 0, 1, 1}, {0, 1, 1, 129} }, /*  7 */
					{ {128, 0,   0, 0}, {0, 0,   0, 0}, {  0, 0, 0, 1}, {0, 0, 1, 129} }, /*  8 */
					{ {128, 0,   1, 1}, {0, 1,   1, 1}, {  1, 1, 1, 1}, {1, 1, 1, 129} }, /*  9 */
					{ {128, 0,   0, 0}, {0, 0,   0, 1}, {  0, 1, 1, 1}, {1, 1, 1, 129} }, /* 10 */
					{ {128, 0,   0, 0}, {0, 0,   0, 0}, {  0, 0, 0, 1}, {0, 1, 1, 129} }, /* 11 */
					{ {128, 0,   0, 1}, {0, 1,   1, 1}, {  1, 1, 1, 1}, {1, 1, 1, 129} }, /* 12 */
					{ {128, 0,   0, 0}, {0, 0,   0, 0}, {  1, 1, 1, 1}, {1, 1, 1, 129} }, /* 13 */
					{ {128, 0,   0, 0}, {1, 1,   1, 1}, {  1, 1, 1, 1}, {1, 1, 1, 129} }, /* 14 */
					{ {128, 0,   0, 0}, {0, 0,   0, 0}, {  0, 0, 0, 0}, {1, 1, 1, 129} }, /* 15 */
					{ {128, 0,   0, 0}, {1, 0,   0, 0}, {  1, 1, 1, 0}, {1, 1, 1, 129} }, /* 16 */
					{ {128, 1, 129, 1}, {0, 0,   0, 1}, {  0, 0, 0, 0}, {0, 0, 0,   0} }, /* 17 */
					{ {128, 0,   0, 0}, {0, 0,   0, 0}, {129, 0, 0, 0}, {1, 1, 1,   0} }, /* 18 */
					{ {128, 1, 129, 1}, {0, 0,   1, 1}, {  0, 0, 0, 1}, {0, 0, 0,   0} }, /* 19 */
					{ {128, 0, 129, 1}, {0, 0,   0, 1}, {  0, 0, 0, 0}, {0, 0, 0,   0} }, /* 20 */
					{ {128, 0,   0, 0}, {1, 0,   0, 0}, {129, 1, 0, 0}, {1, 1, 1,   0} }, /* 21 */
					{ {128, 0,   0, 0}, {0, 0,   0, 0}, {129, 0, 0, 0}, {1, 1, 0,   0} }, /* 22 */
					{ {128, 1,   1, 1}, {0, 0,   1, 1}, {  0, 0, 1, 1}, {0, 0, 0, 129} }, /* 23 */
					{ {128, 0, 129, 1}, {0, 0,   0, 1}, {  0, 0, 0, 1}, {0, 0, 0,   0} }, /* 24 */
					{ {128, 0,   0, 0}, {1, 0,   0, 0}, {129, 0, 0, 0}, {1, 1, 0,   0} }, /* 25 */
					{ {128, 1, 129, 0}, {0, 1,   1, 0}, {  0, 1, 1, 0}, {0, 1, 1,   0} }, /* 26 */
					{ {128, 0, 129, 1}, {0, 1,   1, 0}, {  0, 1, 1, 0}, {1, 1, 0,   0} }, /* 27 */
					{ {128, 0,   0, 1}, {0, 1,   1, 1}, {129, 1, 1, 0}, {1, 0, 0,   0} }, /* 28 */
					{ {128, 0,   0, 0}, {1, 1,   1, 1}, {129, 1, 1, 1}, {0, 0, 0,   0} }, /* 29 */
					{ {128, 1, 129, 1}, {0, 0,   0, 1}, {  1, 0, 0, 0}, {1, 1, 1,   0} }, /* 30 */
					{ {128, 0, 129, 1}, {1, 0,   0, 1}, {  1, 0, 0, 1}, {1, 1, 0,   0} }, /* 31 */
					{ {128, 1,   0, 1}, {0, 1,   0, 1}, {  0, 1, 0, 1}, {0, 1, 0, 129} }, /* 32 */
					{ {128, 0,   0, 0}, {1, 1,   1, 1}, {  0, 0, 0, 0}, {1, 1, 1, 129} }, /* 33 */
					{ {128, 1,   0, 1}, {1, 0, 129, 0}, {  0, 1, 0, 1}, {1, 0, 1,   0} }, /* 34 */
					{ {128, 0,   1, 1}, {0, 0,   1, 1}, {129, 1, 0, 0}, {1, 1, 0,   0} }, /* 35 */
					{ {128, 0, 129, 1}, {1, 1,   0, 0}, {  0, 0, 1, 1}, {1, 1, 0,   0} }, /* 36 */
					{ {128, 1,   0, 1}, {0, 1,   0, 1}, {129, 0, 1, 0}, {1, 0, 1,   0} }, /* 37 */
					{ {128, 1,   1, 0}, {1, 0,   0, 1}, {  0, 1, 1, 0}, {1, 0, 0, 129} }, /* 38 */
					{ {128, 1,   0, 1}, {1, 0,   1, 0}, {  1, 0, 1, 0}, {0, 1, 0, 129} }, /* 39 */
					{ {128, 1, 129, 1}, {0, 0,   1, 1}, {  1, 1, 0, 0}, {1, 1, 1,   0} }, /* 40 */
					{ {128, 0,   0, 1}, {0, 0,   1, 1}, {129, 1, 0, 0}, {1, 0, 0,   0} }, /* 41 */
					{ {128, 0, 129, 1}, {0, 0,   1, 0}, {  0, 1, 0, 0}, {1, 1, 0,   0} }, /* 42 */
					{ {128, 0, 129, 1}, {1, 0,   1, 1}, {  1, 1, 0, 1}, {1, 1, 0,   0} }, /* 43 */
					{ {128, 1, 129, 0}, {1, 0,   0, 1}, {  1, 0, 0, 1}, {0, 1, 1,   0} }, /* 44 */
					{ {128, 0,   1, 1}, {1, 1,   0, 0}, {  1, 1, 0, 0}, {0, 0, 1, 129} }, /* 45 */
					{ {128, 1,   1, 0}, {0, 1,   1, 0}, {  1, 0, 0, 1}, {1, 0, 0, 129} }, /* 46 */
					{ {128, 0,   0, 0}, {0, 1, 129, 0}, {  0, 1, 1, 0}, {0, 0, 0,   0} }, /* 47 */
					{ {128, 1,   0, 0}, {1, 1, 129, 0}, {  0, 1, 0, 0}, {0, 0, 0,   0} }, /* 48 */
					{ {128, 0, 129, 0}, {0, 1,   1, 1}, {  0, 0, 1, 0}, {0, 0, 0,   0} }, /* 49 */
					{ {128, 0,   0, 0}, {0, 0, 129, 0}, {  0, 1, 1, 1}, {0, 0, 1,   0} }, /* 50 */
					{ {128, 0,   0, 0}, {0, 1,   0, 0}, {129, 1, 1, 0}, {0, 1, 0,   0} }, /* 51 */
					{ {128, 1,   1, 0}, {1, 1,   0, 0}, {  1, 0, 0, 1}, {0, 0, 1, 129} }, /* 52 */
					{ {128, 0,   1, 1}, {0, 1,   1, 0}, {  1, 1, 0, 0}, {1, 0, 0, 129} }, /* 53 */
					{ {128, 1, 129, 0}, {0, 0,   1, 1}, {  1, 0, 0, 1}, {1, 1, 0,   0} }, /* 54 */
					{ {128, 0, 129, 1}, {1, 0,   0, 1}, {  1, 1, 0, 0}, {0, 1, 1,   0} }, /* 55 */
					{ {128, 1,   1, 0}, {1, 1,   0, 0}, {  1, 1, 0, 0}, {1, 0, 0, 129} }, /* 56 */
					{ {128, 1,   1, 0}, {0, 0,   1, 1}, {  0, 0, 1, 1}, {1, 0, 0, 129} }, /* 57 */
					{ {128, 1,   1, 1}, {1, 1,   1, 0}, {  1, 0, 0, 0}, {0, 0, 0, 129} }, /* 58 */
					{ {128, 0,   0, 1}, {1, 0,   0, 0}, {  1, 1, 1, 0}, {0, 1, 1, 129} }, /* 59 */
					{ {128, 0,   0, 0}, {1, 1,   1, 1}, {  0, 0, 1, 1}, {0, 0, 1, 129} }, /* 60 */
					{ {128, 0, 129, 1}, {0, 0,   1, 1}, {  1, 1, 1, 1}, {0, 0, 0,   0} }, /* 61 */
					{ {128, 0, 129, 0}, {0, 0,   1, 0}, {  1, 1, 1, 0}, {1, 1, 1,   0} }, /* 62 */
					{ {128, 1,   0, 0}, {0, 1,   0, 0}, {  0, 1, 1, 1}, {0, 1, 1, 129} }  /* 63 */
				},
				{   /* Partition table for 3-subset BPTC */
					{ {128, 0, 1, 129}, {0,   0,   1, 1}, {  0,   2,   2, 1}, {  2,   2, 2, 130} }, /*  0 */
					{ {128, 0, 0, 129}, {0,   0,   1, 1}, {130,   2,   1, 1}, {  2,   2, 2,   1} }, /*  1 */
					{ {128, 0, 0,   0}, {2,   0,   0, 1}, {130,   2,   1, 1}, {  2,   2, 1, 129} }, /*  2 */
					{ {128, 2, 2, 130}, {0,   0,   2, 2}, {  0,   0,   1, 1}, {  0,   1, 1, 129} }, /*  3 */
					{ {128, 0, 0,   0}, {0,   0,   0, 0}, {129,   1,   2, 2}, {  1,   1, 2, 130} }, /*  4 */
					{ {128, 0, 1, 129}, {0,   0,   1, 1}, {  0,   0,   2, 2}, {  0,   0, 2, 130} }, /*  5 */
					{ {128, 0, 2, 130}, {0,   0,   2, 2}, {  1,   1,   1, 1}, {  1,   1, 1, 129} }, /*  6 */
					{ {128, 0, 1,   1}, {0,   0,   1, 1}, {130,   2,   1, 1}, {  2,   2, 1, 129} }, /*  7 */
					{ {128, 0, 0,   0}, {0,   0,   0, 0}, {129,   1,   1, 1}, {  2,   2, 2, 130} }, /*  8 */
					{ {128, 0, 0,   0}, {1,   1,   1, 1}, {129,   1,   1, 1}, {  2,   2, 2, 130} }, /*  9 */
					{ {128, 0, 0,   0}, {1,   1, 129, 1}, {  2,   2,   2, 2}, {  2,   2, 2, 130} }, /* 10 */
					{ {128, 0, 1,   2}, {0,   0, 129, 2}, {  0,   0,   1, 2}, {  0,   0, 1, 130} }, /* 11 */
					{ {128, 1, 1,   2}, {0,   1, 129, 2}, {  0,   1,   1, 2}, {  0,   1, 1, 130} }, /* 12 */
					{ {128, 1, 2,   2}, {0, 129,   2, 2}, {  0,   1,   2, 2}, {  0,   1, 2, 130} }, /* 13 */
					{ {128, 0, 1, 129}, {0,   1,   1, 2}, {  1,   1,   2, 2}, {  1,   2, 2, 130} }, /* 14 */
					{ {128, 0, 1, 129}, {2,   0,   0, 1}, {130,   2,   0, 0}, {  2,   2, 2,   0} }, /* 15 */
					{ {128, 0, 0, 129}, {0,   0,   1, 1}, {  0,   1,   1, 2}, {  1,   1, 2, 130} }, /* 16 */
					{ {128, 1, 1, 129}, {0,   0,   1, 1}, {130,   0,   0, 1}, {  2,   2, 0,   0} }, /* 17 */
					{ {128, 0, 0,   0}, {1,   1,   2, 2}, {129,   1,   2, 2}, {  1,   1, 2, 130} }, /* 18 */
					{ {128, 0, 2, 130}, {0,   0,   2, 2}, {  0,   0,   2, 2}, {  1,   1, 1, 129} }, /* 19 */
					{ {128, 1, 1, 129}, {0,   1,   1, 1}, {  0,   2,   2, 2}, {  0,   2, 2, 130} }, /* 20 */
					{ {128, 0, 0, 129}, {0,   0,   0, 1}, {130,   2,   2, 1}, {  2,   2, 2,   1} }, /* 21 */
					{ {128, 0, 0,   0}, {0,   0, 129, 1}, {  0,   1,   2, 2}, {  0,   1, 2, 130} }, /* 22 */
					{ {128, 0, 0,   0}, {1,   1,   0, 0}, {130,   2, 129, 0}, {  2,   2, 1,   0} }, /* 23 */
					{ {128, 1, 2, 130}, {0, 129,   2, 2}, {  0,   0,   1, 1}, {  0,   0, 0,   0} }, /* 24 */
					{ {128, 0, 1,   2}, {0,   0,   1, 2}, {129,   1,   2, 2}, {  2,   2, 2, 130} }, /* 25 */
					{ {128, 1, 1,   0}, {1,   2, 130, 1}, {129,   2,   2, 1}, {  0,   1, 1,   0} }, /* 26 */
					{ {128, 0, 0,   0}, {0,   1, 129, 0}, {  1,   2, 130, 1}, {  1,   2, 2,   1} }, /* 27 */
					{ {128, 0, 2,   2}, {1,   1,   0, 2}, {129,   1,   0, 2}, {  0,   0, 2, 130} }, /* 28 */
					{ {128, 1, 1,   0}, {0, 129,   1, 0}, {  2,   0,   0, 2}, {  2,   2, 2, 130} }, /* 29 */
					{ {128, 0, 1,   1}, {0,   1,   2, 2}, {  0,   1, 130, 2}, {  0,   0, 1, 129} }, /* 30 */
					{ {128, 0, 0,   0}, {2,   0,   0, 0}, {130,   2,   1, 1}, {  2,   2, 2, 129} }, /* 31 */
					{ {128, 0, 0,   0}, {0,   0,   0, 2}, {129,   1,   2, 2}, {  1,   2, 2, 130} }, /* 32 */
					{ {128, 2, 2, 130}, {0,   0,   2, 2}, {  0,   0,   1, 2}, {  0,   0, 1, 129} }, /* 33 */
					{ {128, 0, 1, 129}, {0,   0,   1, 2}, {  0,   0,   2, 2}, {  0,   2, 2, 130} }, /* 34 */
					{ {128, 1, 2,   0}, {0, 129,   2, 0}, {  0,   1, 130, 0}, {  0,   1, 2,   0} }, /* 35 */
					{ {128, 0, 0,   0}, {1,   1, 129, 1}, {  2,   2, 130, 2}, {  0,   0, 0,   0} }, /* 36 */
					{ {128, 1, 2,   0}, {1,   2,   0, 1}, {130,   0, 129, 2}, {  0,   1, 2,   0} }, /* 37 */
					{ {128, 1, 2,   0}, {2,   0,   1, 2}, {129, 130,   0, 1}, {  0,   1, 2,   0} }, /* 38 */
					{ {128, 0, 1,   1}, {2,   2,   0, 0}, {  1,   1, 130, 2}, {  0,   0, 1, 129} }, /* 39 */
					{ {128, 0, 1,   1}, {1,   1, 130, 2}, {  2,   2,   0, 0}, {  0,   0, 1, 129} }, /* 40 */
					{ {128, 1, 0, 129}, {0,   1,   0, 1}, {  2,   2,   2, 2}, {  2,   2, 2, 130} }, /* 41 */
					{ {128, 0, 0,   0}, {0,   0,   0, 0}, {130,   1,   2, 1}, {  2,   1, 2, 129} }, /* 42 */
					{ {128, 0, 2,   2}, {1, 129,   2, 2}, {  0,   0,   2, 2}, {  1,   1, 2, 130} }, /* 43 */
					{ {128, 0, 2, 130}, {0,   0,   1, 1}, {  0,   0,   2, 2}, {  0,   0, 1, 129} }, /* 44 */
					{ {128, 2, 2,   0}, {1,   2, 130, 1}, {  0,   2,   2, 0}, {  1,   2, 2, 129} }, /* 45 */
					{ {128, 1, 0,   1}, {2,   2, 130, 2}, {  2,   2,   2, 2}, {  0,   1, 0, 129} }, /* 46 */
					{ {128, 0, 0,   0}, {2,   1,   2, 1}, {130,   1,   2, 1}, {  2,   1, 2, 129} }, /* 47 */
					{ {128, 1, 0, 129}, {0,   1,   0, 1}, {  0,   1,   0, 1}, {  2,   2, 2, 130} }, /* 48 */
					{ {128, 2, 2, 130}, {0,   1,   1, 1}, {  0,   2,   2, 2}, {  0,   1, 1, 129} }, /* 49 */
					{ {128, 0, 0,   2}, {1, 129,   1, 2}, {  0,   0,   0, 2}, {  1,   1, 1, 130} }, /* 50 */
					{ {128, 0, 0,   0}, {2, 129,   1, 2}, {  2,   1,   1, 2}, {  2,   1, 1, 130} }, /* 51 */
					{ {128, 2, 2,   2}, {0, 129,   1, 1}, {  0,   1,   1, 1}, {  0,   2, 2, 130} }, /* 52 */
					{ {128, 0, 0,   2}, {1,   1,   1, 2}, {129,   1,   1, 2}, {  0,   0, 0, 130} }, /* 53 */
					{ {128, 1, 1,   0}, {0, 129,   1, 0}, {  0,   1,   1, 0}, {  2,   2, 2, 130} }, /* 54 */
					{ {128, 0, 0,   0}, {0,   0,   0, 0}, {  2,   1, 129, 2}, {  2,   1, 1, 130} }, /* 55 */
					{ {128, 1, 1,   0}, {0, 129,   1, 0}, {  2,   2,   2, 2}, {  2,   2, 2, 130} }, /* 56 */
					{ {128, 0, 2,   2}, {0,   0,   1, 1}, {  0,   0, 129, 1}, {  0,   0, 2, 130} }, /* 57 */
					{ {128, 0, 2,   2}, {1,   1,   2, 2}, {129,   1,   2, 2}, {  0,   0, 2, 130} }, /* 58 */
					{ {128, 0, 0,   0}, {0,   0,   0, 0}, {  0,   0,   0, 0}, {  2, 129, 1, 130} }, /* 59 */
					{ {128, 0, 0, 130}, {0,   0,   0, 1}, {  0,   0,   0, 2}, {  0,   0, 0, 129} }, /* 60 */
					{ {128, 2, 2,   2}, {1,   2,   2, 2}, {  0,   2,   2, 2}, {129,   2, 2, 130} }, /* 61 */
					{ {128, 1, 0, 129}, {2,   2,   2, 2}, {  2,   2,   2, 2}, {  2,   2, 2, 130} }, /* 62 */
					{ {128, 1, 1, 129}, {2,   0,   1, 1}, {130,   2,   0, 1}, {  2,   2, 2,   0} }  /* 63 */
				}
			};

			static int aWeight2[] = { 0, 21, 43, 64 };
			static int aWeight3[] = { 0, 9, 18, 27, 37, 46, 55, 64 };
			static int aWeight4[] = { 0, 4, 9, 13, 17, 21, 26, 30, 34, 38, 43, 47, 51, 55, 60, 64 };

			static unsigned char sModeHasPBits = 0b11001011;

			bcdec__bitstream_t bstream;
			int mode, partition, numPartitions, numEndpoints, i, j, k, rotation, partitionSet;
			int indexSelectionBit, indexBits, indexBits2, index, index2;
			int endpoints[6][4];
			char indices[4][4];
			int r, g, b, a;
			int* weights, * weights2;
			unsigned char* decompressed;

			decompressed = (unsigned char*)decompressedBlock;

			bstream.low = ((unsigned long long*)compressedBlock)[0];
			bstream.high = ((unsigned long long*)compressedBlock)[1];

			for (mode = 0; mode < 8 && (0 == bcdec__bitstream_read_bit(&bstream)); ++mode);

			/* unexpected mode, clear the block (transparent black) */
			if (mode >= 8) {
				for (i = 0; i < 4; ++i) {
					for (j = 0; j < 4; ++j) {
						decompressed[j * 4 + 0] = 0;
						decompressed[j * 4 + 1] = 0;
						decompressed[j * 4 + 2] = 0;
						decompressed[j * 4 + 3] = 0;
					}
					decompressed += destinationPitch;
				}

				return;
			}

			partition = 0;
			numPartitions = 1;
			rotation = 0;
			indexSelectionBit = 0;

			if (mode == 0 || mode == 1 || mode == 2 || mode == 3 || mode == 7) {
				numPartitions = (mode == 0 || mode == 2) ? 3 : 2;
				partition = bcdec__bitstream_read_bits(&bstream, (mode == 0) ? 4 : 6);
			}

			numEndpoints = numPartitions * 2;

			if (mode == 4 || mode == 5) {
				rotation = bcdec__bitstream_read_bits(&bstream, 2);

				if (mode == 4) {
					indexSelectionBit = bcdec__bitstream_read_bit(&bstream);
				}
			}

			/* Extract endpoints */
			/* RGB */
			for (i = 0; i < 3; ++i) {
				for (j = 0; j < numEndpoints; ++j) {
					endpoints[j][i] = bcdec__bitstream_read_bits(&bstream, actual_bits_count[0][mode]);
				}
			}
			/* Alpha (if any) */
			if (actual_bits_count[1][mode] > 0) {
				for (j = 0; j < numEndpoints; ++j) {
					endpoints[j][3] = bcdec__bitstream_read_bits(&bstream, actual_bits_count[1][mode]);
				}
			}

			/* Fully decode endpoints */
			/* First handle modes that have P-bits */
			if (mode == 0 || mode == 1 || mode == 3 || mode == 6 || mode == 7) {
				for (i = 0; i < numEndpoints; ++i) {
					/* component-wise left-shift */
					for (j = 0; j < 4; ++j) {
						endpoints[i][j] <<= 1;
					}
				}

				/* if P-bit is shared */
				if (mode == 1) {
					i = bcdec__bitstream_read_bit(&bstream);
					j = bcdec__bitstream_read_bit(&bstream);

					/* rgb component-wise insert pbits */
					for (k = 0; k < 3; ++k) {
						endpoints[0][k] |= i;
						endpoints[1][k] |= i;
						endpoints[2][k] |= j;
						endpoints[3][k] |= j;
					}
				}
				else if (sModeHasPBits & (1 << mode)) {
					/* unique P-bit per endpoint */
					for (i = 0; i < numEndpoints; ++i) {
						j = bcdec__bitstream_read_bit(&bstream);
						for (k = 0; k < 4; ++k) {
							endpoints[i][k] |= j;
						}
					}
				}
			}

			for (i = 0; i < numEndpoints; ++i) {
				/* get color components precision including pbit */
				j = actual_bits_count[0][mode] + ((sModeHasPBits >> mode) & 1);

				for (k = 0; k < 3; ++k) {
					/* left shift endpoint components so that their MSB lies in bit 7 */
					endpoints[i][k] = endpoints[i][k] << (8 - j);
					/* Replicate each component's MSB into the LSBs revealed by the left-shift operation above */
					endpoints[i][k] = endpoints[i][k] | (endpoints[i][k] >> j);
				}

				/* get alpha component precision including pbit */
				j = actual_bits_count[1][mode] + ((sModeHasPBits >> mode) & 1);

				/* left shift endpoint components so that their MSB lies in bit 7 */
				endpoints[i][3] = endpoints[i][3] << (8 - j);
				/* Replicate each component's MSB into the LSBs revealed by the left-shift operation above */
				endpoints[i][3] = endpoints[i][3] | (endpoints[i][3] >> j);
			}

			/* If this mode does not explicitly define the alpha component */
			/* set alpha equal to 1.0 */
			if (!actual_bits_count[1][mode]) {
				for (j = 0; j < numEndpoints; ++j) {
					endpoints[j][3] = 0xFF;
				}
			}

			/* Determine weights tables */
			indexBits = (mode == 0 || mode == 1) ? 3 : ((mode == 6) ? 4 : 2);
			indexBits2 = (mode == 4) ? 3 : ((mode == 5) ? 2 : 0);
			weights = (indexBits == 2) ? aWeight2 : ((indexBits == 3) ? aWeight3 : aWeight4);
			weights2 = (indexBits2 == 2) ? aWeight2 : aWeight3;

			/* Quite inconvenient that indices aren't interleaved so we have to make 2 passes here */
			/* Pass #1: collecting color indices */
			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					partitionSet = (numPartitions == 1) ? ((i | j) ? 0 : 128) : partition_sets[numPartitions - 2][partition][i][j];

					indexBits = (mode == 0 || mode == 1) ? 3 : ((mode == 6) ? 4 : 2);
					/* fix-up index is specified with one less bit */
					/* The fix-up index for subset 0 is always index 0 */
					if (partitionSet & 0x80) {
						indexBits--;
					}

					indices[i][j] = bcdec__bitstream_read_bits(&bstream, indexBits);
				}
			}

			/* Pass #2: reading alpha indices (if any) and interpolating & rotating */
			for (i = 0; i < 4; ++i) {
				for (j = 0; j < 4; ++j) {
					partitionSet = (numPartitions == 1) ? ((i | j) ? 0 : 128) : partition_sets[numPartitions - 2][partition][i][j];
					partitionSet &= 0x03;

					index = indices[i][j];

					if (!indexBits2) {
						r = bcdec__interpolate(endpoints[partitionSet * 2][0], endpoints[partitionSet * 2 + 1][0], weights, index);
						g = bcdec__interpolate(endpoints[partitionSet * 2][1], endpoints[partitionSet * 2 + 1][1], weights, index);
						b = bcdec__interpolate(endpoints[partitionSet * 2][2], endpoints[partitionSet * 2 + 1][2], weights, index);
						a = bcdec__interpolate(endpoints[partitionSet * 2][3], endpoints[partitionSet * 2 + 1][3], weights, index);
					}
					else {
						index2 = bcdec__bitstream_read_bits(&bstream, (i | j) ? indexBits2 : (indexBits2 - 1));
						/* The index value for interpolating color comes from the secondary index bits for the texel
						   if the mode has an index selection bit and its value is one, and from the primary index bits otherwise.
						   The alpha index comes from the secondary index bits if the block has a secondary index and
						   the block either doesn’t have an index selection bit or that bit is zero, and from the primary index bits otherwise. */
						if (!indexSelectionBit) {
							r = bcdec__interpolate(endpoints[partitionSet * 2][0], endpoints[partitionSet * 2 + 1][0], weights, index);
							g = bcdec__interpolate(endpoints[partitionSet * 2][1], endpoints[partitionSet * 2 + 1][1], weights, index);
							b = bcdec__interpolate(endpoints[partitionSet * 2][2], endpoints[partitionSet * 2 + 1][2], weights, index);
							a = bcdec__interpolate(endpoints[partitionSet * 2][3], endpoints[partitionSet * 2 + 1][3], weights2, index2);
						}
						else {
							r = bcdec__interpolate(endpoints[partitionSet * 2][0], endpoints[partitionSet * 2 + 1][0], weights2, index2);
							g = bcdec__interpolate(endpoints[partitionSet * 2][1], endpoints[partitionSet * 2 + 1][1], weights2, index2);
							b = bcdec__interpolate(endpoints[partitionSet * 2][2], endpoints[partitionSet * 2 + 1][2], weights2, index2);
							a = bcdec__interpolate(endpoints[partitionSet * 2][3], endpoints[partitionSet * 2 + 1][3], weights, index);
						}
					}

					switch (rotation) {
					case 1: {   /* 01 – Block format is Scalar(R) Vector(AGB) - swap A and R */
						bcdec__swap_values(&a, &r);
					} break;
					case 2: {   /* 10 – Block format is Scalar(G) Vector(RAB) - swap A and G */
						bcdec__swap_values(&a, &g);
					} break;
					case 3: {   /* 11 - Block format is Scalar(B) Vector(RGA) - swap A and B */
						bcdec__swap_values(&a, &b);
					} break;
					}

					decompressed[j * 4 + 0] = r;
					decompressed[j * 4 + 1] = g;
					decompressed[j * 4 + 2] = b;
					decompressed[j * 4 + 3] = a;
				}

				decompressed += destinationPitch;
			}
		}

#endif /* BCDEC_IMPLEMENTATION */
	}


	//-------------------------------------------------------------------------------------------------
	// Based on STB v1.08, under the MIT license
	// See https://github.com/nothings/stb/
	//-------------------------------------------------------------------------------------------------
	namespace stb
	{
		// compression mode (bitflags)
#define STB_DXT_NORMAL    0
#define STB_DXT_DITHER    1   // use dithering. dubious win. never use for normal maps and the like!
#define STB_DXT_HIGHQUAL  2   // high quality mode, does two refinement steps instead of 1. ~30-40% slower.

		static void stb_compress_dxt_block(unsigned char* dest, const unsigned char* src_rgba_four_bytes_per_pixel, int alpha, int mode);
		static void stb_compress_bc4_block(unsigned char* dest, const unsigned char* src_r_one_byte_per_pixel);
		static void stb_compress_bc5_block(unsigned char* dest, const unsigned char* src_rg_two_byte_per_pixel);

#define STB_COMPRESS_DXT_BLOCK


		// configuration options for DXT encoder. set them in the project/makefile or just define
		// them at the top.

		// STB_DXT_USE_ROUNDING_BIAS
		//     use a rounding bias during color interpolation. this is closer to what "ideal"
		//     interpolation would do but doesn't match the S3TC/DX10 spec. old versions (pre-1.03)
		//     implicitly had this turned on.
		//
		//     in case you're targeting a specific type of hardware (e.g. console programmers):
		//     NVidia and Intel GPUs (as of 2010) as well as DX9 ref use DXT decoders that are closer
		//     to STB_DXT_USE_ROUNDING_BIAS. AMD/ATI, S3 and DX10 ref are closer to rounding with no bias.
		//     you also see "(a*5 + b*3) / 8" on some old GPU designs.
		// #define STB_DXT_USE_ROUNDING_BIAS

#ifndef STBD_ABS
#define STBD_ABS(i)           abs(i)
#endif

#ifndef STBD_FABS
#define STBD_FABS(x)          fabs(x)
#endif

#ifndef STBD_MEMSET
#define STBD_MEMSET           memset
#endif

		static int stb__Mul8Bit(int a, int b)
		{
			int t = a * b + 128;
			return (t + (t >> 8)) >> 8;
		}

		static void stb__From16Bit(unsigned char* out, unsigned short v)
		{
			int rv = (v & 0xf800) >> 11;
			int gv = (v & 0x07e0) >> 5;
			int bv = (v & 0x001f) >> 0;

			out[0] = stb__Expand5[rv];
			out[1] = stb__Expand6[gv];
			out[2] = stb__Expand5[bv];
			out[3] = 0;
		}

		static unsigned short stb__Aint16_tBit(int r, int g, int b)
		{
			return uint16((stb__Mul8Bit(r, 31) << 11) + (stb__Mul8Bit(g, 63) << 5) + stb__Mul8Bit(b, 31));
		}

		// linear interpolation at 1/3 point between a and b, using desired rounding type
		static int stb__Lerp13(int a, int b)
		{
#ifdef STB_DXT_USE_ROUNDING_BIAS
			// with rounding bias
			return a + stb__Mul8Bit(b - a, 0x55);
#else
			// without rounding bias
			// replace "/ 3" by "* 0xaaab) >> 17" if your compiler sucks or you really need every ounce of speed.
			return (2 * a + b) / 3;
#endif
		}

		// lerp RGB color
		static void stb__Lerp13RGB(unsigned char* out, unsigned char* p1, unsigned char* p2)
		{
			out[0] = (uint8)stb__Lerp13(p1[0], p2[0]);
			out[1] = (uint8)stb__Lerp13(p1[1], p2[1]);
			out[2] = (uint8)stb__Lerp13(p1[2], p2[2]);
		}

		/****************************************************************************/

		// compute table to reproduce constant colors as accurately as possible
		static void stb__PrepareOptTable(unsigned char* Table, const unsigned char* expand, int size)
		{
			int i, mn, mx;
			for (i = 0; i < 256; i++) {
				int bestErr = 256;
				for (mn = 0; mn < size; mn++) {
					for (mx = 0; mx < size; mx++) {
						int mine = expand[mn];
						int maxe = expand[mx];
						int err = STBD_ABS(stb__Lerp13(maxe, mine) - i);

						// DX10 spec says that interpolation must be within 3% of "correct" result,
						// add this as error term. (normally we'd expect a random distribution of
						// +-1.5% error, but nowhere in the spec does it say that the error has to be
						// unbiased - better safe than sorry).
						err += STBD_ABS(maxe - mine) * 3 / 100;

						if (err < bestErr)
						{
							Table[i * 2 + 0] = (uint8)mx;
							Table[i * 2 + 1] = (uint8)mn;
							bestErr = err;
						}
					}
				}
			}
		}

		static void stb__EvalColors(unsigned char* color, unsigned short c0, unsigned short c1)
		{
			stb__From16Bit(color + 0, c0);
			stb__From16Bit(color + 4, c1);
			stb__Lerp13RGB(color + 8, color + 0, color + 4);
			stb__Lerp13RGB(color + 12, color + 4, color + 0);
		}

		// Block dithering function. Simply dithers a block to 565 RGB.
		// (Floyd-Steinberg)
		static void stb__DitherBlock(unsigned char* dest, unsigned char* block)
		{
			int err[8], * ep1 = err, * ep2 = err + 4, * et;
			int ch, y;

			// process channels separately
			for (ch = 0; ch < 3; ++ch) {
				unsigned char* bp = block + ch, * dp = dest + ch;
				unsigned char* quant = (ch == 1) ? stb__QuantGTab + 8 : stb__QuantRBTab + 8;
				STBD_MEMSET(err, 0, sizeof(err));
				for (y = 0; y < 4; ++y) {
					dp[0] = quant[bp[0] + ((3 * ep2[1] + 5 * ep2[0]) >> 4)];
					ep1[0] = bp[0] - dp[0];
					dp[4] = quant[bp[4] + ((7 * ep1[0] + 3 * ep2[2] + 5 * ep2[1] + ep2[0]) >> 4)];
					ep1[1] = bp[4] - dp[4];
					dp[8] = quant[bp[8] + ((7 * ep1[1] + 3 * ep2[3] + 5 * ep2[2] + ep2[1]) >> 4)];
					ep1[2] = bp[8] - dp[8];
					dp[12] = quant[bp[12] + ((7 * ep1[2] + 5 * ep2[3] + ep2[2]) >> 4)];
					ep1[3] = bp[12] - dp[12];
					bp += 16;
					dp += 16;
					et = ep1, ep1 = ep2, ep2 = et; // swap
				}
			}
		}

		// The color matching function
		static unsigned int stb__MatchColorsBlock(unsigned char* block, unsigned char* color, int dither)
		{
			unsigned int mask = 0;
			int dirr = color[0 * 4 + 0] - color[1 * 4 + 0];
			int dirg = color[0 * 4 + 1] - color[1 * 4 + 1];
			int dirb = color[0 * 4 + 2] - color[1 * 4 + 2];
			int dots[16];
			int stops[4];
			int i;
			int c0Point, halfPoint, c3Point;

			for (i = 0; i < 16; i++)
				dots[i] = block[i * 4 + 0] * dirr + block[i * 4 + 1] * dirg + block[i * 4 + 2] * dirb;

			for (i = 0; i < 4; i++)
				stops[i] = color[i * 4 + 0] * dirr + color[i * 4 + 1] * dirg + color[i * 4 + 2] * dirb;

			// think of the colors as arranged on a line; project point onto that line, then choose
			// next color out of available ones. we compute the crossover points for "best color in top
			// half"/"best in bottom half" and then the same inside that subinterval.
			//
			// relying on this 1d approximation isn't always optimal in terms of euclidean distance,
			// but it's very close and a lot faster.
			// http://cbloomrants.blogspot.com/2008/12/12-08-08-dxtc-summary.html

			c0Point = (stops[1] + stops[3]) >> 1;
			halfPoint = (stops[3] + stops[2]) >> 1;
			c3Point = (stops[2] + stops[0]) >> 1;

			if (!dither) {
				// the version without dithering is straightforward
				for (i = 15; i >= 0; i--) {
					int dot = dots[i];
					mask <<= 2;

					if (dot < halfPoint)
						mask |= (dot < c0Point) ? 1 : 3;
					else
						mask |= (dot < c3Point) ? 2 : 0;
				}
			}
			else {
				// with floyd-steinberg dithering
				int err[8], * ep1 = err, * ep2 = err + 4;
				int* dp = dots, y;

				c0Point <<= 4;
				halfPoint <<= 4;
				c3Point <<= 4;
				for (i = 0; i < 8; i++)
					err[i] = 0;

				for (y = 0; y < 4; y++)
				{
					int dot, lmask, step;

					dot = (dp[0] << 4) + (3 * ep2[1] + 5 * ep2[0]);
					if (dot < halfPoint)
						step = (dot < c0Point) ? 1 : 3;
					else
						step = (dot < c3Point) ? 2 : 0;
					ep1[0] = dp[0] - stops[step];
					lmask = step;

					dot = (dp[1] << 4) + (7 * ep1[0] + 3 * ep2[2] + 5 * ep2[1] + ep2[0]);
					if (dot < halfPoint)
						step = (dot < c0Point) ? 1 : 3;
					else
						step = (dot < c3Point) ? 2 : 0;
					ep1[1] = dp[1] - stops[step];
					lmask |= step << 2;

					dot = (dp[2] << 4) + (7 * ep1[1] + 3 * ep2[3] + 5 * ep2[2] + ep2[1]);
					if (dot < halfPoint)
						step = (dot < c0Point) ? 1 : 3;
					else
						step = (dot < c3Point) ? 2 : 0;
					ep1[2] = dp[2] - stops[step];
					lmask |= step << 4;

					dot = (dp[3] << 4) + (7 * ep1[2] + 5 * ep2[3] + ep2[2]);
					if (dot < halfPoint)
						step = (dot < c0Point) ? 1 : 3;
					else
						step = (dot < c3Point) ? 2 : 0;
					ep1[3] = dp[3] - stops[step];
					lmask |= step << 6;

					dp += 4;
					mask |= lmask << (y * 8);
					{ int* et = ep1; ep1 = ep2; ep2 = et; } // swap
				}
			}

			return mask;
		}

		// The color optimization function. (Clever code, part 1)
		static void stb__OptimizeColorsBlock(unsigned char* block, unsigned short* pmax16, unsigned short* pmin16)
		{
			int mind = 0x7fffffff, maxd = -0x7fffffff;
			unsigned char* minp, * maxp;
			double magn;
			int v_r, v_g, v_b;
			static const int nIterPower = 4;
			float covf[6], vfr, vfg, vfb;

			// determine color distribution
			int cov[6];
			int mu[3], min[3], max[3];
			int ch, i, iter;

			for (ch = 0; ch < 3; ch++)
			{
				const unsigned char* bp = ((const unsigned char*)block) + ch;
				int muv, minv, maxv;

				muv = minv = maxv = bp[0];
				for (i = 4; i < 64; i += 4)
				{
					muv += bp[i];
					if (bp[i] < minv) minv = bp[i];
					else if (bp[i] > maxv) maxv = bp[i];
				}

				mu[ch] = (muv + 8) >> 4;
				min[ch] = minv;
				max[ch] = maxv;
			}

			// determine covariance matrix
			for (i = 0; i < 6; i++)
				cov[i] = 0;

			for (i = 0; i < 16; i++)
			{
				int r = block[i * 4 + 0] - mu[0];
				int g = block[i * 4 + 1] - mu[1];
				int b = block[i * 4 + 2] - mu[2];

				cov[0] += r * r;
				cov[1] += r * g;
				cov[2] += r * b;
				cov[3] += g * g;
				cov[4] += g * b;
				cov[5] += b * b;
			}

			// convert covariance matrix to float, find principal axis via power iter
			for (i = 0; i < 6; i++)
				covf[i] = cov[i] / 255.0f;

			vfr = (float)(max[0] - min[0]);
			vfg = (float)(max[1] - min[1]);
			vfb = (float)(max[2] - min[2]);

			for (iter = 0; iter < nIterPower; iter++)
			{
				float r = vfr * covf[0] + vfg * covf[1] + vfb * covf[2];
				float g = vfr * covf[1] + vfg * covf[3] + vfb * covf[4];
				float b = vfr * covf[2] + vfg * covf[4] + vfb * covf[5];

				vfr = r;
				vfg = g;
				vfb = b;
			}

			magn = STBD_FABS(vfr);
			if (STBD_FABS(vfg) > magn) magn = STBD_FABS(vfg);
			if (STBD_FABS(vfb) > magn) magn = STBD_FABS(vfb);

			if (magn < 4.0f) { // too small, default to luminance
				v_r = 299; // JPEG YCbCr luma coefs, scaled by 1000.
				v_g = 587;
				v_b = 114;
			}
			else {
				magn = 512.0 / magn;
				v_r = (int)(vfr * magn);
				v_g = (int)(vfg * magn);
				v_b = (int)(vfb * magn);
			}

			// Pick colors at extreme points
			minp = maxp = block;
			for (i = 0; i < 16; i++)
			{
				int dot = block[i * 4 + 0] * v_r + block[i * 4 + 1] * v_g + block[i * 4 + 2] * v_b;

				if (dot < mind) {
					mind = dot;
					minp = block + i * 4;
				}

				if (dot > maxd) {
					maxd = dot;
					maxp = block + i * 4;
				}
			}

			*pmax16 = stb__Aint16_tBit(maxp[0], maxp[1], maxp[2]);
			*pmin16 = stb__Aint16_tBit(minp[0], minp[1], minp[2]);
		}

		static int stb__sclamp(float y, int p0, int p1)
		{
			int x = (int)y;
			if (x < p0) return p0;
			if (x > p1) return p1;
			return x;
		}

		// The refinement function. (Clever code, part 2)
		// Tries to optimize colors to suit block contents better.
		// (By solving a least squares system via normal equations+Cramer's rule)
		static int stb__RefineBlock(unsigned char* block, unsigned short* pmax16, unsigned short* pmin16, unsigned int mask)
		{
			static const int w1Tab[4] = { 3,0,2,1 };
			static const int prods[4] = { 0x090000,0x000900,0x040102,0x010402 };
			// ^some magic to save a lot of multiplies in the accumulating loop...
			// (precomputed products of weights for least squares system, accumulated inside one 32-bit register)

			float frb, fg;
			unsigned short oldMin, oldMax, min16, max16;
			int i, akku = 0, xx, xy, yy;
			int At1_r, At1_g, At1_b;
			int At2_r, At2_g, At2_b;
			unsigned int cm = mask;

			oldMin = *pmin16;
			oldMax = *pmax16;

			if ((mask ^ (mask << 2)) < 4) // all pixels have the same index?
			{
				// yes, linear system would be singular; solve using optimal
				// single-color match on average color
				int r = 8, g = 8, b = 8;
				for (i = 0; i < 16; ++i) {
					r += block[i * 4 + 0];
					g += block[i * 4 + 1];
					b += block[i * 4 + 2];
				}

				r >>= 4; g >>= 4; b >>= 4;

				max16 = (stb__OMatch5[r][0] << 11) | (stb__OMatch6[g][0] << 5) | stb__OMatch5[b][0];
				min16 = (stb__OMatch5[r][1] << 11) | (stb__OMatch6[g][1] << 5) | stb__OMatch5[b][1];
			}
			else {
				At1_r = At1_g = At1_b = 0;
				At2_r = At2_g = At2_b = 0;
				for (i = 0; i < 16; ++i, cm >>= 2) {
					int step = cm & 3;
					int w1 = w1Tab[step];
					int r = block[i * 4 + 0];
					int g = block[i * 4 + 1];
					int b = block[i * 4 + 2];

					akku += prods[step];
					At1_r += w1 * r;
					At1_g += w1 * g;
					At1_b += w1 * b;
					At2_r += r;
					At2_g += g;
					At2_b += b;
				}

				At2_r = 3 * At2_r - At1_r;
				At2_g = 3 * At2_g - At1_g;
				At2_b = 3 * At2_b - At1_b;

				// extract solutions and decide solvability
				xx = akku >> 16;
				yy = (akku >> 8) & 0xff;
				xy = (akku >> 0) & 0xff;

				frb = 3.0f * 31.0f / 255.0f / (xx * yy - xy * xy);
				fg = frb * 63.0f / 31.0f;

				// solve.
				max16 = (uint16)stb__sclamp((At1_r * yy - At2_r * xy) * frb + 0.5f, 0, 31) << 11;
				max16 |= (uint16)stb__sclamp((At1_g * yy - At2_g * xy) * fg + 0.5f, 0, 63) << 5;
				max16 |= (uint16)stb__sclamp((At1_b * yy - At2_b * xy) * frb + 0.5f, 0, 31) << 0;

				min16 = (uint16)stb__sclamp((At2_r * xx - At1_r * xy) * frb + 0.5f, 0, 31) << 11;
				min16 |= (uint16)stb__sclamp((At2_g * xx - At1_g * xy) * fg + 0.5f, 0, 63) << 5;
				min16 |= (uint16)stb__sclamp((At2_b * xx - At1_b * xy) * frb + 0.5f, 0, 31) << 0;
			}

			*pmin16 = min16;
			*pmax16 = max16;
			return oldMin != min16 || oldMax != max16;
		}

		// Color block compression
		static void stb__CompressColorBlock(unsigned char* dest, unsigned char* block, int mode)
		{
			unsigned int mask;
			int i;
			int dither;
			int refinecount;
			unsigned short max16, min16;
			unsigned char dblock[16 * 4], color[4 * 4];

			dither = mode & STB_DXT_DITHER;
			refinecount = (mode & STB_DXT_HIGHQUAL) ? 2 : 1;

			// check if block is constant
			for (i = 1; i < 16; i++)
				if (((unsigned int*)block)[i] != ((unsigned int*)block)[0])
					break;

			if (i == 16)
			{ // constant color
				int r = block[0], g = block[1], b = block[2];
				mask = 0xaaaaaaaa;
				max16 = (stb__OMatch5[r][0] << 11) | (stb__OMatch6[g][0] << 5) | stb__OMatch5[b][0];
				min16 = (stb__OMatch5[r][1] << 11) | (stb__OMatch6[g][1] << 5) | stb__OMatch5[b][1];
			}
			else
			{
				// first step: compute dithered version for PCA if desired
				if (dither)
					stb__DitherBlock(dblock, block);

				// second step: pca+map along principal axis
				stb__OptimizeColorsBlock(dither ? dblock : block, &max16, &min16);
				if (max16 != min16)
				{
					stb__EvalColors(color, max16, min16);
					mask = stb__MatchColorsBlock(block, color, dither);
				}
				else
					mask = 0;

				// third step: refine (multiple times if requested)
				for (i = 0; i < refinecount; i++)
				{
					unsigned int lastmask = mask;

					if (stb__RefineBlock(dither ? dblock : block, &max16, &min16, mask))
					{
						if (max16 != min16)
						{
							stb__EvalColors(color, max16, min16);
							mask = stb__MatchColorsBlock(block, color, dither);
						}
						else
						{
							mask = 0;
							break;
						}
					}

					if (mask == lastmask)
						break;
				}
			}

			// write the color block
			if (max16 < min16)
			{
				unsigned short t = min16;
				min16 = max16;
				max16 = t;
				mask ^= 0x55555555;
			}

			dest[0] = (unsigned char)(max16);
			dest[1] = (unsigned char)(max16 >> 8);
			dest[2] = (unsigned char)(min16);
			dest[3] = (unsigned char)(min16 >> 8);
			dest[4] = (unsigned char)(mask);
			dest[5] = (unsigned char)(mask >> 8);
			dest[6] = (unsigned char)(mask >> 16);
			dest[7] = (unsigned char)(mask >> 24);
		}


		// Alpha block compression (this is easy for a change)
		static void stb__CompressAlphaBlock(unsigned char* dest, unsigned char* src, int stride)
		{
			int i, dist, bias, dist4, dist2, bits, mask;

			// find min/max color
			int mn, mx;
			mn = mx = src[0];

			for (i = 1; i < 16; i++)
			{
				if (src[i * stride] < mn) mn = src[i * stride];
				else if (src[i * stride] > mx) mx = src[i * stride];
			}

			// encode them
			((uint8*)dest)[0] = (uint8)mx;
			((uint8*)dest)[1] = (uint8)mn;
			dest += 2;

			// determine bias and emit color indices
			// given the choice of mx/mn, these indices are optimal:
			// http://fgiesen.wordpress.com/2009/12/15/dxt5-alpha-block-index-determination/
			dist = mx - mn;
			dist4 = dist * 4;
			dist2 = dist * 2;
			bias = (dist < 8) ? (dist - 1) : (dist / 2 + 2);
			bias -= mn * 7;
			bits = 0, mask = 0;

			for (i = 0; i < 16; i++)
			{
				int a = src[i * stride] * 7 + bias;
				int ind, t;

				// select index. this is a "linear scale" lerp factor between 0 (val=min) and 7 (val=max).
				t = (a >= dist4) ? -1 : 0; ind = t & 4; a -= dist4 & t;
				t = (a >= dist2) ? -1 : 0; ind += t & 2; a -= dist2 & t;
				ind += (a >= dist);

				// turn linear scale into DXT index (0/1 are extremal pts)
				ind = -ind & 7;
				ind ^= (2 > ind);

				// write index
				mask |= ind << bits;
				if ((bits += 3) >= 8) {
					*dest++ = (uint8)mask;
					mask >>= 8;
					bits -= 8;
				}
			}
		}


		static void stb__InitDXT()
		{
			int i;
			for (i = 0; i < 32; i++)
				stb__Expand5[i] = uint8((i << 3) | (i >> 2));

			for (i = 0; i < 64; i++)
				stb__Expand6[i] = uint8((i << 2) | (i >> 4));

			for (i = 0; i < 256 + 16; i++)
			{
				int v = i - 8 < 0 ? 0 : i - 8 > 255 ? 255 : i - 8;
				stb__QuantRBTab[i] = stb__Expand5[stb__Mul8Bit(v, 31)];
				stb__QuantGTab[i] = stb__Expand6[stb__Mul8Bit(v, 63)];
			}

			stb__PrepareOptTable(&stb__OMatch5[0][0], stb__Expand5, 32);
			stb__PrepareOptTable(&stb__OMatch6[0][0], stb__Expand6, 64);
		}


		void stb_compress_dxt_block(unsigned char* dest, const unsigned char* src, int alpha, int mode)
		{
			// If this check fails it means miro::initialize() has not been called.
			check(s_initialised_dxtc_decompress);

			unsigned char data[16][4];

			if (alpha > 0)
			{
				int i;
				if (alpha == 2)
				{
					CompressAlphaBtc2u(src, dest);
				}
				else
				{
					stb__CompressAlphaBlock(dest, (unsigned char*)src + 3, 4);
				}
				dest += 8;
				// make a new copy of the data in which alpha is opaque,
				// because code uses a fast test for color constancy
				FMemory::Memcpy(data, src, 4 * 16);
				for (i = 0; i < 16; ++i)
					data[i][3] = 255;
				src = &data[0][0];
			}

			stb__CompressColorBlock(dest, (unsigned char*)src, mode);
		}

		void stb_compress_bc4_block(unsigned char* dest, const unsigned char* src)
		{
			stb__CompressAlphaBlock(dest, (unsigned char*)src, 1);
		}

		void stb_compress_bc5_block(unsigned char* dest, const unsigned char* src)
		{
			stb__CompressAlphaBlock(dest, (unsigned char*)src, 2);
			stb__CompressAlphaBlock(dest + 8, (unsigned char*)src + 1, 2);
		}

	}


#endif //MIRO_INCLUDE_BC


#if MIRO_INCLUDE_ASTC

	//-------------------------------------------------------------------------------------------------
	// Based on ASTCRT, under the BSD 3-Clause License.
	// Modified for additional format support (alpha), and integration with UE platform libs.
	// See: https://github.com/daoo/astcrt
	//-------------------------------------------------------------------------------------------------

	namespace astcrt
	{
		constexpr int APPROX_COLOR_EPSILON = 0; // 50
		constexpr size_t BLOCK_BYTES = 16;

		constexpr size_t MAXIMUM_ENCODED_WEIGHT_BITS = 96;
		constexpr size_t MAXIMUM_ENCODED_WEIGHT_BYTES = 12;

		constexpr size_t MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES = 12;
		constexpr size_t MAX_BLOCK_WIDTH = 12;
		constexpr size_t MAX_BLOCK_HEIGHT = 12;


		FORCEINLINE bool getbit(size_t number, size_t n) {
			return (number >> n) & 1;
		}

		FORCEINLINE uint8 getbits(uint8 number, uint8 msb, uint8 lsb) {
			int count = msb - lsb + 1;
			return static_cast<uint8>((number >> lsb) & ((1 << count) - 1));
		}

		FORCEINLINE void orbits8_ptr(uint8* ptr,
			size_t bitoffset,
			size_t number,
			size_t bitcount) {
			miro_check(bitcount <= 8);
			miro_check((number >> bitcount) == 0);

			size_t index = bitoffset / 8;
			size_t shift = bitoffset % 8;

			// Depending on the offset we might have to consider two bytes when
			// writing, for instance if we are writing 8 bits and the offset is 4,
			// then we have to write 4 bits to the first byte (ptr[index]) and 4 bits
			// to the second byte (ptr[index+1]).
			//
			// FIXME: Writing to the last byte when the number of bytes is a multiple of 2
			// will write past the allocated memory.

			uint8* p = ptr + index;
			size_t mask = number << shift;

			miro_check((p[0] & mask) == 0);
			miro_check((p[1] & (mask >> 8)) == 0);

			p[0] |= static_cast<uint8>(mask & 0xFF);
			p[1] |= static_cast<uint8>((mask >> 8) & 0xFF);
		}

		inline uint16 getbytes2(const uint8* ptr, size_t byteoffset) {
			const uint8* p = ptr + byteoffset;
			return static_cast<uint16>((p[1] << 8) | p[0]);
		}

		inline void setbytes2(uint8* ptr, size_t byteoffset, uint16 bytes) {
			ptr[byteoffset + 0] = static_cast<uint8>(bytes & 0xFF);
			ptr[byteoffset + 1] = static_cast<uint8>((bytes >> 8) & 0xFF);
		}

		inline void split_high_low(uint8 n, size_t i, uint8& high, uint8& low) {
			miro_check(i < 8);

			uint8 low_mask = static_cast<uint8>((1 << i) - 1);

			low = n & low_mask;
			high = static_cast<uint8>(n >> i);
		}

		class bitwriter {
		public:
			explicit bitwriter(uint8* ptr) : ptr_(ptr), bitoffset_(0) {
				// assumption that all bits in ptr are zero after the offset

				// writing beyound the bounds of the allocated memory is undefined
				// behaviour
			}

			// Specialized function that can't write more than 8 bits.
			void write8(uint8 number, size_t bitcount) {
				orbits8_ptr(ptr_, bitoffset_, number, bitcount);

				bitoffset_ += bitcount;
			}

			size_t offset() const { return bitoffset_; }

		private:
			uint8* ptr_;
			size_t bitoffset_;  // in bits
		};

		constexpr uint8 bit_reverse_table[256] = {
			0x00, 0x80, 0x40, 0xC0, 0x20, 0xA0, 0x60, 0xE0, 0x10, 0x90, 0x50, 0xD0,
			0x30, 0xB0, 0x70, 0xF0, 0x08, 0x88, 0x48, 0xC8, 0x28, 0xA8, 0x68, 0xE8,
			0x18, 0x98, 0x58, 0xD8, 0x38, 0xB8, 0x78, 0xF8, 0x04, 0x84, 0x44, 0xC4,
			0x24, 0xA4, 0x64, 0xE4, 0x14, 0x94, 0x54, 0xD4, 0x34, 0xB4, 0x74, 0xF4,
			0x0C, 0x8C, 0x4C, 0xCC, 0x2C, 0xAC, 0x6C, 0xEC, 0x1C, 0x9C, 0x5C, 0xDC,
			0x3C, 0xBC, 0x7C, 0xFC, 0x02, 0x82, 0x42, 0xC2, 0x22, 0xA2, 0x62, 0xE2,
			0x12, 0x92, 0x52, 0xD2, 0x32, 0xB2, 0x72, 0xF2, 0x0A, 0x8A, 0x4A, 0xCA,
			0x2A, 0xAA, 0x6A, 0xEA, 0x1A, 0x9A, 0x5A, 0xDA, 0x3A, 0xBA, 0x7A, 0xFA,
			0x06, 0x86, 0x46, 0xC6, 0x26, 0xA6, 0x66, 0xE6, 0x16, 0x96, 0x56, 0xD6,
			0x36, 0xB6, 0x76, 0xF6, 0x0E, 0x8E, 0x4E, 0xCE, 0x2E, 0xAE, 0x6E, 0xEE,
			0x1E, 0x9E, 0x5E, 0xDE, 0x3E, 0xBE, 0x7E, 0xFE, 0x01, 0x81, 0x41, 0xC1,
			0x21, 0xA1, 0x61, 0xE1, 0x11, 0x91, 0x51, 0xD1, 0x31, 0xB1, 0x71, 0xF1,
			0x09, 0x89, 0x49, 0xC9, 0x29, 0xA9, 0x69, 0xE9, 0x19, 0x99, 0x59, 0xD9,
			0x39, 0xB9, 0x79, 0xF9, 0x05, 0x85, 0x45, 0xC5, 0x25, 0xA5, 0x65, 0xE5,
			0x15, 0x95, 0x55, 0xD5, 0x35, 0xB5, 0x75, 0xF5, 0x0D, 0x8D, 0x4D, 0xCD,
			0x2D, 0xAD, 0x6D, 0xED, 0x1D, 0x9D, 0x5D, 0xDD, 0x3D, 0xBD, 0x7D, 0xFD,
			0x03, 0x83, 0x43, 0xC3, 0x23, 0xA3, 0x63, 0xE3, 0x13, 0x93, 0x53, 0xD3,
			0x33, 0xB3, 0x73, 0xF3, 0x0B, 0x8B, 0x4B, 0xCB, 0x2B, 0xAB, 0x6B, 0xEB,
			0x1B, 0x9B, 0x5B, 0xDB, 0x3B, 0xBB, 0x7B, 0xFB, 0x07, 0x87, 0x47, 0xC7,
			0x27, 0xA7, 0x67, 0xE7, 0x17, 0x97, 0x57, 0xD7, 0x37, 0xB7, 0x77, 0xF7,
			0x0F, 0x8F, 0x4F, 0xCF, 0x2F, 0xAF, 0x6F, 0xEF, 0x1F, 0x9F, 0x5F, 0xDF,
			0x3F, 0xBF, 0x7F, 0xFF };

		/**
		 * Reverse a byte, total function.
		 */
		inline uint8 reverse_byte(uint8 number) {
			return bit_reverse_table[number];
		}

		/**
		 * Reverse a sequence of bytes.
		 *
		 * Assumes that the bits written to (using bitwise or) are zero and that they
		 * will not clash with bits already written to target sequence. That is it is
		 * possible to write to a non-zero byte as long as the bits that are actually
		 * written to are zero.
		 */
		inline void reverse_bytes(const uint8* source,
			size_t bytecount,
			uint8* target)
		{
			for (int i = 0; i < static_cast<int>(bytecount); ++i)
			{
				miro_check((reverse_byte(source[i]) & target[-i]) == 0);
				target[-i] = target[-i] | reverse_byte(source[i]);
			}
		}

		inline void copy_bytes(const uint8* source, size_t bytecount, uint8* target, size_t bitoffset)
		{
			for (size_t i = 0; i < bytecount; ++i)
			{
				orbits8_ptr(target, bitoffset + i * 8, source[i], 8);
			}
		}


		union unorm8_t
		{
			struct RgbaColorType
			{
				uint8 b, g, r, a;
			} channels;
			uint8 components[4];
			uint32 bits;
		};

		union unorm16_t
		{
			struct RgbaColorType
			{
				uint16 b, g, r, a;
			} channels;
			uint16 components[4];
			uint64 bits;
		};



		template <typename T>
		struct vec3_t
		{
		public:
			vec3_t() {}
			vec3_t(T x_, T y_, T z_) : r(x_), g(y_), b(z_) {}
			T& components(size_t i) { return ((T*)this)[i]; }
			const T& components(size_t i) const { return ((T*)this)[i]; }
			T r, g, b;
		};

		typedef vec3_t<float> vec3f_t;
		typedef vec3_t<int> vec3i_t;

		template <typename T>
		struct vec4_t
		{
		public:
			vec4_t() {}
			vec4_t(T x_, T y_, T z_, T w_) : r(x_), g(y_), b(z_), a(w_) {}
			vec3_t<T> rgb() const { return vec3_t<T>(r, g, b); }
			T& components(size_t i) { return ((T*)this)[i]; }
			const T& components(size_t i) const { return ((T*)this)[i]; }
			T r, g, b, a;
		};

		typedef vec4_t<float> vec4f_t;
		typedef vec4_t<int> vec4i_t;

		template <typename T>
		vec3_t<T> operator+(vec3_t<T> a, vec3_t<T> b)
		{
			vec3_t<T> result;
			result.r = a.r + b.r;
			result.g = a.g + b.g;
			result.b = a.b + b.b;
			return result;
		}

		template <typename T>
		vec4_t<T> operator+(vec4_t<T> a, vec4_t<T> b)
		{
			vec4_t<T> result;
			result.r = a.r + b.r;
			result.g = a.g + b.g;
			result.b = a.b + b.b;
			result.a = a.a + b.a;
			return result;
		}

		template <typename T>
		vec3_t<T> operator-(vec3_t<T> a, vec3_t<T> b)
		{
			vec3_t<T> result;
			result.r = a.r - b.r;
			result.g = a.g - b.g;
			result.b = a.b - b.b;
			return result;
		}

		template <typename T>
		vec4_t<T> operator-(vec4_t<T> a, vec4_t<T> b)
		{
			vec4_t<T> result;
			result.r = a.r - b.r;
			result.g = a.g - b.g;
			result.b = a.b - b.b;
			result.a = a.a - b.a;
			return result;
		}

		template <typename T>
		vec3_t<T> operator*(vec3_t<T> a, vec3_t<T> b)
		{
			vec3_t<T> result;
			result.r = a.r * b.r;
			result.g = a.g * b.g;
			result.b = a.b * b.b;
			return result;
		}

		template <typename T>
		vec4_t<T> operator*(vec4_t<T> a, vec4_t<T> b)
		{
			vec4_t<T> result;
			result.r = a.r * b.r;
			result.g = a.g * b.g;
			result.b = a.b * b.b;
			result.a = a.a * b.a;
			return result;
		}

		template <typename T>
		vec4_t<T> operator/(vec4_t<T> a, vec4_t<T> b)
		{
			vec4_t<T> result;
			result.r = a.r / b.r;
			result.g = a.g / b.g;
			result.b = a.b / b.b;
			result.a = a.a / b.a;
			return result;
		}

		template <typename T>
		vec3_t<T> operator*(vec3_t<T> a, T b)
		{
			vec3_t<T> result;
			result.r = a.r * b;
			result.g = a.g * b;
			result.b = a.b * b;
			return result;
		}

		template <typename T>
		vec4_t<T> operator*(vec4_t<T> a, T b)
		{
			vec4_t<T> result;
			result.r = a.r * b;
			result.g = a.g * b;
			result.b = a.b * b;
			result.a = a.a * b;
			return result;
		}

		template <typename T>
		vec3_t<T> operator/(vec3_t<T> a, T b)
		{
			vec3_t<T> result;
			result.r = a.r / b;
			result.g = a.g / b;
			result.b = a.b / b;
			return result;
		}

		template <typename T>
		vec4_t<T> operator/(vec4_t<T> a, T b)
		{
			vec4_t<T> result;
			result.r = a.r / b;
			result.g = a.g / b;
			result.b = a.b / b;
			result.a = a.a / b;
			return result;
		}

		template <typename T>
		vec3_t<T> operator/(vec3_t<T> a, vec3_t<T> b)
		{
			vec3_t<T> result;
			result.x = a.x / b.x;
			result.y = a.y / b.y;
			result.z = a.z / b.z;
			return result;
		}

		template <typename T>
		bool operator==(vec3_t<T> a, vec3_t<T> b)
		{
			return a.r == b.r && a.g == b.g && a.b == b.b;
		}

		template <typename T>
		bool operator==(vec4_t<T> a, vec4_t<T> b)
		{
			return a.r == b.r && a.g == b.g && a.b == b.b && a.a == b.a;
		}

		template <typename T>
		bool operator!=(vec3_t<T> a, vec3_t<T> b)
		{
			return a.r != b.r || a.g != b.g || a.b != b.b;
		}

		template <typename T>
		T dot(vec3_t<T> a, vec3_t<T> b)
		{
			return a.r * b.r + a.g * b.g + a.b * b.b;
		}
		template <typename T>
		T dot(vec4_t<T> a, vec4_t<T> b)
		{
			return a.r * b.r + a.g * b.g + a.b * b.b + a.a * b.a;
		}

		template <typename T>
		T quadrance(vec3_t<T> a)
		{
			return dot(a, a);
		}

		template <typename T>
		T quadrance(vec4_t<T> a)
		{
			return dot(a, a);
		}

		template <typename T>
		T norm(vec3_t<T> a)
		{
			return static_cast<T>(sqrt(quadrance(a)));
		}

		template <typename T>
		T norm(vec4_t<T> a)
		{
			return static_cast<T>(sqrt(quadrance(a)));
		}

		template <typename T>
		T distance(vec3_t<T> a, vec3_t<T> b)
		{
			return norm(a - b);
		}

		template <typename T>
		T qd(vec3_t<T> a, vec3_t<T> b)
		{
			return quadrance(a - b);
		}

		template <typename T>
		vec3_t<T> signorm(vec3_t<T> a)
		{
			T x = norm(a);

			// Safety fix for degenerated cases.
			// \todo This should be intercepted earlier.
			// miro_check(x != 0.0);
			if (x == 0.0)
				return vec3_t<T>(0, 1, 0);

			return a / x;
		}

		template <typename T>
		vec4_t<T> signorm(vec4_t<T> a)
		{
			T x = norm(a);

			// Safety fix for degenerated cases.
			// \todo This should be intercepted earlier.
			// miro_check(x != 0.0);
			if (x == 0.0)
				return vec4_t<T>(0, 1, 0, 0);

			return a / x;
		}

		template <typename T>
		vec3_t<T> vecmin(vec3_t<T> a, vec3_t<T> b)
		{
			vec3_t<T> result;
			result.x = FMath::Min(a.x, b.x);
			result.y = FMath::Min(a.y, b.y);
			result.z = FMath::Min(a.z, b.z);
			return result;
		}

		template <typename T>
		vec3_t<T> vecmax(vec3_t<T> a, vec3_t<T> b)
		{
			vec3_t<T> result;
			result.x = FMath::Max(a.x, b.x);
			result.y = FMath::Max(a.y, b.y);
			result.z = FMath::Max(a.z, b.z);
			return result;
		}

		template <typename T>
		T qd_to_line(vec3_t<T> m, vec3_t<T> k, T kk, vec3_t<T> p)
		{
			T t = dot(p - m, k) / kk;
			vec3_t<T> q = k * t + m;
			return qd(p, q);
		}


		inline bool is_greyscale(vec3i_t color)
		{
			// integer equality is transitive
			return color.r == color.g && color.g == color.b;
		}

		inline int luminance(vec3i_t color)
		{
			return (color.r + color.g + color.b) / 3;
		}

		inline bool approx_equal(vec3i_t a, vec3i_t b, int32 Epsilon)
		{
			return quadrance(a - b) <= Epsilon;
		}

		inline bool approx_equal(vec4i_t a, vec4i_t b, int32 Epsilon)
		{
			return quadrance(a - b) <= Epsilon;
		}

		inline vec3i_t clamp_rgb(vec3i_t color)
		{
			vec3i_t result;
			result.r = FMath::Clamp(color.r, 0, 255);
			result.g = FMath::Clamp(color.g, 0, 255);
			result.b = FMath::Clamp(color.b, 0, 255);
			return result;
		}

		inline vec3f_t clamp_rgb(vec3f_t color)
		{
			vec3f_t result;
			result.r = FMath::Clamp(color.r, 0.0f, 255.0f);
			result.g = FMath::Clamp(color.g, 0.0f, 255.0f);
			result.b = FMath::Clamp(color.b, 0.0f, 255.0f);
			return result;
		}

		inline vec4f_t clamp_rgba(vec4f_t color)
		{
			vec4f_t result;
			result.r = FMath::Clamp(color.r, 0.0f, 255.0f);
			result.g = FMath::Clamp(color.g, 0.0f, 255.0f);
			result.b = FMath::Clamp(color.b, 0.0f, 255.0f);
			result.a = FMath::Clamp(color.a, 0.0f, 255.0f);
			return result;
		}

		inline bool is_rgb(float color)
		{
			return color >= 0.0f && color <= 255.0f;
		}

		inline bool is_rgb(vec3f_t color)
		{
			return is_rgb(color.r) && is_rgb(color.g) && is_rgb(color.b);
		}

		inline vec3i_t floor(vec3f_t color)
		{
			vec3i_t result;
			result.r = static_cast<int>(FMath::Floor(color.r));
			result.g = static_cast<int>(FMath::Floor(color.g));
			result.b = static_cast<int>(FMath::Floor(color.b));
			return result;
		}

		inline vec3i_t round(vec3f_t color)
		{
			vec3i_t result;
			result.r = static_cast<int>(FMath::RoundToInt32(color.r));
			result.g = static_cast<int>(FMath::RoundToInt32(color.g));
			result.b = static_cast<int>(FMath::RoundToInt32(color.b));
			return result;
		}

		inline vec4i_t round(vec4f_t color)
		{
			vec4i_t result;
			result.r = static_cast<int>(FMath::RoundToInt32(color.r));
			result.g = static_cast<int>(FMath::RoundToInt32(color.g));
			result.b = static_cast<int>(FMath::RoundToInt32(color.b));
			result.a = static_cast<int>(FMath::RoundToInt32(color.a));
			return result;
		}

		inline vec3i_t to_vec3i(unorm8_t color)
		{
			vec3i_t result;
			result.r = color.channels.r;
			result.g = color.channels.g;
			result.b = color.channels.b;
			return result;
		}

		inline vec4i_t to_vec4i(unorm8_t color)
		{
			vec4i_t result;
			result.r = color.channels.r;
			result.g = color.channels.g;
			result.b = color.channels.b;
			result.a = color.channels.a;
			return result;
		}

		inline vec3i_t to_vec3i(vec3f_t color)
		{
			vec3i_t result;
			result.r = static_cast<int>(color.r);
			result.g = static_cast<int>(color.g);
			result.b = static_cast<int>(color.b);
			return result;
		}

		inline vec3f_t to_vec3f(unorm8_t color)
		{
			vec3f_t result;
			result.r = color.channels.r;
			result.g = color.channels.g;
			result.b = color.channels.b;
			return result;
		}

		inline vec4f_t to_vec4f(unorm8_t color)
		{
			vec4f_t result;
			result.r = color.channels.r;
			result.g = color.channels.g;
			result.b = color.channels.b;
			result.a = color.channels.a;
			return result;
		}

		inline vec3f_t to_vec3f(vec3i_t color)
		{
			vec3f_t result;
			result.r = static_cast<float>(color.r);
			result.g = static_cast<float>(color.g);
			result.b = static_cast<float>(color.b);
			return result;
		}

		inline vec4f_t to_vec4f(vec4i_t color)
		{
			vec4f_t result;
			result.r = static_cast<float>(color.r);
			result.g = static_cast<float>(color.g);
			result.b = static_cast<float>(color.b);
			result.a = static_cast<float>(color.a);
			return result;
		}

		inline unorm8_t to_unorm8(vec3i_t color)
		{
			unorm8_t result;
			result.channels.r = static_cast<uint8>(color.r);
			result.channels.g = static_cast<uint8>(color.g);
			result.channels.b = static_cast<uint8>(color.b);
			result.channels.a = 255;
			return result;
		}

		inline unorm8_t to_unorm8(vec4i_t color)
		{
			unorm8_t result;
			result.channels.r = static_cast<uint8>(color.r);
			result.channels.g = static_cast<uint8>(color.g);
			result.channels.b = static_cast<uint8>(color.b);
			result.channels.a = static_cast<uint8>(color.a);
			return result;
		}

		inline unorm16_t unorm8_to_unorm16(unorm8_t c8)
		{
			// (x / 255) * (2^16-1) = x * 65535 / 255 = x * 257
			unorm16_t result;
			result.channels.r = static_cast<uint16>(c8.channels.r * 257);
			result.channels.g = static_cast<uint16>(c8.channels.g * 257);
			result.channels.b = static_cast<uint16>(c8.channels.b * 257);
			result.channels.a = static_cast<uint16>(c8.channels.a * 257);
			return result;
		}

		struct PhysicalBlock
		{
			uint8 data[BLOCK_BYTES];
		};

		inline void void_extent_to_physical(unorm16_t color, PhysicalBlock* pb)
		{
			pb->data[0] = 0xFC;
			pb->data[1] = 0xFD;
			pb->data[2] = 0xFF;
			pb->data[3] = 0xFF;
			pb->data[4] = 0xFF;
			pb->data[5] = 0xFF;
			pb->data[6] = 0xFF;
			pb->data[7] = 0xFF;

			setbytes2(pb->data, 8, color.channels.r);
			setbytes2(pb->data, 10, color.channels.g);
			setbytes2(pb->data, 12, color.channels.b);
			setbytes2(pb->data, 14, color.channels.a);
		}

		enum color_endpoint_mode_t
		{
			CEM_LDR_LUMINANCE_DIRECT = 0,
			CEM_LDR_LUMINANCE_BASE_OFFSET = 1,
			CEM_HDR_LUMINANCE_LARGE_RANGE = 2,
			CEM_HDR_LUMINANCE_SMALL_RANGE = 3,
			CEM_LDR_LUMINANCE_ALPHA_DIRECT = 4,
			CEM_LDR_LUMINANCE_ALPHA_BASE_OFFSET = 5,
			CEM_LDR_RGB_BASE_SCALE = 6,
			CEM_HDR_RGB_BASE_SCALE = 7,
			CEM_LDR_RGB_DIRECT = 8,
			CEM_LDR_RGB_BASE_OFFSET = 9,
			CEM_LDR_RGB_BASE_SCALE_PLUS_TWO_ALPHA = 10,
			CEM_HDR_RGB = 11,
			CEM_LDR_RGBA_DIRECT = 12,
			CEM_LDR_RGBA_BASE_OFFSET = 13,
			CEM_HDR_RGB_LDR_ALPHA = 14,
			CEM_HDR_RGB_HDR_ALPHA = 15,
			CEM_MAX = 16
		};

		constexpr uint8 cem_value_count[CEM_MAX] = { 2,2,2,2, 4,4,4,4, 6,6,6,6, 8,8,8,8 };


		/**
		 * Define normalized (starting at zero) numeric ranges that can be represented
		 * with 8 bits or less.
		 */
		enum range_t
		{
			RANGE_2,
			RANGE_3,
			RANGE_4,
			RANGE_5,
			RANGE_6,
			RANGE_8,
			RANGE_10,
			RANGE_12,
			RANGE_16,
			RANGE_20,
			RANGE_24,
			RANGE_32,
			RANGE_40,
			RANGE_48,
			RANGE_64,
			RANGE_80,
			RANGE_96,
			RANGE_128,
			RANGE_160,
			RANGE_192,
			RANGE_256,
			RANGE_MAX
		};

#ifdef UE_MIRO_DEBUG
		/**
		 * Table of maximum value for each range, minimum is always zero.
		 */
		const uint8 range_max_table[RANGE_MAX] = { 1,  2,  3,  4,   5,   7,   9,
													11, 15, 19, 23,  31,  39,  47,
													63, 79, 95, 127, 159, 191, 255 };
#endif




		const uint8 integer_from_trits[3][3][3][3][3] =
		{
			{
				{{{0,1,2},{4,5,6},{8,9,10}},{{16,17,18},{20,21,22},{24,25,26}},{{3,7,15},{19,23,27},{12,13,14}}},
				{{{32,33,34},{36,37,38},{40,41,42}},{{48,49,50},{52,53,54},{56,57,58}},{{35,39,47},{51,55,59},{44,45,46}}},
				{{{64,65,66},{68,69,70},{72,73,74}},{{80,81,82},{84,85,86},{88,89,90}},{{67,71,79},{83,87,91},{76,77,78}}}
			},
			{
				{{{128,129,130},{132,133,134},{136,137,138}},{{144,145,146},{148,149,150},{152,153,154}},{{131,135,143},{147,151,155},{140,141,142}}},
				{{{160,161,162},{164,165,166},{168,169,170}},{{176,177,178},{180,181,182},{184,185,186}},{{163,167,175},{179,183,187},{172,173,174}}},
				{{{192,193,194},{196,197,198},{200,201,202}},{{208,209,210},{212,213,214},{216,217,218}},{{195,199,207},{211,215,219},{204,205,206}}}
			},
			{
				{{{96,97,98},{100,101,102},{104,105,106}},{{112,113,114},{116,117,118},{120,121,122}},{{99,103,111},{115,119,123},{108,109,110}}},
				{{{224,225,226},{228,229,230},{232,233,234}},{{240,241,242},{244,245,246},{248,249,250}},{{227,231,239},{243,247,251},{236,237,238}}},
				{{{28,29,30},{60,61,62},{92,93,94}},{{156,157,158},{188,189,190},{220,221,222}},{{31,63,127},{159,191,255},{252,253,254}}}
			}
		};

		const uint8 integer_from_quints[5][5][5] =
		{
			{{0,1,2,3,4},{8,9,10,11,12},{16,17,18,19,20},{24,25,26,27,28},{5,13,21,29,6}},
			{{32,33,34,35,36},{40,41,42,43,44},{48,49,50,51,52},{56,57,58,59,60},{37,45,53,61,14}},
			{{64,65,66,67,68},{72,73,74,75,76},{80,81,82,83,84},{88,89,90,91,92},{69,77,85,93,22}},
			{{96,97,98,99,100},{104,105,106,107,108},{112,113,114,115,116},{120,121,122,123,124},{101,109,117,125,30}},
			{{102,103,70,71,38},{110,111,78,79,46},{118,119,86,87,54},{126,127,94,95,62},{39,47,55,63,31}}
		};


		const uint8 color_unquantize_table[21][256] =
		{
			{0,255},
			{0,128,255},
			{0,85,170,255},
			{0,64,128,192,255},
			{0,255,51,204,102,153},
			{0,36,73,109,146,182,219,255},
			{0,255,28,227,56,199,84,171,113,142},
			{0,255,69,186,23,232,92,163,46,209,116,139},
			{0,17,34,51,68,85,102,119,136,153,170,187,204,221,238,255},
			{0,255,67,188,13,242,80,175,27,228,94,161,40,215,107,148,54,201,121,134},
			{0,255,33,222,66,189,99,156,11,244,44,211,77,178,110,145,22,233,55,200,88,167,121,134},
			{0,8,16,24,33,41,49,57,66,74,82,90,99,107,115,123,132,140,148,156,165,173,181,189,198,206,214,222,231,239,247,255},
			{0,255,32,223,65,190,97,158,6,249,39,216,71,184,104,151,13,242,45,210,78,177,110,145,19,236,52,203,84,171,117,138,26,229,58,197,91,164,123,132},
			{0,255,16,239,32,223,48,207,65,190,81,174,97,158,113,142,5,250,21,234,38,217,54,201,70,185,86,169,103,152,119,136,11,244,27,228,43,212,59,196,76,179,92,163,108,147,124,131},
			{0,4,8,12,16,20,24,28,32,36,40,44,48,52,56,60,65,69,73,77,81,85,89,93,97,101,105,109,113,117,121,125,130,134,138,142,146,150,154,158,162,166,170,174,178,182,186,190,195,199,203,207,211,215,219,223,227,231,235,239,243,247,251,255},
			{0,255,16,239,32,223,48,207,64,191,80,175,96,159,112,143,3,252,19,236,35,220,51,204,67,188,83,172,100,155,116,139,6,249,22,233,38,217,54,201,71,184,87,168,103,152,119,136,9,246,25,230,42,213,58,197,74,181,90,165,106,149,122,133,13,242,29,226,45,210,61,194,77,178,93,162,109,146,125,130},
			{0,255,8,247,16,239,24,231,32,223,40,215,48,207,56,199,64,191,72,183,80,175,88,167,96,159,104,151,112,143,120,135,2,253,10,245,18,237,26,229,35,220,43,212,51,204,59,196,67,188,75,180,83,172,91,164,99,156,107,148,115,140,123,132,5,250,13,242,21,234,29,226,37,218,45,210,53,202,61,194,70,185,78,177,86,169,94,161,102,153,110,145,118,137,126,129},
			{0,2,4,6,8,10,12,14,16,18,20,22,24,26,28,30,32,34,36,38,40,42,44,46,48,50,52,54,56,58,60,62,64,66,68,70,72,74,76,78,80,82,84,86,88,90,92,94,96,98,100,102,104,106,108,110,112,114,116,118,120,122,124,126,129,131,133,135,137,139,141,143,145,147,149,151,153,155,157,159,161,163,165,167,169,171,173,175,177,179,181,183,185,187,189,191,193,195,197,199,201,203,205,207,209,211,213,215,217,219,221,223,225,227,229,231,233,235,237,239,241,243,245,247,249,251,253,255},
			{0,255,8,247,16,239,24,231,32,223,40,215,48,207,56,199,64,191,72,183,80,175,88,167,96,159,104,151,112,143,120,135,1,254,9,246,17,238,25,230,33,222,41,214,49,206,57,198,65,190,73,182,81,174,89,166,97,158,105,150,113,142,121,134,3,252,11,244,19,236,27,228,35,220,43,212,51,204,59,196,67,188,75,180,83,172,91,164,99,156,107,148,115,140,123,132,4,251,12,243,20,235,28,227,36,219,44,211,52,203,60,195,68,187,76,179,84,171,92,163,100,155,108,147,116,139,124,131,6,249,14,241,22,233,30,225,38,217,46,209,54,201,62,193,70,185,78,177,86,169,94,161,102,153,110,145,118,137,126,129},
			{0,255,4,251,8,247,12,243,16,239,20,235,24,231,28,227,32,223,36,219,40,215,44,211,48,207,52,203,56,199,60,195,64,191,68,187,72,183,76,179,80,175,84,171,88,167,92,163,96,159,100,155,104,151,108,147,112,143,116,139,120,135,124,131,1,254,5,250,9,246,13,242,17,238,21,234,25,230,29,226,33,222,37,218,41,214,45,210,49,206,53,202,57,198,61,194,65,190,69,186,73,182,77,178,81,174,85,170,89,166,93,162,97,158,101,154,105,150,109,146,113,142,117,138,121,134,125,130,2,253,6,249,10,245,14,241,18,237,22,233,26,229,30,225,34,221,38,217,42,213,46,209,50,205,54,201,58,197,62,193,66,189,70,185,74,181,78,177,82,173,86,169,90,165,94,161,98,157,102,153,106,149,110,145,114,141,118,137,122,133,126,129},
			{0,1,2,3,4,5,6,7,8,9,10,11,12,13,14,15,16,17,18,19,20,21,22,23,24,25,26,27,28,29,30,31,32,33,34,35,36,37,38,39,40,41,42,43,44,45,46,47,48,49,50,51,52,53,54,55,56,57,58,59,60,61,62,63,64,65,66,67,68,69,70,71,72,73,74,75,76,77,78,79,80,81,82,83,84,85,86,87,88,89,90,91,92,93,94,95,96,97,98,99,100,101,102,103,104,105,106,107,108,109,110,111,112,113,114,115,116,117,118,119,120,121,122,123,124,125,126,127,128,129,130,131,132,133,134,135,136,137,138,139,140,141,142,143,144,145,146,147,148,149,150,151,152,153,154,155,156,157,158,159,160,161,162,163,164,165,166,167,168,169,170,171,172,173,174,175,176,177,178,179,180,181,182,183,184,185,186,187,188,189,190,191,192,193,194,195,196,197,198,199,200,201,202,203,204,205,206,207,208,209,210,211,212,213,214,215,216,217,218,219,220,221,222,223,224,225,226,227,228,229,230,231,232,233,234,235,236,237,238,239,240,241,242,243,244,245,246,247,248,249,250,251,252,253,254,255}
		};

		const uint8 color_quantize_table[21][256] =
		{
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1},
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2},
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3},
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4},
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1},
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7},
			{0,0,0,0,0,0,0,0,0,0,0,0,0,0,0,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1},
			{0,0,0,0,0,0,0,0,0,0,0,0,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,1,1,1,1,1,1,1,1,1,1,1,1},
			{0,0,0,0,0,0,0,0,0,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,1,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,2,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,3,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,4,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,5,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,6,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,7,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,8,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,9,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,10,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,11,12,12,12,12,12,12,12,12,12,12,12,12,12,12,12,12,12,13,13,13,13,13,13,13,13,13,13,13,13,13,13,13,13,13,14,14,14,14,14,14,14,14,14,14,14,14,14,14,14,14,14,15,15,15,15,15,15,15,15,15},
			{0,0,0,0,0,0,0,4,4,4,4,4,4,4,4,4,4,4,4,4,4,8,8,8,8,8,8,8,8,8,8,8,8,8,12,12,12,12,12,12,12,12,12,12,12,12,12,12,16,16,16,16,16,16,16,16,16,16,16,16,16,2,2,2,2,2,2,2,2,2,2,2,2,2,6,6,6,6,6,6,6,6,6,6,6,6,6,6,10,10,10,10,10,10,10,10,10,10,10,10,10,14,14,14,14,14,14,14,14,14,14,14,14,14,14,18,18,18,18,18,18,18,18,18,18,18,18,18,19,19,19,19,19,19,19,19,19,19,19,19,19,15,15,15,15,15,15,15,15,15,15,15,15,15,15,11,11,11,11,11,11,11,11,11,11,11,11,11,7,7,7,7,7,7,7,7,7,7,7,7,7,7,3,3,3,3,3,3,3,3,3,3,3,3,3,17,17,17,17,17,17,17,17,17,17,17,17,17,13,13,13,13,13,13,13,13,13,13,13,13,13,13,9,9,9,9,9,9,9,9,9,9,9,9,9,5,5,5,5,5,5,5,5,5,5,5,5,5,5,1,1,1,1,1,1,1},
			{0,0,0,0,0,0,8,8,8,8,8,8,8,8,8,8,8,16,16,16,16,16,16,16,16,16,16,16,2,2,2,2,2,2,2,2,2,2,2,10,10,10,10,10,10,10,10,10,10,10,18,18,18,18,18,18,18,18,18,18,18,4,4,4,4,4,4,4,4,4,4,4,12,12,12,12,12,12,12,12,12,12,12,20,20,20,20,20,20,20,20,20,20,20,6,6,6,6,6,6,6,6,6,6,6,14,14,14,14,14,14,14,14,14,14,14,22,22,22,22,22,22,22,22,22,22,22,22,23,23,23,23,23,23,23,23,23,23,23,23,15,15,15,15,15,15,15,15,15,15,15,7,7,7,7,7,7,7,7,7,7,7,21,21,21,21,21,21,21,21,21,21,21,13,13,13,13,13,13,13,13,13,13,13,5,5,5,5,5,5,5,5,5,5,5,19,19,19,19,19,19,19,19,19,19,19,11,11,11,11,11,11,11,11,11,11,11,3,3,3,3,3,3,3,3,3,3,3,17,17,17,17,17,17,17,17,17,17,17,9,9,9,9,9,9,9,9,9,9,9,1,1,1,1,1,1},
			{0,0,0,0,0,1,1,1,1,1,1,1,1,2,2,2,2,2,2,2,2,3,3,3,3,3,3,3,3,4,4,4,4,4,4,4,4,4,5,5,5,5,5,5,5,5,6,6,6,6,6,6,6,6,7,7,7,7,7,7,7,7,8,8,8,8,8,8,8,8,8,9,9,9,9,9,9,9,9,10,10,10,10,10,10,10,10,11,11,11,11,11,11,11,11,12,12,12,12,12,12,12,12,12,13,13,13,13,13,13,13,13,14,14,14,14,14,14,14,14,15,15,15,15,15,15,15,15,16,16,16,16,16,16,16,16,16,17,17,17,17,17,17,17,17,18,18,18,18,18,18,18,18,19,19,19,19,19,19,19,19,20,20,20,20,20,20,20,20,20,21,21,21,21,21,21,21,21,22,22,22,22,22,22,22,22,23,23,23,23,23,23,23,23,24,24,24,24,24,24,24,24,24,25,25,25,25,25,25,25,25,26,26,26,26,26,26,26,26,27,27,27,27,27,27,27,27,28,28,28,28,28,28,28,28,28,29,29,29,29,29,29,29,29,30,30,30,30,30,30,30,30,31,31,31,31},
			{0,0,0,0,8,8,8,8,8,8,16,16,16,16,16,16,16,24,24,24,24,24,24,32,32,32,32,32,32,2,2,2,2,2,2,2,10,10,10,10,10,10,10,18,18,18,18,18,18,26,26,26,26,26,26,26,34,34,34,34,34,34,4,4,4,4,4,4,4,12,12,12,12,12,12,20,20,20,20,20,20,20,28,28,28,28,28,28,36,36,36,36,36,36,6,6,6,6,6,6,6,14,14,14,14,14,14,14,22,22,22,22,22,22,30,30,30,30,30,30,30,38,38,38,38,38,38,38,39,39,39,39,39,39,39,31,31,31,31,31,31,31,23,23,23,23,23,23,15,15,15,15,15,15,15,7,7,7,7,7,7,7,37,37,37,37,37,37,29,29,29,29,29,29,21,21,21,21,21,21,21,13,13,13,13,13,13,5,5,5,5,5,5,5,35,35,35,35,35,35,27,27,27,27,27,27,27,19,19,19,19,19,19,11,11,11,11,11,11,11,3,3,3,3,3,3,3,33,33,33,33,33,33,25,25,25,25,25,25,17,17,17,17,17,17,17,9,9,9,9,9,9,1,1,1,1},
			{0,0,0,16,16,16,16,16,16,32,32,32,32,32,2,2,2,2,2,18,18,18,18,18,18,34,34,34,34,34,4,4,4,4,4,4,20,20,20,20,20,36,36,36,36,36,6,6,6,6,6,6,22,22,22,22,22,38,38,38,38,38,8,8,8,8,8,8,24,24,24,24,24,24,40,40,40,40,40,10,10,10,10,10,26,26,26,26,26,26,42,42,42,42,42,12,12,12,12,12,12,28,28,28,28,28,44,44,44,44,44,14,14,14,14,14,14,30,30,30,30,30,46,46,46,46,46,46,47,47,47,47,47,47,31,31,31,31,31,15,15,15,15,15,15,45,45,45,45,45,29,29,29,29,29,13,13,13,13,13,13,43,43,43,43,43,27,27,27,27,27,27,11,11,11,11,11,41,41,41,41,41,25,25,25,25,25,25,9,9,9,9,9,9,39,39,39,39,39,23,23,23,23,23,7,7,7,7,7,7,37,37,37,37,37,21,21,21,21,21,5,5,5,5,5,5,35,35,35,35,35,19,19,19,19,19,19,3,3,3,3,3,33,33,33,33,33,17,17,17,17,17,17,1,1,1},
			{0,0,0,1,1,1,1,2,2,2,2,3,3,3,3,4,4,4,4,5,5,5,5,6,6,6,6,7,7,7,7,8,8,8,8,9,9,9,9,10,10,10,10,11,11,11,11,12,12,12,12,13,13,13,13,14,14,14,14,15,15,15,15,16,16,16,16,16,17,17,17,17,18,18,18,18,19,19,19,19,20,20,20,20,21,21,21,21,22,22,22,22,23,23,23,23,24,24,24,24,25,25,25,25,26,26,26,26,27,27,27,27,28,28,28,28,29,29,29,29,30,30,30,30,31,31,31,31,32,32,32,32,32,33,33,33,33,34,34,34,34,35,35,35,35,36,36,36,36,37,37,37,37,38,38,38,38,39,39,39,39,40,40,40,40,41,41,41,41,42,42,42,42,43,43,43,43,44,44,44,44,45,45,45,45,46,46,46,46,47,47,47,47,48,48,48,48,48,49,49,49,49,50,50,50,50,51,51,51,51,52,52,52,52,53,53,53,53,54,54,54,54,55,55,55,55,56,56,56,56,57,57,57,57,58,58,58,58,59,59,59,59,60,60,60,60,61,61,61,61,62,62,62,62,63,63},
			{0,0,16,16,16,32,32,32,48,48,48,48,64,64,64,2,2,2,18,18,18,34,34,34,50,50,50,50,66,66,66,4,4,4,20,20,20,36,36,36,36,52,52,52,68,68,68,6,6,6,22,22,22,38,38,38,38,54,54,54,70,70,70,8,8,8,24,24,24,24,40,40,40,56,56,56,72,72,72,10,10,10,26,26,26,26,42,42,42,58,58,58,74,74,74,12,12,12,12,28,28,28,44,44,44,60,60,60,76,76,76,14,14,14,14,30,30,30,46,46,46,62,62,62,78,78,78,78,79,79,79,79,63,63,63,47,47,47,31,31,31,15,15,15,15,77,77,77,61,61,61,45,45,45,29,29,29,13,13,13,13,75,75,75,59,59,59,43,43,43,27,27,27,27,11,11,11,73,73,73,57,57,57,41,41,41,25,25,25,25,9,9,9,71,71,71,55,55,55,39,39,39,39,23,23,23,7,7,7,69,69,69,53,53,53,37,37,37,37,21,21,21,5,5,5,67,67,67,51,51,51,51,35,35,35,19,19,19,3,3,3,65,65,65,49,49,49,49,33,33,33,17,17,17,1,1},
			{0,0,32,32,64,64,64,2,2,2,34,34,66,66,66,4,4,4,36,36,68,68,68,6,6,6,38,38,70,70,70,8,8,8,40,40,40,72,72,10,10,10,42,42,42,74,74,12,12,12,44,44,44,76,76,14,14,14,46,46,46,78,78,16,16,16,48,48,48,80,80,18,18,18,50,50,50,82,82,20,20,20,52,52,52,84,84,22,22,22,54,54,54,86,86,24,24,24,56,56,56,88,88,26,26,26,58,58,58,90,90,28,28,28,60,60,60,92,92,30,30,30,62,62,62,94,94,94,95,95,95,63,63,63,31,31,31,93,93,61,61,61,29,29,29,91,91,59,59,59,27,27,27,89,89,57,57,57,25,25,25,87,87,55,55,55,23,23,23,85,85,53,53,53,21,21,21,83,83,51,51,51,19,19,19,81,81,49,49,49,17,17,17,79,79,47,47,47,15,15,15,77,77,45,45,45,13,13,13,75,75,43,43,43,11,11,11,73,73,41,41,41,9,9,9,71,71,71,39,39,7,7,7,69,69,69,37,37,5,5,5,67,67,67,35,35,3,3,3,65,65,65,33,33,1,1},
			{0,0,1,1,2,2,3,3,4,4,5,5,6,6,7,7,8,8,9,9,10,10,11,11,12,12,13,13,14,14,15,15,16,16,17,17,18,18,19,19,20,20,21,21,22,22,23,23,24,24,25,25,26,26,27,27,28,28,29,29,30,30,31,31,32,32,33,33,34,34,35,35,36,36,37,37,38,38,39,39,40,40,41,41,42,42,43,43,44,44,45,45,46,46,47,47,48,48,49,49,50,50,51,51,52,52,53,53,54,54,55,55,56,56,57,57,58,58,59,59,60,60,61,61,62,62,63,63,64,64,64,65,65,66,66,67,67,68,68,69,69,70,70,71,71,72,72,73,73,74,74,75,75,76,76,77,77,78,78,79,79,80,80,81,81,82,82,83,83,84,84,85,85,86,86,87,87,88,88,89,89,90,90,91,91,92,92,93,93,94,94,95,95,96,96,97,97,98,98,99,99,100,100,101,101,102,102,103,103,104,104,105,105,106,106,107,107,108,108,109,109,110,110,111,111,112,112,113,113,114,114,115,115,116,116,117,117,118,118,119,119,120,120,121,121,122,122,123,123,124,124,125,125,126,126,127},
			{0,32,32,64,96,96,128,2,2,34,34,66,98,98,130,4,4,36,36,68,100,100,132,6,6,38,38,70,102,102,134,8,8,40,40,72,104,104,136,10,10,42,42,74,106,106,138,12,12,44,44,76,108,108,140,14,14,46,46,78,110,110,142,16,16,48,48,80,112,112,144,18,18,50,50,82,114,114,146,20,20,52,52,84,116,116,148,22,22,54,54,86,118,118,150,24,24,56,56,88,120,120,152,26,26,58,58,90,122,122,154,28,28,60,60,92,124,124,156,30,30,62,62,94,126,126,158,158,159,159,127,127,95,63,63,31,31,157,125,125,93,61,61,29,29,155,123,123,91,59,59,27,27,153,121,121,89,57,57,25,25,151,119,119,87,55,55,23,23,149,117,117,85,53,53,21,21,147,115,115,83,51,51,19,19,145,113,113,81,49,49,17,17,143,111,111,79,47,47,15,15,141,109,109,77,45,45,13,13,139,107,107,75,43,43,11,11,137,105,105,73,41,41,9,9,135,103,103,71,39,39,7,7,133,101,101,69,37,37,5,5,131,99,99,67,35,35,3,3,129,97,97,65,33,33,1},
			{0,64,128,2,2,66,130,4,4,68,132,6,6,70,134,8,8,72,136,10,10,74,138,12,12,76,140,14,14,78,142,16,16,80,144,18,18,82,146,20,20,84,148,22,22,86,150,24,24,88,152,26,26,90,154,28,28,92,156,30,30,94,158,32,32,96,160,34,34,98,162,36,36,100,164,38,38,102,166,40,40,104,168,42,42,106,170,44,44,108,172,46,46,110,174,48,48,112,176,50,50,114,178,52,52,116,180,54,54,118,182,56,56,120,184,58,58,122,186,60,60,124,188,62,62,126,190,190,191,191,127,63,63,189,125,61,61,187,123,59,59,185,121,57,57,183,119,55,55,181,117,53,53,179,115,51,51,177,113,49,49,175,111,47,47,173,109,45,45,171,107,43,43,169,105,41,41,167,103,39,39,165,101,37,37,163,99,35,35,161,97,33,33,159,95,31,31,157,93,29,29,155,91,27,27,153,89,25,25,151,87,23,23,149,85,21,21,147,83,19,19,145,81,17,17,143,79,15,15,141,77,13,13,139,75,11,11,137,73,9,9,135,71,7,7,133,69,5,5,131,67,3,3,129,65,1},
			{0,1,2,3,4,5,6,7,8,9,10,11,12,13,14,15,16,17,18,19,20,21,22,23,24,25,26,27,28,29,30,31,32,33,34,35,36,37,38,39,40,41,42,43,44,45,46,47,48,49,50,51,52,53,54,55,56,57,58,59,60,61,62,63,64,65,66,67,68,69,70,71,72,73,74,75,76,77,78,79,80,81,82,83,84,85,86,87,88,89,90,91,92,93,94,95,96,97,98,99,100,101,102,103,104,105,106,107,108,109,110,111,112,113,114,115,116,117,118,119,120,121,122,123,124,125,126,127,128,129,130,131,132,133,134,135,136,137,138,139,140,141,142,143,144,145,146,147,148,149,150,151,152,153,154,155,156,157,158,159,160,161,162,163,164,165,166,167,168,169,170,171,172,173,174,175,176,177,178,179,180,181,182,183,184,185,186,187,188,189,190,191,192,193,194,195,196,197,198,199,200,201,202,203,204,205,206,207,208,209,210,211,212,213,214,215,216,217,218,219,220,221,222,223,224,225,226,227,228,229,230,231,232,233,234,235,236,237,238,239,240,241,242,243,244,245,246,247,248,249,250,251,252,253,254,255}
		};


		const uint8 weight_quantize_table[12][1025] = {
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1},
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2},
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3},
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4},
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1},
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7,
			 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7,
			 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7},
			{0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0,
			 0, 0, 0, 0, 0, 0, 0, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2,
			 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 2, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4, 4,
			 4, 4, 4, 4, 4, 4, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6,
			 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 6, 8, 8, 8, 8, 8, 8, 8,
			 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8,
			 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8,
			 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8,
			 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8,
			 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 8, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9,
			 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9,
			 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9,
			 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9,
			 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9, 9,
			 9, 9, 9, 9, 9, 9, 9, 9, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7,
			 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7,
			 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7,
			 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7,
			 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 7, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5, 5,
			 5, 5, 5, 5, 5, 5, 5, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3,
			 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 3, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1,
			 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1},
			{0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,
			 0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,
			 0,  0,  0,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1},
			{0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,
			 0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  1,  1,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,
			 1,  1,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15},
			{0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,
			 0,  0,  0,  0,  0,  0,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  17, 17, 17, 17, 17,
			 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  1,  1,  1,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1},
			{0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 18, 18, 18, 18, 18, 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20,
			 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20,
			 20, 20, 20, 20, 20, 20, 20, 20, 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22,
			 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22,
			 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22,
			 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23,
			 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23,
			 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  21, 21, 21, 21, 21, 21, 21, 21,
			 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21,
			 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  19, 19, 19, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  17, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1},
			{0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  0,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,
			 1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  1,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,  2,
			 2,  2,  2,  2,  2,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,
			 3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  3,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,
			 4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  4,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,  5,
			 5,  5,  5,  5,  5,  5,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,  6,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,
			 7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  7,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,  8,
			 8,  8,  8,  8,  8,  8,  8,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,  9,
			 9,  10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10,
			 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 10, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11, 11,
			 11, 11, 11, 11, 11, 11, 11, 11, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12, 12,
			 12, 12, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13,
			 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 13, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14, 14,
			 14, 14, 14, 14, 14, 14, 14, 14, 14, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15, 15,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 16,
			 16, 16, 16, 16, 16, 16, 16, 16, 16, 16, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17, 17,
			 17, 17, 17, 17, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18,
			 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 18, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19,
			 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 19, 20, 20, 20, 20, 20, 20, 20, 20,
			 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20, 20,
			 20, 20, 20, 20, 20, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21,
			 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 21, 22,
			 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22,
			 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 22, 23, 23, 23, 23, 23, 23, 23,
			 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23, 23,
			 23, 23, 23, 23, 23, 23, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24,
			 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24, 24,
			 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25,
			 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 25, 26, 26, 26, 26, 26, 26,
			 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26, 26,
			 26, 26, 26, 26, 26, 26, 26, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27,
			 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27, 27,
			 27, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28,
			 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 28, 29, 29, 29, 29, 29,
			 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29, 29,
			 29, 29, 29, 29, 29, 29, 29, 29, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30,
			 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30, 30,
			 30, 30, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31, 31} };



		/**
		 * Table that describes the number of trits or quints along with bits required
		 * for storing each range.
		 */
		const uint8 bits_trits_quints_table[RANGE_MAX][3] = {
			{1, 0, 0},  // RANGE_2
			{0, 1, 0},  // RANGE_3
			{2, 0, 0},  // RANGE_4
			{0, 0, 1},  // RANGE_5
			{1, 1, 0},  // RANGE_6
			{3, 0, 0},  // RANGE_8
			{1, 0, 1},  // RANGE_10
			{2, 1, 0},  // RANGE_12
			{4, 0, 0},  // RANGE_16
			{2, 0, 1},  // RANGE_20
			{3, 1, 0},  // RANGE_24
			{5, 0, 0},  // RANGE_32
			{3, 0, 1},  // RANGE_40
			{4, 1, 0},  // RANGE_48
			{6, 0, 0},  // RANGE_64
			{4, 0, 1},  // RANGE_80
			{5, 1, 0},  // RANGE_96
			{7, 0, 0},  // RANGE_128
			{5, 0, 1},  // RANGE_160
			{6, 1, 0},  // RANGE_192
			{8, 0, 0}   // RANGE_256
		};

		/**
		 * Encode a group of 5 numbers using trits and bits.
		 */
		inline void encode_trits(size_t bits,
			uint8 b0,
			uint8 b1,
			uint8 b2,
			uint8 b3,
			uint8 b4,
			bitwriter& writer)
		{
			uint8 t0, t1, t2, t3, t4;
			uint8 m0, m1, m2, m3, m4;

			split_high_low(b0, bits, t0, m0);
			split_high_low(b1, bits, t1, m1);
			split_high_low(b2, bits, t2, m2);
			split_high_low(b3, bits, t3, m3);
			split_high_low(b4, bits, t4, m4);

			miro_check(t0 < 3);
			miro_check(t1 < 3);
			miro_check(t2 < 3);
			miro_check(t3 < 3);
			miro_check(t4 < 3);

			if (t0 < 3 && t1 < 3 && t2 < 3 && t3 < 3 && t4 < 3)
			{
				uint8 packed = integer_from_trits[t4][t3][t2][t1][t0];

				writer.write8(m0, bits);
				writer.write8(getbits(packed, 1, 0), 2);
				writer.write8(m1, bits);
				writer.write8(getbits(packed, 3, 2), 2);
				writer.write8(m2, bits);
				writer.write8(getbits(packed, 4, 4), 1);
				writer.write8(m3, bits);
				writer.write8(getbits(packed, 6, 5), 2);
				writer.write8(m4, bits);
				writer.write8(getbits(packed, 7, 7), 1);
			}
		}

		/**
		 * Encode a group of 3 numbers using quints and bits.
		 */
		inline void encode_quints(size_t bits,
			uint8 b0,
			uint8 b1,
			uint8 b2,
			bitwriter& writer) {
			uint8 q0, q1, q2;
			uint8 m0, m1, m2;

			split_high_low(b0, bits, q0, m0);
			split_high_low(b1, bits, q1, m1);
			split_high_low(b2, bits, q2, m2);

			miro_check(q0 < 5);
			miro_check(q1 < 5);
			miro_check(q2 < 5);

			if (q0 < 5 && q1 < 5 && q2 < 5)
			{
				uint8 packed = integer_from_quints[q2][q1][q0];

				writer.write8(m0, bits);
				writer.write8(getbits(packed, 2, 0), 3);
				writer.write8(m1, bits);
				writer.write8(getbits(packed, 4, 3), 2);
				writer.write8(m2, bits);
				writer.write8(getbits(packed, 6, 5), 2);
			}
		}

		/**
		 * Encode a sequence of numbers using using one trit and a custom number of
		 * bits per number.
		 */
		inline void encode_trits(const uint8* numbers,
			size_t count,
			bitwriter& writer,
			size_t bits) {
			for (size_t i = 0; i < count; i += 5) {
				uint8 b0 = numbers[i + 0];
				uint8 b1 = i + 1 >= count ? 0 : numbers[i + 1];
				uint8 b2 = i + 2 >= count ? 0 : numbers[i + 2];
				uint8 b3 = i + 3 >= count ? 0 : numbers[i + 3];
				uint8 b4 = i + 4 >= count ? 0 : numbers[i + 4];

				encode_trits(bits, b0, b1, b2, b3, b4, writer);
			}
		}

		/**
		 * Encode a sequence of numbers using one quint and the custom number of bits
		 * per number.
		 */
		inline void encode_quints(const uint8* numbers,
			size_t count,
			bitwriter& writer,
			size_t bits) {
			for (size_t i = 0; i < count; i += 3) {
				uint8 b0 = numbers[i + 0];
				uint8 b1 = i + 1 >= count ? 0 : numbers[i + 1];
				uint8 b2 = i + 2 >= count ? 0 : numbers[i + 2];
				encode_quints(bits, b0, b1, b2, writer);
			}
		}

		/**
		 * Encode a sequence of numbers using binary representation with the selected
		 * bit count.
		 */
		inline void encode_binary(const uint8* numbers,
			size_t count,
			bitwriter& writer,
			size_t bits) {
			miro_check(count > 0);
			for (size_t i = 0; i < count; ++i) {
				writer.write8(numbers[i], bits);
			}
		}

		/**
		 * Encode a sequence of numbers in a specific range using the binary integer
		 * sequence encoding. The numbers are assumed to be in the correct range and
		 * the memory we are writing to is assumed to be zero-initialized.
		 */
		inline void integer_sequence_encode(const uint8* numbers,
			size_t count,
			range_t range,
			bitwriter writer) {

#ifdef UE_MIRO_DEBUG
			for (size_t i = 0; i < count; ++i) {
				miro_check(numbers[i] <= range_max_table[range]);
			}
#endif

			size_t bits = bits_trits_quints_table[range][0];
			size_t trits = bits_trits_quints_table[range][1];
			size_t quints = bits_trits_quints_table[range][2];

			if (trits == 1) {
				encode_trits(numbers, count, writer, bits);
			}
			else if (quints == 1) {
				encode_quints(numbers, count, writer, bits);
			}
			else {
				encode_binary(numbers, count, writer, bits);
			}
		}

		inline void integer_sequence_encode(const uint8* numbers, size_t count, range_t range, uint8* output)
		{
			integer_sequence_encode(numbers, count, range, bitwriter(output));
		}

		/**
		 * Compute the number of bits required to store a number of items in a specific
		 * range using the binary integer sequence encoding.
		 */
		inline size_t compute_ise_bitcount(size_t items, range_t range)
		{
			size_t bits = bits_trits_quints_table[range][0];
			size_t trits = bits_trits_quints_table[range][1];
			size_t quints = bits_trits_quints_table[range][2];

			if (trits) {
				return ((8 + 5 * bits) * items + 4) / 5;
			}

			if (quints) {
				return ((7 + 3 * bits) * items + 2) / 3;
			}

			return items * bits;
		}


		inline uint8 quantize_color(range_t quant, int c)
		{
			miro_check(c >= 0 && c <= 255);
			if (c >= 0 && c <= 255)
			{
				return color_quantize_table[quant][c];
			}
			return 0;
		}

		vec3i_t quantize_color(range_t quant, vec3i_t c)
		{
			vec3i_t result;
			result.r = color_quantize_table[quant][c.r];
			result.g = color_quantize_table[quant][c.g];
			result.b = color_quantize_table[quant][c.b];
			return result;
		}

		vec4i_t quantize_color(range_t quant, vec4i_t c)
		{
			vec4i_t result;
			result.r = color_quantize_table[quant][c.r];
			result.g = color_quantize_table[quant][c.g];
			result.b = color_quantize_table[quant][c.b];
			result.a = color_quantize_table[quant][c.a];
			return result;
		}

		uint8 unquantize_color(range_t quant, int c)
		{
			miro_check(c >= 0 && c <= 255);
			if (c >= 0 && c <= 255)
			{
				return color_unquantize_table[quant][c];
			}
			return 0;
		}

		vec3i_t unquantize_color(range_t quant, vec3i_t c)
		{
			vec3i_t result;
			result.r = color_unquantize_table[quant][c.r];
			result.g = color_unquantize_table[quant][c.g];
			result.b = color_unquantize_table[quant][c.b];
			return result;
		}

		vec4i_t unquantize_color(range_t quant, vec4i_t c)
		{
			vec4i_t result;
			result.r = color_unquantize_table[quant][c.r];
			result.g = color_unquantize_table[quant][c.g];
			result.b = color_unquantize_table[quant][c.b];
			result.a = color_unquantize_table[quant][c.a];
			return result;
		}


		int color_channel_sum(vec3i_t color)
		{
			return color.r + color.g + color.b;
		}

		int color_channel_sum(vec4i_t color)
		{
			return color.r + color.g + color.b + color.a;
		}

		void encode_luminance_direct(range_t endpoint_quant,
			int32 v0,
			int32 v1,
			uint8 endpoint_quantized[2],
			uint8 endpoint_unquantized[2])
		{
			endpoint_quantized[0] = quantize_color(endpoint_quant, v0);
			endpoint_quantized[1] = quantize_color(endpoint_quant, v1);
			endpoint_unquantized[0] = unquantize_color(endpoint_quant, endpoint_quantized[0]);
			endpoint_unquantized[1] = unquantize_color(endpoint_quant, endpoint_quantized[1]);
		}

		void encode_rgb_base_scale(range_t endpoint_quant,
			vec3i_t e1,
			uint8 scale,
			uint8 endpoint_quantized[4],
			vec3i_t endpoint_unquantized[2])
		{
			vec3i_t e1q = quantize_color(endpoint_quant, e1);
			vec3i_t e1u = unquantize_color(endpoint_quant, e1q);

			uint8 scaleq = color_quantize_table[endpoint_quant][scale];
			uint8 scaleu = color_unquantize_table[endpoint_quant][scaleq];
			vec3i_t e0u = (e1u * int32(scaleu)) / 256;

			endpoint_quantized[0] = static_cast<uint8>(e1q.r);
			endpoint_quantized[1] = static_cast<uint8>(e1q.g);
			endpoint_quantized[2] = static_cast<uint8>(e1q.b);
			endpoint_quantized[3] = static_cast<uint8>(scaleq);

			endpoint_unquantized[0] = e0u;
			endpoint_unquantized[1] = e1u;
		}

		void encode_rgb_direct(range_t endpoint_quant,
			vec3i_t& InOutE0,
			vec3i_t& InOutE1,
			uint8 endpoint_quantized[6],
			vec3i_t endpoint_unquantized[2])
		{
			vec3i_t e0q = quantize_color(endpoint_quant, InOutE0);
			vec3i_t e1q = quantize_color(endpoint_quant, InOutE1);
			vec3i_t e0u = unquantize_color(endpoint_quant, e0q);
			vec3i_t e1u = unquantize_color(endpoint_quant, e1q);

			// ASTC uses a different blue contraction encoding when the sum of values for
			// the first endpoint is larger than the sum of values in the second
			// endpoint. Sort the endpoints to ensure that the normal encoding is used.
			if (color_channel_sum(e0u) > color_channel_sum(e1u))
			{
				endpoint_quantized[0] = static_cast<uint8>(e1q.r);
				endpoint_quantized[1] = static_cast<uint8>(e0q.r);
				endpoint_quantized[2] = static_cast<uint8>(e1q.g);
				endpoint_quantized[3] = static_cast<uint8>(e0q.g);
				endpoint_quantized[4] = static_cast<uint8>(e1q.b);
				endpoint_quantized[5] = static_cast<uint8>(e0q.b);

				endpoint_unquantized[0] = e1u;
				endpoint_unquantized[1] = e0u;

				// We need them swapped in the calling code too.
				Swap(InOutE0, InOutE1);
			}
			else
			{
				endpoint_quantized[0] = static_cast<uint8>(e0q.r);
				endpoint_quantized[1] = static_cast<uint8>(e1q.r);
				endpoint_quantized[2] = static_cast<uint8>(e0q.g);
				endpoint_quantized[3] = static_cast<uint8>(e1q.g);
				endpoint_quantized[4] = static_cast<uint8>(e0q.b);
				endpoint_quantized[5] = static_cast<uint8>(e1q.b);

				endpoint_unquantized[0] = e0u;
				endpoint_unquantized[1] = e1u;
			}
		}


		void encode_rgb_direct(range_t endpoint_quant,
			vec4i_t& e0,
			vec4i_t& e1,
			uint8 endpoint_quantized[6],
			vec3i_t endpoint_unquantized[2])
		{
			vec3i_t e0q = quantize_color(endpoint_quant, e0.rgb());
			vec3i_t e1q = quantize_color(endpoint_quant, e1.rgb());
			vec3i_t e0u = unquantize_color(endpoint_quant, e0q);
			vec3i_t e1u = unquantize_color(endpoint_quant, e1q);

			// ASTC uses a different blue contraction encoding when the sum of values for
			// the first endpoint is larger than the sum of values in the second
			// endpoint. Sort the endpoints to ensure that the normal encoding is used.
			if (color_channel_sum(e0u) > color_channel_sum(e1u))
			{
				endpoint_quantized[0] = static_cast<uint8>(e1q.r);
				endpoint_quantized[1] = static_cast<uint8>(e0q.r);
				endpoint_quantized[2] = static_cast<uint8>(e1q.g);
				endpoint_quantized[3] = static_cast<uint8>(e0q.g);
				endpoint_quantized[4] = static_cast<uint8>(e1q.b);
				endpoint_quantized[5] = static_cast<uint8>(e0q.b);

				endpoint_unquantized[0] = e1u;
				endpoint_unquantized[1] = e0u;

				Swap(e0, e1);
			}
			else
			{
				endpoint_quantized[0] = static_cast<uint8>(e0q.r);
				endpoint_quantized[1] = static_cast<uint8>(e1q.r);
				endpoint_quantized[2] = static_cast<uint8>(e0q.g);
				endpoint_quantized[3] = static_cast<uint8>(e1q.g);
				endpoint_quantized[4] = static_cast<uint8>(e0q.b);
				endpoint_quantized[5] = static_cast<uint8>(e1q.b);

				endpoint_unquantized[0] = e0u;
				endpoint_unquantized[1] = e1u;
			}
		}

		void encode_rgba_direct(range_t endpoint_quant,
			vec4i_t& e0,
			vec4i_t& e1,
			uint8 endpoint_quantized[8],
			vec4i_t endpoint_unquantized[2],
			bool bDualPlane)
		{
			vec4i_t e0q = quantize_color(endpoint_quant, e0);
			vec4i_t e1q = quantize_color(endpoint_quant, e1);
			vec4i_t e0u = unquantize_color(endpoint_quant, e0q);
			vec4i_t e1u = unquantize_color(endpoint_quant, e1q);

			// ASTC uses a different blue contraction encoding when the sum of values for
			// the first endpoint is larger than the sum of values in the second
			// endpoint. Sort the endpoints to ensure that the normal encoding is used.
			if (color_channel_sum(e0u.rgb()) > color_channel_sum(e1u.rgb()))
			{
				endpoint_quantized[0] = static_cast<uint8>(e1q.r);
				endpoint_quantized[1] = static_cast<uint8>(e0q.r);
				endpoint_quantized[2] = static_cast<uint8>(e1q.g);
				endpoint_quantized[3] = static_cast<uint8>(e0q.g);
				endpoint_quantized[4] = static_cast<uint8>(e1q.b);
				endpoint_quantized[5] = static_cast<uint8>(e0q.b);
				endpoint_quantized[6] = static_cast<uint8>(e1q.a);
				endpoint_quantized[7] = static_cast<uint8>(e0q.a);

				endpoint_unquantized[0] = e1u;
				endpoint_unquantized[1] = e0u;

				Swap(e0, e1);
			}
			else
			{
				endpoint_quantized[0] = static_cast<uint8>(e0q.r);
				endpoint_quantized[1] = static_cast<uint8>(e1q.r);
				endpoint_quantized[2] = static_cast<uint8>(e0q.g);
				endpoint_quantized[3] = static_cast<uint8>(e1q.g);
				endpoint_quantized[4] = static_cast<uint8>(e0q.b);
				endpoint_quantized[5] = static_cast<uint8>(e1q.b);
				endpoint_quantized[6] = static_cast<uint8>(e0q.a);
				endpoint_quantized[7] = static_cast<uint8>(e1q.a);

				endpoint_unquantized[0] = e0u;
				endpoint_unquantized[1] = e1u;
			}

			// Sort alpha endpoints
			if (bDualPlane)
			{
				if (e0.a > e1.a)
				{
					endpoint_quantized[6] = static_cast<uint8>(e1q.a);
					endpoint_quantized[7] = static_cast<uint8>(e0q.a);
					endpoint_unquantized[0].a = e1u.a;
					endpoint_unquantized[1].a = e0u.a;
				}
				else
				{
					endpoint_quantized[6] = static_cast<uint8>(e0q.a);
					endpoint_quantized[7] = static_cast<uint8>(e1q.a);
					endpoint_unquantized[0].a = e0u.a;
					endpoint_unquantized[1].a = e1u.a;
				}
			}
		}

		void encode_la_direct(range_t endpoint_quant,
			vec4i_t e0,
			vec4i_t e1,
			uint8 endpoint_quantized[4],
			vec4i_t endpoint_unquantized[2])
		{
			int32 l0 = (e0.r+e0.g+e0.b)/3;
			int32 a0 = e0.a;
			int32 l1 = (e1.r+e1.g+e1.b)/3;
			int32 a1 = e1.a;

			int32 l0q = quantize_color(endpoint_quant, l0);
			int32 a0q = quantize_color(endpoint_quant, a0);
			int32 l1q = quantize_color(endpoint_quant, l1);
			int32 a1q = quantize_color(endpoint_quant, a1);

			int32 l0u = unquantize_color(endpoint_quant, l0q);
			int32 a0u = unquantize_color(endpoint_quant, a0q);
			int32 l1u = unquantize_color(endpoint_quant, l1q);
			int32 a1u = unquantize_color(endpoint_quant, a1q);

			endpoint_quantized[0] = static_cast<uint8>(l0q);
			endpoint_quantized[1] = static_cast<uint8>(l1q);
			endpoint_quantized[2] = static_cast<uint8>(a0q);
			endpoint_quantized[3] = static_cast<uint8>(a1q);

			endpoint_unquantized[0] = vec4i_t(l0u, l0u, l0u, a0u);
			endpoint_unquantized[1] = vec4i_t(l1u, l1u, l1u, a1u);
		}

		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		inline void symbolic_to_physical
		(
			color_endpoint_mode_t color_endpoint_mode,
			range_t endpoint_quant,
			range_t weight_quant,

			size_t partition_count,

			const uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES],

			// +1 needed here because orbits_8ptr breaks when the offset reaches
			// the last byte which always happens if the weight mode is RANGE_32.
			const uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1],

			PhysicalBlock* pb,
			bool dual_plane = false
		)
		{
			miro_check(weight_quant <= RANGE_32);
			miro_check(endpoint_quant < RANGE_MAX);
			miro_check(color_endpoint_mode < CEM_MAX);
			miro_check(partition_count == 1);
			miro_check(compute_ise_bitcount(BLOCK_WIDTH * BLOCK_HEIGHT, weight_quant) < MAXIMUM_ENCODED_WEIGHT_BITS);

			size_t n = BLOCK_WIDTH;
			size_t m = BLOCK_HEIGHT;

			static const bool h_table[RANGE_32 + 1] = { 0, 0, 0, 0, 0, 0,
														1, 1, 1, 1, 1, 1 };

			static const uint8 r_table[RANGE_32 + 1] = { 0x2, 0x3, 0x4, 0x5, 0x6, 0x7,
														   0x2, 0x3, 0x4, 0x5, 0x6, 0x7 };

			bool h = (weight_quant <= RANGE_32) ? h_table[weight_quant] : false;
			size_t r = (weight_quant < RANGE_32) ? r_table[weight_quant] : 0;

			bool d = dual_plane;

			size_t part_value = partition_count - 1;

			size_t cem_offset = 13;
			size_t ced_offset = 17;

			size_t cem_bits = 4;
			size_t cem = color_endpoint_mode;

			// Block mode

			// Actually weight-grid and block sizes don't need to match. We are defining the weight-grid size here but for
			// now we make them match.
			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				// Use the first row of Table C.2.8 in the ASTC specification.
				size_t a = m - 2;
				size_t b = n - 4;

				orbits8_ptr(pb->data, 0, getbit(r, 1), 1);
				orbits8_ptr(pb->data, 1, getbit(r, 2), 1);
				orbits8_ptr(pb->data, 2, 0, 1);
				orbits8_ptr(pb->data, 3, 0, 1);
				orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
				orbits8_ptr(pb->data, 5, a, 2);
				orbits8_ptr(pb->data, 7, b, 2);
				orbits8_ptr(pb->data, 9, h, 1);
				orbits8_ptr(pb->data, 10, d, 1);
				
				// This would be for rgba
				//check(pb->data[0] == 0b01010011);
				//check(pb->data[1] == 0b000);
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				// This encoder doesn't support 6x6 and dual plane: that requires mismatching and interpolating the pixel grid and weight grid
				miro_check(dual_plane == 0);

				// Use the tenth row of Table C.2.8 in the ASTC specification.
				size_t a = m - 6;
				size_t b = n - 6;

				orbits8_ptr(pb->data, 0, 0, 2);
				orbits8_ptr(pb->data, 2, getbit(r, 1), 1);
				orbits8_ptr(pb->data, 3, getbit(r, 2), 1);
				orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
				orbits8_ptr(pb->data, 5, a, 2);
				orbits8_ptr(pb->data, 7, 0, 1);
				orbits8_ptr(pb->data, 8, 1, 1);
				orbits8_ptr(pb->data, 9, b, 2);

				// This would be for rgba
				//check(pb->data[0] == 0b00001000);
				//check(pb->data[1] == 0b001);
			}
			else if (BLOCK_WIDTH == 8 && BLOCK_HEIGHT == 8)
			{
				// This encoder doesn't support 8x8 and dual plane: that requires mismatching and interpolating the pixel grid and weight grid
				miro_check(dual_plane == 0);

				// Use the tenth row of Table C.2.8 in the ASTC specification.
				size_t a = m - 6;
				size_t b = n - 6;

				orbits8_ptr(pb->data, 0, 0, 2);
				orbits8_ptr(pb->data, 2, getbit(r, 1), 1);
				orbits8_ptr(pb->data, 3, getbit(r, 2), 1);
				orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
				orbits8_ptr(pb->data, 5, a, 2);
				orbits8_ptr(pb->data, 7, 0, 1);
				orbits8_ptr(pb->data, 8, 1, 1);
				orbits8_ptr(pb->data, 9, b, 2);
			}
			else
			{
				check(false);
			}


			// Partitions
			orbits8_ptr(pb->data, 11, part_value, 2);

			// CEM
			orbits8_ptr(pb->data, cem_offset, cem, cem_bits);

			copy_bytes(endpoint_ise, MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES, pb->data, ced_offset);

			reverse_bytes(weights_ise, MAXIMUM_ENCODED_WEIGHT_BYTES, pb->data + 15);

			if (dual_plane)
			{
				size_t bits_for_weights = compute_ise_bitcount(2 * BLOCK_WIDTH * BLOCK_HEIGHT, weight_quant);

				// \TODO: dual plane always sets alpha as the separate channel
				size_t secondPlaneChannel = 3;
				orbits8_ptr(pb->data, 128 - bits_for_weights - 2, secondPlaneChannel, 2);
			}
		}


		inline void symbolic_to_physical
		(
			int32 GRID_WIDTH, int32 GRID_HEIGHT,
			color_endpoint_mode_t color_endpoint_mode,
			range_t endpoint_quant,
			range_t weight_quant,

			size_t partition_count,

			const uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES],

			// FIXME: +1 needed here because orbits_8ptr breaks when the offset reaches
			// the last byte which always happens if the weight mode is RANGE_32.
			const uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1],

			PhysicalBlock* pb,
			bool dual_plane = false
		)
		{
			miro_check(weight_quant <= RANGE_32);
			miro_check(endpoint_quant < RANGE_MAX);
			miro_check(color_endpoint_mode < CEM_MAX);
			miro_check(partition_count == 1);
			miro_check(compute_ise_bitcount(GRID_WIDTH * GRID_HEIGHT, weight_quant) < MAXIMUM_ENCODED_WEIGHT_BITS);

			size_t n = GRID_WIDTH;
			size_t m = GRID_HEIGHT;

			static const bool h_table[RANGE_32 + 1] = { 0, 0, 0, 0, 0, 0,
														1, 1, 1, 1, 1, 1 };

			static const uint8 r_table[RANGE_32 + 1] = { 0x2, 0x3, 0x4, 0x5, 0x6, 0x7,
														   0x2, 0x3, 0x4, 0x5, 0x6, 0x7 };

			bool h = (weight_quant <= RANGE_32) ? h_table[weight_quant] : false;
			size_t r = (weight_quant < RANGE_32) ? r_table[weight_quant] : 0;

			bool d = dual_plane;

			size_t part_value = partition_count - 1;

			size_t cem_offset = 13;
			size_t ced_offset = 17;

			size_t cem_bits = 4;
			size_t cem = color_endpoint_mode;

			// Block mode

			// Actually weight-grid and block sizes don't need to match. We are defining the weight-grid size here but for
			// now we make them match.
			if (GRID_WIDTH == 4 && GRID_HEIGHT == 4)
			{
				// Use the first row of Table C.2.8 in the ASTC specification.
				size_t a = m - 2;
				size_t b = n - 4;

				orbits8_ptr(pb->data, 0, getbit(r, 1), 1);
				orbits8_ptr(pb->data, 1, getbit(r, 2), 1);
				orbits8_ptr(pb->data, 2, 0, 1);
				orbits8_ptr(pb->data, 3, 0, 1);
				orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
				orbits8_ptr(pb->data, 5, a, 2);
				orbits8_ptr(pb->data, 7, b, 2);
				orbits8_ptr(pb->data, 9, h, 1);
				orbits8_ptr(pb->data, 10, d, 1);
			}
			else if (GRID_WIDTH == 6 && GRID_HEIGHT == 6)
			{
				// This encoder doesn't support 6x6 and dual plane: that requires mismatching and interpolating the pixel grid and weight grid
				miro_check(dual_plane == 0);

				// Use the tenth row of Table C.2.8 in the ASTC specification.
				size_t a = m - 6;
				size_t b = n - 6;

				orbits8_ptr(pb->data, 0, 0, 2);
				orbits8_ptr(pb->data, 2, getbit(r, 1), 1);
				orbits8_ptr(pb->data, 3, getbit(r, 2), 1);
				orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
				orbits8_ptr(pb->data, 5, a, 2);
				orbits8_ptr(pb->data, 7, 0, 1);
				orbits8_ptr(pb->data, 8, 1, 1);
				orbits8_ptr(pb->data, 9, b, 2);
			}
			else if (GRID_WIDTH == 8 && GRID_HEIGHT == 8)
			{
				// This encoder doesn't support 8x8 and dual plane: that requires mismatching and interpolating the pixel grid and weight grid
				miro_check(dual_plane == 0);

				// Use the tenth row of Table C.2.8 in the ASTC specification.
				size_t a = m - 6;
				size_t b = n - 6;

				orbits8_ptr(pb->data, 0, 0, 2);
				orbits8_ptr(pb->data, 2, getbit(r, 1), 1);
				orbits8_ptr(pb->data, 3, getbit(r, 2), 1);
				orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
				orbits8_ptr(pb->data, 5, a, 2);
				orbits8_ptr(pb->data, 7, 0, 1);
				orbits8_ptr(pb->data, 8, 1, 1);
				orbits8_ptr(pb->data, 9, b, 2);
			}
			else if (GRID_WIDTH == 7 && GRID_HEIGHT == 7)
			{
				if (weight_quant == RANGE_3)
				{
					// Use the tenth row of Table C.2.8 in the ASTC specification with r=0b011, a=1, b=1
#ifdef UE_MIRO_DEBUG
					size_t a = m - 6;
					size_t b = n - 6;
					orbits8_ptr(pb->data, 0, 0, 2);
					orbits8_ptr(pb->data, 2, getbit(r, 1), 1);
					orbits8_ptr(pb->data, 3, getbit(r, 2), 1);
					orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
					orbits8_ptr(pb->data, 5, a, 2);
					orbits8_ptr(pb->data, 7, 0, 1);
					orbits8_ptr(pb->data, 8, 1, 1);
					orbits8_ptr(pb->data, 9, b, 2);
					check(pb->data[0] == 0b00110100);
					check(pb->data[1] == 0b011);
#else
					pb->data[0] = 0b00110100;
					pb->data[1] = 0b011;
#endif
				}
				else
				{
					// Use the tenth row of Table C.2.8 in the ASTC specification with r=0b010, a=1, b=1
					miro_check(weight_quant == RANGE_2);

#ifdef UE_MIRO_DEBUG
					size_t a = m - 6;
					size_t b = n - 6;
					orbits8_ptr(pb->data, 0, 0, 2);
					orbits8_ptr(pb->data, 2, getbit(r, 1), 1);
					orbits8_ptr(pb->data, 3, getbit(r, 2), 1);
					orbits8_ptr(pb->data, 4, getbit(r, 0), 1);
					orbits8_ptr(pb->data, 5, a, 2);
					orbits8_ptr(pb->data, 7, 0, 1);
					orbits8_ptr(pb->data, 8, 1, 1);
					orbits8_ptr(pb->data, 9, b, 2);
					check(pb->data[0] == 0b00100100);
					check(pb->data[1] == 0b011);
#else
					pb->data[0] = 0b00100100;
					pb->data[1] = 0b011;
#endif
				}

				// This encoding row doesn't support dual plane.
				miro_check(!dual_plane);
			}
			else
			{
				check(false);
			}


			// Partitions
			orbits8_ptr(pb->data, 11, part_value, 2);

			// CEM
			orbits8_ptr(pb->data, cem_offset, cem, cem_bits);

			copy_bytes(endpoint_ise, MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES, pb->data, ced_offset);

			reverse_bytes(weights_ise, MAXIMUM_ENCODED_WEIGHT_BYTES, pb->data + 15);

			if (dual_plane)
			{
				size_t bits_for_weights = compute_ise_bitcount(2 * GRID_WIDTH * GRID_HEIGHT, weight_quant);

				// \TODO: dual plane always sets alpha as the separate channel
				size_t secondPlaneChannel = 3;
				orbits8_ptr(pb->data, 128 - bits_for_weights - 2, secondPlaneChannel, 2);
			}
		}


		uint8 quantize_weight(range_t weight_quant, size_t weight)
		{
			miro_check(weight_quant <= RANGE_32);
			// anticto: this may happen because of rounding in some ranges.
			// apparently, it is ok to clamp it, based on what the arm reference implementation does.
			if (weight > 1024) weight = 1024;
			//miro_check(weight <= 1024);
			if ((weight_quant <= RANGE_32) && (weight <= 1024))
			{
				return weight_quantize_table[weight_quant][weight];
			}
			return 0;
		}

		/**
		 * Project a texel to a line and quantize the result in 1 dimension.
		 *
		 * The line is defined by t=k*x + m. This function calculates and quantizes x
		 * by projecting n=t-m onto k, x=|n|/|k|. Since k and m is derived from the
		 * minimum and maximum of all texel values the result will be in the range [0,
		 * 1].
		 *
		 * To quantize the result using the weight_quantize_table the value needs to
		 * be extended to the range [0, 1024].
		 *
		 * @param k the derivative of the line
		 * @param m the minimum endpoint
		 * @param t the texel value
		 */
		size_t project(size_t k, size_t m, size_t t)
		{
			miro_check(k > 0);
			// anticto fix: underflow is possible because we use the unquantized limit, which may be
			// bigger than the value. so i think we need to clamp.
			// return size_t((t - m) * 1024) / k;
			return size_t(FMath::Max(0, FMath::Min(int(k), int(t) - int(m))) * 1024) / k;
		}


		/**
		 * Project a texel to a line and quantize the result in 3 dimensions.
		 */
		size_t project(vec3i_t k, int kk, vec3i_t m, vec3i_t t)
		{
			miro_check(kk > 0);
			return static_cast<size_t>(FMath::Clamp(dot(t - m, k) * 1024 / kk, 0, 1024));
		}

		size_t project(vec4i_t k, int kk, vec4i_t m, vec4i_t t)
		{
			miro_check(kk > 0);
			return static_cast<size_t>(FMath::Clamp(dot(t - m, k) * 1024 / kk, 0, 1024));
		}

		template<int32 BLOCK_TEXEL_COUNT>
		void calculate_quantized_weights_luminance(const uint8 texels[BLOCK_TEXEL_COUNT],
			range_t quant,
			uint8 l0,
			uint8 l1,
			uint8 weights[BLOCK_TEXEL_COUNT])
		{
			miro_check(l0 <= l1);
			if (l0 < l1)
			{
				size_t k = l1 - l0;
				size_t m = l0;

				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					size_t t = static_cast<size_t>(texels[i]);
					weights[i] = quantize_weight(quant, project(k, m, t));
				}
			}
			else
			{
				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					miro_check(static_cast<size_t>(texels[i]) == l0);
					weights[i] = quantize_weight(quant, 0);
				}
			}
		}


		template<int32 BLOCK_TEXEL_COUNT>
		void calculate_quantized_weights_channel(const unorm8_t texels[BLOCK_TEXEL_COUNT],
			range_t quant,
			uint8 l0,
			uint8 l1,
			uint8 weights[BLOCK_TEXEL_COUNT],
			int channel)
		{
			if (l0 == l1)
			{
				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					weights[i] = 0;  // quantize_weight(quant, 0) is always 0
				}
			}
			else
			{
				miro_check(l0 < l1);

				size_t k = l1 - l0;
				size_t m = l0;

				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					size_t t = static_cast<size_t>(texels[i].components[channel]);
					weights[i] = quantize_weight(quant, project(k, m, t));
				}
			}
		}


		template<int32 BLOCK_TEXEL_COUNT>
		void calculate_quantized_weights_rgb(const unorm8_t texels[BLOCK_TEXEL_COUNT],
			range_t quant,
			vec3i_t e0,
			vec3i_t e1,
			uint8 weights[BLOCK_TEXEL_COUNT])
		{
			if (e0 == e1)
			{
				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					weights[i] = 0;  // quantize_weight(quant, 0) is always 0
				}
			}
			else
			{
				vec3i_t k = e1 - e0;
				vec3i_t m = e0;

				int kk = dot(k, k);
				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					weights[i] = quantize_weight(quant, project(k, kk, m, to_vec3i(texels[i])));
				}
			}
		}

		template<int32 BLOCK_TEXEL_COUNT>
		void calculate_quantized_weights_rgba(const unorm8_t texels[BLOCK_TEXEL_COUNT],
			range_t quant,
			vec4i_t e0,
			vec4i_t e1,
			uint8 weights[BLOCK_TEXEL_COUNT])
		{
			if (e0 == e1)
			{
				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					weights[i] = 0;  // quantize_weight(quant, 0) is always 0
				}
			}
			else
			{
				vec4i_t k = e1 - e0;
				vec4i_t m = e0;

				int kk = dot(k, k);
				for (size_t i = 0; i < BLOCK_TEXEL_COUNT; ++i)
				{
					weights[i] = quantize_weight(quant, project(k, kk, m, to_vec4i(texels[i])));
				}
			}
		}

#ifdef UE_MIRO_DEBUG
		// This should be compiled out, together with the actual tables.
		range_t endpoint_quantization(int32 WeightX, int32 WeightY, int32 Partitions, range_t weight_quant, color_endpoint_mode_t endpoint_mode, bool bDualPlane)
		{
			// Brute force: shouldn't happen at runtime
			int8 ce_range = RANGE_MAX;

			miro_check(!bDualPlane || Partitions < 4);
			int32 ConfigBits = 17;
			if (Partitions > 1)
			{
				check(false);
			}

			int32 WeightZ = 1;
			int32 NumWeights = WeightX * WeightY * WeightZ;
			miro_check(NumWeights <= 64);

			if (bDualPlane)
			{
				ConfigBits += 2;
				NumWeights *= 2;
			}

			int32 WeightBits = compute_ise_bitcount(NumWeights, weight_quant);
			miro_check(WeightBits >= 24);
			miro_check(WeightBits <= MAXIMUM_ENCODED_WEIGHT_BITS);
			int32 RemainingBits = 128 - ConfigBits - WeightBits;

			int32 CEMValueCount = cem_value_count[endpoint_mode] * Partitions;
			miro_check(CEMValueCount <= 18);

			for (ce_range = RANGE_MAX - 1; ce_range >= 0; --ce_range)
			{
				int32 RangeBits = compute_ise_bitcount(CEMValueCount, static_cast<range_t>(ce_range));
				if (RangeBits <= RemainingBits)
				{
					break;
				}
			}

			miro_check(ce_range >= 0 && ce_range <= RANGE_MAX);
			return static_cast<range_t>(ce_range);
		}
#endif


		/** Write void extent block bits for LDR mode and unused extent coordinates. */
		void encode_void_extent(vec3i_t color, PhysicalBlock* physical_block)
		{
			void_extent_to_physical(unorm8_to_unorm16(to_unorm8(color)), physical_block);
		}


		void encode_void_extent(vec4i_t color, PhysicalBlock* physical_block)
		{
			void_extent_to_physical(unorm8_to_unorm16(to_unorm8(color)), physical_block);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void encode_luminance(const uint8 texels[BLOCK_WIDTH * BLOCK_HEIGHT], PhysicalBlock* physical_block)
		{
			size_t partition_count = 1;
			bool bDualPlane = false;

			color_endpoint_mode_t color_endpoint_mode = CEM_LDR_LUMINANCE_DIRECT;
			range_t weight_quant = RANGE_MAX;
			range_t endpoint_quant = RANGE_MAX;

			uint8 l0 = 255;
			uint8 l1 = 0;
			for (size_t i = 0; i < BLOCK_WIDTH * BLOCK_HEIGHT; ++i)
			{
				l0 = FMath::Min(l0, texels[i]);
				l1 = FMath::Max(l1, texels[i]);
			}

			uint8 endpoint_unquantized[2];
			uint8 endpoint_quantized[2];
			constexpr uint8 NumEndpointValues = 2;

			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				// Hack because we have excess bits with the encoding that we support. It should be RANGE_48, but we only support up to RANGE_32.
				check(false);
				weight_quant = RANGE_32;
				endpoint_quant = RANGE_256;
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				weight_quant = RANGE_6;
				endpoint_quant = RANGE_256;
			}
			else if (BLOCK_WIDTH == 8 && BLOCK_HEIGHT == 8)
			{
				weight_quant = RANGE_2;
				endpoint_quant = RANGE_256;
			}
			else
			{
				check(false);
			}

#ifdef UE_MIRO_DEBUG
			range_t ExpectedEndpointQuant = endpoint_quantization(BLOCK_WIDTH, BLOCK_HEIGHT, partition_count, weight_quant, color_endpoint_mode, bDualPlane);
			miro_check(endpoint_quant == ExpectedEndpointQuant);
#endif

			encode_luminance_direct(endpoint_quant, l0, l1, endpoint_quantized, endpoint_unquantized);

			uint8 weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_luminance<BLOCK_WIDTH* BLOCK_HEIGHT>(
				texels, weight_quant, endpoint_unquantized[0], endpoint_unquantized[1], weights_quantized);

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(endpoint_quantized, NumEndpointValues, endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(weights_quantized, BLOCK_WIDTH * BLOCK_HEIGHT, weight_quant, weights_ise);

			symbolic_to_physical<BLOCK_WIDTH, BLOCK_HEIGHT>(color_endpoint_mode, endpoint_quant, weight_quant,
				partition_count, endpoint_ise,
				weights_ise, physical_block);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void encode_rgb_single_partition(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT],
			vec3f_t e0,
			vec3f_t e1,
			PhysicalBlock* physical_block)
		{
			constexpr size_t partition_count = 1;
			bool bDualPlane = false;

			color_endpoint_mode_t color_endpoint_mode = CEM_LDR_RGB_DIRECT;
			range_t weight_quant = RANGE_MAX;
			range_t endpoint_quant = RANGE_MAX;

			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				endpoint_quant = RANGE_256;
				weight_quant = RANGE_12;
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				endpoint_quant = RANGE_256;
				weight_quant = RANGE_3;
			}
			else if (BLOCK_WIDTH == 8 && BLOCK_HEIGHT == 8)
			{
				endpoint_quant = RANGE_192;
				weight_quant = RANGE_2;
			}
			else
			{
				check(false);
			}

#ifdef UE_MIRO_DEBUG
			range_t ExpectedEndpointQuant = endpoint_quantization(BLOCK_WIDTH, BLOCK_HEIGHT, partition_count, weight_quant, color_endpoint_mode, bDualPlane);
			miro_check(endpoint_quant == ExpectedEndpointQuant);
#endif

			vec3i_t endpoint_unquantized[2];
			uint8 endpoint_quantized[6];
			vec3i_t endpoint0 = round(e0);
			vec3i_t endpoint1 = round(e1);

			encode_rgb_direct(endpoint_quant, endpoint0, endpoint1, endpoint_quantized, endpoint_unquantized);

			uint8 weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_rgb<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant, endpoint_unquantized[0], endpoint_unquantized[1], weights_quantized);

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(endpoint_quantized, 6, endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(weights_quantized, BLOCK_WIDTH * BLOCK_HEIGHT, weight_quant, weights_ise);

			symbolic_to_physical<BLOCK_WIDTH, BLOCK_HEIGHT>(color_endpoint_mode, endpoint_quant, weight_quant, partition_count, endpoint_ise, weights_ise, physical_block);
		}



		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void encode_rgba_single_partition_dual_plane(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT],
			vec4f_t e0,
			vec4f_t e1,
			PhysicalBlock* physical_block)
		{
			size_t partition_count = 1;
			bool bDualPlane = false;

			color_endpoint_mode_t color_endpoint_mode = CEM_LDR_RGBA_DIRECT;

			range_t weight_quant = RANGE_MAX;
			range_t endpoint_quant = RANGE_MAX;
			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				weight_quant = RANGE_5;
				endpoint_quant = RANGE_16;
				// weight_quant = RANGE_4;
				// endpoint_quant = RANGE_48;
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				weight_quant = RANGE_2;
				endpoint_quant = RANGE_24;
			}
			else
			{
				check(false);
			}

			// There is only one valid option for endpoint_quant
#ifdef UE_MIRO_DEBUG
			range_t ExpectedEndpointQuant = endpoint_quantization(BLOCK_WIDTH, BLOCK_HEIGHT, partition_count, weight_quant, color_endpoint_mode, bDualPlane);
			miro_check(endpoint_quant == ExpectedEndpointQuant);
#endif

			vec4i_t endpoint_unquantized[2];
			uint8 endpoint_quantized[8];
			vec4i_t endpoint0 = round(e0);
			vec4i_t endpoint1 = round(e1);
			encode_rgba_direct(endpoint_quant, endpoint0, endpoint1, endpoint_quantized, endpoint_unquantized, true);

			uint8 weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_rgb<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant,
				endpoint_unquantized[0].rgb(),
				endpoint_unquantized[1].rgb(),
				weights_quantized);


			uint8 alpha_weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_channel<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant,
				(uint8)endpoint_unquantized[0].a,
				(uint8)endpoint_unquantized[1].a,
				alpha_weights_quantized,
				3);

			uint8 final_weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT * 2];
			for (size_t i = 0; i < BLOCK_WIDTH * BLOCK_HEIGHT; i++)
			{
				final_weights_quantized[2 * i] = weights_quantized[i];
				final_weights_quantized[2 * i + 1] = alpha_weights_quantized[i];
			}

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(endpoint_quantized, 8, endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(final_weights_quantized, BLOCK_WIDTH * BLOCK_HEIGHT * 2, weight_quant, weights_ise);

			symbolic_to_physical<BLOCK_WIDTH, BLOCK_HEIGHT>(color_endpoint_mode, endpoint_quant, weight_quant,
				partition_count, endpoint_ise,
				weights_ise, physical_block, true);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void encode_rgba_single_partition(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT],
			vec4f_t e0, vec4f_t e1,
			PhysicalBlock* physical_block,
			bool bDualPlane)
		{
			size_t partition_count = 1;

			color_endpoint_mode_t color_endpoint_mode = CEM_LDR_RGBA_DIRECT;

			range_t weight_quant = RANGE_MAX;
			range_t endpoint_quant = RANGE_MAX;
			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				endpoint_quant = RANGE_192;
				weight_quant = RANGE_8;
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				endpoint_quant = RANGE_24;
				weight_quant = RANGE_4;
				//endpoint_quant = RANGE_80;
				//weight_quant = RANGE_3;
			}
			else if (BLOCK_WIDTH == 8 && BLOCK_HEIGHT == 8)
			{
				endpoint_quant = RANGE_48;
				weight_quant = RANGE_2;
			}
			else
			{
				check(false);
			}

#ifdef UE_MIRO_DEBUG
			range_t ExpectedEndpointQuant = endpoint_quantization(BLOCK_WIDTH, BLOCK_HEIGHT, partition_count, weight_quant, color_endpoint_mode, bDualPlane);
			miro_check(endpoint_quant == ExpectedEndpointQuant);
#endif

			vec4i_t endpoint_unquantized[2];
			uint8 endpoint_quantized[8];
			vec4i_t endpoint0 = round(e0);
			vec4i_t endpoint1 = round(e1);
			encode_rgba_direct(endpoint_quant, endpoint0, endpoint1, endpoint_quantized, endpoint_unquantized, false);

			uint8 weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_rgba<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant,
				endpoint_unquantized[0],
				endpoint_unquantized[1],
				weights_quantized);

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(endpoint_quantized, 8, endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(weights_quantized, BLOCK_WIDTH * BLOCK_HEIGHT, weight_quant, weights_ise);

			symbolic_to_physical<BLOCK_WIDTH, BLOCK_HEIGHT>(color_endpoint_mode, endpoint_quant, weight_quant, partition_count,
				endpoint_ise, weights_ise,
				physical_block, bDualPlane);
		}


		void encode_rgba_single_partition_8x8_infill_7x7(const unorm8_t texels[8 * 8],
			vec4f_t e0, vec4f_t e1,
			PhysicalBlock* physical_block,
			bool bDualPlane)
		{
			size_t partition_count = 1;
			// texel matrix
			constexpr int32 BLOCK_X = 8;
			constexpr int32 BLOCK_Y = 8;
			// weight matrix
			constexpr int32 GRID_X = 7;
			constexpr int32 GRID_Y = 7;

			FVector3f ve0(e0.r, e0.g, e0.b);
			FVector3f ve1(e1.r, e1.g, e1.b);

			// Resample texels to the weight grid.
			// Reference implementation
			unorm8_t grid_texels[GRID_X * GRID_Y];
			for (int32 gy = 0; gy < GRID_Y; ++gy)
			{
				for (int32 gx = 0; gx < GRID_X; ++gx)
				{
					int32 tx_16 = ((gx << 16) / GRID_X) * BLOCK_X;
					int32 ty_16 = ((gy << 16) / GRID_Y) * BLOCK_Y;
					int32 wx_16 = tx_16 & 0xffff;
					int32 wy_16 = ty_16 & 0xffff;

					int32 tx0 = (tx_16 >> 16);
					int32 tx1 = FMath::Min(tx0 + 1, BLOCK_X - 1);
					int32 ty0 = (ty_16 >> 16);
					int32 ty1 = FMath::Min(ty0 + 1, BLOCK_Y - 1);

					int32 offset_00 = ty0 * BLOCK_X + tx0;
					miro_check(offset_00 < BLOCK_X * BLOCK_Y);
					int32 offset_10 = ty0 * BLOCK_X + tx1;
					miro_check(offset_10 < BLOCK_X * BLOCK_Y);
					int32 offset_01 = ty1 * BLOCK_X + tx0;
					miro_check(offset_01 < BLOCK_X * BLOCK_Y);
					int32 offset_11 = ty1 * BLOCK_X + tx1;
					miro_check(offset_11 < BLOCK_X * BLOCK_Y);

					vec4i_t e0_16 = to_vec4i(texels[offset_00]) * (0x10000 - wx_16) + to_vec4i(texels[offset_10]) * wx_16;
					vec4i_t e1_16 = to_vec4i(texels[offset_01]) * (0x10000 - wx_16) + to_vec4i(texels[offset_11]) * wx_16;

					vec4i_t v_16 = (e0_16 / 0x10000) * (0x10000 - wy_16) + (e1_16 / 0x10000) * wy_16;

					unorm8_t InterpolatedTexel = to_unorm8(v_16 / 0x10000);

					int32 OffsetGrid = gy * GRID_X + gx;
					// bleh hack
					//int32 OffsetGrid = gy * GRID_X + (GRID_X-gx-1);

					grid_texels[OffsetGrid] = InterpolatedTexel;
				}
			}

			// Look for a good endpoint encoding option.
			struct FEncodingOption
			{
				range_t weight_quant = RANGE_MAX;
				range_t endpoint_quant = RANGE_MAX;
				color_endpoint_mode_t color_endpoint_mode = CEM_MAX;
				int32 EndpointValueCount = 0;
				vec4i_t endpoint0;
				vec4i_t endpoint1;
				vec4i_t endpoint_unquantized[2];
				uint8 endpoint_quantized[8];
				bool bValid = false;
			};

			constexpr int32 MaxOptionCount = 4;
			FEncodingOption Options[MaxOptionCount];

			//constexpr float GreyscaleThreshold = 0.4f;
			//float Luminance0 = (e0.r + e0.g + e0.b) / 3.0f;
			//float Luminance1 = (e1.r + e1.g + e1.b) / 3.0f;
			//bool bIsGreyscale = FMath::IsNearlyEqual(e0.r, Luminance0, GreyscaleThreshold) && FMath::IsNearlyEqual(e0.g, Luminance0, 0.4f) && FMath::IsNearlyEqual(e0.b, Luminance0, GreyscaleThreshold)
			//	&& FMath::IsNearlyEqual(e1.r, Luminance1, GreyscaleThreshold) && FMath::IsNearlyEqual(e1.g, Luminance1, GreyscaleThreshold) && FMath::IsNearlyEqual(e1.b, Luminance1, GreyscaleThreshold);

			constexpr float WhiteThreshold = 2.0f;
			bool bAlphaWhite = FMath::IsNearlyEqual(e0.a, 255.0f, WhiteThreshold) && FMath::IsNearlyEqual(e1.a, 255.0f, WhiteThreshold);

			//constexpr float LinearThreshold = 0.01f;
			//bool bRGBLinear = false;
			//if (ve1.GetMin() > UE_SMALL_NUMBER)
			//{
			//	vec4f_t RatioV = e0 / e1;
			//	float RatioAverage = (RatioV.r + RatioV.g + RatioV.b) / 3.0f;
			//	bRGBLinear = FMath::IsNearlyEqual(RatioV.r, RatioAverage, LinearThreshold) && FMath::IsNearlyEqual(RatioV.g, RatioAverage, LinearThreshold) && FMath::IsNearlyEqual(RatioV.b, RatioAverage, LinearThreshold);
			//}

			int32 LastOption = 0;

			if (bAlphaWhite)
			{
				// There is no need to encode the alpha

				// another option
				{
					check(LastOption<MaxOptionCount);
					FEncodingOption& Option = Options[LastOption++];

					Option.color_endpoint_mode = CEM_LDR_LUMINANCE_DIRECT;
					Option.EndpointValueCount = 2;
					Option.weight_quant = RANGE_3;
					Option.endpoint_quant = RANGE_256;
					Option.endpoint0 = round(e0);
					Option.endpoint1 = round(e1);

					uint8 endpoint_rgb_unquantized[2];
					encode_luminance_direct(Option.endpoint_quant, Option.endpoint0.r, Option.endpoint1.r, Option.endpoint_quantized, endpoint_rgb_unquantized);

					Option.endpoint_unquantized[0].r = endpoint_rgb_unquantized[0];
					Option.endpoint_unquantized[0].g = endpoint_rgb_unquantized[0];
					Option.endpoint_unquantized[0].b = endpoint_rgb_unquantized[0];
					Option.endpoint_unquantized[0].a = 255;
					Option.endpoint_unquantized[1].r = endpoint_rgb_unquantized[1];
					Option.endpoint_unquantized[1].g = endpoint_rgb_unquantized[1];
					Option.endpoint_unquantized[1].b = endpoint_rgb_unquantized[1];
					Option.endpoint_unquantized[1].a = 255;
				}

				// another option
				//if (bRGBLinear)
				{
					check(LastOption < MaxOptionCount);
					FEncodingOption& Option = Options[LastOption++];

					Option.color_endpoint_mode = CEM_LDR_RGB_BASE_SCALE;
					Option.EndpointValueCount = 4;
					Option.weight_quant = RANGE_3;
					Option.endpoint_quant = RANGE_256;
					Option.endpoint0 = round(e0);
					Option.endpoint1 = round(e1);

					int32 MaxComponent = (ve1[0] > ve1[1])
						? ((ve1[0] > ve1[2]) ? 0 : ((ve1[1] > ve1[2]) ? 1 : 2))
						: ((ve1[1] > ve1[2]) ? 1 : 2);
					float Ratio = 1.0;
					if (ve1[MaxComponent] > UE_SMALL_NUMBER)
					{
						Ratio = ve0[MaxComponent] / ve1[MaxComponent];
					}
					uint8 Scale = 0;
					if (Ratio < 1.0f)
					{
						Scale = static_cast<int8>(Ratio * 255.0f);
					}
					else
					{
						Scale = static_cast<int8>((1.0f / Ratio) * 255.0f);
						Swap(Option.endpoint0, Option.endpoint1);
					}

					vec3i_t endpoint_rgb_unquantized[2];
					encode_rgb_base_scale(Option.endpoint_quant, Option.endpoint1.rgb(), Scale, Option.endpoint_quantized, endpoint_rgb_unquantized);

					Option.endpoint_unquantized[0].r = endpoint_rgb_unquantized[0].r;
					Option.endpoint_unquantized[0].g = endpoint_rgb_unquantized[0].g;
					Option.endpoint_unquantized[0].b = endpoint_rgb_unquantized[0].b;
					Option.endpoint_unquantized[0].a = 255;
					Option.endpoint_unquantized[1].r = endpoint_rgb_unquantized[1].r;
					Option.endpoint_unquantized[1].g = endpoint_rgb_unquantized[1].g;
					Option.endpoint_unquantized[1].b = endpoint_rgb_unquantized[1].b;
					Option.endpoint_unquantized[1].a = 255;
				}

				// another option
				{
					check(LastOption < MaxOptionCount);
					FEncodingOption& Option = Options[LastOption++];

					Option.color_endpoint_mode = CEM_LDR_RGB_DIRECT;
					Option.EndpointValueCount = 6;
					Option.weight_quant = RANGE_2;
					Option.endpoint_quant = RANGE_256;
					Option.endpoint0 = round(e0);
					Option.endpoint1 = round(e1);

					vec3i_t endpoint_rgb_unquantized[2];
					encode_rgb_direct(Option.endpoint_quant, Option.endpoint0, Option.endpoint1, Option.endpoint_quantized, endpoint_rgb_unquantized);

					Option.endpoint_unquantized[0].r = endpoint_rgb_unquantized[0].r;
					Option.endpoint_unquantized[0].g = endpoint_rgb_unquantized[0].g;
					Option.endpoint_unquantized[0].b = endpoint_rgb_unquantized[0].b;
					Option.endpoint_unquantized[0].a = 255;
					Option.endpoint_unquantized[1].r = endpoint_rgb_unquantized[1].r;
					Option.endpoint_unquantized[1].g = endpoint_rgb_unquantized[1].g;
					Option.endpoint_unquantized[1].b = endpoint_rgb_unquantized[1].b;
					Option.endpoint_unquantized[1].a = 255;
				}

			}
			else
			{
				// luminance and alpha.
				{
					check(LastOption < MaxOptionCount);
					FEncodingOption& Option = Options[LastOption++];

					Option.color_endpoint_mode = CEM_LDR_LUMINANCE_ALPHA_DIRECT;
					Option.EndpointValueCount = 4;
					Option.weight_quant = RANGE_3;
					Option.endpoint_quant = RANGE_256;
					Option.endpoint0 = round(e0);
					Option.endpoint1 = round(e1);

					encode_la_direct(Option.endpoint_quant, Option.endpoint0, Option.endpoint1, Option.endpoint_quantized, Option.endpoint_unquantized);
				}

				// most generic option.
				{
					check(LastOption < MaxOptionCount);
					FEncodingOption& Option = Options[LastOption++];
	
					Option.color_endpoint_mode = CEM_LDR_RGBA_DIRECT;
					Option.EndpointValueCount = 8;
					Option.weight_quant = RANGE_2;
					Option.endpoint_quant = RANGE_192;
					Option.endpoint0 = round(e0);
					Option.endpoint1 = round(e1);
	
					encode_rgba_direct(Option.endpoint_quant, Option.endpoint0, Option.endpoint1, Option.endpoint_quantized, Option.endpoint_unquantized, false);
				}
			}
			check(LastOption <= MaxOptionCount);

			int32 MinError = TNumericLimits<int32>::Max();
			int32 MinErrorOptionIndex = -1;

			for (int32 o = 0; o < LastOption; ++o)
			{
				FEncodingOption& Option = Options[o];

#ifdef UE_MIRO_DEBUG
				range_t ExpectedEndpointQuant = endpoint_quantization(GRID_X, GRID_Y, partition_count, Option.weight_quant, Option.color_endpoint_mode, bDualPlane);
				miro_check(Option.endpoint_quant == ExpectedEndpointQuant);
#endif

				int32 EndpointError = quadrance(Option.endpoint0 - Option.endpoint_unquantized[0])
					+ quadrance(Option.endpoint1 - Option.endpoint_unquantized[1]);

				// Heuristic to compensate for the bigger weight precision in some options.
				//const int32 RangeErrorFactor[RANGE_MAX] = { 128, 85, 64, 51, 42, 32, 25, 20, 16, 12, 10, 8, 6, 5, 4, 1, 1, 1, 1, 1, 1 };
				//OptionError *= RangeErrorFactor[Option.weight_quant];

				if (EndpointError < MinError)
				{
					MinError = EndpointError;
					MinErrorOptionIndex = o;
				}
			}

			if (MinErrorOptionIndex < 0 || MinErrorOptionIndex >= LastOption)
			{
				check(false);
				return;
			}

			const FEncodingOption& Option = Options[MinErrorOptionIndex];

			uint8 grid_weights_quantized[MAX_BLOCK_WIDTH * MAX_BLOCK_HEIGHT];
			calculate_quantized_weights_rgba<GRID_X* GRID_Y>(grid_texels, Option.weight_quant,
				Option.endpoint_unquantized[0], Option.endpoint_unquantized[1],
				grid_weights_quantized);

#ifdef UE_MIRO_DEBUG
			if (s_DebugBlock == s_CurrentBlock)
			{
				UE_LOGF(LogMutableCore, Log, "endpoints f : (%.3f,%.3f,%.3f,%.3f) (%.3f,%.3f,%.3f,%.3f)",
					e0.r, e0.g, e0.b, e0.a,
					e1.r, e1.g, e1.b, e1.a);
				UE_LOGF(LogMutableCore, Log, "endpoints d : (%3d,%3d,%3d,%3d) (%3d,%3d,%3d,%3d)",
					Option.endpoint_unquantized[0].r, Option.endpoint_unquantized[0].g, Option.endpoint_unquantized[0].b, Option.endpoint_unquantized[0].a,
					Option.endpoint_unquantized[1].r, Option.endpoint_unquantized[1].g, Option.endpoint_unquantized[1].b, Option.endpoint_unquantized[1].a);

				UE_LOGF(LogMutableCore, Log, "texel block pixels :");
				int T = 0;
				for (uint64 Y = 0; Y < BLOCK_Y; ++Y)
				{
					FString Line;
					for (uint64 X = 0; X < BLOCK_X; ++X)
					{
						vec4i_t texel = to_vec4i(texels[T++]);
						Line += FString::Printf(TEXT("(%3d,%3d,%3d,%3d), "), texel.r, texel.g, texel.b, texel.a);
					}

					UE_LOGF(LogMutableCore, Log, "%ls", *Line);
				}

				UE_LOGF(LogMutableCore, Log, "texel grid pixels :");
				T = 0;
				for (uint64 Y = 0; Y < GRID_Y; ++Y)
				{
					FString Line;
					for (uint64 X = 0; X < GRID_X; ++X)
					{
						vec4i_t texel = to_vec4i(grid_texels[T++]);
						Line += FString::Printf(TEXT("(%3d,%3d,%3d,%3d), "), texel.r, texel.g, texel.b, texel.a);
					}

					UE_LOGF(LogMutableCore, Log, "%ls", *Line);
				}

				UE_LOGF(LogMutableCore, Log, "texel grid encoded :");
				T = 0;
				for (uint64 Y = 0; Y < GRID_Y; ++Y)
				{
					FString Line;
					for (uint64 X = 0; X < GRID_X; ++X)
					{
						uint8 texel = grid_weights_quantized[T++];
						Line += FString::Printf(TEXT("%3d, "), texel);
					}

					UE_LOGF(LogMutableCore, Log, "%ls", *Line);
				}
			}
#endif

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(Option.endpoint_quantized, Option.EndpointValueCount, Option.endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(grid_weights_quantized, GRID_X * GRID_Y, Option.weight_quant, weights_ise);

			symbolic_to_physical(GRID_X, GRID_Y, Option.color_endpoint_mode, Option.endpoint_quant, Option.weight_quant, partition_count,
				endpoint_ise, weights_ise,
				physical_block, bDualPlane);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void encode_la_single_partition(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT],
			vec3f_t e0,
			vec3f_t e1,
			PhysicalBlock* physical_block)
		{
			size_t partition_count = 1;
			bool bDualPlane = false;

			color_endpoint_mode_t color_endpoint_mode = CEM_LDR_LUMINANCE_ALPHA_DIRECT;
			range_t weight_quant = RANGE_MAX;
			range_t endpoint_quant = RANGE_MAX;

			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				weight_quant = RANGE_12;
				endpoint_quant = RANGE_256;
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				weight_quant = RANGE_4;
				endpoint_quant = RANGE_256;
			}
			else if (BLOCK_WIDTH == 8 && BLOCK_HEIGHT == 8)
			{
				weight_quant = RANGE_2;
				endpoint_quant = RANGE_256;
			}
			else
			{
				miro_check(false);
			}

#ifdef UE_MIRO_DEBUG
			range_t ExpectedEndpointQuant = endpoint_quantization(BLOCK_WIDTH, BLOCK_HEIGHT, partition_count, weight_quant, color_endpoint_mode, bDualPlane);
			miro_check(endpoint_quant == ExpectedEndpointQuant);
#endif

			vec3i_t endpoint_unquantized[2];
			uint8 endpoint_quantized[6];
			vec3i_t endpoint0 = round(e0);
			vec3i_t endpoint1 = round(e1);
			encode_rgb_direct(endpoint_quant, endpoint0, endpoint1, endpoint_quantized, endpoint_unquantized);

			uint8 weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			// TODO: optimize with rg skipping b
			calculate_quantized_weights_rgb<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant, endpoint_unquantized[0], endpoint_unquantized[1], weights_quantized);

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(endpoint_quantized, 4, endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(weights_quantized, BLOCK_WIDTH * BLOCK_HEIGHT, weight_quant, weights_ise);

			symbolic_to_physical<BLOCK_WIDTH, BLOCK_HEIGHT>(color_endpoint_mode, endpoint_quant, weight_quant,
				partition_count, endpoint_ise,
				weights_ise, physical_block, bDualPlane);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void encode_la_single_partition_dual_plane(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT],
			vec4f_t e0,
			vec4f_t e1,
			PhysicalBlock* physical_block)
		{
			size_t partition_count = 1;
			bool bDualPlane = true;

			color_endpoint_mode_t color_endpoint_mode = CEM_LDR_LUMINANCE_ALPHA_DIRECT;
			range_t weight_quant = RANGE_MAX;
			range_t endpoint_quant = RANGE_MAX;

			if (BLOCK_WIDTH == 4 && BLOCK_HEIGHT == 4)
			{
				weight_quant = RANGE_6;
				endpoint_quant = RANGE_64;
			}
			else if (BLOCK_WIDTH == 6 && BLOCK_HEIGHT == 6)
			{
				weight_quant = RANGE_2;
				endpoint_quant = RANGE_256;
			}
			else
			{
				miro_check(false);
			}

#ifdef UE_MIRO_DEBUG
			range_t ExpectedEndpointQuant = endpoint_quantization(BLOCK_WIDTH, BLOCK_HEIGHT, partition_count, weight_quant, color_endpoint_mode, bDualPlane);
			miro_check(endpoint_quant == ExpectedEndpointQuant);
#endif

			vec4i_t endpoint_unquantized[2];
			uint8 endpoint_quantized[8];
			vec4i_t endpoint0 = round(e0);
			vec4i_t endpoint1 = round(e1);
			encode_rgba_direct(endpoint_quant, endpoint0, endpoint1, endpoint_quantized, endpoint_unquantized, true);

			uint8 r_weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_channel<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant,
				(uint8)endpoint_unquantized[0].r,
				(uint8)endpoint_unquantized[1].r,
				r_weights_quantized,
				2);


			uint8 g_weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT];
			calculate_quantized_weights_channel<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, weight_quant,
				(uint8)endpoint_unquantized[0].g,
				(uint8)endpoint_unquantized[1].g,
				g_weights_quantized,
				1);

			uint8 final_weights_quantized[BLOCK_WIDTH * BLOCK_HEIGHT * 2];
			for (size_t i = 0; i < BLOCK_WIDTH * BLOCK_HEIGHT; i++)
			{
				final_weights_quantized[2 * i] = r_weights_quantized[i];
				final_weights_quantized[2 * i + 1] = g_weights_quantized[i];
			}

			uint8 endpoint_ise[MAXIMUM_ENCODED_COLOR_ENDPOINT_BYTES] = { 0 };
			integer_sequence_encode(endpoint_quantized, 4, endpoint_quant, endpoint_ise);

			uint8 weights_ise[MAXIMUM_ENCODED_WEIGHT_BYTES + 1] = { 0 };
			integer_sequence_encode(final_weights_quantized, BLOCK_WIDTH * BLOCK_HEIGHT * 2, weight_quant, weights_ise);

			symbolic_to_physical<BLOCK_WIDTH, BLOCK_HEIGHT>(color_endpoint_mode, endpoint_quant, weight_quant,
				partition_count, endpoint_ise,
				weights_ise, physical_block, bDualPlane);
		}


		bool is_solid(const unorm8_t* texels, size_t count, unorm8_t* color, int32 Epsilon)
		{
			for (size_t i = 0; i < count; ++i)
			{
				if (!approx_equal(to_vec3i(texels[i]), to_vec3i(texels[0]), Epsilon))
				{
					return false;
				}
			}

			// TODO: Calculate average color?
			*color = texels[0];
			return true;
		}


		bool is_solid_rgba(const unorm8_t* texels, size_t count, unorm8_t* color, int32 Epsilon)
		{
			for (size_t i = 0; i < count; ++i)
			{
				if (!approx_equal(to_vec4i(texels[i]), to_vec4i(texels[0]), Epsilon))
				{
					return false;
				}
			}

			// TODO: Calculate average color?
			*color = texels[0];
			return true;
		}


		bool is_greyscale(const unorm8_t* texels, size_t count, uint8* luminances, int32 Epsilon)
		{
			for (size_t i = 0; i < count; ++i)
			{
				vec3i_t color = to_vec3i(texels[i]);
				luminances[i] = static_cast<uint8>(luminance(color));
				vec3i_t lum(luminances[i], luminances[i], luminances[i]);
				if (!approx_equal(color, lum, Epsilon))
				{
					return false;
				}
			}

			return true;
		}


		struct mat3x3f_t
		{
		public:
			mat3x3f_t() {}

			mat3x3f_t(float m00, float m01, float m02,
				float m10, float m11, float m12,
				float m20, float m21, float m22)
			{
				m[0] = vec3f_t(m00, m01, m02);
				m[1] = vec3f_t(m10, m11, m12);
				m[2] = vec3f_t(m20, m21, m22);
			}

			const vec3f_t& row(size_t i) const { return m[i]; }

			float& at(size_t i, size_t j) { return m[i].components(j); }
			const float& at(size_t i, size_t j) const { return m[i].components(j); }

		private:
			vec3f_t m[3];
		};


		struct mat4x4f_t
		{
		public:
			mat4x4f_t() {}

			mat4x4f_t(float m00, float m01, float m02, float m03,
				float m10, float m11, float m12, float m13,
				float m20, float m21, float m22, float m23,
				float m30, float m31, float m32, float m33)
			{
				m[0] = vec4f_t(m00, m01, m02, m03);
				m[1] = vec4f_t(m10, m11, m12, m13);
				m[2] = vec4f_t(m20, m21, m22, m23);
				m[3] = vec4f_t(m30, m31, m32, m33);
			}

			const vec4f_t& row(size_t i) const { return m[i]; }

			float& at(size_t i, size_t j) { return m[i].components(j); }
			const float& at(size_t i, size_t j) const { return m[i].components(j); }

		private:
			vec4f_t m[4];
		};

		inline vec3f_t operator*(const mat3x3f_t& a, vec3f_t b)
		{
			vec3f_t tmp;
			tmp.r = dot(a.row(0), b);
			tmp.g = dot(a.row(1), b);
			tmp.b = dot(a.row(2), b);
			return tmp;
		}

		inline vec4f_t operator*(const mat4x4f_t& a, vec4f_t b)
		{
			vec4f_t tmp;
			tmp.r = dot(a.row(0), b);
			tmp.g = dot(a.row(1), b);
			tmp.b = dot(a.row(2), b);
			tmp.a = dot(a.row(3), b);
			return tmp;
		}

		vec3f_t mean(const unorm8_t* texels, size_t count)
		{
			vec3i_t sum(0, 0, 0);
			for (size_t i = 0; i < count; ++i)
			{
				sum = sum + to_vec3i(texels[i]);
			}

			return to_vec3f(sum) / static_cast<float>(count);
		}

		vec4f_t mean4(const unorm8_t* texels, size_t count)
		{
			vec4i_t sum(0, 0, 0, 0);
			for (size_t i = 0; i < count; ++i)
			{
				sum = sum + to_vec4i(texels[i]);
			}

			return to_vec4f(sum) / static_cast<float>(count);
		}

		void subtract(const unorm8_t* texels, size_t count, vec3f_t v, vec3f_t* output)
		{
			for (size_t i = 0; i < count; ++i)
			{
				output[i] = to_vec3f(texels[i]) - v;
			}
		}

		void subtract(const unorm8_t* texels, size_t count, vec4f_t v, vec4f_t* output)
		{
			for (size_t i = 0; i < count; ++i)
			{
				output[i] = to_vec4f(texels[i]) - v;
			}
		}

		mat3x3f_t covariance(const vec3f_t* m, size_t count)
		{
			mat3x3f_t cov;
			for (size_t i = 0; i < 3; ++i)
			{
				for (size_t j = 0; j < 3; ++j)
				{
					float s = 0;
					for (size_t k = 0; k < count; ++k)
					{
						s += m[k].components(i) * m[k].components(j);
					}
					cov.at(i, j) = s / static_cast<float>(count - 1);
				}
			}

			return cov;
		}

		mat4x4f_t covariance(const vec4f_t* m, size_t count)
		{
			mat4x4f_t cov;
			for (size_t i = 0; i < 4; ++i)
			{
				for (size_t j = 0; j < 4; ++j)
				{
					float s = 0;
					for (size_t k = 0; k < count; ++k)
					{
						s += m[k].components(i) * m[k].components(j);
					}
					cov.at(i, j) = s / static_cast<float>(count - 1);
				}
			}

			return cov;
		}

		void eigen_vector(const mat3x3f_t& a, vec3f_t& eig)
		{
			vec3f_t b = signorm(vec3f_t(1, 3, 2));  // FIXME: Magic number
			for (size_t i = 0; i < 8; ++i)
			{
				b = signorm(a * b);
			}

			eig = b;
		}

		void eigen_vector(const mat4x4f_t& a, vec4f_t& eig)
		{
			vec4f_t b = signorm(vec4f_t(1, 3, 2, 4));  // FIXME: Magic number
			for (size_t i = 0; i < 8; ++i)
			{
				b = signorm(a * b);
			}

			eig = b;
		}


		void find_min_max(const unorm8_t* texels,
			size_t count,
			vec3f_t line_k,
			vec3f_t line_m,
			vec3f_t& e0,
			vec3f_t& e1)
		{
			miro_check(FMath::IsNearlyEqual(quadrance(line_k), 1.0, 0.0001f));

			float a, b;
			{
				float t = dot(to_vec3f(texels[0]) - line_m, line_k);
				a = t;
				b = t;
			}

			for (size_t i = 1; i < count; ++i) {
				float t = dot(to_vec3f(texels[i]) - line_m, line_k);
				a = FMath::Min(a, t);
				b = FMath::Max(b, t);
			}

			e0 = clamp_rgb(line_k * a + line_m);
			e1 = clamp_rgb(line_k * b + line_m);
		}


		void find_min_max(const unorm8_t* texels,
			size_t count,
			vec4f_t line_k,
			vec4f_t line_m,
			vec4f_t& e0,
			vec4f_t& e1)
		{
			miro_check(FMath::IsNearlyEqual(quadrance(line_k), 1.0, 0.0001f));

			float a, b;
			{
				float t = dot(to_vec4f(texels[0]) - line_m, line_k);
				a = t;
				b = t;
			}

			for (size_t i = 1; i < count; ++i) {
				float t = dot(to_vec4f(texels[i]) - line_m, line_k);
				a = FMath::Min(a, t);
				b = FMath::Max(b, t);
			}

			e0 = clamp_rgba(line_k * a + line_m);
			e1 = clamp_rgba(line_k * b + line_m);
		}


		template<int32 BLOCK_TEXEL_COUNT>
		void principal_component_analysis(const unorm8_t texels[BLOCK_TEXEL_COUNT],
			size_t count,
			vec3f_t& line_k,
			vec3f_t& line_m)
		{
			line_m = mean(texels, count);

			vec3f_t n[BLOCK_TEXEL_COUNT];
			subtract(texels, count, line_m, n);

			mat3x3f_t w = covariance(n, count);

			eigen_vector(w, line_k);
		}


		template<int32 BLOCK_TEXEL_COUNT>
		void principal_component_analysis(const unorm8_t texels[BLOCK_TEXEL_COUNT],
			size_t count,
			vec4f_t& line_k,
			vec4f_t& line_m)
		{
			line_m = mean4(texels, count);

			vec4f_t n[BLOCK_TEXEL_COUNT];
			subtract(texels, count, line_m, n);

			mat4x4f_t w = covariance(n, count);

			eigen_vector(w, line_k);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void compress_block(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT], PhysicalBlock* physical_block)
		{
			// Option 1: block mode 10111111100 : void extent block.
			{
				unorm8_t color;
				int32 Epsilon = APPROX_COLOR_EPSILON;
				if (is_solid(texels, BLOCK_WIDTH * BLOCK_HEIGHT, &color, Epsilon))
				{
					encode_void_extent(to_vec3i(color), physical_block);
					return;
				}
			}

			// Option 2: luminance
			// We skip this option for 4x4 beacuse of a bug.
			if (BLOCK_WIDTH>4)
			{
				uint8 luminances[BLOCK_WIDTH * BLOCK_HEIGHT];
				int32 Epsilon = APPROX_COLOR_EPSILON;
				if (is_greyscale(texels, BLOCK_WIDTH * BLOCK_HEIGHT, luminances, Epsilon))
				{
					encode_luminance<BLOCK_WIDTH, BLOCK_HEIGHT>(luminances, physical_block);
					return;
				}
			}

			// Option 3: generic
			vec3f_t k, m;
			principal_component_analysis<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m);
			vec3f_t e0, e1;
			find_min_max(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m, e0, e1);
			encode_rgb_single_partition<BLOCK_WIDTH, BLOCK_HEIGHT>(texels, e0, e1, physical_block);
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void compress_block_rgba(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT], PhysicalBlock* physical_block)
		{
			unorm8_t color;

			// Option 1: block mode 10111111100 : void extent block.

			// \TODO: This actually helps reduce the error due to the bad quality of the non-void extent blocks
			// When that quality improves, reduce this epsilon.
			int32 Epsilon = APPROX_COLOR_EPSILON;

			bool isSolidRGBA = is_solid_rgba(texels, BLOCK_WIDTH * BLOCK_HEIGHT, &color, Epsilon);
			if (isSolidRGBA)
			{
				encode_void_extent(to_vec4i(color), physical_block);
				return;
			}

			// Option 2

			bool bDualPlane = false;
			if (bDualPlane)
			{
				// dual plane usually yields worse results!
				vec3f_t e0, e1;
				bool isSolid = is_solid(texels, BLOCK_WIDTH * BLOCK_HEIGHT, &color, Epsilon);
				if (isSolid)
				{
					e0 = e1 = to_vec3f(texels[0]);
				}
				else
				{
					vec3f_t k, m;
					principal_component_analysis<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m);
					find_min_max(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m, e0, e1);
				}

				// Find alpha min and max
				uint8 l0 = 255;
				uint8 l1 = 0;
				for (size_t i = 0; i < BLOCK_WIDTH * BLOCK_HEIGHT; ++i)
				{
					l0 = FMath::Min(l0, texels[i].components[3]);
					l1 = FMath::Max(l1, texels[i].components[3]);
				}

				vec4f_t re0(e0.r, e0.g, e0.b, float(l0));
				vec4f_t re1(e1.r, e1.g, e1.b, float(l1));

				encode_rgba_single_partition_dual_plane<BLOCK_WIDTH, BLOCK_HEIGHT>(texels, re0, re1, physical_block);
			}
			else
			{
				// Analysing as 4D
				vec4f_t re0, re1;
				vec4f_t k, m;
				principal_component_analysis<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m);
				find_min_max(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m, re0, re1);

				// This would be an RGB analysis with a hacky patch for alpha limits.
				// {
				//vec3f_t e0, e1;
				//bool isSolid = is_solid(texels, BLOCK_WIDTH * BLOCK_HEIGHT, &color, Epsilon);
				//if (isSolid)
				//{
				//	e0 = e1 = to_vec3f(texels[0]);
				//}
				//else
				//{
				//	vec3f_t k, m;
				//	principal_component_analysis<BLOCK_WIDTH* BLOCK_HEIGHT>(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m);
				//	find_min_max(texels, BLOCK_WIDTH * BLOCK_HEIGHT, k, m, e0, e1);
				//}

				//// Find alpha min and max
				//uint8 l0 = 255;
				//uint8 l1 = 0;
				//for (size_t i = 0; i < BLOCK_WIDTH * BLOCK_HEIGHT; ++i)
				//{
				//	l0 = FMath::Min(l0, texels[i].components[3]);
				//	l1 = FMath::Max(l1, texels[i].components[3]);
				//}
				//vec4f_t re0(e0.r, e0.g, e0.b, float(l0));
				//vec4f_t re1(e1.r, e1.g, e1.b, float(l1));
				//}

				if (BLOCK_WIDTH < 7 && BLOCK_HEIGHT < 7)
				{
					// Option 1
					encode_rgba_single_partition<BLOCK_WIDTH, BLOCK_HEIGHT>(texels, re0, re1, physical_block, bDualPlane);
				}
				else
				{
					// Option 2
					encode_rgba_single_partition_8x8_infill_7x7(texels, re0, re1, physical_block, bDualPlane);
				}
			}
		}


		template<int32 BLOCK_WIDTH, int32 BLOCK_HEIGHT>
		void compress_block_la(const unorm8_t texels[BLOCK_WIDTH * BLOCK_HEIGHT], PhysicalBlock* physical_block)
		{
			unorm8_t color;
			int32 Epsilon = APPROX_COLOR_EPSILON;
			bool isSolidRGBA = is_solid_rgba(texels, BLOCK_WIDTH * BLOCK_HEIGHT, &color, Epsilon);
			if (isSolidRGBA)
			{
				encode_void_extent(vec4i_t(color.components[2], color.components[2], color.components[2], color.components[1]), physical_block);
				return;
			}

			uint8 r0 = 255;
			uint8 r1 = 0;
			uint8 g0 = 255;
			uint8 g1 = 0;
			for (size_t i = 0; i < BLOCK_WIDTH * BLOCK_HEIGHT; ++i)
			{
				r0 = FMath::Min(r0, texels[i].components[2]);
				r1 = FMath::Max(r1, texels[i].components[2]);
				g0 = FMath::Min(g0, texels[i].components[1]);
				g1 = FMath::Max(g1, texels[i].components[1]);
			}

			// Avoid dual-plane by default
			bool bUseDualPlane = false;
			if (bUseDualPlane)
			{
				vec4f_t re0(r0, g0, r0, g0);
				vec4f_t re1(r1, g1, r1, g1);
				encode_la_single_partition_dual_plane<BLOCK_WIDTH, BLOCK_HEIGHT>(texels, re0, re1, physical_block);
			}
			else
			{
				vec3f_t re0(r0, g0, 0);
				vec3f_t re1(r1, g1, 0);
				encode_la_single_partition<BLOCK_WIDTH, BLOCK_HEIGHT>(texels, re0, re1, physical_block);
			}
		}

	}


	// Based on astc_dec released under Apache license
	// See: https://github.com/richgel999/astc_dec
	// Modified for integration.
	namespace astcdec
	{

#define DE_LENGTH_OF_ARRAY(x) (sizeof(x)/sizeof(x[0]))
#define DE_UNREF(x) (void)x

		static bool inBounds(int32 v, int32 l, int32 h)
		{
			return (v >= l) && (v < h);
		}

		static bool inRange(int32 v, int32 l, int32 h)
		{
			return (v >= l) && (v <= h);
		}

		using UVec4 = FUint32Vector4;
		using IVec4 = FInt32Vector4;

		inline FUint32Vector4 asUint(FInt32Vector4 v)
		{
			return FUint32Vector4(FMath::Max(0, v[0]), FMath::Max(0, v[1]), FMath::Max(0, v[2]), FMath::Max(0, v[3]));
		}

		static uint32_t deDivRoundUp32(uint32_t a, uint32_t b)
		{
			return (a + b - 1) / b;
		}

		static bool deInBounds32(uint32_t v, uint32_t l, uint32_t h)
		{
			return (v >= l) && (v < h);
		}


		// Common utilities
		enum
		{
			MAX_BLOCK_WIDTH = 12,
			MAX_BLOCK_HEIGHT = 12
		};
		inline uint32 getBit(uint32 src, int ndx)
		{
			check(inBounds(ndx, 0, 32));
			return (src >> ndx) & 1;
		}
		inline uint32 getBits(uint32 src, int low, int high)
		{
			const int numBits = (high - low) + 1;
			check(inRange(numBits, 1, 32));
			if (numBits < 32)
				return (uint32)((src >> low) & ((1u << numBits) - 1));
			else
				return (uint32)((src >> low) & 0xFFFFFFFFu);
		}
		inline bool isBitSet(uint32 src, int ndx)
		{
			return getBit(src, ndx) != 0;
		}

		FORCEINLINE uint32 NewReverseBits(uint32 src, int numBits)
		{
			miro_check(numBits <= 8);

			// If we were to really support 32 bits
			//uint32 ReversedSrc =
			//	(astcrt::bit_reverse_table[(src >>  0) & 0xff] << 24) |
			//	(astcrt::bit_reverse_table[(src >>  8) & 0xff] << 16) |
			//	(astcrt::bit_reverse_table[(src >> 16) & 0xff] <<  8) |
			//	(astcrt::bit_reverse_table[(src >> 24) & 0xff] <<  0);
			//return ReversedSrc >> (32 - numBits);

			uint8 r = astcrt::bit_reverse_table[src];

			// Alernative without table
			//r = src;
			//r = (r & 0xF0) >> 4 | (r & 0x0F) << 4;
			//r = (r & 0xCC) >> 2 | (r & 0x33) << 2;
			//r = (r & 0xAA) >> 1 | (r & 0x55) << 1;

			return r >> (8 - numBits);
		}

		inline uint32 bitReplicationScale(uint32 src, int numSrcBits, int numDstBits)
		{
			check(numSrcBits <= numDstBits);
			check((src & ((1 << numSrcBits) - 1)) == src);
			uint32 dst = 0;
			for (int shift = numDstBits - numSrcBits; shift > -numSrcBits; shift -= numSrcBits)
				dst |= shift >= 0 ? src << shift : src >> -shift;
			return dst;
		}

		inline int32 signExtend(int32 src, int numSrcBits)
		{
			check(inRange(numSrcBits, 2, 31));
			const bool negative = (src & (1 << (numSrcBits - 1))) != 0;
			return src | (negative ? ~((1 << numSrcBits) - 1) : 0);
		}

		enum ISEMode
		{
			ISEMODE_TRIT = 0,
			ISEMODE_QUINT,
			ISEMODE_PLAIN_BIT,
			ISEMODE_LAST
		};

		struct ISEParams
		{
			ISEMode		mode;
			int			numBits;
			ISEParams(ISEMode mode_, int numBits_) : mode(mode_), numBits(numBits_) {}
		};

		inline int computeNumRequiredBits(const ISEParams& iseParams, int numValues)
		{
			switch (iseParams.mode)
			{
			case ISEMODE_TRIT:			return deDivRoundUp32(numValues * 8, 5) + numValues * iseParams.numBits;
			case ISEMODE_QUINT:			return deDivRoundUp32(numValues * 7, 3) + numValues * iseParams.numBits;
			case ISEMODE_PLAIN_BIT:		return numValues * iseParams.numBits;
			default:
				check(false);
				return -1;
			}
		}

		ISEParams computeMaximumRangeISEParams(int numAvailableBits, int numValuesInSequence)
		{
			int curBitsForTritMode = 6;
			int curBitsForQuintMode = 5;
			int curBitsForPlainBitMode = 8;
			while (true)
			{
				check(curBitsForTritMode > 0 || curBitsForQuintMode > 0 || curBitsForPlainBitMode > 0);
				const int tritRange = curBitsForTritMode > 0 ? (3 << curBitsForTritMode) - 1 : -1;
				const int quintRange = curBitsForQuintMode > 0 ? (5 << curBitsForQuintMode) - 1 : -1;
				const int plainBitRange = curBitsForPlainBitMode > 0 ? (1 << curBitsForPlainBitMode) - 1 : -1;
				const int maxRange = FMath::Max(FMath::Max(tritRange, quintRange), plainBitRange);
				if (maxRange == tritRange)
				{
					const ISEParams params(ISEMODE_TRIT, curBitsForTritMode);
					if (computeNumRequiredBits(params, numValuesInSequence) <= numAvailableBits)
						return ISEParams(ISEMODE_TRIT, curBitsForTritMode);
					curBitsForTritMode--;
				}
				else if (maxRange == quintRange)
				{
					const ISEParams params(ISEMODE_QUINT, curBitsForQuintMode);
					if (computeNumRequiredBits(params, numValuesInSequence) <= numAvailableBits)
						return ISEParams(ISEMODE_QUINT, curBitsForQuintMode);
					curBitsForQuintMode--;
				}
				else
				{
					const ISEParams params(ISEMODE_PLAIN_BIT, curBitsForPlainBitMode);
					check(maxRange == plainBitRange);
					if (computeNumRequiredBits(params, numValuesInSequence) <= numAvailableBits)
						return ISEParams(ISEMODE_PLAIN_BIT, curBitsForPlainBitMode);
					curBitsForPlainBitMode--;
				}
			}
		}

		inline int computeNumColorEndpointValues(uint32 endpointMode)
		{
			check(endpointMode < 16);
			return (endpointMode / 4 + 1) * 2;
		}

		// Decompression utilities
		enum DecompressResult
		{
			DECOMPRESS_RESULT_VALID_BLOCK = 0,	//!< Decompressed valid block
			DECOMPRESS_RESULT_ERROR,				//!< Encountered error while decompressing, error color written
			DECOMPRESS_RESULT_LAST
		};

		// A helper for getting bits from a 128-bit block.
		class Block128
		{
		private:
			typedef uint64 Word;
			enum
			{
				WORD_BYTES = sizeof(Word),
				WORD_BITS = 8 * WORD_BYTES,
				NUM_WORDS = 128 / WORD_BITS
			};
			//DE_STATIC_ASSERT(128 % WORD_BITS == 0);
		public:
			Block128(const uint8* src)
			{
				for (int wordNdx = 0; wordNdx < NUM_WORDS; wordNdx++)
				{
					m_words[wordNdx] = 0;
					for (int byteNdx = 0; byteNdx < WORD_BYTES; byteNdx++)
						m_words[wordNdx] |= (Word)src[wordNdx * WORD_BYTES + byteNdx] << (8 * byteNdx);
				}
			}
			FORCEINLINE uint32 getBit(int ndx) const
			{
				miro_check(inBounds(ndx, 0, 128));
				return (m_words[ndx / WORD_BITS] >> (ndx % WORD_BITS)) & 1;
			}

			FORCEINLINE uint32 getBits(int low, int high) const
			{
				miro_check(inBounds(low, 0, 128));
				miro_check(inBounds(high, 0, 128));
				miro_check(inRange(high - low + 1, 0, 32));
				if (high - low + 1 == 0)
					return 0;
				const int word0Ndx = low / WORD_BITS;
				const int word1Ndx = high / WORD_BITS;
				// \note "foo << bar << 1" done instead of "foo << (bar+1)" to avoid overflow, i.e. shift amount being too big.
				if (word0Ndx == word1Ndx)
					return (uint32)((m_words[word0Ndx] & ((((Word)1 << high % WORD_BITS << 1) - 1))) >> ((Word)low % WORD_BITS));
				else
				{
					miro_check(word1Ndx == word0Ndx + 1);
					return (uint32)(m_words[word0Ndx] >> (low % WORD_BITS)) |
						(uint32)((m_words[word1Ndx] & (((Word)1 << high % WORD_BITS << 1) - 1)) << (high - low - high % WORD_BITS));
				}
			}
			FORCEINLINE bool isBitSet(int ndx) const
			{
				miro_check(inBounds(ndx, 0, 128));
				return getBit(ndx) != 0;
			}

			FORCEINLINE bool isZero() const
			{
				for (int wordNdx = 0; wordNdx < NUM_WORDS; wordNdx++)
				{
					if (m_words[wordNdx]) return false;
				}
				return true;
			}

			FORCEINLINE uint64 GetWord(int i) const { return m_words[i]; }

		private:
			Word m_words[NUM_WORDS];
		};

		// A helper for sequential access into a Block128.
		template<bool Forward>
		class BitAccessStream
		{
		public:
			BitAccessStream(const Block128& src, int startNdxInSrc, int length)
				: m_src(src)
				, m_startNdxInSrc(startNdxInSrc)
				, m_length(length)
				, m_ndx(0)
			{
			}

			// Get the next num bits. Bits at positions greater than or equal to m_length are zeros.
			uint32 getNext(int num)
			{
				if (num == 0 || m_ndx >= m_length)
					return 0;
				const int end = m_ndx + num;
				const int numBitsFromSrc = FMath::Max(0, FMath::Min(m_length, end) - m_ndx);
				const int low = m_ndx;
				const int high = m_ndx + numBitsFromSrc - 1;
				m_ndx += num;
				if (Forward)
				{
					return m_src.getBits(m_startNdxInSrc + low, m_startNdxInSrc + high);
				}
				else
				{
					uint32 PreReverse = m_src.getBits(m_startNdxInSrc - high, m_startNdxInSrc - low);
					uint32 NewReverse = NewReverseBits(PreReverse, numBitsFromSrc);
					return NewReverse;
				}
			}

		private:
			const Block128& m_src;
			const int			m_startNdxInSrc;
			const int			m_length;
			int					m_ndx;
		};

		struct ISEDecodedResult
		{
			uint32 m;
			uint32 tq; //!< Trit or quint value, depending on ISE mode.
			uint32 v;
		};

		// Data from an ASTC block's "block mode" part (i.e. bits [0,10]).
		struct ASTCBlockMode
		{
			bool		isError;
			// \note Following fields only relevant if !isError.
			bool		isVoidExtent;
			// \note Following fields only relevant if !isVoidExtent.
			bool		isDualPlane;
			int			weightGridWidth;
			int			weightGridHeight;
			ISEParams	weightISEParams;
			ASTCBlockMode(void)
				: isError(true)
				, isVoidExtent(true)
				, isDualPlane(true)
				, weightGridWidth(-1)
				, weightGridHeight(-1)
				, weightISEParams(ISEMODE_LAST, -1)
			{
			}
		};

		inline int computeNumWeights(const ASTCBlockMode& mode)
		{
			return mode.weightGridWidth * mode.weightGridHeight * (mode.isDualPlane ? 2 : 1);
		}

		struct ColorEndpointPair
		{
			UVec4 e0;
			UVec4 e1;
		};

		struct TexelWeightPair
		{
			uint32 w[2];
		};

		ASTCBlockMode getASTCBlockMode(uint32 blockModeData)
		{
			ASTCBlockMode blockMode;
			blockMode.isError = true; // \note Set to false later, if not error.
			blockMode.isVoidExtent = getBits(blockModeData, 0, 8) == 0x1fc;
			if (!blockMode.isVoidExtent)
			{
				if ((getBits(blockModeData, 0, 1) == 0 && getBits(blockModeData, 6, 8) == 7) || getBits(blockModeData, 0, 3) == 0)
					return blockMode; // Invalid ("reserved").
				uint32 r = (uint32)-1; // \note Set in the following branches.
				if (getBits(blockModeData, 0, 1) == 0)
				{
					const uint32 r0 = getBit(blockModeData, 4);
					const uint32 r1 = getBit(blockModeData, 2);
					const uint32 r2 = getBit(blockModeData, 3);
					const uint32 i78 = getBits(blockModeData, 7, 8);
					r = (r2 << 2) | (r1 << 1) | (r0 << 0);
					if (i78 == 3)
					{
						const bool i5 = isBitSet(blockModeData, 5);
						blockMode.weightGridWidth = i5 ? 10 : 6;
						blockMode.weightGridHeight = i5 ? 6 : 10;
					}
					else
					{
						const uint32 a = getBits(blockModeData, 5, 6);
						switch (i78)
						{
						case 0:		blockMode.weightGridWidth = 12;		blockMode.weightGridHeight = a + 2;									break;
						case 1:		blockMode.weightGridWidth = a + 2;	blockMode.weightGridHeight = 12;									break;
						case 2:		blockMode.weightGridWidth = a + 6;	blockMode.weightGridHeight = getBits(blockModeData, 9, 10) + 6;		break;
						default: check(false);
						}
					}
				}
				else
				{
					const uint32 r0 = getBit(blockModeData, 4);
					const uint32 r1 = getBit(blockModeData, 0);
					const uint32 r2 = getBit(blockModeData, 1);
					const uint32 i23 = getBits(blockModeData, 2, 3);
					const uint32 a = getBits(blockModeData, 5, 6);
					r = (r2 << 2) | (r1 << 1) | (r0 << 0);
					if (i23 == 3)
					{
						const uint32	b = getBit(blockModeData, 7);
						const bool		i8 = isBitSet(blockModeData, 8);
						blockMode.weightGridWidth = i8 ? b + 2 : a + 2;
						blockMode.weightGridHeight = i8 ? a + 2 : b + 6;
					}
					else
					{
						const uint32 b = getBits(blockModeData, 7, 8);
						switch (i23)
						{
						case 0:		blockMode.weightGridWidth = b + 4;	blockMode.weightGridHeight = a + 2;	break;
						case 1:		blockMode.weightGridWidth = b + 8;	blockMode.weightGridHeight = a + 2;	break;
						case 2:		blockMode.weightGridWidth = a + 2;	blockMode.weightGridHeight = b + 8;	break;
						default: check(false);
						}
					}
				}
				const bool	zeroDH = getBits(blockModeData, 0, 1) == 0 && getBits(blockModeData, 7, 8) == 2;
				const bool	h = zeroDH ? 0 : isBitSet(blockModeData, 9);
				blockMode.isDualPlane = zeroDH ? 0 : isBitSet(blockModeData, 10);
				{
					ISEMode& m = blockMode.weightISEParams.mode;
					int& b = blockMode.weightISEParams.numBits;
					m = ISEMODE_PLAIN_BIT;
					b = 0;
					if (h)
					{
						switch (r)
						{
						case 2:							m = ISEMODE_QUINT;	b = 1;	break;
						case 3:		m = ISEMODE_TRIT;						b = 2;	break;
						case 4:												b = 4;	break;
						case 5:							m = ISEMODE_QUINT;	b = 2;	break;
						case 6:		m = ISEMODE_TRIT;						b = 3;	break;
						case 7:												b = 5;	break;
						default:	check(false);
						}
					}
					else
					{
						switch (r)
						{
						case 2:												b = 1;	break;
						case 3:		m = ISEMODE_TRIT;								break;
						case 4:												b = 2;	break;
						case 5:							m = ISEMODE_QUINT;			break;
						case 6:		m = ISEMODE_TRIT;						b = 1;	break;
						case 7:												b = 3;	break;
						default:	check(false);
						}
					}
				}
			}
			blockMode.isError = false;
			return blockMode;
		}

		inline void setASTCErrorColorBlock(void* dst, int blockWidth, int blockHeight, bool isSRGB)
		{
			if (isSRGB)
			{
				uint8* const dstU = (uint8*)dst;
				for (int i = 0; i < blockWidth * blockHeight; i++)
				{
					dstU[4 * i + 0] = 0xff;
					dstU[4 * i + 1] = 0;
					dstU[4 * i + 2] = 0xff;
					dstU[4 * i + 3] = 0xff;
				}
			}
			else
			{
				float* const dstF = (float*)dst;
				for (int i = 0; i < blockWidth * blockHeight; i++)
				{
					dstF[4 * i + 0] = 1.0f;
					dstF[4 * i + 1] = 0.0f;
					dstF[4 * i + 2] = 1.0f;
					dstF[4 * i + 3] = 1.0f;
				}
			}
		}

		template<int32 BlockSize>
		void decodeVoidExtentBlockSRGB(void* dst, const Block128& blockData)
		{
			uint64 HighWord = blockData.GetWord(1);

			uint8 rgba[4];
			rgba[0] = uint8(HighWord >> 8);
			rgba[1] = uint8(HighWord >> 24);
			rgba[2] = uint8(HighWord >> 40);
			rgba[3] = uint8(HighWord >> 56);

			uint32* const dstU = (uint32*)dst;
			for (int i = 0; i < BlockSize * BlockSize; i++)
			{
				FMemory::Memcpy(&dstU[i], rgba, 4);
			}
		}

		void decodeColorEndpointModes(uint32* endpointModesDst, const Block128& blockData, int numPartitions, int extraCemBitsStart)
		{
			if (numPartitions == 1)
				endpointModesDst[0] = blockData.getBits(13, 16);
			else
			{
				const uint32 highLevelSelector = blockData.getBits(23, 24);
				if (highLevelSelector == 0)
				{
					const uint32 mode = blockData.getBits(25, 28);
					for (int i = 0; i < numPartitions; i++)
						endpointModesDst[i] = mode;
				}
				else
				{
					for (int partNdx = 0; partNdx < numPartitions; partNdx++)
					{
						const uint32 cemClass = highLevelSelector - (blockData.isBitSet(25 + partNdx) ? 0 : 1);
						const uint32 lowBit0Ndx = numPartitions + 2 * partNdx;
						const uint32 lowBit1Ndx = numPartitions + 2 * partNdx + 1;
						const uint32 lowBit0 = blockData.getBit(lowBit0Ndx < 4 ? 25 + lowBit0Ndx : extraCemBitsStart + lowBit0Ndx - 4);
						const uint32 lowBit1 = blockData.getBit(lowBit1Ndx < 4 ? 25 + lowBit1Ndx : extraCemBitsStart + lowBit1Ndx - 4);
						endpointModesDst[partNdx] = (cemClass << 2) | (lowBit1 << 1) | lowBit0;
					}
				}
			}
		}

		int computeNumColorEndpointValues(const uint32* endpointModes, int numPartitions)
		{
			int result = 0;
			for (int i = 0; i < numPartitions; i++)
				result += computeNumColorEndpointValues(endpointModes[i]);
			return result;
		}

		template<bool Forward>
		void decodeISETritBlock(ISEDecodedResult* dst, int numValues, BitAccessStream<Forward>& data, int numBits)
		{
			check(inRange(numValues, 1, 5));
			uint32 m[5];
			m[0] = data.getNext(numBits);
			uint32 T01 = data.getNext(2);
			m[1] = data.getNext(numBits);
			uint32 T23 = data.getNext(2);
			m[2] = data.getNext(numBits);
			uint32 T4 = data.getNext(1);
			m[3] = data.getNext(numBits);
			uint32 T56 = data.getNext(2);
			m[4] = data.getNext(numBits);
			uint32 T7 = data.getNext(1);
			switch (numValues)
			{
				// \note Fall-throughs.
			case 1: T23 = 0;
			case 2: T4 = 0;
			case 3: T56 = 0;
			case 4: T7 = 0;
			case 5: break;
			default:
				check(false);
			}
			const uint32 T = (T7 << 7) | (T56 << 5) | (T4 << 4) | (T23 << 2) | (T01 << 0);
			static const uint32 tritsFromT[256][5] =
			{
				{ 0,0,0,0,0 }, { 1,0,0,0,0 }, { 2,0,0,0,0 }, { 0,0,2,0,0 }, { 0,1,0,0,0 }, { 1,1,0,0,0 }, { 2,1,0,0,0 }, { 1,0,2,0,0 }, { 0,2,0,0,0 }, { 1,2,0,0,0 }, { 2,2,0,0,0 }, { 2,0,2,0,0 }, { 0,2,2,0,0 }, { 1,2,2,0,0 }, { 2,2,2,0,0 }, { 2,0,2,0,0 },
				{ 0,0,1,0,0 }, { 1,0,1,0,0 }, { 2,0,1,0,0 }, { 0,1,2,0,0 }, { 0,1,1,0,0 }, { 1,1,1,0,0 }, { 2,1,1,0,0 }, { 1,1,2,0,0 }, { 0,2,1,0,0 }, { 1,2,1,0,0 }, { 2,2,1,0,0 }, { 2,1,2,0,0 }, { 0,0,0,2,2 }, { 1,0,0,2,2 }, { 2,0,0,2,2 }, { 0,0,2,2,2 },
				{ 0,0,0,1,0 }, { 1,0,0,1,0 }, { 2,0,0,1,0 }, { 0,0,2,1,0 }, { 0,1,0,1,0 }, { 1,1,0,1,0 }, { 2,1,0,1,0 }, { 1,0,2,1,0 }, { 0,2,0,1,0 }, { 1,2,0,1,0 }, { 2,2,0,1,0 }, { 2,0,2,1,0 }, { 0,2,2,1,0 }, { 1,2,2,1,0 }, { 2,2,2,1,0 }, { 2,0,2,1,0 },
				{ 0,0,1,1,0 }, { 1,0,1,1,0 }, { 2,0,1,1,0 }, { 0,1,2,1,0 }, { 0,1,1,1,0 }, { 1,1,1,1,0 }, { 2,1,1,1,0 }, { 1,1,2,1,0 }, { 0,2,1,1,0 }, { 1,2,1,1,0 }, { 2,2,1,1,0 }, { 2,1,2,1,0 }, { 0,1,0,2,2 }, { 1,1,0,2,2 }, { 2,1,0,2,2 }, { 1,0,2,2,2 },
				{ 0,0,0,2,0 }, { 1,0,0,2,0 }, { 2,0,0,2,0 }, { 0,0,2,2,0 }, { 0,1,0,2,0 }, { 1,1,0,2,0 }, { 2,1,0,2,0 }, { 1,0,2,2,0 }, { 0,2,0,2,0 }, { 1,2,0,2,0 }, { 2,2,0,2,0 }, { 2,0,2,2,0 }, { 0,2,2,2,0 }, { 1,2,2,2,0 }, { 2,2,2,2,0 }, { 2,0,2,2,0 },
				{ 0,0,1,2,0 }, { 1,0,1,2,0 }, { 2,0,1,2,0 }, { 0,1,2,2,0 }, { 0,1,1,2,0 }, { 1,1,1,2,0 }, { 2,1,1,2,0 }, { 1,1,2,2,0 }, { 0,2,1,2,0 }, { 1,2,1,2,0 }, { 2,2,1,2,0 }, { 2,1,2,2,0 }, { 0,2,0,2,2 }, { 1,2,0,2,2 }, { 2,2,0,2,2 }, { 2,0,2,2,2 },
				{ 0,0,0,0,2 }, { 1,0,0,0,2 }, { 2,0,0,0,2 }, { 0,0,2,0,2 }, { 0,1,0,0,2 }, { 1,1,0,0,2 }, { 2,1,0,0,2 }, { 1,0,2,0,2 }, { 0,2,0,0,2 }, { 1,2,0,0,2 }, { 2,2,0,0,2 }, { 2,0,2,0,2 }, { 0,2,2,0,2 }, { 1,2,2,0,2 }, { 2,2,2,0,2 }, { 2,0,2,0,2 },
				{ 0,0,1,0,2 }, { 1,0,1,0,2 }, { 2,0,1,0,2 }, { 0,1,2,0,2 }, { 0,1,1,0,2 }, { 1,1,1,0,2 }, { 2,1,1,0,2 }, { 1,1,2,0,2 }, { 0,2,1,0,2 }, { 1,2,1,0,2 }, { 2,2,1,0,2 }, { 2,1,2,0,2 }, { 0,2,2,2,2 }, { 1,2,2,2,2 }, { 2,2,2,2,2 }, { 2,0,2,2,2 },
				{ 0,0,0,0,1 }, { 1,0,0,0,1 }, { 2,0,0,0,1 }, { 0,0,2,0,1 }, { 0,1,0,0,1 }, { 1,1,0,0,1 }, { 2,1,0,0,1 }, { 1,0,2,0,1 }, { 0,2,0,0,1 }, { 1,2,0,0,1 }, { 2,2,0,0,1 }, { 2,0,2,0,1 }, { 0,2,2,0,1 }, { 1,2,2,0,1 }, { 2,2,2,0,1 }, { 2,0,2,0,1 },
				{ 0,0,1,0,1 }, { 1,0,1,0,1 }, { 2,0,1,0,1 }, { 0,1,2,0,1 }, { 0,1,1,0,1 }, { 1,1,1,0,1 }, { 2,1,1,0,1 }, { 1,1,2,0,1 }, { 0,2,1,0,1 }, { 1,2,1,0,1 }, { 2,2,1,0,1 }, { 2,1,2,0,1 }, { 0,0,1,2,2 }, { 1,0,1,2,2 }, { 2,0,1,2,2 }, { 0,1,2,2,2 },
				{ 0,0,0,1,1 }, { 1,0,0,1,1 }, { 2,0,0,1,1 }, { 0,0,2,1,1 }, { 0,1,0,1,1 }, { 1,1,0,1,1 }, { 2,1,0,1,1 }, { 1,0,2,1,1 }, { 0,2,0,1,1 }, { 1,2,0,1,1 }, { 2,2,0,1,1 }, { 2,0,2,1,1 }, { 0,2,2,1,1 }, { 1,2,2,1,1 }, { 2,2,2,1,1 }, { 2,0,2,1,1 },
				{ 0,0,1,1,1 }, { 1,0,1,1,1 }, { 2,0,1,1,1 }, { 0,1,2,1,1 }, { 0,1,1,1,1 }, { 1,1,1,1,1 }, { 2,1,1,1,1 }, { 1,1,2,1,1 }, { 0,2,1,1,1 }, { 1,2,1,1,1 }, { 2,2,1,1,1 }, { 2,1,2,1,1 }, { 0,1,1,2,2 }, { 1,1,1,2,2 }, { 2,1,1,2,2 }, { 1,1,2,2,2 },
				{ 0,0,0,2,1 }, { 1,0,0,2,1 }, { 2,0,0,2,1 }, { 0,0,2,2,1 }, { 0,1,0,2,1 }, { 1,1,0,2,1 }, { 2,1,0,2,1 }, { 1,0,2,2,1 }, { 0,2,0,2,1 }, { 1,2,0,2,1 }, { 2,2,0,2,1 }, { 2,0,2,2,1 }, { 0,2,2,2,1 }, { 1,2,2,2,1 }, { 2,2,2,2,1 }, { 2,0,2,2,1 },
				{ 0,0,1,2,1 }, { 1,0,1,2,1 }, { 2,0,1,2,1 }, { 0,1,2,2,1 }, { 0,1,1,2,1 }, { 1,1,1,2,1 }, { 2,1,1,2,1 }, { 1,1,2,2,1 }, { 0,2,1,2,1 }, { 1,2,1,2,1 }, { 2,2,1,2,1 }, { 2,1,2,2,1 }, { 0,2,1,2,2 }, { 1,2,1,2,2 }, { 2,2,1,2,2 }, { 2,1,2,2,2 },
				{ 0,0,0,1,2 }, { 1,0,0,1,2 }, { 2,0,0,1,2 }, { 0,0,2,1,2 }, { 0,1,0,1,2 }, { 1,1,0,1,2 }, { 2,1,0,1,2 }, { 1,0,2,1,2 }, { 0,2,0,1,2 }, { 1,2,0,1,2 }, { 2,2,0,1,2 }, { 2,0,2,1,2 }, { 0,2,2,1,2 }, { 1,2,2,1,2 }, { 2,2,2,1,2 }, { 2,0,2,1,2 },
				{ 0,0,1,1,2 }, { 1,0,1,1,2 }, { 2,0,1,1,2 }, { 0,1,2,1,2 }, { 0,1,1,1,2 }, { 1,1,1,1,2 }, { 2,1,1,1,2 }, { 1,1,2,1,2 }, { 0,2,1,1,2 }, { 1,2,1,1,2 }, { 2,2,1,1,2 }, { 2,1,2,1,2 }, { 0,2,2,2,2 }, { 1,2,2,2,2 }, { 2,2,2,2,2 }, { 2,1,2,2,2 }
			};
			const uint32(&trits)[5] = tritsFromT[T];
			for (int i = 0; i < numValues; i++)
			{
				dst[i].m = m[i];
				dst[i].tq = trits[i];
				dst[i].v = (trits[i] << numBits) + m[i];
			}
		}

		template<bool Forward>
		void decodeISEQuintBlock(ISEDecodedResult* dst, int numValues, BitAccessStream<Forward>& data, int numBits)
		{
			check(inRange(numValues, 1, 3));
			uint32 m[3];
			m[0] = data.getNext(numBits);
			uint32 Q012 = data.getNext(3);
			m[1] = data.getNext(numBits);
			uint32 Q34 = data.getNext(2);
			m[2] = data.getNext(numBits);
			uint32 Q56 = data.getNext(2);
			switch (numValues)
			{
				// \note Fall-throughs.
			case 1: Q34 = 0;
			case 2: Q56 = 0;
			case 3: break;
			default:
				check(false);
			}
			const uint32 Q = (Q56 << 5) | (Q34 << 3) | (Q012 << 0);
			static const uint32 quintsFromQ[128][3] =
			{
				{ 0,0,0 }, { 1,0,0 }, { 2,0,0 }, { 3,0,0 }, { 4,0,0 }, { 0,4,0 }, { 4,4,0 }, { 4,4,4 }, { 0,1,0 }, { 1,1,0 }, { 2,1,0 }, { 3,1,0 }, { 4,1,0 }, { 1,4,0 }, { 4,4,1 }, { 4,4,4 },
				{ 0,2,0 }, { 1,2,0 }, { 2,2,0 }, { 3,2,0 }, { 4,2,0 }, { 2,4,0 }, { 4,4,2 }, { 4,4,4 }, { 0,3,0 }, { 1,3,0 }, { 2,3,0 }, { 3,3,0 }, { 4,3,0 }, { 3,4,0 }, { 4,4,3 }, { 4,4,4 },
				{ 0,0,1 }, { 1,0,1 }, { 2,0,1 }, { 3,0,1 }, { 4,0,1 }, { 0,4,1 }, { 4,0,4 }, { 0,4,4 }, { 0,1,1 }, { 1,1,1 }, { 2,1,1 }, { 3,1,1 }, { 4,1,1 }, { 1,4,1 }, { 4,1,4 }, { 1,4,4 },
				{ 0,2,1 }, { 1,2,1 }, { 2,2,1 }, { 3,2,1 }, { 4,2,1 }, { 2,4,1 }, { 4,2,4 }, { 2,4,4 }, { 0,3,1 }, { 1,3,1 }, { 2,3,1 }, { 3,3,1 }, { 4,3,1 }, { 3,4,1 }, { 4,3,4 }, { 3,4,4 },
				{ 0,0,2 }, { 1,0,2 }, { 2,0,2 }, { 3,0,2 }, { 4,0,2 }, { 0,4,2 }, { 2,0,4 }, { 3,0,4 }, { 0,1,2 }, { 1,1,2 }, { 2,1,2 }, { 3,1,2 }, { 4,1,2 }, { 1,4,2 }, { 2,1,4 }, { 3,1,4 },
				{ 0,2,2 }, { 1,2,2 }, { 2,2,2 }, { 3,2,2 }, { 4,2,2 }, { 2,4,2 }, { 2,2,4 }, { 3,2,4 }, { 0,3,2 }, { 1,3,2 }, { 2,3,2 }, { 3,3,2 }, { 4,3,2 }, { 3,4,2 }, { 2,3,4 }, { 3,3,4 },
				{ 0,0,3 }, { 1,0,3 }, { 2,0,3 }, { 3,0,3 }, { 4,0,3 }, { 0,4,3 }, { 0,0,4 }, { 1,0,4 }, { 0,1,3 }, { 1,1,3 }, { 2,1,3 }, { 3,1,3 }, { 4,1,3 }, { 1,4,3 }, { 0,1,4 }, { 1,1,4 },
				{ 0,2,3 }, { 1,2,3 }, { 2,2,3 }, { 3,2,3 }, { 4,2,3 }, { 2,4,3 }, { 0,2,4 }, { 1,2,4 }, { 0,3,3 }, { 1,3,3 }, { 2,3,3 }, { 3,3,3 }, { 4,3,3 }, { 3,4,3 }, { 0,3,4 }, { 1,3,4 }
			};
			if (Q < 128)
			{
				const uint32(&quints)[3] = quintsFromQ[Q];
				for (int i = 0; i < numValues; i++)
				{
					dst[i].m = m[i];
					dst[i].tq = quints[i];
					dst[i].v = (quints[i] << numBits) + m[i];
				}
			}
			else
			{
				for (int i = 0; i < numValues; i++)
				{
					dst[i].m = 0;
					dst[i].tq = 0;
					dst[i].v = 0;
				}
			}
		}

		template<bool Forward>
		inline void decodeISEBitBlock(ISEDecodedResult* dst, BitAccessStream<Forward>& data, int numBits)
		{
			dst[0].m = data.getNext(numBits);
			dst[0].v = dst[0].m;
		}

		template<bool Forward>
		void decodeISE(ISEDecodedResult* dst, int numValues, BitAccessStream<Forward>& data, const ISEParams& params)
		{
			if (params.mode == ISEMODE_TRIT)
			{
				const int numBlocks = deDivRoundUp32(numValues, 5);
				for (int blockNdx = 0; blockNdx < numBlocks; blockNdx++)
				{
					const int numValuesInBlock = blockNdx == numBlocks - 1 ? numValues - 5 * (numBlocks - 1) : 5;
					decodeISETritBlock(&dst[5 * blockNdx], numValuesInBlock, data, params.numBits);
				}
			}
			else if (params.mode == ISEMODE_QUINT)
			{
				const int numBlocks = deDivRoundUp32(numValues, 3);
				for (int blockNdx = 0; blockNdx < numBlocks; blockNdx++)
				{
					const int numValuesInBlock = blockNdx == numBlocks - 1 ? numValues - 3 * (numBlocks - 1) : 3;
					decodeISEQuintBlock(&dst[3 * blockNdx], numValuesInBlock, data, params.numBits);
				}
			}
			else
			{
				check(params.mode == ISEMODE_PLAIN_BIT);
				for (int i = 0; i < numValues; i++)
					decodeISEBitBlock(&dst[i], data, params.numBits);
			}
		}

		void unquantizeColorEndpoints(uint32* dst, const ISEDecodedResult* iseResults, int numEndpoints, const ISEParams& iseParams)
		{
			if (iseParams.mode == ISEMODE_TRIT || iseParams.mode == ISEMODE_QUINT)
			{
				const int rangeCase = iseParams.numBits * 2 - (iseParams.mode == ISEMODE_TRIT ? 2 : 1);
				check(inRange(rangeCase, 0, 10));
				static const uint32	Ca[11] = { 204, 113, 93, 54, 44, 26, 22, 13, 11, 6, 5 };
				const uint32			C = Ca[rangeCase];
				for (int endpointNdx = 0; endpointNdx < numEndpoints; endpointNdx++)
				{
					const uint32 a = getBit(iseResults[endpointNdx].m, 0);
					const uint32 b = getBit(iseResults[endpointNdx].m, 1);
					const uint32 c = getBit(iseResults[endpointNdx].m, 2);
					const uint32 d = getBit(iseResults[endpointNdx].m, 3);
					const uint32 e = getBit(iseResults[endpointNdx].m, 4);
					const uint32 f = getBit(iseResults[endpointNdx].m, 5);
					const uint32 A = a == 0 ? 0 : (1 << 9) - 1;
					const uint32 B = rangeCase == 0 ? 0
						: rangeCase == 1 ? 0
						: rangeCase == 2 ? (b << 8) | (b << 4) | (b << 2) | (b << 1)
						: rangeCase == 3 ? (b << 8) | (b << 3) | (b << 2)
						: rangeCase == 4 ? (c << 8) | (b << 7) | (c << 3) | (b << 2) | (c << 1) | (b << 0)
						: rangeCase == 5 ? (c << 8) | (b << 7) | (c << 2) | (b << 1) | (c << 0)
						: rangeCase == 6 ? (d << 8) | (c << 7) | (b << 6) | (d << 2) | (c << 1) | (b << 0)
						: rangeCase == 7 ? (d << 8) | (c << 7) | (b << 6) | (d << 1) | (c << 0)
						: rangeCase == 8 ? (e << 8) | (d << 7) | (c << 6) | (b << 5) | (e << 1) | (d << 0)
						: rangeCase == 9 ? (e << 8) | (d << 7) | (c << 6) | (b << 5) | (e << 0)
						: rangeCase == 10 ? (f << 8) | (e << 7) | (d << 6) | (c << 5) | (b << 4) | (f << 0)
						: (uint32)-1;
					check(B != (uint32)-1);
					dst[endpointNdx] = (((iseResults[endpointNdx].tq * C + B) ^ A) >> 2) | (A & 0x80);
				}
			}
			else
			{
				check(iseParams.mode == ISEMODE_PLAIN_BIT);
				for (int endpointNdx = 0; endpointNdx < numEndpoints; endpointNdx++)
					dst[endpointNdx] = bitReplicationScale(iseResults[endpointNdx].v, iseParams.numBits, 8);
			}
		}

		inline void bitTransferSigned(int32& a, int32& b)
		{
			b >>= 1;
			b |= a & 0x80;
			a >>= 1;
			a &= 0x3f;
			if (isBitSet(a, 5))
				a -= 0x40;
		}

		inline UVec4 clampedRGBA(const IVec4& rgba)
		{
			return UVec4(FMath::Clamp(rgba[0], 0, 0xff),
				FMath::Clamp(rgba[1], 0, 0xff),
				FMath::Clamp(rgba[2], 0, 0xff),
				FMath::Clamp(rgba[3], 0, 0xff));
		}

		inline IVec4 blueContract(int r, int g, int b, int a)
		{
			return IVec4((r + b) >> 1, (g + b) >> 1, b, a);
		}

		inline bool isColorEndpointModeHDR(uint32 mode)
		{
			return mode == 2 ||
				mode == 3 ||
				mode == 7 ||
				mode == 11 ||
				mode == 14 ||
				mode == 15;
		}

		void decodeHDREndpointMode7(UVec4& e0, UVec4& e1, uint32 v0, uint32 v1, uint32 v2, uint32 v3)
		{
			const uint32 m10 = getBit(v1, 7) | (getBit(v2, 7) << 1);
			const uint32 m23 = getBits(v0, 6, 7);
			const uint32 majComp = m10 != 3 ? m10
				: m23 != 3 ? m23
				: 0;
			const uint32 mode = m10 != 3 ? m23
				: m23 != 3 ? 4
				: 5;
			int32			red = (int32)getBits(v0, 0, 5);
			int32			green = (int32)getBits(v1, 0, 4);
			int32			blue = (int32)getBits(v2, 0, 4);
			int32			scale = (int32)getBits(v3, 0, 4);
			{
#define SHOR(DST_VAR, SHIFT, BIT_VAR) (DST_VAR) |= (BIT_VAR) << (SHIFT)
#define ASSIGN_X_BITS(V0,S0, V1,S1, V2,S2, V3,S3, V4,S4, V5,S5, V6,S6) do { SHOR(V0,S0,x0); SHOR(V1,S1,x1); SHOR(V2,S2,x2); SHOR(V3,S3,x3); SHOR(V4,S4,x4); SHOR(V5,S5,x5); SHOR(V6,S6,x6); } while (false)
				const uint32	x0 = getBit(v1, 6);
				const uint32	x1 = getBit(v1, 5);
				const uint32	x2 = getBit(v2, 6);
				const uint32	x3 = getBit(v2, 5);
				const uint32	x4 = getBit(v3, 7);
				const uint32	x5 = getBit(v3, 6);
				const uint32	x6 = getBit(v3, 5);
				int32& R = red;
				int32& G = green;
				int32& B = blue;
				int32& S = scale;
				switch (mode)
				{
				case 0: ASSIGN_X_BITS(R, 9, R, 8, R, 7, R, 10, R, 6, S, 6, S, 5); break;
				case 1: ASSIGN_X_BITS(R, 8, G, 5, R, 7, B, 5, R, 6, R, 10, R, 9); break;
				case 2: ASSIGN_X_BITS(R, 9, R, 8, R, 7, R, 6, S, 7, S, 6, S, 5); break;
				case 3: ASSIGN_X_BITS(R, 8, G, 5, R, 7, B, 5, R, 6, S, 6, S, 5); break;
				case 4: ASSIGN_X_BITS(G, 6, G, 5, B, 6, B, 5, R, 6, R, 7, S, 5); break;
				case 5: ASSIGN_X_BITS(G, 6, G, 5, B, 6, B, 5, R, 6, S, 6, S, 5); break;
				default:
					check(false);
				}
#undef ASSIGN_X_BITS
#undef SHOR
			}
			static const int shiftAmounts[] = { 1, 1, 2, 3, 4, 5 };
			check(mode < DE_LENGTH_OF_ARRAY(shiftAmounts));
			red <<= shiftAmounts[mode];
			green <<= shiftAmounts[mode];
			blue <<= shiftAmounts[mode];
			scale <<= shiftAmounts[mode];
			if (mode != 5)
			{
				green = red - green;
				blue = red - blue;
			}
			if (majComp == 1)
				Swap(red, green);
			else if (majComp == 2)
				Swap(red, blue);
			e0 = UVec4(FMath::Clamp(red - scale, 0, 0xfff),
				FMath::Clamp(green - scale, 0, 0xfff),
				FMath::Clamp(blue - scale, 0, 0xfff),
				0x780);
			e1 = UVec4(FMath::Clamp(red, 0, 0xfff),
				FMath::Clamp(green, 0, 0xfff),
				FMath::Clamp(blue, 0, 0xfff),
				0x780);
		}

		void decodeHDREndpointMode11(UVec4& e0, UVec4& e1, uint32 v0, uint32 v1, uint32 v2, uint32 v3, uint32 v4, uint32 v5)
		{
			const uint32 major = (getBit(v5, 7) << 1) | getBit(v4, 7);
			if (major == 3)
			{
				e0 = UVec4(v0 << 4, v2 << 4, getBits(v4, 0, 6) << 5, 0x780);
				e1 = UVec4(v1 << 4, v3 << 4, getBits(v5, 0, 6) << 5, 0x780);
			}
			else
			{
				const uint32 mode = (getBit(v3, 7) << 2) | (getBit(v2, 7) << 1) | getBit(v1, 7);
				int32 a = (int32)((getBit(v1, 6) << 8) | v0);
				int32 c = (int32)(getBits(v1, 0, 5));
				int32 b0 = (int32)(getBits(v2, 0, 5));
				int32 b1 = (int32)(getBits(v3, 0, 5));
				int32 d0 = (int32)(getBits(v4, 0, 4));
				int32 d1 = (int32)(getBits(v5, 0, 4));
				{
#define SHOR(DST_VAR, SHIFT, BIT_VAR) (DST_VAR) |= (BIT_VAR) << (SHIFT)
#define ASSIGN_X_BITS(V0,S0, V1,S1, V2,S2, V3,S3, V4,S4, V5,S5) do { SHOR(V0,S0,x0); SHOR(V1,S1,x1); SHOR(V2,S2,x2); SHOR(V3,S3,x3); SHOR(V4,S4,x4); SHOR(V5,S5,x5); } while (false)
					const uint32 x0 = getBit(v2, 6);
					const uint32 x1 = getBit(v3, 6);
					const uint32 x2 = getBit(v4, 6);
					const uint32 x3 = getBit(v5, 6);
					const uint32 x4 = getBit(v4, 5);
					const uint32 x5 = getBit(v5, 5);
					switch (mode)
					{
					case 0: ASSIGN_X_BITS(b0, 6, b1, 6, d0, 6, d1, 6, d0, 5, d1, 5); break;
					case 1: ASSIGN_X_BITS(b0, 6, b1, 6, b0, 7, b1, 7, d0, 5, d1, 5); break;
					case 2: ASSIGN_X_BITS(a, 9, c, 6, d0, 6, d1, 6, d0, 5, d1, 5); break;
					case 3: ASSIGN_X_BITS(b0, 6, b1, 6, a, 9, c, 6, d0, 5, d1, 5); break;
					case 4: ASSIGN_X_BITS(b0, 6, b1, 6, b0, 7, b1, 7, a, 9, a, 10); break;
					case 5: ASSIGN_X_BITS(a, 9, a, 10, c, 7, c, 6, d0, 5, d1, 5); break;
					case 6: ASSIGN_X_BITS(b0, 6, b1, 6, a, 11, c, 6, a, 9, a, 10); break;
					case 7: ASSIGN_X_BITS(a, 9, a, 10, a, 11, c, 6, d0, 5, d1, 5); break;
					default:
						check(false);
					}
#undef ASSIGN_X_BITS
#undef SHOR
				}
				static const int numDBits[] = { 7, 6, 7, 6, 5, 6, 5, 6 };
				check(mode < DE_LENGTH_OF_ARRAY(numDBits));
				d0 = signExtend(d0, numDBits[mode]);
				d1 = signExtend(d1, numDBits[mode]);
				const int shiftAmount = (mode >> 1) ^ 3;
				a <<= shiftAmount;
				c <<= shiftAmount;
				b0 <<= shiftAmount;
				b1 <<= shiftAmount;
				d0 <<= shiftAmount;
				d1 <<= shiftAmount;
				e0 = UVec4(FMath::Clamp(a - c, 0, 0xfff),
					FMath::Clamp(a - b0 - c - d0, 0, 0xfff),
					FMath::Clamp(a - b1 - c - d1, 0, 0xfff),
					0x780);
				e1 = UVec4(FMath::Clamp(a, 0, 0xfff),
					FMath::Clamp(a - b0, 0, 0xfff),
					FMath::Clamp(a - b1, 0, 0xfff),
					0x780);
				if (major == 1)
				{
					Swap(e0[0], e0[1]);
					Swap(e1[0], e1[1]);
				}
				else if (major == 2)
				{
					Swap(e0[0], e0[2]);
					Swap(e1[0], e1[2]);
				}
			}
		}

		void decodeHDREndpointMode15(UVec4& e0, UVec4& e1, uint32 v0, uint32 v1, uint32 v2, uint32 v3, uint32 v4, uint32 v5, uint32 v6In, uint32 v7In)
		{
			decodeHDREndpointMode11(e0, e1, v0, v1, v2, v3, v4, v5);
			const uint32	mode = (getBit(v7In, 7) << 1) | getBit(v6In, 7);
			int32			v6 = (int32)getBits(v6In, 0, 6);
			int32			v7 = (int32)getBits(v7In, 0, 6);
			if (mode == 3)
			{
				e0[3] = v6 << 5;
				e1[3] = v7 << 5;
			}
			else
			{
				v6 |= (v7 << (mode + 1)) & 0x780;
				v7 &= (0x3f >> mode);
				v7 ^= 0x20 >> mode;
				v7 -= 0x20 >> mode;
				v6 <<= 4 - mode;
				v7 <<= 4 - mode;
				v7 += v6;
				v7 = FMath::Clamp(v7, 0, 0xfff);
				e0[3] = v6;
				e1[3] = v7;
			}
		}

		void decodeColorEndpoints(ColorEndpointPair* dst, const uint32* unquantizedEndpoints, const uint32* endpointModes, int numPartitions)
		{
			int unquantizedNdx = 0;
			for (int partitionNdx = 0; partitionNdx < numPartitions; partitionNdx++)
			{
				const uint32		endpointMode = endpointModes[partitionNdx];
				const uint32* v = &unquantizedEndpoints[unquantizedNdx];
				UVec4& e0 = dst[partitionNdx].e0;
				UVec4& e1 = dst[partitionNdx].e1;
				unquantizedNdx += computeNumColorEndpointValues(endpointMode);
				switch (endpointMode)
				{
				case 0:
					e0 = UVec4(v[0], v[0], v[0], 0xff);
					e1 = UVec4(v[1], v[1], v[1], 0xff);
					break;
				case 1:
				{
					const uint32 L0 = (v[0] >> 2) | (getBits(v[1], 6, 7) << 6);
					const uint32 L1 = FMath::Min(0xffu, L0 + getBits(v[1], 0, 5));
					e0 = UVec4(L0, L0, L0, 0xff);
					e1 = UVec4(L1, L1, L1, 0xff);
					break;
				}
				case 2:
				{
					const uint32 v1Gr = v[1] >= v[0];
					const uint32 y0 = v1Gr ? v[0] << 4 : (v[1] << 4) + 8;
					const uint32 y1 = v1Gr ? v[1] << 4 : (v[0] << 4) - 8;
					e0 = UVec4(y0, y0, y0, 0x780);
					e1 = UVec4(y1, y1, y1, 0x780);
					break;
				}
				case 3:
				{
					const bool		m = isBitSet(v[0], 7);
					const uint32	y0 = m ? (getBits(v[1], 5, 7) << 9) | (getBits(v[0], 0, 6) << 2)
						: (getBits(v[1], 4, 7) << 8) | (getBits(v[0], 0, 6) << 1);
					const uint32	d = m ? getBits(v[1], 0, 4) << 2
						: getBits(v[1], 0, 3) << 1;
					const uint32	y1 = FMath::Min(0xfffu, y0 + d);
					e0 = UVec4(y0, y0, y0, 0x780);
					e1 = UVec4(y1, y1, y1, 0x780);
					break;
				}
				case 4:
					e0 = UVec4(v[0], v[0], v[0], v[2]);
					e1 = UVec4(v[1], v[1], v[1], v[3]);
					break;
				case 5:
				{
					int32 v0 = (int32)v[0];
					int32 v1 = (int32)v[1];
					int32 v2 = (int32)v[2];
					int32 v3 = (int32)v[3];
					bitTransferSigned(v1, v0);
					bitTransferSigned(v3, v2);
					e0 = clampedRGBA(IVec4(v0, v0, v0, v2));
					e1 = clampedRGBA(IVec4(v0 + v1, v0 + v1, v0 + v1, v2 + v3));
					break;
				}
				case 6:
					e0 = UVec4((v[0] * v[3]) >> 8, (v[1] * v[3]) >> 8, (v[2] * v[3]) >> 8, 0xff);
					e1 = UVec4(v[0], v[1], v[2], 0xff);
					break;
				case 7:
					decodeHDREndpointMode7(e0, e1, v[0], v[1], v[2], v[3]);
					break;
				case 8:
					if (v[1] + v[3] + v[5] >= v[0] + v[2] + v[4])
					{
						e0 = UVec4(v[0], v[2], v[4], 0xff);
						e1 = UVec4(v[1], v[3], v[5], 0xff);
					}
					else
					{
						e0 = asUint(blueContract(v[1], v[3], v[5], 0xff));
						e1 = asUint(blueContract(v[0], v[2], v[4], 0xff));
					}
					break;
				case 9:
				{
					int32 v0 = (int32)v[0];
					int32 v1 = (int32)v[1];
					int32 v2 = (int32)v[2];
					int32 v3 = (int32)v[3];
					int32 v4 = (int32)v[4];
					int32 v5 = (int32)v[5];
					bitTransferSigned(v1, v0);
					bitTransferSigned(v3, v2);
					bitTransferSigned(v5, v4);
					if (v1 + v3 + v5 >= 0)
					{
						e0 = clampedRGBA(IVec4(v0, v2, v4, 0xff));
						e1 = clampedRGBA(IVec4(v0 + v1, v2 + v3, v4 + v5, 0xff));
					}
					else
					{
						e0 = clampedRGBA(blueContract(v0 + v1, v2 + v3, v4 + v5, 0xff));
						e1 = clampedRGBA(blueContract(v0, v2, v4, 0xff));
					}
					break;
				}
				case 10:
					e0 = UVec4((v[0] * v[3]) >> 8, (v[1] * v[3]) >> 8, (v[2] * v[3]) >> 8, v[4]);
					e1 = UVec4(v[0], v[1], v[2], v[5]);
					break;
				case 11:
					decodeHDREndpointMode11(e0, e1, v[0], v[1], v[2], v[3], v[4], v[5]);
					break;
				case 12:
					if (v[1] + v[3] + v[5] >= v[0] + v[2] + v[4])
					{
						e0 = UVec4(v[0], v[2], v[4], v[6]);
						e1 = UVec4(v[1], v[3], v[5], v[7]);
					}
					else
					{
						e0 = clampedRGBA(blueContract(v[1], v[3], v[5], v[7]));
						e1 = clampedRGBA(blueContract(v[0], v[2], v[4], v[6]));
					}
					break;
				case 13:
				{
					int32 v0 = (int32)v[0];
					int32 v1 = (int32)v[1];
					int32 v2 = (int32)v[2];
					int32 v3 = (int32)v[3];
					int32 v4 = (int32)v[4];
					int32 v5 = (int32)v[5];
					int32 v6 = (int32)v[6];
					int32 v7 = (int32)v[7];
					bitTransferSigned(v1, v0);
					bitTransferSigned(v3, v2);
					bitTransferSigned(v5, v4);
					bitTransferSigned(v7, v6);
					if (v1 + v3 + v5 >= 0)
					{
						e0 = clampedRGBA(IVec4(v0, v2, v4, v6));
						e1 = clampedRGBA(IVec4(v0 + v1, v2 + v3, v4 + v5, v6 + v7));
					}
					else
					{
						e0 = clampedRGBA(blueContract(v0 + v1, v2 + v3, v4 + v5, v6 + v7));
						e1 = clampedRGBA(blueContract(v0, v2, v4, v6));
					}
					break;
				}
				case 14:
					decodeHDREndpointMode11(e0, e1, v[0], v[1], v[2], v[3], v[4], v[5]);
					e0[3] = v[6];
					e1[3] = v[7];
					break;
				case 15:
					decodeHDREndpointMode15(e0, e1, v[0], v[1], v[2], v[3], v[4], v[5], v[6], v[7]);
					break;
				default:
					check(false);
				}
			}
		}

		void computeColorEndpoints(ColorEndpointPair* dst, const Block128& blockData, const uint32* endpointModes, int numPartitions, int numColorEndpointValues, const ISEParams& iseParams, int numBitsAvailable)
		{
			const int			colorEndpointDataStart = numPartitions == 1 ? 17 : 29;
			ISEDecodedResult	colorEndpointData[18];
			{
				BitAccessStream<true> dataStream(blockData, colorEndpointDataStart, numBitsAvailable);
				decodeISE(&colorEndpointData[0], numColorEndpointValues, dataStream, iseParams);
			}
			{
				uint32 unquantizedEndpoints[18];
				unquantizeColorEndpoints(&unquantizedEndpoints[0], &colorEndpointData[0], numColorEndpointValues, iseParams);
				decodeColorEndpoints(dst, &unquantizedEndpoints[0], &endpointModes[0], numPartitions);
			}
		}

		void unquantizeWeights(uint32 dst[64], const ISEDecodedResult* weightGrid, const ASTCBlockMode& blockMode)
		{
			const int			numWeights = computeNumWeights(blockMode);
			const ISEParams& iseParams = blockMode.weightISEParams;
			if (iseParams.mode == ISEMODE_TRIT || iseParams.mode == ISEMODE_QUINT)
			{
				const int rangeCase = iseParams.numBits * 2 + (iseParams.mode == ISEMODE_QUINT ? 1 : 0);
				if (rangeCase == 0 || rangeCase == 1)
				{
					static const uint32 map0[3] = { 0, 32, 63 };
					static const uint32 map1[5] = { 0, 16, 32, 47, 63 };
					const uint32* const map = rangeCase == 0 ? &map0[0] : &map1[0];
					for (int i = 0; i < numWeights; i++)
					{
						check(weightGrid[i].v < (rangeCase == 0 ? 3u : 5u));
						dst[i] = map[weightGrid[i].v];
					}
				}
				else
				{
					check(rangeCase <= 6);
					static const uint32	Ca[5] = { 50, 28, 23, 13, 11 };
					const uint32			C = Ca[rangeCase - 2];
					for (int weightNdx = 0; weightNdx < numWeights; weightNdx++)
					{
						const uint32 a = getBit(weightGrid[weightNdx].m, 0);
						const uint32 b = getBit(weightGrid[weightNdx].m, 1);
						const uint32 c = getBit(weightGrid[weightNdx].m, 2);
						const uint32 A = a == 0 ? 0 : (1 << 7) - 1;
						const uint32 B = rangeCase == 2 ? 0
							: rangeCase == 3 ? 0
							: rangeCase == 4 ? (b << 6) | (b << 2) | (b << 0)
							: rangeCase == 5 ? (b << 6) | (b << 1)
							: rangeCase == 6 ? (c << 6) | (b << 5) | (c << 1) | (b << 0)
							: (uint32)-1;
						dst[weightNdx] = (((weightGrid[weightNdx].tq * C + B) ^ A) >> 2) | (A & 0x20);
					}
				}
			}
			else
			{
				check(iseParams.mode == ISEMODE_PLAIN_BIT);
				for (int weightNdx = 0; weightNdx < numWeights; weightNdx++)
					dst[weightNdx] = bitReplicationScale(weightGrid[weightNdx].v, iseParams.numBits, 6);
			}
			for (int weightNdx = 0; weightNdx < numWeights; weightNdx++)
				dst[weightNdx] += dst[weightNdx] > 32 ? 1 : 0;
			// Initialize nonexistent weights to poison values
			for (int weightNdx = numWeights; weightNdx < 64; weightNdx++)
				dst[weightNdx] = ~0u;
		}

		void interpolateWeights(TexelWeightPair* dst, const uint32(&unquantizedWeights)[64], int blockWidth, int blockHeight, const ASTCBlockMode& blockMode)
		{
			const int		numWeightsPerTexel = blockMode.isDualPlane ? 2 : 1;
			const uint32	scaleX = (1024 + blockWidth / 2) / (blockWidth - 1);
			const uint32	scaleY = (1024 + blockHeight / 2) / (blockHeight - 1);
			miro_check(blockMode.weightGridWidth * blockMode.weightGridHeight * numWeightsPerTexel <= DE_LENGTH_OF_ARRAY(unquantizedWeights));
			for (int texelY = 0; texelY < blockHeight; texelY++)
			{
				for (int texelX = 0; texelX < blockWidth; texelX++)
				{
					const uint32 gX = (scaleX * texelX * (blockMode.weightGridWidth - 1) + 32) >> 6;
					const uint32 gY = (scaleY * texelY * (blockMode.weightGridHeight - 1) + 32) >> 6;
					const uint32 jX = gX >> 4;
					const uint32 jY = gY >> 4;
					const uint32 fX = gX & 0xf;
					const uint32 fY = gY & 0xf;
					const uint32 w11 = (fX * fY + 8) >> 4;
					const uint32 w10 = fY - w11;
					const uint32 w01 = fX - w11;
					const uint32 w00 = 16 - fX - fY + w11;
					const uint32 i00 = jY * blockMode.weightGridWidth + jX;
					const uint32 i01 = i00 + 1;
					const uint32 i10 = i00 + blockMode.weightGridWidth;
					const uint32 i11 = i00 + blockMode.weightGridWidth + 1;
					// These addresses can be out of bounds, but respective weights will be 0 then.
					miro_check(deInBounds32(i00, 0, blockMode.weightGridWidth * blockMode.weightGridHeight) || w00 == 0);
					miro_check(deInBounds32(i01, 0, blockMode.weightGridWidth * blockMode.weightGridHeight) || w01 == 0);
					miro_check(deInBounds32(i10, 0, blockMode.weightGridWidth * blockMode.weightGridHeight) || w10 == 0);
					miro_check(deInBounds32(i11, 0, blockMode.weightGridWidth * blockMode.weightGridHeight) || w11 == 0);
					for (int texelWeightNdx = 0; texelWeightNdx < numWeightsPerTexel; texelWeightNdx++)
					{
						// & 0x3f clamps address to bounds of unquantizedWeights
						const uint32 p00 = unquantizedWeights[(i00 * numWeightsPerTexel + texelWeightNdx) & 0x3f];
						const uint32 p01 = unquantizedWeights[(i01 * numWeightsPerTexel + texelWeightNdx) & 0x3f];
						const uint32 p10 = unquantizedWeights[(i10 * numWeightsPerTexel + texelWeightNdx) & 0x3f];
						const uint32 p11 = unquantizedWeights[(i11 * numWeightsPerTexel + texelWeightNdx) & 0x3f];
						dst[texelY * blockWidth + texelX].w[texelWeightNdx] = (p00 * w00 + p01 * w01 + p10 * w10 + p11 * w11 + 8) >> 4;
					}
				}
			}
		}

		void computeTexelWeights(TexelWeightPair* dst, const Block128& blockData, int blockWidth, int blockHeight, const ASTCBlockMode& blockMode)
		{
			ISEDecodedResult weightGrid[64];
			{
				BitAccessStream<false> dataStream(blockData, 127, computeNumRequiredBits(blockMode.weightISEParams, computeNumWeights(blockMode)));
				decodeISE(&weightGrid[0], computeNumWeights(blockMode), dataStream, blockMode.weightISEParams);
			}
			{
				uint32 unquantizedWeights[64];
				unquantizeWeights(&unquantizedWeights[0], &weightGrid[0], blockMode);
				interpolateWeights(dst, unquantizedWeights, blockWidth, blockHeight, blockMode);
			}
		}

		inline uint32 hash52(uint32 v)
		{
			uint32 p = v;
			p ^= p >> 15;	p -= p << 17;	p += p << 7;	p += p << 4;
			p ^= p >> 5;	p += p << 16;	p ^= p >> 7;	p ^= p >> 3;
			p ^= p << 6;	p ^= p >> 17;
			return p;
		}

		int computeTexelPartition(uint32 seedIn, uint32 xIn, uint32 yIn, uint32 zIn, int numPartitions, bool smallBlock)
		{
			check(zIn == 0);
			const uint32	x = smallBlock ? xIn << 1 : xIn;
			const uint32	y = smallBlock ? yIn << 1 : yIn;
			const uint32	z = smallBlock ? zIn << 1 : zIn;
			const uint32	seed = seedIn + 1024 * (numPartitions - 1);
			const uint32	rnum = hash52(seed);
			uint8			seed1 = (uint8)(rnum & 0xf);
			uint8			seed2 = (uint8)((rnum >> 4) & 0xf);
			uint8			seed3 = (uint8)((rnum >> 8) & 0xf);
			uint8			seed4 = (uint8)((rnum >> 12) & 0xf);
			uint8			seed5 = (uint8)((rnum >> 16) & 0xf);
			uint8			seed6 = (uint8)((rnum >> 20) & 0xf);
			uint8			seed7 = (uint8)((rnum >> 24) & 0xf);
			uint8			seed8 = (uint8)((rnum >> 28) & 0xf);
			uint8			seed9 = (uint8)((rnum >> 18) & 0xf);
			uint8			seed10 = (uint8)((rnum >> 22) & 0xf);
			uint8			seed11 = (uint8)((rnum >> 26) & 0xf);
			uint8			seed12 = (uint8)(((rnum >> 30) | (rnum << 2)) & 0xf);
			seed1 = (uint8)(seed1 * seed1);
			seed2 = (uint8)(seed2 * seed2);
			seed3 = (uint8)(seed3 * seed3);
			seed4 = (uint8)(seed4 * seed4);
			seed5 = (uint8)(seed5 * seed5);
			seed6 = (uint8)(seed6 * seed6);
			seed7 = (uint8)(seed7 * seed7);
			seed8 = (uint8)(seed8 * seed8);
			seed9 = (uint8)(seed9 * seed9);
			seed10 = (uint8)(seed10 * seed10);
			seed11 = (uint8)(seed11 * seed11);
			seed12 = (uint8)(seed12 * seed12);
			const int shA = (seed & 2) != 0 ? 4 : 5;
			const int shB = numPartitions == 3 ? 6 : 5;
			const int sh1 = (seed & 1) != 0 ? shA : shB;
			const int sh2 = (seed & 1) != 0 ? shB : shA;
			const int sh3 = (seed & 0x10) != 0 ? sh1 : sh2;
			seed1 = (uint8)(seed1 >> sh1);
			seed2 = (uint8)(seed2 >> sh2);
			seed3 = (uint8)(seed3 >> sh1);
			seed4 = (uint8)(seed4 >> sh2);
			seed5 = (uint8)(seed5 >> sh1);
			seed6 = (uint8)(seed6 >> sh2);
			seed7 = (uint8)(seed7 >> sh1);
			seed8 = (uint8)(seed8 >> sh2);
			seed9 = (uint8)(seed9 >> sh3);
			seed10 = (uint8)(seed10 >> sh3);
			seed11 = (uint8)(seed11 >> sh3);
			seed12 = (uint8)(seed12 >> sh3);
			const int a = 0x3f & (seed1 * x + seed2 * y + seed11 * z + (rnum >> 14));
			const int b = 0x3f & (seed3 * x + seed4 * y + seed12 * z + (rnum >> 10));
			const int c = numPartitions >= 3 ? 0x3f & (seed5 * x + seed6 * y + seed9 * z + (rnum >> 6)) : 0;
			const int d = numPartitions >= 4 ? 0x3f & (seed7 * x + seed8 * y + seed10 * z + (rnum >> 2)) : 0;
			return a >= b && a >= c && a >= d ? 0
				: b >= c && b >= d ? 1
				: c >= d ? 2
				: 3;
		}

		template<int32 BlockSize>
		DecompressResult setTexelColorsSRGB(void* dst, ColorEndpointPair* colorEndpoints, TexelWeightPair* texelWeights, int ccs, uint32 partitionIndexSeed, int numPartitions, const uint32* colorEndpointModes)
		{
			const bool			smallBlock = BlockSize * BlockSize < 31;
			DecompressResult	result = DECOMPRESS_RESULT_VALID_BLOCK;
			bool				isHDREndpoint[4] = { false,false,false,false };
			for (int i = 0; i < numPartitions; i++)
			{
				isHDREndpoint[i] = isColorEndpointModeHDR(colorEndpointModes[i]);

				// rg - REMOVING HDR SUPPORT FOR NOW
				if (isHDREndpoint[i])
					return DECOMPRESS_RESULT_ERROR;
			}

			for (int texelY = 0; texelY < BlockSize; texelY++)
				for (int texelX = 0; texelX < BlockSize; texelX++)
				{
					const int texelNdx = texelY * BlockSize + texelX;
					int	 colorEndpointNdx = numPartitions == 1 ? 0 : computeTexelPartition(partitionIndexSeed, texelX, texelY, 0, numPartitions, smallBlock);
					check(colorEndpointNdx < numPartitions);
					colorEndpointNdx = FMath::Clamp<int>(colorEndpointNdx, 0, FMath::Min(3, numPartitions));
					const UVec4& e0 = colorEndpoints[colorEndpointNdx].e0;
					const UVec4& e1 = colorEndpoints[colorEndpointNdx].e1;
					const TexelWeightPair& weight = texelWeights[texelNdx];

					for (int channelNdx = 0; channelNdx < 4; channelNdx++)
					{
						const uint32 c0 = (e0[channelNdx] << 8) | 0x80;
						const uint32 c1 = (e1[channelNdx] << 8) | 0x80;
						const uint32 w = weight.w[ccs == channelNdx ? 1 : 0];
						const uint32 c = (c0 * (64 - w) + c1 * w + 32) / 64;
						((uint8*)dst)[texelNdx * 4 + channelNdx] = (uint8)((c & 0xff00) >> 8);
					}
				}
			return result;
		}

		template<int32 BlockSize>
		void decompressBlockSRGB(void* dst, const Block128& blockData)
		{
			bool isLDR = true;
			bool isSRGB = true;

			// Decode block mode.
			const ASTCBlockMode blockMode = getASTCBlockMode(blockData.getBits(0, 10));

			// Check for block mode errors.
			if (blockMode.isError)
			{
				setASTCErrorColorBlock(dst, BlockSize, BlockSize, isSRGB);
				return;
			}
			// Separate path for void-extent.
			if (blockMode.isVoidExtent)
			{
				return decodeVoidExtentBlockSRGB<BlockSize>(dst, blockData);
			}

			// Compute weight grid values.
			const int numWeights = computeNumWeights(blockMode);
			const int numWeightDataBits = computeNumRequiredBits(blockMode.weightISEParams, numWeights);
			const int numPartitions = (int)blockData.getBits(11, 12) + 1;
			// Check for errors in weight grid, partition and dual-plane parameters.
			if (numWeights > 64 ||
				numWeightDataBits > 96 ||
				numWeightDataBits < 24 ||
				blockMode.weightGridWidth > BlockSize ||
				blockMode.weightGridHeight > BlockSize ||
				(numPartitions == 4 && blockMode.isDualPlane))
			{
				setASTCErrorColorBlock(dst, BlockSize, BlockSize, isSRGB);
				return;
			}
			// Compute number of bits available for color endpoint data.
			const bool	isSingleUniqueCem = numPartitions == 1 || blockData.getBits(23, 24) == 0;
			const int	numConfigDataBits = (numPartitions == 1 ? 17 : isSingleUniqueCem ? 29 : 25 + 3 * numPartitions) +
				(blockMode.isDualPlane ? 2 : 0);
			const int	numBitsForColorEndpoints = 128 - numWeightDataBits - numConfigDataBits;
			const int	extraCemBitsStart = 127 - numWeightDataBits - (isSingleUniqueCem ? -1
				: numPartitions == 4 ? 7
				: numPartitions == 3 ? 4
				: numPartitions == 2 ? 1
				: 0);
			// Decode color endpoint modes.
			uint32 colorEndpointModes[4];
			decodeColorEndpointModes(&colorEndpointModes[0], blockData, numPartitions, extraCemBitsStart);
			const int numColorEndpointValues = computeNumColorEndpointValues(colorEndpointModes, numPartitions);
			// Check for errors in color endpoint value count.
			if (numColorEndpointValues > 18 || numBitsForColorEndpoints < (int)deDivRoundUp32(13 * numColorEndpointValues, 5))
			{
				setASTCErrorColorBlock(dst, BlockSize, BlockSize, isSRGB);
				return;
			}
			// Compute color endpoints.
			ColorEndpointPair colorEndpoints[4];
			computeColorEndpoints(&colorEndpoints[0], blockData, &colorEndpointModes[0], numPartitions, numColorEndpointValues,
				computeMaximumRangeISEParams(numBitsForColorEndpoints, numColorEndpointValues), numBitsForColorEndpoints);
			// Compute texel weights.
			TexelWeightPair texelWeights[MAX_BLOCK_WIDTH * MAX_BLOCK_HEIGHT];
			computeTexelWeights(&texelWeights[0], blockData, BlockSize, BlockSize, blockMode);
			// Set texel colors.
			const int		ccs = blockMode.isDualPlane ? (int)blockData.getBits(extraCemBitsStart - 2, extraCemBitsStart - 1) : -1;
			const uint32	partitionIndexSeed = numPartitions > 1 ? blockData.getBits(13, 22) : (uint32)-1;
			setTexelColorsSRGB<BlockSize>(dst, &colorEndpoints[0], &texelWeights[0], ccs, partitionIndexSeed, numPartitions, &colorEndpointModes[0]);
		}


#ifdef UE_MIRO_DEBUG
		template<int32 BlockSize>
		void LogBlock(const Block128& blockData)
		{
			// Decode block mode.
			uint32 BlockMode = blockData.getBits(0, 10);
			const ASTCBlockMode blockMode = getASTCBlockMode(BlockMode);

#define MODE_TO_BINARY(byte)  \
  ((byte) & 0x400 ? L'1' : L'0'), \
  ((byte) & 0x200 ? L'1' : L'0'), \
  ((byte) & 0x100 ? L'1' : L'0'), \
  ((byte) & 0x080 ? L'1' : L'0'), \
  ((byte) & 0x040 ? L'1' : L'0'), \
  ((byte) & 0x020 ? L'1' : L'0'), \
  ((byte) & 0x010 ? L'1' : L'0'), \
  ((byte) & 0x008 ? L'1' : L'0'), \
  ((byte) & 0x004 ? L'1' : L'0'), \
  ((byte) & 0x002 ? L'1' : L'0'), \
  ((byte) & 0x001 ? L'1' : L'0') 

			UE_LOGF(LogMutableCore, Log, "block_mode : %c%c %c%c%c%c %c%c%c%c%c", MODE_TO_BINARY(BlockMode));


			// Check for block mode errors.
			if (blockMode.isError)
			{
				miro_check(false);
			}
			// Separate path for void-extent.
			if (blockMode.isVoidExtent)
			{
				return; // decodeVoidExtentBlock(dst, blockData, blockWidth, blockHeight, isSRGB, isLDR);
			}

			// Compute weight grid values.
			const int numWeights = computeNumWeights(blockMode);
			const int numWeightDataBits = computeNumRequiredBits(blockMode.weightISEParams, numWeights);
			const int numPartitions = (int)blockData.getBits(11, 12) + 1;

			UE_LOGF(LogMutableCore, Log, "weight_grid : %d x %d", blockMode.weightGridWidth, blockMode.weightGridHeight);
			UE_LOGF(LogMutableCore, Log, "num_weights : %d", numWeights);
			UE_LOGF(LogMutableCore, Log, "num_weights_data_bits : %d", numWeightDataBits);
			UE_LOGF(LogMutableCore, Log, "num_partitions : %d", numPartitions);
			UE_LOGF(LogMutableCore, Log, "dual_plane : %d", blockMode.isDualPlane);

			// Check for errors in weight grid, partition and dual-plane parameters.
			if (numWeights > 64 ||
				numWeightDataBits > 96 ||
				numWeightDataBits < 24 ||
				blockMode.weightGridWidth > BlockSize ||
				blockMode.weightGridHeight > BlockSize ||
				(numPartitions == 4 && blockMode.isDualPlane))
			{
				miro_check(false);
			}

			// Compute number of bits available for color endpoint data.
			const bool	isSingleUniqueCem = numPartitions == 1 || blockData.getBits(23, 24) == 0;
			const int	numConfigDataBits = (numPartitions == 1 ? 17 : isSingleUniqueCem ? 29 : 25 + 3 * numPartitions) +
				(blockMode.isDualPlane ? 2 : 0);
			const int	numBitsForColorEndpoints = 128 - numWeightDataBits - numConfigDataBits;
			const int	extraCemBitsStart = 127 - numWeightDataBits - (isSingleUniqueCem ? -1
				: numPartitions == 4 ? 7
				: numPartitions == 3 ? 4
				: numPartitions == 2 ? 1
				: 0);

			// Decode color endpoint modes.
			uint32 colorEndpointModes[4];
			decodeColorEndpointModes(&colorEndpointModes[0], blockData, numPartitions, extraCemBitsStart);
			const int numColorEndpointValues = computeNumColorEndpointValues(colorEndpointModes, numPartitions);

			UE_LOGF(LogMutableCore, Log, "num_endpoint_data_bits : %d", numBitsForColorEndpoints);
			UE_LOGF(LogMutableCore, Log, "endpoint_mode : %d", colorEndpointModes[0]);
			UE_LOGF(LogMutableCore, Log, "num_endpoint_values : %d", numColorEndpointValues);

			// Check for errors in color endpoint value count.
			if (numColorEndpointValues > 18 || numBitsForColorEndpoints < (int)deDivRoundUp32(13 * numColorEndpointValues, 5))
			{
				miro_check(false);
			}
			// Compute color endpoints.
			ColorEndpointPair colorEndpoints[4];
			computeColorEndpoints(&colorEndpoints[0], blockData, &colorEndpointModes[0], numPartitions, numColorEndpointValues,
				computeMaximumRangeISEParams(numBitsForColorEndpoints, numColorEndpointValues), numBitsForColorEndpoints);

			UE_LOGF(LogMutableCore, Log, "endpoints : ( %3d, %3d, %3d, %3d ), ( %3d, %3d, %3d, %3d )",
				colorEndpoints[0].e0[0], colorEndpoints[0].e0[1], colorEndpoints[0].e0[2], colorEndpoints[0].e0[3],
				colorEndpoints[0].e1[0], colorEndpoints[0].e1[1], colorEndpoints[0].e1[2], colorEndpoints[0].e1[3]);

			// Compute texel weights.
			TexelWeightPair texelWeights[MAX_BLOCK_WIDTH * MAX_BLOCK_HEIGHT];
			//computeTexelWeights(&texelWeights[0], blockData, blockWidth, blockHeight, blockMode);

			ISEDecodedResult weightGrid[64];
			{
				BitAccessStream<false> dataStream(blockData, 127, numWeightDataBits);
				decodeISE(&weightGrid[0], computeNumWeights(blockMode), dataStream, blockMode.weightISEParams);
			}
			uint32 unquantizedWeights[64];
			{
				unquantizeWeights(&unquantizedWeights[0], &weightGrid[0], blockMode);
				interpolateWeights(&texelWeights[0], unquantizedWeights, BlockSize, BlockSize, blockMode);
			}

			int i = 0;
			//for (int32 Row = 0; Row < blockHeight; ++Row)
			//{
			//	UE_LOGF(LogMutableCore, Log, "texel weights : %3d %3d %3d %3d %3d %3d %3d %3d", texelWeights[i++].w[0], texelWeights[i++].w[0], texelWeights[i++].w[0], texelWeights[i++].w[0], texelWeights[i++].w[0], texelWeights[i++].w[0], texelWeights[i++].w[0], texelWeights[i++].w[0]);
			//}

			i = 0;
			for (int32 Row = 0; Row < blockMode.weightGridHeight; ++Row)
			{
				UE_LOGF(LogMutableCore, Log, "texel weights : %3d %3d %3d %3d %3d %3d %3d", unquantizedWeights[i++], unquantizedWeights[i++], unquantizedWeights[i++], unquantizedWeights[i++], unquantizedWeights[i++], unquantizedWeights[i++], unquantizedWeights[i++]);
			}

			// Set texel colors.
			const int		ccs = blockMode.isDualPlane ? (int)blockData.getBits(extraCemBitsStart - 2, extraCemBitsStart - 1) : -1;
			const uint32	partitionIndexSeed = numPartitions > 1 ? blockData.getBits(13, 22) : (uint32)-1;
			astcrt::vec4f_t dst[MAX_BLOCK_WIDTH * MAX_BLOCK_HEIGHT];
			DecompressResult Result = setTexelColorsSRGB<BlockSize>(dst, &colorEndpoints[0], &texelWeights[0], ccs, partitionIndexSeed, numPartitions, &colorEndpointModes[0]);
			check(Result == DecompressResult::DECOMPRESS_RESULT_VALID_BLOCK);

			UE_LOGF(LogMutableCore, Log, "decoded texels:");
			i = 0;
			for (int32 Row = 0; Row < BlockSize; ++Row)
			{
				FString Line;
				for (uint64 X = 0; X < BlockSize; ++X)
				{
					astcrt::vec4i_t texel;
					texel.components(0) = FMath::Clamp<int>((int)(dst[i].components(0) * 65536.0f + .5f), 0, 65535) >> 8;
					texel.components(1) = FMath::Clamp<int>((int)(dst[i].components(1) * 65536.0f + .5f), 0, 65535) >> 8;
					texel.components(2) = FMath::Clamp<int>((int)(dst[i].components(2) * 65536.0f + .5f), 0, 65535) >> 8;
					texel.components(3) = FMath::Clamp<int>((int)(dst[i].components(3) * 65536.0f + .5f), 0, 65535) >> 8;
					Line += FString::Printf(TEXT("(%3d,%3d,%3d,%3d), "), texel.r, texel.g, texel.b, texel.a);
					i++;
				}

				UE_LOGF(LogMutableCore, Log, "%ls", *Line);
			}

#undef MODE_TO_BINARY

		}
#endif //UE_MIRO_DEBUG

		template<int32 BlockSize>
		void decompressSRGB(uint8* Dest, const uint8* Block)
		{
			const Block128 blockData(Block);

			// (anticto) shortcut for blank blocks. This is not correct really, an all-zero block is invalid but we support it 
			// so that zero-initialized ASTC textures are actually black.
			if (blockData.isZero())
			{
				FMemory::Memzero(Dest, BlockSize * BlockSize * 4);
			}
			else
			{
				decompressBlockSRGB<BlockSize>(Dest, blockData);
			}
		}

	}


#endif // MIRO_INCLUDE_ASTC


} // namespace impl


using namespace impl;


//-------------------------------------------------------------------------------------------------
// Implementation of the miro interface
//-------------------------------------------------------------------------------------------------
namespace miro
{

	//---------------------------------------------------------------------------------------------
	void init_astc_decompress()
	{

#if MIRO_INCLUDE_ASTC

		if (!s_initialised_astc_decompress)
		{
			//arm::build_quantization_mode_table();

			//// pregenerate astc decompression tables for 4x4x1 blocks
			//arm::get_block_size_descriptor();
			//arm::generate_partition_tables();

			s_initialised_astc_decompress = 1;
		}

#endif

	}


	//---------------------------------------------------------------------------------------------
	void init_dxtc_decompress()
	{

#if MIRO_INCLUDE_BC

		if (!s_initialised_dxtc_decompress)
		{
			impl::stb::stb__InitDXT();

			s_initialised_dxtc_decompress = 1;
		}

#endif

	}


	//---------------------------------------------------------------------------------------------
	void initialize()
	{
		// do it only if necessary
		//init_astc_decompress();

		init_dxtc_decompress();
	}


	//---------------------------------------------------------------------------------------------
	void finalize()
	{
	}


#if MIRO_INCLUDE_BC

	//---------------------------------------------------------------------------------------------
	void RGB_to_BC1(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);
		//for ( uint32 y = 0; y < by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, Quality
			] (uint32 y)
			{
				uint8* rowTo = to + 8 * bx * y;
				for (uint32 x = 0; x < bx; ++x)
				{
					FMiroPixelBlock4x4_4Bytes block;

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);

							block.Bytes[b++] = pPixelSource[0];
							block.Bytes[b++] = pPixelSource[1];
							block.Bytes[b++] = pPixelSource[2];
							block.Bytes[b++] = 255;
						}
					}

					int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
					stb::stb_compress_dxt_block(rowTo, block.Bytes, 0, STBQuality);

					rowTo += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGBA_to_BC1(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);
		//for ( uint32 y = 0; y < by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, Quality
			] (uint32 y)
			{
				uint8* rowTo = to + 8 * bx * y;
				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 4];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);

							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[2];
							block[b++] = pPixelSource[3];
						}
					}

					int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
					stb::stb_compress_dxt_block(rowTo, block, 0, STBQuality);

					rowTo += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC1_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 8 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 4];

					bcdec::bcdec_bc1(rowFrom, block, 4 * 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 3 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = block[b + 2];
							}

							b += 4;
						}
					}

					rowFrom += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC1_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 8 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 4];

					bcdec::bcdec_bc1(rowFrom, block, 4 * 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 4 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = block[b + 2];
								pPixelDest[3] = block[b + 3];
							}

							b += 4;
						}
					}

					rowFrom += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGB_to_BC2(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, Quality
			] (uint32 y)
			{
				uint8* rowTo = to + 16 * bx * y;

				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 4];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);

							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[2];
							block[b++] = 255;
						}
					}

					int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
					stb::stb_compress_dxt_block(rowTo, block, 2, STBQuality);

					rowTo += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGBA_to_BC2(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 y = 0; y < by; ++y)
		ParallelFor(by,
			[
				bx, sx, sy, from, to, Quality
			] (uint32 y)
			{
				uint8* rowTo = to + 16 * bx * y;
				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 4];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);

							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[2];
							block[b++] = pPixelSource[3];
						}
					}

					int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
					stb::stb_compress_dxt_block(rowTo, block, 2, STBQuality);

					rowTo += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC2_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 16 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 4];

					uint8 const* colorBlock = rowFrom + 8;
					uint8 const* alphaBlock = rowFrom;

					bcdec::bcdec_bc2(rowFrom, block, 4 * 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 4 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = block[b + 2];
								pPixelDest[3] = block[b + 3];
							}

							b += 4;
						}
					}

					rowFrom += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC2_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 16 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 4];

					//uint8 const* colorBlock = from + 8;

					// TODO, can skip alpha part
					bcdec::bcdec_bc2(rowFrom, block, 4 * 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 3 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = block[b + 2];
							}

							b += 4;
						}
					}

					rowFrom += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGB_to_BC3(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, Quality
			] (uint32 y)
			{
				uint8* rowTo = to + 16 * bx * y;

				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 4];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);

							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[2];
							block[b++] = 255;
						}
					}

					int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
					stb::stb_compress_dxt_block(rowTo, block, 1, STBQuality);

					rowTo += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGBA_to_BC3(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, Quality
			] (uint32 y)
			{
				uint8* rowTo = to + 16 * bx * y;

				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 4];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);

							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[2];
							block[b++] = pPixelSource[3];
						}
					}

					int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
					stb::stb_compress_dxt_block(rowTo, block, 1, STBQuality);

					rowTo += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC3_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 16 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					FMiroPixelBlock4x4_4Bytes block;

					//uint8 const* colorBlock = from + 8;
					//uint8 const* alphaBlock = from;

					bcdec::bcdec_bc3(rowFrom, block.Bytes, 4 * 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 4 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block.Bytes[b + 0];
								pPixelDest[1] = block.Bytes[b + 1];
								pPixelDest[2] = block.Bytes[b + 2];
								pPixelDest[3] = block.Bytes[b + 3];
							}

							b += 4;
						}
					}

					rowFrom += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC3_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 16 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 4];

					bcdec::bcdec_bc3(rowFrom, block, 4 * 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 3 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = block[b + 2];
							}

							b += 4;
						}
					}

					rowFrom += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC1_to_BC3(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);
		uint32 bcount = bx * by;

		const auto& ProcessBlock = [from, to, Quality](uint32 b)
		{
			const uint8* blockFrom = from + 8 * b;
			uint8* blockTo = to + 16 * b;

			bool hasAlpha = BC1BlockHasAlpha(blockFrom);

			if (hasAlpha)
			{
				uint8 block[4 * 4 * 4];
				bcdec::bcdec_bc1(blockFrom, block, 4 * 4);

				int quality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
				stb::stb_compress_dxt_block(blockTo, block, 1, Quality);
			}
			else
			{
				// set the constant white alpha part
				FMemory::Memset(blockTo, 0xff, 8);

				// Copy the color part
				FMemory::Memcpy(blockTo + 8, blockFrom, 8);
			}
		};

		// \TODO: No benefit from setting a threshold?
		constexpr uint32 BlockCountConcurrencyThreshold = 0;
		if (bcount > BlockCountConcurrencyThreshold)
		{
			ParallelFor(bcount, ProcessBlock);
		}
		//else
		//{
		//	for (uint32 p = 0; p < bcount; ++p)
		//	{
		//		ProcessBlock(p);
		//	}
		//}

	}


	//---------------------------------------------------------------------------------------------
	void L_to_BC4(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		// We don't support quality setting for this format yet
		(void)Quality;

		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to
			] (uint32 y)
			{
				uint8* rowTo = to + 8 * bx * y;
				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 1 * (sx * rj + ri);
							block[b++] = pPixelSource[0];
						}
					}

					stb::stb_compress_bc4_block(rowTo, block);
					rowTo += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC4_to_L(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 8 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4];

					bcdec::bcdec_bc4(rowFrom, block, 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 1 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b];
							}

							++b;
						}
					}

					rowFrom += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC4_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 8 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4];

					bcdec::bcdec_bc4(rowFrom, block, 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 3 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b];
								pPixelDest[1] = block[b];
								pPixelDest[2] = block[b];
							}

							++b;
						}
					}

					rowFrom += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC4_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 8 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4];

					bcdec::bcdec_bc4(rowFrom, block, 4);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 4 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b];
								pPixelDest[1] = block[b];
								pPixelDest[2] = block[b];
								pPixelDest[3] = 255;
							}

							++b;
						}
					}

					rowFrom += 8;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGBA_to_BC5(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		// We don't support quality setting for this format yet
		(void)Quality;

		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to
			] (uint32 y)
			{
				uint8* rowTo = to + 16 * bx * y;
				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 2];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (int i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							int ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);
							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
						}
					}

					stb::stb_compress_bc5_block(rowTo, block);
					rowTo += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void RGB_to_BC5(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		// We don't support quality setting for this format yet
		(void)Quality;

		uint32 bx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 by = FMath::DivideAndRoundUp(sy, 4u);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to
			] (uint32 y)
			{
				uint8* rowTo = to + 16 * bx * y;
				for (uint32 x = 0; x < bx; ++x)
				{
					uint8 block[4 * 4 * 2];

					int b = 0;
					for (uint32 j = 0; j < 4; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * 4 + j);

						for (uint32 i = 0; i < 4; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * 4 + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);
							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[1];
						}
					}

					stb::stb_compress_bc5_block(rowTo, block);
					rowTo += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC5_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 16 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 2];
					bcdec::bcdec_bc5(rowFrom, block, 4 * 2);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 4 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = 255;
								pPixelDest[3] = 255;
							}

							b += 2;
						}
					}

					rowFrom += 16;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	void BC5_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		uint32 nx = FMath::DivideAndRoundUp(sx, 4u);
		uint32 ny = FMath::DivideAndRoundUp(sy, 4u);

		//for (uint32 by = 0; by < ny; ++by)
		ParallelFor(ny,
			[
				nx, sx, sy, from, to
			] (uint32 by)
			{
				const uint8* rowFrom = from + 16 * nx * by;

				for (uint32 bx = 0; bx < nx; ++bx)
				{
					uint8 block[4 * 4 * 2];
					bcdec::bcdec_bc5(rowFrom, block, 4 * 2);

					int b = 0;
					for (int j = 0; j < 4; ++j)
					{
						for (int i = 0; i < 4; ++i)
						{
							if (by * 4 + j < sy && bx * 4 + i < sx)
							{
								uint8* pPixelDest = to + 3 * (sx * (by * 4 + j) + (bx * 4 + i));

								pPixelDest[0] = block[b + 0];
								pPixelDest[1] = block[b + 1];
								pPixelDest[2] = 255;
							}

							b += 2;
						}
					}

					rowFrom += 16;
				}
			});
	}

#endif // MIRO_INCLUDE_BC


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
#if MIRO_INCLUDE_ASTC

	template<uint32 BLOCK_SIZE>
	void Generic_RGB_to_ASTCRGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, &physical_block_zero
			] (uint32 y)
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);
				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (uint32 j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (uint32 i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);

							block[b++] = pPixelSource[2];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[0];
							block[b++] = 0;
						}
					}

					*rowTo = physical_block_zero;
					astcrt::compress_block<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);

					++rowTo;
				}
			});
	}


	template<uint32 BLOCK_SIZE>
	void Generic_RGBA_to_ASTCRGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, &physical_block_zero
			] (uint32 y)
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);

				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (uint32 j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (uint32 i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);

							block[b++] = pPixelSource[2];
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[0];
							block[b++] = 0;
						}
					}

					*rowTo = physical_block_zero;
					astcrt::compress_block<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);

					++rowTo;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BlockSize>
	void Generic_ASTCRGBL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		init_astc_decompress();

		for (uint32 y = 0; y < sy; y += BlockSize)
		{
			for (uint32 x = 0; x < sx; x += BlockSize)
			{
				uint8 Block[BlockSize * BlockSize * 4];
				astcdec::decompressSRGB<BlockSize>(Block, from);

				for (uint32 py = 0; py < BlockSize; py++)
				{
					for (uint32 px = 0; px < BlockSize; px++)
					{
						uint32 xi = x + px;
						uint32 yi = y + py;

						if (xi < sx && yi < sy)
						{
							uint8* toPixel = to + (yi * sx + xi) * 3;
							toPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
							toPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
							toPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
						}
					}
				}

				from += 16;
			}
		}

	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BlockSize>
	void Generic_ASTCRGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		init_astc_decompress();

		for (uint32 y = 0; y < sy; y += BlockSize)
		{
			for (uint32 x = 0; x < sx; x += BlockSize)
			{
				uint8 Block[BlockSize * BlockSize * 4];
				astcdec::decompressSRGB<BlockSize>(Block, from);

				for (uint32 py = 0; py < BlockSize; py++)
				{
					for (uint32 px = 0; px < BlockSize; px++)
					{
						uint32 xi = x + px;
						uint32 yi = y + py;

						if (xi < sx && yi < sy)
						{
							uint8* toPixel = to + (yi * sx + xi) * 4;
							toPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
							toPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
							toPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
							toPixel[3] = 255;
						}
					}
				}

				from += 16;
			}
		}

	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BLOCK_SIZE>
	void Generic_RGBA_to_ASTCRGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

#ifdef UE_MIRO_DEBUG
		s_CurrentBlock = 0;
		for (uint32 y = 0; y < by; ++y)
#else
		ParallelFor(by, [bx, sx, sy, from, to, &physical_block_zero](uint32 y)
#endif
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);

				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (int j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (int i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);

							block[b++] = (uint8)pPixelSource[2];
							block[b++] = (uint8)pPixelSource[1];
							block[b++] = (uint8)pPixelSource[0];
							block[b++] = (uint8)pPixelSource[3];
						}
					}

					*rowTo = physical_block_zero;
					astcrt::compress_block_rgba<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);

					++rowTo;

#ifdef UE_MIRO_DEBUG
					++s_CurrentBlock;
				}
			}
#else
	}
});
#endif
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BLOCK_SIZE>
	void Generic_RGB_to_ASTCRGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, &physical_block_zero
			] (uint32 y)
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);

				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (int j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (int i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);

							block[b++] = (uint8)pPixelSource[2];
							block[b++] = (uint8)pPixelSource[1];
							block[b++] = (uint8)pPixelSource[0];
							block[b++] = 0;
						}
					}

					*rowTo = physical_block_zero;
					// \TODO: In this case we shoiuld weight-out A in all calculations, or even just compress as RGB.
					astcrt::compress_block_rgba<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);

					++rowTo;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BlockSize>
	void Generic_ASTCRGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		init_astc_decompress();

		const int32 NumBlocksY = FMath::DivideAndRoundUp(sy, BlockSize);
		const int32 NumBlocksX = FMath::DivideAndRoundUp(sx, BlockSize);

		if (sx <= 0 || sy <= 0)
		{
			return;
		}

#ifdef UE_MIRO_DEBUG
		s_CurrentBlock = 0;
		for (int32 BlockY = 0; BlockY < NumBlocksY; ++BlockY)
#else
		ParallelFor(NumBlocksY, [NumBlocksX, sx, sy, from, to](int32 BlockY)
#endif
			{
				for (int32 BlockX = 0; BlockX < NumBlocksX; ++BlockX)
				{
					uint8 Block[BlockSize * BlockSize * 4];

					constexpr int32 CompressedBlockSize = 16;
					const uint8* SrcBlockPtr = from + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

#ifdef UE_MIRO_DEBUG
					if (s_CurrentBlock == s_DebugBlock)
					{
						astcdec::LogBlock<BlockSize>(SrcBlockPtr);
					}
#endif

					astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

					for (uint32 py = 0; py < BlockSize; py++)
					{
						for (uint32 px = 0; px < BlockSize; px++)
						{
							uint32 xi = FMath::Min(BlockX * BlockSize + px, sx - 1);
							uint32 yi = FMath::Min(BlockY * BlockSize + py, sy - 1);

							uint8* toPixel = to + (yi * sx + xi) * 4;
							toPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
							toPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
							toPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
							toPixel[3] = Block[py * BlockSize * 4 + px * 4 + 3];
						}
					}

#ifdef UE_MIRO_DEBUG
					++s_CurrentBlock;
				}
			}
#else
	}
		});
#endif
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BlockSize>
	void Generic_ASTCRGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		init_astc_decompress();

		const int32 NumBlocksY = FMath::DivideAndRoundUp(sy, BlockSize);
		const int32 NumBlocksX = FMath::DivideAndRoundUp(sx, BlockSize);

		//for (uint32 BlockY = 0; BlockY < NumBlocksY; ++BlockY)
		ParallelFor(NumBlocksY, [NumBlocksX, sx, sy, from, to](int32 BlockY)
			{
				for (int32 BlockX = 0; BlockX < NumBlocksX; ++BlockX)
				{
					uint8 Block[BlockSize * BlockSize * 4];

					constexpr int32 CompressedBlockSize = 16;
					const uint8* SrcBlockPtr = from + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

					astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

					for (uint32 py = 0; py < BlockSize; py++)
					{
						for (uint32 px = 0; px < BlockSize; px++)
						{
							uint32 xi = BlockX * BlockSize + px;
							uint32 yi = BlockY * BlockSize + py;

							if (xi < sx && yi < sy)
							{
								uint8* toPixel = to + (yi * sx + xi) * 3;
								toPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
								toPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
								toPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
							}
						}
					}
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BLOCK_SIZE>
	void Generic_L_to_ASTCRGBL(uint32 sx, uint32 sy, const uint8_t* from, uint8_t* to, int32 Quality)
	{
		// Almost the same as RGB_to_ASTC4x4RGBL
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, &physical_block_zero
			] (uint32 y)
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);
				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (int j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (int i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + (sx * rj + ri);

							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[0];
							block[b++] = pPixelSource[0];
							block[b++] = 0;
						}
					}

					*rowTo = physical_block_zero;
					astcrt::compress_block<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);

					++rowTo;
				}
			});
	}

	//---------------------------------------------------------------------------------------------
	template<uint32 BLOCK_SIZE>
	void Generic_RGB_to_ASTCRGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, &physical_block_zero
			] (uint32 y)
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);

				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (int j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (int i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + 3 * (sx * rj + ri);

							block[b++] = 0;
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[0];
							block[b++] = 0;
						}
					}

					*rowTo = physical_block_zero;
					//astcrt::compress_block_la<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);
					astcrt::compress_block<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);
					++rowTo;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BLOCK_SIZE>
	void Generic_RGBA_to_ASTCRGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		(void)Quality;
		astcrt::PhysicalBlock physical_block_zero = { {0} };
		astcrt::PhysicalBlock* dst_re = reinterpret_cast<astcrt::PhysicalBlock*>(to);

		uint32 bx = FMath::DivideAndRoundUp(sx, BLOCK_SIZE);
		uint32 by = FMath::DivideAndRoundUp(sy, BLOCK_SIZE);

		//for ( uint32 y=0; y<by; ++y )
		ParallelFor(by,
			[
				bx, sx, sy, from, to, &physical_block_zero
			] (uint32 y)
			{
				astcrt::PhysicalBlock* rowTo = reinterpret_cast<astcrt::PhysicalBlock*>(to + sizeof(astcrt::PhysicalBlock) * bx * y);

				for (uint32 x = 0; x < bx; ++x)
				{
					// Fetch the block
					uint8 block[BLOCK_SIZE * BLOCK_SIZE * 4];
					int b = 0;
					for (int j = 0; j < BLOCK_SIZE; ++j)
					{
						// Clamp y in case block goes beyond the image. Biases the result.
						uint32 rj = FMath::Min(sy - 1, y * BLOCK_SIZE + j);

						for (int i = 0; i < BLOCK_SIZE; ++i)
						{
							// Clamp x in case block goes beyond the image. Biases the result.
							uint32 ri = FMath::Min(sx - 1, x * BLOCK_SIZE + i);

							const uint8* pPixelSource = from + 4 * (sx * rj + ri);

							block[b++] = 0;
							block[b++] = pPixelSource[1];
							block[b++] = pPixelSource[0];
							block[b++] = 0;
						}
					}

					*rowTo = physical_block_zero;
					//astcrt::compress_block_la<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);
					astcrt::compress_block<BLOCK_SIZE, BLOCK_SIZE>((astcrt::unorm8_t*)block, rowTo);
					++rowTo;
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BlockSize>
	void Generic_ASTCRGL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		init_astc_decompress();

		const int32 NumBlocksY = FMath::DivideAndRoundUp(sy, BlockSize);
		const int32 NumBlocksX = FMath::DivideAndRoundUp(sx, BlockSize);

		//for (uint32 by = 0; by < NumBlocksY; ++by)
		ParallelFor(NumBlocksY, [NumBlocksX, sx, sy, from, to](int32 by)
			{
				for (int32 bx = 0; bx < NumBlocksX; ++bx)
				{
					constexpr int32 CompressedBlockSize = 16;
					const uint8* SrcBlockPtr = from + (by * NumBlocksX + bx) * CompressedBlockSize;

					uint8 Block[BlockSize * BlockSize * 4];
					astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

					for (uint32 BlockY = 0; BlockY < BlockSize; BlockY++)
					{
						for (uint32 BlockX = 0; BlockX < BlockSize; BlockX++)
						{
							uint32 xi = bx * BlockSize + BlockX;
							uint32 yi = by * BlockSize + BlockY;

							if (xi < sx && yi < sy)
							{
								uint8* ToPixel = to + (yi * sx + xi) * 3;
								ToPixel[0] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 0];
								ToPixel[1] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 1];
								ToPixel[2] = 255;
							}
						}
					}
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	template<uint32 BlockSize>
	void Generic_ASTCRGL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		init_astc_decompress();

		const int32 NumBlocksY = FMath::DivideAndRoundUp(sy, BlockSize);
		const int32 NumBlocksX = FMath::DivideAndRoundUp(sx, BlockSize);

		//for (uint32 by = 0; by < NumBlocksY; ++by)
		ParallelFor(NumBlocksY, [NumBlocksX, sx, sy, from, to](int32 by)
			{
				for (int32 bx = 0; bx < NumBlocksX; ++bx)
				{
					constexpr int32 CompressedBlockSize = 16;
					const uint8* SrcBlockPtr = from + (by * NumBlocksX + bx) * CompressedBlockSize;

					uint8 Block[BlockSize * BlockSize * 4];
					astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

					for (uint32 BlockY = 0; BlockY < BlockSize; BlockY++)
					{
						for (uint32 BlockX = 0; BlockX < BlockSize; BlockX++)
						{
							uint32 xi = bx * BlockSize + BlockX;
							uint32 yi = by * BlockSize + BlockY;

							if (xi < sx && yi < sy)
							{
								uint8* ToPixel = to + (yi * sx + xi) * 4;
								ToPixel[0] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 0];
								ToPixel[1] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 1];
								ToPixel[2] = 255;
								ToPixel[3] = 255;
							}
						}
					}
				}
			});
	}


	//---------------------------------------------------------------------------------------------
	// 4x4
	//---------------------------------------------------------------------------------------------
	void ASTC4x4RGBL_to_ASTC4x4RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		int32 BlockCount = FMath::DivideAndRoundUp(sy, 4u) * FMath::DivideAndRoundUp(sx, 4u);
		FMemory::Memcpy(to, from, BlockCount * 16);
	}

	void ASTC4x4RGBAL_to_ASTC4x4RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		int32 BlockCount = FMath::DivideAndRoundUp(sy, 4u) * FMath::DivideAndRoundUp(sx, 4u);
		FMemory::Memcpy(to, from, BlockCount * 16);
	}

	void RGBA_to_ASTC4x4RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGBAL<4>(sx, sy, from, to, Quality);
	}

	void RGB_to_ASTC4x4RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGBAL<4>(sx, sy, from, to, Quality);
	}

	void ASTC4x4RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGBA<4>(sx, sy, from, to);
	}

	void ASTC4x4RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGB<4>(sx, sy, from, to);
	}

	void RGB_to_ASTC4x4RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGBL<4>(sx, sy, from, to, Quality);
	}

	void RGBA_to_ASTC4x4RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGBL<4>(sx, sy, from, to, Quality);
	}

	void ASTC4x4RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGB<4>(sx, sy, from, to);
	}

	void ASTC4x4RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGBA<4>(sx, sy, from, to);
	}

	void L_to_ASTC4x4RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_L_to_ASTCRGBL<4>(sx, sy, from, to, Quality);
	}

	void RGB_to_ASTC4x4RGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGL<4>(sx, sy, from, to, Quality);
	}

	void RGBA_to_ASTC4x4RGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGL<4>(sx, sy, from, to, Quality);
	}

	void ASTC4x4RGL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGB<4>(sx, sy, from, to);
	}

	void ASTC4x4RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGBA<4>(sx, sy, from, to);
	}


	//---------------------------------------------------------------------------------------------
	// 6x6
	//---------------------------------------------------------------------------------------------
	void RGBA_to_ASTC6x6RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGBAL<6>(sx, sy, from, to, Quality);
	}

	void RGB_to_ASTC6x6RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGBAL<6>(sx, sy, from, to, Quality);
	}

	void ASTC6x6RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGBA<6>(sx, sy, from, to);
	}

	void ASTC6x6RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGB<6>(sx, sy, from, to);
	}

	void RGB_to_ASTC6x6RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGBL<6>(sx, sy, from, to, Quality);
	}

	void RGBA_to_ASTC6x6RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGBL<6>(sx, sy, from, to, Quality);
	}

	void ASTC6x6RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGB<6>(sx, sy, from, to);
	}

	void ASTC6x6RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGBA<6>(sx, sy, from, to);
	}

	void RGB_to_ASTC6x6RGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGL<6>(sx, sy, from, to, Quality);
	}

	void RGBA_to_ASTC6x6RGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGL<6>(sx, sy, from, to, Quality);
	}

	void ASTC6x6RGL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGB<6>(sx, sy, from, to);
	}

	void ASTC6x6RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGBA<6>(sx, sy, from, to);
	}


	//---------------------------------------------------------------------------------------------
	// 8x8
	//---------------------------------------------------------------------------------------------
	void RGBA_to_ASTC8x8RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGBAL<8>(sx, sy, from, to, Quality);
	}

	void RGB_to_ASTC8x8RGBAL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGBAL<8>(sx, sy, from, to, Quality);
	}

	void ASTC8x8RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGBA<8>(sx, sy, from, to);
	}

	void ASTC8x8RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGB<8>(sx, sy, from, to);
	}

	void RGB_to_ASTC8x8RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGBL<8>(sx, sy, from, to, Quality);
	}

	void RGBA_to_ASTC8x8RGBL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGBL<8>(sx, sy, from, to, Quality);
	}

	void ASTC8x8RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGB<8>(sx, sy, from, to);
	}

	void ASTC8x8RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGBA<8>(sx, sy, from, to);
	}

	void RGB_to_ASTC8x8RGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGB_to_ASTCRGL<8>(sx, sy, from, to, Quality);
	}

	void RGBA_to_ASTC8x8RGL(uint32 sx, uint32 sy, const uint8* from, uint8* to, int32 Quality)
	{
		Generic_RGBA_to_ASTCRGL<8>(sx, sy, from, to, Quality);
	}

	void ASTC8x8RGL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGB<8>(sx, sy, from, to);
	}

	void ASTC8x8RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGBA<8>(sx, sy, from, to);
	}


	//---------------------------------------------------------------------------------------------
	// 10x10
	//---------------------------------------------------------------------------------------------
	void ASTC10x10RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGBA<10>(sx, sy, from, to);
	}

	void ASTC10x10RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGB<10>(sx, sy, from, to);
	}

	void ASTC10x10RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGB<10>(sx, sy, from, to);
	}

	void ASTC10x10RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGBA<10>(sx, sy, from, to);
	}

	void ASTC10x10RGL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGB<10>(sx, sy, from, to);
	}

	void ASTC10x10RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGBA<10>(sx, sy, from, to);
	}


	//---------------------------------------------------------------------------------------------
	// 12x12
	//---------------------------------------------------------------------------------------------
	void ASTC12x12RGBAL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGBA<12>(sx, sy, from, to);
	}

	void ASTC12x12RGBAL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBAL_to_RGB<12>(sx, sy, from, to);
	}

	void ASTC12x12RGBL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGB<12>(sx, sy, from, to);
	}

	void ASTC12x12RGBL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGBL_to_RGBA<12>(sx, sy, from, to);
	}

	void ASTC12x12RGL_to_RGB(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGB<12>(sx, sy, from, to);
	}

	void ASTC12x12RGL_to_RGBA(uint32 sx, uint32 sy, const uint8* from, uint8* to)
	{
		Generic_ASTCRGL_to_RGBA<12>(sx, sy, from, to);
	}

#endif // MIRO_INCLUDE_ASTC
}

namespace miro::SubImageCompression
{

#if MIRO_INCLUDE_BC
    void RGB_To_BC1SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		const int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_4Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize*4);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[2];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 3] = 255;
					}
				}

				constexpr int32 CompressedBlockSize = 8;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_dxt_block(DestBlockPtr, Block.Bytes, 0, STBQuality);
			}
		}
	}

    void RGBA_To_BC1SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		const int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_4Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize*4);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[2];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 3] = FromBlockPtr[3];
					}
				}

				constexpr int32 CompressedBlockSize = 8;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_dxt_block(DestBlockPtr, Block.Bytes, 0, STBQuality);
			}
		}
	}

	void RGB_To_BC2SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_4Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize*4);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[2];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 3] = 255;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_dxt_block(DestBlockPtr, Block.Bytes, 2, STBQuality);
			}
		}
	}

	void RGBA_To_BC2SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		const int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_4Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize*4);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[2];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 3] = FromBlockPtr[3];
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_dxt_block(DestBlockPtr, Block.Bytes, 2, STBQuality);
			}
		}
	}

	void RGB_To_BC3SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_4Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize*4);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[2];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 3] = 255;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_dxt_block(DestBlockPtr, Block.Bytes, 1, STBQuality);
			}
		}
	}

	void RGBA_To_BC3SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		const int32 STBQuality = Quality < 2 ? STB_DXT_NORMAL : STB_DXT_HIGHQUAL;
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_4Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize*4);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[2];
						Block.Bytes[BY * BlockSize * 4 + BX * 4 + 3] = FromBlockPtr[3];
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_dxt_block(DestBlockPtr, Block.Bytes, 1, STBQuality);
			}
		}
	}

	void L_To_BC4SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				FMiroPixelBlock4x4_1Bytes Block;
				static_assert(sizeof(Block) == BlockSize*BlockSize);

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 1;

						Block.Bytes[BY * BlockSize * 1 + BX * 1 + 0] = FromBlockPtr[0];
					}
				}

				constexpr int32 CompressedBlockSize = 8;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_bc4_block(DestBlockPtr, Block.Bytes);
			}
		}
	}

	void RGB_To_BC5SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		(void)Quality;

		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
			
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*2];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block[BY * BlockSize * 2 + BX * 2 + 0] = FromBlockPtr[0];
						Block[BY * BlockSize * 2 + BX * 2 + 1] = FromBlockPtr[1];
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_bc5_block(DestBlockPtr, Block);
			}
		}
	}

	void RGBA_To_BC5SubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To, int32 Quality)
	{
		(void)Quality;

		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
			
		constexpr uint32 BlockSize = 4; 

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*2];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block[BY * BlockSize * 2 + BX * 2 + 0] = FromBlockPtr[0];
						Block[BY * BlockSize * 2 + BX * 2 + 1] = FromBlockPtr[1];
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				stb::stb_compress_bc5_block(DestBlockPtr, Block);
			}
		}
	}

#endif // MIRO_INCLUDE_BC

#if MIRO_INCLUDE_ASTC
	template<uint32 BlockSize>
	void GenericSubImage_RGB_To_ASTCRGL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = 0;
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = 0;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_RGBA_To_ASTCRGL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = 0;
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = 0;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}


	template<uint32 BlockSize>
	void GenericSubImage_L_To_ASTCRGBL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 1;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = 0;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_RGB_To_ASTCRGBAL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[2];
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = 0;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_RGBA_To_ASTCRGBAL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[2];
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = FromBlockPtr[3];
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block_rgba<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_RGBA_To_ASTCRGBL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 4;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[2];
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = 0;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_RGB_To_ASTCRGBL(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		(void)Quality;
	
		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}
		
		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(ToSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		constexpr astcrt::PhysicalBlock PhysicalBlockZero = { {0} };
		
		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				alignas(4) uint8 Block[BlockSize*BlockSize*4];

				for (uint32 BY = 0; BY < BlockSize; ++BY)
				{
					for (uint32 BX = 0; BX < BlockSize; ++BX)
					{
						uint32 SrcX = FMath::Min(BlockX * BlockSize + BX, uint32(SubSize.X) - 1);
						uint32 SrcY = FMath::Min(BlockY * BlockSize + BY, uint32(SubSize.Y) - 1);

						const uint8* FromBlockPtr = From + (SrcY * FromSize.X + SrcX) * 3;

						Block[BY * BlockSize * 4 + BX * 4 + 0] = FromBlockPtr[2];
						Block[BY * BlockSize * 4 + BX * 4 + 1] = FromBlockPtr[1];
						Block[BY * BlockSize * 4 + BX * 4 + 2] = FromBlockPtr[0];
						Block[BY * BlockSize * 4 + BX * 4 + 3] = 0;
					}
				}

				constexpr int32 CompressedBlockSize = 16;
				uint8* DestBlockPtr = To + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;
				
				astcrt::PhysicalBlock* DestPhysicalBlock = reinterpret_cast<astcrt::PhysicalBlock*>(DestBlockPtr);
				*DestPhysicalBlock = PhysicalBlockZero;
				astcrt::compress_block<BlockSize, BlockSize>((astcrt::unorm8_t*)Block, DestPhysicalBlock);
			}
		}
	}


	void RGBA_To_ASTC4x4RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGBAL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC4x4RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGBAL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC4x4RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGBL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGBA_To_ASTC4x4RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGBL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void L_To_ASTC4x4RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_L_To_ASTCRGBL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC4x4RGLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGBA_To_ASTC4x4RGLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGL<4>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	//---------------------------------------------------------------------------------------------
	// 6x6
	//---------------------------------------------------------------------------------------------
	void RGBA_To_ASTC6x6RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGBAL<6>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC6x6RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGBAL<6>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC6x6RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGBL<6>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGBA_To_ASTC6x6RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGBL<6>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC6x6RGLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGL<6>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGBA_To_ASTC6x6RGLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGL<6>(FromSize, ToSize, SubSize, From, To, Quality);
	}


	//---------------------------------------------------------------------------------------------
	// 8x8
	//---------------------------------------------------------------------------------------------
	void RGBA_To_ASTC8x8RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGBAL<8>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC8x8RGBALSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGBAL<8>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC8x8RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGBL<8>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGBA_To_ASTC8x8RGBLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGBL<8>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGB_To_ASTC8x8RGLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGB_To_ASTCRGL<8>(FromSize, ToSize, SubSize, From, To, Quality);
	}

	void RGBA_To_ASTC8x8RGLSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To, int32 Quality)
	{
		GenericSubImage_RGBA_To_ASTCRGL<8>(FromSize, ToSize, SubSize, From, To, Quality);
	}


#endif // MIRO_INCLUDE_ASTC

}

namespace miro::SubImageDecompression
{

#if MIRO_INCLUDE_ASTC
	template<uint32 BlockSize>
	void GenericSubImage_ASTCRGBL_To_RGB(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		init_astc_decompress();

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				uint8 Block[BlockSize * BlockSize * 4];

				constexpr int32 CompressedBlockSize = 16;
				const uint8* SrcBlockPtr = From + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

				for (uint32 py = 0; py < BlockSize; py++)
				{
					for (uint32 px = 0; px < BlockSize; px++)
					{
						uint32 xi = FMath::Min(BlockX * BlockSize + px, uint32(SubSize.X) - 1);
						uint32 yi = FMath::Min(BlockY * BlockSize + py, uint32(SubSize.Y) - 1);

						uint8* ToPixel = To + (yi * ToSize.X + xi) * 3;
						ToPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
						ToPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
						ToPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
					}
				}
			}
		}

	}

	template<uint32 BlockSize>
	void GenericSubImage_ASTCRGBL_To_RGBA(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		init_astc_decompress();

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				uint8 Block[BlockSize * BlockSize * 4];

				constexpr int32 CompressedBlockSize = 16;
				const uint8* SrcBlockPtr = From + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

				for (uint32 py = 0; py < BlockSize; py++)
				{
					for (uint32 px = 0; px < BlockSize; px++)
					{
						uint32 xi = FMath::Min(BlockX * BlockSize + px, uint32(SubSize.X) - 1);
						uint32 yi = FMath::Min(BlockY * BlockSize + py, uint32(SubSize.Y) - 1);

						uint8* ToPixel = To + (yi * ToSize.X + xi) * 4;
						ToPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
						ToPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
						ToPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
						ToPixel[3] = 255;
					}
				}
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_ASTCRGBAL_To_RGBA(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		init_astc_decompress();

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		if (SubSize.X <= 0 || SubSize.Y <= 0)
		{
			return;
		}

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				uint8 Block[BlockSize * BlockSize * 4];

				constexpr int32 CompressedBlockSize = 16;
				const uint8* SrcBlockPtr = From + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

				for (uint32 py = 0; py < BlockSize; py++)
				{
					for (uint32 px = 0; px < BlockSize; px++)
					{
						uint32 xi = FMath::Min(BlockX * BlockSize + px, uint32(SubSize.X) - 1);
						uint32 yi = FMath::Min(BlockY * BlockSize + py, uint32(SubSize.Y) - 1);

						uint8* ToPixel = To + (yi * ToSize.X + xi) * 4;
						ToPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
						ToPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
						ToPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
						ToPixel[3] = Block[py * BlockSize * 4 + px * 4 + 3];
					}
				}
			}
		}
	}


	template<uint32 BlockSize>
	void GenericSubImage_ASTCRGBAL_To_RGB(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		init_astc_decompress();

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 BlockY = 0; BlockY < NumSubBlocksY; ++BlockY)
		{
			for (uint32 BlockX = 0; BlockX < NumSubBlocksX; ++BlockX)
			{
				uint8 Block[BlockSize * BlockSize * 4];

				constexpr int32 CompressedBlockSize = 16;
				const uint8* SrcBlockPtr = From + (BlockY * NumBlocksX + BlockX) * CompressedBlockSize;

				astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

				for (uint32 py = 0; py < BlockSize; py++)
				{
					for (uint32 px = 0; px < BlockSize; px++)
					{
						uint32 xi = BlockX * BlockSize + px;
						uint32 yi = BlockY * BlockSize + py;

						if (LIKELY((xi < ToSize.X) & (yi < ToSize.Y)))
						{
							uint8* ToPixel = To + (yi * ToSize.X + xi) * 3;
							ToPixel[0] = Block[py * BlockSize * 4 + px * 4 + 0];
							ToPixel[1] = Block[py * BlockSize * 4 + px * 4 + 1];
							ToPixel[2] = Block[py * BlockSize * 4 + px * 4 + 2];
						}
					}
				}
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_ASTCRGL_To_RGB(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		init_astc_decompress();

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				constexpr int32 CompressedBlockSize = 16;
				const uint8* SrcBlockPtr = From + (by * NumBlocksX + bx) * CompressedBlockSize;

				uint8 Block[BlockSize * BlockSize * 4];
				astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

				for (uint32 BlockY = 0; BlockY < BlockSize; BlockY++)
				{
					for (uint32 BlockX = 0; BlockX < BlockSize; BlockX++)
					{
						uint32 xi = bx * BlockSize + BlockX;
						uint32 yi = by * BlockSize + BlockY;

						if (LIKELY((xi < ToSize.X) & (yi < ToSize.Y)))
						{
							uint8* ToPixel = To + (yi * ToSize.X + xi) * 3;
							ToPixel[0] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 0];
							ToPixel[1] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 1];
							ToPixel[2] = 255;
						}
					}
				}
			}
		}
	}

	template<uint32 BlockSize>
	void GenericSubImage_ASTCRGL_To_RGBA(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		init_astc_decompress();

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				constexpr int32 CompressedBlockSize = 16;
				const uint8* SrcBlockPtr = From + (by * NumBlocksX + bx) * CompressedBlockSize;

				uint8 Block[BlockSize * BlockSize * 4];
				astcdec::decompressSRGB<BlockSize>(Block, SrcBlockPtr);

				for (uint32 BlockY = 0; BlockY < BlockSize; BlockY++)
				{
					for (uint32 BlockX = 0; BlockX < BlockSize; BlockX++)
					{
						uint32 xi = bx * BlockSize + BlockX;
						uint32 yi = by * BlockSize + BlockY;

						if (LIKELY((xi < ToSize.X) & (yi < ToSize.Y)))
						{
							uint8* ToPixel = To + (yi * ToSize.X + xi) * 4;
							ToPixel[0] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 0];
							ToPixel[1] = Block[BlockY * BlockSize * 4 + BlockX * 4 + 1];
							ToPixel[2] = 255;
							ToPixel[3] = 255;
						}
					}
				}
			}
		}
	}


	void ASTC4x4RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGBA<4>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC4x4RGBAL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGB<4>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC4x4RGBL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGB<4>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC4x4RGBL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGBA<4>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC4x4RGL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGB<4>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC4x4RGL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGBA<4>(FromSize, ToSize, SubSize, From, To);
	}


	//---------------------------------------------------------------------------------------------
	// 6x6
	//---------------------------------------------------------------------------------------------
	void ASTC6x6RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGBA<6>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC6x6RGBAL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGB<6>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC6x6RGBL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGB<6>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC6x6RGBL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGBA<6>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC6x6RGL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGB<6>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC6x6RGL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGBA<6>(FromSize, ToSize, SubSize, From, To);
	}

	//---------------------------------------------------------------------------------------------
	// 8x8
	//---------------------------------------------------------------------------------------------

	void ASTC8x8RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGBA<8>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC8x8RGBAL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGB<8>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC8x8RGBL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGB<8>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC8x8RGBL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGBA<8>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC8x8RGL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGB<8>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC8x8RGL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGBA<8>(FromSize, ToSize, SubSize, From, To);
	}


	//---------------------------------------------------------------------------------------------
	// 10x10
	//---------------------------------------------------------------------------------------------
	void ASTC10x10RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGBA<10>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC10x10RGBAL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGB<10>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC10x10RGBL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGB<10>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC10x10RGBL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGBA<10>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC10x10RGL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGB<10>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC10x10RGL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGBA<10>(FromSize, ToSize, SubSize, From, To);
	}


	//---------------------------------------------------------------------------------------------
	// 12x12
	//---------------------------------------------------------------------------------------------
	void ASTC12x12RGBAL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGBA<12>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC12x12RGBAL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBAL_To_RGB<12>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC12x12RGBL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGB<12>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC12x12RGBL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGBL_To_RGBA<12>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC12x12RGL_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGB<12>(FromSize, ToSize, SubSize, From, To);
	}

	void ASTC12x12RGL_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To)
	{
		GenericSubImage_ASTCRGL_To_RGBA<12>(FromSize, ToSize, SubSize, From, To);
	}

#endif // MIRO_INCLUDE_ASTC
	// BC Formats.

#if MIRO_INCLUDE_BC
	void BC1_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 8 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				uint8 Block[4 * 4 * 4];

				bcdec::bcdec_bc1(RowFrom, Block, 4 * 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 3 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = Block[b + 2];
						}

						b += 4;
					}
				}

				RowFrom += 8;
			}
		}
	}


	void BC1_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 8 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				uint8 Block[4 * 4 * 4];
				bcdec::bcdec_bc1(RowFrom, Block, 4 * 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 4 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = Block[b + 2];
							pPixelDest[3] = Block[b + 3];
						}

						b += 4;
					}
				}

				RowFrom += 8;
			}
		}
	}

	void BC2_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 16 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksY; ++bx)
			{
				uint8 Block[4 * 4 * 4];

				//uint8 const* ColorBlock = RowFrom + 8;
				//uint8 const* AlphaBlock = RowFrom;

				bcdec::bcdec_bc2(RowFrom, Block, 4 * 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 4 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = Block[b + 2];
							pPixelDest[3] = Block[b + 3];
						}

						b += 4;
					}
				}

				RowFrom += 16;
			}
		}
	}

	void BC2_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksX; ++by)
		{
			const uint8* RowFrom = From + 16 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksY; ++bx)
			{
				uint8 Block[4 * 4 * 4];

				//uint8 const* ColorBlock = from + 8;

				// TODO, can skip alpha part
				bcdec::bcdec_bc2(RowFrom, Block, 4 * 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 3 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = Block[b + 2];
						}

						b += 4;
					}
				}

				RowFrom += 16;
			}
		}
	}

	void BC3_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 16 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				FMiroPixelBlock4x4_4Bytes Block;

				//uint8 const* ColorBlock = From + 8;
				//uint8 const* AlphaBlock = From;

				bcdec::bcdec_bc3(RowFrom, Block.Bytes, 4 * 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 4 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block.Bytes[b + 0];
							pPixelDest[1] = Block.Bytes[b + 1];
							pPixelDest[2] = Block.Bytes[b + 2];
							pPixelDest[3] = Block.Bytes[b + 3];
						}

						b += 4;
					}
				}

				RowFrom += 16;
			}
		}
	}

	void BC3_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 16 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksY; ++bx)
			{
				uint8 Block[4 * 4 * 4];

				bcdec::bcdec_bc3(RowFrom, Block, 4 * 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 3 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = Block[b + 2];
						}

						b += 4;
					}
				}

				RowFrom += 16;
			}
		}
	}

	void BC4_To_LSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 8 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumBlocksX; ++bx)
			{
				uint8 Block[4 * 4];

				bcdec::bcdec_bc4(RowFrom, Block, 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 1 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b];
						}

						++b;
					}
				}

				RowFrom += 8;
			}
		}
	}


	void BC4_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 8 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				uint8 Block[4 * 4];

				bcdec::bcdec_bc4(RowFrom, Block, 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 3 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b];
							pPixelDest[1] = Block[b];
							pPixelDest[2] = Block[b];
						}

						++b;
					}
				}

				RowFrom += 8;
			}
		}
	}


	void BC4_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 8 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				uint8 Block[4 * 4];

				bcdec::bcdec_bc4(RowFrom, Block, 4);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 4 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b];
							pPixelDest[1] = Block[b];
							pPixelDest[2] = Block[b];
							pPixelDest[3] = 255;
						}

						++b;
					}
				}

				RowFrom += 8;
			}
		}
	}


	void BC5_To_RGBASubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 16 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				uint8 Block[4 * 4 * 2];
				bcdec::bcdec_bc5(RowFrom, Block, 4 * 2);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 4 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = 255;
							pPixelDest[3] = 255;
						}

						b += 2;
					}
				}

				RowFrom += 16;
			}
		}
	}

	void BC5_To_RGBSubImage(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		constexpr uint32 BlockSize = 4;

		const uint32 NumBlocksX = FMath::DivideAndRoundUp(uint32(FromSize.X), BlockSize);
		const uint32 NumSubBlocksX = FMath::DivideAndRoundUp(uint32(SubSize.X), BlockSize);
		const uint32 NumSubBlocksY = FMath::DivideAndRoundUp(uint32(SubSize.Y), BlockSize);

		for (uint32 by = 0; by < NumSubBlocksY; ++by)
		{
			const uint8* RowFrom = From + 16 * NumBlocksX * by;

			for (uint32 bx = 0; bx < NumSubBlocksX; ++bx)
			{
				uint8 Block[4 * 4 * 2];
				bcdec::bcdec_bc5(RowFrom, Block, 4 * 2);

				int32 b = 0;
				for (int32 j = 0; j < 4; ++j)
				{
					for (int32 i = 0; i < 4; ++i)
					{
						if (LIKELY((by * 4 + j < ToSize.Y) & (bx * 4 + i < ToSize.X)))
						{
							uint8* pPixelDest = To + 3 * (ToSize.X * (by * 4 + j) + (bx * 4 + i));

							pPixelDest[0] = Block[b + 0];
							pPixelDest[1] = Block[b + 1];
							pPixelDest[2] = 255;
						}

						b += 2;
					}
				}

				RowFrom += 16;
			}
		}
	}
#endif
}

#ifdef UE_MIRO_DEBUG
UE_ENABLE_OPTIMIZATION
#endif
