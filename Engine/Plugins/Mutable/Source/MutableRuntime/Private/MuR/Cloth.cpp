// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/Cloth.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


namespace UE::Mutable::Private
{
	bool FCloth::operator==(const FCloth& Other) const
	{
		return 
			ClothingAsset == Other.ClothingAsset &&
			Data == Other.Data && 
			AssetLODIndex == Other.AssetLODIndex;
	}


	int32 FCloth::GetDataSize() const
	{
		return Data.GetAllocatedSize();
	}


	void FCloth::Serialise(FOutputArchive& Ar) const
	{
		Ar << AssetLODIndex;
		Ar << Data;
		// Ar << PassthroughObject; // Serialized in the OP Args.
	}


	void FCloth::Unserialise(FInputArchive& Ar)
	{
		Ar >> AssetLODIndex;
		Ar >> Data;
	}


	bool FCloth::IsValid() const
	{
		return ClothingAsset.IsValid() && AssetLODIndex >= 0;
	}
}

