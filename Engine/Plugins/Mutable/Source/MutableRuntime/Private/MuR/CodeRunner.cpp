// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/CodeRunner.h"

#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Instance.h"
#include "MuR/LOD.h"
#include "MuR/Material.h"
#include "MuR/Mesh.h"
#include "MuR/SkeletalMesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableString.h"
#include "MuR/MutableTrace.h"
#include "MuR/GeometryUtils.h"
#include "MuR/OpImageApplyComposite.h"
#include "MuR/OpImageBinarise.h"
#include "MuR/OpImageBlend.h"
#include "MuR/OpImageLayer.h"
#include "MuR/OpImageColorMap.h"
#include "MuR/OpImageDisplace.h"
#include "MuR/OpImageInterpolate.h"
#include "MuR/OpImageLuminance.h"
#include "MuR/OpImageProject.h"
#include "MuR/OpImageRasterMesh.h"
#include "MuR/OpImageTransform.h"
#include "MuR/OpImageMultiCompose.h"
#include "MuR/OpLayoutPack.h"
#include "MuR/OpLayoutRemoveBlocks.h"
#include "MuR/OpMeshApplyLayout.h"
#include "MuR/OpMeshPrepareLayout.h"
#include "MuR/OpMeshApplyPose.h"
#include "MuR/OpMeshBind.h"
#include "MuR/OpMeshClipDeform.h"
#include "MuR/OpMeshClipMorphPlane.h"
#include "MuR/OpMeshClipWithMesh.h"
#include "MuR/OpMeshDifference.h"
#include "MuR/OpMeshExtractLayoutBlock.h"
#include "MuR/OpMeshFormat.h"
#include "MuR/OpMeshMerge.h"
#include "MuR/OpMeshMorph.h"
#include "MuR/OpMeshRemove.h"
#include "MuR/OpMeshReshape.h"
#include "MuR/OpMeshTransform.h"
#include "MuR/OpMeshTransformWithMesh.h"
#include "MuR/OpMeshTransformWithBone.h"
#include "MuR/OpSkeletalMeshMerge.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/System.h"
#include "MuR/External/FloatAdapter.h"
#include "MuR/External/MaterialAdapter.h"
#include "MuR/External/TextureAdapter.h"
#include "MuR/External/MeshAdapter.h"
#include "MuR/External/Operation.h"
#include "MuR/External/VectorAdapter.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/Tuple.h"

#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK
#include "GenericPlatform/GenericPlatformStackWalk.h"
#endif

namespace
{

int32 ForcedProjectionMode = -1;
FAutoConsoleVariableRef CVarForceProjectionSamplingMode (
	TEXT("mutable.ForceProjectionMode"),
	ForcedProjectionMode,
	TEXT("force mutable to use an specific projection mode, 0 = Point + None, 1 = Bilinear + TotalAreaHeuristic, -1 uses the values provided by the projector."),
	ECVF_Default);

float GlobalProjectionLodBias = 0.0f;
FAutoConsoleVariableRef CVarGlobalProjectionLodBias (
	TEXT("mutable.GlobalProjectionLodBias"),
	GlobalProjectionLodBias,
	TEXT("Lod bias applied to the lod resulting form the best mip computation for ImageProject operations, only used if a min filter method different than None is used."),
	ECVF_Default);

bool bUseProjectionVectorImpl = true;
FAutoConsoleVariableRef CVarUseProjectionVectorImpl (
	TEXT("mutable.UseProjectionVectorImpl"),
	bUseProjectionVectorImpl,
	TEXT("If set to true, enables the vectorized implementation of the projection pixel processing."),
	ECVF_Default);

float GlobalImageTransformLodBias = 0.0f;
FAutoConsoleVariableRef CVarGlobalImageTransformLodBias (
	TEXT("mutable.GlobalImageTransformLodBias"),
	GlobalImageTransformLodBias,
		TEXT("Lod bias applied to the lod resulting form the best mip computation for ImageTransform operations"),
	ECVF_Default);

bool bUseImageTransformVectorImpl = true;
FAutoConsoleVariableRef CVarUseImageTransformVectorImpl (
	TEXT("mutable.UseImageTransformVectorImpl"),
	bUseImageTransformVectorImpl,
	TEXT("If set to true, enables the vectorized implementation of the image transform pixel processing."),
	ECVF_Default);

int32 MaxWorkerThreadsForIterativeOpExec = 4;
FAutoConsoleVariableRef CVarMaxWorkerThreadsForIterativeOpExec(
	TEXT("mutable.MaxWorkerThreadsForIterativeOpExec"),
	MaxWorkerThreadsForIterativeOpExec,
	TEXT("Sets the maximum number of worker threads to consider executing the iterative variant of an operation.")
	TEXT("A value of -1 forces the iterative op execution, a value 0 prevents the iterative variant to be used."),
	ECVF_Default);
	
TAutoConsoleVariable<bool> CVarForceGeometryOnFirstGeneration(
	TEXT("mutable.ForceGeometryOnFirstGeneration"),
	false,
	TEXT("If set to true, forces geometry generation on first generation even if the LOD will be streamed."));

int32 MutableMeshesLODBias = 0;
FAutoConsoleVariableRef CVarMutableMeshesLODBias(
	TEXT("mutable.MeshLODBias"), MutableMeshesLODBias,
	TEXT("LOD Bias applied to Mutable-generated meshes when Mesh LOD streaming is turned off."),
	ECVF_Default);

TAutoConsoleVariable<bool> CVarUseMergeLODMeshesForConversion(
	TEXT("mutable.UseOptimizedMergeForConversion"),
	true,
	TEXT("If set to true, Merge all LOD meshes and strips data that not required for the final conversion to UObject."));
}


#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK

#define AddOp(...) Invoke([&](){ AddOp(__VA_ARGS__); });

namespace UE::Mutable::Private::Private
{
	FString DumpItemScheduledCallstack(const FScheduledOp& Item)
	{
		constexpr SIZE_T MaxStringSize = 16 * 1024;
		ANSICHAR StackTrace[MaxStringSize];

		FString OutputString;

		constexpr uint32 EntriesToSkip = 3;
		for (uint32 Index = EntriesToSkip; Index < Item.StackDepth; ++Index)
		{
			StackTrace[0] = 0;
			FPlatformStackWalk::ProgramCounterToHumanReadableString(Index, Item.ScheduleCallstack[Index], StackTrace, MaxStringSize, nullptr);
			OutputString += FString::Printf(TEXT("\t\t%d %s\n"), Index - EntriesToSkip, ANSI_TO_TCHAR(StackTrace));
		}

		return OutputString;
	}
}

#endif


namespace UE::Mutable::Private
{

	TSharedRef<CodeRunner> CodeRunner::Create(
		const TSharedRef<FLiveInstance>& InLiveInstance,
		const FSettings& InSettings,
		FSystem& InSystem,
		EExecutionStrategy InExecutionStrategy,
		FOperation::ADDRESS At,
		uint16 ExecutionOptions,
		FScheduledOp::EType Type)
	{
		return MakeShared<CodeRunner>(InLiveInstance, FPrivateToken {},
				InSettings, InSystem, InExecutionStrategy, At, ExecutionOptions, Type);
	}


    CodeRunner::CodeRunner(
		const TSharedRef<FLiveInstance>& InLiveInstance,
    	FPrivateToken PrivateToken, 
		const FSettings& InSettings,
		FSystem& InSystem,
		EExecutionStrategy InExecutionStrategy,
		FOperation::ADDRESS At,
		uint16 ExecutionOptions,
		FScheduledOp::EType Type)
		: Settings(InSettings)
		, LiveInstance(InLiveInstance)
		, ExecutionStrategy(InExecutionStrategy)
		, System(InSystem)
		, Model(*InLiveInstance->Model.Get())
		, Program(InLiveInstance->Model->GetProgram())
		, Parameters(*InLiveInstance->Parameters)
		, ExternalResourceProvider(InLiveInstance->ExternalResourceProvider)
	{
		MUTABLE_CPUPROFILER_SCOPE(CodeRunner_Create)
		
		RootOPAddress = At;
		RootOpExecutionOptions = ExecutionOptions;
		RootOpType = Type;
		
		if (Type == FScheduledOp::EType::ImageDesc)
		{
			ImageDescConstantImages.Reserve(32);
		}
	
		// We will read this in the end, so make sure we keep it.
		GetMemory().UpdateHitCount(FCacheAddress(At, 0, ExecutionOptions, Type), 1);
		
		// Push the root operation
		FScheduledOp RootOp;
		RootOp.At = At;
		RootOp.ExecutionOptions = ExecutionOptions;
		RootOp.Type = static_cast<uint16>(Type);
		AddOp(RootOp);
	}


	FProgramCache& CodeRunner::GetMemory()
    {
		return *LiveInstance->Cache;
	}

	FLiveInstanceLogger& CodeRunner::GetLogger()
    {
		return *LiveInstance->UpdateLogger;
	}

	TTuple<UE::Tasks::FTask, TFunction<void()>> CodeRunner::LoadExternalImageAsync(FExternalResourceId Id, uint8 MipmapsToSkip, TFunction<void(TManagedPtr<FImage>)>& ResultCallback)
    {
		MUTABLE_CPUPROFILER_SCOPE(LoadExternalImageAsync);
		
		if (ExternalResourceProvider)
		{
			if (Id.ReferenceResourceId == PASSTHROUGH_ID_INVALID)
			{
				// It's a parameter image
				constexpr bool bLoadMipTail = true;
				return ExternalResourceProvider->GetImageAsync(Id.ImageParameter, MipmapsToSkip, bLoadMipTail, ResultCallback);
			}
			else
			{
				// It's an image reference
				return ExternalResourceProvider->GetReferencedImageAsync( Id.ReferenceResourceId, MipmapsToSkip, ResultCallback);
			}
		}
		else
		{
			// Not found and there is no generator!
			check(false);
		}

		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	}


	TTuple<UE::Tasks::FTask, TFunction<void()>> CodeRunner::LoadExternalMeshAsync(FExternalResourceId Id, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 ConversionFlags, TFunction<void(TManagedPtr<FMesh>)>& ResultCallback)
	{
		MUTABLE_CPUPROFILER_SCOPE(LoadExternalImageAsync);

		if (ExternalResourceProvider)
		{
			if (Id.ReferenceResourceId == PASSTHROUGH_ID_INVALID)
			{
				// It's a parameter mesh
				return ExternalResourceProvider->GetMeshAsync(Id.MeshParameter, BaseLODIndex, BaseSectionIndex, LODBias, ConversionFlags, ResultCallback);
			}
			else
			{
				// It's a mesh reference
				check(false);
			}
		}
		else
		{
			// Not found and there is no generator!
			check(false);
		}

		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	}

	TTuple<UE::Tasks::FTask, TFunction<void()>> CodeRunner::LoadExternalSkeletalMeshAsync(FExternalResourceId Id, int32 LODBegin, int32 LODEnd, int32 GeometryLODBegin, int32 GeometryLODEnd, uint8 ConversionFlags, TFunction<void(TManagedPtr<FSkeletalMesh>)>& ResultCallback)
	{
		MUTABLE_CPUPROFILER_SCOPE(LoadExternalSkeletalMeshAsync);

		if (ExternalResourceProvider)
		{
			if (Id.ReferenceResourceId == PASSTHROUGH_ID_INVALID)
			{
				// It's a parameter mesh
				return ExternalResourceProvider->GetSkeletalMeshAsync(
						Id.MeshParameter, LODBegin, LODEnd, GeometryLODBegin, GeometryLODEnd, ConversionFlags, ResultCallback);
			}
			else
			{
				// It's a mesh reference
				check(false);
			}
		}
		else
		{
			// Not found and there is no generator!
			check(false);
		}

		return MakeTuple(UE::Tasks::MakeCompletedTask<void>(), []() -> void {});
	}


	
	FExtendedImageDesc CodeRunner::GetExternalImageDesc(UTexture* Image)
	{
		MUTABLE_CPUPROFILER_SCOPE(GetExternalImageDesc);

		if (ExternalResourceProvider)
		{
			return ExternalResourceProvider->GetImageDesc(Image);
		}
		else
		{
			// Not found and there is no generator!
			check(false);
		}

		return FExtendedImageDesc();
	}

	
    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Conditional( const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Conditional);
		
		EOpType type = Program.GetOpType(Item.At);
		FOperation::ConditionalArgs Args = Program.GetOpArgs<FOperation::ConditionalArgs>(Item.At);

        // Conditionals have the following execution stages:
        // 0: we need to run the condition
        // 1: we need to run the branch
        // 2: we need to fetch the result and store it in this op
        switch( Item.Stage )
        {
        case 0:
        {
            AddOp( FScheduledOp( Item.At,Item,1 ),
                   FScheduledOp( Args.condition, Item ) );
            break;
        }

        case 1:
        {
            // Get the condition result

            // If there is no expression, we'll assume true.
            bool value = true;
            value = LoadBool( FCacheAddress(Args.condition, Item.ExecutionIndex, Item.ExecutionOptions) );

            FOperation::ADDRESS resultAt = value ? Args.yes : Args.no;

            // Schedule the end of this instruction if necessary
            AddOp( FScheduledOp( Item.At, Item, 2, (uint32)value),
				FScheduledOp( resultAt, Item) );

            break;
        }

        case 2:
        {
            FOperation::ADDRESS resultAt = Item.CustomState ? Args.yes : Args.no;

            // Store the final result
            FCacheAddress cat( Item );
            FCacheAddress rat( resultAt, Item );
            switch (GetOpDataType(type))
            {
            case EDataType::Bool:			StoreBool( cat, LoadBool(rat) ); break;
            case EDataType::Int:			StoreInt( cat, LoadInt(rat) ); break;
            case EDataType::Scalar:			StoreScalar( cat, LoadScalar(rat) ); break;
			case EDataType::String:			StoreString( cat, LoadString( rat ) ); break;
            case EDataType::Color:			StoreColor( cat, LoadColor( rat ) ); break;
            case EDataType::Projector:		StoreProjector( cat, LoadProjector(rat) ); break;
            case EDataType::Mesh:			StoreMesh( cat, LoadMesh(rat) ); break;
            case EDataType::Image:			StoreImage( cat, LoadImage(rat) ); break;
            case EDataType::Layout:			StoreLayout( cat, LoadLayout(rat) ); break;
            case EDataType::Instance:		StoreInstance( cat, LoadInstance(rat) ); break;
			case EDataType::ExtensionData:	StoreExtensionData( cat, LoadExtensionData(rat) ); break;
			case EDataType::Material:		StoreMaterial( cat, LoadMaterial(rat) ); break;
			case EDataType::SkeletalMesh:	StoreSkeletalMesh( cat, LoadSkeletalMesh(rat) ); break;
            default:
                // Not implemented
                check( false );
            }

            break;
        }

        default:
            check(false);
        }
    }


	//---------------------------------------------------------------------------------------------
	void CodeRunner::RunCode_Switch(const FScheduledOp& Item)
	{
		EOpType Type = Program.GetOpType(Item.At);

		const uint8* Data = Program.GetOpArgsPointer(Item.At);

		FOperation::ADDRESS VarAddress;
		FMemory::Memcpy(&VarAddress, Data, sizeof(FOperation::ADDRESS));
		Data += sizeof(FOperation::ADDRESS);

		FOperation::ADDRESS DefAddress;
		FMemory::Memcpy(&DefAddress, Data, sizeof(FOperation::ADDRESS));
		Data += sizeof(FOperation::ADDRESS);

		FOperation::FSwitchCaseDescriptor CaseDesc;
		FMemory::Memcpy(&CaseDesc, Data, sizeof(FOperation::FSwitchCaseDescriptor));
		Data += sizeof(FOperation::FSwitchCaseDescriptor);

		switch (Item.Stage)
		{
		case 0:
		{
			if (VarAddress)
			{
				AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(VarAddress, Item));
			}
			else
			{
				switch (GetOpDataType(Type))
				{
				case EDataType::Bool:				StoreBool(Item, false); break;
				case EDataType::Int:				StoreInt(Item, 0); break;
				case EDataType::Scalar:				StoreScalar(Item, 0.0f); break;
				case EDataType::String:				StoreString(Item, nullptr); break;
				case EDataType::Color:				StoreColor(Item, FVector4f()); break;
				case EDataType::Projector:			StoreProjector(Item, FProjector()); break;
				case EDataType::Mesh:				StoreMesh(Item, nullptr); break;
				case EDataType::Image:				StoreImage(Item, nullptr); break;
				case EDataType::Layout:				StoreLayout(Item, nullptr); break;
				case EDataType::Instance:			StoreInstance(Item, nullptr); break;
				case EDataType::ExtensionData:		StoreExtensionData(Item, MakeManaged<FExtensionData>()); break;
				case EDataType::Material:			StoreMaterial(Item, MakeManaged<FMaterial>()); break;
				case EDataType::InstancedStruct:	StoreInstancedStruct(Item, MakeManaged<FInstancedStruct>()); break;
				case EDataType::SkeletalMesh:		StoreSkeletalMesh(Item, MakeManaged<FSkeletalMesh>()); break;
				default:
					// Not implemented
					check(false);
				}
			}
			break;
		}

		case 1:
		{
			// Get the variable result
			int32 Var = LoadInt(FCacheAddress(VarAddress, Item));

			FOperation::ADDRESS ValueAt = DefAddress;

			if (!CaseDesc.bUseRanges)
			{
				for (uint32 C = 0; C < CaseDesc.Count; ++C)
				{
					int32 Condition;
					FMemory::Memcpy(&Condition, Data, sizeof(int32));
					Data += sizeof(int32);

					FOperation::ADDRESS CaseAt;
					FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					if (CaseAt && Var == (int32)Condition)
					{
						ValueAt = CaseAt;
						break;
					}
				}
			}
			else
			{
				for (uint32 C = 0; C < CaseDesc.Count; ++C)
				{
					int32 ConditionStart;
					FMemory::Memcpy(&ConditionStart, Data, sizeof(int32));
					Data += sizeof(int32);

					uint32 RangeSize;
					FMemory::Memcpy(&RangeSize, Data, sizeof(uint32));
					Data += sizeof(uint32);

					FOperation::ADDRESS CaseAt;
					FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					if (CaseAt && Var >= ConditionStart && Var < int32(ConditionStart + RangeSize))
					{
						ValueAt = CaseAt;
						break;
					}
				}

			}

            // Schedule the end of this instruction if necessary
            AddOp(FScheduledOp(Item.At, Item, 2, ValueAt),
				FScheduledOp(ValueAt, Item));

            break;
        }

        case 2:
        {
			FOperation::ADDRESS ResultAt = FOperation::ADDRESS(Item.CustomState);

            // Store the final result
            FCacheAddress ItemAddress(Item);
            FCacheAddress ResultAddress(ResultAt, Item);
            switch (GetOpDataType(Type))
            {
            case EDataType::Bool:				StoreBool(ItemAddress, LoadBool(ResultAddress)); break;
            case EDataType::Int:				StoreInt(ItemAddress, LoadInt(ResultAddress)); break;
            case EDataType::Scalar:				StoreScalar(ItemAddress, LoadScalar(ResultAddress)); break;
            case EDataType::String:				StoreString(ItemAddress, LoadString(ResultAddress)); break;
            case EDataType::Color:				StoreColor(ItemAddress, LoadColor(ResultAddress)); break;
            case EDataType::Projector:			StoreProjector(ItemAddress, LoadProjector(ResultAddress)); break;
			case EDataType::Mesh:				StoreMesh(ItemAddress, LoadMesh(ResultAddress)); break;
            case EDataType::Image:				StoreImage(ItemAddress, LoadImage(ResultAddress)); break;
            case EDataType::Layout:				StoreLayout(ItemAddress, LoadLayout(ResultAddress)); break;
            case EDataType::Instance:			StoreInstance(ItemAddress, LoadInstance(ResultAddress)); break;
			case EDataType::ExtensionData:		StoreExtensionData(ItemAddress, LoadExtensionData(ResultAddress)); break;
			case EDataType::Material:			StoreMaterial(ItemAddress, LoadMaterial(ResultAddress)); break;
			case EDataType::InstancedStruct:	StoreInstancedStruct(ItemAddress, LoadInstancedStruct(ResultAddress)); break;
			case EDataType::SkeletalMesh:		StoreSkeletalMesh(ItemAddress, LoadSkeletalMesh(ResultAddress)); break;
            default:
                // Not implemented
                check(false);
            }

            break;
        }

        default:
            check(false);
        }
    }


    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Instance(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Instance);
		
		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {
        case EOpType::IN_ADDCOMPONENT:
        {
        	MUTABLE_CPUPROFILER_SCOPE(IN_ADDCOMPONENT);
        		
			FOperation::InstanceAddComponentArgs Args = Program.GetOpArgs<FOperation::InstanceAddComponentArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.Instance, Item),
                           FScheduledOp( Args.Value, Item) );
                break;

            case 1:
            {
				TManagedPtr<const FInstance> Base = LoadInstance( FCacheAddress(Args.Instance,Item) );
				TManagedPtr<FInstance> Result = nullptr;

				if (!Base)
				{
					Result = MakeManaged<FInstance>();
				}
				else
				{
					Result = UE::Mutable::Private::CloneOrTakeOver<FInstance>(MoveTemp(Base));
				}

            	TManagedPtr<const FInstance> Instance;
                if (Args.Value)
                {
                    Instance = LoadInstance(FCacheAddress(Args.Value,Item));
                }
            		
            	int32 NewComponentIndex = Result->AddComponent();

            	if (!Instance->Components.IsEmpty())
            	{
            		Result->Components[NewComponentIndex] = Instance->Components[0];
            	}
            		
            	Result->Components[NewComponentIndex].Id = Args.ExternalId;
            		
                StoreInstance( Item, Result );
                break;
            }

            default:
                check(false);
            }

            break;
        }
        
		case EOpType::IN_ADDEXTENSIONDATA:
		{
        	MUTABLE_CPUPROFILER_SCOPE(IN_ADDEXTENSIONDATA);
        		
			FOperation::InstanceAddExtensionDataArgs Args = Program.GetOpArgs<FOperation::InstanceAddExtensionDataArgs>(Item.At);
			switch (Item.Stage)
			{
				case 0:
				{
					// Must pass in an Instance op and FExtensionData op
					check(Args.Instance);
					check(Args.ExtensionData);

					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Instance, Item),
						FScheduledOp(Args.ExtensionData, Item));

					break;
				}

				case 1:
				{
					// Assemble result
					TManagedPtr<FInstance> Instance;
					if (Args.Instance)
					{
						Instance = UE::Mutable::Private::CloneOrTakeOver<FInstance>(LoadInstance(FCacheAddress(Args.Instance, Item)));
					}
					else
					{
						Instance = MakeManaged<FInstance>();
					}

					if (TManagedPtr<const FExtensionData> ExtensionData = LoadExtensionData(FCacheAddress(Args.ExtensionData, Item)))
					{
						Instance->ExtensionData.Add(ExtensionData);
					}

					StoreInstance(Item, Instance);
					break;
				}

				default:
					check(false);
			}
			
			break;
		}
        	
		default:
            check(false);
        }
    }


    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_InstanceAddResource(const FScheduledOp& Item)
    {
		MUTABLE_CPUPROFILER_SCOPE(RunCode_InstanceAddResource);
		
		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {
        case EOpType::IN_ADDOVERLAYMATERIAL:
        	{
        		MUTABLE_CPUPROFILER_SCOPE(IN_ADDOVERLAYMATERIAL);
        		
        		FOperation::InstanceAddOverlayMaterialArgs Args = Program.GetOpArgs<FOperation::InstanceAddOverlayMaterialArgs>(Item.At);
        		switch (Item.Stage)
        		{
        		case 0:
        			{
        				AddOp(FScheduledOp(Item.At, Item, 1), 
        					FScheduledOp(Args.Instance, Item));
        				break;
        			}

        		case 1:
        			{
        				TManagedPtr<const FInstance> Base = LoadInstance(FCacheAddress(Args.Instance, Item));
						TManagedPtr<FInstance> Result = nullptr;
						if (!Base)
						{
							Result = MakeManaged<FInstance>();
						}
						else
						{
							Result = UE::Mutable::Private::CloneOrTakeOver<FInstance>(MoveTemp(Base));
						}

        				if (Args.Material)
        				{
        					const FMaterialId MaterialId = FSystem::GetMaterialId(LiveInstance->MaterialIdRegistry, Model, Parameters, Args.Material);
        					
        					const FName SlotName = Args.SlotName == TNumericLimits<uint32>::Max() ? FName() : FName(Program.ConstantNames[Args.SlotName]);
        					if (SlotName.IsNone())
        					{
        						Result->SetOverlayMaterialId(0, MaterialId);
        					}
					        else
					        {
					        	Result->AddOverlayMaterial(0, SlotName, MaterialId);
					        }
        				}

        				StoreInstance(Item, Result);
        				break;
        			}

        		default:
        			check(false);
        		}

        		break;
        	}

        case EOpType::IN_ADDOVERRIDEMATERIAL:
        	{
        		MUTABLE_CPUPROFILER_SCOPE(IN_ADDOVERRIDEMATERIAL);
        		
        		FOperation::InstanceAddOverrideMaterialArgs Args = Program.GetOpArgs<FOperation::InstanceAddOverrideMaterialArgs>(Item.At);
        		switch (Item.Stage)
        		{
        		case 0:
        			{
        				AddOp(FScheduledOp(Item.At, Item, 1),
        					FScheduledOp(Args.Instance, Item));
        				break;
        			}

        		case 1:
        			{
        				TManagedPtr<const FInstance> Base = LoadInstance(FCacheAddress(Args.Instance, Item));
        				TManagedPtr<FInstance> Result;
        				if (!Base)
        				{
        					Result = MakeManaged<FInstance>();
        				}
        				else
        				{
        					Result = UE::Mutable::Private::CloneOrTakeOver<FInstance>(MoveTemp(Base));
        				}

        				if (Args.Material)
        				{
        					FMaterialId MaterialId = FSystem::GetMaterialId(LiveInstance->MaterialIdRegistry, Model, Parameters, Args.Material);
        					const FString& SlotName = Program.ConstantStrings[Args.SlotName];
        					Result->AddOverrideMaterial(0, FName(SlotName), MaterialId);
        				}

        				StoreInstance(Item, Result);
        				break;
        			}

        		default:
        			check(false);
        		}

        		break;
        	}
        
		case EOpType::IN_ADDSKELETALMESH:
        	{
        		MUTABLE_CPUPROFILER_SCOPE(IN_ADDSKELETALMESH);
        		
        		FOperation::InstanceAddArgs Args = Program.GetOpArgs<FOperation::InstanceAddArgs>(Item.At);
        		switch (Item.Stage)
        		{
        		case 0:
        			{
        				AddOp(FScheduledOp(Item.At, Item, 1), 
        					FScheduledOp(Args.instance, Item));
        				break;
        			}

        		case 1:
        			{
        				TManagedPtr<const FInstance> InInstance;
        				if (Args.instance)
        				{
        					InInstance = LoadInstance(FCacheAddress(Args.instance, Item));
        				}

        				TManagedPtr<FInstance> Instance;
        				if (InInstance)
        				{
        					Instance = UE::Mutable::Private::CloneOrTakeOver<FInstance>(MoveTemp(InInstance));
        				}
        				else
        				{
        					Instance = MakeManaged<FInstance>();
        				}


        				if (Args.value)
        				{
        					FSkeletalMeshId SkeletalMeshId = FSystem::GetSkeletalMeshId(LiveInstance->SkeletalMeshIdRegistry, Model, Parameters, Args.value);
        					Instance->SetSkeletalMeshId(0, SkeletalMeshId);
        				}
						
        				StoreInstance( Item, Instance);
        				break;
        			}

        		default:
        			unimplemented();
        		}

        		break;
        	}
        	
        default:
			check(false);
        }
    }
	

    //---------------------------------------------------------------------------------------------
    bool CodeRunner::RunCode_ConstantResource(const FScheduledOp& Item)
    {
		MUTABLE_CPUPROFILER_SCOPE(RunCode_Constant);
		
		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {
		case EOpType::ED_CONSTANT:
		{
        	MUTABLE_CPUPROFILER_SCOPE(ED_CONSTANT);
        		
			FOperation::ExternalDataConstantArgs Args = Program.GetOpArgs<FOperation::ExternalDataConstantArgs>(Item.At);

			// Assume the ROM has been loaded previously
			TManagedPtr<UE::Mutable::Private::FExtensionData> ExtensionData = MakeManaged<UE::Mutable::Private::FExtensionData>();
			ExtensionData->PassthroughObject = FindPassthrough<UObject>(Args.ExternalObjectId);

            StoreExtensionData(Item, ExtensionData);
            break;
		}

		case EOpType::MI_CONSTANT:
		{
        	MUTABLE_CPUPROFILER_SCOPE(MI_CONSTANT);
        		
			FOperation::MaterialConstantArgs Args = Program.GetOpArgs<FOperation::MaterialConstantArgs>(Item.At);

        	TManagedPtr<const FMaterial> SourceConst = Program.ConstantMaterials[Args.Value];
       		check(SourceConst);
        		
			TManagedPtr<FMaterial> Source = UE::Mutable::Private::CloneOrTakeOver<FMaterial>(MoveTemp(SourceConst));
        	Source->PassthroughObject = FindPassthrough<UMaterialInterface>(Args.ID);
        		
			StoreMaterial(Item, Source);
			break;
		}

        default:
            if (type!=EOpType::NONE)
            {
                // Operation not implemented
                check( false );
            }
            break;
        }

		// Success
		return true;
    }

	void CodeRunner::RunCode_Mesh(const FScheduledOp& Item)
    {
		MUTABLE_CPUPROFILER_SCOPE(RunCode_Mesh);

		check((EMeshContentFlags)Item.ExecutionOptions != EMeshContentFlags::None); 


		EOpType type = Program.GetOpType(Item.At);

        switch (type)
        {
        
        case EOpType::ME_APPLYLAYOUT:
        {
			FOperation::MeshApplyLayoutArgs Args = Program.GetOpArgs<FOperation::MeshApplyLayoutArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Mesh, Item));
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Mesh, Item),
						FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0));
				}
				break;
			}
            case 1:
            {
           		MUTABLE_CPUPROFILER_SCOPE(ME_APPLYLAYOUT)
            		
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Base = LoadMesh(FCacheAddress(Args.Mesh, Item));
					StoreMesh(Item, Base);
				}
				else
				{
					TManagedPtr<const FMesh> Base = LoadMesh(FCacheAddress(Args.Mesh, Item));
					TManagedPtr<const FLayout> Layout = LoadLayout(
							FCacheAddress(FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0)));

					if (Base)
					{
						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(Base));

						int32 texCoordsSet = Args.Channel;

						MeshApplyLayout(Result.Get(), Layout.Get(), texCoordsSet);
					
						StoreMesh(Item, Result);
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
				}

                break;
            }

            default:
                check(false);
            }

            break;
        }


		case EOpType::ME_PREPARELAYOUT:
		{
			FOperation::MeshPrepareLayoutArgs Args = Program.GetOpArgs<FOperation::MeshPrepareLayoutArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Mesh, Item));
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Mesh, Item),
						FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0));
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_PREPARELAYOUT);

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Base = LoadMesh(FCacheAddress(Args.Mesh, Item));
					StoreMesh(Item, Base);
				}
				else
				{
					TManagedPtr<const FMesh> Base = LoadMesh(FCacheAddress(Args.Mesh, Item));
					TManagedPtr<const FLayout> Layout = LoadLayout(
							FCacheAddress(FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0)));

					if (Base && Layout)
					{
						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(Base));

						MeshPrepareLayout(*Result, *Layout, Args.LayoutChannel, Args.bNormalizeUVs, Args.bClampUVIslands, Args.bEnsureAllVerticesHaveLayoutBlock, Args.bUseAbsoluteBlockIds);
					
						StoreMesh(Item, Result);
					}
					else
					{
						StoreMesh(Item, Base);
					}
				}

				break;
			}

			default:
				check(false);
			}

			break;
		}

        case EOpType::ME_DIFFERENCE:
        {
			const uint8* data = Program.GetOpArgsPointer(Item.At);

			FOperation::ADDRESS BaseAt = 0;
			FMemory::Memcpy(&BaseAt, data, sizeof(FOperation::ADDRESS)); 
			data += sizeof(FOperation::ADDRESS);

			FOperation::ADDRESS TargetAt = 0;
			FMemory::Memcpy(&TargetAt, data, sizeof(FOperation::ADDRESS)); 
			data += sizeof(FOperation::ADDRESS);

            switch (Item.Stage)
            {
            case 0:
			{
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
                	TManagedPtr<const FMesh> Result = CreateMesh();
					StoreMesh(Item, Result);
				}
				else
				{
					if (BaseAt && TargetAt)
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
								FScheduledOp(BaseAt, Item),
								FScheduledOp(TargetAt, Item));
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
				}
				break;
			}
            case 1:
            {
       	        MUTABLE_CPUPROFILER_SCOPE(ME_DIFFERENCE)

				check(EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))

				TManagedPtr<const FMesh> Base = LoadMesh(FCacheAddress(BaseAt, Item));
				TManagedPtr<const FMesh> pTarget = LoadMesh(FCacheAddress(TargetAt, Item));

				TArray<EMeshBufferSemantic, TInlineAllocator<8>> Semantics;
				TArray<int32, TInlineAllocator<8>> SemanticIndices;

				uint8 bIgnoreTextureCoords = 0;
				FMemory::Memcpy(&bIgnoreTextureCoords, data, sizeof(uint8)); 
				data += sizeof(uint8);

				uint8 NumChannels = 0;
				FMemory::Memcpy(&NumChannels, data, sizeof(uint8)); 
				data += sizeof(uint8);

				for (uint8 i = 0; i < NumChannels; ++i)
				{
					uint8 Semantic = 0;
					FMemory::Memcpy(&Semantic, data, sizeof(uint8)); 
					data += sizeof(uint8);
					
					uint8 SemanticIndex = 0;
					FMemory::Memcpy(&SemanticIndex, data, sizeof(uint8)); 
					data += sizeof(uint8);

					Semantics.Add(EMeshBufferSemantic(Semantic));
					SemanticIndices.Add(SemanticIndex);
				}

				TManagedPtr<FMesh> Result = CreateMesh();
				bool bOutSuccess = false;
				MeshDifference(Result.Get(), Base.Get(), pTarget.Get(),
							   NumChannels, Semantics.GetData(), SemanticIndices.GetData(),
							   bIgnoreTextureCoords != 0, bOutSuccess);

				StoreMesh(Item, Result);

				break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_MORPH:
        {
			const FOperation::MeshMorphArgs Args = Program.GetOpArgs<FOperation::MeshMorphArgs>(Item.At);

			switch (Item.Stage)
            {
            case 0:
			{
				if (Args.Base)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Base, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Base, Item),
							FScheduledOp(Args.Factor, Item));
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
                break;
			}

            case 1:
            {
       		    MUTABLE_CPUPROFILER_SCOPE(ME_MORPH_1)
				
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
                	StoreMesh(Item, LoadMesh(FCacheAddress(Args.Base, Item)));
				}
				else
				{
					TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.Base, Item));
					float Factor = LoadScalar(FCacheAddress(Args.Factor, Item));

					if (FMath::IsNearlyZero(Factor))
					{
						StoreMesh(Item, BaseMesh);
					}
					else 
					{
						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(BaseMesh));
						if (Result && Result->HasMorphs() && Program.ConstantStrings.IsValidIndex(Args.Name))
						{
							// Morph data from the Base Mesh
							FName Name(Program.ConstantStrings[Args.Name]);
							MeshMorph(Result.Get(), Name, Factor);
						
						}

						StoreMesh(Item, Result);
					}
				}

                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_MERGE:
        {
			FOperation::MeshMergeArgs Args = Program.GetOpArgs<FOperation::MeshMergeArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Base, Item),
					FScheduledOp(Args.Added, Item));
				break;
			}
            case 1:
            {
				MUTABLE_CPUPROFILER_SCOPE(ME_MERGE_1)

                TManagedPtr<const FMesh> pA = LoadMesh(FCacheAddress(Args.Base, Item));
                TManagedPtr<const FMesh> pB = LoadMesh(FCacheAddress(Args.Added, Item));

                if (pA && pB && pA->GetVertexCount() && pB->GetVertexCount())
                {
					check(!pA->IsReference() && !pB->IsReference());

					TManagedPtr<FMesh> Result = CreateMesh(pA->GetDataSize() + pB->GetDataSize());

					MeshMerge(Result.Get(), pA, pB, !Args.NewSurfaceID);

                    if (Args.NewSurfaceID)
                    {
						check(pB->GetSurfaceCount() == 1);
						Result->Surfaces.Last().Id = Args.NewSurfaceID;
                    }

					StoreMesh(Item, Result);
                }
                else if (pA && (pA->GetVertexCount() || pA->IsReference()))
                {
					StoreMesh(Item, pA);
                }
                else if (pB && (pB->GetVertexCount() || pB->IsReference()))
                {
					TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(pB));

                    check(Result->IsReference() || (Result->GetSurfaceCount() == 1) );

                    if (Result->GetSurfaceCount() > 0 && Args.NewSurfaceID)
                    {
                        Result->Surfaces.Last().Id = Args.NewSurfaceID;
                    }

					StoreMesh(Item, Result);
                }
                else
                {
					StoreMesh(Item, CreateMesh());
                }

                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_MASKCLIPMESH:
        {
			FOperation::MeshMaskClipMeshArgs Args = Program.GetOpArgs<FOperation::MeshMaskClipMeshArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.source, Item));
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.source, Item),
						FScheduledOp(Args.clip, Item));
				}
				break;
			}
            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(ME_MASKCLIPMESH_1)
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.source, Item));
					StoreMesh(Item, Source);
				}
				else
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.source, Item));
					TManagedPtr<const FMesh> pClip = LoadMesh(FCacheAddress(Args.clip, Item));

					// Only if both are valid.
					if (Source && pClip)
					{
						TManagedPtr<FMesh> Result = CreateMesh();
					
						bool bOutSuccess = false;
						MeshMaskClipMesh(Result.Get(), Source.Get(), pClip.Get(), bOutSuccess);
					
						if (!bOutSuccess)
						{
							StoreMesh(Item, nullptr);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
					
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

		case EOpType::ME_MASKCLIPUVMASK:
		{
			FOperation::MeshMaskClipUVMaskArgs Args = Program.GetOpArgs<FOperation::MeshMaskClipUVMaskArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Source, Item));
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Source, Item),
						FScheduledOp(Args.UVSource, Item),
						FScheduledOp::FromOpAndOptions(Args.MaskImage, Item, 0),
						FScheduledOp::FromOpAndOptions(Args.MaskLayout, Item, 0));
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_MASKCLIPUVMASK_1);

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					StoreMesh(Item, Source);
				}
				else
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					TManagedPtr<const FMesh> UVSource = LoadMesh(FCacheAddress(Args.UVSource, Item));
					TManagedPtr<const FImage> MaskImage = LoadImage(FCacheAddress(FScheduledOp::FromOpAndOptions( Args.MaskImage, Item, 0)));
					TManagedPtr<const FLayout> MaskLayout = LoadLayout(
							FCacheAddress(FScheduledOp::FromOpAndOptions(Args.MaskLayout, Item, 0)));

					// Only if both are valid.
					if (Source && MaskImage)
					{
						TManagedPtr<FMesh> Result = CreateMesh();

						bool bOutSuccess = false;
						MakeMeshMaskFromUVMask(Result.Get(), Source.Get(), UVSource.Get(), MaskImage.Get(), Args.LayoutIndex, bOutSuccess);
						
						if (!bOutSuccess)
						{
							StoreMesh(Item, nullptr);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else if (Source && MaskLayout)
					{					
						TManagedPtr<FMesh> Result = CreateMesh();

						bool bOutSuccess = false;
						MakeMeshMaskFromLayout(Result.Get(), Source.Get(), UVSource.Get(), MaskLayout.Get(), Args.LayoutIndex, bOutSuccess);

						if (!bOutSuccess)
						{
							StoreMesh(Item, nullptr);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
				}
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::ME_MASKDIFF:
		{
			FOperation::MeshMaskDiffArgs Args = Program.GetOpArgs<FOperation::MeshMaskDiffArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Source, Item));
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Source, Item),
						FScheduledOp(Args.Fragment, Item));
				}

				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_MASKDIFF_1)
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					StoreMesh(Item, Source);
				}
				else
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					TManagedPtr<const FMesh> pClip = LoadMesh(FCacheAddress(Args.Fragment, Item));

					// Only if both are valid.
					if (Source && pClip)
					{
						TManagedPtr<FMesh> Result = CreateMesh();

						bool bOutSuccess = false;
						MeshMaskDiff(Result.Get(), Source.Get(), pClip.Get(), bOutSuccess);
						
						if (!bOutSuccess)
						{
							StoreMesh(Item, nullptr);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_FORMAT:
        {
			FOperation::MeshFormatArgs Args = Program.GetOpArgs<FOperation::MeshFormatArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (Args.source && Args.format)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.source, Item),
							FScheduledOp(Args.format, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
            case 1:
            {
				MUTABLE_CPUPROFILER_SCOPE(ME_FORMAT_1)
				TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.source,Item));
				TManagedPtr<const FMesh> Format = LoadMesh(FCacheAddress(Args.format,Item));

				if (Source && Source->IsReference())
				{
					StoreMesh(Item, Source);
				}
				else if (Source)
				{
					uint8 Flags = Args.Flags;
					if (!Format && !(Flags & FOperation::MeshFormatArgs::ResetBufferIndices))
					{
						StoreMesh(Item, Source);
					}
					else if (!Format)
					{
						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(Source));

						if (Flags & FOperation::MeshFormatArgs::ResetBufferIndices)
						{
							Result->ResetBufferIndices();
						}

						StoreMesh(Item, Result);
					}
					else
					{
						TManagedPtr<FMesh> Result = CreateMesh();

						bool bOutSuccess = false;
						MeshFormat(Result.Get(), Source.Get(), Format.Get(),
							true,
							(Flags & FOperation::MeshFormatArgs::Vertex) != 0,
							(Flags & FOperation::MeshFormatArgs::Index) != 0,
							(Flags & FOperation::MeshFormatArgs::IgnoreMissing) != 0,
							bOutSuccess);
						check(bOutSuccess);

						if (Flags & FOperation::MeshFormatArgs::ResetBufferIndices)
						{
							Result->ResetBufferIndices();
						}

						if (Flags & FOperation::MeshFormatArgs::OptimizeBuffers)
						{
							MUTABLE_CPUPROFILER_SCOPE(MeshOptimizeBuffers)
							MeshOptimizeBuffers(Result.Get());
						}

						StoreMesh(Item, Result);
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_EXTRACTLAYOUTBLOCK:
        {
            const uint8* Data = Program.GetOpArgsPointer(Item.At);

            FOperation::ADDRESS SourceAddress;
            FMemory::Memcpy(&SourceAddress, Data, sizeof(FOperation::ADDRESS));
            Data += sizeof(FOperation::ADDRESS);

            uint16 LayoutIndex;
			FMemory::Memcpy(&LayoutIndex, Data, sizeof(uint16));
            Data += sizeof(uint16);

            uint16 BlockCount;
			FMemory::Memcpy(&BlockCount, Data, sizeof(uint16));
            Data += sizeof(uint16);

            switch (Item.Stage)
            {
            case 0:
			{
				if (SourceAddress)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(SourceAddress, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_EXTRACTLAYOUTBLOCK_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> SourceMesh = LoadMesh(FCacheAddress(SourceAddress, Item));
					StoreMesh(Item, SourceMesh);
				}
				else
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(SourceAddress, Item));

					// Access with memcpy necessary for unaligned memory access issues.
					uint64 Blocks[512];
					check(BlockCount < 512);

					BlockCount = FMath::Min(512, int32(BlockCount));
					FMemory::Memcpy(Blocks, Data, sizeof(uint64)*BlockCount);

					if (Source)
					{
						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(Source));

						if (BlockCount > 0)
						{
							TArrayView<uint64> BlockIds(Blocks, BlockCount);
							MeshExtractLayoutBlock(Result.Get(), LayoutIndex, BlockIds);
						}
						else
						{
							MeshExtractLayoutBlock(Result.Get(), LayoutIndex);
						}

						StoreMesh(Item, Result);
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_TRANSFORM:
        {
			FOperation::MeshTransformArgs Args = Program.GetOpArgs<FOperation::MeshTransformArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (Args.source)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.source, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_TRANSFORM_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
                	TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.source, Item));
					StoreMesh(Item, Source);
				}
				else
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.source,Item));

					const FMatrix44f& mat = Program.ConstantMatrices[Args.matrix];

					TManagedPtr<FMesh> Result = CreateMesh(Source ? Source->GetDataSize() : 0);

					bool bOutSuccess = false;
					MeshTransform(Result.Get(), Source.Get(), mat, bOutSuccess);

					if (!bOutSuccess)
					{
						StoreMesh(Item, Source);
					}
					else
					{
						StoreMesh(Item, Result);
					}
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_CLIPMORPHPLANE:
        {
			FOperation::MeshClipMorphPlaneArgs Args = Program.GetOpArgs<FOperation::MeshClipMorphPlaneArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (Args.Source)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Source, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
            case 1:
            {
           		MUTABLE_CPUPROFILER_SCOPE(ME_CLIPMORPHPLANE_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
                	TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					StoreMesh(Item, Source);
				}
				else
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					
					check(Args.MorphShape < (uint32)Program.ConstantShapes.Num());

					// Should be an ellipse
					const FShape& MorphShape = Program.ConstantShapes[Args.MorphShape];

					const FVector3f& Origin = MorphShape.position;
					const FVector3f& Normal = MorphShape.up;

					bool bRemoveFaceIfAllVerticesCulled = Args.FaceCullStrategy==EFaceCullStrategy::AllVerticesCulled;

					if (Args.VertexSelectionType == EClipVertexSelectionType::Shape)
					{
						check(Args.VertexSelectionShapeOrBone < (uint32)Program.ConstantShapes.Num());

						// Should be None or an axis aligned box
						const FShape& SelectionShape = Program.ConstantShapes[Args.VertexSelectionShapeOrBone];

						TManagedPtr<FMesh> Result = CreateMesh(Source ? Source->GetDataSize() : 0);
						
						bool bOutSuccess = false;
						MeshClipMorphPlane(Result.Get(), Source.Get(), Origin, Normal, Args.Dist, Args.Factor, MorphShape.size[0], MorphShape.size[1], MorphShape.size[2], SelectionShape, bRemoveFaceIfAllVerticesCulled, bOutSuccess, NAME_None, -1);
						if (!bOutSuccess)
						{
							StoreMesh(Item, Source);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}

					else if (Args.VertexSelectionType == EClipVertexSelectionType::BoneHierarchy)
					{
						FShape SelectionShape;
						SelectionShape.type = (uint8)FShape::Type::None;

						TManagedPtr<FMesh> Result = CreateMesh(Source ? Source->GetDataSize() : 0);
						
						check(Args.VertexSelectionShapeOrBone <= MAX_uint32);
						FName BoneName = Program.ConstantNames[Args.VertexSelectionShapeOrBone];

						bool bOutSuccess = false;
						MeshClipMorphPlane(Result.Get(), Source.Get(), Origin, Normal, Args.Dist, Args.Factor, MorphShape.size[0], MorphShape.size[1], MorphShape.size[2], SelectionShape, bRemoveFaceIfAllVerticesCulled, bOutSuccess, BoneName, Args.MaxBoneRadius);

						if (!bOutSuccess)
						{
							StoreMesh(Item, Source);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						// No vertex selection
						FShape SelectionShape;
						SelectionShape.type = (uint8)FShape::Type::None;

						TManagedPtr<FMesh> Result = CreateMesh(Source ? Source->GetDataSize() : 0);
						
						bool bOutSuccess = false;
						MeshClipMorphPlane(Result.Get(), Source.Get(), Origin, Normal, Args.Dist, Args.Factor, MorphShape.size[0], MorphShape.size[1], MorphShape.size[2], SelectionShape, bRemoveFaceIfAllVerticesCulled, bOutSuccess, NAME_None, -1.0f);
						
						if (!bOutSuccess)
						{
							StoreMesh(Item, Source);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }


        case EOpType::ME_CLIPWITHMESH:
        {
			FOperation::MeshClipWithMeshArgs Args = Program.GetOpArgs<FOperation::MeshClipWithMeshArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (Args.Source)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Source, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Source, Item),
							FScheduledOp(Args.ClipMesh, Item));
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
            case 1:
            {
				MUTABLE_CPUPROFILER_SCOPE(ME_CLIPWITHMESH_1)
				
				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					StoreMesh(Item, Source);
				}
				else
				{

					TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(Args.Source, Item));
					TManagedPtr<const FMesh> Clip = LoadMesh(FCacheAddress(Args.ClipMesh, Item));

					if (Source && Clip)
					{
						TManagedPtr<FMesh> Result = CreateMesh(Source->GetDataSize());

						const bool bRemoveAllVerticesCulled = Args.FaceCullStrategy == EFaceCullStrategy::AllVerticesCulled;
						bool bOutSuccess = false;
						MeshClipWithMesh(Result.Get(), Source.Get(), Clip.Get(), bRemoveAllVerticesCulled, bOutSuccess);

						if (!bOutSuccess)
						{
							StoreMesh(Item, Source);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, Source);
					}
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }
		case EOpType::ME_CLIPDEFORM:
		{
			FOperation::MeshClipDeformArgs Args = Program.GetOpArgs<FOperation::MeshClipDeformArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.mesh)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.mesh, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.mesh, Item),
							FScheduledOp(Args.clipShape, Item));
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_CLIPDEFORM_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.mesh, Item));
					StoreMesh(Item, BaseMesh);
				}
				else
				{
					TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.mesh, Item));
					TManagedPtr<const FMesh> ClipShape = LoadMesh(FCacheAddress(Args.clipShape, Item));

					if (BaseMesh && ClipShape)
					{
						TManagedPtr<FMesh> Result = CreateMesh(BaseMesh->GetDataSize());

						bool bRemoveIfAllVerticesCulled = Args.FaceCullStrategy == EFaceCullStrategy::AllVerticesCulled;

						bool bOutSuccess = false;
						MeshClipDeform(Result.Get(), BaseMesh.Get(), ClipShape.Get(), Args.clipWeightThreshold, bRemoveIfAllVerticesCulled, bOutSuccess);
						if (!bOutSuccess)
						{
							StoreMesh(Item, BaseMesh);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, BaseMesh);
					}
				}
				break;
			}

			default:
				check(false);
			}

			break;
		}

        case EOpType::ME_APPLYPOSE:
        {
			FOperation::MeshApplyPoseArgs Args = Program.GetOpArgs<FOperation::MeshApplyPoseArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (Args.base)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.base, Item),
						FScheduledOp(Args.pose, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
            case 1:
            {
          		MUTABLE_CPUPROFILER_SCOPE(ME_APPLYPOSE_1)

                TManagedPtr<const FMesh> Base = LoadMesh(FCacheAddress(Args.base, Item));
                TManagedPtr<const FMesh> Pose = LoadMesh(FCacheAddress(Args.pose, Item));

                // Only if both are valid.
                if (Base && Pose)
				{
					TManagedPtr<FMesh> Result = CreateMesh(Base->GetSkeleton() ? Base->GetDataSize() : 0);

					bool bOutSuccess = false;
					MeshApplyPose(Result.Get(), Base.Get(), Pose.Get(), bOutSuccess);
					
					if (!bOutSuccess)
					{
						StoreMesh(Item, Base);
					}
					else
					{
						StoreMesh(Item, Result);
					}
                }
				else
				{
					StoreMesh(Item, Base);
				}

                break;
            }

            default:
                check(false);
            }

            break;
        }

		case EOpType::ME_BINDSHAPE:
		{
			FOperation::MeshBindShapeArgs Args = Program.GetOpArgs<FOperation::MeshBindShapeArgs>(Item.At);
			const uint8* Data = Program.GetOpArgsPointer(Item.At);

			constexpr uint8 ShapeContentFilter = (uint8)(EMeshContentFlags::GeometryData | EMeshContentFlags::PoseData);
			const EShapeBindingMethod BindingMethod = static_cast<EShapeBindingMethod>(Args.bindingMethod); 

			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.mesh)
				{
					if (BindingMethod == EShapeBindingMethod::ReshapeClosestProject)
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.mesh, Item),
							FScheduledOp::FromOpAndOptions(Args.shape, Item, ShapeContentFilter));
					}
					else
					{
						if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
						{
							AddOp(FScheduledOp(Item.At, Item, 1),
								FScheduledOp(Args.mesh, Item));
						}
						else
						{
							AddOp(FScheduledOp(Item.At, Item, 1),
								FScheduledOp(Args.mesh, Item),
								FScheduledOp::FromOpAndOptions(Args.shape, Item, ShapeContentFilter));
						}
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_BINDSHAPE_1)
				
				if (BindingMethod == EShapeBindingMethod::ReshapeClosestProject)
				{
					TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.mesh, Item));
					TManagedPtr<const FMesh> Shape = LoadMesh(FScheduledOp::FromOpAndOptions(Args.shape, Item, ShapeContentFilter));
					// Bones are stored after the Args
					Data += sizeof(Args);

					// Rebuilding array of bone names ----
					TArray<FName> BonesToDeform;
					TArray<FName> PhysicsToDeform;

					int32 NumBones;
					FMemory::Memcpy(&NumBones, Data, sizeof(int32)); 
					Data += sizeof(int32);
					
					TArray<uint32> BoneIds;
					if (NumBones > 0)
					{
						BoneIds.SetNumUninitialized(NumBones);
						FMemory::Memcpy(BoneIds.GetData(), Data, NumBones * sizeof(uint32));
						Data += NumBones * sizeof(uint32);

						BonesToDeform.Reserve(NumBones);
						for (uint32 BoneId : BoneIds)
						{
							BonesToDeform.Add(Program.ConstantNames[BoneId]);
						}
					}

					int32 NumPhysicsBodies;
					FMemory::Memcpy(&NumPhysicsBodies, Data, sizeof(int32));
					Data += sizeof(int32);

					if (NumPhysicsBodies > 0)
					{
						BoneIds.SetNumUninitialized(NumPhysicsBodies, EAllowShrinking::No);
						FMemory::Memcpy(BoneIds.GetData(), Data, NumPhysicsBodies * sizeof(uint32));
						Data += NumPhysicsBodies * sizeof(uint32);

						PhysicsToDeform.Reserve(NumPhysicsBodies);
						for (uint32 PhysicsBoneId : BoneIds)
						{
							PhysicsToDeform.Add(Program.ConstantNames[PhysicsBoneId]);
						}
					}

					EMeshBindShapeFlags BindFlags = static_cast<EMeshBindShapeFlags>(Args.flags);
					EMeshContentFlags MeshContentFilter = (EMeshContentFlags)Item.ExecutionOptions;

					if (!EnumHasAnyFlags(MeshContentFilter, EMeshContentFlags::GeometryData))
					{
						EnumRemoveFlags(BindFlags, 
								EMeshBindShapeFlags::EnableRigidParts |
								EMeshBindShapeFlags::ReshapeVertices  |
								EMeshBindShapeFlags::ApplyLaplacian   |
								EMeshBindShapeFlags::RecomputeNormals);
					}

					if (!EnumHasAnyFlags(MeshContentFilter, EMeshContentFlags::PhysicsData))
					{
						EnumRemoveFlags(BindFlags, EMeshBindShapeFlags::ReshapePhysicsVolumes);
					}

					if (!EnumHasAnyFlags(MeshContentFilter, EMeshContentFlags::PoseData))
					{
						EnumRemoveFlags(BindFlags, EMeshBindShapeFlags::ReshapeSkeleton);
					}

					FMeshBindColorChannelUsages ColorChannelUsages;
					FMemory::Memcpy(&ColorChannelUsages, &Args.ColorUsage, sizeof(ColorChannelUsages));
					static_assert(sizeof(ColorChannelUsages) == sizeof(Args.ColorUsage));

					if (Shape)
					{
						TNotNull<const FMesh*> BaseShapeMesh[1] = { TNotNull<const FMesh*>(Shape.Get()) };
						GeometryUtils::FMeshGeometry BaseShapeGeometry;

						constexpr bool bNeedsNormals = true;

						GeometryUtils::GetMergedGeometryFromMeshes(BaseShapeMesh, bNeedsNormals, BaseShapeGeometry);
						GeometryUtils::FMeshAdapter MeshAdapter(BaseShapeGeometry);

						constexpr bool bAutoBuild = false;
						UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter> BaseShapeAABBTree(&MeshAdapter, bAutoBuild);	
						{
							MUTABLE_CPUPROFILER_SCOPE(BuildBaseShapeAABBTree);
							BaseShapeAABBTree.Build();
						}

						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(BaseMesh));

						bool bOutSuccess = false;
						MeshBindShapeReshape(Result.Get(), BaseShapeAABBTree, BonesToDeform, PhysicsToDeform, BindFlags, ColorChannelUsages, bOutSuccess);

						StoreMesh(Item, Result);
					}
					else 
					{
						StoreMesh(Item, BaseMesh);
					}
				}	
				else
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.mesh, Item));
						StoreMesh(Item, BaseMesh);
					}
					else
					{
						TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.mesh, Item));

						TManagedPtr<const FMesh> Shape = 
								LoadMesh(FScheduledOp::FromOpAndOptions(Args.shape, Item, ShapeContentFilter));

						if (Shape && BaseMesh)
						{
							TNotNull<const FMesh*> BaseShapeMesh[1] = { TNotNull<const FMesh*>(Shape.Get()) };
							GeometryUtils::FMeshGeometry BaseShapeGeometry;

							constexpr bool bNeedsNormals = true;
							GeometryUtils::GetMergedGeometryFromMeshes(BaseShapeMesh, bNeedsNormals, BaseShapeGeometry);
							GeometryUtils::FMeshAdapter MeshAdapter(BaseShapeGeometry);

							constexpr bool bAutoBuild = false;
							UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter> ShapeAABBTree(&MeshAdapter, bAutoBuild);	
							{
								MUTABLE_CPUPROFILER_SCOPE(BuildBaseShapeAABBTree);
								ShapeAABBTree.Build();
							}

							TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(BaseMesh));

							bool bOutSuccess = false;
							MeshBindShapeClipDeform(Result.Get(), ShapeAABBTree, BindingMethod, bOutSuccess);

							StoreMesh(Item, Result);
						}
						else
						{
							StoreMesh(Item, BaseMesh);
						}
					}
				}

				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::ME_APPLYSHAPE:
		{
			FOperation::MeshApplyShapeArgs Args = Program.GetOpArgs<FOperation::MeshApplyShapeArgs>(Item.At);

			EMeshBindShapeFlags ReshapeFlags = static_cast<EMeshBindShapeFlags>(Args.flags);

			bool bNeedsShapeGeometry = EnumHasAnyFlags(ReshapeFlags, EMeshBindShapeFlags::ReshapeSkeleton | EMeshBindShapeFlags::ReshapePhysicsVolumes);

			EMeshContentFlags ShapeContentFilter = EMeshContentFlags(Item.ExecutionOptions);
			if (bNeedsShapeGeometry)
			{
				EnumAddFlags(ShapeContentFilter, EMeshContentFlags::GeometryData); 
			}

			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.mesh)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.mesh, Item),
						FScheduledOp::FromOpAndOptions(Args.shape, Item, (uint8)ShapeContentFilter));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_APPLYSHAPE_1)
					
				TManagedPtr<const FMesh> BaseMesh = LoadMesh(FCacheAddress(Args.mesh, Item));
				TManagedPtr<const FMesh> Shape = LoadMesh(FScheduledOp::FromOpAndOptions(Args.shape, Item, (uint8)ShapeContentFilter));

				const EMeshContentFlags MeshContentFilter = (EMeshContentFlags)Item.ExecutionOptions;

				if (!EnumHasAnyFlags(MeshContentFilter, EMeshContentFlags::GeometryData))
				{
					EnumRemoveFlags(ReshapeFlags, 
							EMeshBindShapeFlags::EnableRigidParts |
							EMeshBindShapeFlags::ReshapeVertices  |
							EMeshBindShapeFlags::ApplyLaplacian   |
							EMeshBindShapeFlags::RecomputeNormals);
				}

				if (!EnumHasAnyFlags(MeshContentFilter, EMeshContentFlags::PhysicsData))
				{
					EnumRemoveFlags(ReshapeFlags, EMeshBindShapeFlags::ReshapePhysicsVolumes);
				}

				if (!EnumHasAnyFlags(MeshContentFilter, EMeshContentFlags::PoseData))
				{
					EnumRemoveFlags(ReshapeFlags, EMeshBindShapeFlags::ReshapeSkeleton);
				}

				const bool bReshapeVertices = EnumHasAnyFlags(ReshapeFlags, EMeshBindShapeFlags::ReshapeVertices);


				if (Shape && !Shape->GetVertexBuffers().IsDescriptor())
				{
					TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(BaseMesh));

					TNotNull<const FMesh*> ShapeMesh[1] = { TNotNull<const FMesh*>(Shape.Get()) };
					GeometryUtils::FMeshGeometry ShapeGeometry;

					constexpr bool bNeedsNormals = true;
					GeometryUtils::GetMergedGeometryFromMeshes(ShapeMesh, bNeedsNormals, ShapeGeometry);

					bool bOutSuccess = false;
					MeshApplyShape(Result.Get(), ShapeGeometry, ReshapeFlags, bOutSuccess, GetLogger());
					
					StoreMesh(Item, Result);
				}
				else
				{
					StoreMesh(Item, BaseMesh);
				}
				
				break;
			}

			default:
				check(false);
			}

			break;
		}

        case EOpType::ME_REMOVEMASK:
        {
       		MUTABLE_CPUPROFILER_SCOPE(ME_REMOVEMASK)
        		
            // Decode op
            // TODO: Partial decode for each stage
            const uint8* Data = Program.GetOpArgsPointer(Item.At);

            FOperation::ADDRESS SourceAddress;
            FMemory::Memcpy(&SourceAddress, Data, sizeof(FOperation::ADDRESS)); 
			Data += sizeof(FOperation::ADDRESS);

			EFaceCullStrategy FaceCullStrategy;
			FMemory::Memcpy(&FaceCullStrategy, Data, sizeof(EFaceCullStrategy));
			Data += sizeof(EFaceCullStrategy);

            TArray<FScheduledOp, TInlineAllocator<4>> Conditions;
			TArray<FOperation::ADDRESS, TInlineAllocator<4>> Masks;

            uint16 NumRemoves;
			FMemory::Memcpy(&NumRemoves, Data, sizeof(uint16)); 
			Data += sizeof(uint16);

            for(uint16 RemoveIndex = 0; RemoveIndex < NumRemoves; ++RemoveIndex)
            {
                FOperation::ADDRESS Condition;
				FMemory::Memcpy(&Condition, Data, sizeof(FOperation::ADDRESS)); 
				Data += sizeof(FOperation::ADDRESS);
                
				Conditions.Emplace(Condition, Item);

                FOperation::ADDRESS Mask;
				FMemory::Memcpy(&Mask, Data, sizeof(FOperation::ADDRESS)); 
				Data += sizeof(FOperation::ADDRESS);

                Masks.Add(Mask);
            }


            // Schedule next stages
            switch (Item.Stage)
            {
            case 0:
			{
				if (SourceAddress)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1), 
								FScheduledOp(SourceAddress, Item));
					}
					else
					{
						// Request the conditions
						AddOp(FScheduledOp(Item.At, Item, 1), Conditions);
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
            case 1:
            {
        		MUTABLE_CPUPROFILER_SCOPE(ME_REMOVEMASK_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
                {
                	TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(SourceAddress, Item));
					StoreMesh(Item, Source);
				}
				else
				{
					// Request the source and the necessary masks
					// \todo: store condition values in heap?
					TArray<FScheduledOp, TInlineAllocator<4>> Deps;
					Deps.Reserve(Conditions.Num());

					Deps.Emplace(SourceAddress, Item);
					for( int32 RemoveIndex = 0; RemoveIndex < Conditions.Num(); ++RemoveIndex)
					{
						// If there is no expression, we'll assume true.
						bool bValue = true;
						if (Conditions[RemoveIndex].At)
						{
							bool bSuccess = GetMemory().GetBoolIfSet(FCacheAddress(Conditions[RemoveIndex].At, Item), bValue);
							check(bSuccess);
						}

						if (bValue)
						{
							Deps.Emplace(Masks[RemoveIndex], Item);
						}
					}

					if (SourceAddress)
					{
						AddOp(FScheduledOp(Item.At, Item, 2), Deps);
					}
				}
                break;
            }

            case 2:
            {
            	MUTABLE_CPUPROFILER_SCOPE(ME_REMOVEMASK_2)
            	
                // \todo: single remove operation with all masks?
                TManagedPtr<const FMesh> Source = LoadMesh(FCacheAddress(SourceAddress, Item));

				if (Source)
				{
					TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(Source));

					for (int32 RemoveIndex = 0; RemoveIndex < Conditions.Num(); ++RemoveIndex)
					{
						// If there is no expression, we'll assume true.
						bool bValue = true;
						if (Conditions[RemoveIndex].At)
						{
							bValue = LoadBool(FCacheAddress(Conditions[RemoveIndex].At, Item));
						}

						if (bValue)
						{
							TManagedPtr<const FMesh> Mask = LoadMesh(FCacheAddress(Masks[RemoveIndex], Item));
							if (Mask)
							{
								bool bRemoveIfAllVerticesCulled = FaceCullStrategy == EFaceCullStrategy::AllVerticesCulled;
								MeshRemoveMaskInline(Result.Get(), Mask.Get(), bRemoveIfAllVerticesCulled);
							}
						}
					}

					StoreMesh(Item, Result);
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_ADDMETADATA:
		{
			MUTABLE_CPUPROFILER_SCOPE(ME_ADDMETADATA)

			const FOperation::MeshAddMetadataArgs OpArgs = Program.GetOpArgs<FOperation::MeshAddMetadataArgs>(Item.At);

			// Schedule next stages
			switch (Item.Stage)
			{
			case 0:
			{
				if (OpArgs.Source)
				{
					AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(OpArgs.Source, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}

			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_ADDMETADATA_2)

				TManagedPtr<const FMesh> SourceMesh = LoadMesh(FCacheAddress(OpArgs.Source, Item));
				TManagedPtr<FMesh> Result = nullptr;

				if (SourceMesh)
				{
					Result = CloneOrTakeOver(MoveTemp(SourceMesh));

					if (OpArgs.SkeletonId != PASSTHROUGH_ID_INVALID)
					{
						Result->SkeletonObjects.Add(FindPassthrough<USkeleton>(OpArgs.SkeletonId));
					}

					using OpEnumFlags = FOperation::MeshAddMetadataArgs::EnumFlags;

					// Append Tags
					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsGameplayTagList))
					{
						const TArray<uint32>& ConstantGameplayTagList = Program.ConstantUInt32Lists[OpArgs.GameplayTags.ListIndex];
						Result->GameplayTags.Reserve(Result->GameplayTags.Num() + ConstantGameplayTagList.Num());

						for (uint32 Index : ConstantGameplayTagList)
						{
							Result->GameplayTags.Add(Program.ConstantNames[Index]);
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasGameplayTags))
					{
						Result->GameplayTags.Add(Program.ConstantNames[OpArgs.GameplayTags.NameIndex]);
					}
					
					// Asset User Data
					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsAssetUserDataList))
					{
						for (uint32 PassthroughId : Program.ConstantUInt32Lists[OpArgs.AssetUserDataIds.ListAddress])
						{
							check(PassthroughId != PASSTHROUGH_ID_INVALID);
							Result->AssetUserData.Emplace(FindPassthrough<UAssetUserData>(PassthroughId));
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasAssetUserData))
					{
						check(OpArgs.AssetUserDataIds.PassthroughId != PASSTHROUGH_ID_INVALID);
						Result->AssetUserData.Add(FindPassthrough<UAssetUserData>(OpArgs.AssetUserDataIds.PassthroughId));
					}
					
					// Animation Slots
					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsAnimationSlotList))
					{
						const TArray<uint32>& AnimSlotNames = Program.ConstantUInt32Lists[OpArgs.AnimSlotNames.ListIndex];
						const TArray<uint32>& AnimInstances = Program.ConstantUInt32Lists[OpArgs.AnimInstances.ListAddress];
						
						for (int32 Index = 0; Index < AnimSlotNames.Num(); ++Index)
						{
							const FName& AnimationSlotName = Program.ConstantNames[AnimSlotNames[Index]];

							PASSTHROUGH_ID AnimInstanceId = AnimInstances[Index];
							check(AnimInstanceId != PASSTHROUGH_ID_INVALID);
							TPassthroughObjectPtr<UClass> AnimInstance = FindPassthrough<UClass>(AnimInstanceId);
							
							Result->AnimationSlots.Emplace(AnimationSlotName, AnimInstance);
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasAnimationSlots))
					{
						check(OpArgs.AnimInstances.PassthroughId != PASSTHROUGH_ID_INVALID);

						const FName& AnimationSlotName = Program.ConstantNames[OpArgs.AnimSlotNames.NameIndex];
						TPassthroughObjectPtr<UClass> AnimInstance = FindPassthrough<UClass>(OpArgs.AnimInstances.PassthroughId);
						Result->AnimationSlots.Emplace(AnimationSlotName, AnimInstance);
					}
					
					// Pose Priority
					if (OpArgs.BonePosePriority != 0)
					{
						for (FMesh::FBonePose& BonePose : Result->BonePoses)
						{
							BonePose.BonePosePriority = OpArgs.BonePosePriority;
						}
					}

					// Sockets
					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsSocketsList) &&
						Program.ConstantUInt32Lists.IsValidIndex(OpArgs.Sockets.ListAddress))
					{
						const TArray<uint32>& ConstantSocketsList = Program.ConstantUInt32Lists[OpArgs.Sockets.ListAddress];
						Result->Sockets.Reserve(ConstantSocketsList.Num());

						for (uint32 SocketId : ConstantSocketsList)
						{
							Result->Sockets.Emplace(Program.ConstantSockets.FindChecked(SocketId));
						}
					}
					else if(EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasSockets))
					{
						Result->Sockets.Emplace(Program.ConstantSockets.FindChecked(OpArgs.Sockets.SocketId));
					}

					if (OpArgs.SocketPriority != 0)
					{
						for (FMeshSocket& Socket : Result->Sockets)
						{
							Socket.Priority = OpArgs.SocketPriority;
						}
					}

					if (OpArgs.PhysicsAssetId != PASSTHROUGH_ID_INVALID)
					{
						Result->PhysicsAssets.Emplace(FindPassthrough<UPhysicsAsset>(OpArgs.PhysicsAssetId));
					}

					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsAdditionalPhysicsAssetList) &&
						Program.ConstantUInt32Lists.IsValidIndex(OpArgs.AdditionalPhysicsAssetsIds.ListAddress))
					{
						const TArray<uint32>& ConstantPassthroughIdList = Program.ConstantUInt32Lists[OpArgs.AdditionalPhysicsAssetsIds.ListAddress];
						Result->AdditionalPhysicsAssets.Reserve(Result->AdditionalPhysicsAssets.Num() + ConstantPassthroughIdList.Num());

						for (uint32 PassthroughId : ConstantPassthroughIdList)
						{
							check(PassthroughId != PASSTHROUGH_ID_INVALID);
							Result->AdditionalPhysicsAssets.Emplace(FindPassthrough<UPhysicsAsset>(PassthroughId));
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasAdditionalPhysicsAsset))
					{
						check(OpArgs.AdditionalPhysicsAssetsIds.PassthroughId != PASSTHROUGH_ID_INVALID);
						Result->AdditionalPhysicsAssets.Emplace(FindPassthrough<UPhysicsAsset>(OpArgs.AdditionalPhysicsAssetsIds.PassthroughId));
					}

					check(Result->Morph.Names.Num() == Result->Morph.UsageFlagsPerMorph.Num());
					
					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsRealTimeMorphNamesList) &&
						Program.ConstantUInt32Lists.IsValidIndex(OpArgs.RealTimeMorphNames.ListAddress))
					{
						const TArray<uint32>& ConstantRealTimeMorphList = Program.ConstantUInt32Lists[OpArgs.RealTimeMorphNames.ListAddress];

						for (uint32 MorphNameIndex : ConstantRealTimeMorphList)
						{
							if (ensureAlways(Program.ConstantStrings.IsValidIndex(MorphNameIndex)))
							{
								FName RealTimeMorphName = FName(Program.ConstantStrings[MorphNameIndex]);

								int32 FoundIndex = Result->Morph.Names.IndexOfByKey(RealTimeMorphName);
								if (FoundIndex != INDEX_NONE)
								{
									check(Result->Morph.UsageFlagsPerMorph.IsValidIndex(FoundIndex));
									EnumAddFlags(Result->Morph.UsageFlagsPerMorph[FoundIndex], EMorphUsageFlags::RealTime);
								}
							}
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasRealTimeMorphNames) &&
						Program.ConstantStrings.IsValidIndex(OpArgs.RealTimeMorphNames.StringAddress))
					{
						FName RealTimeMorphName = FName(Program.ConstantStrings[OpArgs.RealTimeMorphNames.StringAddress]);
						int32 FoundIndex = Result->Morph.Names.IndexOfByKey(RealTimeMorphName);

						if (FoundIndex != INDEX_NONE)
						{
							check(Result->Morph.UsageFlagsPerMorph.IsValidIndex(FoundIndex));
							EnumAddFlags(Result->Morph.UsageFlagsPerMorph[FoundIndex], EMorphUsageFlags::RealTime);
						}
					}
				}

				StoreMesh(Item, Result);
				break;
			}
			default:
				check(false);
			}

			break;
		}

		case EOpType::ME_SETMATERIALSLOTID:
		{
			const FOperation::MeshSetMaterialSlotIdArgs Args = Program.GetOpArgs<FOperation::MeshSetMaterialSlotIdArgs>(Item.At);

			// Schedule next stages
			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.Mesh)
				{
					AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(Args.Mesh, Item));
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}

			case 1:
			{
				TManagedPtr<const FMesh> Mesh = LoadMesh(FCacheAddress(Args.Mesh, Item));

				if (Mesh && Mesh->GetVertexCount() > 0)
				{
					TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(Mesh));

					Result->EnsureSurfaceData();
					Result->Surfaces.Last().Id = Args.MaterialSlotId;

					StoreMesh(Item, Result);
				}
				else
				{
					StoreMesh(Item, nullptr);
				}

				break;
			}
			default:
				check(false);
			}
			break;
		}

        case EOpType::ME_PROJECT:
        {
			FOperation::MeshProjectArgs Args = Program.GetOpArgs<FOperation::MeshProjectArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				if (Args.Mesh)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Mesh, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Mesh, Item),
							FScheduledOp(Args.Projector, Item));
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_PROJECT_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> Mesh = LoadMesh(FCacheAddress(Args.Mesh, Item));
					StoreMesh(Item, Mesh);
				}
				else
				{
					TManagedPtr<const FMesh> Mesh = LoadMesh(FCacheAddress(Args.Mesh, Item));
					const FProjector Projector = LoadProjector(FCacheAddress(Args.Projector, Item));

					if (Mesh && Mesh->GetVertexBuffers().GetBufferCount() > 0)
					{
						TManagedPtr<FMesh> Result = CreateMesh();

						bool bOutSuccess = false;
						MeshProject(Result.Get(), Mesh.Get(), Projector, Args.LayoutIndex, bOutSuccess);
						if (!bOutSuccess)
						{
							StoreMesh(Item, Mesh);
						}
						else
						{
							// Result->GetPrivate()->CheckIntegrity();
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, Mesh);
					}
				}
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::ME_TRANSFORMWITHMESH:
		{
			FOperation::MeshTransformWithinMeshArgs Args = Program.GetOpArgs<FOperation::MeshTransformWithinMeshArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.sourceMesh)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.sourceMesh, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.sourceMesh, Item),
							FScheduledOp(Args.boundingMesh, Item),
							FScheduledOp(Args.matrix, Item));
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_TRANSFORMWITHMESH_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> SourceMesh = LoadMesh(FCacheAddress(Args.sourceMesh, Item));
					StoreMesh(Item, SourceMesh);
				}
				else
				{
					TManagedPtr<const FMesh> SourceMesh = LoadMesh(FCacheAddress(Args.sourceMesh,Item));
					TManagedPtr<const FMesh> BoundingMesh = LoadMesh(FCacheAddress(Args.boundingMesh, Item));
					const FMatrix44f& Transform = LoadMatrix(FCacheAddress(Args.matrix, Item));

					if (SourceMesh)
					{
						TManagedPtr<FMesh> Result = CreateMesh(SourceMesh->GetDataSize());

						bool bOutSuccess = false;
						MeshTransformWithMesh(Result.Get(), SourceMesh.Get(), BoundingMesh.Get(), Transform, bOutSuccess);
						if (!bOutSuccess)
						{
							StoreMesh(Item, SourceMesh);
						}
						else
						{
							StoreMesh(Item, Result);
						}
					}
					else
					{
						StoreMesh(Item, SourceMesh);
					}
				}
				break;
			}

			default:
				check(false);
			}
			break;
		}

		case EOpType::ME_TRANSFORMWITHBONE:
		{
			FOperation::MeshTransformWithBoneArgs Args = Program.GetOpArgs<FOperation::MeshTransformWithBoneArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.SourceMesh)
				{
					if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.SourceMesh, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.SourceMesh, Item),
							FScheduledOp(Args.Matrix, Item));
					}
				}
				else
				{
					StoreMesh(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(ME_TRANSFORMWITHBONE_1)

				if (!EnumHasAnyFlags(EMeshContentFlags(Item.ExecutionOptions), EMeshContentFlags::GeometryData))
				{
					TManagedPtr<const FMesh> SourceMesh = LoadMesh(FCacheAddress(Args.SourceMesh, Item));
					StoreMesh(Item, SourceMesh);
				}
				else
				{
					TManagedPtr<const FMesh> SourceMesh = LoadMesh(FCacheAddress(Args.SourceMesh, Item));
					const FMatrix44f& Transform = LoadMatrix(FCacheAddress(Args.Matrix, Item));
					FName BoneName = Program.ConstantNames[Args.BoneId];

					if (SourceMesh)
					{
						TManagedPtr<FMesh> Result = CloneOrTakeOver(MoveTemp(SourceMesh));

						MeshTransformWithBoneInline(Result.Get(), Transform, BoneName, Args.ThresholdFactor);
						StoreMesh(Item, Result);
					}
					else
					{
						StoreMesh(Item, SourceMesh);
					}
				}
				break;
			}

			default:
				check(false);
			}
			break;
		}

        case EOpType::ME_SKELETALMESH_BREAK:
		{
			FOperation::MeshSkeletalMeshBreakArgs Args = Program.GetOpArgs<FOperation::MeshSkeletalMeshBreakArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(Args.SkeletalMeshParameter, Item));
				break;

			case 1:
				{
					// This has been moved to a task. It should have been intercepted in IssueOp.
					check(false);
				}

			default:
				check(false);
			}

			break;
		}
        	
		default:
			if (type!=EOpType::NONE)
			{
				// Operation not implemented
				check( false );
			}
			break;
		}
	}

    void CodeRunner::RunCode_Image(const FScheduledOp& Item)
    {
		MUTABLE_CPUPROFILER_SCOPE(RunCode_Image);

		FImageOperator ImOp = MakeImageOperator(this);
		
		EOpType type = Program.GetOpType(Item.At);
		switch (type)
        {

        case EOpType::IM_LAYERCOLOR:
        {
			FOperation::ImageLayerColorArgs Args = Program.GetOpArgs<FOperation::ImageLayerColorArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.base, Item),
                           FScheduledOp::FromOpAndOptions( Args.color, Item, 0),
                           FScheduledOp( Args.mask, Item) );
                break;

            case 1:
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
				break;

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_LAYER:
        {
			FOperation::ImageLayerArgs Args = Program.GetOpArgs<FOperation::ImageLayerArgs>(Item.At);

			if (ExecutionStrategy == EExecutionStrategy::MinimizeMemory)
			{
				switch (Item.Stage)
				{
				case 0:
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.base, Item));
					break;

				case 1:
					// Request the rest of the data.
					AddOp(FScheduledOp(Item.At, Item, 2),
						FScheduledOp(Args.blended, Item),
						FScheduledOp(Args.mask, Item));
					break;

				case 2:
					// This has been moved to a task. It should have been intercepted in IssueOp.
					check(false);
					break;

				default:
					check(false);
				}
			}
			else
			{
				switch (Item.Stage)
				{
				case 0:
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.base, Item),
						FScheduledOp(Args.blended, Item),
						FScheduledOp(Args.mask, Item));
					break;

				case 1:
					// This has been moved to a task. It should have been intercepted in IssueOp.
					check(false);
					break;

				default:
					check(false);
				}
			}

            break;
        }

        case EOpType::IM_MULTILAYER:
        {
			FOperation::ImageMultiLayerArgs Args = Program.GetOpArgs<FOperation::ImageMultiLayerArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                       FScheduledOp( Args.rangeSize, Item ),
					   FScheduledOp(Args.base, Item));
                break;

            case 1:
            {
           		MUTABLE_CPUPROFILER_SCOPE(IM_MULTILAYER_1)
            		
                // We now know the number of iterations
                int32 Iterations = 0;
                if (Args.rangeSize)
                {
                    FCacheAddress RangeAddress(Args.rangeSize,Item);

                    // We support both integers and scalars here, which is not common.
                    // \todo: review if this is necessary or we can enforce it at compile time.
                    EDataType RangeSizeType = GetOpDataType(Program.GetOpType(Args.rangeSize) );
                    if (RangeSizeType == EDataType::Int)
                    {
						Iterations = LoadInt(RangeAddress);
                    }
                    else if (RangeSizeType == EDataType::Scalar)
                    {
						Iterations = int32( LoadScalar(RangeAddress) );
                    }
                }

				TManagedPtr<const FImage> Base = LoadImage(FCacheAddress(Args.base, Item));

				if (Iterations <= 0)
				{
					// There are no layers: return the base
					StoreImage(Item, Base);
				}
				else
				{
					// Store the base
					TManagedPtr<FImage> New = CloneOrTakeOver(MoveTemp(Base));
					EImageFormat InitialBaseFormat = New->GetFormat();

					// Reset relevancy map.
					New->Flags &= ~FImage::EImageFlags::IF_HAS_RELEVANCY_MAP;

					// This shouldn't happen in optimised models, but it could happen in editors, etc.
					// \todo: raise a performance warning?
					EImageFormat BaseFormat = GetUncompressedFormat(New->GetFormat());
					if (New->GetFormat() != BaseFormat)
					{
						TManagedPtr<FImage> Formatted = CreateImage( New->GetSizeX(), New->GetSizeY(), New->GetLODCount(), BaseFormat, EInitializationType::NotInitialized );

						bool bSuccess = false;
						ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), New.Get());
						check(bSuccess); // Decompression cannot fail

						New = Formatted;
					}

					FScheduledOpData Data;
					Data.Resource = New;
					Data.MultiLayer.Iterations = Iterations;
					Data.MultiLayer.OriginalBaseFormat = InitialBaseFormat;
					Data.MultiLayer.bBlendOnlyOneMip = false;
					int32 HeapDataIndex = HeapData.Add(Data);

					// Request the first layer
					int32 CurrentIteration = 0;
					FScheduledOp ItemForFirstLayerBlend = Item;

					ItemForFirstLayerBlend.ExecutionIndex = 
							GetMemory().GetOrAddModifiedExecutionIndex(Item.ExecutionIndex, Args.rangeId, CurrentIteration);

					AddOp(FScheduledOp(Item.At, Item, 2, HeapDataIndex), 
							FScheduledOp(Args.base, Item), 
							FScheduledOp(Args.blended, ItemForFirstLayerBlend), 
							FScheduledOp(Args.mask, ItemForFirstLayerBlend));
				}

                break;
            }

            default:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_MULTILAYER_default)

				FScheduledOpData& Data = HeapData[Item.CustomState];

				int32 Iterations = Data.MultiLayer.Iterations;
				int32 CurrentIteration = Item.Stage - 2;
				check(CurrentIteration >= 0 && CurrentIteration < 120);

				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Layer %d of %d"), CurrentIteration, Iterations));

				// Process the current layer

				TManagedPtr<FImage> Base = StaticCastManagedPtr<FImage>(ConstCastManagedPtr<FResource>(Data.Resource));
				 
  
				
                FScheduledOp ItemForLayerBlend = Item;
                {
                    ItemForLayerBlend.ExecutionIndex = 
							GetMemory().GetOrAddModifiedExecutionIndex(Item.ExecutionIndex, Args.rangeId, CurrentIteration);

					ItemForLayerBlend.CustomState = 0;

                    TManagedPtr<const FImage> Blended = LoadImage(FCacheAddress(Args.blended, ItemForLayerBlend));
                    TManagedPtr<const FImage> Mask = LoadImage(FCacheAddress(Args.mask, ItemForLayerBlend));

					if (Blended)
					{
						// This shouldn't happen in optimised models, but it could happen in editors, etc.
						// \todo: raise a performance warning?
						if (Blended->GetFormat() != Base->GetFormat())
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageFormat_BlendedFixForMultilayer);

							TManagedPtr<FImage> Formatted = CreateImage(Blended->GetSizeX(), Blended->GetSizeY(), Blended->GetLODCount(), Base->GetFormat(), EInitializationType::NotInitialized);

							bool bSuccess = false;
							ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), Blended.Get());
							check(bSuccess);

							Blended = Formatted;
						}

						// TODO: This shouldn't happen, but be defensive.
						FImageSize ResultSize = Base->GetSize();
						if (Blended->GetSize() != ResultSize)
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageResize_BlendedFixForMultilayer);

							TManagedPtr<FImage> Resized = CreateImage(ResultSize[0], ResultSize[1], Blended->GetLODCount(), Blended->GetFormat(), EInitializationType::NotInitialized);
							ImOp.ImageResizeLinear(Resized.Get(), 0, Blended.Get());

							Blended = Resized;
						}

						if (Mask && !(Mask->GetFormat() == EImageFormat::L_UByte || Mask->GetFormat() == EImageFormat::L_UByteRLE))
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageFormat_MaskFixForMultilayer);
							
							TManagedPtr<FImage> Formatted = CreateImage(Mask->GetSizeX(), Mask->GetSizeY(), Mask->GetLODCount(), EImageFormat::L_UByte, EInitializationType::NotInitialized);

							bool bSuccess = false;
							ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), Mask.Get());
							check(bSuccess);

							Mask = Formatted;
						}

						if (Mask && Mask->GetSize() != ResultSize)
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageResize_MaskFixForMultilayer);

							TManagedPtr<FImage> Resized = CreateImage(ResultSize[0], ResultSize[1], Mask->GetLODCount(), Mask->GetFormat(), EInitializationType::NotInitialized);
							ImOp.ImageResizeLinear(Resized.Get(), 0, Mask.Get());

							Mask = Resized;
						}

						if (Blended->GetLODCount() < Base->GetLODCount())
						{
							Data.MultiLayer.bBlendOnlyOneMip = true;
						}

						bool bApplyColorBlendToAlpha = false;

						// This becomes true if we need to update the mips of the resulting image
						// This could happen in the base image has mips, but one of the blended one doesn't.
						bool bBlendOnlyOneMip = Data.MultiLayer.bBlendOnlyOneMip;
					
						const int32 LODBegin = 0;
						const int32 LODEnd   = bBlendOnlyOneMip ? 1 : Base->GetLODCount();

						bool bUseBaseSourceFromBaseAlpha = false;   //(Args.flags & OP::ImageLayerArgs::F_BASE_RGB_FROM_ALPHA) != 0;
						bool bUseBlendSourceFromBlendAlpha = false; //(Args.flags & OP::ImageLayerArgs::F_BLENDED_RGB_FROM_ALPHA) != 0;
					
						if (EBlendType(Args.blendType) != EBlendType::BT_NORMAL_COMBINE)
						{
							bool bUseMaskFromBlendAlpha = static_cast<bool>(Args.bUseMaskFromBlended);

							FBlendFuncType* ColorBlendFunc = SelectBlendFunc(EBlendType(Args.blendType));
							FBlendFuncType* AlphaBlendFunc = nullptr; //ColorBlendFunc;

							if (!bApplyColorBlendToAlpha)
							{
								AlphaBlendFunc = SelectBlendFunc(EBlendType(Args.blendTypeAlpha));
							}

							constexpr uint8 BlendAlphaSourceChannel = 3;

							ImageLayerBlend(
									Base.Get(), Base.Get(), Blended.Get(), Mask.Get(), 
									ColorBlendFunc, AlphaBlendFunc, 
									LODBegin, LODEnd, 
									bApplyColorBlendToAlpha, 
									bUseMaskFromBlendAlpha, 
									bUseBaseSourceFromBaseAlpha,
									bUseBlendSourceFromBlendAlpha, 
									BlendAlphaSourceChannel);
						}
						else
						{
							ImageLayerCombine(
									Base.Get(), Base.Get(), Blended.Get(), Mask.Get(),
									OpImageLayerCombineOps::NormalCombine,
									LODBegin, LODEnd);
						}
					}
				}

				// Are we done?
				if (CurrentIteration + 1 == Iterations)
				{
					if (Data.MultiLayer.bBlendOnlyOneMip)
					{
						MUTABLE_CPUPROFILER_SCOPE(ImageLayer_MipFix);
						FMipmapGenerationSettings DummyMipSettings{};
						ImageMipmapInPlace(Settings.ImageCompressionQuality, Base.Get(), DummyMipSettings);
					}

					// TODO: Reconvert to OriginalBaseFormat if necessary?

					Data.Resource = nullptr;
					StoreImage(Item, Base);
					break;
				}
				else
				{
					// Request a new layer
					++CurrentIteration;
					FScheduledOp ItemForNextLayerBlend = Item;
					ItemForNextLayerBlend.ExecutionIndex = 
							GetMemory().GetOrAddModifiedExecutionIndex(Item.ExecutionIndex, Args.rangeId, CurrentIteration);

					AddOp(FScheduledOp(Item.At, Item, 2+CurrentIteration, Item.CustomState), 
							FScheduledOp(Args.blended, ItemForNextLayerBlend), 
							FScheduledOp(Args.mask, ItemForNextLayerBlend));

				}

                break;
            }

            } // switch stage

            break;
        }

		case EOpType::IM_NORMALCOMPOSITE:
		{
			FOperation::ImageNormalCompositeArgs Args = Program.GetOpArgs<FOperation::ImageNormalCompositeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				if (Args.base && Args.normal)
				{
					int32 OutChannel = -1;
					switch (Args.mode)
					{
						case ECompositeImageMode::CIM_NormalRoughnessToRed   : OutChannel = 0; break;
						case ECompositeImageMode::CIM_NormalRoughnessToGreen : OutChannel = 1; break;
						case ECompositeImageMode::CIM_NormalRoughnessToBlue  : OutChannel = 2; break;
						case ECompositeImageMode::CIM_NormalRoughnessToAlpha : OutChannel = 3; break;
						default: OutChannel = -1; break;
					}

					if (OutChannel < 0)
					{
						AddOp(FScheduledOp(Item.At, Item, 1, (uint32)OutChannel),
							FScheduledOp(Args.base, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 1, (uint32)OutChannel),
							FScheduledOp(Args.base, Item),
							FScheduledOp(Args.normal, Item));
					}
				}
				else
				{
					StoreImage(Item, nullptr);
				}
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(IM_NORMALCOMPOSITE_1)

				int32 OutChannel = (int32)Item.CustomState;
				if (OutChannel < 0)
				{
					StoreImage(Item, LoadImage(FCacheAddress(Args.base, Item)));
				}
				else
				{
					TManagedPtr<const FImage> Base = LoadImage(FCacheAddress(Args.base, Item));
					TManagedPtr<const FImage> Normal = LoadImage(FCacheAddress(Args.normal, Item));

					if (Normal->GetLODCount() < Base->GetLODCount())
					{
						MUTABLE_CPUPROFILER_SCOPE(ImageNormalComposite_EmergencyFix);

						int32 StartLevel = Normal->GetLODCount() - 1;
						int32 LevelCount = Base->GetLODCount();
						
						TManagedPtr<FImage> NormalFix = CloneOrTakeOver(MoveTemp(Normal));
						NormalFix->DataStorage.SetNumLODs(LevelCount);

						FMipmapGenerationSettings MipSettings{};
						ImOp.ImageMipmap(Settings.ImageCompressionQuality, NormalFix.Get(), NormalFix.Get(), StartLevel, LevelCount, MipSettings);

						Normal = NormalFix;
					}

					TManagedPtr<FImage> Result = CreateImage(Base->GetSizeX(), Base->GetSizeY(), Base->GetLODCount(), Base->GetFormat(), EInitializationType::NotInitialized);
					ImageNormalComposite(Result.Get(), Base.Get(), Normal.Get(), OutChannel, Args.power);

					StoreImage(Item, Result);
				}
				break;
			}

			default:
				check(false);
			}

			break;
		}

        case EOpType::IM_PIXELFORMAT:
        {
			FOperation::ImagePixelFormatArgs Args = Program.GetOpArgs<FOperation::ImagePixelFormatArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp( Args.source, Item) );
                break;

            case 1:
            {
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
				break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_CONFORM:
        {
			FOperation::ImageConformArgs Args = Program.GetOpArgs<FOperation::ImageConformArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(Args.Source, Item));
                break;

            case 1:
            {
				MUTABLE_CPUPROFILER_SCOPE(IM_CONFORM)

				TManagedPtr<const FImage> Source = LoadImage(FCacheAddress(Args.Source, Item));
			
				EImageFormat SourceFormat = Source->GetFormat();
				EImageFormat TargetFormat = SourceFormat;

				if (Args.ConformDescriptor.FormatIfAlpha != EImageFormat::None && 
					GetImageFormatData(SourceFormat).Channels > 3)
				{
					TargetFormat = Args.ConformDescriptor.FormatIfAlpha;
				}
				else if (Args.ConformDescriptor.m_format != EImageFormat::None)
				{
					TargetFormat = Args.ConformDescriptor.m_format;
				}
			
				EImageFormat SourceUncompressedFormat = GetUncompressedFormat(Source->GetFormat());

				int32 MipsToSkip = Item.ExecutionOptions;
				FImageSize ConformSize = FImageSize(
						FMath::Max(1, Args.ConformDescriptor.m_size.X >> MipsToSkip),
						FMath::Max(1, Args.ConformDescriptor.m_size.Y >> MipsToSkip));
	
				FImageSize TargetSize = Args.ConformDescriptor.m_size == FImageSize{0,0} ? Source->GetSize() : ConformSize;


				if (TargetSize != Source->GetSize() && SourceUncompressedFormat != Source->GetFormat())
				{
					MUTABLE_CPUPROFILER_SCOPE(IM_CONFORM::Decompress)
					TManagedPtr<FImage> Uncompressed = CreateImage(Source->GetSizeX(), Source->GetSizeY(), Source->GetLODCount(), SourceUncompressedFormat, EInitializationType::NotInitialized);
					
					bool bSuccess = true;
					ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Uncompressed.Get(), Source.Get());
					check(bSuccess);

					Source = Uncompressed;
				}	

				if (Source->GetSize() != TargetSize)
				{
					MUTABLE_CPUPROFILER_SCOPE(IM_CONFORM::Resize)
					check(Source->GetFormat() == GetUncompressedFormat(Source->GetFormat()));

					// ImageResizeLinear does not populate mips, the new image drops the mips. The ImageMipMap step will repopulate if needed.
					// TODO: Optimization, if The resize exactly down by two we could simply drop a mip.
					TManagedPtr<FImage> Resized = CreateImage(TargetSize.X, TargetSize.Y, 1, Source->GetFormat(), EInitializationType::NotInitialized);
					ImOp.ImageResizeLinear(Resized.Get(), Settings.ImageCompressionQuality, Source.Get());

					Source = Resized;
				}


				if (Source->GetFormat() != TargetFormat)
				{
					MUTABLE_CPUPROFILER_SCOPE(IM_CONFORM::Refromat)
					TManagedPtr<FImage> Reformated = CreateImage(Source->GetSizeX(), Source->GetSizeY(), Source->GetLODCount(), TargetFormat, EInitializationType::NotInitialized);

					bool bSuccess = true;
					ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Reformated.Get(), Source.Get());
					check(bSuccess);

					Source = Reformated;
				}


				const int32 MaxLevelCount = FImage::GetMipmapCount(Source->GetSizeX(), Source->GetSizeY());

				int32 TargetNumLODs = Source->GetLODCount();
				if (Args.ConformDescriptor.m_lods == 0xFF)
				{
					TargetNumLODs = FMath::Max(Source->GetLODCount(), MaxLevelCount);
				}
				else if (Args.ConformDescriptor.m_lods != 0)
				{
					TargetNumLODs = FMath::Max((int32)Args.ConformDescriptor.m_lods, Source->GetLODCount());
					TargetNumLODs = FMath::Min(TargetNumLODs, MaxLevelCount);
				}

				if (Source->GetLODCount() < TargetNumLODs)
				{
					int32 StartLOD = Source->GetLODCount() - 1;

					// We can add mips in place, takeover source.
					TManagedPtr<FImage> Image = CloneOrTakeOver(MoveTemp(Source));
			
					Image->DataStorage.SetNumLODs(TargetNumLODs);

					FMipmapGenerationSettings MipSettings{};
					ImOp.ImageMipmap(Settings.ImageCompressionQuality, Image.Get(), Image.Get(), StartLOD, TargetNumLODs, MipSettings);
					
					Source = Image;
				}

				StoreImage(Item, Source);

				break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_MIPMAP:
        {
			FOperation::ImageMipmapArgs Args = Program.GetOpArgs<FOperation::ImageMipmapArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
				AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp( Args.source, Item) );
                break;

            case 1:
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
				break;

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_RESIZE:
        {
			FOperation::ImageResizeArgs Args = Program.GetOpArgs<FOperation::ImageResizeArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp( Args.Source, Item) );
                break;

            case 1:
            {
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_RESIZELIKE:
        {
			FOperation::ImageResizeLikeArgs Args = Program.GetOpArgs<FOperation::ImageResizeLikeArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp(FScheduledOp(Item.At, Item, 1),
                      	FScheduledOp(Args.Source, Item),
                        FScheduledOp(Args.SizeSource, Item));
                break;

            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_RESIZELIKE_1)
            	
                TManagedPtr<const FImage> Base = LoadImage( FCacheAddress(Args.Source,Item) );
                TManagedPtr<const FImage> SizeBase = LoadImage( FCacheAddress(Args.SizeSource,Item) );
				FImageSize DestSize = SizeBase->GetSize();
				SizeBase = nullptr;

                if (Base->GetSize() != DestSize)
                {
					int32 BaseLODCount = Base->GetLODCount();
					TManagedPtr<FImage> Result = CreateImage(DestSize[0], DestSize[1], BaseLODCount, Base->GetFormat(), EInitializationType::NotInitialized);
					ImOp.ImageResizeLinear(Result.Get(), Settings.ImageCompressionQuality, Base.Get());
					Base = nullptr;

                    // If the source image had mips, generate them as well for the resized image.
                    // This shouldn't happen often since "ResizeLike" should be usually optimised out
                    // during model compilation. The mipmap generation below is not very precise with
                    // the number of mips that are needed and will probably generate too many
                    bool bSourceHasMips = BaseLODCount > 1;
                    
					if (bSourceHasMips)
                    {
						int32 LevelCount = FImage::GetMipmapCount(Result->GetSizeX(), Result->GetSizeY());	
						Result->DataStorage.SetNumLODs(LevelCount);

						FMipmapGenerationSettings MipSettings{};
						ImOp.ImageMipmap(Settings.ImageCompressionQuality, Result.Get(), Result.Get(), 0, LevelCount, MipSettings);
                    }				

					StoreImage(Item, Result);
				}
                else
                {
					StoreImage(Item, Base);
				}
				
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_RESIZEREL:
        {
			FOperation::ImageResizeRelArgs Args = Program.GetOpArgs<FOperation::ImageResizeRelArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp( Args.Source, Item) );
                break;

            case 1:
            {
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
            }

            default:
                check(false);
            }


            break;
        }
        case EOpType::IM_BLANKLAYOUT:
        {
			FOperation::ImageBlankLayoutArgs Args = Program.GetOpArgs<FOperation::ImageBlankLayoutArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0));
                break;

            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_BLANKLAYOUT_1)
            		
                TManagedPtr<const FLayout> pLayout = LoadLayout(FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0));

                FIntPoint SizeInBlocks = pLayout->GetGridSize();

				FIntPoint BlockSizeInPixels(Args.BlockSize[0], Args.BlockSize[1]);

				// Image size if we don't skip any mipmap
				FIntPoint FullImageSizeInPixels = SizeInBlocks * BlockSizeInPixels;
				int32 FullImageMipCount = FImage::GetMipmapCount(FullImageSizeInPixels.X, FullImageSizeInPixels.Y);

				FIntPoint ImageSizeInPixels = FullImageSizeInPixels;
				int32 MipsToSkip = Item.ExecutionOptions;
				MipsToSkip = FMath::Min(MipsToSkip, FullImageMipCount);
				if (MipsToSkip > 0)
				{
					//FIntPoint ReducedBlockSizeInPixels;

					// This method tries to reduce only the block size, but it fails if the image is still too big
					// If we want to generate only a subset of mipmaps, reduce the layout block size accordingly.
					//ReducedBlockSizeInPixels.X = BlockSizeInPixels.X >> MipsToSkip;
					//ReducedBlockSizeInPixels.Y = BlockSizeInPixels.Y >> MipsToSkip;
					//const FImageFormatData& FormatData = GetImageFormatData((EImageFormat)Args.format);
					//int32 MinBlockSize = FMath::Max(FormatData.PixelsPerBlockX, FormatData.PixelsPerBlockY);
					//ReducedBlockSizeInPixels.X = FMath::Max<int32>(ReducedBlockSizeInPixels.X, FormatData.PixelsPerBlockX);
					//ReducedBlockSizeInPixels.Y = FMath::Max<int32>(ReducedBlockSizeInPixels.Y, FormatData.PixelsPerBlockY);
					//FIntPoint ReducedImageSizeInPixels = SizeInBlocks * ReducedBlockSizeInPixels;

					// This method simply reduces the size and assumes all the other operations will handle degeenrate cases.
					ImageSizeInPixels = FullImageSizeInPixels / (1 << MipsToSkip);
					
					//if (ReducedImageSizeInPixels!= ImageSizeInPixels)
					//{
					//	check(false);
					//}
				}

                int32 MipsToGenerate = 1;
                if (Args.GenerateMipmaps)
                {
                    if (Args.MipmapCount == 0)
                    {
						MipsToGenerate = FImage::GetMipmapCount(ImageSizeInPixels.X, ImageSizeInPixels.Y);
                    }
                    else
                    {
						MipsToGenerate = FMath::Max(Args.MipmapCount - MipsToSkip, 1);
                    }
                }

				// It needs to be initialized in case it has gaps.
                TManagedPtr<FImage> New = CreateImage(ImageSizeInPixels.X, ImageSizeInPixels.Y, MipsToGenerate, EImageFormat(Args.Format), EInitializationType::Black );
                StoreImage(Item, New);
                break;
            }

            default:
                check(false);
            }


            break;
        }

        case EOpType::IM_COMPOSE:
        {
			FOperation::ImageComposeArgs Args = Program.GetOpArgs<FOperation::ImageComposeArgs>(Item.At);

			if (ExecutionStrategy == EExecutionStrategy::MinimizeMemory)
			{
            	switch (Item.Stage)
				{
				case 0:
					AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp::FromOpAndOptions(Args.layout, Item, 0));
					break;
				case 1:
				{
					TManagedPtr<const FLayout> ComposeLayout = 
							LoadLayout(FCacheAddress(Args.layout, FScheduledOp::FromOpAndOptions(Args.layout, Item, 0)));

					FScheduledOpData Data;
					Data.Resource = ComposeLayout;
					int32 DataPos = HeapData.Add(Data);

					int32 RelBlockIndex = ComposeLayout->FindBlock(Args.BlockId);

					if (RelBlockIndex >= 0)
					{
						AddOp(FScheduledOp(Item.At, Item, 2, DataPos), FScheduledOp(Args.base, Item));
					}
					else
					{
						// Jump directly to stage 3, no need to load mask or blockImage.
						AddOp(FScheduledOp(Item.At, Item, 3, DataPos), FScheduledOp(Args.base, Item));
					}

					break;
				}
				case 2:
				{
					AddOp(FScheduledOp(Item.At, Item, 3, Item.CustomState),
						  FScheduledOp(Args.blockImage, Item),
						  FScheduledOp(Args.mask, Item));
					break;
				}

				case 3:
					// This has been moved to a task. It should have been intercepted in IssueOp.
					check(false);
					break;

				default:
					check(false);
				}
			}
			else
			{
            	switch (Item.Stage)
				{
				case 0:
					AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp::FromOpAndOptions(Args.layout, Item, 0));
					break;

				case 1:
				{	
					TManagedPtr<const FLayout> ComposeLayout = 
							LoadLayout(FCacheAddress(Args.layout, FScheduledOp::FromOpAndOptions(Args.layout, Item, 0)));

					FScheduledOpData Data;
					Data.Resource = ComposeLayout;
					int32 DataPos = HeapData.Add(Data);

					int32 RelBlockIndex = ComposeLayout->FindBlock(Args.BlockId);
					if (RelBlockIndex >= 0)
					{
						AddOp(FScheduledOp(Item.At, Item, 2, DataPos),
							  FScheduledOp(Args.base, Item),
							  FScheduledOp(Args.blockImage, Item),
							  FScheduledOp(Args.mask, Item));
					}
					else
					{
						AddOp(FScheduledOp(Item.At, Item, 2, DataPos), FScheduledOp(Args.base, Item));
					}
					break;
				}

				case 2:
					// This has been moved to a task. It should have been intercepted in IssueOp.
					check(false);
					break;

				default:
					check(false);
				}
			}

            break;
        }
		
		case EOpType::IM_MULTICOMPOSE:
		{
			FOperation::ImageMultiComposeArgs Args = Program.GetOpArgs<FOperation::ImageMultiComposeArgs>(Item.At);
			constexpr float LODRoundTolerance = 0.1f;
			
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0), 
						FScheduledOp::FromOpAndOptions(Args.SourceLayout, Item, 0));
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageMultiCompose_1);

				TManagedPtr<const FLayout> ComposeLayout =
					LoadLayout(FCacheAddress(FScheduledOp::FromOpAndOptions(Args.Layout, Item, 0)));


				TManagedPtr<const FLayout> SourceLayout =
					LoadLayout(FCacheAddress(FScheduledOp::FromOpAndOptions(Args.SourceLayout, Item, 0)));

				bool bHasBlocksToCompose = false;
				for (const FLayoutBlock& Block : SourceLayout->Blocks)
				{
					if (ComposeLayout->FindBlock(Block.Id) >= 0)
					{
						bHasBlocksToCompose = true;
						break;
					}
				}

				int32 HeapIndex = HeapData.Emplace();
				HeapData.Emplace();
				HeapData.Emplace();
				HeapData.Emplace();

				HeapData[HeapIndex + 0].MultiCompose.bSplitInMultipleIterations = 
						MaxWorkerThreadsForIterativeOpExec < 0 || 
						(int32)LowLevelTasks::FScheduler::Get().GetNumWorkers() < MaxWorkerThreadsForIterativeOpExec;

				HeapData[HeapIndex + 0].MultiCompose.bHasBlocksToCompose = bHasBlocksToCompose;
				HeapData[HeapIndex + 0].Resource = ComposeLayout;

				HeapData[HeapIndex + 1].Resource = SourceLayout;

				if (bHasBlocksToCompose)
				{
					AddOp(FScheduledOp(Item.At, Item, 2, HeapIndex), FScheduledOp(Args.Base, Item));
				}
				else
				{
					// Jump directly to stage 3, no need to load the source image.
					AddOp(FScheduledOp(Item.At, Item, 3, HeapIndex), FScheduledOp(Args.Base, Item));
				}

				break;
			}
			case 2:
			{
				MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageMultiCompose_2);

				FScheduledOpData& Data = HeapData[Item.CustomState + 0];
				TManagedPtr<const FLayout> ComposeLayout = StaticCastManagedPtr<const FLayout>(Data.Resource);

				FScheduledOpData& SourceData = HeapData[Item.CustomState + 1];
				TManagedPtr<const FLayout> SourceLayout = StaticCastManagedPtr<const FLayout>(SourceData.Resource);

				FIntVector2 SourceLayoutGridSize = SourceLayout->GetGridSize();
				FIntVector2 ComposeLayoutGridSize = ComposeLayout->GetGridSize();
				
				FIntVector2 DstImageSize = FIntVector2(
						Args.LayoutBlockSizeInPixelsX*ComposeLayoutGridSize.X, 
						Args.LayoutBlockSizeInPixelsY*ComposeLayoutGridSize.Y);

				const int32 NumLODsToSkip = Item.ExecutionOptions;
				for (int32 LODIndex = 0; LODIndex < NumLODsToSkip; ++LODIndex)
				{
					DstImageSize.X = FMath::DivideAndRoundUp(DstImageSize.X, 2);	
					DstImageSize.Y = FMath::DivideAndRoundUp(DstImageSize.Y, 2);	
				}

				FIntVector2 SrcImageSize = FIntVector2(Args.SourceSizeX, Args.SourceSizeY);

				FIntVector2 SourceLayoutGridSizeInPixels = FIntVector2(
						SrcImageSize.X / SourceLayoutGridSize.X,
						SrcImageSize.Y / SourceLayoutGridSize.Y);

				FIntVector2 DstLayoutGridSizeInPixels = FIntVector2(
						DstImageSize.X / ComposeLayoutGridSize.X,
						DstImageSize.Y / ComposeLayoutGridSize.Y);

				float BestMaxLODForSourceImage = 0.0f;
				float BestMinLODForSourceImage = std::numeric_limits<float>::max();

				for (const FLayoutBlock& SrcBlock : SourceLayout->Blocks)
				{
					int32 FoundDstBlockIndex = ComposeLayout->FindBlock(SrcBlock.Id);

					if (FoundDstBlockIndex >= 0)
					{
						const FLayoutBlock& DstBlock = ComposeLayout->Blocks[FoundDstBlockIndex];
						
						FIntRect SrcRectInPixels = FIntRect(
								SrcBlock.Min.X * SourceLayoutGridSizeInPixels.X, 
								SrcBlock.Min.Y * SourceLayoutGridSizeInPixels.Y, 
								(SrcBlock.Min.X + SrcBlock.Size.X) * SourceLayoutGridSizeInPixels.X, 
								(SrcBlock.Min.Y + SrcBlock.Size.Y) * SourceLayoutGridSizeInPixels.Y);

						FIntRect DstRectInPixels = FIntRect(
								DstBlock.Min.X * DstLayoutGridSizeInPixels.X, 
								DstBlock.Min.Y * DstLayoutGridSizeInPixels.Y, 
								(DstBlock.Min.X + DstBlock.Size.X) * DstLayoutGridSizeInPixels.X, 
								(DstBlock.Min.Y + DstBlock.Size.Y) * DstLayoutGridSizeInPixels.Y);
	
						float BestLODForRect = FMath::Max(0.0f, 
								ImageMultiComposeComputeBestLODForSrcRect(DstRectInPixels, SrcRectInPixels)) + LODRoundTolerance;

						BestMinLODForSourceImage = FMath::Min(BestLODForRect, BestMinLODForSourceImage);
						BestMaxLODForSourceImage = FMath::Max(BestLODForRect, BestMaxLODForSourceImage);
					}
				}

				Data.MultiCompose.SourceLOD = FMath::FloorToInt32(BestMinLODForSourceImage);
				Data.MultiCompose.SourceNumLODs = FMath::FloorToInt32(BestMaxLODForSourceImage) - FMath::FloorToInt32(BestMinLODForSourceImage) + 1;

				AddOp(FScheduledOp(Item.At, Item, 3, Item.CustomState),	
						FScheduledOp::FromOpAndOptions(Args.SourceImage, Item, Data.MultiCompose.SourceLOD));
				break;
			}
			case 3:
			{
				MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageMultiCompose_3);

				FScheduledOpData& Data = HeapData[Item.CustomState + 0];
				TManagedPtr<const FLayout> ComposeLayout = StaticCastManagedPtr<const FLayout>(Data.Resource);

				FScheduledOpData& SourceData = HeapData[Item.CustomState + 1];
				TManagedPtr<const FLayout> SourceLayout = StaticCastManagedPtr<const FLayout>(SourceData.Resource);

				TManagedPtr<const FImage> BaseImage = LoadImage(FCacheAddress(Args.Base, Item));

				if (BaseImage && Data.MultiCompose.bHasBlocksToCompose)
				{
					int32 SourceImageLOD = Data.MultiCompose.SourceLOD;

					TManagedPtr<const FImage> SourceImage = 
							LoadImage(FCacheAddress(Args.SourceImage, Item.ExecutionIndex, SourceImageLOD));

					if (SourceImage)
					{
						const FImageSize ExpectedSourceSize = FImageSize(
								FMath::Max<uint16>(Args.SourceSizeX >> SourceImageLOD, 1), 
								FMath::Max<uint16>(Args.SourceSizeY >> SourceImageLOD, 1));

						if (SourceImage->GetSize() != ExpectedSourceSize)
						{
							MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageMultiCompose_SizeFixup);
							
							TManagedPtr<FImage> Resized = CreateImage(ExpectedSourceSize.X, ExpectedSourceSize.Y, 1, SourceImage->GetFormat(), EInitializationType::NotInitialized);
							ImOp.ImageResizeLinear(Resized.Get(), Settings.ImageCompressionQuality, SourceImage.Get());
							
							SourceImage = Resized;	
						}

						FIntVector2 SourceLayoutGridSize = SourceLayout->GetGridSize();
						FIntVector2 ComposeLayoutGridSize = ComposeLayout->GetGridSize();

						FIntVector2 SourceLayoutGridSizeInPixels = FIntVector2(
								SourceImage->GetSizeX() / SourceLayoutGridSize.X,
								SourceImage->GetSizeY() / SourceLayoutGridSize.Y);

						FIntVector2 ComposeLayoutGridSizeInPixels = FIntVector2(
								BaseImage->GetSizeX() / ComposeLayoutGridSize.X,
								BaseImage->GetSizeY() / ComposeLayoutGridSize.Y);

						Data.MultiCompose.NumElemsToProcess = 0;
						int32 NumSrcLODsNeeded = 1;
						for (const FLayoutBlock& SrcBlock : SourceLayout->Blocks)
						{
							int32 FoundDstBlockIndex = ComposeLayout->FindBlock(SrcBlock.Id);

							if (FoundDstBlockIndex >= 0)
							{
								const FLayoutBlock& DstBlock = ComposeLayout->Blocks[FoundDstBlockIndex];
								FIntRect SrcRectInPixels = FIntRect(
										SrcBlock.Min.X * SourceLayoutGridSizeInPixels.X, 
										SrcBlock.Min.Y * SourceLayoutGridSizeInPixels.Y, 
										(SrcBlock.Min.X + SrcBlock.Size.X) * SourceLayoutGridSizeInPixels.X, 
										(SrcBlock.Min.Y + SrcBlock.Size.Y) * SourceLayoutGridSizeInPixels.Y);

								FIntRect DstRectInPixels = FIntRect(
										DstBlock.Min.X * ComposeLayoutGridSizeInPixels.X, 
										DstBlock.Min.Y * ComposeLayoutGridSizeInPixels.Y, 
										(DstBlock.Min.X + DstBlock.Size.X) * ComposeLayoutGridSizeInPixels.X, 
										(DstBlock.Min.Y + DstBlock.Size.Y) * ComposeLayoutGridSizeInPixels.Y);

								float BestLODForRect = FMath::Max(0.0f,
										ImageMultiComposeComputeBestLODForSrcRect(DstRectInPixels, SrcRectInPixels));
								
								int32 SelectedLOD = FMath::FloorToInt32(BestLODForRect + LODRoundTolerance);

								NumSrcLODsNeeded = FMath::Max(SelectedLOD + 1, NumSrcLODsNeeded);
								++Data.MultiCompose.NumElemsToProcess;		
							}	
						}

						if (SourceImage->GetLODCount() < NumSrcLODsNeeded)
						{
							MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageMultiCompose_BilinearMipGen);

							TManagedPtr<FImage> OwnedSource = CloneOrTakeOver(MoveTemp(SourceImage));
							OwnedSource->DataStorage.SetNumLODs(NumSrcLODsNeeded);

							FMipmapGenerationSettings MipSettings{};
							ImOp.ImageMipmap(Settings.ImageCompressionQuality, OwnedSource.Get(), OwnedSource.Get(), 0, NumSrcLODsNeeded, MipSettings);
							//ImageMipmapInPlace(0, OwnedSource.Get(), FMipmapGenerationSettings{});

							SourceImage = OwnedSource;
						}

						HeapData[Item.CustomState + 2].Resource = SourceImage; 
						HeapData[Item.CustomState + 3].Resource = CloneOrTakeOver(MoveTemp(BaseImage)); 

						AddOp(FScheduledOp(Item.At, Item, 4, Item.CustomState));
					}
					else
					{
						StoreImage(Item, BaseImage);

						HeapData[Item.CustomState + 0].Resource = nullptr;
						HeapData[Item.CustomState + 1].Resource = nullptr;
						HeapData[Item.CustomState + 2].Resource = nullptr;
						HeapData[Item.CustomState + 3].Resource = nullptr;
					}
				}
				else
				{
					StoreImage(Item, BaseImage);

					HeapData[Item.CustomState + 0].Resource = nullptr;
					HeapData[Item.CustomState + 1].Resource = nullptr;
					HeapData[Item.CustomState + 2].Resource = nullptr;
					HeapData[Item.CustomState + 3].Resource = nullptr;
				}

				break;
			}

			default:
			{
				check(Item.Stage >= 4);

				MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageMultiCompose_Iters);

				FScheduledOpData& Data = HeapData[Item.CustomState + 0];
				FScheduledOpData& ComposeLayoutData = Data;
				FScheduledOpData& SourceLayoutData  = HeapData[Item.CustomState + 1];
				FScheduledOpData& SourceImageData   = HeapData[Item.CustomState + 2];
				FScheduledOpData& ResultImageData   = HeapData[Item.CustomState + 3];
				
				TManagedPtr<const FLayout> ComposeLayout = StaticCastManagedPtr<const FLayout>(ComposeLayoutData.Resource);
				TManagedPtr<const FLayout> SourceLayout = StaticCastManagedPtr<const FLayout>(SourceLayoutData.Resource);
				TManagedPtr<const FImage> SourceImage = StaticCastManagedPtr<const FImage>(SourceImageData.Resource);
				TManagedPtr<FImage> ResultImage = ConstCastManagedPtr<FImage>(StaticCastManagedPtr<const FImage>(ResultImageData.Resource));

				check(SourceImage);
				check(ResultImage);

				int32 IterationBegin = 0;
				int32 IterationEnd = IterationBegin + Data.MultiCompose.NumElemsToProcess;

				if (Data.MultiCompose.bSplitInMultipleIterations)
				{
					const int32 CurrentIteration = Item.Stage - 4;
					constexpr int32 MaxIterations = FScheduledOp::MaxNumStages - 4;
					const int32 ElemsPerIter = FMath::DivideAndRoundUp<int32>(Data.MultiCompose.NumElemsToProcess, MaxIterations);

					IterationBegin = CurrentIteration*ElemsPerIter; 
					IterationEnd = FMath::Min<int32>(IterationBegin + ElemsPerIter, Data.MultiCompose.NumElemsToProcess);
				}

				int32 SourceImageLOD = Data.MultiCompose.SourceLOD;

				FIntVector2 SourceLayoutGridSize = SourceLayout->GetGridSize();
				FIntVector2 ComposeLayoutGridSize = ComposeLayout->GetGridSize();

				FIntVector2 SourceLayoutGridSizeInPixels = FIntVector2(
						SourceImage->GetSizeX() / SourceLayoutGridSize.X,
						SourceImage->GetSizeY() / SourceLayoutGridSize.Y);

				FIntVector2 ComposeLayoutGridSizeInPixels = FIntVector2(
						ResultImage->GetSizeX() / ComposeLayoutGridSize.X,
						ResultImage->GetSizeY() / ComposeLayoutGridSize.Y);

				TArray<FIntRect, TInlineAllocator<16>> SrcRects;
				TArray<FIntRect, TInlineAllocator<16>> DstRects;
				TArray<int32,    TInlineAllocator<16>> SelectedLODForRect;

				int32 ElemIndex = 0;
				int32 NumSrcLODsNeeded = 1;
				for (const FLayoutBlock& SrcBlock : SourceLayout->Blocks)
				{
					int32 FoundDstBlockIndex = ComposeLayout->FindBlock(SrcBlock.Id);

					if (FoundDstBlockIndex >= 0)
					{
						if (ElemIndex >= IterationBegin && ElemIndex < IterationEnd)
						{
							const FLayoutBlock& DstBlock = ComposeLayout->Blocks[FoundDstBlockIndex];
							FIntRect& SrcRectInPixels = SrcRects.Emplace_GetRef(
									SrcBlock.Min.X * SourceLayoutGridSizeInPixels.X, 
									SrcBlock.Min.Y * SourceLayoutGridSizeInPixels.Y, 
									(SrcBlock.Min.X + SrcBlock.Size.X) * SourceLayoutGridSizeInPixels.X, 
									(SrcBlock.Min.Y + SrcBlock.Size.Y) * SourceLayoutGridSizeInPixels.Y);

							FIntRect& DstRectInPixels = DstRects.Emplace_GetRef(
									DstBlock.Min.X * ComposeLayoutGridSizeInPixels.X, 
									DstBlock.Min.Y * ComposeLayoutGridSizeInPixels.Y, 
									(DstBlock.Min.X + DstBlock.Size.X) * ComposeLayoutGridSizeInPixels.X, 
									(DstBlock.Min.Y + DstBlock.Size.Y) * ComposeLayoutGridSizeInPixels.Y);

							float BestLODForRect = FMath::Max(0.0f,
									ImageMultiComposeComputeBestLODForSrcRect(DstRectInPixels, SrcRectInPixels));
							
							int32 SelectedLOD = FMath::FloorToInt32(BestLODForRect + LODRoundTolerance);

							NumSrcLODsNeeded = FMath::Max(SelectedLOD + 1, NumSrcLODsNeeded);
							SelectedLODForRect.Emplace(SelectedLOD);
						}

						if (++ElemIndex >= IterationEnd)
						{
							break;
						}
					}
				}

				// Fix src rect sizes for the selected LOD.
				for (int32 RectIndex = SrcRects.Num() - 1; RectIndex >= 0; --RectIndex)
				{
					int32 SelectedLOD = SelectedLODForRect[RectIndex];

					for (int32 LODIndex = 0; LODIndex < SelectedLOD; ++LODIndex)
					{
						SrcRects[RectIndex] = FIntRect::DivideAndRoundUp(SrcRects[RectIndex], 2);
					}
				}

				ImageMultiCompose(*ResultImage, *SourceImage, DstRects, SrcRects, SelectedLODForRect, Settings.ImageCompressionQuality);
				ResultImage->DataStorage.SetNumLODs(1);

				if (ElemIndex >= Data.MultiCompose.NumElemsToProcess)
				{
					HeapData[Item.CustomState + 0].Resource = nullptr;
					HeapData[Item.CustomState + 1].Resource = nullptr;
					HeapData[Item.CustomState + 2].Resource = nullptr;
					HeapData[Item.CustomState + 3].Resource = nullptr;

					StoreImage(Item, ResultImage);
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, Item.Stage + 1, Item.CustomState));
				}
			}
			}	

			break;
		}

        case EOpType::IM_INTERPOLATE:
        {
			FOperation::ImageInterpolateArgs Args = Program.GetOpArgs<FOperation::ImageInterpolateArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                       FScheduledOp( Args.Factor, Item) );
                break;

            case 1:
            {
           		MUTABLE_CPUPROFILER_SCOPE(IM_INTERPOLATE_1)
            	
                // Targets must be consecutive
                int32 count = 0;
                for ( int32 i=0
                    ; i<MUTABLE_OP_MAX_INTERPOLATE_COUNT && Args.Targets[i]
                    ; ++i )
                {
                    count++;
                }

                float factor = LoadScalar( FCacheAddress(Args.Factor,Item) );

                float delta = 1.0f/(count-1);
                int32 min = (int32)floorf( factor/delta );
                int32 max = (int32)ceilf( factor/delta );

                float bifactor = factor/delta - min;

                FScheduledOpData data;
                data.Interpolate.Bifactor = bifactor;
				data.Interpolate.Min = FMath::Clamp(min, 0, count - 1);
				data.Interpolate.Max = FMath::Clamp(max, 0, count - 1);
				uint32 dataPos = uint32(HeapData.Add(data));

                if ( bifactor < UE_SMALL_NUMBER )
                {
                    AddOp( FScheduledOp( Item.At, Item, 2, dataPos),
                            FScheduledOp( Args.Targets[min], Item) );
                }
                else if ( bifactor > 1.0f-UE_SMALL_NUMBER )
                {
                    AddOp( FScheduledOp( Item.At, Item, 2, dataPos),
                            FScheduledOp( Args.Targets[max], Item) );
                }
                else
                {
                    AddOp( FScheduledOp( Item.At, Item, 2, dataPos),
                            FScheduledOp( Args.Targets[min], Item),
                            FScheduledOp( Args.Targets[max], Item) );
                }
                break;
            }

            case 2:
            {
           		MUTABLE_CPUPROFILER_SCOPE(IM_INTERPOLATE_2)
            		
                // Targets must be consecutive
                int32 count = 0;
                for ( int32 i=0
                    ; i<MUTABLE_OP_MAX_INTERPOLATE_COUNT && Args.Targets[i]
                    ; ++i )
                {
                    count++;
                }

                // Factor from 0 to 1 between the two targets
                const FScheduledOpData& Data = HeapData[Item.CustomState];
                float Bifactor = Data.Interpolate.Bifactor;
                int32 Min = Data.Interpolate.Min;
                int32 Max = Data.Interpolate.Max;

                if (Bifactor < UE_SMALL_NUMBER)
                {
                    TManagedPtr<const FImage> Source = LoadImage(FCacheAddress(Args.Targets[Min], Item));
					StoreImage(Item, Source);
				}
                else if (Bifactor > 1.0f - UE_SMALL_NUMBER)
                {
                    TManagedPtr<const FImage> Source = LoadImage(FCacheAddress(Args.Targets[Max], Item));
					StoreImage(Item, Source);
				}
                else
                {
					TManagedPtr<const FImage> pMin = LoadImage(FCacheAddress(Args.Targets[Min], Item));
                    TManagedPtr<const FImage> pMax = LoadImage(FCacheAddress(Args.Targets[Max], Item));

                    if (pMin && pMax)
                    {						
						TManagedPtr<FImage> pNew = CloneOrTakeOver(MoveTemp(pMin));

						// Be defensive: ensure image sizes match.
						if (pNew->GetSize() != pMax->GetSize())
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageResize_ForInterpolate);
							TManagedPtr<FImage> Resized = CreateImage(pNew->GetSizeX(), pNew->GetSizeY(), pMax->GetLODCount(), pMax->GetFormat(), EInitializationType::NotInitialized);
							ImOp.ImageResizeLinear(Resized.Get(), 0, pMax.Get());
							pMax = Resized;
						}

						// Be defensive: ensure format matches.
						if (pNew->GetFormat() != pMax->GetFormat())
						{
							MUTABLE_CPUPROFILER_SCOPE(Format_ForInterpolate);

							TManagedPtr<FImage> Formatted = CreateImage(pMax->GetSizeX(), pMax->GetSizeY(), pMax->GetLODCount(), pNew->GetFormat(), EInitializationType::NotInitialized);
							
							bool bSuccess = false;
							ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), pMax.Get());
							check(bSuccess);
							
							pMax = Formatted;
						}

						int32 LevelCount = FMath::Max(pNew->GetLODCount(), pMax->GetLODCount());

						if (pNew->GetLODCount() != LevelCount)
						{
							MUTABLE_CPUPROFILER_SCOPE(Mipmap_ForInterpolate);
						
							int32 StartLevel = pNew->GetLODCount() - 1;
							// pNew is local owned, no need to CloneOrTakeOver.
							pNew->DataStorage.SetNumLODs(LevelCount);

							FMipmapGenerationSettings MipSettings{};
							ImOp.ImageMipmap(Settings.ImageCompressionQuality, pNew.Get(), pNew.Get(), StartLevel, LevelCount, MipSettings);

						}

						if (pMax->GetLODCount() != LevelCount)
						{
							MUTABLE_CPUPROFILER_SCOPE(Mipmap_ForInterpolate);

							int32 StartLevel = pMax->GetLODCount() - 1;

							TManagedPtr<FImage> MaxFix = CloneOrTakeOver(MoveTemp(pMax));

							MaxFix->DataStorage.SetNumLODs(LevelCount);
							
							FMipmapGenerationSettings MipSettings{};
							ImOp.ImageMipmap(Settings.ImageCompressionQuality, MaxFix.Get(), MaxFix.Get(), StartLevel, LevelCount, MipSettings);

							pMax = MaxFix;
						}

                        ImageInterpolate(pNew.Get(), pMax.Get(), Bifactor);

						StoreImage(Item, pNew);
					}
                    else if (pMin)
                    {
						StoreImage(Item, pMin);
					}
                    else if (pMax)
                    {
						StoreImage(Item, pMax);
					}
				}

                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_SATURATE:
        {
			FOperation::ImageSaturateArgs Args = Program.GetOpArgs<FOperation::ImageSaturateArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                        FScheduledOp( Args.Base, Item ),
                        FScheduledOp::FromOpAndOptions( Args.Factor, Item, 0 ));
                break;

            case 1:
            {
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_LUMINANCE:
        {
			FOperation::ImageLuminanceArgs Args = Program.GetOpArgs<FOperation::ImageLuminanceArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                        FScheduledOp( Args.Base, Item ) );
                break;

            case 1:
            {
           		MUTABLE_CPUPROFILER_SCOPE(IM_LUMINANCE_1)
            		
                TManagedPtr<const FImage> Base = LoadImage( FCacheAddress(Args.Base,Item) );

				TManagedPtr<FImage> Result = CreateImage(Base->GetSizeX(), Base->GetSizeY(), Base->GetLODCount(), EImageFormat::L_UByte, EInitializationType::NotInitialized);
                ImageLuminance(Result.Get(), Base.Get());

				StoreImage(Item, Result);
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_SWIZZLE:
        {
			FOperation::ImageSwizzleArgs Args = Program.GetOpArgs<FOperation::ImageSwizzleArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                        FScheduledOp( Args.sources[0], Item ),
                        FScheduledOp( Args.sources[1], Item ),
                        FScheduledOp( Args.sources[2], Item ),
                        FScheduledOp( Args.sources[3], Item ) );
                break;

            case 1:
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
				break;

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_COLORMAP:
        {
			FOperation::ImageColorMapArgs Args = Program.GetOpArgs<FOperation::ImageColorMapArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.Base, Item ),
                           FScheduledOp( Args.Mask, Item ),
                           FScheduledOp( Args.Map, Item ) );
                break;

            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_COLORMAP_1)
            		
                TManagedPtr<const FImage> Source = LoadImage( FCacheAddress(Args.Base,Item) );
                TManagedPtr<const FImage> Mask = LoadImage( FCacheAddress(Args.Mask,Item) );
                TManagedPtr<const FImage> Map = LoadImage( FCacheAddress(Args.Map,Item) );

				bool bOnlyOneMip = (Mask->GetLODCount() < Source->GetLODCount());

				// Be defensive: ensure image sizes match.
				if (Mask->GetSize() != Source->GetSize())
				{
					MUTABLE_CPUPROFILER_SCOPE(ImageResize_ForColormap);
					TManagedPtr<FImage> Resized = CreateImage(Source->GetSizeX(), Source->GetSizeY(), 1, Mask->GetFormat(), EInitializationType::NotInitialized);
					ImOp.ImageResizeLinear(Resized.Get(), 0, Mask.Get());
					Mask = Resized;
				}

				TManagedPtr<FImage> Result = CreateImage(Source->GetSizeX(), Source->GetSizeY(), Source->GetLODCount(), Source->GetFormat(), EInitializationType::NotInitialized);
				ImageColorMap( Result.Get(), Source.Get(), Mask.Get(), Map.Get(), bOnlyOneMip);

				if (bOnlyOneMip)
				{
					MUTABLE_CPUPROFILER_SCOPE(ImageColorMap_MipFix);
					FMipmapGenerationSettings DummyMipSettings{};
					ImageMipmapInPlace(Settings.ImageCompressionQuality, Result.Get(), DummyMipSettings);
				}

				StoreImage(Item, Result);
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_BINARISE:
        {
			FOperation::ImageBinariseArgs Args = Program.GetOpArgs<FOperation::ImageBinariseArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                        FScheduledOp( Args.Base, Item ),
                        FScheduledOp::FromOpAndOptions( Args.Threshold, Item, 0 ) );
                break;

            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_BINARISE_1)
            		
                TManagedPtr<const FImage> pA = LoadImage( FCacheAddress(Args.Base,Item) );

                float c = LoadScalar(FScheduledOp::FromOpAndOptions(Args.Threshold, Item, 0));

                TManagedPtr<FImage> Result = CreateImage(pA->GetSizeX(), pA->GetSizeY(), pA->GetLODCount(), EImageFormat::L_UByte, EInitializationType::NotInitialized);
				ImageBinarise(Result.Get(), pA.Get(), c);

				StoreImage(Item, Result);
                break;
            }

            default:
                check(false);
            }

            break;
        }

		case EOpType::IM_INVERT:
		{
			FOperation::ImageInvertArgs Args = Program.GetOpArgs<FOperation::ImageInvertArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(Args.Base, Item));
				break;

			case 1:
			{
				// This has been moved to a task. It should have been intercepted in IssueOp.
				check(false);
			}

			default:
				check(false);
			}

			break;
		}

        case EOpType::IM_PLAINCOLOR:
        {
			FOperation::ImagePlainColorArgs Args = Program.GetOpArgs<FOperation::ImagePlainColorArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
				AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp::FromOpAndOptions( Args.Color, Item, 0 ) );
                break;

            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_PLAINCOLOR_1)
            		
				FVector4f c = LoadColor(FScheduledOp::FromOpAndOptions(Args.Color, Item, 0));

				uint16 SizeX = Args.Size[0];
				uint16 SizeY = Args.Size[1];
				int32 LODs = Args.LODs;
				
				// This means all the mip chain
				if (LODs == 0)
				{
					LODs = FMath::CeilLogTwo(FMath::Max(SizeX,SizeY));
				}

				for (int32 l=0; l<Item.ExecutionOptions; ++l)
				{
					SizeX = FMath::Max(uint16(1), FMath::DivideAndRoundUp(SizeX, uint16(2)));
					SizeY = FMath::Max(uint16(1), FMath::DivideAndRoundUp(SizeY, uint16(2)));
					--LODs;
				}

                TManagedPtr<FImage> pA = CreateImage( SizeX, SizeY, FMath::Max(LODs,1), EImageFormat(Args.Format), EInitializationType::NotInitialized );

				ImOp.FillColor(pA.Get(), c);

				StoreImage(Item, pA);
                break;
            }

            default:
                check(false);
            }

            break;
        }

		case EOpType::IM_REFERENCE:
		{
			FOperation::ResourceReferenceArgs Args = Program.GetOpArgs<FOperation::ResourceReferenceArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				TManagedPtr<FImage> Result;
				if (Args.ForceLoad)
				{
					// This should never be reached because it should have been caught as a Task in IssueOp
					check(false);
				}
				else
				{
					TPassthroughObjectPtr<UTexture> ExternalObject = FindPassthrough<UTexture>(Args.ID);
					Result = FImage::CreateAsReference(ExternalObject, Args.ImageDesc, false);
				}
				StoreImage(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Item, Args.variable);
			UTexture* Image = Parameters.GetTextureValue(Args.variable, Index.Get());

       		TPassthroughObjectPtr<UTexture> ExternalObject(Image);
			TManagedPtr<FImage> Result = FImage::CreateAsReference(ExternalObject, FImageDesc(), false);
        		
			StoreImage(Item, Result);
			break;
		}

        case EOpType::IM_CROP:
        {
			FOperation::ImageCropArgs Args = Program.GetOpArgs<FOperation::ImageCropArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp( Args.source, Item ) );
                break;

            case 1:
            {
            	MUTABLE_CPUPROFILER_SCOPE(IM_CROP_1)
            		
                TManagedPtr<const FImage> pA = LoadImage( FCacheAddress(Args.source,Item) );

                box< UE::Math::TIntVector2<int32> > rect;
                rect.min[0] = Args.minX;
                rect.min[1] = Args.minY;
                rect.size[0] = Args.sizeX;
                rect.size[1] = Args.sizeY;

				// Apply ther mipmap reduction to the crop rectangle.
				int32 MipsToSkip = Item.ExecutionOptions;
				while ( MipsToSkip>0 && rect.size[0]>0 && rect.size[1]>0 )
				{
					rect.min[0] /= 2;
					rect.min[1] /= 2;
					rect.size[0] /= 2;
					rect.size[1] /= 2;
					MipsToSkip--;
				}

				TManagedPtr<FImage> Result;
				if (!rect.IsEmpty())
				{
					Result = CreateImage( rect.size[0], rect.size[1], 1, pA->GetFormat(), EInitializationType::NotInitialized);
					ImOp.ImageCrop(Result, Settings.ImageCompressionQuality, pA.Get(), rect);
				}
				
				StoreImage(Item, Result);
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_PATCH:
        {
			// TODO: This is optimized for memory-usage but base and patch could be requested at the same time
			FOperation::ImagePatchArgs Args = Program.GetOpArgs<FOperation::ImagePatchArgs>(Item.At);
            switch (Item.Stage)
            {
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(Args.base, Item));
				break;

			case 1:
				AddOp(FScheduledOp(Item.At, Item, 2), FScheduledOp(Args.patch, Item));
				break;

			case 2:
            {
                MUTABLE_CPUPROFILER_SCOPE(IM_PATCH_1)

                TManagedPtr<const FImage> pA = LoadImage( FCacheAddress(Args.base,Item) );
                TManagedPtr<const FImage> pB = LoadImage( FCacheAddress(Args.patch,Item) );

				// Failsafe
				if (!pA || !pB)
				{
					StoreImage(Item, pA);
					break;
				}

				// Apply the mipmap reduction to the crop rectangle.
				int32 MipsToSkip = Item.ExecutionOptions;
				box<FIntVector2> rect;
				rect.min[0] = Args.minX / (1 << MipsToSkip);
				rect.min[1] = Args.minY / (1 << MipsToSkip);
				rect.size[0] = pB->GetSizeX();
				rect.size[1] = pB->GetSizeY();

                TManagedPtr<FImage> Result = CloneOrTakeOver(MoveTemp(pA));

				bool bApplyPatch = !rect.IsEmpty();
				if (bApplyPatch)
				{
					// Change the block image format if it doesn't match the composed image
					// This is usually enforced at object compilation time.
					if (Result->GetFormat() != pB->GetFormat())
					{
						MUTABLE_CPUPROFILER_SCOPE(ImagPatchReformat);

						EImageFormat format = GetMostGenericFormat(Result->GetFormat(), pB->GetFormat());

						const FImageFormatData& finfo = GetImageFormatData(format);
						if (finfo.PixelsPerBlockX == 0)
						{
							format = GetUncompressedFormat(format);
						}

						if (Result->GetFormat() != format)
						{
							TManagedPtr<FImage> Formatted = CreateImage(Result->GetSizeX(), Result->GetSizeY(), Result->GetLODCount(), format, EInitializationType::NotInitialized);
							bool bSuccess = false;
							ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), Result.Get());
							check(bSuccess);
							Result = Formatted;
						}
						if (pB->GetFormat() != format)
						{
							TManagedPtr<FImage> Formatted = CreateImage(pB->GetSizeX(), pB->GetSizeY(), pB->GetLODCount(), format, EInitializationType::NotInitialized);
							bool bSuccess = false;
							ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), pB.Get());
							check(bSuccess);
							pB = Formatted;
						}
					}

					// Don't patch if below the image compression block size.
					const FImageFormatData& finfo = GetImageFormatData(Result->GetFormat());
					bApplyPatch =
						(rect.min[0] % finfo.PixelsPerBlockX == 0) &&
						(rect.min[1] % finfo.PixelsPerBlockY == 0) &&
						(rect.size[0] % finfo.PixelsPerBlockX == 0) &&
						(rect.size[1] % finfo.PixelsPerBlockY == 0) &&
						(rect.min[0] + rect.size[0]) <= Result->GetSizeX() &&
						(rect.min[1] + rect.size[1]) <= Result->GetSizeY()
						;
				}

				if (bApplyPatch)
				{
					ImOp.ImageCompose(Result.Get(), pB.Get(), rect);
					Result->Flags = 0;
				}
				else
				{
					// This happens very often when skipping mips, and floods the log.
					//UE_LOGF( LogMutableCore, Verbose, "Skipped patch operation for image not fitting the block compression size. Small image? Patch rect is (%d, %d), (%d, %d), base is (%d, %d)",
					//	rect.min[0], rect.min[1], rect.size[0], rect.size[1], Result->GetSizeX(), Result->GetSizeY());
				}

				StoreImage( Item, Result );
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_RASTERMESH:
        {
			FOperation::ImageRasterMeshArgs Args = Program.GetOpArgs<FOperation::ImageRasterMeshArgs>(Item.At);

			constexpr uint8 MeshContentFilter = (uint8)(EMeshContentFlags::GeometryData | EMeshContentFlags::PoseData);
            switch (Item.Stage)
            {
            case 0:
				if (Args.image)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp::FromOpAndOptions(Args.mesh, Item, MeshContentFilter),
						FScheduledOp::FromOpAndOptions(Args.projector, Item, 0));
				}
				else
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp::FromOpAndOptions(Args.mesh, Item, MeshContentFilter));
				}
                break;

			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(IM_RASTERMESH_1)

				
				TManagedPtr<const FMesh> Mesh = 
						LoadMesh(FScheduledOp::FromOpAndOptions(Args.mesh, Item, MeshContentFilter));

				// If no image, we are generating a flat mesh UV raster. This is the final stage in this case.
				if (!Args.image)
				{
					uint16 SizeX = Args.sizeX;
					uint16 SizeY = Args.sizeY;
					UE::Math::TIntVector2<uint16> CropMin(Args.CropMinX, Args.CropMinY);
					UE::Math::TIntVector2<uint16> UncroppedSize(Args.UncroppedSizeX, Args.UncroppedSizeY);

					// Drop mips while possible
					int32 MipsToDrop = Item.ExecutionOptions;
					bool bUseCrop = UncroppedSize[0] > 0;
					while (MipsToDrop && !(SizeX % 2) && !(SizeY % 2))
					{
						SizeX = FMath::Max(uint16(1), FMath::DivideAndRoundUp(SizeX, uint16(2)));
						SizeY = FMath::Max(uint16(1), FMath::DivideAndRoundUp(SizeY, uint16(2)));
						if (bUseCrop)
						{
							CropMin[0] = FMath::DivideAndRoundUp(CropMin[0], uint16(2));
							CropMin[1] = FMath::DivideAndRoundUp(CropMin[1], uint16(2));
							UncroppedSize[0] = FMath::Max(uint16(1), FMath::DivideAndRoundUp(UncroppedSize[0], uint16(2)));
							UncroppedSize[1] = FMath::Max(uint16(1), FMath::DivideAndRoundUp(UncroppedSize[1], uint16(2)));
						}
						--MipsToDrop;
					}

                    // Flat mesh UV raster
					TManagedPtr<FImage> ResultImage = CreateImage(SizeX, SizeY, 1, EImageFormat::L_UByte, EInitializationType::Black);
					if (Mesh)
					{
						ImageRasterMesh(Mesh.Get(), ResultImage.Get(), Args.LayoutIndex, Args.BlockId, CropMin, UncroppedSize);
					}

					// Stop execution.
					StoreImage(Item, ResultImage);
					break;
				}

				const int32 MipsToSkip = Item.ExecutionOptions;
				int32 ProjectionMip = MipsToSkip;

				FScheduledOpData Data;
				Data.RasterMesh.Mip = ProjectionMip;
				Data.RasterMesh.MipValue = static_cast<float>(ProjectionMip);
				FProjector Projector = LoadProjector(FScheduledOp::FromOpAndOptions(Args.projector, Item, 0));

				EMinFilterMethod MinFilterMethod = Invoke([&]() -> EMinFilterMethod
				{
					if (ForcedProjectionMode == 0)
					{
						return EMinFilterMethod::None;
					}
					else if (ForcedProjectionMode == 1)
					{
						return EMinFilterMethod::TotalAreaHeuristic;
					}
						
					return static_cast<EMinFilterMethod>(Args.MinFilterMethod);
				});

				if (MinFilterMethod == EMinFilterMethod::TotalAreaHeuristic)
				{
					FVector2f TargetImageSizeF = FVector2f(
						FMath::Max(Args.sizeX >> MipsToSkip, 1),
						FMath::Max(Args.sizeY >> MipsToSkip, 1));
					FVector2f SourceImageSizeF = FVector2f(Args.SourceSizeX, Args.SourceSizeY);
						
					if (Mesh)
					{ 
						const float ComputedMip = ComputeProjectedFootprintBestMip(Mesh.Get(), Projector, TargetImageSizeF, SourceImageSizeF);

						Data.RasterMesh.MipValue = FMath::Max(0.0f, ComputedMip + GlobalProjectionLodBias);
						Data.RasterMesh.Mip = static_cast<uint8>(FMath::FloorToInt32(Data.RasterMesh.MipValue));
					}
				}
		
				const int32 DataHeapAddress = HeapData.Add(Data);

				// Mesh is need again in the next stage, store it in the heap.
				HeapData[DataHeapAddress].Resource = Mesh;

				AddOp(FScheduledOp(Item.At, Item, 2, DataHeapAddress),
					FScheduledOp::FromOpAndOptions(Args.projector, Item, 0),
					FScheduledOp::FromOpAndOptions(Args.image, Item, Data.RasterMesh.Mip),
					FScheduledOp(Args.mask, Item),
					FScheduledOp::FromOpAndOptions(Args.angleFadeProperties, Item, 0));

				break;
			}

            case 2:
            {
				MUTABLE_CPUPROFILER_SCOPE(IM_RASTERMESH_2)

				if (!Args.image)
				{
					// This case is treated at the previous stage.
					check(false);
					StoreImage(Item, nullptr);
					break;
				}

				FScheduledOpData& Data = HeapData[Item.CustomState];

				// Unsafe downcast, should be fine as it is known to be a Mesh.
				TManagedPtr<const FMesh> Mesh = StaticCastManagedPtr<FMesh>(ConstCastManagedPtr<FResource>(Data.Resource));
				Data.Resource = nullptr;

				if (!Mesh)
				{
					check(false);
					StoreImage(Item, nullptr);
					break;
				}

				uint16 SizeX = Args.sizeX;
				uint16 SizeY = Args.sizeY;
				UE::Math::TIntVector2<uint16> CropMin(Args.CropMinX, Args.CropMinY);
				UE::Math::TIntVector2<uint16> UncroppedSize(Args.UncroppedSizeX, Args.UncroppedSizeY);

				// Drop mips while possible
				int32 MipsToDrop = Item.ExecutionOptions;
				bool bUseCrop = UncroppedSize[0] > 0;
				while (MipsToDrop && !(SizeX % 2) && !(SizeY % 2))
				{
					SizeX = FMath::Max(uint16(1),FMath::DivideAndRoundUp(SizeX, uint16(2)));
					SizeY = FMath::Max(uint16(1),FMath::DivideAndRoundUp(SizeY, uint16(2)));
					if (bUseCrop)
					{
						CropMin[0] = FMath::DivideAndRoundUp(CropMin[0], uint16(2));
						CropMin[1] = FMath::DivideAndRoundUp(CropMin[1], uint16(2));
						UncroppedSize[0] = FMath::Max(uint16(1), FMath::DivideAndRoundUp(UncroppedSize[0], uint16(2)));
						UncroppedSize[1] = FMath::Max(uint16(1), FMath::DivideAndRoundUp(UncroppedSize[1], uint16(2)));
					}
					--MipsToDrop;
				}

				// Raster with projection
				TManagedPtr<const FImage> Source = LoadImage(FCacheAddress(Args.image, Item.ExecutionIndex, Data.RasterMesh.Mip));

				TManagedPtr<const FImage> Mask = nullptr;
				if (Args.mask)
				{
					Mask = LoadImage(FCacheAddress(Args.mask, Item));

					// TODO: This shouldn't happen, but be defensive.
					FImageSize ResultSize(SizeX, SizeY);
					if (Mask && Mask->GetSize() != ResultSize)
					{
						MUTABLE_CPUPROFILER_SCOPE(ImageResize_MaskFixForProjection);

						TManagedPtr<FImage> Resized = CreateImage(SizeX, SizeY, Mask->GetLODCount(), Mask->GetFormat(), EInitializationType::NotInitialized);
						ImOp.ImageResizeLinear(Resized.Get(), 0, Mask.Get());
						Mask = Resized;
					}
				}

				float fadeStart = 180.0f;
				float fadeEnd = 180.0f;
				if ( Args.angleFadeProperties )
				{
					FVector4f fadeProperties = LoadColor(FScheduledOp::FromOpAndOptions(Args.angleFadeProperties, Item, 0));
					fadeStart = fadeProperties[0];
					fadeEnd = fadeProperties[1];
				}
				const float FadeStartRad = FMath::DegreesToRadians(fadeStart);
				const float FadeEndRad = FMath::DegreesToRadians(fadeEnd);

				EImageFormat Format = Source ? GetUncompressedFormat(Source->GetFormat()) : EImageFormat::L_UByte;

				if (Source && Source->GetFormat() != Format)
				{
					MUTABLE_CPUPROFILER_SCOPE(RunCode_RasterMesh_ReformatSource);
					TManagedPtr<FImage> Formatted = CreateImage(Source->GetSizeX(), Source->GetSizeY(), Source->GetLODCount(), Format, EInitializationType::NotInitialized);
					bool bSuccess = false;
					ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), Source.Get());
					check(bSuccess); 
					Source = Formatted;
				}
			
				EMinFilterMethod MinFilterMethod = Invoke([&]() -> EMinFilterMethod
				{
					if (ForcedProjectionMode == 0)
					{
						return EMinFilterMethod::None;
					}
					else if (ForcedProjectionMode == 1)
					{
						return EMinFilterMethod::TotalAreaHeuristic;
					}
						
					return static_cast<EMinFilterMethod>(Args.MinFilterMethod);
				});

				if (MinFilterMethod == EMinFilterMethod::TotalAreaHeuristic)
				{
					const uint16 Mip = Data.RasterMesh.Mip;
					const FImageSize ExpectedSourceSize = FImageSize(
							FMath::Max<uint16>(Args.SourceSizeX >> Mip, 1), 
							FMath::Max<uint16>(Args.SourceSizeY >> Mip, 1));

					if (Source->GetSize() != ExpectedSourceSize)
					{
						MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageRasterMesh_SizeFixup);	
						
						TManagedPtr<FImage> Resized = CreateImage(ExpectedSourceSize.X, ExpectedSourceSize.Y, 1, Format, EInitializationType::NotInitialized);
						ImOp.ImageResizeLinear(Resized.Get(), Settings.ImageCompressionQuality, Source.Get());
						
						Source = Resized;	
					}
				}

				// Allocate memory for the temporary buffers
				FScratchImageProject Scratch;
				Scratch.Vertices.SetNum(Mesh->GetVertexCount());
				Scratch.CulledVertex.SetNum(Mesh->GetVertexCount());

				ESamplingMethod SamplingMethod = Invoke([&]() -> ESamplingMethod
				{
					if (ForcedProjectionMode == 0)
					{
						return ESamplingMethod::Point;
					}
					else if (ForcedProjectionMode == 1)
					{
						return ESamplingMethod::BiLinear;
					}
					
					return static_cast<ESamplingMethod>(Args.SamplingMethod);
				});

				if (SamplingMethod == ESamplingMethod::BiLinear)
				{
					if (Source->GetLODCount() < 2 && Source->GetSizeX() > 1 && Source->GetSizeY() > 1)
					{
						MUTABLE_CPUPROFILER_SCOPE(RunCode_RasterMesh_BilinearMipGen);

						TManagedPtr<FImage> OwnedSource = CloneOrTakeOver(MoveTemp(Source));

						OwnedSource->DataStorage.SetNumLODs(2);
						ImageMipmapInPlace(0, OwnedSource.Get(), FMipmapGenerationSettings{});

						Source = OwnedSource;
					}
				}

				// Allocate new image after bilinear mip generation to reduce operation memory peak.
				TManagedPtr<FImage> New = CreateImage(SizeX, SizeY, 1, Format, EInitializationType::Black);

				if (Args.projector && Source && Source->GetSizeX() > 0 && Source->GetSizeY() > 0)
				{
					FProjector Projector = LoadProjector(FScheduledOp::FromOpAndOptions(Args.projector, Item, 0));

					switch (Projector.type)
					{
					case EProjectorType::Planar:
						ImageRasterProjectedPlanar(Mesh.Get(), New.Get(),
							Source.Get(), Mask.Get(),
							Args.bIsRGBFadingEnabled, Args.bIsAlphaFadingEnabled,
							SamplingMethod,
							FadeStartRad, FadeEndRad, FMath::Frac(Data.RasterMesh.MipValue),
							Args.LayoutIndex, Args.BlockId,
							CropMin, UncroppedSize,
							&Scratch, bUseProjectionVectorImpl);
						break;

					case EProjectorType::Wrapping:
						ImageRasterProjectedWrapping(Mesh.Get(), New.Get(),
							Source.Get(), Mask.Get(),
							Args.bIsRGBFadingEnabled, Args.bIsAlphaFadingEnabled,
							SamplingMethod,
							FadeStartRad, FadeEndRad, FMath::Frac(Data.RasterMesh.MipValue),
							Args.LayoutIndex, Args.BlockId,
							CropMin, UncroppedSize,
							&Scratch, bUseProjectionVectorImpl);
						break;

					case EProjectorType::Cylindrical:
						ImageRasterProjectedCylindrical(Mesh.Get(), New.Get(),
							Source.Get(), Mask.Get(),
							Args.bIsRGBFadingEnabled, Args.bIsAlphaFadingEnabled,
							SamplingMethod,
							FadeStartRad, FadeEndRad, FMath::Frac(Data.RasterMesh.MipValue),
							Args.LayoutIndex,
							Projector.projectionAngle,
							CropMin, UncroppedSize,
							&Scratch, bUseProjectionVectorImpl);
						break;

					default:
						check(false);
						break;
					}
				}

				StoreImage(Item, New);

                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_MAKEGROWMAP:
        {
			FOperation::ImageMakeGrowMapArgs Args = Program.GetOpArgs<FOperation::ImageMakeGrowMapArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1), FScheduledOp( Args.mask, Item) );
                break;

            case 1:
            {
                MUTABLE_CPUPROFILER_SCOPE(IM_MAKEGROWMAP_1)

                TManagedPtr<const FImage> Mask = LoadImage( FCacheAddress(Args.mask,Item) );

                TManagedPtr<FImage> Result = CreateImage( Mask->GetSizeX(), Mask->GetSizeY(), Mask->GetLODCount(), EImageFormat::L_UByte, EInitializationType::NotInitialized);

				TManagedPtr<FImage> OwnedMask = CloneOrTakeOver(MoveTemp(Mask));

                ImageMakeGrowMap(Result.Get(), OwnedMask.Get(), Args.border);
				Result->Flags |= FImage::IF_CANNOT_BE_SCALED;

                StoreImage( Item, Result);
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_DISPLACE:
        {
			FOperation::ImageDisplaceArgs Args = Program.GetOpArgs<FOperation::ImageDisplaceArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                        FScheduledOp( Args.Source, Item ),
                        FScheduledOp( Args.DisplacementMap, Item ) );
				break;

            case 1:
            {
                MUTABLE_CPUPROFILER_SCOPE(IM_DISPLACE_1)

                TManagedPtr<const FImage> Source = LoadImage( FCacheAddress(Args.Source,Item) );
                TManagedPtr<const FImage> pMap = LoadImage( FCacheAddress(Args.DisplacementMap,Item) );

				if (!Source)
				{
					StoreImage(Item, nullptr);
					break;
				}

				// TODO: This shouldn't happen: displacement maps cannot be scaled because their information
				// is resolution sensitive (pixel offsets). If the size doesn't match, scale the source, apply 
				// displacement and then unscale it.
				FImageSize OriginalSourceScale = Source->GetSize();
				if (OriginalSourceScale[0]>0 && OriginalSourceScale[1]>0 && OriginalSourceScale != pMap->GetSize())
				{
					MUTABLE_CPUPROFILER_SCOPE(ImageResize_EmergencyHackForDisplacementStep1);

					TManagedPtr<FImage> Resized = CreateImage(pMap->GetSizeX(), pMap->GetSizeY(), Source->GetLODCount(), Source->GetFormat(), EInitializationType::NotInitialized);
					ImOp.ImageResizeLinear(Resized.Get(), 0, Source.Get());
					Source = Resized;
				}

				// This works based on the assumption that displacement maps never read from a position they actually write to.
				// Since they are used for UV border expansion, this should always be the case.
				TManagedPtr<FImage> Result = CloneOrTakeOver(MoveTemp(Source));

				if (OriginalSourceScale[0] > 0 && OriginalSourceScale[1] > 0)
				{
					ImageDisplace(Result.Get(), Result.Get(), pMap.Get());

					if (OriginalSourceScale != Result->GetSize())
					{
						MUTABLE_CPUPROFILER_SCOPE(ImageResize_EmergencyHackForDisplacementStep2);
						TManagedPtr<FImage> Resized = CreateImage(OriginalSourceScale[0], OriginalSourceScale[1], Result->GetLODCount(), Result->GetFormat(), EInitializationType::NotInitialized);
						ImOp.ImageResizeLinear(Resized.Get(), 0, Result.Get());
						Result = Resized;
					}
				}

                StoreImage( Item, Result);
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::IM_TRANSFORM:
        {
            const FOperation::ImageTransformArgs Args = Program.GetOpArgs<FOperation::ImageTransformArgs>(Item.At);

            switch (Item.Stage)
            {
            case 0:
			{
				const TArray<FScheduledOp, TFixedAllocator<2>> Deps = 
				{
					FScheduledOp(Args.ScaleX, Item),
					FScheduledOp(Args.ScaleY, Item),
				};

                AddOp(FScheduledOp(Item.At, Item, 1), Deps);

				break;
			}
			case 1:
			{
            	MUTABLE_CPUPROFILER_SCOPE(IM_TRANSFORM_1)

				FVector2f Scale = FVector2f(
                        Args.ScaleX ? LoadScalar(FCacheAddress(Args.ScaleX, Item)) : 1.0f,
                        Args.ScaleY ? LoadScalar(FCacheAddress(Args.ScaleY, Item)) : 1.0f);
	
				using FUint16Vector2 = UE::Math::TIntVector2<uint16>;
				const FUint16Vector2 DestSizeI = Invoke([&]() 
				{
					int32 MipsToDrop = Item.ExecutionOptions;
					
					FUint16Vector2 Size = FUint16Vector2(
							Args.SizeX > 0 ? Args.SizeX : Args.SourceSizeX, 
							Args.SizeY > 0 ? Args.SizeY : Args.SourceSizeY); 

					while (MipsToDrop && !(Size.X % 2) && !(Size.Y % 2))
					{
						Size.X = FMath::Max(uint16(1), FMath::DivideAndRoundUp(Size.X, uint16(2)));
						Size.Y = FMath::Max(uint16(1), FMath::DivideAndRoundUp(Size.Y, uint16(2)));
						--MipsToDrop;
					}

					return FUint16Vector2(FMath::Max(Size.X, uint16(1)), FMath::Max(Size.Y, uint16(1)));
				});

				const FVector2f DestSize   = FVector2f(DestSizeI.X, DestSizeI.Y);
				const FVector2f SourceSize = FVector2f(FMath::Max(Args.SourceSizeX, uint16(1)), FMath::Max(Args.SourceSizeY, uint16(1)));

				FVector2f AspectCorrectionScale = FVector2f(1.0f, 1.0f);
				if (Args.bKeepAspectRatio)
				{
					const float DestAspectOverSrcAspect = (DestSize.X * SourceSize.Y) / (DestSize.Y * SourceSize.X);

					AspectCorrectionScale = DestAspectOverSrcAspect > 1.0f 
										  ? FVector2f(1.0f/DestAspectOverSrcAspect, 1.0f) 
										  : FVector2f(1.0f, DestAspectOverSrcAspect); 
				}
			
				const FTransform2f Transform = FTransform2f(FVector2f(-0.5f))
					.Concatenate(FTransform2f(FScale2f(Scale)))
					.Concatenate(FTransform2f(FScale2f(AspectCorrectionScale)))
					.Concatenate(FTransform2f(FVector2f(0.5f)));

				FBox2f NormalizedCropRect(ForceInit);
				NormalizedCropRect += Transform.TransformPoint(FVector2f(0.0f, 0.0f));
				NormalizedCropRect += Transform.TransformPoint(FVector2f(1.0f, 0.0f));
				NormalizedCropRect += Transform.TransformPoint(FVector2f(0.0f, 1.0f));
				NormalizedCropRect += Transform.TransformPoint(FVector2f(1.0f, 1.0f));

				const FVector2f ScaledSourceSize = NormalizedCropRect.GetSize() * DestSize;

				const float BestMip = 
					FMath::Log2(FMath::Max(1.0f, FMath::Square(SourceSize.GetMin()))) * 0.5f - 
				    FMath::Log2(FMath::Max(1.0f, FMath::Square(ScaledSourceSize.GetMin()))) * 0.5f;

				FScheduledOpData OpHeapData;
				OpHeapData.ImageTransform.SizeX = DestSizeI.X;
				OpHeapData.ImageTransform.SizeY = DestSizeI.Y;
				FPlatformMath::StoreHalf(&OpHeapData.ImageTransform.ScaleXEncodedHalf, Scale.X);
				FPlatformMath::StoreHalf(&OpHeapData.ImageTransform.ScaleYEncodedHalf, Scale.Y);
				OpHeapData.ImageTransform.MipValue = BestMip + GlobalImageTransformLodBias;

				const int32 HeapDataAddress = HeapData.Add(OpHeapData);

				const uint8 Mip = static_cast<uint8>(FMath::Max(0, FMath::FloorToInt(OpHeapData.ImageTransform.MipValue)));
				const TArray<FScheduledOp, TFixedAllocator<4>> Deps = 
				{
					FScheduledOp::FromOpAndOptions(Args.Base, Item, Mip),
					FScheduledOp(Args.OffsetX,  Item),
					FScheduledOp(Args.OffsetY,  Item),
					FScheduledOp(Args.Rotation, Item) 
				};
				
                AddOp(FScheduledOp(Item.At, Item, 2, HeapDataAddress), Deps);

				break;
			}
            case 2:
            {
				MUTABLE_CPUPROFILER_SCOPE(IM_TRANSFORM_2);
			
				const FScheduledOpData OpHeapData = HeapData[Item.CustomState];

				const uint8 Mip = static_cast<uint8>(FMath::Max(0, FMath::FloorToInt(OpHeapData.ImageTransform.MipValue)));
				TManagedPtr<const FImage> Source = LoadImage(FCacheAddress(Args.Base, Item.ExecutionIndex, Mip));

				const FVector2f Offset = FVector2f(
                        Args.OffsetX ? LoadScalar(FCacheAddress(Args.OffsetX, Item)) : 0.0f,
                        Args.OffsetY ? LoadScalar(FCacheAddress(Args.OffsetY, Item)) : 0.0f);

                FVector2f Scale = FVector2f(
						FPlatformMath::LoadHalf(&OpHeapData.ImageTransform.ScaleXEncodedHalf),
						FPlatformMath::LoadHalf(&OpHeapData.ImageTransform.ScaleYEncodedHalf));

				FVector2f AspectCorrectionScale = FVector2f(1.0f, 1.0f);
				if (Args.bKeepAspectRatio)
				{
					const FVector2f DestSize   = FVector2f(OpHeapData.ImageTransform.SizeX, OpHeapData.ImageTransform.SizeY);
					const FVector2f SourceSize = FVector2f(FMath::Max(Args.SourceSizeX, uint16(1)), FMath::Max(Args.SourceSizeY, uint16(1)));
					
					const float DestAspectOverSrcAspect = (DestSize.X * SourceSize.Y) / (DestSize.Y * SourceSize.X);
					
					AspectCorrectionScale = DestAspectOverSrcAspect > 1.0f 
										  ? FVector2f(1.0f/DestAspectOverSrcAspect, 1.0f) 
										  : FVector2f(1.0f, DestAspectOverSrcAspect); 
				}

				// Map Range [0..1] to a full rotation
                const float RotationRad = LoadScalar(FCacheAddress(Args.Rotation, Item)) * UE_TWO_PI;
	
				EImageFormat SourceFormat = Source->GetFormat();
				EImageFormat Format = GetUncompressedFormat(SourceFormat);

				if (Format != SourceFormat)
				{
					MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageTransform_FormatFixup);	
					TManagedPtr<FImage> Formatted = CreateImage(Source->GetSizeX(), Source->GetSizeY(), Source->GetLODCount(), Format, EInitializationType::NotInitialized);
					bool bSuccess = false;
					ImOp.ImagePixelFormat(bSuccess, Settings.ImageCompressionQuality, Formatted.Get(), Source.Get());
					check(bSuccess); 

					Source = Formatted;
				}

				const FImageSize ExpectedSourceSize = FImageSize(
						FMath::Max<uint16>(Args.SourceSizeX >> (uint16)Mip, 1), 
						FMath::Max<uint16>(Args.SourceSizeY >> (uint16)Mip, 1));
				if (Source->GetSize() != ExpectedSourceSize)
				{
					MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageTransform_SizeFixup);	
					
					TManagedPtr<FImage> Resized = CreateImage(ExpectedSourceSize.X, ExpectedSourceSize.Y, 1, Format, EInitializationType::NotInitialized);
					ImOp.ImageResizeLinear(Resized.Get(), Settings.ImageCompressionQuality, Source.Get());
					
					Source = Resized;	
				}

				if (Source->GetLODCount() < 2 && Source->GetSizeX() > 1 && Source->GetSizeY() > 1)
				{
					MUTABLE_CPUPROFILER_SCOPE(RunCode_ImageTransform_BilinearMipGen);

					TManagedPtr<FImage> OwnedSource = CloneOrTakeOver(MoveTemp(Source));
					OwnedSource->DataStorage.SetNumLODs(2);

					ImageMipmapInPlace(0, OwnedSource.Get(), FMipmapGenerationSettings{});

					Source = OwnedSource;
				}

				Scale.X = FMath::IsNearlyZero(Scale.X, UE_KINDA_SMALL_NUMBER) ? UE_KINDA_SMALL_NUMBER : Scale.X;
				Scale.Y = FMath::IsNearlyZero(Scale.Y, UE_KINDA_SMALL_NUMBER) ? UE_KINDA_SMALL_NUMBER : Scale.Y;

				AspectCorrectionScale.X = FMath::IsNearlyZero(AspectCorrectionScale.X, UE_KINDA_SMALL_NUMBER) 
									    ? UE_KINDA_SMALL_NUMBER 
										: AspectCorrectionScale.X;

				AspectCorrectionScale.Y = FMath::IsNearlyZero(AspectCorrectionScale.Y, UE_KINDA_SMALL_NUMBER) 
										? UE_KINDA_SMALL_NUMBER 
										: AspectCorrectionScale.Y;

				const FTransform2f Transform = FTransform2f(FVector2f(-0.5f))
						.Concatenate(FTransform2f(FScale2f(Scale)))
						.Concatenate(FTransform2f(FQuat2f(RotationRad)))
						.Concatenate(FTransform2f(FScale2f(AspectCorrectionScale)))
						.Concatenate(FTransform2f(Offset + FVector2f(0.5f)));

				const EAddressMode AddressMode = static_cast<EAddressMode>(Args.AddressMode);

				const EInitializationType InitType = AddressMode == EAddressMode::ClampToBlack 
											       ? EInitializationType::Black
											       : EInitializationType::NotInitialized;

				TManagedPtr<FImage> Result = CreateImage(
						OpHeapData.ImageTransform.SizeX, OpHeapData.ImageTransform.SizeY, 1, Format, InitType);

				const float MipFactor = FMath::Frac(FMath::Max(0.0f, OpHeapData.ImageTransform.MipValue));
				ImageTransform(Result.Get(), Source.Get(), Transform, MipFactor, AddressMode, bUseImageTransformVectorImpl);

				StoreImage(Item, Result);

                break;
            }

            default:
                check(false);
            }

			break;
		}

		case EOpType::IM_MATERIAL_BREAK:
		{
			const FOperation::MaterialBreakArgs Args = Program.GetOpArgs<FOperation::MaterialBreakArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Material, Item));
				break;
			}
			case 1:
			{
				TManagedPtr<const FMaterial> Material;

				bool bSuccess = GetMemory().GetMaterialIfSet(FCacheAddress(Args.Material, Item), Material);
				check(bSuccess);

				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];
				const FParameterKey ParameterKey = { ParameterName, Args.LayerIndex };
				const FMaterial::FImageParameterData* ImageParameterData = Material->ImageParameters.Find(ParameterKey);
				bool bParameterFound = false;

				if (ImageParameterData)
				{
					const TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>> ImageParameter = ImageParameterData->ImageParameter;
					bParameterFound = true;

					if (ImageParameter.IsType<FOperation::ADDRESS>())
					{
						FScheduledOp ItemNextStage(Item.At, Item, 2);
						ItemNextStage.CustomState = ImageParameter.Get<FOperation::ADDRESS>();

						AddOp(ItemNextStage,
							FScheduledOp(ImageParameter.Get<FOperation::ADDRESS>(), Item));
					}
					else
					{
						StoreImage(Item, ImageParameter.Get<TManagedPtr<const FImage>>());
					}
				}
				else if (const UMaterialInterface* MaterialInterface = Material->PassthroughObject.Get())
				{
					UTexture* ParameterValue = nullptr;
					FHashedMaterialParameterInfo ParameterInfo;
					ParameterInfo.Name = FScriptName(ParameterName);
					ParameterInfo.Index = (int32)ParameterKey.LayerIndex;
					ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

					// Find image using the parameter info
					bParameterFound = MaterialInterface->GetTextureParameterValue(ParameterInfo, ParameterValue);

					if (bParameterFound)
					{
						TPassthroughObjectPtr<UTexture> ExternalObject(ParameterValue);
						TManagedPtr<FImage> Result = FImage::CreateAsReference(ExternalObject, FImageDesc(), false);

						StoreImage(Item, Result);
					}
				}
				
				if (!bParameterFound)
				{
					GetLogger().LogError(TEXT("Expected Texture parameter [%s] not found in Break Material node."), *ParameterName.ToString());
					GetMemory().SetAborted(FCacheAddress(Item));
				}

				break;
			}
			case 2:
			{
				TManagedPtr<const FMaterial> Material = LoadMaterial(FCacheAddress(Args.Material, Item));
				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];
				const FParameterKey ParameterKey = { ParameterName, Args.LayerIndex };
				const FMaterial::FImageParameterData* ImageParameterData = Material->ImageParameters.Find(ParameterKey);

				if (!ImageParameterData)
				{
					StoreImage(Item, nullptr);
				}
				else
				{
					const TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>>& ImageParameter = ImageParameterData->ImageParameter;
					TManagedPtr<const FImage> pImage = LoadImage(FCacheAddress(ImageParameter.Get<FOperation::ADDRESS>(), Item));

					StoreImage(Item, pImage);
				}

				break;
			}

			default:
				unimplemented();
			}

			break;
		}

		case EOpType::IM_PARAMETER_CONVERT:
        {
        	FOperation::ImageParameterConvertArgs Args = Program.GetOpArgs<FOperation::ImageParameterConvertArgs>(Item.At);
        	switch (Item.Stage)
        	{
        	case 0:
        		AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp(Args.ImageParameter, Item));
        		break;

        	case 1:
        	{
        		// This has been moved to a task. It should have been intercepted in IssueOp.
        		check(false);
        	}

        	default:
        		check(false);
        	}

	        break;
        }
			
        default:
            if (type!=EOpType::NONE)
            {
                // Operation not implemented
                check( false );
            }
            break;
        }
    }	
	
    TSharedPtr<FRangeIndex> BuildCurrentOpRangeIndex(const FScheduledOp& Item, const FParameters& Params, const FModel& InModel, FProgramCache& ProgramCache, int32 ParameterIndex)
    {
        if (!Item.ExecutionIndex)
        {
            return nullptr;
        }

        // TODO: optimise to avoid allocating the index here, we could access internal
        // data directly.
		TSharedPtr<FRangeIndex> Index = Params.NewRangeIndex(ParameterIndex);
        if (!Index)
        {
            return nullptr;
        }

		const FProgram& Program = InModel.GetProgram();
		const FParameterDesc& ParamDesc = Program.Parameters[ParameterIndex];
        for (int32 RangeIndexInParam = 0; RangeIndexInParam < ParamDesc.Ranges.Num(); ++RangeIndexInParam)
        {
            uint32 RangeIndexInModel = ParamDesc.Ranges[RangeIndexInParam];
            
            int32 Position = ProgramCache.GetExecutionIndexValue(Item.ExecutionIndex, RangeIndexInModel);

			Index->Values[RangeIndexInParam] = Position;
        }

        return Index;
    }
	
	TSharedPtr<FRangeIndex> CodeRunner::BuildCurrentOpRangeIndex(const FScheduledOp& Item, int32 ParameterIndex)
	{
		return UE::Mutable::Private::BuildCurrentOpRangeIndex(Item, Parameters, Model, GetMemory(), ParameterIndex);
	}

    void CodeRunner::RunCode_Bool(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Bool);

		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {

        case EOpType::BO_CONSTANT:
        {
			FOperation::BoolConstantArgs Args = Program.GetOpArgs<FOperation::BoolConstantArgs>(Item.At);
            bool result = Args.bValue;
            StoreBool( Item, result );
            break;
        }

        case EOpType::BO_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
            bool result = false;
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
            result = Parameters.GetBoolValue( Args.variable, Index.Get() );
            StoreBool( Item, result );
            break;
        }

        case EOpType::BO_AND:
        {
			FOperation::BoolBinaryArgs Args = Program.GetOpArgs<FOperation::BoolBinaryArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				// Try to avoid the op entirely if we have some children cached
				bool bSkip = false;
				bool bAValue = false;
				if (Args.A && GetMemory().GetBoolIfSet(FCacheAddress(Args.A, Item), bAValue))
				{
					 if (!bAValue)
					 {
						StoreBool(Item, false);
						bSkip = true;
					 }
				}

				bool bBValue = false;
				if (!bSkip && Args.B && GetMemory().GetBoolIfSet(FCacheAddress(Args.B, Item), bBValue))
				{
					 if (!bBValue)
					 {
						StoreBool(Item, false);
						bSkip = true;
					 }
				}

				if (!bSkip)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						   FScheduledOp(Args.A, Item));
				}
				break;
			}

            case 1:
            {
                bool bAValue = LoadBool(FCacheAddress(Args.A, Item));
                if (!bAValue)
                {
                    StoreBool(Item, false);
                }
                else
                {
                    AddOp(FScheduledOp(Item.At, Item, 2),
                           FScheduledOp(Args.B, Item));
                }
                break;
            }

            case 2:
            {
                // We arrived here because a is true
                bool bBValue = LoadBool(FCacheAddress(Args.B, Item));
                StoreBool(Item, bBValue);
                break;
            }

            default:
                check(false);
            }
            break;
        }

        case EOpType::BO_OR:
        {
			FOperation::BoolBinaryArgs Args = Program.GetOpArgs<FOperation::BoolBinaryArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
			{
				// Try to avoid the op entirely if we have some children cached
				bool bSkip = false;

				bool bAValue = false;
				if (Args.A && GetMemory().GetBoolIfSet(FCacheAddress(Args.A, Item), bAValue))
				{
					 if (bAValue)
					 {
						StoreBool(Item, true);
						bSkip = true;
					 }
				}

				bool bBValue = false;
				if (!bSkip && Args.B && GetMemory().GetBoolIfSet(FCacheAddress(Args.B, Item), bBValue))
				{
					 if (bBValue)
					 {
						StoreBool(Item, true);
						bSkip = true;
					 }
				}

				if (!bSkip)
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
						   FScheduledOp(Args.A, Item));
				}
				break;
			}

            case 1:
            {
                bool bAValue = Args.A ? LoadBool(FCacheAddress(Args.A, Item)) : false;
                if (bAValue)
                {
                    StoreBool(Item, true);
                }
                else
                {
                    AddOp(FScheduledOp(Item.At, Item, 2),
                           FScheduledOp(Args.B, Item));
                }
                break;
            }

            case 2:
            {
                // We arrived here because a is false
                bool bBValue = Args.B ? LoadBool(FCacheAddress(Args.B, Item)) : false;
                StoreBool(Item, bBValue);
                break;
            }

            default:
                check(false);
            }
            break;
        }

        case EOpType::BO_NOT:
        {
			FOperation::BoolNotArgs Args = Program.GetOpArgs<FOperation::BoolNotArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
				AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.A, Item));
                break;

            case 1:
            {
                bool bResult = !LoadBool(FCacheAddress(Args.A, Item));
                StoreBool(Item, bResult);
                break;
            }

            default:
                check(false);
            }
            break;
        }

        case EOpType::BO_EQUAL_INT_CONST:
        {
			FOperation::BoolEqualScalarConstArgs Args = Program.GetOpArgs<FOperation::BoolEqualScalarConstArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Value, Item));
				break;
			}
			case 1:
            {
                int32 a = LoadInt(FCacheAddress(Args.Value, Item));
                bool result = a == Args.Constant;
                StoreBool( Item, result );
                break;
            }

            default:
                check(false);
            }
            break;
        }

        default:
            check( false );
            break;
        }
    }
	
    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Int(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Int);

		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {

        case EOpType::NU_CONSTANT:
        {
			FOperation::IntConstantArgs Args = Program.GetOpArgs<FOperation::IntConstantArgs>(Item.At);
            int32 result = Args.Value;
            StoreInt( Item, result );
            break;
        }

        case EOpType::NU_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
            int32 result = Parameters.GetIntValue( Args.variable, Index.Get());

            // Check that the value is actually valid. Otherwise set the default.
            if ( Parameters.GetIntPossibleValueCount( Args.variable ) )
            {
                bool valid = false;
                for (int32 ValueIndex = 0; !valid && ValueIndex < Parameters.GetIntPossibleValueCount(Args.variable); ++ValueIndex)
                {
                    if ( result == Parameters.GetIntPossibleValue( Args.variable, ValueIndex ))
                    {
                        valid = true;
                    }
                }

                if (!valid)
                {
                    result = Parameters.GetIntPossibleValue( Args.variable, 0 );
                }
            }

            StoreInt( Item, result );
            break;
        }

        default:
            check( false );
            break;
        }
    }

    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Scalar(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Scalar);

		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {

        case EOpType::SC_CONSTANT:
        {
			FOperation::ScalarConstantArgs Args = Program.GetOpArgs<FOperation::ScalarConstantArgs>(Item.At);
			float result = Args.Value;
            StoreScalar( Item, result );
            break;
        }

        case EOpType::SC_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
            float result = Parameters.GetFloatValue( Args.variable, Index.Get());
            StoreScalar( Item, result );
            break;
        }

        case EOpType::SC_CURVE:
        {
			FOperation::ScalarCurveArgs Args = Program.GetOpArgs<FOperation::ScalarCurveArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                AddOp( FScheduledOp( Item.At, Item, 1),
                        FScheduledOp( Args.time, Item) );
                break;

            case 1:
            {
                float time = LoadScalar( FCacheAddress(Args.time,Item) );

                const FRichCurve& Curve = Program.ConstantCurves[Args.curve];
                float Result = Curve.Eval(time);

                StoreScalar( Item, Result );
                break;
            }

            default:
                check(false);
            }
            break;
        }

        case EOpType::SC_ARITHMETIC:
        {
			FOperation::ArithmeticArgs Args = Program.GetOpArgs<FOperation::ArithmeticArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.A, Item),
                           FScheduledOp( Args.B, Item) );
                break;

            case 1:
            {
                float a = LoadScalar( FCacheAddress(Args.A,Item) );
                float b = LoadScalar( FCacheAddress(Args.B,Item) );

                float result = 1.0f;
                switch (Args.Operation)
                {
                case FOperation::ArithmeticArgs::ADD:
                    result = a + b;
                    break;

                case FOperation::ArithmeticArgs::MULTIPLY:
                    result = a * b;
                    break;

                case FOperation::ArithmeticArgs::SUBTRACT:
                    result = a - b;
                    break;

                case FOperation::ArithmeticArgs::DIVIDE:
                    result = a / b;
                    break;

                default:
                    checkf(false, TEXT("Arithmetic operation not implemented."));
                    break;
                }

                StoreScalar( Item, result );
                break;
            }

            default:
                check(false);
            }
            break;
        }

		case EOpType::SC_MATERIAL_BREAK:
		{
			const FOperation::MaterialBreakArgs Args = Program.GetOpArgs<FOperation::MaterialBreakArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Material, Item));
				break;
			}
			case 1:
			{
				TManagedPtr<const FMaterial> Material = LoadMaterial(FCacheAddress(Args.Material, Item));
				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];
				const FParameterKey ParameterKey = { ParameterName, Args.LayerIndex };
				const float* Value = Material->ScalarParameters.Find(ParameterKey);
				bool bParameterFound = false;

				if (Value)
				{
					bParameterFound = true;

					StoreScalar(Item, *Value);
				}
				else if (const UMaterialInterface* MaterialInterface = Material->PassthroughObject.Get())
				{
					float ParameterValue = 0.0f;

					FHashedMaterialParameterInfo ParameterInfo;
					ParameterInfo.Name = FScriptName(ParameterName);
					ParameterInfo.Index = (int32)ParameterKey.LayerIndex;
					ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

					// Find float using the parameter info
					bParameterFound = MaterialInterface->GetScalarParameterValue(ParameterInfo, ParameterValue);

					if (bParameterFound)
					{
						StoreScalar(Item, ParameterValue);
					}
				}

				if(!bParameterFound)
				{
					GetLogger().LogError(TEXT("Expected Scalar parameter [%s] not found in Break Material node."), *ParameterName.ToString());
					GetMemory().SetAborted(FCacheAddress(Item));
				}

				break;
			}

			default:
				unimplemented();
			}

			break;
		}

        default:
            check( false );
            break;
        }
    }

    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_String(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_String );

		EOpType type = Program.GetOpType( Item.At );
        switch ( type )
        {

        case EOpType::ST_CONSTANT:
        {
			FOperation::ResourceConstantArgs Args = Program.GetOpArgs<FOperation::ResourceConstantArgs>( Item.At );
            check( Args.value < (uint32)Program.ConstantStrings.Num() );

            const FString& Result = Program.ConstantStrings[Args.value];
            StoreString( Item, MakeManaged<String>(Result) );

            break;
        }

        case EOpType::ST_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>( Item.At );
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
			FString Result;
			Parameters.GetStringValue(Args.variable, Result, Index.Get());
            StoreString( Item, MakeManaged<String>(Result) );
            break;
        }

        default:
            check( false );
            break;
        }
    }


    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Color(const FScheduledOp& Item)
    {
		MUTABLE_CPUPROFILER_SCOPE(RunCode_Color);

		EOpType type = Program.GetOpType(Item.At);

        switch ( type )
        {

        case EOpType::CO_CONSTANT:
        {
			FOperation::ColorConstantArgs Args = Program.GetOpArgs<FOperation::ColorConstantArgs>(Item.At);
            StoreColor( Item, Args.Value );
            break;
        }

        case EOpType::CO_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
            FVector4f V;
            Parameters.GetColorValue( Args.variable, V, Index.Get());
            StoreColor( Item, V );
            break;
        }

        case EOpType::CO_SAMPLEIMAGE:
        {
			FOperation::ColorSampleImageArgs Args = Program.GetOpArgs<FOperation::ColorSampleImageArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.X, Item),
                           FScheduledOp( Args.Y, Item),
						   // Don't skip mips for the texture to sample
                           FScheduledOp::FromOpAndOptions( Args.Image, Item, 0) );
                break;

            case 1:
            {
                float x = Args.X ? LoadScalar( FCacheAddress(Args.X,Item) ) : 0.5f;
                float y = Args.Y ? LoadScalar( FCacheAddress(Args.Y,Item) ) : 0.5f;

                TManagedPtr<const FImage> pImage = LoadImage(FScheduledOp::FromOpAndOptions(Args.Image, Item, 0));

				FVector4f result;
                if (pImage)
                {
                    if (Args.Filter)
                    {
                        // TODO
                        result = pImage->Sample(FVector2f(x, y));
                    }
                    else
                    {
                        result = pImage->Sample(FVector2f(x, y));
                    }
                }
                else
                {
                    result = FVector4f();
                }

                StoreColor( Item, result );
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::CO_SWIZZLE:
        {
			FOperation::ColorSwizzleArgs Args = Program.GetOpArgs<FOperation::ColorSwizzleArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.sources[0], Item),
                           FScheduledOp( Args.sources[1], Item),
                           FScheduledOp( Args.sources[2], Item),
                           FScheduledOp( Args.sources[3], Item) );
                break;

            case 1:
            {
				FVector4f result;

                for (int32 t=0;t<MUTABLE_OP_MAX_SWIZZLE_CHANNELS;++t)
                {
                    if ( Args.sources[t] )
                    {
                        FVector4f p = LoadColor( FCacheAddress(Args.sources[t],Item) );
                        result[t] = p[ Args.sourceChannels[t] ];
                    }
                }

                StoreColor( Item, result );
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::CO_FROMSCALARS:
        {
			FOperation::ColorFromScalarsArgs Args = Program.GetOpArgs<FOperation::ColorFromScalarsArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.V[0], Item),
                           FScheduledOp( Args.V[1], Item),
                           FScheduledOp( Args.V[2], Item),
                           FScheduledOp( Args.V[3], Item));
                break;

            case 1:
            {
				FVector4f Result = FVector4f(0, 0, 0, 1);

				for (int32 t = 0; t < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++t)
				{
					if (Args.V[t])
					{
						Result[t] = LoadScalar(FCacheAddress(Args.V[t], Item));
					}
				}

                StoreColor( Item, Result );
                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::CO_ARITHMETIC:
        {
			FOperation::ArithmeticArgs Args = Program.GetOpArgs<FOperation::ArithmeticArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.A, Item),
                           FScheduledOp( Args.B, Item));
                break;

            case 1:
            {
				EOpType otype = Program.GetOpType( Args.A );
                EDataType dtype = GetOpDataType( otype );
                check( dtype == EDataType::Color );
                otype = Program.GetOpType( Args.B );
                dtype = GetOpDataType( otype );
                check( dtype == EDataType::Color );
				FVector4f a = Args.A ? LoadColor( FCacheAddress( Args.A, Item ) )
                                 : FVector4f( 0, 0, 0, 0 );
				FVector4f b = Args.B ? LoadColor( FCacheAddress( Args.B, Item ) )
                                 : FVector4f( 0, 0, 0, 0 );

				FVector4f result = FVector4f(0,0,0,0);
                switch (Args.Operation)
                {
                case FOperation::ArithmeticArgs::ADD:
                    result = a + b;
                    break;

                case FOperation::ArithmeticArgs::MULTIPLY:
                    result = a * b;
                    break;

                case FOperation::ArithmeticArgs::SUBTRACT:
                    result = a - b;
                    break;

                case FOperation::ArithmeticArgs::DIVIDE:
                    result = a / b;
                    break;

                default:
                    checkf(false, TEXT("Arithmetic operation not implemented."));
                    break;
                }

                StoreColor( Item, result );
                break;
            }

            default:
                check(false);
            }

            break;
        }

		case EOpType::CO_LINEARTOSRGB:
		{
			FOperation::ColorArgs Args = Program.GetOpArgs<FOperation::ColorArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Color, Item));
				break;

			case 1:
			{
				FVector4f Color = Args.Color ? LoadColor(FCacheAddress(Args.Color, Item))
					: FVector4f(0, 0, 0, 0);

				ConvertLinearColorToSRGB(Color);

				StoreColor(Item, Color);
				break;
			}

			default: unimplemented();
			}

			break;
		}

		case EOpType::CO_MATERIAL_BREAK:
		{
			const FOperation::MaterialBreakArgs Args = Program.GetOpArgs<FOperation::MaterialBreakArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Material, Item));
				break;
			}
			case 1:
			{
				TManagedPtr<const FMaterial> Material = LoadMaterial(FCacheAddress(Args.Material, Item));
				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];
				const FParameterKey ParameterKey = { ParameterName, Args.LayerIndex };
				const FVector4f* Value = Material->ColorParameters.Find(ParameterKey);
				bool bParameterFound = false;

				if (Value)
				{
					bParameterFound = true;

					StoreColor(Item, *Value);
				}
				else if (const UMaterialInterface* MaterialInterface = Material->PassthroughObject.Get())
				{
					FLinearColor ParameterValue;

					FHashedMaterialParameterInfo ParameterInfo;
					ParameterInfo.Name = FScriptName(ParameterName);
					ParameterInfo.Index = (int32)ParameterKey.LayerIndex;
					ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

					// Find float using the parameter info
					bParameterFound = MaterialInterface->GetVectorParameterValue(ParameterInfo, ParameterValue);

					if (bParameterFound)
					{
						StoreColor(Item, ParameterValue);
					}
				}

				if(!bParameterFound)
				{
					GetLogger().LogError(TEXT("Expected Color parameter [%s] not found in Break Material node."), *ParameterName.ToString());
					GetMemory().SetAborted(FCacheAddress(Item));
				}

				break;
			}

			default:
				unimplemented();
			}
			
			break;
		}

        default:
            check( false );
            break;
        }
    }


    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Projector(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Projector);

		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {

        case EOpType::PR_CONSTANT:
        {
			FOperation::ResourceConstantArgs Args = Program.GetOpArgs<FOperation::ResourceConstantArgs>(Item.At);
            FProjector Result = Program.ConstantProjectors[Args.value];
            StoreProjector( Item, Result );
            break;
        }

        case EOpType::PR_PARAMETER:
        {
			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
            FProjector Result = Parameters.GetProjectorValue(Args.variable,Index.Get());

            // The type cannot be changed, take it from the default value
            const FProjector& def = Program.Parameters[Args.variable].DefaultValue.Get<FParamProjectorType>();
            Result.type = def.type;

            StoreProjector( Item, Result );
            break;
        }

        default:
            check( false );
            break;
        }
    }

    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Matrix(const FScheduledOp& Item)
    {
		MUTABLE_CPUPROFILER_SCOPE(RunCode_Transform);

		EOpType type = Program.GetOpType(Item.At);

		switch ( type )
		{
		case EOpType::MA_CONSTANT:
			{
				FOperation::MatrixConstantArgs Args = Program.GetOpArgs<FOperation::MatrixConstantArgs>(Item.At);
				StoreMatrix( Item, Program.ConstantMatrices[Args.value] );
				break;
			}

		case EOpType::MA_PARAMETER:
			{
				FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
				TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex( Item, Args.variable );
				FMatrix44f Value;
				Parameters.GetMatrixValue( Args.variable, Value, Index.Get());
				StoreMatrix( Item, Value );
				break;
			}
		}
    }

	//---------------------------------------------------------------------------------------------
	void CodeRunner::RunCode_Material(const FScheduledOp& Item)
	{
		MUTABLE_CPUPROFILER_SCOPE(RunCode_Material);
		
		EOpType type = Program.GetOpType(Item.At);

		switch (type)
		{
		case EOpType::MI_PARAMETER:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_MI_PARAMETER);
			check(Item.Stage == 0);

			FOperation::MaterialParameterArgs Args = Program.GetOpArgs<FOperation::MaterialParameterArgs >(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Item, Args.Variable);

			UMaterialInterface* MaterialInterface = Parameters.GetMaterialValue(Args.Variable, Index.Get());
			TManagedPtr<FMaterial> Material = MakeManaged<FMaterial>();
			
			Material->PassthroughObject = TPassthroughObjectPtr(MaterialInterface);
 
			if (MaterialInterface)
			{
				// Get Name indexes for each type of parameters
				//Images
				const TArray<uint32>& ImageParameterNames = Program.ConstantUInt32Lists[Args.ImageParameterNames];
				const TArray<uint32>& ImageParameterOperations = Program.ConstantUInt32Lists[Args.ImageParameterAddress];

				// Get Parameter layers from program bytecode
				const uint8* Data = Program.GetOpArgsPointer(Item.At);
				Data += sizeof(FOperation::MaterialParameterArgs);

				// Textures
				for (int32 ParameterIndex = 0; ParameterIndex < ImageParameterNames.Num(); ++ParameterIndex)
				{
					// Get Image Parameter Name
					const FName& ParameterName = Program.ConstantNames[ImageParameterNames[ParameterIndex]];

					// Get Image Parameter Layer
					int8 LayerIndex;
					FMemory::Memcpy(&LayerIndex, Data, sizeof(int8));
					Data += sizeof(int8);

					// Construct image parameter info
					FHashedMaterialParameterInfo ImageParameterInfo;
					ImageParameterInfo.Name = FScriptName(ParameterName);
					ImageParameterInfo.Index = (int32)LayerIndex;
					ImageParameterInfo.Association = ImageParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

					// Find Texture using the parameter info
					UTexture* Texture = nullptr;
					bool bParameterFound = MaterialInterface->GetTextureParameterValue(ImageParameterInfo, Texture);

					// Store Texture into the FMaterial
					if (bParameterFound)
					{
						TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>> NewParameterImage;
						NewParameterImage.Set<FOperation::ADDRESS>(ImageParameterOperations[ParameterIndex]);

						FMaterial::FImageParameterData NewImageParameter;
						NewImageParameter.ImageParameter = NewParameterImage;

						Material->ImageParameters.Add({ParameterName, LayerIndex}, NewImageParameter);
					}
				}
			}

			StoreMaterial(Item, Material);

			break;
		}
			
		case EOpType::MI_MODIFY:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeMaterial_MI_MODIFY);
				
			FOperation::MaterialModifyArgs Args = Program.GetOpArgs<FOperation::MaterialModifyArgs>(Item.At);
			int32 ParameterCount = Args.NumParameters;

			//Dynamic data can't be stored into arg structures
			const uint8* Data = Program.GetOpArgsPointer(Item.At);
			Data += sizeof(FOperation::MaterialModifyArgs);

			switch (Item.Stage)
			{
			case 0:
			{
				TArray<FScheduledOp> Deps;

				// First, Schedule the material
				Deps.Emplace(Args.Material, Item);

				// Then, Schedule parameter operations
				for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
				{
					FOperation::ADDRESS NewName;
					FMemory::Memcpy(&NewName, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					int8 LayerIndex;
					FMemory::Memcpy(&LayerIndex, Data, sizeof(int8));
					Data += sizeof(int8);

					FOperation::ADDRESS NewParameterOperation;
					FMemory::Memcpy(&NewParameterOperation, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					// Evaluate non lazy branches here
					if (GetOpDataType(Program.GetOpType(NewParameterOperation)) != EDataType::Image)
					{
						Deps.Emplace(NewParameterOperation, Item);
					}
					else
					{
						// Jump Image Parameter Index
						Data += sizeof(int32);
					}
				}

				AddOp(FScheduledOp(Item.At, Item, 1), Deps);
				break;
			}
			case 1:
			{
				TArray<FOperation::ADDRESS> ParameterNames;
				TArray<int8> ParameterLayerIndex;
				TArray<FOperation::ADDRESS> ParameterOperations;
				TArray<int32> ImageParameterIndexArray;

				for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
				{
					FOperation::ADDRESS NewName;
					FMemory::Memcpy(&NewName, Data, sizeof(FOperation::ADDRESS));
					ParameterNames.Add(NewName);
					Data += sizeof(FOperation::ADDRESS);

					int8 LayerIndex;
					FMemory::Memcpy(&LayerIndex, Data, sizeof(int8));
					ParameterLayerIndex.Add(LayerIndex);
					Data += sizeof(int8);

					FOperation::ADDRESS NewParameterOperation;
					FMemory::Memcpy(&NewParameterOperation, Data, sizeof(FOperation::ADDRESS));
					ParameterOperations.Add(NewParameterOperation);
					Data += sizeof(FOperation::ADDRESS);

					// Evaluate lazy branches here
					if (GetOpDataType(Program.GetOpType(NewParameterOperation)) == EDataType::Image)
					{
						int32 ImageParameterIndex;
						FMemory::Memcpy(&ImageParameterIndex, Data, sizeof(int32));
						ImageParameterIndexArray.Add(ImageParameterIndex);
						Data += sizeof(int32);
					}
				}

				TManagedPtr<const FMaterial> Material = LoadMaterial(FCacheAddress(Args.Material, Item));
				TManagedPtr<FMaterial> Result;

				if (Material.IsValid())
				{
					Result = UE::Mutable::Private::CloneOrTakeOver<FMaterial>(MoveTemp(Material));
					int32 ImageIndex = 0;
				
					// ParameterIndex 0 is reserver for the material operation
					for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
					{
						// Get the parameter name from the constant string
						const FName& ParameterName = Program.ConstantNames[ParameterNames[ParameterIndex]];

						// Construct the parameter key
						FParameterKey ParameterKey = { ParameterName, ParameterLayerIndex[ParameterIndex] };
						EDataType OperationType = GetOpDataType(Program.GetOpType(ParameterOperations[ParameterIndex]));

						switch (OperationType)
						{
						case EDataType::Image:
						{
							//TODO: Evaluate the non-lazy textures
							TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>> NewParameterImage;
							NewParameterImage.Set<FOperation::ADDRESS>(ParameterOperations[ParameterIndex]);

							FMaterial::FImageParameterData ImageParameterData;
							ImageParameterData.ImageParameter = NewParameterImage;
							ImageParameterData.ImagePropertyIndex = ImageParameterIndexArray[ImageIndex];
							
							FMaterial::FImageParameterData& Value = Result->ImageParameters.FindOrAdd(ParameterKey);
							Value = ImageParameterData;

							ImageIndex++;
							break;
						}
						case EDataType::Color:
						{
							const FVector4f Color = LoadColor(FCacheAddress(ParameterOperations[ParameterIndex], Item));
							FVector4f& Value = Result->ColorParameters.FindOrAdd(ParameterKey);
							Value = Color;
							
							break;
						}
						case EDataType::Scalar:
						{
							float Scalar = LoadScalar(FCacheAddress(ParameterOperations[ParameterIndex], Item));
							float& Value = Result->ScalarParameters.FindOrAdd(ParameterKey);
							Value = Scalar;
							
							break;
						}
						case EDataType::None:
						{
							// If the Address is invalid, we remove the parameter from the material.

							if (Result->ImageParameters.Contains(ParameterKey))
							{
								Result->ImageParameters.Remove(ParameterKey);
								ImageIndex++;
							}
							else if (Result->ColorParameters.Contains(ParameterKey))
							{
								Result->ColorParameters.Remove(ParameterKey);
							}
							else if (Result->ScalarParameters.Contains(ParameterKey))
							{
								Result->ScalarParameters.Remove(ParameterKey);
							}

							break;
						}
						default:
							unimplemented();
						}
					}
				}

				StoreMaterial(Item, Result);
				break;
			}

			default:
				break;
			}
				
			break;
		}

		case EOpType::MI_SKELETALMESHOBJECT_BREAK:
		{
			MUTABLE_CPUPROFILER_SCOPE(MI_SKELETALMESHOBJECT_BREAK);

			FOperation::MaterialSkeletalMeshObjectBreakArgs Args = Program.GetOpArgs<FOperation::MaterialSkeletalMeshObjectBreakArgs >(Item.At);
			
			switch (Item.Stage)
			{
			case 0:
				{
					// Request the preparation of the skeletal mesh before doing any work
					AddOp( FScheduledOp( Item.At, Item, 1),
						   FScheduledOp( Args.SkeletalMeshObject, Item));
					break;
				}
			case 1:
				{
					TManagedPtr<const FSkeletalMesh> LoadedSkeletalMesh = LoadSkeletalMesh(FCacheAddress(Args.SkeletalMeshObject, Item));
					
					TPassthroughObjectPtr<USkeletalMesh> PassthroughMesh = LoadedSkeletalMesh->PassthroughObject;
					check(PassthroughMesh);
					
					TObjectPtr<UMaterialInterface> FoundMaterialInterface = nullptr;
					if (USkeletalMesh* EngineSkeletalMesh = PassthroughMesh.Get())
					{
						// From the Skeletal mesh get the material in the override slot with the provided name.
						const FName& SlotName = Program.ConstantNames[Args.SlotName];
						FSkeletalMaterial* FoundSlotNameMaterial = EngineSkeletalMesh->GetMaterials().FindByPredicate(
						[SlotName](const FSkeletalMaterial& Material)
						{
							return Material.MaterialSlotName == SlotName;
						});

						if (FoundSlotNameMaterial)
						{
							FoundMaterialInterface = FoundSlotNameMaterial->MaterialInterface;
						}
					}

					TManagedPtr<FMaterial> MutableMaterial = MakeManaged<FMaterial>();
					MutableMaterial->PassthroughObject = TPassthroughObjectPtr(FoundMaterialInterface.Get());

					StoreMaterial(Item, MutableMaterial);

					break;
				}
			
			default:
				check(false);
			}
			
			break;
		}
			
		case EOpType::MI_SKELETALMESH_BREAK:
		{
			MUTABLE_CPUPROFILER_SCOPE(MI_SKELETALMESH_BREAK);

			FOperation::MaterialSkeletalMeshBreakArgs Args = Program.GetOpArgs<FOperation::MaterialSkeletalMeshBreakArgs >(Item.At);
		
			const uint16 Options = SkeletalMeshOptionsPack(true, MAX_uint8, 0, 0, 0); // Do not schedule geometry.

			switch (Item.Stage)
			{
			case 0:
				{
					// Request the preparation of the skeletal mesh before doing any work
					AddOp(FScheduledOp( Item.At, Item, 1),
						FScheduledOp::FromOpAndOptions(Args.SkeletalMesh, Item, Options));
					break;
				}
			case 1:
				{
					const FName& SlotName = Program.ConstantNames[Args.SlotName];
			
					const TManagedPtr<const FSkeletalMesh> SkeletalMesh = LoadSkeletalMesh(FScheduledOp::FromOpAndOptions(Args.SkeletalMesh, Item, Options));
					if (SkeletalMesh)
					{
						const int32 Index = SkeletalMesh->MaterialSlotNames.Find(SlotName);
						if (Index != INDEX_NONE)
						{
							const TVariant<FOperation::ADDRESS, TManagedPtr<const FMaterial>>& LazyMaterial = SkeletalMesh->MaterialSlotMaterials[Index];
							if (LazyMaterial.IsType<TManagedPtr<const FMaterial>>())
							{
								StoreMaterial(Item, LazyMaterial.Get<TManagedPtr<const FMaterial>>());
							}
							else
							{
								FScheduledOp NewItem = Item;
								NewItem.CustomState = LazyMaterial.Get<FOperation::ADDRESS>();
								NewItem.Stage = 2;
								
								AddOp(NewItem,
									FScheduledOp(LazyMaterial.Get<FOperation::ADDRESS>(), Item));
							}
						}
						else
						{
							StoreMaterial(Item, nullptr);
						}
					}
					else
					{
						StoreMaterial(Item, nullptr);
					}
					
					break;
				}
			case 2:
				{
					const FOperation::ADDRESS Address = Item.CustomState;
					
					TManagedPtr<const FMaterial> Material = LoadMaterial(FCacheAddress(Address, Item));
					StoreMaterial(Item, Material);
					break;
				}

			default:
				unimplemented();
			}
		
			break;
		}
			
		default: 
			unimplemented();
		}
	}


    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode_Layout(const FScheduledOp& Item)
    {
        MUTABLE_CPUPROFILER_SCOPE(RunCode_Layout);

		check(Item.ExecutionOptions == 0);

		EOpType type = Program.GetOpType(Item.At);
        switch (type)
        {

        case EOpType::LA_CONSTANT:
        {
			FOperation::ResourceConstantArgs Args = Program.GetOpArgs<FOperation::ResourceConstantArgs>(Item.At);
            check( Args.value < (uint32)Program.ConstantLayouts.Num() );

            TManagedPtr<const FLayout> Result = Program.ConstantLayouts[ Args.value ];
            StoreLayout(Item, Result);
            break;
        }

        case EOpType::LA_MERGE:
        {
			FOperation::LayoutMergeArgs Args = Program.GetOpArgs<FOperation::LayoutMergeArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.Base, Item),
                           FScheduledOp( Args.Added, Item) );
                break;

            case 1:
            {
				MUTABLE_CPUPROFILER_SCOPE(RunCode_Layout_LA_MERGE)
                TManagedPtr<const FLayout> pA = LoadLayout( FCacheAddress(Args.Base,Item) );
                TManagedPtr<const FLayout> pB = LoadLayout( FCacheAddress(Args.Added,Item) );

                TManagedPtr<const FLayout> Result;

                if (pA && pB)
                {
					Result = LayoutMerge(pA.Get(), pB.Get());
                }
                else if (pA)
                {
                    Result = pA->Clone();
                }
                else if (pB)
                {
                    Result = pB->Clone();
                }

				StoreLayout(Item, Result);

                break;
            }

            default:
                check(false);
            }

            break;
        }

        case EOpType::LA_PACK:
        {
			FOperation::LayoutPackArgs Args = Program.GetOpArgs<FOperation::LayoutPackArgs>(Item.At);
            switch (Item.Stage)
            {
            case 0:
                    AddOp( FScheduledOp( Item.At, Item, 1),
                           FScheduledOp( Args.Source, Item) );
                break;

            case 1:
            {
				MUTABLE_CPUPROFILER_SCOPE(RunCode_Layout_LA_PACK)
                TManagedPtr<const FLayout> Source = LoadLayout( FCacheAddress(Args.Source,Item) );

				TManagedPtr<FLayout> Result;

				if (Source)
				{
					Result = Source->Clone();

					LayoutPack3(Result.Get(), Source.Get() );
				}
                StoreLayout( Item, Result );
                break;
            }

            default:
                check(false);
            }

            break;
        }

		case EOpType::LA_FROMMESH:
		{

			FOperation::LayoutFromMeshArgs Args = Program.GetOpArgs<FOperation::LayoutFromMeshArgs>(Item.At);

			constexpr uint8 MeshContentFilter = (uint8)(EMeshContentFlags::AllFlags);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp::FromOpAndOptions(Args.Mesh, Item, MeshContentFilter));
				break;

			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(RunCode_Layout_LA_FROMMESH)

				TManagedPtr<const FMesh> Mesh = LoadMesh(
						FScheduledOp::FromOpAndOptions(Args.Mesh, Item, MeshContentFilter));

				TManagedPtr<const FLayout> Result = LayoutFromMesh_RemoveBlocks(Mesh.Get(), Args.LayoutIndex);
				StoreLayout(Item, Result);

				if (Result)
				{
					GetMemory().MarkAddressToKeep(Item);
				}

				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::LA_REMOVEBLOCKS:
		{
			FOperation::LayoutRemoveBlocksArgs Args = Program.GetOpArgs<FOperation::LayoutRemoveBlocksArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Source, Item),
					FScheduledOp(Args.ReferenceLayout, Item));
				break;

			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(RunCode_Layout_LA_REMOVEBLOCKS)
				TManagedPtr<const FLayout> Source = LoadLayout(FCacheAddress(Args.Source, Item));
				TManagedPtr<const FLayout> ReferenceLayout = LoadLayout(FCacheAddress(Args.ReferenceLayout, Item));

				TManagedPtr<const FLayout> Result;

				if (Source && ReferenceLayout)
				{
					Result = LayoutRemoveBlocks(Source.Get(), ReferenceLayout.Get());
				}
				else if (Source)
				{
					Result = Source;
				}

				StoreLayout(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}

        default:
            // Operation not implemented
            check( false );
            break;
        }
    }


    //---------------------------------------------------------------------------------------------
    void CodeRunner::RunCode(const FScheduledOp& Item)
    {
		//UE_LOGF( LogMutableCore, Log, "Running :%5d , %d ", Item.At, Item.Stage );
		check(Item.Type == static_cast<uint16>(FScheduledOp::EType::Full));

		EOpType type = Program.GetOpType(Item.At);
		//UE_LOGF(LogMutableCore, Log, "Running :%5d , %d, of type %d ", Item.At, Item.Stage, type);

		// Very spammy, for debugging purposes.
		//if (System)
		//{
		//	System.WorkingMemoryManager.LogWorkingMemory( this );
		//}

		switch ( type )
        {
        case EOpType::NONE:
            break;

        case EOpType::NU_CONDITIONAL:
        case EOpType::SC_CONDITIONAL:
        case EOpType::CO_CONDITIONAL:
        case EOpType::IM_CONDITIONAL:
        case EOpType::ME_CONDITIONAL:
        case EOpType::LA_CONDITIONAL:
        case EOpType::IN_CONDITIONAL:
		case EOpType::ED_CONDITIONAL:
		case EOpType::MI_CONDITIONAL:
		case EOpType::SK_CONDITIONAL:
		case EOpType::LD_CONDITIONAL:
		case EOpType::IS_CONDITIONAL:
            RunCode_Conditional(Item);
            break;

		case EOpType::ED_CONSTANT:
		case EOpType::MI_CONSTANT:
            RunCode_ConstantResource(Item);
            break;

        case EOpType::NU_SWITCH:
        case EOpType::SC_SWITCH:
        case EOpType::CO_SWITCH:
        case EOpType::IM_SWITCH:
        case EOpType::ME_SWITCH:
        case EOpType::LA_SWITCH:
        case EOpType::IN_SWITCH:
		case EOpType::ED_SWITCH:
		case EOpType::MI_SWITCH:
		case EOpType::IS_SWITCH:
		case EOpType::SK_SWITCH:
		case EOpType::LD_SWITCH:
            RunCode_Switch(Item);
            break;

        case EOpType::IN_ADDOVERLAYMATERIAL:
        case EOpType::IN_ADDOVERRIDEMATERIAL:
    	case EOpType::IN_ADDSKELETALMESH:
            RunCode_InstanceAddResource(Item);
            break;

		case EOpType::SC_EXTERNAL:
		case EOpType::CO_EXTERNAL:
		case EOpType::IM_EXTERNAL:
		case EOpType::ME_EXTERNAL:
		case EOpType::MI_EXTERNAL:
		case EOpType::IS_EXTERNAL:
			{
				const uint8* Data = Program.GetOpArgsPointer(Item.At);

				FOperation::ExternalArgs Args = Program.GetOpArgs<FOperation::ExternalArgs>(Item.At);
				Data += sizeof(FOperation::ExternalArgs);
				
				const FInstancedStruct& OperationInstancedStruct = Program.ExternalOperationProvider->Get(Item.At);
				if (!OperationInstancedStruct.IsValid()) // Plugin not loaded
				{
					GetLogger().LogError(TEXT("Trying to evaluate an External Operation which has not been loaded."));
					GetMemory().SetAborted(FCacheAddress(Item));

					break;
				}
					
				const FExternalOperation* Operation = OperationInstancedStruct.GetPtr<FExternalOperation>();

				const TArray<TPair<FText, const UScriptStruct*>>& Inputs = Operation->GetInputs();
				if (Inputs.Num() != Args.NumOperants)
				{
					GetLogger().LogError(TEXT("External Operation inputs do not match compiled data: %s"), *OperationInstancedStruct.GetScriptStruct()->GetName());
					GetMemory().SetAborted(FCacheAddress(Item));

					return;
				}
				
				switch (Item.Stage)
				{
				case 0:
					{
						TArray<FScheduledOp, TInlineAllocator<8>> Dependencies;

						for (int32 Index = 0; Index < Args.NumOperants; ++Index)
						{
							FOperation::ADDRESS OperantAt;
							FMemory::Memcpy(&OperantAt, Data, sizeof(FOperation::ADDRESS));
							Data += sizeof(FOperation::ADDRESS);

							Dependencies.Add(FScheduledOp(OperantAt, Item));
						}

						AddOp(FScheduledOp(Item.At, Item, 1), Dependencies);
						
						break;
					}

				case 1:
					{
						FContext Context;
						
						// Inputs
						for (int32 Index = 0; Index < Args.NumOperants; ++Index)
						{
							FOperation::ADDRESS OperantAt;
							FMemory::Memcpy(&OperantAt, Data, sizeof(FOperation::ADDRESS));
							Data += sizeof(FOperation::ADDRESS);

							const TPair<FText, const UScriptStruct*>& Input = Inputs[Index];

							const EOpType InputOpType = Program.GetOpType(OperantAt);
							const EDataType InputDataType = GetOpDataType(InputOpType);
							switch (InputDataType)
							{
							case EDataType::Color:
								{
									const FVector4f Vector = LoadColor(FCacheAddress(OperantAt, Item));
									
									FValue Value;
									Value.Ptr = MakeManaged<FInstancedStruct>();
									Value.Ptr->InitializeAs<FVectorAdapter>().Value = Vector;

									Context.Inputs.Add(Input.Key.ToString(), MoveTemp(Value));
									break;
								}
								
							case EDataType::Image:
								{
									const TManagedPtr<const FImage> Image = LoadImage(FCacheAddress(OperantAt, Item));
									
									FValue Value;
									Value.Ptr = MakeManaged<FInstancedStruct>();
									Value.Ptr->InitializeAs<FTextureAdapter>().Image = ConstCastManagedPtr<FImage>(Image);
									
									Context.Inputs.Add(Input.Key.ToString(), MoveTemp(Value));
									break;
								}

							case EDataType::Mesh:
								{
									const TManagedPtr<const FMesh> Mesh = LoadMesh(FCacheAddress(OperantAt, Item));

									FValue Value;
									Value.Ptr = MakeManaged<FInstancedStruct>();
									Value.Ptr->InitializeAs<FMeshAdapter>().Mesh = ConstCastManagedPtr<FMesh>(Mesh);

									Context.Inputs.Add(Input.Key.ToString(), MoveTemp(Value));
									break;
								}
								
							case EDataType::Material:
								{
									const TManagedPtr<const FMaterial> Material = LoadMaterial(FCacheAddress(OperantAt, Item));

									FValue Value;
									Value.Ptr = MakeManaged<FInstancedStruct>();
									Value.Ptr->InitializeAs<FMaterialAdapter>().Material = ConstCastManagedPtr<FMaterial>(Material);

									Context.Inputs.Add(Input.Key.ToString(), MoveTemp(Value));
									break;
								}
								
							case EDataType::InstancedStruct:
								{
									const TManagedPtr<const FInstancedStruct> InstancedStruct = LoadInstancedStruct(FCacheAddress(OperantAt, Item));
									
									FValue Value;
									Value.Ptr = MakeManaged<FInstancedStruct>();
									Value.Ptr = ConstCastManagedPtr<FInstancedStruct>(InstancedStruct);
									
									Context.Inputs.Add(Input.Key.ToString(), MoveTemp(Value));
									break;
								}
									
							case EDataType::Scalar:
								{
									const float Float = LoadScalar(FCacheAddress(OperantAt, Item));

									FValue Value;
									Value.Ptr = MakeManaged<FInstancedStruct>();
									Value.Ptr->InitializeAs<FFloatAdapter>().Value = Float;

									Context.Inputs.Add(Input.Key.ToString(), MoveTemp(Value));
									break;
								}
									
							default: 
								unimplemented();
							}
						}

						// Evaluate
						Operation->Evaluate(Context);

						if (!Context.Output.Ptr)
						{
							GetLogger().LogError(TEXT("External Operation output not set: %s"), *OperationInstancedStruct.GetScriptStruct()->GetName());
							GetMemory().SetAborted(FCacheAddress(Item));
							return;
						}
						
						// Store output
						const EDataType OuputDataType = GetOpDataType(type);
						switch (OuputDataType)
						{
						case EDataType::Color:
							StoreColor(FCacheAddress(Item.At, Item), Context.Output.Get<FVectorAdapter>().Value);
							break;
							
						case EDataType::Image:
							StoreImage(FCacheAddress(Item.At, Item), Context.Output.Get<FTextureAdapter>().Image);
							break;
							
						case EDataType::Mesh:
							StoreMesh(FCacheAddress(Item.At, Item), Context.Output.Get<FMeshAdapter>().Mesh);
							break;
							
						case EDataType::Material:
						{
							const TManagedPtr<FMaterial> OutputMaterial = Context.Output.Get<FMaterialAdapter>().Material;
							StoreMaterial(FCacheAddress(Item.At, Item), OutputMaterial ? OutputMaterial :  MakeManaged<FMaterial>());
							break;
						}
						
						case EDataType::InstancedStruct:
							StoreInstancedStruct(FCacheAddress(Item.At, Item), Context.Output.Ptr);
							break;
									
						case EDataType::Scalar:
							StoreScalar(FCacheAddress(Item.At, Item), Context.Output.Get<FFloatAdapter>().Value);
							break;
									
						default: 
							unimplemented();
						}

						break;
					}

				default:
					unimplemented();
				}

				break;	
			}

		case EOpType::IS_PARAMETER:
			{
				FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
				TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Item, Args.variable);
				TManagedPtr<const FInstancedStruct> Result = Parameters.GetInstancedStructValue(Args.variable, Index.Get());
				StoreInstancedStruct(Item, Result);

				break;
			}

		case EOpType::SK_PARAMETER:
			{
				FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
				TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Item, Args.variable);
				USkeletalMesh* SkeletalMesh = Parameters.GetSkeletalMeshValue(Args.variable, Index.Get());

				TManagedPtr<FSkeletalMesh> Result = MakeManaged<FSkeletalMesh>();
				Result->PassthroughObject = TPassthroughObjectPtr<USkeletalMesh>(SkeletalMesh);
        		
				StoreSkeletalMesh(Item, Result);
				break;
			}

		case EOpType::SK_MERGE:
		{
			FOperation::FSkeletalMeshMergeArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshMergeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.BaseMesh, Item),
					FScheduledOp(Args.AddedMesh, Item));
				break;
			}
			case 1:
			{
				MUTABLE_CPUPROFILER_SCOPE(SK_MERGE_1);

				TManagedPtr<const FSkeletalMesh> BaseMesh = LoadSkeletalMesh(FCacheAddress(Args.BaseMesh, Item));
				TManagedPtr<const FSkeletalMesh> AddedMesh = LoadSkeletalMesh(FCacheAddress(Args.AddedMesh, Item));

				if (BaseMesh && AddedMesh)
				{
					TManagedPtr<FSkeletalMesh> Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(BaseMesh));
					TManagedPtr<FSkeletalMesh> Added = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(AddedMesh));

					SkeletalMeshMerge(*Result.Get(), *Added.Get());

					StoreSkeletalMesh(Item, Result);
				}
				else
				{
					StoreSkeletalMesh(Item, BaseMesh ? BaseMesh : AddedMesh);
				}

				break;
			}
			default:
				check(false);
			}

			break;
		}
		case EOpType::LD_NEW:
			{
				MUTABLE_CPUPROFILER_SCOPE(LD_NEW);
        		
				FOperation::FLODNewArgs Args = Program.GetOpArgs<FOperation::FLODNewArgs>(Item.At);

				const TArray<FOperation::ADDRESS>& MeshAddresses = Program.ConstantUInt32Lists[Args.Meshes];
        		
				switch (Item.Stage)
				{
				case 0:
					{
						TArray<FScheduledOp> Deps;
						for (FOperation::ADDRESS MeshAddress : MeshAddresses)
						{
							Deps.Emplace(MeshAddress, Item);
						}

						AddOp(FScheduledOp( Item.At, Item, 1), Deps);
						break;
					}

				case 1:
					{
						TManagedPtr<FLOD> Result = MakeManaged<FLOD>();
            		
						for (FOperation::ADDRESS MeshAddress : MeshAddresses)
						{
							TManagedPtr<const FMesh> Mesh = LoadMesh(FCacheAddress(MeshAddress, Item));
							Result->Meshes.Add(Mesh);
						}
            		
						StoreLOD(Item, Result);
						break;
					}

				default:
					check(false);
				}

				break;
			}
			
		case EOpType::SK_NEW:
			{
				MUTABLE_CPUPROFILER_SCOPE(SK_NEW);
        		
				FOperation::FSkeletalMeshNewArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshNewArgs>(Item.At);
				
				const TArray<FOperation::ADDRESS>& MaterialSlotMaterials = Program.ConstantUInt32Lists[Args.MaterialSlotMaterials];
				const TArray<FOperation::CONSTANT_NAME>& MaterialSlotNames = Program.ConstantUInt32Lists[Args.MaterialSlotNames];
				const TArray<uint32>& MaterialSlotIds = Program.ConstantUInt32Lists[Args.MaterialSlotIds];
				const TArray<FOperation::ADDRESS>& LODs = Program.ConstantUInt32Lists[Args.LODs];
				
				const bool bForceGeometryGeneration = CVarForceGeometryOnFirstGeneration.GetValueOnAnyThread();
				
				bool bInitialGeneration;
				uint8 ItemLODIndex;
				uint8 ItemFirstLODResident;
				uint8 ItemFirstLODAvailable;
				uint8 ItemNumLODs;
				SkeletalMeshOptionsUnpack(Item.ExecutionOptions, bInitialGeneration, ItemLODIndex, ItemFirstLODResident, ItemFirstLODAvailable, ItemNumLODs);

				const uint8 NumLODs = FMath::Min(ItemNumLODs, static_cast<uint8>(LODs.Num()));
				const uint8 FirstLODAvailable = ItemFirstLODAvailable;
				const uint8 FirstLODResident = ItemFirstLODResident;
				
				switch (Item.Stage)
				{
				case 0:
					{
						TArray<FScheduledOp> Deps;

						if (bInitialGeneration)
						{
							for (uint8 LODIndex = FirstLODAvailable; LODIndex < NumLODs; ++LODIndex)
							{
								EMeshContentFlags MeshContentFilter = EMeshContentFlags::AllFlags; 
								if (!bForceGeometryGeneration)
								{
									if (LODIndex < FirstLODResident) // If resident
									{
										EnumRemoveFlags(MeshContentFilter, EMeshContentFlags::GeometryData);
									}
								}
								
								FScheduledOp LODOp(LODs[LODIndex], Item);
								LODOp.ExecutionOptions = static_cast<uint8>(MeshContentFilter);
								
								Deps.Add(LODOp);
							}

						}
						else if(LODs.IsValidIndex(ItemLODIndex))
						{
							FScheduledOp LODOp(LODs[ItemLODIndex], Item);
							LODOp.ExecutionOptions = static_cast<uint8>(EMeshContentFlags::AllFlags);
							
							Deps.Add(LODOp);
						}
						
						AddOp(FScheduledOp(Item.At, Item, 1), Deps);
						break;
					}

				case 1:
					{
						TManagedPtr<FSkeletalMesh> Result = MakeManaged<FSkeletalMesh>();
						Result->LODs.SetNum(NumLODs);
							
						if (bInitialGeneration)
						{
							for (int32 MaterialIndex = 0; MaterialIndex < MaterialSlotMaterials.Num(); ++MaterialIndex)
							{
								const FOperation::ADDRESS MaterialAddress = MaterialSlotMaterials[MaterialIndex];
							
								TVariant<FOperation::ADDRESS, TManagedPtr<const FMaterial>> LazyMaterial;
								LazyMaterial.Set<FOperation::ADDRESS>(MaterialAddress);
							
								Result->MaterialSlotMaterials.Add(LazyMaterial);
							
								const FOperation::CONSTANT_NAME MaterialNameIndex = MaterialSlotNames[MaterialIndex];
								const FName& Name = Program.ConstantNames[MaterialNameIndex];
								Result->MaterialSlotNames.Add(Name);
							
								const uint32 Id = MaterialSlotIds[MaterialIndex];
								Result->MaterialSlotIds.Add(Id);
							}
							
							for (uint8 LODIndex = FirstLODAvailable; LODIndex < NumLODs; ++LODIndex)
							{
								const FOperation::ADDRESS LODAddress = LODs[LODIndex];

								EMeshContentFlags MeshContentFilter = EMeshContentFlags::AllFlags; 
								if (!bForceGeometryGeneration)
								{
									if (LODIndex < FirstLODResident) // If resident
									{
										EnumRemoveFlags(MeshContentFilter, EMeshContentFlags::GeometryData);
									}
								}
								
								TManagedPtr<const FLOD> SkeletalMeshLOD = LoadLOD(FCacheAddress(LODAddress, Item.ExecutionIndex, static_cast<uint8>(MeshContentFilter)));
								Result->LODs[LODIndex] = SkeletalMeshLOD;
							}
						}
						else if (LODs.IsValidIndex(ItemLODIndex))
						{
							const FOperation::ADDRESS LODAddress = LODs[ItemLODIndex];

							TManagedPtr<const FLOD> SkeletalMeshLOD = LoadLOD(FCacheAddress(LODAddress, Item.ExecutionIndex, static_cast<uint8>(EMeshContentFlags::AllFlags)));
							Result->LODs[ItemLODIndex] = SkeletalMeshLOD;
						}

						StoreSkeletalMesh(Item, Result);
						break;
					}

				default:
					check(false);
				}

				break;
			}
			
		case EOpType::SKO_CONVERT:
			{
				MUTABLE_CPUPROFILER_SCOPE(SKO_CONVERT);
        		
				FOperation::FSkeletalMeshObjectConvertArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshObjectConvertArgs>(Item.At);

				bool bInitialGeneration;
				bool bStreamMeshLODs;
				uint8 LODIndex;
				SkeletalMeshObjectOptionsUnpack(Item.ExecutionOptions, bInitialGeneration, bStreamMeshLODs, LODIndex);
				
				int8 FirstLODAvailable = FMath::Clamp(Args.FirstLODAvailable + MutableMeshesLODBias, Args.FirstLODAvailable, Args.NumLODs - 1);
				
				int32 FirstLODResident = bStreamMeshLODs ? Args.FirstLODResident : FirstLODAvailable;
				FirstLODResident = FMath::Max(FirstLODResident, FirstLODAvailable);
				
				const uint16 Options = SkeletalMeshOptionsPack(bInitialGeneration, LODIndex, FirstLODResident, FirstLODAvailable, Args.NumLODs);
				
				switch (Item.Stage)
				{
				case 0:
				{
					AddOp(FScheduledOp(Item.At, Item, 1), FScheduledOp::FromOpAndOptions(Args.SkeletalMesh, Item, Options));
					break;
				}

				case 1:
				{
					TManagedPtr<const FSkeletalMesh> SkeletalMesh = LoadSkeletalMesh(FScheduledOp::FromOpAndOptions(Args.SkeletalMesh, Item, Options));
					

					auto MergeLODMeshes = [this, bInitialGeneration](FLOD& LOD)
						{
							if (CVarUseMergeLODMeshesForConversion.GetValueOnAnyThread())
							{

								int32 FinalMeshSize = 0;
								for (const TManagedPtr<const FMesh>& Mesh : LOD.Meshes)
								{
									if (Mesh)
									{
										FinalMeshSize += Mesh->GetDataSize();
									}
								}

								TManagedPtr<FMesh> Result = CreateMesh(FinalMeshSize);

								MergeLODMeshesForConversion(Result.Get(), LOD.Meshes, bInitialGeneration);

								LOD.Meshes.SetNum(1);
								LOD.Meshes[0] = Result;
							}
							else
							{
								TManagedPtr<const FMesh> LastMesh;
								for (TManagedPtr<const FMesh>& Mesh : LOD.Meshes)
								{
									if (Mesh && Mesh->GetVertexCount() > 0)
									{
										if (LastMesh)
										{
											TManagedPtr<FMesh> Result = CreateMesh(LastMesh->GetDataSize() + Mesh->GetDataSize());

											MeshMerge(Result.Get(), LastMesh, Mesh, false);

											LastMesh = Result;
										}
										else
										{
											LastMesh = Mesh;
										}
									}

									Mesh.Reset();
								}

								LOD.Meshes.SetNum(1);
								LOD.Meshes[0] = LastMesh;
							}
						};


					if (bInitialGeneration)
					{
						[&]()
						{
							if (!SkeletalMesh)
							{
								StoreSkeletalMesh(Item, nullptr); 
								return;
							}	
								
							bool bValid = true;
					
							const FName& Name = Program.ConstantNames[Args.Name];
							
#if DO_CHECK			
							for (int32 LocalLODIndex = 0; LocalLODIndex < FirstLODAvailable; ++LocalLODIndex)
							{
								if (!SkeletalMesh->LODs.IsValidIndex(LocalLODIndex))
								{
									check(false);
									bValid = false;
									break;
								}
							
								if (SkeletalMesh->LODs[LocalLODIndex])
								{
									check(false);
									bValid = false;
									break;
								}
							}
#endif	
								
							for (int32 LocalLODIndex = FirstLODAvailable; LocalLODIndex < Args.NumLODs; ++LocalLODIndex)
							{
								if (!SkeletalMesh->LODs.IsValidIndex(LocalLODIndex))
								{
									GetLogger().LogError(TEXT("Skeletal Mesh %s does not contain LOD %i."), *Name.ToString(), LocalLODIndex);
									bValid = false;
									break;
								}
								
								if (!SkeletalMesh->LODs[LocalLODIndex] || SkeletalMesh->LODs[LocalLODIndex]->Meshes.IsEmpty())
								{
									GetLogger().LogError(TEXT("Skeletal Mesh %s LOD %i does not contain data."), *Name.ToString(), LocalLODIndex);
									bValid = false;
									break;
								}
							}
								
							if (bValid)
							{
								TManagedPtr<FSkeletalMesh> NewSkeletalMesh = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(SkeletalMesh));
							
								NewSkeletalMesh->FirstLODAvailable = FirstLODAvailable;
								NewSkeletalMesh->FirstLODResident = FirstLODResident;

								{
									NewSkeletalMesh->MinLODs.Default = Args.MinLODsDefault;

#if WITH_EDITORONLY_DATA
									const TArray<uint32>& Keys = Program.ConstantUInt32Lists[Args.MinLODsKeys];
									const TArray<int32>& Values = Program.ConstantInt32Lists[Args.MinLODsValues];
								
									NewSkeletalMesh->MinLODs.PerPlatform.Reserve(Keys.Num());
									for (int32 Index = 0; Index < Keys.Num(); ++Index)
									{
										NewSkeletalMesh->MinLODs.PerPlatform.Add(Program.ConstantNames[Keys[Index]], Values[Index]);
									}
#endif
								}
								
								{
									NewSkeletalMesh->MinQualityLevelLODs.Default = Args.MinQualityLevelLODsDefault;

									const TArray<int32>& Keys = Program.ConstantInt32Lists[Args.MinQualityLevelLODsKeys];
									const TArray<int32>& Values = Program.ConstantInt32Lists[Args.MinQualityLevelLODsValues];
								
									NewSkeletalMesh->MinQualityLevelLODs.PerQuality.Reserve(Keys.Num());
									for (int32 Index = 0; Index < Keys.Num(); ++Index)
									{
										NewSkeletalMesh->MinQualityLevelLODs.PerQuality.Add(Keys[Index], Values[Index]);
									}
								}
								
								NewSkeletalMesh->ScreenSize = Program.ConstantFloatLists[Args.ScreenSize];
								NewSkeletalMesh->LODHysteresis = Program.ConstantFloatLists[Args.LODHysteresis];
								NewSkeletalMesh->bSupportUniformlyDistributedSampling = Program.ConstantBoolLists[Args.bSupportUniformlyDistributedSampling];
								NewSkeletalMesh->bAllowCPUAccess = Program.ConstantBoolLists[Args.bAllowCPUAccess];
								NewSkeletalMesh->Name = Program.ConstantNames[Args.Name];
								
								MUTABLE_CPUPROFILER_SCOPE(SKO_CONVERT_MeshMerge);

								for (int32 LocalLODIndex = FirstLODAvailable; LocalLODIndex < Args.NumLODs; ++LocalLODIndex)
								{
									if (!NewSkeletalMesh->LODs[LocalLODIndex] || NewSkeletalMesh->LODs[LocalLODIndex]->Meshes.Num() <= 1)
									{
										continue;
									}

									TManagedPtr<FLOD> LOD = UE::Mutable::Private::CloneOrTakeOver<FLOD>(MoveTemp(NewSkeletalMesh->LODs[LocalLODIndex]));

									MergeLODMeshes(*LOD.Get());

									NewSkeletalMesh->LODs[LocalLODIndex] = LOD;
								}

								StoreSkeletalMesh(Item, NewSkeletalMesh); // TODO SKMPIN This should be StoreSkeletalMeshObject and store a TPassthroughObject<USkeletalMesh>(new USkeletalMesh)
							}
							else
							{
								StoreSkeletalMesh(Item, nullptr); 
							}
						}();
					}
					else
					{
						if (SkeletalMesh && SkeletalMesh->LODs.IsValidIndex(LODIndex))
						{ 
							TManagedPtr<FSkeletalMesh> NewSkeletalMesh = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(SkeletalMesh));

							if (NewSkeletalMesh->LODs[LODIndex] && NewSkeletalMesh->LODs[LODIndex]->Meshes.Num() > 1)
							{
								TManagedPtr<FLOD> LOD = Private::CloneOrTakeOver<FLOD>(MoveTemp(NewSkeletalMesh->LODs[LODIndex]));
								
								MergeLODMeshes(*LOD.Get());

								NewSkeletalMesh->LODs[LODIndex] = LOD;
							}

							StoreSkeletalMesh(Item, NewSkeletalMesh);
						}
						else
						{
							StoreSkeletalMesh(Item, SkeletalMesh);
						}
					}
					
					break;
				}

				default:
					unimplemented();
				}

				break;
			}

		case EOpType::SK_CONVERT:
        {
        	FOperation::FSkeletalMeshConvertArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshConvertArgs>(Item.At);
        	switch (Item.Stage)
        	{
        	case 0:
        		AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.SkeletalMeshObject, Item));
        		break;

        	case 1:
        	{
        		// This has been moved to a task. It should have been intercepted in IssueOp.
        		check(false);
        	}

        	default:
        		check(false);
        	}

	        break;
        }

		case EOpType::SK_MORPH:
		{
			FOperation::FSkeletalMeshMorphArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshMorphArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Base, Item),
						FScheduledOp::FromOpAndOptions(Args.Factor, Item, 0));
				break;
			}
			case 1:
			{
				float Factor = LoadScalar(FScheduledOp::FromOpAndOptions(Args.Factor, Item, 0));
				
				TManagedPtr<const FSkeletalMesh> Base = LoadSkeletalMesh(FScheduledOp(Args.Base, Item));

				if (FMath::IsNearlyZero(Factor))
				{
					StoreSkeletalMesh(Item, Base);
				}
				else
				{
					MUTABLE_CPUPROFILER_SCOPE(CodeRunner::SK_MORPH)

					TManagedPtr<FSkeletalMesh> Result;
	
					if (Base && Program.ConstantStrings.IsValidIndex(Args.MorphName))
					{
						int32 NumMeshesMorphed = 0;

						FName MorphName(Program.ConstantStrings[Args.MorphName]);

						Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(Base));

						for (TManagedPtr<const FLOD>& LOD : Result->LODs)
						{
							if (!LOD)
							{
								continue;
							}

							TManagedPtr<FLOD> MutableLOD = UE::Mutable::Private::CloneOrTakeOver<FLOD>(MoveTemp(LOD));
							for (TManagedPtr<const FMesh>& Mesh : MutableLOD->Meshes)
							{
								if (!Mesh)
								{
									continue;
								}

								if (Mesh->Morph.Names.Contains(MorphName))
								{
									TManagedPtr<FMesh> MutableMesh = CloneOrTakeOver(MoveTemp(Mesh));
									MeshMorph(MutableMesh.Get(), MorphName, Factor);
									Mesh = MutableMesh;

									++NumMeshesMorphed;
								}
							}

							LOD = MutableLOD;
						}

						if (NumMeshesMorphed == 0)
						{
							GetLogger().LogInfo(TEXT("Morph: SkeletalMesh - SkeletalMesh does not contain morph named [%s]."), *MorphName.ToString());
						}
					}

					StoreSkeletalMesh(Item, Result);
				}

				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::SK_RESHAPE:
		{
			MUTABLE_CPUPROFILER_SCOPE(CodeRunner::SK_RESHAPE);
			FOperation::FSkeletalMeshReshapeArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshReshapeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Base, Item),
						FScheduledOp(Args.BaseShape, Item),
						FScheduledOp(Args.TargetShape, Item));
				break;
			}
			case 1:
			{
				TManagedPtr<const FSkeletalMesh> Base = LoadSkeletalMesh(FScheduledOp(Args.Base, Item));
				TManagedPtr<const FSkeletalMesh> BaseShape = LoadSkeletalMesh(FScheduledOp(Args.BaseShape, Item));
				TManagedPtr<const FSkeletalMesh> TargetShape = LoadSkeletalMesh(FScheduledOp(Args.TargetShape, Item));

				if (!(BaseShape && TargetShape && Base))
				{
					StoreSkeletalMesh(Item, Base);
				}
				else
				{

					TArray<FName, TInlineAllocator<64>> BonesToDeform;
					TArray<FName, TInlineAllocator<64>> PhysicsToDeform;
					
					BonesToDeform.Reserve(Args.NumBones);
					PhysicsToDeform.Reserve(Args.NumPhysics);

					// BonesPtr and PhysicsPtr may not be aligned to 4, read them using Memcpy to avoid possible UB.
					const uint8* BonesPtr = Program.GetOpArgsPointer(Item.At) + sizeof(Args);
					const uint8* PhysicsPtr = BonesPtr + Args.NumBones*sizeof(int32);

					for (int32 BoneIndex = 0; BoneIndex < Args.NumBones; ++BoneIndex)
					{
						int32 Id;
						FMemory::Memcpy(&Id, BonesPtr + BoneIndex*sizeof(int32), sizeof(int32)); 
						BonesToDeform.Add(Program.ConstantNames[Id]);
					}

					for (int32 PhysIndex = 0; PhysIndex < Args.NumPhysics; ++PhysIndex)
					{
						int32 Id;
						FMemory::Memcpy(&Id, PhysicsPtr + PhysIndex*sizeof(int32), sizeof(int32)); 
						PhysicsToDeform.Add(Program.ConstantNames[Id]);
					}

					EMeshBindShapeFlags BindFlags = static_cast<EMeshBindShapeFlags>(Args.Flags);
					EMeshContentFlags MeshContentFilter = (EMeshContentFlags)Item.ExecutionOptions;

					FMeshBindColorChannelUsages ColorChannelUsages;
					FMemory::Memcpy(&ColorChannelUsages, &Args.ColorUsage, sizeof(ColorChannelUsages));
					static_assert(sizeof(ColorChannelUsages) == sizeof(Args.ColorUsage));

					TManagedPtr<FSkeletalMesh> Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(Base));

					bool bInitialGeneration;
					uint8 ItemLODIndex;
					uint8 ItemFirstLODResident;
					uint8 ItemFirstLODAvailable;
					uint8 ItemNumLODs;
					SkeletalMeshOptionsUnpack(Item.ExecutionOptions, bInitialGeneration, ItemLODIndex, ItemFirstLODResident, ItemFirstLODAvailable, ItemNumLODs);
				
					const int32 GeometryLODBegin = bInitialGeneration ? ItemFirstLODResident : ItemLODIndex;
					const int32 GeometryLODEnd   = bInitialGeneration ? ItemNumLODs : ItemLODIndex + 1;

					for (int32 LODIndex = GeometryLODBegin; LODIndex < GeometryLODEnd; ++LODIndex)
					{
						if (!(Result->LODs.IsValidIndex(LODIndex) && Result->LODs[LODIndex]))
						{
							continue;
						}

						if (!(BaseShape->LODs.IsValidIndex(LODIndex) && TargetShape->LODs.IsValidIndex(LODIndex) && BaseShape->LODs[LODIndex] && TargetShape->LODs[LODIndex]))
						{
							GetLogger().LogInfo(TEXT("Reshape: SkeletalMesh LOD [%i] - Missing shape mesh."), LODIndex);

							continue;
						}

						const TArray<TManagedPtr<const FMesh>>& BaseShapeMeshes = BaseShape->LODs[LODIndex]->Meshes;
						const TArray<TManagedPtr<const FMesh>>& TargetShapeMeshes = TargetShape->LODs[LODIndex]->Meshes;

						const int32 NumMeshes = BaseShapeMeshes.Num();

						if (NumMeshes != TargetShape->LODs[LODIndex]->Meshes.Num())
						{
							GetLogger().LogError(TEXT("Reshape: SkeletalMesh LOD [%i] - Base shape and target shape have different topology."), LODIndex);

							continue;
						}

						TArray<TNotNull<const FMesh*>, TInlineAllocator<32>> FilteredBaseShapeMeshes;
						TArray<TNotNull<const FMesh*>, TInlineAllocator<32>> FilteredTargetShapeMeshes;

						for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
						{
							if (BaseShapeMeshes[MeshIndex] && TargetShapeMeshes[MeshIndex])
							{
								FilteredBaseShapeMeshes.Add(BaseShapeMeshes[MeshIndex].Get());	
								FilteredTargetShapeMeshes.Add(TargetShapeMeshes[MeshIndex].Get());	
							}
						}

						GeometryUtils::FMeshGeometry BaseShapeGeometry;
						GeometryUtils::FMeshGeometry TargetShapeGeometry;

						constexpr bool bNeedsNormals = true;
						GeometryUtils::GetMergedGeometryFromMeshes(FilteredBaseShapeMeshes, bNeedsNormals, BaseShapeGeometry);
						GeometryUtils::GetMergedGeometryFromMeshes(FilteredTargetShapeMeshes, bNeedsNormals, TargetShapeGeometry);

						const bool bBaseShapeGeoemtryAddaptable = BaseShapeGeometry.Positions.Num() && BaseShapeGeometry.Triangles.Num();

						if (bBaseShapeGeoemtryAddaptable)
						{
							GeometryUtils::FMeshAdapter MeshAdapter(BaseShapeGeometry);

							constexpr bool bAutoBuild = false;
							UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter> BaseShapeAABBTree(&MeshAdapter, bAutoBuild);	
							{
								MUTABLE_CPUPROFILER_SCOPE(BuildBaseShapeAABBTree);
								BaseShapeAABBTree.Build();
							}

							TManagedPtr<FLOD> MutableLOD = UE::Mutable::Private::CloneOrTakeOver<FLOD>(MoveTemp(Result->LODs[LODIndex]));
							for (TManagedPtr<const FMesh>& Mesh : MutableLOD->Meshes)
							{
								if (!Mesh)
								{
									continue;
								}

								TManagedPtr<FMesh> MutableMesh = CloneOrTakeOver(MoveTemp(Mesh));

								bool bOutSuccess = false;
								MeshBindShapeReshape(MutableMesh.Get(), BaseShapeAABBTree, BonesToDeform, PhysicsToDeform, BindFlags, ColorChannelUsages, bOutSuccess);

								if (bOutSuccess)
								{
									MeshApplyShape(MutableMesh.Get(), TargetShapeGeometry, BindFlags, bOutSuccess, GetLogger());
								}

								Mesh = MutableMesh;
							}

							Result->LODs[LODIndex] = MutableLOD;
						}
						else
						{
							GetLogger().LogError(TEXT("Reshape: SkeletalMesh LOD [%i] - Shape mesh not compatible."), LODIndex);
						}

					}

					StoreSkeletalMesh(Item, Result);

				}

				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::SK_MATERIALMODIFY:
		{
			MUTABLE_CPUPROFILER_SCOPE(SK_MATERIALMODIFY);
			
			FOperation::FSkeletalMeshMaterialModifyArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshMaterialModifyArgs>(Item.At);
	
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.SkeletalMesh, Item));
				break;
			}

			case 1:
			{
				int32 FoundMaterialSlotIndex = INDEX_NONE;

				TManagedPtr<const FSkeletalMesh> SkeletalMesh = LoadSkeletalMesh(FCacheAddress(Args.SkeletalMesh, Item));
				if (SkeletalMesh)
				{
					FName MaterialSlotName = FName(Program.ConstantNames[Args.MaterialSlotName]);
					
					FoundMaterialSlotIndex = SkeletalMesh->MaterialSlotNames.IndexOfByPredicate(
					[MaterialSlotName](const FName& Name)
					{
						return MaterialSlotName == Name;
					});

					if (FoundMaterialSlotIndex == INDEX_NONE)
					{
						GetLogger().LogInfo(TEXT("Material Modify: SkeletalMesh - Material slot [%s] not found."), *MaterialSlotName.ToString());
					}

				}

				if (FoundMaterialSlotIndex != INDEX_NONE)
				{
					TManagedPtr<FSkeletalMesh> Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(SkeletalMesh));
					
					check(Result->MaterialSlotMaterials.IsValidIndex(FoundMaterialSlotIndex));
					Result->MaterialSlotMaterials[FoundMaterialSlotIndex].Emplace<FOperation::ADDRESS>(Args.NewMaterial); 
					
					StoreSkeletalMesh(Item, Result);
				}
				else
				{
					StoreSkeletalMesh(Item, SkeletalMesh);
				}
				
				break;
			}

			default:
				check(false);
			}

			break;

		}

		case EOpType::SK_CLIPMESHWITHMESH:
		{	
			FOperation::FSkeletalMeshClipMeshWithMeshArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshClipMeshWithMeshArgs>(Item.At);
		
			switch (Item.Stage)
			{
				case 0:
				{
					AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Source, Item),
							FScheduledOp(Args.Clip, Item));
					break;
				}
				case 1:
				{
					MUTABLE_CPUPROFILER_SCOPE(SK_CLIPMESHWITHMESH);

					TManagedPtr<const FSkeletalMesh> Source = LoadSkeletalMesh(FScheduledOp(Args.Source, Item));
					TManagedPtr<const FSkeletalMesh> Clip = LoadSkeletalMesh(FScheduledOp(Args.Clip, Item));

					if (Source && Clip)
					{

						bool bInitialGeneration;
						uint8 ItemLODIndex;
						uint8 ItemFirstLODResident;
						uint8 ItemFirstLODAvailable;
						uint8 ItemNumLODs;
						SkeletalMeshOptionsUnpack(Item.ExecutionOptions, bInitialGeneration, ItemLODIndex, ItemFirstLODResident, ItemFirstLODAvailable, ItemNumLODs);
					
						const int32 GeometryLODBegin = bInitialGeneration ? ItemFirstLODResident : ItemLODIndex;
						const int32 GeometryLODEnd   = bInitialGeneration ? ItemNumLODs : ItemLODIndex + 1;

						TManagedPtr<FSkeletalMesh> Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(Source));

						for (int32 LODIndex = GeometryLODBegin; LODIndex < GeometryLODEnd; ++LODIndex)
						{
							if (!Result->LODs.IsValidIndex(LODIndex) || !Result->LODs[LODIndex])
							{
								continue;
							}

							if (!(Clip->LODs.IsValidIndex(LODIndex) && Clip->LODs[LODIndex]))
							{
								GetLogger().LogInfo(TEXT("Clip Mesh With Mesh: SkeletalMesh LOD [%i] - Missing clip mesh."), LODIndex);

								continue;
							}

							TManagedPtr<FLOD> MutableLOD = UE::Mutable::Private::CloneOrTakeOver<FLOD>(MoveTemp(Result->LODs[LODIndex]));

							TArray<TNotNull<const FMesh*>, TInlineAllocator<32>> FilteredMeshes;

							for (const TManagedPtr<const FMesh>& ClipMesh : Clip->LODs[LODIndex]->Meshes)
							{
								if (ClipMesh)
								{
									check(!ClipMesh->GetVertexBuffers().IsDescriptor());
									FilteredMeshes.Add(ClipMesh.Get());
								}
							}

							GeometryUtils::FMeshGeometry ClipMeshGeometry;

							constexpr bool bNeedsNormals = false;
							GeometryUtils::GetMergedGeometryFromMeshes(FilteredMeshes, bNeedsNormals, ClipMeshGeometry);
							GeometryUtils::FMeshAdapter ClipMeshAdapter(ClipMeshGeometry);

							constexpr bool bAutoBuild = false;
							UE::Geometry::TMeshAABBTree3 ClipAABBTree(&ClipMeshAdapter, bAutoBuild);
							{
								MUTABLE_CPUPROFILER_SCOPE(SK_CLIPMESHWITHMESH::BuildClipTree);
								
								ClipAABBTree.Build();
							}


							const int32 NumMeshes = MutableLOD->Meshes.Num();
							for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
							{
								if (!MutableLOD->Meshes[MeshIndex]) 
								{
									continue;
								}	

								check(!MutableLOD->Meshes[MeshIndex]->GetVertexBuffers().IsDescriptor());

								TBitArray<> VerticesToCull;
								MeshClipMeshClassifyVerticesWithAABBTree(
										VerticesToCull, 
										TNotNull<const FMesh*>(MutableLOD->Meshes[MeshIndex].Get()), 
										ClipAABBTree);

								if (VerticesToCull.Find(true) != INDEX_NONE)
								{
									TManagedPtr<FMesh> MutableMesh = CloneOrTakeOver(MoveTemp(MutableLOD->Meshes[MeshIndex]));

									const bool bRemoveIfAllVerticesCulled = Args.FaceCullStrategy == EFaceCullStrategy::AllVerticesCulled;
									MeshRemoveVerticesWithCullSet(MutableMesh.Get(), VerticesToCull, bRemoveIfAllVerticesCulled);

									MutableLOD->Meshes[MeshIndex] = MutableMesh;
								}
							}

							Result->LODs[LODIndex] = MutableLOD;
						}

						StoreSkeletalMesh(Item, Result);
					}
					else
					{
						StoreSkeletalMesh(Item, Source);
					}

					break;
				}
				
				default: 
					check(false);
			};
			break;
		}
			
		case EOpType::SK_TRANSFORM:
		{
			MUTABLE_CPUPROFILER_SCOPE(SK_TRANSFORM)

			FOperation::FSkeletalMeshTransformArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshTransformArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
				{
					if (Args.Source)
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.Source, Item));
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
					break;
				}
			case 1:
				{
					TManagedPtr<const FSkeletalMesh> Source = LoadSkeletalMesh(FCacheAddress(Args.Source,Item));

					TManagedPtr<FSkeletalMesh> Result;
					
					if (Source)
					{
						Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(Source));

						const FMatrix44f& Matrix = Program.ConstantMatrices[Args.Matrix];
							
						bool bOutSuccess = false;

						[&]()
						{
							for (TManagedPtr<const FLOD>& LOD : Result->LODs)
							{
								if (!LOD)
								{
									continue;
								}

								TManagedPtr<FLOD> MutableLOD = UE::Mutable::Private::CloneOrTakeOver<FLOD>(MoveTemp(LOD));
								for (TManagedPtr<const FMesh>& Mesh : MutableLOD->Meshes)
								{
									if (!Mesh)
									{
										continue;
									}

									TManagedPtr<FMesh> MeshResult = MakeManaged<FMesh>();

									MeshTransform(MeshResult.Get(), Mesh.Get(), Matrix, bOutSuccess);
									if (!bOutSuccess)
									{
										Result = nullptr;
										return;
									}
						
									Mesh = MeshResult;
								}

								LOD = MutableLOD;
							}	
						}();
					}

					StoreSkeletalMesh(Item, Result);
					break;
				}

			default:
				unimplemented();
			}

			break;
		}

			
		case EOpType::SK_TRANSFORMWITHBONE:
		{
			MUTABLE_CPUPROFILER_SCOPE(SK_TRANSFORMWITHBONE)

			FOperation::FSkeletalMeshTransformWithBoneArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshTransformWithBoneArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				{
					if (Args.SourceSkeletalMesh)
					{
						AddOp(FScheduledOp(Item.At, Item, 1),
							FScheduledOp(Args.SourceSkeletalMesh, Item),
							FScheduledOp(Args.Matrix, Item));
					}
					else
					{
						StoreMesh(Item, nullptr);
					}
					break;
				}
			case 1:
				{
					TManagedPtr<const FSkeletalMesh> SourceSkeletalMesh = LoadSkeletalMesh(FCacheAddress(Args.SourceSkeletalMesh, Item));
					const FMatrix44f& Transform = LoadMatrix(FCacheAddress(Args.Matrix, Item));
					FName BoneName = Program.ConstantNames[Args.BoneId];

					if (SourceSkeletalMesh)
					{
						TManagedPtr<FSkeletalMesh> Result = UE::Mutable::Private::CloneOrTakeOver<FSkeletalMesh>(MoveTemp(SourceSkeletalMesh));

						for (TManagedPtr<const FLOD>& LOD : Result->LODs)
						{
							if (!LOD)
							{
								continue;
							}

							TManagedPtr<FLOD> MutableLOD = UE::Mutable::Private::CloneOrTakeOver<FLOD>(MoveTemp(LOD));
							for (TManagedPtr<const FMesh>& Mesh : MutableLOD->Meshes)
							{
								if (!Mesh)
								{
									continue;
								}

								TManagedPtr<FMesh> MutableMesh = CloneOrTakeOver(MoveTemp(Mesh));

								MeshTransformWithBoneInline(MutableMesh.Get(), Transform, BoneName, Args.ThresholdFactor);
								Mesh = MutableMesh;
							}

							LOD = MutableLOD;
						}
							
						StoreSkeletalMesh(Item, Result);
					}
					else
					{
						StoreSkeletalMesh(Item, SourceSkeletalMesh);
					}
					break;
				}

			default:
				unimplemented();
			}
			break;
		}
			
		default:
		{
			EDataType DataType = GetOpDataType(type);
			switch (DataType)
			{
			case EDataType::Instance:
				RunCode_Instance(Item);
				break;

			case EDataType::Mesh:
				RunCode_Mesh(Item);
				EnsureBudgetBelow(0);
				break;

			case EDataType::Image:
				RunCode_Image(Item);
				EnsureBudgetBelow(0);
				break;

			case EDataType::Layout:
				RunCode_Layout(Item);
				break;

			case EDataType::Bool:
				RunCode_Bool(Item);
				break;

			case EDataType::Scalar:
				RunCode_Scalar(Item);
				break;

			case EDataType::String:
				RunCode_String(Item);
				break;

			case EDataType::Int:
				RunCode_Int(Item);
				break;

			case EDataType::Projector:
				RunCode_Projector(Item);
				break;

			case EDataType::Color:
				RunCode_Color(Item);
				break;

			case EDataType::Matrix:
				RunCode_Matrix(Item);
				break;

			case EDataType::Material:
				RunCode_Material(Item);
				break;
				
			default:
				check(false);
				break;
			}
			break;
		}

        }

    }

	
	void CodeRunner::RunCodeImageDesc(const FScheduledOp& Item)
	{
		MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc);
		check(Item.Type == static_cast<uint16>(FScheduledOp::EType::ImageDesc));

		EOpType Type = Program.GetOpType(Item.At);
		switch (Type)
		{

		case EOpType::IM_CONSTANT:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_CONSTANT);

			check(Item.Stage == 0);
			FOperation::ResourceConstantArgs Args = Program.GetOpArgs<FOperation::ResourceConstantArgs>(Item.At);
			int32 ImageIndex = Args.value;

			FExtendedImageDesc Result;

			Result.m_format = Program.ConstantImages[ImageIndex].ImageFormat;
			Result.m_size[0] = Program.ConstantImages[ImageIndex].ImageSizeX;
			Result.m_size[1] = Program.ConstantImages[ImageIndex].ImageSizeY;
			Result.m_lods = Program.ConstantImages[ImageIndex].LODCount;

			int32 LODIndexIndex = Program.ConstantImages[ImageIndex].FirstIndex;
			{
				int32 LODIndex = 0;
				for (; LODIndex < Result.m_lods; ++LODIndex)
				{
					const int32 CurrentIndexIndex = LODIndexIndex + LODIndex;
					const FConstantResourceIndex CurrentIndex = Program.ConstantImageLODIndices[CurrentIndexIndex];

					bool bIsLODAvailable = false;
					if (!CurrentIndex.Streamable)
					{
						bIsLODAvailable = true;
					}
					else
					{
						uint32 RomId = CurrentIndex.Index;
						bIsLODAvailable = System.StreamInterface->DoesBlockExist(&Model, RomId);
					}

					if (bIsLODAvailable)
					{
						break;
					}
				}

				Result.FirstLODAvailable = LODIndex;
			}

			ImageDescConstantImages.Add(ImageIndex);

			StoreImageDesc(Item, Result);
			break;
		}

		case EOpType::IM_PARAMETER_CONVERT:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_PARAMETER_CONVERT);

			FOperation::ImageParameterConvertArgs Args = Program.GetOpArgs<FOperation::ImageParameterConvertArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.ImageParameter, Item));
					
				break;
			}

			case 1:
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.ImageParameter, Item)));
				break;
			}

			default:
				unimplemented();
			}

			break;
		}

		case EOpType::IM_REFERENCE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_REFERENCE);
			check(Item.Stage == 0);

			FOperation::ResourceReferenceArgs Args = Program.GetOpArgs<FOperation::ResourceReferenceArgs>(Item.At);	

			StoreImageDesc(Item, FExtendedImageDesc{ Args.ImageDesc, 0 });
			break;
		}

		case EOpType::IM_PARAMETER:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_PARAMETER);
			check(Item.Stage == 0);

			FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);
			UTexture* Image = Parameters.GetTextureValue(Args.variable);

			StoreImageDesc(Item, GetExternalImageDesc(Image));
			break;
		}

		case EOpType::IM_CONDITIONAL:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_CONDITIONAL);
			FOperation::ConditionalArgs Args = Program.GetOpArgs<FOperation::ConditionalArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				// We need to run the full condition result
				FScheduledOp FullConditionOp(Args.condition, Item);
				FullConditionOp.Type = static_cast<uint16>(FScheduledOp::EType::Full);

				AddOp(FScheduledOp(Item.At, Item, 1), FullConditionOp);
				break;
			}

			case 1:
			{
				bool bValue = LoadBool(FCacheAddress(Args.condition, Item.ExecutionIndex, Item.ExecutionOptions));
				FOperation::ADDRESS ResultAt = bValue ? Args.yes : Args.no;

				AddOp(FScheduledOp(Item.At, Item, 2, ResultAt), 
						FScheduledOp(ResultAt, Item, 0));
				break;
			}

			case 2: 
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Item.CustomState, Item))); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_SWITCH:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_SWITCH);
			const uint8* Data = Program.GetOpArgsPointer(Item.At);
		
			FOperation::ADDRESS VarAddress;
			FMemory::Memcpy(&VarAddress, Data, sizeof(FOperation::ADDRESS));
			Data += sizeof(FOperation::ADDRESS);

			FOperation::ADDRESS DefAddress;
			FMemory::Memcpy(&DefAddress, Data, sizeof(FOperation::ADDRESS));
			Data += sizeof(FOperation::ADDRESS);

			FOperation::FSwitchCaseDescriptor CaseDesc;
			FMemory::Memcpy(&CaseDesc, Data, sizeof(FOperation::FSwitchCaseDescriptor));
			Data += sizeof(FOperation::FSwitchCaseDescriptor);

			switch (Item.Stage)
			{
			case 0:
			{
				if (VarAddress)
				{
					// We need to run the full condition result
					FScheduledOp FullVariableOp(VarAddress, Item);
					FullVariableOp.Type = static_cast<uint16>(FScheduledOp::EType::Full);
					AddOp(FScheduledOp(Item.At, Item, 1), FullVariableOp);
				}
				else
				{
					StoreImageDesc(Item, FExtendedImageDesc{});
				}
				break;
			}

			case 1:
			{
				// Get the variable result
				int32 Var = LoadInt(FCacheAddress(VarAddress, Item.ExecutionIndex, Item.ExecutionOptions, FScheduledOp::EType::Full));

				FOperation::ADDRESS ValueAt = DefAddress;

				if (!CaseDesc.bUseRanges)
				{
					for (uint32 C = 0; C < CaseDesc.Count; ++C)
					{
						int32 Condition;
						FMemory::Memcpy(&Condition, Data, sizeof(int32));
						Data += sizeof(int32);

						FOperation::ADDRESS CaseAt;
						FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
						Data += sizeof(FOperation::ADDRESS);

						if (CaseAt && Var == (int32)Condition)
						{
							ValueAt = CaseAt;
							break;
						}
					}
				}
				else
				{
					for (uint32 C = 0; C < CaseDesc.Count; ++C)
					{
						int32 ConditionStart;
						FMemory::Memcpy(&ConditionStart, Data, sizeof(int32));
						Data += sizeof(int32);

						uint32 RangeSize;
						FMemory::Memcpy(&RangeSize, Data, sizeof(uint32));
						Data += sizeof(int32);

						FOperation::ADDRESS CaseAt;
						FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
						Data += sizeof(FOperation::ADDRESS);

						if (CaseAt && Var >= ConditionStart && Var < int32(ConditionStart + RangeSize))
						{
							ValueAt = CaseAt;
							break;
						}
					}

				}

				if (ValueAt)
				{
					AddOp(FScheduledOp(Item.At, Item, 2, ValueAt),
							FScheduledOp(ValueAt, Item, 0));
				}
				else
				{
					StoreImageDesc(Item, FExtendedImageDesc{}); 
				}
				break;
			}

			case 2: 
			{
				check(Item.CustomState);

				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Item.CustomState, Item))); 

				break;
			}
			default: check(false); break;
			}
			break;
		}

		case EOpType::IM_LAYERCOLOR:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_LAYERCOLOR);
			FOperation::ImageLayerColorArgs Args = Program.GetOpArgs<FOperation::ImageLayerColorArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.base, Item, 0), 
						FScheduledOp(Args.mask, Item, 0));

				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.base, Item));

				if (Args.mask)
				{
					FExtendedImageDesc MaskResult = LoadImageDesc(FCacheAddress(Args.mask, Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, MaskResult.FirstLODAvailable);
				}

				StoreImageDesc(Item, Result); 

				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_LAYER:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_LAYER);
			FOperation::ImageLayerArgs Args = Program.GetOpArgs<FOperation::ImageLayerArgs>(Item.At);	
			switch (Item.Stage)
			{
			case 0:
			{	
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.base, Item, 0),
						FScheduledOp(Args.mask, Item, 0),
						FScheduledOp(Args.blended, Item, 0));

				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.base, Item));

				if (Args.mask)
				{
					FExtendedImageDesc MaskResult = LoadImageDesc(FCacheAddress(Args.mask, Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, MaskResult.FirstLODAvailable);
				}

				if (Args.blended)
				{
					FExtendedImageDesc BlenResult = LoadImageDesc(FCacheAddress(Args.blended, Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, BlenResult.FirstLODAvailable);
				}
		
				StoreImageDesc(Item, Result); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_MULTILAYER:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_MULTILAYER);
			FOperation::ImageMultiLayerArgs Args = Program.GetOpArgs<FOperation::ImageMultiLayerArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				//TODO: For now multilayer operations will only check the base to get the descriptor.
				// but all iterations should be checked for available mips. 
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.base, Item, 0));
				break;
			}
			case 1: 
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.base, Item))); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_NORMALCOMPOSITE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_NORMALCOMPOSITE);
			FOperation::ImageNormalCompositeArgs Args = Program.GetOpArgs<FOperation::ImageNormalCompositeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.base, Item, 0)); 
				break;
			}
			case 1: 
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.base, Item))); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_PIXELFORMAT:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_PIXELFORMAT);
			FOperation::ImagePixelFormatArgs Args = Program.GetOpArgs<FOperation::ImagePixelFormatArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.source, Item, 0));
				break;

			case 1:
			{

				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.source, Item));

				EImageFormat OldFormat = Result.m_format;
				EImageFormat NewFormat = Args.format;
				if (Args.formatIfAlpha != EImageFormat::None && GetImageFormatData(OldFormat).Channels > 3)
				{
					NewFormat = Args.formatIfAlpha;
				}

				Result.m_format = NewFormat;

				StoreImageDesc(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_CONFORM:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_CONFORM);
			FOperation::ImageConformArgs Args = Program.GetOpArgs<FOperation::ImageConformArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Source, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Source, Item));
 
				EImageFormat OldFormat = Result.m_format;
				EImageFormat NewFormat = OldFormat;

				if (Args.ConformDescriptor.FormatIfAlpha != EImageFormat::None && 
					GetImageFormatData(OldFormat).Channels > 3)
				{
					NewFormat = Args.ConformDescriptor.FormatIfAlpha;
				}
				else if (Args.ConformDescriptor.m_format != EImageFormat::None)
				{
					NewFormat = Args.ConformDescriptor.m_format;
				}

				FImageSize TargetSize = Result.m_size;
				if (Args.ConformDescriptor.m_size != FImageSize(0, 0))
				{
					TargetSize = Args.ConformDescriptor.m_size;
				}

				const int32 MaxLevelCount = FImage::GetMipmapCount(TargetSize.X, TargetSize.Y);
				int32 TargetNumLODs = Result.m_lods;
				if (Args.ConformDescriptor.m_lods == 0xFF)
				{
					TargetNumLODs = FMath::Max((int32)Result.m_lods, MaxLevelCount);
				}
				else if (Args.ConformDescriptor.m_lods != 0)
				{
					TargetNumLODs = FMath::Max((int32)Args.ConformDescriptor.m_lods, (int32)Result.m_lods);
					TargetNumLODs = FMath::Min(TargetNumLODs, MaxLevelCount); 
				}

				Result.m_format = NewFormat;
				Result.m_lods = TargetNumLODs;
				Result.m_size = TargetSize;

				StoreImageDesc(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}


		case EOpType::IM_MIPMAP:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_MIPMAP);
			FOperation::ImageMipmapArgs Args = Program.GetOpArgs<FOperation::ImageMipmapArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.source, Item, 0));
				break;

			case 1:
			{
				// Somewhat synched with Full op execution code.	
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.source, Item));

				int32 LevelCount = Args.levels;
				int32 MaxLevelCount = FImage::GetMipmapCount(Result.m_size[0], Result.m_size[1]);
				if (LevelCount == 0)
				{
					LevelCount = MaxLevelCount;
				}
				else if (LevelCount > MaxLevelCount)
				{
					// If code generation is smart enough, this should never happen.
					// \todo But apparently it does, sometimes.
					LevelCount = MaxLevelCount;
				}

				// At least keep the levels we already have.
				int32 StartLevel = Result.m_lods;
				LevelCount = FMath::Max(StartLevel, LevelCount);

				// Update result.
				Result.m_lods = LevelCount;

				StoreImageDesc(Item, Result);
				break;
			}

			default:
				check(false);
			}
			break;
		}

		case EOpType::IM_RESIZE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_RESIZE);
			FOperation::ImageResizeArgs Args = Program.GetOpArgs<FOperation::ImageResizeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Source, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Source, Item));

				Result.m_size[0] = Args.Size[0];
				Result.m_size[1] = Args.Size[1];

				StoreImageDesc(Item, Result);

				break;
			}
			default:
				check(false);
			}
			break;
		}

		case EOpType::IM_RESIZELIKE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_RESIZELIKE);
			FOperation::ImageResizeLikeArgs Args = Program.GetOpArgs<FOperation::ImageResizeLikeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Source, Item, 0), 
						FScheduledOp(Args.SizeSource, Item, 0));

				break;
			}

			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Source, Item));
	
				if (Args.SizeSource)
				{
					FExtendedImageDesc SizeSourceResult = LoadImageDesc(FCacheAddress(Args.SizeSource, Item));
					Result.m_size = SizeSourceResult.m_size;
				}

				StoreImageDesc(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_RESIZEREL:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_RESIZEREL);
			FOperation::ImageResizeRelArgs Args = Program.GetOpArgs<FOperation::ImageResizeRelArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Source, Item, 0));
				break;

			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Source, Item));
				
				FImageSize DestSize(
					uint16(Result.m_size[0] * Args.Factor[0] + 0.5f),
					uint16(Result.m_size[1] * Args.Factor[1] + 0.5f));

				Result.m_size = DestSize;

				StoreImageDesc(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_BLANKLAYOUT:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_BLANKLAYOUT);
			FOperation::ImageBlankLayoutArgs Args = Program.GetOpArgs<FOperation::ImageBlankLayoutArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				// We need to run the full layout
				FScheduledOp FullLayoutOp(Args.Layout, Item);
				FullLayoutOp.Type = static_cast<uint16>(FScheduledOp::EType::Full);
				AddOp(FScheduledOp(Item.At, Item, 1), FullLayoutOp);
				break;
			}

			case 1:
			{
				
				TManagedPtr<const FLayout> Layout = LoadLayout(FCacheAddress(Args.Layout, Item.ExecutionIndex, Item.ExecutionOptions, FScheduledOp::EType::Full));

				FIntPoint SizeInBlocks = Layout->GetGridSize();
				FIntPoint BlockSizeInPixels(Args.BlockSize[0], Args.BlockSize[1]);
				FIntPoint ImageSizeInPixels = SizeInBlocks * BlockSizeInPixels;

				FImageSize DestSize(uint16(ImageSizeInPixels.X), uint16(ImageSizeInPixels.Y));
			
				FExtendedImageDesc Result;
			
				Result.m_size = DestSize;
				Result.m_format = Args.Format;
				
				if (Args.GenerateMipmaps)
				{
					if (Args.MipmapCount == 0)
					{
						Result.m_lods = FImage::GetMipmapCount(ImageSizeInPixels.X, ImageSizeInPixels.Y);
					}
					else
					{
						Result.m_lods = Args.MipmapCount;
					}
				}

				StoreImageDesc(Item, Result);
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_COMPOSE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_COMPOSE);
			FOperation::ImageComposeArgs Args = Program.GetOpArgs<FOperation::ImageComposeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.base, Item, 0),
						FScheduledOp(Args.blockImage, Item, 0)); 
				break;
			} 
			case 1:
			{	
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.base, Item));

				if (Args.blockImage)
				{
					FExtendedImageDesc BlockResult = LoadImageDesc(FCacheAddress(Args.blockImage, Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, BlockResult.FirstLODAvailable);
				}

				StoreImageDesc(Item, Result); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_MULTICOMPOSE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_MULTICOMPOSE);
			FOperation::ImageMultiComposeArgs Args = Program.GetOpArgs<FOperation::ImageMultiComposeArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
					FScheduledOp(Args.Base, Item, 0),
					FScheduledOp(Args.SourceImage, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Base, Item));

				if (Args.SourceImage)
				{
					FExtendedImageDesc BlockResult = LoadImageDesc(FCacheAddress(Args.SourceImage, Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, BlockResult.FirstLODAvailable);
				}

				StoreImageDesc(Item, Result);
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_INTERPOLATE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_INTERPOLATE);
			FOperation::ImageInterpolateArgs Args = Program.GetOpArgs<FOperation::ImageInterpolateArgs>(Item.At);

			int32 NumImages = 0;
			for (; NumImages < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++NumImages)
			{
				if (!Args.Targets[NumImages])
				{
					break;
				}
			}

			switch (Item.Stage)
			{
			case 0: 
			{
				TArray<FScheduledOp, TFixedAllocator<MUTABLE_OP_MAX_INTERPOLATE_COUNT>> Deps;
				for (int32 ImageIndex = 0; ImageIndex < NumImages; ++ImageIndex)
				{
					Deps.Add(FScheduledOp(Args.Targets[ImageIndex], Item, 0));
				}

				AddOp(FScheduledOp(Item.At, Item, 1), Deps); 
				break;
			}
			case 1: 
			{
				check(Args.Targets[0]);
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Targets[0], Item));
				
				for (int32 ImageIndex = 1; ImageIndex < NumImages; ++ImageIndex)
				{
					FExtendedImageDesc TargetResult = LoadImageDesc(FCacheAddress(Args.Targets[ImageIndex], Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, TargetResult.FirstLODAvailable);
				}

				StoreImageDesc(Item, Result); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_SATURATE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_SATURATE);
			FOperation::ImageSaturateArgs Args = Program.GetOpArgs<FOperation::ImageSaturateArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Base, Item, 0)); 
				break;
			}
			case 1: 
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.Base, Item))); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_LUMINANCE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_LUMINANCE);
			FOperation::ImageLuminanceArgs Args = Program.GetOpArgs<FOperation::ImageLuminanceArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Base, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Base, Item));
				Result.m_format = EImageFormat::L_UByte;
				
				StoreImageDesc(Item, Result);
				break;
			}
			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_SWIZZLE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_SWIZZLE);
			FOperation::ImageSwizzleArgs Args = Program.GetOpArgs<FOperation::ImageSwizzleArgs>(Item.At);
			
			TArray<FOperation::ADDRESS, TFixedAllocator<4>> ValidArgs;
			for (int32 SourceIndex = 0; SourceIndex < 4; ++SourceIndex)
			{
				if (Args.sources[SourceIndex])
				{
					ValidArgs.AddUnique(Args.sources[SourceIndex]);
				}
			}

			switch (Item.Stage)
			{
			case 0:
			{
				TArray<FScheduledOp, TFixedAllocator<4>> Deps;
				for (int32 ArgIndex = 0; ArgIndex < ValidArgs.Num(); ++ArgIndex)
				{
					Deps.Add(FScheduledOp(ValidArgs[ArgIndex], Item, 0));
				}

				AddOp(FScheduledOp(Item.At, Item, 1), Deps);

				break;
			}
			case 1:
			{	
				check(ValidArgs.Num() > 0);

				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(ValidArgs[0], Item));
				Result.m_format = Args.format;
				
				for (int32 ArgIndex = 1; ArgIndex < ValidArgs.Num(); ++ArgIndex)
				{
					FExtendedImageDesc SourceResult = LoadImageDesc(FCacheAddress(ValidArgs[ArgIndex], Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, SourceResult.FirstLODAvailable);
				}

				StoreImageDesc(Item, Result);
				break;
			}
			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_COLORMAP:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_COLORMAP);
			FOperation::ImageColorMapArgs Args = Program.GetOpArgs<FOperation::ImageColorMapArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{	
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Base, Item, 0));
				break;
			}
			case 1: 
			{	
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.Base, Item))); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_BINARISE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_BINARIZE);
			FOperation::ImageBinariseArgs Args = Program.GetOpArgs<FOperation::ImageBinariseArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Base, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Base, Item));
				Result.m_format = EImageFormat::L_UByte;

				StoreImageDesc(Item, Result);
				break;
			}
			default:
				check(false);
			}
			break;
		}

		case EOpType::IM_INVERT:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_INVERT);
			FOperation::ImageInvertArgs Args = Program.GetOpArgs<FOperation::ImageInvertArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Base, Item, 0)); 
				break;
			}
			case 1: 
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.Base, Item)));

				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_PLAINCOLOR:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_PLAINCOLOR);
			FOperation::ImagePlainColorArgs Args = Program.GetOpArgs<FOperation::ImagePlainColorArgs>(Item.At);
			
			FExtendedImageDesc Result;
			
			Result.m_size[0] = Args.Size[0];
			Result.m_size[1] = Args.Size[1];
			Result.m_lods = Args.LODs;
			Result.m_format = Args.Format;

			StoreImageDesc(Item, Result);
			break;
		}

		case EOpType::IM_CROP:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_CROP);
			FOperation::ImageCropArgs Args = Program.GetOpArgs<FOperation::ImageCropArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.source, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.source, Item));
				
				Result.m_size[0] = Args.sizeX;
				Result.m_size[1] = Args.sizeY;
				Result.m_lods = 1;

				StoreImageDesc(Item, Result);
				break;
			}
			default:
				check(false);
			}
			break;
		}

		case EOpType::IM_PATCH:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_PATCH);

			FOperation::ImagePatchArgs Args = Program.GetOpArgs<FOperation::ImagePatchArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.base, Item, 0),
						FScheduledOp(Args.patch, Item, 0)); 
				break;
			}
			case 1: 
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.base, Item));

				if (Args.patch)
				{
					FExtendedImageDesc PatchImageDesc = LoadImageDesc(FCacheAddress(Args.patch, Item));
					Result.FirstLODAvailable = FMath::Max(Result.FirstLODAvailable, PatchImageDesc.FirstLODAvailable);
				}

				StoreImageDesc(Item, Result); 
				break;
			}
			default: check(false);
			}
			break;
		}

		case EOpType::IM_RASTERMESH:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_RASTERMESH);

			FOperation::ImageRasterMeshArgs Args = Program.GetOpArgs<FOperation::ImageRasterMeshArgs>(Item.At);
			FExtendedImageDesc Result;
			
			Result.m_size[0] = Args.sizeX;
			Result.m_size[1] = Args.sizeY;
			Result.m_lods = 1;
			Result.m_format = EImageFormat::L_UByte;

			StoreImageDesc(Item, Result);
			break;
		}

		case EOpType::IM_MAKEGROWMAP:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_MAKEGROWMAP);

			FOperation::ImageMakeGrowMapArgs Args = Program.GetOpArgs<FOperation::ImageMakeGrowMapArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.mask, Item, 0));
				break;
			}
			case 1:
			{
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.mask, Item));

				Result.m_format = EImageFormat::L_UByte;
				Result.m_lods = 1;

				StoreImageDesc(Item, Result);
				break;
			}
			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_DISPLACE:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_DISPLACE);

			FOperation::ImageDisplaceArgs Args = Program.GetOpArgs<FOperation::ImageDisplaceArgs>(Item.At);
			switch (Item.Stage)
			{
			case 0: 
			{
				AddOp(FScheduledOp(Item.At, Item, 1),
						FScheduledOp(Args.Source, Item, 0));
				break;
			}
			case 1: 
			{
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(Args.Source, Item))); 
				break;
			}
			default: check(false);
			}
			break;
		}

        case EOpType::IM_TRANSFORM:
        {
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_TRANSFORM);
			FOperation::ImageTransformArgs Args = Program.GetOpArgs<FOperation::ImageTransformArgs>(Item.At);

            switch (Item.Stage)
            {
            case 0:
			{
				AddOp(FScheduledOp(Item.At, Item, 1), 
						FScheduledOp(Args.Base, Item, 0));	
                break;
			}
            case 1:
            {
				FExtendedImageDesc Result = LoadImageDesc(FCacheAddress(Args.Base, Item));
			
				Result.m_lods = 1;
				Result.m_format = GetUncompressedFormat(Result.m_format);
				
				if (!(Args.SizeX == 0 && Args.SizeY == 0))
				{
					Result.m_size[0] = Args.SizeX;
					Result.m_size[1] = Args.SizeY;
				}

				StoreImageDesc(Item, Result);
                break;
            }

            default:
                check(false);
            }

			break;
		}

		case EOpType::IM_MATERIAL_BREAK:
		{
			MUTABLE_CPUPROFILER_SCOPE(RunCodeImageDesc_IM_MATERIAL_BREAK);
			const FOperation::MaterialBreakArgs Args = Program.GetOpArgs<FOperation::MaterialBreakArgs>(Item.At);

			switch (Item.Stage)
			{
			case 0:
			{
				FScheduledOp MaterialOp(Args.Material, Item);
				MaterialOp.Type = static_cast<uint16>(FScheduledOp::EType::Full);

				AddOp(FScheduledOp(Item.At, Item, 1), MaterialOp);
				break;
			}
			case 1:
			{
				FCacheAddress CacheMaterialOp(Args.Material, Item);
				CacheMaterialOp.Type =  static_cast<uint16>(FScheduledOp::EType::Full);

				TManagedPtr<const FMaterial> Material = LoadMaterial(CacheMaterialOp);

				int32 HeapDataIndex = HeapData.Emplace();	
				FScheduledOpData& OpHeapData = HeapData[HeapDataIndex];

				OpHeapData.Resource = Material;

				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];
				const FParameterKey ParameterKey = { ParameterName, Args.LayerIndex };
				const FMaterial::FImageParameterData* ImageParameterData = Material->ImageParameters.Find(ParameterKey);
				bool bParameterFound = false;

				if (ImageParameterData)
				{
					bParameterFound = true;

					const TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>> ImageParameter = ImageParameterData->ImageParameter;
					if (ImageParameter.IsType<FOperation::ADDRESS>())
					{
						OpHeapData.MaterialBreak.ImageAddress = ImageParameter.Get<FOperation::ADDRESS>();

						AddOp(FScheduledOp(Item.At, Item, 2, HeapDataIndex), 
							FScheduledOp(OpHeapData.MaterialBreak.ImageAddress, Item));
					}
					else
					{
						StoreImageDesc(Item, FExtendedImageDesc{});
					}
				}
				else if (const UMaterialInterface* MaterialInterface = Material->PassthroughObject.Get())
				{
					UTexture* ParameterValue = nullptr;
					FHashedMaterialParameterInfo ParameterInfo;
					ParameterInfo.Name = FScriptName(ParameterName);
					ParameterInfo.Index = (int32)ParameterKey.LayerIndex;
					ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

					// Find image using the parameter info
					bParameterFound = MaterialInterface->GetTextureParameterValue(ParameterInfo, ParameterValue);

					if (bParameterFound)
					{
						StoreImageDesc(Item, GetExternalImageDesc(ParameterValue));
					}
				}

				if (!bParameterFound)
				{
					StoreImageDesc(Item, FExtendedImageDesc{});
				}

				break;
			}
			case 2:
			{
				FCacheAddress CacheMaterialOp(Args.Material, Item);
				CacheMaterialOp.Type =  static_cast<uint16>(FScheduledOp::EType::Full);

				FScheduledOpData& OpHeapData = HeapData[Item.CustomState];

				TManagedPtr<const FMaterial> Material = StaticCastManagedPtr<const FMaterial>(OpHeapData.Resource);
				FOperation::ADDRESS ImageAddress = OpHeapData.MaterialBreak.ImageAddress;

				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];
				const FParameterKey ParameterKey = { ParameterName, Args.LayerIndex };
				const FMaterial::FImageParameterData* ImageParameterData = Material->ImageParameters.Find(ParameterKey);
					
				StoreImageDesc(Item, LoadImageDesc(FCacheAddress(ImageAddress, Item)));
				break;
			}

			default:
				check(false);
			}

			break;
		}

		case EOpType::IM_PARAMETER_FROM_MATERIAL:
		{
			check(Item.Stage == 0);

			const FOperation::MaterialBreakImageParameterArgs Args = Program.GetOpArgs<FOperation::MaterialBreakImageParameterArgs>(Item.At);

			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Item, Args.MaterialParameter);

			// Get material parameter from the array of parameters
			UMaterialInterface* MaterialInterface = Parameters.GetMaterialValue(Args.MaterialParameter, Index.Get());

			// Get the texture parameter name
			const FName& ParameterName = Program.ConstantNames[Args.ParameterName];

			UTexture* Image = nullptr;

			// Get the parameter texture from the UMaterial
			if (MaterialInterface)
			{
				FHashedMaterialParameterInfo ParameterInfo;
				ParameterInfo.Name = FScriptName(ParameterName);
				ParameterInfo.Index = (int32)Args.LayerIndex;
				ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

				MaterialInterface->GetTextureParameterValue(ParameterInfo, Image);
			}

			if (Image)
			{
				StoreImageDesc(Item, GetExternalImageDesc(Image));
			}
			else
			{
				StoreImageDesc(Item, FExtendedImageDesc{});
			}
			break;
		}

		default:
			if (Type != EOpType::NONE)
			{
				// Operation not implemented
				check(false);

				StoreImageDesc(Item, FExtendedImageDesc{});
			}
			break;
		}
	}

	FImageOperator MakeImageOperator(CodeRunner* Runner)
	{
		return FImageOperator(
			// Create
			[Runner](int32 x, int32 y, int32 m, EImageFormat f, EInitializationType i)
			{
				return Runner->CreateImage(x, y, m, f, i);
			},

			// Clone
			[Runner](const FImage* Image)
			{
				TManagedPtr<FImage> New = Runner->CreateImage(Image->GetSizeX(), Image->GetSizeY(), Image->GetLODCount(), Image->GetFormat(), EInitializationType::NotInitialized);
				New->Copy(Image);
				return New;
			},

			Runner->LiveInstance->PixelFormatOverride
		);
	}
}

#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK

#undef AddOp

#endif
