// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "MuR/Image.h"
#include "MuR/Layout.h"
#include "MuR/Operations.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Settings.h"
#include "MuR/System.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Concepts/SameAs.h"



namespace  UE::Mutable::Private
{
	class FModel;
	class FParameters;
	class FRangeIndex;

    /** Code execution of the mutable virtual machine. */
    class CodeRunner : public TSharedFromThis<CodeRunner>
    {
		// The private token allows only members or friends to call MakeShared.
		struct FPrivateToken { explicit FPrivateToken() = default; };

	public:
		static TSharedRef<CodeRunner> Create(
				const TSharedRef<FLiveInstance>& InLiveInstance,
				const FSettings& InSettings, 
				FSystem& InSystem, 
				EExecutionStrategy,
				FOperation::ADDRESS At,
				uint16 ExecutionOptions,
				FScheduledOp::EType Type);

    	// Private constructor to prevent stack allocation. In general we can not call AsShared() if the lifetime is
		// bounded.
		explicit CodeRunner(
			const TSharedRef<FLiveInstance>& InLiveInstance,	
			FPrivateToken, 
			const FSettings& InSettings, 
			FSystem& InSystem, 
			EExecutionStrategy,
			FOperation::ADDRESS At,
			uint16 ExecutionOptions,
			FScheduledOp::EType Type);

    protected:
		struct FProfileContext
		{
			FMutex Mutex;
			uint32 NumRunOps = 0;
			uint32 RunOpsPerType[int32(EOpType::COUNT)] = {};
		};

	public:
        /** Type of data sometimes stored in the code runner heap to pass info between operation stages. */
        struct FScheduledOpData
        {
			union
			{	
				struct
				{
					float Bifactor;
					int32 Min, Max;
				} Interpolate;

				struct
				{
					int32 Iterations;
					EImageFormat OriginalBaseFormat;
					bool bBlendOnlyOneMip;
				} MultiLayer;

				struct
				{
					uint8 Mip;
					float MipValue;
				} RasterMesh;

				struct
				{
					uint16 SizeX;
					uint16 SizeY;
					uint16 ScaleXEncodedHalf;
					uint16 ScaleYEncodedHalf;
					float MipValue;
				} ImageTransform;

				struct
				{
					uint16 NumElemsToProcess;
					uint8 SourceLOD : 7;
					uint8 bHasBlocksToCompose : 1;
					
					uint8 SourceNumLODs : 7;
					uint8 bSplitInMultipleIterations : 1;
				} MultiCompose;

				struct
				{
					FOperation::ADDRESS ImageAddress;
				} MaterialBreak;

			};

			TManagedPtr<const FResource> Resource;
        };
	protected:
		// Assertion to know when FScheduledOpData size changes. It is ok to modifiy if needed. 
		static_assert(sizeof(FScheduledOpData) == 4*4+sizeof(TManagedPtr<FResource>), "FScheduledOpData size changed.");
    	
        TSharedPtr<FRangeIndex> BuildCurrentOpRangeIndex( const FScheduledOp&, int32 ParameterIndex);

        void RunCode( const FScheduledOp&);

        void RunCode_Conditional(const FScheduledOp&);
        void RunCode_Switch(const FScheduledOp&);
        void RunCode_Instance(const FScheduledOp&);
        void RunCode_InstanceAddResource(const FScheduledOp&);

		/** Return false in case of failure. */
        bool RunCode_ConstantResource(const FScheduledOp&);

        void RunCode_Mesh(const FScheduledOp&);
        void RunCode_Image(const FScheduledOp&);
        void RunCode_Layout(const FScheduledOp&);
        void RunCode_Bool(const FScheduledOp&);
        void RunCode_Int(const FScheduledOp&);
        void RunCode_Scalar(const FScheduledOp&);
        void RunCode_String(const FScheduledOp&);
        void RunCode_Color(const FScheduledOp&);
        void RunCode_Projector(const FScheduledOp&);
        void RunCode_Matrix(const FScheduledOp&);
    	void RunCode_Material(const FScheduledOp&);

    	void RunCodeImageDesc(const FScheduledOp&);
   
		void LaunchWaitForOthersTask(Tasks::FTaskEvent Event, uint64 ReentryCounter = 0);

	public:
		struct FExternalResourceId
		{
			/** If it is an image or mesh reference. */
			PASSTHROUGH_ID ReferenceResourceId = PASSTHROUGH_ID_INVALID;

			/** If it is an image or mesh parameter.*/
			UTexture* ImageParameter = nullptr; 
			USkeletalMesh* MeshParameter = nullptr; 
		};

		/** Load an external image asynchronously, returns an event to wait for complition and a cleanup function that must be called once the event has completed. */
		TTuple<UE::Tasks::FTask, TFunction<void()>> LoadExternalImageAsync(FExternalResourceId Id, uint8 MipmapsToSkip, TFunction<void(TManagedPtr<FImage>)>& ResultCallback);
 	    UE::Mutable::Private::FExtendedImageDesc GetExternalImageDesc(UTexture* Id);

		/** Load an external mesh asynchronously, returns an event to wait for complition and a cleanup function that must be called once the event has completed. */
		TTuple<UE::Tasks::FTask, TFunction<void()>> LoadExternalMeshAsync(FExternalResourceId Id, int32 BaseLODIndex, int32 BaseSectionIndex, int32 LODBias, uint8 ConversionFlags, TFunction<void(TManagedPtr<FMesh>)>& ResultCallback);

		/** Load an external skeletal mesh asynchronously, returns an event to wait for complition and a cleanup function that must be called once the event has completed. */
		TTuple<UE::Tasks::FTask, TFunction<void()>> LoadExternalSkeletalMeshAsync(FExternalResourceId Id, int32 LODBegin, int32 LODEnd, int32 GeometryLODBegin, int32 GeometryLODEnd, uint8 ConversionFlags, TFunction<void(TManagedPtr<FSkeletalMesh>)>& ResultCallback);

		/** Settings that may affect the execution of some operations, like image conversion quality. */
		FSettings Settings;
    	
    	/** Instance whose data is being used by this runner.*/
    	const TSharedRef<FLiveInstance> LiveInstance;

	public:
        /** 
		 * Heap of intermediate data pushed by some instructions and referred by others.
         * It is not released until no operations are pending.
		 */
		TArray<FScheduledOpData> HeapData;
    protected:
		
		/** Image descriptor intermediate results. */
		TMap<FOperation::ADDRESS, FExtendedImageDesc> ImageDescResults;
		TArray<int32> ImageDescConstantImages; 

		void Run(TUniquePtr<FProfileContext>&& ProfileContext, bool bForceInlineExecution);
	public:

		void StartRun(bool bForceInlineExecution);

		/** */
		FExtendedImageDesc GetImageDescResult(FOperation::ADDRESS ResultAddres);

		FProgramCache& GetMemory();

		FLiveInstanceLogger& GetLogger();

		struct FTask
		{
			FTask() {}
			FTask(const FScheduledOp& InOp, TConstArrayView<FScheduledOp> InDeps) 
				: Op(InOp) 
			{
				Deps.Reserve(InDeps.Num());

				for (const FScheduledOp& Dep : InDeps)
				{
					if (Dep.At)
					{
						Deps.Add(Dep);
					}
				}
			}

			FScheduledOp Op;
			TArray<FCacheAddress, TInlineAllocator<3>> Deps;
		};

		class FIssuedTask
		{
		public:
			const FScheduledOp Op;

			UE::Tasks::FTask Event = {};

			FIssuedTask(const FScheduledOp& InOp) : Op(InOp) {}
			virtual ~FIssuedTask() = default;

			/** */
			virtual bool Prepare(CodeRunner*, bool& bOutFailed) { bOutFailed = false; return true; }
			virtual void DoWork() {}

			/** Return true if succeeded. */
			virtual bool Complete(CodeRunner*) = 0;

			/** Return true if the task has been completed. */
			virtual bool IsComplete(CodeRunner*)
			{ 
				return !Event.IsValid() || Event.IsCompleted(); 
			}
		};
    	

		class FLoadMeshRomTask : public CodeRunner::FIssuedTask
		{
		public:
			FLoadMeshRomTask( 
					const FScheduledOp& InOp, 
					int32 InFirstIndex, 
					EMeshContentFlags InRomContentFlags, 
					EMeshContentFlags InExecutionContentFlags)
				: FIssuedTask(InOp)
				, FirstIndex(InFirstIndex)
				, RomContentFlags(InRomContentFlags)
				, ExecutionContentFlags(InExecutionContentFlags)
			{
			}

			// FIssuedTask interface
			virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
			virtual bool Complete(CodeRunner*) override;

		private:
			int32 FirstIndex = -1;
			EMeshContentFlags RomContentFlags = EMeshContentFlags::None;
			EMeshContentFlags ExecutionContentFlags = EMeshContentFlags::None;

			/** Key is the rom index. Value is the future of the streamed rom. */
			TMap<int32, Tasks::TTask<TManagedPtr<const FMesh>>> RomsStreamed;
		};

		class FLoadImageRomsTask : public CodeRunner::FIssuedTask
		{
		public:
			FLoadImageRomsTask(const FScheduledOp& InOp, int32 InImageIndexBegin, int32 InImageIndexEnd)
				: FIssuedTask(InOp)
				, ImageIndexBegin(InImageIndexBegin)
				, ImageIndexEnd(InImageIndexEnd)
			{
			}

			// FIssuedTask interface
			virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
			virtual bool Complete(CodeRunner*) override;

		private:
 			int32 ImageIndexBegin = -1;
			int32 ImageIndexEnd = -1;

			TMap<int32, Tasks::TTask<TManagedPtr<const FImage>>> RomsStreamed;
		};

		template <
			typename... DepsTypes
			UE_REQUIRES(UE::CSameAs<typename TDecay<DepsTypes>::Type, FScheduledOp> && ...)
		>
		void AddOp(const FScheduledOp& Op, DepsTypes&&... InDeps)
		{
			constexpr int32 NumDeps = sizeof...(DepsTypes);
			
			if constexpr (NumDeps == 0)
			{
				OpenTasks.Add(Op);
			}
			else
			{
				FScheduledOp DepsStorage[NumDeps] = { Forward<DepsTypes>(InDeps)... };

				ClosedTasks.Add(FTask(Op, TConstArrayView<FScheduledOp>(DepsStorage, NumDeps)));
				AddChildren(DepsStorage);
			}
			
		}

		void AddOp(const FScheduledOp& Op, TArrayView<const FScheduledOp> InDeps)
		{
			if (!InDeps.Num())
			{
				OpenTasks.Add(Op);
			}
			else
			{
				ClosedTasks.Emplace(Op, InDeps);
				AddChildren(InDeps);
			}
		}
    	
		/** Calculate an approximation of memory used by manging structures in this class. */
		int32 GetInternalMemoryBytes() const
		{
			return sizeof(CodeRunner) 
				+ HeapData.GetAllocatedSize() + ImageDescResults.GetAllocatedSize()
				+ ClosedTasks.GetAllocatedSize() + OpenTasks.GetAllocatedSize()
				// this contains smart pointers, approximate size like this:
				+ IssuedTasks.Max() * ( sizeof(FIssuedTask) + 16);
		}

		FString PrintDependencyGraph();
	protected:

		/** Strategy to choose the order of execution of operations. */
		EExecutionStrategy ExecutionStrategy = EExecutionStrategy::None;

		/** If this flag is enabled, issued operation stage that use tasks will be executed in the mutable thread instead of in a generic worker thread. */
		bool bForceSerialTaskExecution = false;

		/** List of pending operations that we don't know if they cannot be run yet because of dependencies. */
		TArray<FTask> ClosedTasks;

		/** List of tasks that can be run because they don't have any unmet dependency. */
		TArray<FScheduledOp> OpenTasks;

		/** List of tasks that are ready to run concurrently. */
		TArray<TSharedPtr<FIssuedTask>> IssuedTasksOnHold;

		/** List of tasks that have been set to run concurrently and their completion is unknown. */
		TArray<TSharedPtr<FIssuedTask>> IssuedTasks;

	public:

		inline FExtendedImageDesc LoadImageDesc(const FCacheAddress& From)
		{
			return LiveInstance->Cache->LoadImageDesc(From);
		}

		inline bool LoadBool(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadBool(From);
		}

		inline float LoadInt(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadInt(From);
		}

		inline float LoadScalar(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadScalar(From);
		}

		inline FVector4f LoadColor(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadColor(From);
		}

    	inline FMatrix44f LoadMatrix(const FCacheAddress& From) const
	    {
			return LiveInstance->Cache->LoadMatrix(From);
		}

		inline TManagedPtr<const FMaterial> LoadMaterial(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadMaterial(From);
		}

		inline TManagedPtr<const String> LoadString(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadString(From);
		}

		inline FProjector LoadProjector(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadProjector(From);
		}

		inline TManagedPtr<const FMesh> LoadMesh(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadMesh(From);
		}

    	inline TManagedPtr<const FLOD> LoadLOD(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadLOD(From);
		}
    	
    	inline TManagedPtr<const FSkeletalMesh> LoadSkeletalMesh(const FCacheAddress& From) const
	    {
			return LiveInstance->Cache->LoadSkeletalMesh(From);
		}
    	
		inline TManagedPtr<const FImage> LoadImage(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadImage(From);
		}

		inline TManagedPtr<const FLayout> LoadLayout(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadLayout(From);
		}

		inline TManagedPtr<const FInstance> LoadInstance(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadInstance(From);
		}

		inline TManagedPtr<const FExtensionData> LoadExtensionData(const FCacheAddress& From) const
		{
			return LiveInstance->Cache->LoadExtensionData(From);
		}
    	
    	inline TManagedPtr<const FInstancedStruct> LoadInstancedStruct(const FCacheAddress& From) const
	    {
			return LiveInstance->Cache->LoadInstancedStruct(From);
		}

		inline void StoreImageDesc(const FCacheAddress& To, const FExtendedImageDesc& ImageDesc)
		{
			LiveInstance->Cache->StoreImageDesc(To, ImageDesc);
		}

		inline void StoreBool(const FCacheAddress& To, bool Value)
		{
			LiveInstance->Cache->StoreBool(To, Value);
		}

		inline void StoreInt(const FCacheAddress& To, int32 Value)
		{
			LiveInstance->Cache->StoreInt(To, Value);
		}

		inline void StoreScalar(const FCacheAddress& To, float Value)
		{
			LiveInstance->Cache->StoreScalar(To, Value);
		}

		inline void StoreString(const FCacheAddress& To, TManagedPtr<const String> Value)
		{
			LiveInstance->Cache->StoreString(To, Value);
		}

		inline void StoreColor(const FCacheAddress& To, const FVector4f& Value)
		{
			LiveInstance->Cache->StoreColor(To, Value);
		}

    	inline void StoreMatrix(const FCacheAddress& To, const FMatrix44f& Value)
		{
			LiveInstance->Cache->StoreMatrix(To, Value);
		}

		inline void StoreMaterial(const FCacheAddress& To, TManagedPtr<const FMaterial> Value)
		{
			LiveInstance->Cache->StoreMaterial(To, Value);
		}
    	
		inline void StoreProjector(const FCacheAddress& To, const FProjector& Value)
		{
			LiveInstance->Cache->StoreProjector(To, Value);
		}

		inline void StoreMesh(const FCacheAddress& To, const TManagedPtr<const FMesh>& Resource)
		{
			LiveInstance->Cache->StoreMesh(To, Resource);
		}

    	inline void StoreLOD(const FCacheAddress& To, TManagedPtr<const FLOD> Resource)
		{
			LiveInstance->Cache->StoreLOD(To, Resource);
		}

    	inline void StoreSkeletalMesh(const FCacheAddress& To, TManagedPtr<const FSkeletalMesh> Resource)
		{
			LiveInstance->Cache->StoreSkeletalMesh(To, Resource);
		}
    	
		inline void StoreImage(const FCacheAddress& To, const TManagedPtr<const FImage>& Resource)
		{
			LiveInstance->Cache->StoreImage(To, Resource);
		}

		inline void StoreLayout(const FCacheAddress& To, const TManagedPtr<const FLayout>& Resource)
		{
			LiveInstance->Cache->StoreLayout(To, Resource);
		}

		inline void StoreInstance(const FCacheAddress& To, const TManagedPtr<const FInstance>& Resource)
		{
			LiveInstance->Cache->StoreInstance(To, Resource);
		}

		inline void StoreExtensionData(const FCacheAddress& To, const TManagedPtr<const FExtensionData>& Resource)
		{
			LiveInstance->Cache->StoreExtensionData(To, Resource.ToStrongRef());
		}

		inline void StoreInstancedStruct(const FCacheAddress& To, const TManagedPtr<const FInstancedStruct>& Resource)
		{
			LiveInstance->Cache->StoreInstancedStruct(To, Resource.ToStrongRef());
		}
    	
		inline TManagedPtr<FImage> CreateImage(uint32 SizeX, uint32 SizeY, uint32 Lods, EImageFormat Format, EInitializationType Init)
		{
			TManagedPtr<FImage> Result = CreateImageImp(SizeX, SizeY, Lods, Format, Init);
			return Result;
		}

		TManagedPtr<FImage> CreateImageLike( const FImage* Ref, EInitializationType Init)
		{
			TManagedPtr<FImage> Result = CreateImageImp(Ref->GetSizeX(), Ref->GetSizeY(), Ref->GetLODCount(), Ref->GetFormat(), Init);
			return Result;
		}

		inline TManagedPtr<FImage> CloneOrTakeOver(TManagedPtr<const FImage>&& Resource)
		{
			TManagedPtr<FImage> Result;
			if (!Resource.IsUniqueReference())
			{
				if (Resource)
				{
					uint32 DataSize = Resource->GetDataSize();
					EnsureBudgetBelow(DataSize);
					Result = Resource->Clone();
				}
			}
			else
			{
				Result = ConstCastManagedPtr<FImage>(Resource);
			}

			Resource = nullptr;
			return Result;
		}

		[[nodiscard]] inline TManagedPtr<FMesh> CreateMesh(int32 BudgetReserveSize = 0)
		{
			EnsureBudgetBelow(BudgetReserveSize);

			TManagedPtr<FMesh> Result = MakeManaged<FMesh>();
			return Result;
		}

		[[nodiscard]] inline TManagedPtr<FMesh> CloneOrTakeOver(TManagedPtr<const FMesh>&& Resource)
		{
			TManagedPtr<FMesh> Result;
			if (!Resource.IsUniqueReference())
			{
				if (Resource)
				{
					const int32 ResourceDataSize = Resource->GetDataSize();
					Result = CreateMesh(ResourceDataSize); 
					Result->CopyFrom(*Resource);
				}
			}
			else
			{
				Result = ConstCastManagedPtr<FMesh>(Resource);
			}

			Resource = nullptr;
			return Result;
		}

    	template<typename T>
		TPassthroughObjectPtr<T> FindPassthrough(PASSTHROUGH_ID Id)
		{
			if (LiveInstance->PassthroughObjectLoader)
			{
				return LiveInstance->PassthroughObjectLoader->FindChecked<T>(Id);
			}
			else
			{
				return TPassthroughObjectPtr<T>(Id);
			}
		}
    	
		/** This flag is turned on when a streaming error or similar happens. Results are not usable.
		* This should only happen in-editor.
		*/
		bool bUnrecoverableError = false;

		FSystem& System;
    	
    	// Dereference optimization. LiveInstance keeps them alive.
		FModel& Model;
		const FProgram& Program;
		const FParameters& Parameters;
    	
    	TSharedPtr<FExternalResourceProvider> ExternalResourceProvider;
    
    private:
    	
    	inline TManagedPtr<FImage> CreateImageImp(uint32 SizeX, uint32 SizeY, uint32 Lods, EImageFormat Format, EInitializationType Init)
    	{
    		// Make room in the budget
    		uint32 DataSize = FImage::CalculateDataSize(SizeX, SizeY, Lods, Format);
    		EnsureBudgetBelow(DataSize);

    		// Create it
    		TManagedPtr<FImage> Result = MakeManaged<FImage>(SizeX, SizeY, Lods, Format, Init);
    		return Result;
    	}
    	
    	bool EnsureBudgetBelow( uint64 AdditionalMemory ) const
	    {
    		return System.WorkingMemoryManager.EnsureBudgetBelow(AdditionalMemory, LiveInstance);
    	}
    	
		void AddChildren(TArrayView<const FScheduledOp> Deps);
		//void AddChildren(const FScheduledOp& Deps);

		/** Try to create a concurrent task for the given op. Return null if not possible. */
		TSharedPtr<FIssuedTask> IssueOp(FScheduledOp item);

		/** Update debug stats. */
		void UpdateTraces();

		/** */
		bool ShouldIssueTask() const;

		/** */
		void LaunchIssuedTask(const TSharedPtr<FIssuedTask>& TaskToIssue, bool& bOutFailed);
    	
    private:
    	
    	FOperation::ADDRESS RootOPAddress = 0;
    	
    	uint8 RootOpExecutionOptions;
    	
    	FScheduledOp::EType RootOpType;
    };


	/** Helper function to create the memory-tracked image operator. */
	extern FImageOperator MakeImageOperator(CodeRunner* Runner);

	
	TSharedPtr<FRangeIndex> BuildCurrentOpRangeIndex(const FScheduledOp& Item, const FParameters& Params, const FModel& InModel, FProgramCache& ProgramCache, int32 ParameterIndex);
}

#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK
namespace UE::Mutable::Private::Private
{
	FString DumpItemScheduledCallstack(const FScheduledOp& Item);
}
#endif

