// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/CodeRunnerBegin.h"

#include "CodeRunner.h"
#include "MuR/MutableTrace.h"
#include "MuR/Model.h"
#include "MuR/Operations.h"
#include "MuR/System.h"
#include "MuR/External/Operation.h"


namespace UE::Mutable::Private
{
	CodeRunnerBegin::CodeRunnerBegin(
			const TSharedRef<FLiveInstance>& InLiveInstance,
			uint32 InLodMask) :
		Model(InLiveInstance->Model),
		Params(InLiveInstance->Parameters),
		Program(InLiveInstance->Model->GetProgram()),
		LodMask(InLodMask),
		Executed(InLiveInstance->Model->GetProgram().NumOps),
		ProgramCache(InLiveInstance->Cache),
		PassthroughObjectLoader(InLiveInstance->PassthroughObjectLoader)
	{
	}

	
	bool FScheduledOpInline::operator==(const FScheduledOpInline& Other) const
	{
		return At == Other.At && Stage == Other.Stage && ExecutionIndex == Other.ExecutionIndex;
	}


	uint32 GetTypeHash(const FScheduledOpInline& Op)
	{
		uint32 Hash = Op.At;
		
		Hash = HashCombineFast(Hash, Op.Stage);
		Hash = HashCombineFast(Hash, Op.ExecutionIndex);

		return Hash;
	}


	uint32 GetTypeHash(const FCacheAddressInline& Address)
	{
		return HashCombineFast(Address.At, static_cast<uint32>(Address.ExecutionIndex));
	}


	FOpSet::FOpSet(int32 NumElements)
	{
		//Index0.SetNum(NumElements, false);
		IndexOther.Reserve(NumElements / 16);
	}


	bool FOpSet::Contains(const FCacheAddressInline& Item)
	{
		return IndexOther.Contains(Item);
	}


	void FOpSet::Add(const FCacheAddressInline& Item)
	{
		IndexOther.Add(Item);
	}
	
	
	FStackValue FStack::Pop()
	{
		return TArray::Pop(EAllowShrinking::No);
	}
	

	FScheduledOpInline CodeRunnerBegin::PopOp()
	{
		return Items.Pop(EAllowShrinking::No);
	}


	void CodeRunnerBegin::PushOp(const FScheduledOpInline& Item)
	{
		check(GetAddressByteCodeOffset(Item.At) < FOperation::ADDRESS(Program.ByteCode.Num()));

		if (!Item.At)
		{
			return;
		}
		
		Items.Push(Item);
	}
 

	void CodeRunnerBegin::StoreInt(const FCacheAddressInline& To, int32 Value)
	{
		if (!To.At)
		{
			return;
		}
		
		Executed.Add(To);

		FStackValue StackValue;
		StackValue.Set<int32>(Value);
		Stack.Push(StackValue);

		ResultsInt.Add(To, Value);
	}


	void CodeRunnerBegin::StoreFloat(const FCacheAddressInline& To, float Value)
	{
		if (!To.At)
		{
			return;
		}
		
		Executed.Add(To);

		FStackValue StackValue;
		StackValue.Set<float>(Value);
		Stack.Push(StackValue);

		ResultsFloat.Add(To, Value);
	}


	void CodeRunnerBegin::StoreBool(const FCacheAddressInline& To, bool Value)
	{
		if (!To.At)
		{
			return;
		}
		
		Executed.Add(To);

		FStackValue StackValue;
		StackValue.Set<bool>(Value);
		Stack.Push(StackValue);
			
		ResultsBool.Add(To, Value);
	}




	void CodeRunnerBegin::StoreNone(const FCacheAddressInline& To)
	{
		if (!To.At)
		{
			return;
		}
		
		Executed.Add(To);
	}


	int32 CodeRunnerBegin::LoadInt(const FCacheAddressInline& To)
	{
		if (!To.At)
		{
			return 0;	
		}
		
		return Stack.Pop().Get<int32>();
	}


	float CodeRunnerBegin::LoadFloat(const FCacheAddressInline& To)
	{
		if (!To.At)
		{
			return 0.0f;	
		}

		return Stack.Pop().Get<float>();
	}


	bool CodeRunnerBegin::LoadBool(const FCacheAddressInline& To)
	{
		if (!To.At)
		{
			return false;	
		}

		return Stack.Pop().Get<bool>();
	}


	void CodeRunnerBegin::RunCode(const FScheduledOpInline& Root)
		{
		MUTABLE_CPUPROFILER_SCOPE(CodeRunnerBegin::RunCode);
		
		PushOp(Root);
		
		while (!Items.IsEmpty())
		{
			const FScheduledOpInline Item = PopOp();
			
			const EOpType Type = Program.GetOpType(Item.At);

			if (Executed.Contains(Item))
			{
				bool bExecute = false;
				
				switch (GetOpDataType(Type))
				{
				case EDataType::Bool:
					{
						if (Item.bEvaluate)
						{
							if (bool* Result = ResultsBool.Find(Item))
							{
								FStackValue Value;
								Value.Set<bool>(*Result);
					
								Stack.Push(Value);
							}
							else
							{
								bExecute = true;
							}
						}
						
						break;
					}

				case EDataType::Int:
					{
						if (Item.bEvaluate)
						{
							if (int32* Result = ResultsInt.Find(Item))
							{
								FStackValue Value;
								Value.Set<int32>(*Result);
					
								Stack.Push(Value);
							}
							else
							{
								bExecute = true;
							}
						}
						
						break;
					}
					
				case EDataType::Scalar:
					{
						if (Item.bEvaluate)
						{
							if (float* Result = ResultsFloat.Find(Item))
							{
								FStackValue Value;
								Value.Set<float>(*Result);
					
								Stack.Push(Value);
							}
							else
							{
								bExecute = true;
							}
						}
						
						break;
					}

					
				case EDataType::ExtensionData:
				case EDataType::Color:		
				case EDataType::Projector:	
				case EDataType::Mesh:		
				case EDataType::Image:		
				case EDataType::Layout:		
				case EDataType::InstancedStruct:		
				case EDataType::SkeletalMesh:		
				case EDataType::Instance:
				case EDataType::String:
				case EDataType::Material:
				case EDataType::LOD:
					break;

				default:
					unimplemented();
				}
				
				if (!bExecute)
				{
					continue;
				}
			}
			
			switch (Type)
			{
			case EOpType::CO_CONSTANT:
			case EOpType::IM_CONSTANT:
			case EOpType::LA_CONSTANT:
			case EOpType::MA_CONSTANT:
			case EOpType::PR_CONSTANT:
			case EOpType::CO_PARAMETER:
			case EOpType::MA_PARAMETER:
			case EOpType::PR_PARAMETER:
			case EOpType::SK_PARAMETER:
			case EOpType::MI_PARAMETER:
			case EOpType::IM_PARAMETER:
			case EOpType::IM_PARAMETER_FROM_MATERIAL:
			case EOpType::IS_PARAMETER:
				{
					check(!Item.bEvaluate);

					StoreNone(Item);
					break;
				}

			case EOpType::IN_CONDITIONAL:
				check(Item.bEvaluate);

			case EOpType::CO_CONDITIONAL:
			case EOpType::ED_CONDITIONAL:
			case EOpType::IM_CONDITIONAL:
			case EOpType::LA_CONDITIONAL:
			case EOpType::ME_CONDITIONAL:
			case EOpType::SC_CONDITIONAL:
			case EOpType::MI_CONDITIONAL:
			case EOpType::NU_CONDITIONAL:
			case EOpType::SK_CONDITIONAL:
			case EOpType::IS_CONDITIONAL:
				{
					FOperation::ConditionalArgs Args = Program.GetOpArgs<FOperation::ConditionalArgs>(Item.At);

					switch(Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Item,1));
							PushOp(FScheduledOpInline(Args.condition, Item, true));

							break;
						}

					case 1:
						{
							const bool Value = LoadBool(Args.condition);

							FOperation::ADDRESS ResultAt = Value ? Args.yes : Args.no;

							// Schedule the end of this instruction if necessary
							FScheduledOpInline NextStage(Item, 2);
							NextStage.CustomState = ResultAt;
							PushOp(NextStage);
							
							PushOp(FScheduledOpInline(ResultAt, Item));
							
							break;
						}

					case 2:
						{
							if (Item.bEvaluate)
							{
								switch (GetOpDataType(Type))
								{
								case EDataType::Int:
									StoreInt(Item, LoadInt(Item.CustomState));
									break;
							
								case EDataType::Scalar:		
									StoreFloat(Item, LoadFloat(Item.CustomState));
									break;
							
								case EDataType::ExtensionData:
								case EDataType::Color:		
								case EDataType::Projector:	
								case EDataType::Mesh:		
								case EDataType::Image:		
								case EDataType::Layout:		
								case EDataType::String:
								case EDataType::Instance:
								case EDataType::Material:
								case EDataType::SkeletalMesh:
									StoreNone(Item);
									break;

								case EDataType::Bool:
									check(false);
								
								default:
									unimplemented();
								}
							}
							else
							{
								StoreNone(Item);
							}
							
							break;
						}

					default:
						unimplemented();
					}
				
					break;
				}
			
			case EOpType::CO_SWITCH:
			case EOpType::ED_SWITCH:
			case EOpType::IM_SWITCH:
			case EOpType::IN_SWITCH:
			case EOpType::LA_SWITCH:
			case EOpType::ME_SWITCH:
			case EOpType::NU_SWITCH:
			case EOpType::SC_SWITCH:
			case EOpType::MI_SWITCH:
			case EOpType::IS_SWITCH:
			case EOpType::SK_SWITCH:
			case EOpType::LD_SWITCH:
				{
					const uint8* Data = Program.GetOpArgsPointer(Item.At);

					FOperation::ADDRESS VarAddress;
					FMemory::Memcpy(&VarAddress, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					FOperation::ADDRESS DefAddress;
					FMemory::Memcpy(&DefAddress, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					FOperation::FSwitchCaseDescriptor CaseDesc;
					FMemory::Memcpy(&CaseDesc, Data, sizeof(FOperation::FSwitchCaseDescriptor));
					Data += sizeof(FOperation::FSwitchCaseDescriptor);

					switch (Item.Stage)
					{
					case 0:
						{
							if (VarAddress)
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(VarAddress, Item, true));
							}
							else
							{
								if (Item.bEvaluate)
								{
									switch (GetOpDataType(Type))
									{
									case EDataType::Bool:
										StoreBool(Item, false);
										break;
								
									case EDataType::Int:
										StoreInt(Item, 0);
										break;
								
									case EDataType::Scalar:
										StoreFloat(Item, 0.0f);
										break;

									case EDataType::ExtensionData:
									case EDataType::Color:			
									case EDataType::Projector:		
									case EDataType::Mesh:			
									case EDataType::Image:			
									case EDataType::Layout:			
									case EDataType::InstancedStruct:			
									case EDataType::Instance:
									case EDataType::String:
									case EDataType::Material:
									case EDataType::SkeletalMesh:
										StoreNone(Item);
										break;
								
									default:
										unimplemented()
									}
								}
								else
								{
									StoreNone(Item);
								}
							}
							break;
						}

					case 1:
						{
							// Get the variable result
							int32 Var = LoadInt(VarAddress);

							FOperation::ADDRESS ValueAt = DefAddress;

							if (!CaseDesc.bUseRanges)
							{
								for (uint32 C = 0; C < CaseDesc.Count; ++C)
								{
									int32 Condition;
									FMemory::Memcpy(&Condition, Data, sizeof(int32));
									Data += sizeof(int32);

									FOperation::ADDRESS CaseAt;
									FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
									Data += sizeof(FOperation::ADDRESS);

									if (CaseAt && Var == (int32)Condition)
									{
										ValueAt = CaseAt;
										break;
									}
								}
							}
							else
							{
								for (uint32 C = 0; C < CaseDesc.Count; ++C)
								{
									int32 ConditionStart;
									FMemory::Memcpy(&ConditionStart, Data, sizeof(int32));
									Data += sizeof(int32);

									uint32 RangeSize;
									FMemory::Memcpy(&RangeSize, Data, sizeof(uint32));
									Data += sizeof(uint32);

									FOperation::ADDRESS CaseAt;
									FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
									Data += sizeof(FOperation::ADDRESS);

									if (CaseAt && Var >= ConditionStart && Var < int32(ConditionStart + RangeSize))
									{
										ValueAt = CaseAt;
										break;
									}
								}
							}

							// Schedule the end of this instruction if necessary
							FScheduledOpInline NextStage(Item, 2);
							NextStage.CustomState = ValueAt;
							PushOp(NextStage);

							PushOp(FScheduledOpInline(ValueAt, Item));
							
							break;
						}
						
					case 2:
						{
							if (Item.bEvaluate)
							{
								switch (GetOpDataType(Type))
								{
								case EDataType::Bool:
									StoreBool(Item, LoadBool(Item.CustomState));
									break;

								case EDataType::Int:
									StoreInt(Item, LoadInt(Item.CustomState));
									break;

								case EDataType::Scalar:
									StoreFloat(Item, LoadFloat(Item.CustomState));
									break;

								case EDataType::ExtensionData:
								case EDataType::Color:
								case EDataType::Projector:
								case EDataType::Mesh:
								case EDataType::Image:
								case EDataType::Layout:
								case EDataType::InstancedStruct:
								case EDataType::Instance:
								case EDataType::String:
								case EDataType::Material:
									StoreNone(Item);
									break;

								default:
									unimplemented();
								}
							}
							else
							{
							}

							break;
						}

					default:
						unimplemented();
					}

					break;
				}

			case EOpType::IN_ADDOVERLAYMATERIAL:
				{
					check(Item.bEvaluate);

					FOperation::InstanceAddOverlayMaterialArgs Args = Program.GetOpArgs<FOperation::InstanceAddOverlayMaterialArgs>(Item.At);
					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Args.Instance, Item));
							PushOp(FScheduledOpInline(Args.Material, Item, false));

							break;
						}

					default:
						unimplemented();
					}

					break;
				}

			case EOpType::IN_ADDOVERRIDEMATERIAL:
				{
					FOperation::InstanceAddOverrideMaterialArgs Args = Program.GetOpArgs<FOperation::InstanceAddOverrideMaterialArgs>(Item.At);
					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Args.Instance, Item));
							PushOp(FScheduledOpInline(Args.Material, Item, false));

							break;
						}

					default:
						unimplemented();
					}

					break;
				}
				
			case EOpType::IN_ADDCOMPONENT:
				{
					check(Item.bEvaluate);

					FOperation::InstanceAddComponentArgs Args = Program.GetOpArgs<FOperation::InstanceAddComponentArgs>(Item.At);
					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Args.Instance, Item));
							PushOp(FScheduledOpInline(Args.Value, Item));
							
							break;
						}

					default:
						unimplemented();
					}

					break;
				}

			case EOpType::LD_NEW:
				{
					check(!Item.bEvaluate);
        		
					FOperation::FLODNewArgs Args = Program.GetOpArgs<FOperation::FLODNewArgs>(Item.At);

					const TArray<FOperation::ADDRESS>& MeshAddresses = Program.ConstantUInt32Lists[Args.Meshes];
        		
					TArray<FScheduledOp> Deps;
					for (FOperation::ADDRESS MeshAddress : MeshAddresses)
					{
						PushOp(FScheduledOpInline(MeshAddress, Item));
					}

					StoreNone(Item);
		            break;
				}

			case EOpType::SK_NEW:
				{
					check(!Item.bEvaluate);

					MUTABLE_CPUPROFILER_SCOPE(SK_NEW);
        		
					FOperation::FSkeletalMeshNewArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshNewArgs>(Item.At);
				
					const TArray<FOperation::ADDRESS>& MaterialAddresses = Program.ConstantUInt32Lists[Args.MaterialSlotMaterials];
					const TArray<FOperation::ADDRESS>& LODsAddresses = Program.ConstantUInt32Lists[Args.LODs];
					
					TArray<FScheduledOp> Deps;
					for (int32 MaterialIndex = 0; MaterialIndex < MaterialAddresses.Num(); ++MaterialIndex)
					{
						const FOperation::ADDRESS MaterialAddress = MaterialAddresses[MaterialIndex];
							
						PushOp(FScheduledOpInline(MaterialAddress, Item));
					}
						
					for (uint8 LODIndex = 0; LODIndex < LODsAddresses.Num(); ++LODIndex)
					{
						const bool bIsSelectedLod = (1 << LODIndex & LodMask ) != 0;
						if (bIsSelectedLod)
						{
							const FOperation::ADDRESS LODAddress = LODsAddresses[LODIndex];
								
							PushOp(FScheduledOpInline(LODAddress, Item));
						}
					}

					StoreNone(Item);
					break;
				}
			case EOpType::SK_CONVERT:
				{
					check(!Item.bEvaluate);
					
					MUTABLE_CPUPROFILER_SCOPE(SK_CONVERT);
					FOperation::FSkeletalMeshConvertArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshConvertArgs>(Item.At);

					PushOp(FScheduledOpInline(Args.SkeletalMeshObject, Item));

					StoreNone(Item);
					break;
				}
				
			case EOpType::SK_MORPH:
				{
					check(!Item.bEvaluate);
					switch (Item.Stage)
					{
					case 0:
					{
						MUTABLE_CPUPROFILER_SCOPE(SK_MORPH);
        		
						FOperation::FSkeletalMeshMorphArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshMorphArgs>(Item.At);
						PushOp(FScheduledOpInline(Args.Base, Item));

						StoreNone(Item);
						break;
					}
					default:
						unimplemented();
					}

					break;
				}

			case EOpType::SK_RESHAPE:
				{
					check(!Item.bEvaluate);
					switch (Item.Stage)
					{
					case 0:
					{
						MUTABLE_CPUPROFILER_SCOPE(SK_RESHAPE);
        		
						FOperation::FSkeletalMeshReshapeArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshReshapeArgs>(Item.At);
						PushOp(FScheduledOpInline(Args.Base, Item));
						PushOp(FScheduledOpInline(Args.BaseShape, Item));
						PushOp(FScheduledOpInline(Args.TargetShape, Item));

						StoreNone(Item);
						break;
					}
					default:
						unimplemented();
					}

					break;
				}
			case EOpType::SK_MATERIALMODIFY:
				{
					check(!Item.bEvaluate);
					switch (Item.Stage)
					{
					case 0:
					{
						MUTABLE_CPUPROFILER_SCOPE(SK_MATERIALMODIFY);
        		
						FOperation::FSkeletalMeshMaterialModifyArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshMaterialModifyArgs>(Item.At);
						PushOp(FScheduledOpInline(Args.SkeletalMesh, Item));
						PushOp(FScheduledOpInline(Args.NewMaterial, Item));

						StoreNone(Item);
						break;
					}
					default:
						unimplemented();
					}

					break;
				}

			case EOpType::SK_MERGE:
			{
				check(!Item.bEvaluate);

				FOperation::FSkeletalMeshMergeArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshMergeArgs>(Item.At);
				PushOp(FScheduledOpInline(Args.BaseMesh, Item));
				PushOp(FScheduledOpInline(Args.AddedMesh, Item));

				StoreNone(Item);
				break;
			}

			case EOpType::SK_CLIPMESHWITHMESH:
				{
					check(!Item.bEvaluate);
					switch (Item.Stage)
					{
					case 0:
					{
						MUTABLE_CPUPROFILER_SCOPE(SK_CLIPMESHWITHMESH);
        		
						FOperation::FSkeletalMeshClipMeshWithMeshArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshClipMeshWithMeshArgs>(Item.At);
						PushOp(FScheduledOpInline(Args.Source, Item));
						PushOp(FScheduledOpInline(Args.Clip, Item));

						StoreNone(Item);
						break;
					}
					default:
						unimplemented();
					}

					break;
				}
				
			case EOpType::SK_TRANSFORM:
				{
					check(!Item.bEvaluate);

					FOperation::FSkeletalMeshTransformArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshTransformArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					//PushOp(FScheduledOpInline(Args.matrix, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::SK_TRANSFORMWITHBONE:
				{
					check(!Item.bEvaluate);
					
					FOperation::FSkeletalMeshTransformWithBoneArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshTransformWithBoneArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.SourceSkeletalMesh, Item));
					//PushOp(FScheduledOpInline(Args.Matrix, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::SKO_CONVERT:
				{
					check(!Item.bEvaluate);	

					MUTABLE_CPUPROFILER_SCOPE(SKO_CONVERT);
        		
					FOperation::FSkeletalMeshObjectConvertArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshObjectConvertArgs>(Item.At);
				
					PushOp(FScheduledOpInline(Args.SkeletalMesh, Item));

					StoreNone(Item);
					break;
				}
				
			case EOpType::IN_ADDSKELETALMESH:
				{
					check(Item.bEvaluate);

					FOperation::InstanceAddArgs Args = Program.GetOpArgs<FOperation::InstanceAddArgs>(Item.At);
					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Args.instance, Item));
							PushOp(FScheduledOpInline(Args.value, Item, false));
							
							break;
						}

					default:
						unimplemented();
					}

					break;
				}
				
			case EOpType::IN_ADDEXTENSIONDATA:
				{
					check(Item.bEvaluate);

					FOperation::InstanceAddExtensionDataArgs Args = Program.GetOpArgs<FOperation::InstanceAddExtensionDataArgs>(Item.At);

					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Args.Instance, Item));
							PushOp(FScheduledOpInline(Args.ExtensionData, Item));

							break;
						}

					default:
						unimplemented();
					}

					break;
				}
				
			case EOpType::BO_CONSTANT:
				{
					if (Item.bEvaluate)
					{
						FOperation::BoolConstantArgs Args = Program.GetOpArgs<FOperation::BoolConstantArgs>(Item.At);

						StoreBool(Item, Args.bValue);
					}
					else
					{
						StoreNone(Item);	
					}
					
					break;
				}

			case EOpType::BO_PARAMETER:
				{
					FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);

					if (Item.bEvaluate)
					{
						FScheduledOp Op;
						Op.At = Item.At;
						Op.ExecutionIndex = Item.ExecutionIndex;
        		
						TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Op, *Params, *Model.Get(), *ProgramCache, Args.variable);
						const bool Value = Params->GetBoolValue(Args.variable, Index.Get());
					
						StoreBool(Item, Value);
					}
					else
					{
						StoreNone(Item);	
					}
					
					break;
				}

			case EOpType::BO_AND:
				{
					FOperation::BoolBinaryArgs Args = Program.GetOpArgs<FOperation::BoolBinaryArgs>(Item.At);

					if (Item.bEvaluate)
					{
						switch (Item.Stage)
						{
						case 0:
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(Args.A, Item));
								PushOp(FScheduledOpInline(Args.B, Item));

								break;
							}

						case 1:
							{
								const bool ValueB = LoadBool(Args.A);
								const bool ValueA = LoadBool(Args.B);

								const bool Result = ValueA && ValueB;

								StoreBool(Item, Result);
								break;
							}

						default:
							unimplemented();
						}
					}
					else
					{
						PushOp(FScheduledOpInline(Args.A, Item));
						PushOp(FScheduledOpInline(Args.B, Item));
						StoreNone(Item);	
					}
					
					break;
				}

			case EOpType::BO_OR:
				{
					FOperation::BoolBinaryArgs Args = Program.GetOpArgs<FOperation::BoolBinaryArgs>(Item.At);

					if (Item.bEvaluate)
					{
						switch (Item.Stage)
						{
						case 0:
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(Args.A, Item));
								PushOp(FScheduledOpInline(Args.B, Item));

								break;
							}

						case 1:
							{
								const bool ValueB = LoadBool(Args.A);
								const bool ValueA = LoadBool(Args.B);

								const bool Result = ValueA || ValueB;
							
								StoreBool(Item, Result);
								break;
							}

						default:
							unimplemented();
						}	
					}
					else
					{
						PushOp(FScheduledOpInline(Args.A, Item));
						PushOp(FScheduledOpInline(Args.B, Item));
						StoreNone(Item);	
					}
					
					break;
				}

			case EOpType::BO_NOT:
				{
					FOperation::BoolNotArgs Args = Program.GetOpArgs<FOperation::BoolNotArgs>(Item.At);

					if (Item.bEvaluate)
					{
						switch (Item.Stage)
						{
						case 0:
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(Args.A, Item));

								break;
							}
					
						case 1:
							{
								const bool Value = LoadBool(Args.A);
							
								const bool Result = !Value;
							
								StoreBool(Item, Result);
								break;
							}

						default:
							unimplemented();
						}
					}
					else
					{
						PushOp(FScheduledOpInline(Args.A, Item));
						StoreNone(Item);	
					}
					
					break;
				}

			case EOpType::BO_EQUAL_INT_CONST:
				{
					FOperation::BoolEqualScalarConstArgs Args = Program.GetOpArgs<FOperation::BoolEqualScalarConstArgs>(Item.At);

					if (Item.bEvaluate)
					{
						switch (Item.Stage)
						{
						case 0:
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(Args.Value, Item));

								break;
							}
					
						case 1:
							{
								const int32 Value = LoadInt(Args.Value);
							
								const bool Result = Value == Args.Constant;

								StoreBool(Item, Result);
								break;
							}

						default:
							unimplemented();
						}
					}
					else
					{
						PushOp(FScheduledOpInline(Args.Value, Item));
						StoreNone(Item);	
					}
					
					break;
				}

			case EOpType::SC_CONSTANT:
				{
					if (Item.bEvaluate)
					{
						FOperation::ScalarConstantArgs Args = Program.GetOpArgs<FOperation::ScalarConstantArgs>(Item.At);
						
						StoreFloat(Item, Args.Value);
					}
					else
					{
						StoreNone(Item);
					}

					break;
				}
			
				
			case EOpType::SC_PARAMETER:
				{
					FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);

					if (Item.bEvaluate)
					{
						FScheduledOp Op;
						Op.At = Item.At;
						Op.ExecutionIndex = Item.ExecutionIndex;
					
						TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Op, *Params, *Model.Get(), *ProgramCache, Args.variable);
						const float Result = Params->GetFloatValue( Args.variable, Index.Get());

						StoreFloat(Item, Result);
					}
					else
					{
						StoreNone(Item);
					}
					
					break;
				}
				
			case EOpType::SC_ARITHMETIC:
				{
					FOperation::ArithmeticArgs Args = Program.GetOpArgs<FOperation::ArithmeticArgs>(Item.At);

					if (Item.bEvaluate)
					{
						switch (Item.Stage)
						{
						case 0:
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(Args.A, Item));
								PushOp(FScheduledOpInline(Args.B, Item));

								break;
							}

						case 1:
							{
								float ValueA = LoadFloat(Args.A);
								float ValueB = LoadFloat(Args.B);

								float Result = 1.0f;
								switch (Args.Operation)
								{
								case FOperation::ArithmeticArgs::ADD:
									Result = ValueA + ValueB;
									break;

								case FOperation::ArithmeticArgs::MULTIPLY:
									Result = ValueA * ValueB;
									break;

								case FOperation::ArithmeticArgs::SUBTRACT:
									Result = ValueA - ValueB;
									break;

								case FOperation::ArithmeticArgs::DIVIDE:
									Result = ValueA / ValueB;
									break;

								default:
									unimplemented();
								}

								StoreFloat(Item, Result);
								break;
							}

						default:
							unimplemented();
						}
					}
					else
					{
						PushOp(FScheduledOpInline(Args.A, Item));
						PushOp(FScheduledOpInline(Args.B, Item));
						StoreNone(Item);
					}
					
					break;
				}
			
			case EOpType::SC_CURVE:
				{
					FOperation::ScalarCurveArgs Args = Program.GetOpArgs<FOperation::ScalarCurveArgs>(Item.At);

					if (Item.bEvaluate)
					{
						switch (Item.Stage)
						{
						case 0:
							{
								PushOp(FScheduledOpInline(Item, 1));
								PushOp(FScheduledOpInline(Args.time, Item));

								break;
							}

						case 1:
							{
								const float Time = LoadFloat(Args.time);

								const FRichCurve& Curve = Program.ConstantCurves[Args.curve];
								float Result = Curve.Eval(Time);

								StoreFloat(Item, Result);
								break;
							}

						default:
							unimplemented();
						}	
					}
					else
					{
						PushOp(FScheduledOpInline(Args.time, Item));
						StoreNone(Item);
					}
					
					break;
				}
				
			case EOpType::ED_CONSTANT:
				{
					check(Item.bEvaluate);

					FOperation::ExternalDataConstantArgs Args = Program.GetOpArgs<FOperation::ExternalDataConstantArgs>(Item.At);
					PassthroughObjectLoader->Add<UObject>(Args.ExternalObjectId);

					break;
				}

			case EOpType::ST_CONSTANT:
				{
					StoreNone(Item);
					break;
				}

			case EOpType::ST_PARAMETER:
				{
					StoreNone(Item);
					break;
				}

			case EOpType::NU_CONSTANT:
				{
					if (Item.bEvaluate)
					{
						FOperation::IntConstantArgs Args = Program.GetOpArgs<FOperation::IntConstantArgs>(Item.At);

						const int32 Value = Args.Value;
					
						StoreInt(Item, Value);
					}
					else
					{
						StoreNone(Item);
					}
					
					break;
				}

			case EOpType::NU_PARAMETER:
				{
					FOperation::ParameterArgs Args = Program.GetOpArgs<FOperation::ParameterArgs>(Item.At);

					if (Item.bEvaluate)
					{
						const int32 Value = Params->GetIntValue(Args.variable);

						StoreInt(Item, Value);
					}
					else
					{
						StoreNone(Item);
					}
					
					break;
				}
			
			case EOpType::CO_ARITHMETIC:
				{
					check(!Item.bEvaluate);

					FOperation::ArithmeticArgs Args = Program.GetOpArgs<FOperation::ArithmeticArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.A, Item));
					PushOp(FScheduledOpInline(Args.B, Item));

					StoreNone(Item);
					break;	
				}
	
			case EOpType::CO_FROMSCALARS:
				{
					check(!Item.bEvaluate);

					StoreNone(Item);
					break;
				}
				
			case EOpType::CO_SAMPLEIMAGE:
				{
					check(!Item.bEvaluate);

					FOperation::ColorSampleImageArgs Args = Program.GetOpArgs<FOperation::ColorSampleImageArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Image, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::CO_SWIZZLE:
				{
					check(!Item.bEvaluate);

					FOperation::ColorSwizzleArgs Args = Program.GetOpArgs<FOperation::ColorSwizzleArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.sources[0], Item));
					PushOp(FScheduledOpInline(Args.sources[1], Item));
					PushOp(FScheduledOpInline(Args.sources[2], Item));
					PushOp(FScheduledOpInline(Args.sources[3], Item));

					StoreNone(Item);
					break;
				}

			case EOpType::ME_CONSTANT:
				{
					check(!Item.bEvaluate);

					FOperation::MeshConstantArgs Args = Program.GetOpArgs<FOperation::MeshConstantArgs>(Item.At);
					PassthroughObjectLoader->Add<UMaterialInterface>(Args.ClothID);
					
					StoreNone(Item);
					break;
				}
			
			case EOpType::ME_APPLYLAYOUT:
				{
					check(!Item.bEvaluate);

					FOperation::MeshApplyLayoutArgs Args = Program.GetOpArgs<FOperation::MeshApplyLayoutArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Mesh, Item));
					//PushOp(FScheduledOpInline(Args.Layout, Item));

					StoreNone(Item);
					break;
				}
	
	
			case EOpType::ME_PREPARELAYOUT:
				{
					check(!Item.bEvaluate);

					FOperation::MeshPrepareLayoutArgs Args = Program.GetOpArgs<FOperation::MeshPrepareLayoutArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Mesh, Item));
					//PushOp(FScheduledOpInline(Args.Layout, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_DIFFERENCE:
				{
					check(!Item.bEvaluate);

					const uint8* data = Program.GetOpArgsPointer(Item.At);

					FOperation::ADDRESS BaseAt = 0;
					FMemory::Memcpy(&BaseAt, data, sizeof(FOperation::ADDRESS)); 
					data += sizeof(FOperation::ADDRESS);

					FOperation::ADDRESS TargetAt = 0;
					FMemory::Memcpy(&TargetAt, data, sizeof(FOperation::ADDRESS)); 
					data += sizeof(FOperation::ADDRESS);

					PushOp(FScheduledOpInline(BaseAt, Item));
					PushOp(FScheduledOpInline(TargetAt, Item));

					StoreNone(Item);	
					break;
				}
	
			case EOpType::ME_MORPH:
				{
					check(!Item.bEvaluate);

					FOperation::MeshMorphArgs Args = Program.GetOpArgs<FOperation::MeshMorphArgs>(Item.At);

					//PushOp(FScheduledOpInline(Args.Factor, Item));
					PushOp(FScheduledOpInline(Args.Base, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_MERGE:
				{
					check(!Item.bEvaluate);

					FOperation::MeshMergeArgs Args = Program.GetOpArgs<FOperation::MeshMergeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));
					PushOp(FScheduledOpInline(Args.Added, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_MASKCLIPMESH:
				{
					check(!Item.bEvaluate);

					FOperation::MeshMaskClipMeshArgs Args = Program.GetOpArgs<FOperation::MeshMaskClipMeshArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.source, Item));
					PushOp(FScheduledOpInline(Args.clip, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_MASKCLIPUVMASK:
				{
					check(!Item.bEvaluate);

					FOperation::MeshMaskClipUVMaskArgs Args = Program.GetOpArgs<FOperation::MeshMaskClipUVMaskArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					PushOp(FScheduledOpInline(Args.UVSource, Item));
					PushOp(FScheduledOpInline(Args.MaskImage, Item));
					PushOp(FScheduledOpInline(Args.LayoutIndex, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_MASKDIFF:
				{
					check(!Item.bEvaluate);

					FOperation::MeshMaskDiffArgs Args = Program.GetOpArgs<FOperation::MeshMaskDiffArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					PushOp(FScheduledOpInline(Args.Fragment, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::ME_FORMAT:
				{
					check(!Item.bEvaluate);

					FOperation::MeshFormatArgs Args = Program.GetOpArgs<FOperation::MeshFormatArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.source, Item));
					PushOp(FScheduledOpInline(Args.format, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_EXTRACTLAYOUTBLOCK:
				{
					check(!Item.bEvaluate);

					const uint8* Data = Program.GetOpArgsPointer(Item.At);

					FOperation::ADDRESS Source;
					FMemory::Memcpy( &Source, Data, sizeof(FOperation::ADDRESS) );
					Data += sizeof(FOperation::ADDRESS);
					
					PushOp(FScheduledOpInline(Source, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_TRANSFORM:
				{
					check(!Item.bEvaluate);

					FOperation::MeshTransformArgs Args = Program.GetOpArgs<FOperation::MeshTransformArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.source, Item));
					//PushOp(FScheduledOpInline(Args.matrix, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_CLIPMORPHPLANE:
				{
					check(!Item.bEvaluate);

					FOperation::MeshClipMorphPlaneArgs Args = Program.GetOpArgs<FOperation::MeshClipMorphPlaneArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));

					StoreNone(Item);
					break;
				}
			
			case EOpType::ME_CLIPWITHMESH:
				{
					check(!Item.bEvaluate);

					FOperation::MeshClipWithMeshArgs Args = Program.GetOpArgs<FOperation::MeshClipWithMeshArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					PushOp(FScheduledOpInline(Args.ClipMesh, Item));

					StoreNone(Item);
					break;
				}
				
			case EOpType::ME_CLIPDEFORM:
				{
					check(!Item.bEvaluate);

					FOperation::MeshClipDeformArgs Args = Program.GetOpArgs<FOperation::MeshClipDeformArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.mesh, Item));
					PushOp(FScheduledOpInline(Args.clipShape, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::ME_APPLYPOSE:
				{
					check(!Item.bEvaluate);

					FOperation::MeshApplyPoseArgs Args = Program.GetOpArgs<FOperation::MeshApplyPoseArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.base, Item));
					PushOp(FScheduledOpInline(Args.pose, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_BINDSHAPE:
				{
					check(!Item.bEvaluate);

					FOperation::MeshBindShapeArgs Args = Program.GetOpArgs<FOperation::MeshBindShapeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.mesh, Item));
					PushOp(FScheduledOpInline(Args.shape, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_APPLYSHAPE:
				{
					check(!Item.bEvaluate);

					FOperation::MeshApplyShapeArgs Args = Program.GetOpArgs<FOperation::MeshApplyShapeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.mesh, Item));
					PushOp(FScheduledOpInline(Args.shape, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::ME_REMOVEMASK:
				{
					check(!Item.bEvaluate);

					const uint8* Data = Program.GetOpArgsPointer(Item.At);

					FOperation::ADDRESS Source;
					FMemory::Memcpy(&Source,Data,sizeof(FOperation::ADDRESS)); 
					Data += sizeof(FOperation::ADDRESS);

					PushOp(FScheduledOpInline(Source, Item));

					EFaceCullStrategy FaceCullStrategy;
					FMemory::Memcpy(&FaceCullStrategy, Data, sizeof(EFaceCullStrategy));
					Data += sizeof(EFaceCullStrategy);

					uint16 NumRemoves;
					FMemory::Memcpy(&NumRemoves,Data,sizeof(uint16)); 
					Data += sizeof(uint16);

					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Item, 1));

							for (uint16 Index = 0; Index < NumRemoves; ++Index)
							{
								FOperation::ADDRESS Condition;
								FMemory::Memcpy(&Condition, Data, sizeof(FOperation::ADDRESS)); 
								Data += sizeof(FOperation::ADDRESS);

								if (Condition)
								{
									PushOp(FScheduledOpInline(Condition, Item, true));								
								}

								FOperation::ADDRESS Mask;
								FMemory::Memcpy(&Mask, Data,sizeof(FOperation::ADDRESS)); 
								Data += sizeof(FOperation::ADDRESS);
							}

							break;
						}

					case 1:
						{
							for (uint16 Index = 0; Index < NumRemoves; ++Index)
							{
								FOperation::ADDRESS Condition;
								FMemory::Memcpy(&Condition,Data,sizeof(FOperation::ADDRESS)); 
								Data += sizeof(FOperation::ADDRESS);
								
								FOperation::ADDRESS Mask;
								FMemory::Memcpy(&Mask,Data,sizeof(FOperation::ADDRESS)); 
								Data += sizeof(FOperation::ADDRESS);

								bool bValue = true;
								if (Condition)
								{
								 	bValue = LoadBool(Condition);
								}
								
								if (bValue)
								{
									PushOp(FScheduledOpInline(Mask, Item));
								}
							}

							StoreNone(Item);
							break;
						}

					default:
						unimplemented()		
					}
					
					break;
				}

			case EOpType::ME_ADDMETADATA:
				{
					check(!Item.bEvaluate);

					const FOperation::MeshAddMetadataArgs OpArgs = Program.GetOpArgs<FOperation::MeshAddMetadataArgs>(Item.At);
					PushOp(FScheduledOpInline(OpArgs.Source, Item));

					if (OpArgs.SkeletonId != PASSTHROUGH_ID_INVALID)
					{
						PassthroughObjectLoader->Add<USkeleton>(OpArgs.SkeletonId);
					}
					using OpEnumFlags = FOperation::MeshAddMetadataArgs::EnumFlags;

					if (OpArgs.PhysicsAssetId != PASSTHROUGH_ID_INVALID)
					{
						PassthroughObjectLoader->Add<UPhysicsAsset>(OpArgs.PhysicsAssetId);
					}

					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsAdditionalPhysicsAssetList))
					{
						const TArray<PASSTHROUGH_ID>& ConstantPassthroughIdList = Program.ConstantUInt32Lists[OpArgs.AdditionalPhysicsAssetsIds.ListAddress];
						
						for (uint32 PassthroughId : ConstantPassthroughIdList)
						{
							PassthroughObjectLoader->Add<UPhysicsAsset>(PassthroughId);
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasAdditionalPhysicsAsset))
					{
						PassthroughObjectLoader->Add<UPhysicsAsset>(OpArgs.AdditionalPhysicsAssetsIds.PassthroughId);
					}

					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsAssetUserDataList))
					{
						const TArray<PASSTHROUGH_ID>& ConstantPassthroughIdList = Program.ConstantUInt32Lists[OpArgs.AssetUserDataIds.ListAddress];
						
						for (uint32 PassthroughId : ConstantPassthroughIdList)
						{
							PassthroughObjectLoader->Add<UAssetUserData>(PassthroughId);
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasAssetUserData))
					{
						PassthroughObjectLoader->Add<UAssetUserData>(OpArgs.AssetUserDataIds.PassthroughId);
					}
					
					if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::IsAnimationSlotList))
					{
						const TArray<PASSTHROUGH_ID>& ConstantPassthroughIdList = Program.ConstantUInt32Lists[OpArgs.AnimInstances.ListAddress];
						
						for (uint32 PassthroughId : ConstantPassthroughIdList)
						{
							PassthroughObjectLoader->Add<UAnimInstance>(PassthroughId);
						}
					}
					else if (EnumHasAnyFlags(OpArgs.Flags, OpEnumFlags::HasAnimationSlots))
					{
						PassthroughObjectLoader->Add<UAnimInstance>(OpArgs.AnimInstances.PassthroughId);
					}
					
					StoreNone(Item);
					break;
				}
			
			case EOpType::ME_SETMATERIALSLOTID:
			{
				check(!Item.bEvaluate);

				FOperation::MeshSetMaterialSlotIdArgs Args = Program.GetOpArgs<FOperation::MeshSetMaterialSlotIdArgs>(Item.At);
				PushOp(FScheduledOpInline(Args.Mesh, Item));

				StoreNone(Item);
				break;
			}

			case EOpType::ME_PROJECT:
				{
					check(!Item.bEvaluate);

					FOperation::MeshProjectArgs Args = Program.GetOpArgs<FOperation::MeshProjectArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Mesh, Item));
					PushOp(FScheduledOpInline(Args.Projector, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::ME_TRANSFORMWITHMESH:
				{
					check(!Item.bEvaluate);
					
					FOperation::MeshTransformWithinMeshArgs Args = Program.GetOpArgs<FOperation::MeshTransformWithinMeshArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.sourceMesh, Item));
					PushOp(FScheduledOpInline(Args.boundingMesh, Item));
					//PushOp(FScheduledOpInline(Args.matrix, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::ME_TRANSFORMWITHBONE:
				{
					check(!Item.bEvaluate);
					
					FOperation::MeshTransformWithBoneArgs Args = Program.GetOpArgs<FOperation::MeshTransformWithBoneArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.SourceMesh, Item));
					//PushOp(FScheduledOpInline(Args.Matrix, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::ME_SKELETALMESH_BREAK:
				{
					check(!Item.bEvaluate);
					
					FOperation::MeshSkeletalMeshBreakArgs Args = Program.GetOpArgs<FOperation::MeshSkeletalMeshBreakArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.SkeletalMeshParameter, Item));

					StoreNone(Item);
					break;
				}
			
			
			case EOpType::IM_REFERENCE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ResourceReferenceArgs Args = Program.GetOpArgs<FOperation::ResourceReferenceArgs>(Item.At);

					if (!Args.ForceLoad) // Not a passthrough. Only happens with None optimization.
					{
						PassthroughObjectLoader->Add<UTexture>(Args.ID);
					}

					StoreNone(Item);
					break;
				}
				
			case EOpType::IM_LAYERCOLOR:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageLayerColorArgs Args = Program.GetOpArgs<FOperation::ImageLayerColorArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.base, Item));
					PushOp(FScheduledOpInline(Args.mask, Item));
					//PushOp(FScheduledOpInline(Args.color, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_LAYER:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageLayerArgs Args = Program.GetOpArgs<FOperation::ImageLayerArgs>(Item.At);	
					PushOp(FScheduledOpInline(Args.base, Item));
					PushOp(FScheduledOpInline(Args.mask, Item));
					PushOp(FScheduledOpInline(Args.blended, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_MULTILAYER:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageMultiLayerArgs Args = Program.GetOpArgs<FOperation::ImageMultiLayerArgs>(Item.At);

					switch (Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Item, 1));
							PushOp(FScheduledOpInline(Args.rangeSize, Item, true));
							PushOp(FScheduledOpInline(Args.base, Item));

							break;
						}

					case 1:
						{
							int32 NumIterations = 0;
							if (Args.rangeSize)
							{
								const EDataType RangeSizeType = GetOpDataType(Model->GetProgram().GetOpType(Args.rangeSize) );
								if (RangeSizeType == EDataType::Int)
								{
									NumIterations = LoadInt(Args.rangeSize);
								}
								else if (RangeSizeType == EDataType::Scalar)
								{
									NumIterations = LoadFloat(Args.rangeSize);
								}
							}

							for (int32 IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
							{
								FScheduledOpInline ItemCopy = Item;
								ItemCopy.ExecutionIndex = ProgramCache->GetOrAddModifiedExecutionIndex(
										Item.ExecutionIndex, Args.rangeId, IterationIndex);

								PushOp(FScheduledOpInline(Args.mask, Item));
								PushOp(FScheduledOpInline(Args.blended, Item));
							}

							StoreNone(Item);
							break;
						}

					default:
						unimplemented()
					}

					break;
				}

			case EOpType::IM_NORMALCOMPOSITE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageNormalCompositeArgs Args = Program.GetOpArgs<FOperation::ImageNormalCompositeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.base, Item));
					PushOp(FScheduledOpInline(Args.normal, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_PIXELFORMAT:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImagePixelFormatArgs Args = Program.GetOpArgs<FOperation::ImagePixelFormatArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.source, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_CONFORM:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageConformArgs Args = Program.GetOpArgs<FOperation::ImageConformArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));

					StoreNone(Item);
					break;
				}


			case EOpType::IM_MIPMAP:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageMipmapArgs Args = Program.GetOpArgs<FOperation::ImageMipmapArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.source, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_RESIZE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageResizeArgs Args = Program.GetOpArgs<FOperation::ImageResizeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_RESIZELIKE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageResizeLikeArgs Args = Program.GetOpArgs<FOperation::ImageResizeLikeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					PushOp(FScheduledOpInline(Args.SizeSource, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_RESIZEREL:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageResizeRelArgs Args = Program.GetOpArgs<FOperation::ImageResizeRelArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_BLANKLAYOUT:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageBlankLayoutArgs Args = Program.GetOpArgs<FOperation::ImageBlankLayoutArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Layout, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_COMPOSE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageComposeArgs Args = Program.GetOpArgs<FOperation::ImageComposeArgs>(Item.At);
					//PushOp(FScheduledOpInline(Args.layout, Item));
					PushOp(FScheduledOpInline(Args.base, Item));
					PushOp(FScheduledOpInline(Args.blockImage, Item)); 
					//PushOp(FScheduledOpInline(Args.mask, Item)); 

					StoreNone(Item);
					break;
				}

			case EOpType::IM_MULTICOMPOSE:
			{
				FOperation::ImageMultiComposeArgs Args = Program.GetOpArgs<FOperation::ImageMultiComposeArgs>(Item.At);
				PushOp(FScheduledOpInline(Args.Base, Item));
				PushOp(FScheduledOpInline(Args.SourceImage, Item));

				StoreNone(Item);
				break;
			}
	
			case EOpType::IM_INTERPOLATE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageInterpolateArgs Args = Program.GetOpArgs<FOperation::ImageInterpolateArgs>(Item.At);

					//PushOp(FScheduledOpInline(Args.Factor, Item));
					
					for (int32 ImageIndex = 0; ImageIndex < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++ImageIndex)
					{
						if (!Args.Targets[ImageIndex])
						{
							break;
						}

						PushOp(FScheduledOpInline(Args.Targets[ImageIndex], Item));
					}

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_SATURATE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageSaturateArgs Args = Program.GetOpArgs<FOperation::ImageSaturateArgs>(Item.At);
					//PushOp(FScheduledOpInline(Args.Factor, Item, false));
					PushOp(FScheduledOpInline(Args.Base, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_LUMINANCE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageLuminanceArgs Args = Program.GetOpArgs<FOperation::ImageLuminanceArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_SWIZZLE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageSwizzleArgs Args = Program.GetOpArgs<FOperation::ImageSwizzleArgs>(Item.At);
				
					TArray<FOperation::ADDRESS, TFixedAllocator<4>> ValidArgs;
					for (int32 SourceIndex = 0; SourceIndex < 4; ++SourceIndex)
					{
						if (Args.sources[SourceIndex])
						{
							PushOp(FScheduledOpInline(Args.sources[SourceIndex], Item));
						}
					}

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_COLORMAP:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageColorMapArgs Args = Program.GetOpArgs<FOperation::ImageColorMapArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));
					//PushOp(FScheduledOpInline(Args.Mask, Item));
					//PushOp(FScheduledOpInline(Args.Map, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_BINARISE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageBinariseArgs Args = Program.GetOpArgs<FOperation::ImageBinariseArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));
					//PushOp(FScheduledOpInline(Args.Threshold, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_INVERT:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageInvertArgs Args = Program.GetOpArgs<FOperation::ImageInvertArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_PLAINCOLOR:
				{
					check(!Item.bEvaluate);
					
					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_CROP:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageCropArgs Args = Program.GetOpArgs<FOperation::ImageCropArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.source, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_PATCH:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImagePatchArgs Args = Program.GetOpArgs<FOperation::ImagePatchArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.base, Item));
					PushOp(FScheduledOpInline(Args.patch, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_RASTERMESH:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageRasterMeshArgs Args = Program.GetOpArgs<FOperation::ImageRasterMeshArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.mesh, Item));
					//PushOp(FScheduledOpInline(Args.image, Item));
					//PushOp(FScheduledOpInline(Args.angleFadeProperties, Item));
					//PushOp(FScheduledOpInline(Args.mask, Item));
					//PushOp(FScheduledOpInline(Args.projector, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_MAKEGROWMAP:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageMakeGrowMapArgs Args = Program.GetOpArgs<FOperation::ImageMakeGrowMapArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.mask, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_DISPLACE:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageDisplaceArgs Args = Program.GetOpArgs<FOperation::ImageDisplaceArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					//PushOp(FScheduledOpInline(Args.DisplacementMap, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::IM_TRANSFORM:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageTransformArgs Args = Program.GetOpArgs<FOperation::ImageTransformArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));
					//PushOp(FScheduledOpInline(Args.OffsetX, Item));
					//PushOp(FScheduledOpInline(Args.OffsetY, Item));
					//PushOp(FScheduledOpInline(Args.ScaleX, Item));
					//PushOp(FScheduledOpInline(Args.ScaleY, Item));
					//PushOp(FScheduledOpInline(Args.Rotation, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::IM_PARAMETER_CONVERT:
				{
					check(!Item.bEvaluate);
					
					FOperation::ImageParameterConvertArgs Args = Program.GetOpArgs<FOperation::ImageParameterConvertArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.ImageParameter, Item));

					StoreNone(Item);
					break;
				}
				
			case EOpType::LA_MERGE:
				{
					check(!Item.bEvaluate);
					
					FOperation::LayoutMergeArgs Args = Program.GetOpArgs<FOperation::LayoutMergeArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Base, Item));
					PushOp(FScheduledOpInline(Args.Added, Item));

					StoreNone(Item);
					break;
				}
	
			case EOpType::LA_PACK:
				{
					check(!Item.bEvaluate);
					
					FOperation::LayoutPackArgs Args = Program.GetOpArgs<FOperation::LayoutPackArgs>(Item.At);
					PushOp(FScheduledOpInline( Args.Source, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::LA_FROMMESH:
				{
					check(!Item.bEvaluate);
					
					FOperation::LayoutFromMeshArgs Args = Program.GetOpArgs<FOperation::LayoutFromMeshArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Mesh, Item));

					StoreNone(Item);
					break;
				}

			case EOpType::LA_REMOVEBLOCKS:
				{
					check(!Item.bEvaluate);
					
					FOperation::LayoutRemoveBlocksArgs Args = Program.GetOpArgs<FOperation::LayoutRemoveBlocksArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.Source, Item));
					PushOp(FScheduledOpInline(Args.ReferenceLayout, Item));

					StoreNone(Item);
					break;
				}
			
			case EOpType::NONE:
				{
					check(Item.At == 0);
					
					StoreNone(Item);
					break;
				}

			case EOpType::MI_CONSTANT:
				{
					check(!Item.bEvaluate);
					
					FOperation::MaterialConstantArgs Args = Program.GetOpArgs<FOperation::MaterialConstantArgs>(Item.At);
					PassthroughObjectLoader->Add<UMaterialInterface>(Args.ID);

					StoreNone(Item);
					break;
				}

			case EOpType::SC_MATERIAL_BREAK:
			case EOpType::IM_MATERIAL_BREAK:
			case EOpType::CO_MATERIAL_BREAK:
				{
					FOperation::MaterialBreakArgs Args = Program.GetOpArgs<FOperation::MaterialBreakArgs>(Item.At);

					switch (Item.Stage)
					{
						case 0:
						{
							// Push Material Op
							PushOp(FScheduledOpInline(Args.Material, Item));

							break;
						}
						
						default: 
							unimplemented();
					}

					break;
				}

			case EOpType::SC_EXTERNAL:
			case EOpType::CO_EXTERNAL:
			case EOpType::IM_EXTERNAL:
			case EOpType::ME_EXTERNAL:
			case EOpType::MI_EXTERNAL:
			case EOpType::IS_EXTERNAL:
				{
					check(!Item.bEvaluate);
					
					const uint8* Data = Program.GetOpArgsPointer(Item.At);

					FOperation::ExternalArgs Args = Program.GetOpArgs<FOperation::ExternalArgs>(Item.At);
					Data += sizeof(FOperation::ExternalArgs);

					const FInstancedStruct& OperationInstancedStruct = Program.ExternalOperationProvider->Get(Item.At);
					if (!OperationInstancedStruct.IsValid()) // Plugin not loaded
					{
						StoreNone(Item);
						break;
					}

					const FExternalOperation* Operation = OperationInstancedStruct.GetPtr<FExternalOperation>();

					const TArray<TPair<FText, const UScriptStruct*>>& Inputs = Operation->GetInputs();
					if (Inputs.Num() != Args.NumOperants)
					{
						StoreNone(Item);
						break;
					}

					switch(Item.Stage)
					{
					case 0:
						{
							PushOp(FScheduledOpInline(Item.At, 1, Item.ExecutionIndex, false));

							for (int32 Index = 0; Index < Args.NumOperants; ++Index)
							{
								FOperation::ADDRESS OperantAt;
								FMemory::Memcpy(&OperantAt, Data, sizeof(FOperation::ADDRESS));
								Data += sizeof(FOperation::ADDRESS);

								PushOp(FScheduledOpInline(OperantAt, Item));
							}
							
							break;
						}

					case 1:
						{
							for (int32 Index = 0; Index < Args.NumOperants; ++Index)
							{
								FOperation::ADDRESS OperantAt;
								FMemory::Memcpy(&OperantAt, Data, sizeof(FOperation::ADDRESS));
								Data += sizeof(FOperation::ADDRESS);

								const EOpType InputOpType = Program.GetOpType(OperantAt);
								const EDataType InputDataType = GetOpDataType(InputOpType);
								switch (InputDataType)
								{
								case EDataType::Color:
								case EDataType::Image:
								case EDataType::Mesh:
								case EDataType::InstancedStruct:
								case EDataType::Scalar:
									// Nothing
									break;

								case EDataType::Material:
									break;

								default: 
									unimplemented();
								}
							}

							StoreNone(Item);
							break;
						}
						
					default:
						unimplemented()
					}
					
					break;
				}

			case EOpType::MI_SKELETALMESHOBJECT_BREAK:
				{
					const FOperation::MaterialSkeletalMeshObjectBreakArgs Args = Program.GetOpArgs<FOperation::MaterialSkeletalMeshObjectBreakArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.SkeletalMeshObject, Item));
					
					StoreNone(Item);
					break;
				}

			case EOpType::MI_SKELETALMESH_BREAK:
				{
					const FOperation::MaterialSkeletalMeshBreakArgs Args = Program.GetOpArgs<FOperation::MaterialSkeletalMeshBreakArgs>(Item.At);
					PushOp(FScheduledOpInline(Args.SkeletalMesh, Item));
					
					StoreNone(Item);
					break;
				}
				
			case EOpType::MI_MODIFY:
			{
				FOperation::MaterialModifyArgs Args = Program.GetOpArgs<FOperation::MaterialModifyArgs>(Item.At);
				int32 ParameterCount = Args.NumParameters;

				switch (Item.Stage)
				{
					case 0:
					{
						//Dynamic data can't be stored into arg structures
						const uint8* Data = Program.GetOpArgsPointer(Item.At);
						Data += sizeof(FOperation::MaterialModifyArgs);

						// First, Schedule the material
						PushOp(FScheduledOpInline(Args.Material, Item));

						// Store parameter names and schedule the parameter operations
						for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
						{
							FOperation::ADDRESS NewName;
							FMemory::Memcpy(&NewName, Data, sizeof(FOperation::ADDRESS));
							Data += sizeof(FOperation::ADDRESS);

							int8 LayerIndex;
							FMemory::Memcpy(&LayerIndex, Data, sizeof(int8));
							Data += sizeof(int8);

							FOperation::ADDRESS NewParameterOperation;
							FMemory::Memcpy(&NewParameterOperation, Data, sizeof(FOperation::ADDRESS));
							Data += sizeof(FOperation::ADDRESS);

							// Skip 0 addresses since do not need to load anything
							if (NewParameterOperation == 0)
							{
								continue;
							}

							PushOp(FScheduledOpInline(NewParameterOperation, Item));

							// Evaluate non lazy branches here
							if (GetOpDataType(Program.GetOpType(NewParameterOperation)) == EDataType::Image)
							{
								// Jump Image Parameter Index
								Data += sizeof(int32);
							}
						}

						break;
					}
						
				default:
					unimplemented();
							}

				break;
			}

			default:
				unimplemented();
			}
		}

		check(!Stack.Num());
	}
}
