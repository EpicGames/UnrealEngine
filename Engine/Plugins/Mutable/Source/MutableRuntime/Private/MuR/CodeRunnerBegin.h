// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/System.h"


namespace UE::Mutable::Private
{
	class FProgramCache;
	class FInstance;
	class FExtensionData;
	struct FProgram;
	class FModel;
	class FParameters;
	class FSystem;
	class String;

	struct FScheduledOpInline
	{
		explicit FScheduledOpInline(FOperation::ADDRESS InAt = 0, const uint16 InStage = 0, const uint16 InExecutionIndex = 0, bool bInEvaluate = true) :
			At(InAt),
			Stage(InStage),
			bEvaluate(bInEvaluate),
			ExecutionIndex(InExecutionIndex)
		{
		}

		explicit FScheduledOpInline(FOperation::ADDRESS InAt, const FScheduledOpInline& InOpTemplate) :
			At(InAt),
			bEvaluate(InOpTemplate.bEvaluate),
			ExecutionIndex(InOpTemplate.ExecutionIndex)
		{
		}
		
		explicit FScheduledOpInline(FOperation::ADDRESS InAt, const FScheduledOpInline& InOpTemplate, bool bInEvaluate) :
			At(InAt),
			bEvaluate(bInEvaluate),
			ExecutionIndex(InOpTemplate.ExecutionIndex)
		{
		}
		
		FScheduledOpInline(const FScheduledOpInline& InOpTemplate, uint16 InStage) :
			At(InOpTemplate.At),
         	Stage(InStage),
			bEvaluate(InOpTemplate.bEvaluate),
			ExecutionIndex(InOpTemplate.ExecutionIndex)
		{
		}

		bool operator==(const FScheduledOpInline& Other) const;

		FOperation::ADDRESS At = 0;
		uint8 Stage = 0;
		uint8 bEvaluate = 0;
		uint16 ExecutionIndex = 0;
		FOperation::ADDRESS CustomState = 0;
	};


	uint32 GetTypeHash(const FScheduledOpInline& Op);


	struct FCacheAddressInline
	{
		FCacheAddressInline(const FOperation::ADDRESS& InAt)
		{
			At = InAt;
		}
		
		FCacheAddressInline(const FScheduledOpInline& Item)
		{
			At = Item.At;
			ExecutionIndex = Item.ExecutionIndex;
		}
		
		FOperation::ADDRESS At = 0;
		uint16 ExecutionIndex = 0;

		bool operator==(const FCacheAddressInline&) const = default;
	};


	uint32 GetTypeHash(const FCacheAddressInline& Address);


	class FOpSet
	{
	public:
		FOpSet(int32 NumElements);

		bool Contains(const FCacheAddressInline& Item);

		void Add(const FCacheAddressInline& Item);

	private:
		//TBitArray<> Index0;
		TSet<FCacheAddressInline> IndexOther;
	};
	
	using FStackValue = TVariant<bool, int32, float, TManagedPtr<const String>, TManagedPtr<const FInstance>, TManagedPtr<const FMaterial>>;

	class FStack : public TArray<FStackValue, TInlineAllocator<32>>
	{
	public:
		FStackValue Pop();
	};
	
	class CodeRunnerBegin
	{
	public:
		MUTABLERUNTIME_API CodeRunnerBegin(
			const TSharedRef<FLiveInstance>& InLiveInstance,
			uint32 InLodMask);

		void RunCode(const FScheduledOpInline& Root);
		
	private:
		FScheduledOpInline PopOp();
		void PushOp(const FScheduledOpInline& Item);
		
		void StoreNone(const FCacheAddressInline& To);
		
		void StoreInt(const FCacheAddressInline& To, int32 Value);
		void StoreFloat(const FCacheAddressInline& To, float Value);
		void StoreBool(const FCacheAddressInline& To, bool Value);
		
		int32 LoadInt(const FCacheAddressInline& To);
		float LoadFloat(const FCacheAddressInline& To);
		bool LoadBool(const FCacheAddressInline& To);
		
		TSharedPtr<const FModel> Model;
		TSharedPtr<const FParameters> Params;

		const FProgram& Program;
		
		uint32 LodMask;

		/** Operations we have yet to process.*/
		TArray<FScheduledOpInline, TInlineAllocator<256, TAlignedHeapAllocator<PLATFORM_CACHE_LINE_SIZE>>> Items;

		/** Stack of data objects generated during the evaluation of the mutable tree. Can be used to pass data from one stage/op to another during the tree
		 * evaluation process
		 */
		FStack Stack;
		
		/** Collection of operations already processed. */
		FOpSet Executed;

		/** Data caches with the already resolved data. Used to prevent the reevaluation of branches that have already been evaluated.*/
		TMap<FCacheAddressInline, float> ResultsFloat;
		TMap<FCacheAddressInline, bool> ResultsBool;
		TMap<FCacheAddressInline, int32> ResultsInt;

		TSharedPtr<FProgramCache> ProgramCache;

		TSharedPtr<FPassthroughObjectLoader> PassthroughObjectLoader;
	};
}
