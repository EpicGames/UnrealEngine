// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/Find.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/CodeRunner.h"
#include "MuR/System.h"
#include "MuR/Image.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Layout.h"
#include "MuR/Mesh.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/OpImageBlend.h"
#include "MuR/OpImageSaturate.h"
#include "MuR/OpImageInvert.h"
#include "MuR/OpImageLayer.h"
#include "MuR/Operations.h"
#include "MuR/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"
#include "MuR/Serialisation.h"
#include "MuR/Settings.h"
#include "MuR/System.h"
#include "MuR/MutableRuntimeModule.h"
#include "Stats/Stats.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Async/Fundamental/Scheduler.h"
#include "MuR/SkeletalMesh.h"
#include "ClothingAsset.h"

#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK
#include "GenericPlatform/GenericPlatformStackWalk.h"
#endif

static TAutoConsoleVariable<bool> CVarCodeRunnerForceInline(
	TEXT("mutable.CodeRunnerForceInline"),
	false,
	TEXT("If true, force all Code Runners to be inline (do not split them into tasks)."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarRomLoadFailureRate(
	TEXT("mutable.RomLoadFailureRate"),
	0.0f,
	TEXT("Rate of induced failures for rom load testing."),
	ECVF_Default);

DECLARE_CYCLE_STAT(TEXT("MutableCoreTask"), STAT_MutableCoreTask, STATGROUP_Game);

namespace UE::Mutable::Private
{
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_OpenTask, TEXT("MutableRuntime/OpenTask"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_ClosedTasks, TEXT("MutableRuntime/ClosedTasks"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_IssuedTasks, TEXT("MutableRuntime/IssuedTasks"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_IssuedTasksOnHold, TEXT("MutableRuntime/IssuedHoldTasks"));

	void CodeRunner::AddChildren(TConstArrayView<FScheduledOp> Deps)
	{
		for (const FScheduledOp& Dep : Deps)
		{
			bool bRequiresScheduling = GetMemory().AcquireCachedResultForScheduling(FCacheAddress(Dep));

			if (bRequiresScheduling)
			{
#if MUTABLE_DEBUG_CODERUNNER_TASK_SCHEDULE_CALLSTACK
				FScheduledOp& AddedOp = OpenTasks.Add_GetRef(Dep);
				AddedOp.StackDepth = FPlatformStackWalk::CaptureStackBackTrace(&AddedOp.ScheduleCallstack[0], FScheduledOp::CallstackMaxDepth);
#else
				OpenTasks.Add(Dep);
#endif
			}
		}
	}


	bool CodeRunner::ShouldIssueTask() const
	{
		// Can we afford to delay issued tasks?
		bool bCanDelayTasks = IssuedTasks.Num() > 0 || OpenTasks.Num() > 0;
		if (!bCanDelayTasks)
		{
			return true;
		}
		else
		{
			// We could wait. Let's see if we have enough memory to issue tasks anyway.
			bool bHaveEnoughMemory = !System.WorkingMemoryManager.IsMemoryBudgetFull();
			if (bHaveEnoughMemory)
			{
				return  true;
			}
		}

		return false;
	}


	void CodeRunner::UpdateTraces()
	{
		// Code Runner status
		TRACE_COUNTER_SET(MutableRuntime_OpenTask, OpenTasks.Num());
		TRACE_COUNTER_SET(MutableRuntime_ClosedTasks, ClosedTasks.Num());
		TRACE_COUNTER_SET(MutableRuntime_IssuedTasks, IssuedTasks.Num());
		TRACE_COUNTER_SET(MutableRuntime_IssuedTasksOnHold, IssuedTasksOnHold.Num());
	}


	void CodeRunner::LaunchIssuedTask( const TSharedPtr<FIssuedTask>& TaskToIssue, bool& bOutFailed )
	{	
		bool bHasWork = TaskToIssue->Prepare(this, bOutFailed);
		if (bOutFailed)
		{
			bUnrecoverableError = true;
			return;
		}

		// Launch it
		if (bHasWork)
		{
			if (bForceSerialTaskExecution)
			{
				TaskToIssue->Event = {};
				TaskToIssue->DoWork();
			}
			else
			{
				TaskToIssue->Event = UE::Tasks::Launch(TEXT("MutableCore_Task"),
					[TaskToIssue]() { TaskToIssue->DoWork(); },
					UE::Tasks::ETaskPriority::Inherit);
			}
		}

		// Remember it for later processing.
		IssuedTasks.Add(TaskToIssue);
	}

	void CodeRunner::StartRun(bool bForceInlineExecution)
	{
		if (CVarCodeRunnerForceInline.GetValueOnAnyThread())
		{
			bForceInlineExecution = true;
		}
		
		bUnrecoverableError = false;

		HeapData.SetNum(0, EAllowShrinking::No);
		ImageDescResults.Reset();
		ImageDescConstantImages.Reset();
		
		constexpr bool bProfile = false;
		TUniquePtr<FProfileContext> ProfileContext = bProfile ? MakeUnique<FProfileContext>() : nullptr;
		Run(MoveTemp(ProfileContext), bForceInlineExecution);
		
		if (bForceInlineExecution)
		{
			FCacheAddress CacheAddress = FCacheAddress(RootOPAddress, 0, RootOpExecutionOptions, RootOpType);

			// In case of aborted the value will not be read, decrease root address hit count
			// to balance the ophit added at runner creation.
			const bool bAborted = LiveInstance->Cache->IsAborted(CacheAddress);
			if (bAborted)
			{
				LiveInstance->Cache->UpdateHitCount(CacheAddress, -1);
			}
		
			bUnrecoverableError = bUnrecoverableError || bAborted;
		}
	}

	void CodeRunner::LaunchWaitForOthersTask(Tasks::FTaskEvent RunnerRestartEvent, uint64 ReentryCounter)
	{
		UE::Tasks::Launch(TEXT("CheckIfRunnerCanRunTask"),
		[Runner = AsShared(), RunnerRestartEvent, ReentryCounter]() mutable
		{
			if (ReentryCounter > 128)
			{
				UE_LOGF(LogMutableCore, Warning, "LaunchWaitForOtherTask ReentryCounter is higher than exepcetd [%llu].", ReentryCounter);
			}

			int32 Index = Runner->ClosedTasks.Num() - 1;
			for (; Index >= 0; --Index)
			{
				FProgramCache::EAcquireSetResult Result = Runner->GetMemory().TryAcquireCachedResultSet(Runner->ClosedTasks[Index].Deps);

				if (Result != FProgramCache::EAcquireSetResult::Failure)
				{
					break;
				}
			}
		
			// If the loop exited early then we have work to do, trigger the event that will restart the runner.
			// Otherwise, relaunch this task registering a new wait event.
			if (Index >= 0)
			{
				RunnerRestartEvent.Trigger();
			}
			else
			{
				Runner->LaunchWaitForOthersTask(RunnerRestartEvent, ReentryCounter + 1);
			}
		},
		UE::Tasks::Prerequisites(GetMemory().RegisterWaitEvent(TEXT("LaunchWaitForOthersTask"))),
		UE::Tasks::ETaskPriority::Inherit);
	}

    void CodeRunner::Run(TUniquePtr<FProfileContext>&& ProfileContext, bool bForceInlineExecution)
    {
		MUTABLE_CPUPROFILER_SCOPE(CodeRunner_Run);

		// TODO: Move MaxAllowedTime somewhere else more accessible, maybe a cvar.
		const FTimespan MaxAllowedTime = FTimespan::FromMilliseconds(2.0); 
		const FTimespan TimeOut = FTimespan::FromSeconds(FPlatformTime::Seconds()) + MaxAllowedTime;

		bool bSuccess = true;

        while(!OpenTasks.IsEmpty() || !ClosedTasks.IsEmpty() || !IssuedTasks.IsEmpty())
        {
			UpdateTraces();
			// Debug: log the amount of tasks that we'd be able to run concurrently:
			//{
			//	int32 ClosedReady = ClosedTasks.Num();
			//	for (int Index = ClosedTasks.Num() - 1; Index >= 0; --Index)
			//	{
			//		for (const FCacheAddress& Dep : ClosedTasks[Index].Deps)
			//		{
			//			if (Dep.at && !GetMemory().IsValid(Dep))
			//			{
			//				--ClosedReady;
			//				continue;
			//			}
			//		}
			//	}

			//	UE_LOGF(LogMutableCore, Log, "Tasks: %5d open, %5d issued, %5d closed, %d closed ready", OpenTasks.Num(), IssuedTasks.Num(), ClosedTasks.Num(), ClosedReady);
			//}

			for (int32 Index = 0; bSuccess && Index < IssuedTasks.Num(); )
			{
				check(IssuedTasks[Index]);

				bool bWorkDone = IssuedTasks[Index]->IsComplete(this);
				if (bWorkDone)
				{
					const FScheduledOp& item = IssuedTasks[Index]->Op;
					bSuccess = IssuedTasks[Index]->Complete(this);
					IssuedTasks.RemoveAt(Index, EAllowShrinking::No); // with swap? changes order of execution.
				}
				else
				{
					++Index;
				}
			}

			while (!OpenTasks.IsEmpty())
			{
				// Get a new task to run
				FScheduledOp Item;
				switch (ExecutionStrategy)
				{
				//case EExecutionStrategy::MinimizeMemory:
				//{
				//	// TODO: Prioritize operation stages that have a negative memory delta somehow (the result uses less memory than the inputs)
				//  // An attempt to do this with an heuristic estimation per-stage of each operation in a static table was tested in the past but maybe
				//  // there are better ways. 
				//	break;
				//}

				case EExecutionStrategy::None:
				default:
					// Just get one.
					Item = OpenTasks.Pop(EAllowShrinking::No);
					break;

				}

				FCacheAddress CachedAddress(Item);
				
				if (GetMemory().TryAcquireCachedResult(CachedAddress))
				{
					continue;
				}

				// See if we can schedule this Item concurrently
				TSharedPtr<FIssuedTask> IssuedTask = IssueOp(Item);
				if (IssuedTask)
				{
					if (ShouldIssueTask())
					{
						bool bFailed = false;
						LaunchIssuedTask(IssuedTask, bFailed);
						if (bFailed)
						{
							GetMemory().SetAborted(FCacheAddress(IssuedTask->Op));
						}
					}
					else
					{
						IssuedTasksOnHold.Add(IssuedTask);
					}
				}
				else
				{
					// Run immediately
					if (Item.Type == static_cast<uint16>(FScheduledOp::EType::Full))
					{
						RunCode(Item);
					}
					else if (Item.Type == static_cast<uint16>(FScheduledOp::EType::ImageDesc))
					{
						RunCodeImageDesc(Item);
					}
					else
					{
						unimplemented();
					}
				}

				if (ProfileContext)
				{
					TScopeLock<FMutex> LockGuard(ProfileContext->Mutex);

					++ProfileContext->NumRunOps;
					++ProfileContext->RunOpsPerType[int32(LiveInstance->Model->GetProgram().GetOpType(Item.At))];
				}
			}

			UpdateTraces();

			// Look for tasks on hold and see if we can launch them
			while (IssuedTasksOnHold.Num() && ShouldIssueTask())
			{
				TSharedPtr<FIssuedTask> TaskToIssue = IssuedTasksOnHold.Pop(EAllowShrinking::No);

				bool bFailed = false;
				LaunchIssuedTask(TaskToIssue, bFailed);
				if (bFailed)
				{
					GetMemory().SetAborted(FCacheAddress(TaskToIssue->Op));
				}
			}

			// Look for a closed task with dependencies satisfied and move them to the open task list.
			bool bSomeWasReady = false;
			
			for (int32 Index = ClosedTasks.Num() - 1; Index >= 0; --Index)
			{
				FProgramCache::EAcquireSetResult Result = GetMemory().TryAcquireCachedResultSet(ClosedTasks[Index].Deps);

				if (Result == FProgramCache::EAcquireSetResult::Success)
				{
					OpenTasks.Push(ClosedTasks[Index].Op);
				}
				
				if (Result == FProgramCache::EAcquireSetResult::Abort)
				{
					GetMemory().SetAborted(ClosedTasks[Index].Op);
					
					// The parent task is aborted, make sure the 
					for (const FCacheAddress& Dep : ClosedTasks[Index].Deps)
					{
						GetMemory().UpdateHitCount(Dep, -1);
					}
				}

				if (Result != FProgramCache::EAcquireSetResult::Failure)
				{
					ClosedTasks.RemoveAtSwap(Index, EAllowShrinking::No);
				}

				bSomeWasReady |= Result != FProgramCache::EAcquireSetResult::Failure;

				if (bSomeWasReady)
				{
					break;
				}
			}

			UpdateTraces();

			// Debug: Did we dead-lock?
			bool bWaitingForOtherRunners = !(OpenTasks.Num() || IssuedTasks.Num() || !ClosedTasks.Num() || bSomeWasReady);
			if (bWaitingForOtherRunners)
			{
				GetMemory().TriggerWaitEvents();

				UE::Tasks::FTaskEvent WaitForOtherScheduleEvent(TEXT("WaitForOtherRunnersReScheduleEvent"));
				LaunchWaitForOthersTask(WaitForOtherScheduleEvent);

				UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("CoderRunnerFromWaitForOtherRunners"),
				[Runner = AsShared(), ProfileContext = MoveTemp(ProfileContext)]() mutable
				{
					constexpr bool bForceInlineExecution = false;
					Runner->Run(MoveTemp(ProfileContext), bForceInlineExecution);
				},
				UE::Tasks::Prerequisites(WaitForOtherScheduleEvent),
				UE::Tasks::ETaskPriority::Inherit));

				return;
			}

			// If at this point there is no open op and we haven't finished, we need to wait for an issued op to complete.
			if (OpenTasks.IsEmpty() && !IssuedTasks.IsEmpty())
			{
				if (!bForceInlineExecution)
				{
					TArray<UE::Tasks::FTask, TInlineAllocator<8>> IssuedTasksCompletionEvents;
					IssuedTasksCompletionEvents.Reserve(IssuedTasks.Num());

					for (TSharedPtr<FIssuedTask>& IssuedTask : IssuedTasks)
					{
						if (IssuedTask->Event.IsValid())
						{
							IssuedTasksCompletionEvents.Add(IssuedTask->Event);
						}
					}

					GetMemory().TriggerWaitEvents();

					UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("CodeRunnerFromIssuedTasksTask"),
						[Runner = AsShared(), ProfileContext = MoveTemp(ProfileContext)]() mutable
						{
							constexpr bool bForceInlineExecution = false;
							Runner->Run(MoveTemp(ProfileContext), bForceInlineExecution);
						},
						UE::Tasks::Prerequisites(UE::Tasks::Any(IssuedTasksCompletionEvents)),
						UE::Tasks::ETaskPriority::Inherit));
					
					return;
				}	
				else
				{
					MUTABLE_CPUPROFILER_SCOPE(CodeRunner_WaitIssued);
					for (int32 IssuedIndex = 0; IssuedIndex < IssuedTasks.Num(); ++IssuedIndex)
					{
						if (IssuedTasks[IssuedIndex]->Event.IsValid())
						{
							IssuedTasks[IssuedIndex]->Event.Wait();

							break;
						}
					}
				}
			}

			if (!bForceInlineExecution)
			{
				if (FTimespan::FromSeconds(FPlatformTime::Seconds()) > TimeOut)
				{
					GetMemory().TriggerWaitEvents();

					UE::Tasks::AddNested(UE::Tasks::Launch(TEXT("CodeRunnerFromTimeoutTask"),
						[Runner = AsShared(), ProfileContext = MoveTemp(ProfileContext)]() mutable
						{
							constexpr bool bForceInlineExecution = false;
							Runner->Run(MoveTemp(ProfileContext), bForceInlineExecution);
						},
						UE::Tasks::ETaskPriority::Inherit));
					
					return;
				}
			}
		}

		GetMemory().TriggerWaitEvents();
		
		if (ProfileContext)
		{
			UE_LOGF(LogMutableCore, Log, "Mutable Heap Bytes: %d", HeapData.Num()* HeapData.GetTypeSize());
			UE_LOGF(LogMutableCore, Log, "Ran ops : %5d ", ProfileContext->NumRunOps);

			constexpr int32 HistogramSize = 8;
			int32 MostCommonOps[HistogramSize] = {};
			for (int32 OpIndex = 0; OpIndex < int32(EOpType::COUNT); ++OpIndex)
			{
				for (int32 HistIndex = 0; HistIndex < HistogramSize; ++HistIndex)
				{
					if (ProfileContext->RunOpsPerType[OpIndex] > ProfileContext->RunOpsPerType[MostCommonOps[HistIndex]])
					{
						// Displace others
						int32 ElementsToMove = HistogramSize - HistIndex - 1;
						if (ElementsToMove > 0)
						{
							FMemory::Memcpy(&MostCommonOps[HistIndex + 1], &MostCommonOps[HistIndex], sizeof(int32)*ElementsToMove);
						}
						// Set new value
						MostCommonOps[HistIndex] = OpIndex;
						break;
					}
				}
			}

			for (int32 HistIndex = 0; HistIndex < HistogramSize; ++HistIndex)
			{
				UE_LOGF(LogMutableCore, Log, "    op %4d, %4d times.", MostCommonOps[HistIndex], ProfileContext->RunOpsPerType[MostCommonOps[HistIndex]]);
			}
		}
    }

	
	/** */
	FExtendedImageDesc CodeRunner::GetImageDescResult(FOperation::ADDRESS ResultAddress)
	{
		FExtendedImageDesc Result = LiveInstance->Cache->LoadImageDesc(FCacheAddress(ResultAddress, 0, 0, FScheduledOp::EType::ImageDesc));
		Result.ConstantImagesNeededToGenerate = ImageDescConstantImages;

		return Result;
	}

	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageLayerTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageLayerTask(const FScheduledOp& InOp, const FOperation::ImageLayerArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		int32 ImageCompressionQuality = 0;
		FOperation::ImageLayerArgs Args;
		TManagedPtr<const FImage> Blended;
		TManagedPtr<const FImage> Mask;
		TManagedPtr<FImage> Result;
		EImageFormat InitialFormat = EImageFormat::None;
	};


	bool FImageLayerTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageLayerTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		TManagedPtr<const FImage> Base = Runner->LoadImage({ Args.base, Op.ExecutionIndex, Op.ExecutionOptions });
		check(!Base || Base->GetFormat() < EImageFormat::Count);

		Blended = Runner->LoadImage({ Args.blended, Op.ExecutionIndex, Op.ExecutionOptions });
		if (Args.mask)
		{
			Mask = Runner->LoadImage({ Args.mask, Op.ExecutionIndex, Op.ExecutionOptions });
			check(!Mask || Mask->GetFormat() < EImageFormat::Count);
		}

		// Shortcuts
		if (!Base)
		{
			Blended = nullptr;
			Mask = nullptr;

			Runner->StoreImage(Op, Base);
			return false;
		}

		bool bValid = Base->GetSizeX() > 0 && Base->GetSizeY() > 0;
		if (!bValid || !Blended)
		{
			Blended = nullptr;
			Mask = nullptr;

			Runner->StoreImage(Op, Base);
			return false;
		}

		FImageOperator ImOp = MakeImageOperator(Runner);

		// Input data fixes
		InitialFormat = Base->GetFormat();

		if (IsCompressedFormat(InitialFormat))
		{
			EImageFormat UncompressedFormat = GetUncompressedFormat(InitialFormat);
			TManagedPtr<FImage> Formatted = Runner->CreateImage(Base->GetSizeX(), Base->GetSizeY(), Base->GetLODCount(), UncompressedFormat, EInitializationType::NotInitialized );
			bool bSuccess = false;
			ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Base.Get());
			check(bSuccess); // Decompression cannot fail
			Base = Formatted;
		}

		bool bMustHaveSameFormat = !(Args.flags & (FOperation::ImageLayerArgs::F_BASE_RGB_FROM_ALPHA | FOperation::ImageLayerArgs::F_BLENDED_RGB_FROM_ALPHA));

		const EImageFormat TargetBlendFormat = bMustHaveSameFormat
				? Base->GetFormat()
				: GetUncompressedFormat(Blended->GetFormat());

		if (TargetBlendFormat != Blended->GetFormat())
		{
			TManagedPtr<FImage> Formatted = Runner->CreateImage(Blended->GetSizeX(), Blended->GetSizeY(), Blended->GetLODCount(), TargetBlendFormat, EInitializationType::NotInitialized);
			bool bSuccess = false;
			ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Blended.Get());
			check(bSuccess); // Decompression cannot fail
			Blended = Formatted;
		}

		if (Base->GetSize() != Blended->GetSize())
		{
			MUTABLE_CPUPROFILER_SCOPE(ImageResize_EmergencyFix);
			TManagedPtr<FImage> Resized = Runner->CreateImage(Base->GetSizeX(), Base->GetSizeY(), 1, Blended->GetFormat(), EInitializationType::NotInitialized);
			ImOp.ImageResizeLinear(Resized.Get(), ImageCompressionQuality, Blended.Get());
			Blended = Resized;

		}

		if (Mask)
		{
			if (Base->GetSize() != Mask->GetSize())
			{
				MUTABLE_CPUPROFILER_SCOPE(ImageResize_EmergencyFix);
				TManagedPtr<FImage> Resized = Runner->CreateImage(Base->GetSizeX(), Base->GetSizeY(), 1, Mask->GetFormat(), EInitializationType::NotInitialized);
				ImOp.ImageResizeLinear(Resized.Get(), ImageCompressionQuality, Mask.Get());
				Mask = Resized;

			}

			if (Mask->GetLODCount() < Base->GetLODCount())
			{
				MUTABLE_CPUPROFILER_SCOPE(ImageLayer_EmergencyFix);

				int32 StartLevel = Mask->GetLODCount() - 1;
				int32 LevelCount = Base->GetLODCount();

				// Uncompress mask to avoid excessive RLE-compression and decompression in the following ops.
				TManagedPtr<FImage> UncompressedMask = Runner->CreateImage(Mask->GetSizeX(), Mask->GetSizeY(), Mask->GetLODCount(), GetUncompressedFormat(Mask->GetFormat()), EInitializationType::NotInitialized);
				bool bSuccess = false;
				ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, UncompressedMask.Get(), Mask.Get());

				TManagedPtr<FImage> MaskFix = UncompressedMask;
				MaskFix->DataStorage.SetNumLODs(LevelCount);

				FMipmapGenerationSettings Settings{};
				ImOp.ImageMipmap(ImageCompressionQuality, MaskFix.Get(), MaskFix.Get(), StartLevel, LevelCount, Settings);

				Mask = MaskFix;
			}
		}

		// Create destination data
		Result = Runner->CloneOrTakeOver(MoveTemp(Base));
		return true;
	}


	void FImageLayerTask::DoWork()
	{
		MUTABLE_CPUPROFILER_SCOPE(FImageLayerTask);

		// This runs in a worker thread.
		bool bOnlyOneMip = Blended->GetLODCount() < Result->GetLODCount();

		const int32 LODBegin = 0;
		const int32 LODEnd   = bOnlyOneMip ? 1 : Result->GetLODCount();
		
		if (EBlendType(Args.blendType) != EBlendType::BT_NORMAL_COMBINE)
		{
			bool bApplyColorBlendToAlpha       = (Args.flags & FOperation::ImageLayerArgs::F_APPLY_TO_ALPHA        ) != 0;
			bool bUseBaseSourceFromBaseAlpha   = (Args.flags & FOperation::ImageLayerArgs::F_BASE_RGB_FROM_ALPHA   ) != 0;
			bool bUseBlendSourceFromBlendAlpha = (Args.flags & FOperation::ImageLayerArgs::F_BLENDED_RGB_FROM_ALPHA) != 0;
			bool bUseMaskFromBlendAlpha        = (Args.flags & FOperation::ImageLayerArgs::F_USE_MASK_FROM_BLENDED ) != 0;

			FBlendFuncType* ColorBlendFunc = SelectBlendFunc(EBlendType(Args.blendType));
			FBlendFuncType* AlphaBlendFunc = nullptr; //ColorBlendFunc;

			if (!bApplyColorBlendToAlpha)
			{
				AlphaBlendFunc = SelectBlendFunc(EBlendType(Args.blendTypeAlpha));
			}

			ImageLayerBlend(
					Result.Get(), Result.Get(), Blended.Get(), Mask.Get(), 
					ColorBlendFunc, AlphaBlendFunc, 
					LODBegin, LODEnd, 
					bApplyColorBlendToAlpha, 
					bUseMaskFromBlendAlpha,
					bUseBaseSourceFromBaseAlpha,
					bUseBlendSourceFromBlendAlpha, 
					Args.BlendAlphaSourceChannel);
		}
		else
		{
			ImageLayerCombine(
					Result.Get(), Result.Get(), Blended.Get(), Mask.Get(), 
					OpImageLayerCombineOps::NormalCombine, 
					LODBegin, LODEnd);
		}

		if (bOnlyOneMip)
		{
			MUTABLE_CPUPROFILER_SCOPE(ImageLayer_MipFix);
			FMipmapGenerationSettings DummyMipSettings{};
			ImageMipmapInPlace(ImageCompressionQuality, Result.Get(), DummyMipSettings);
		}

		// Reset relevancy map.
		Result->Flags &= ~FImage::EImageFlags::IF_HAS_RELEVANCY_MAP;
	}

	bool FImageLayerTask::Complete( CodeRunner* Runner )
	{
		// This runs in the Runner thread

		Blended = nullptr;
		Mask    = nullptr;
		// If no shortcut was taken
		if (Result)
		{
			if (InitialFormat != Result->GetFormat())
			{
				TManagedPtr<FImage> Formatted = Runner->CreateImage(Result->GetSizeX(), Result->GetSizeY(), Result->GetLODCount(), InitialFormat, EInitializationType::NotInitialized);
				bool bSuccess = false;

				FImageOperator ImOp = MakeImageOperator(Runner);
				ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Result.Get());
				check(bSuccess);

				Result = Formatted;
			}

			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageLayerColorTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageLayerColorTask(const FScheduledOp& InOp, const FOperation::ImageLayerColorArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		int32 ImageCompressionQuality = 0;
		FOperation::ImageLayerColorArgs Args;
		FVector4f Color;
		TManagedPtr<const FImage> Mask;
		TManagedPtr<FImage> Result;
		EImageFormat InitialFormat = EImageFormat::None;
	};


	bool FImageLayerColorTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageLayerColorTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		TManagedPtr<const FImage> Base = Runner->LoadImage({ Args.base, Op.ExecutionIndex, Op.ExecutionOptions });
		check(!Base || Base->GetFormat() < EImageFormat::Count);

		Color = Runner->LoadColor({ Args.color, Op.ExecutionIndex, 0 });
		if (Args.mask)
		{
			Mask = Runner->LoadImage({ Args.mask, Op.ExecutionIndex, Op.ExecutionOptions });
			check(!Mask || Mask->GetFormat() < EImageFormat::Count);
		}

		// Shortcuts
		if (!Base)
		{
			Mask = nullptr;
			Runner->StoreImage(Op, Base);
			return false;
		}

		bool bValid = Base->GetSizeX() > 0 && Base->GetSizeY() > 0;
		if (!bValid)
		{
			Mask = nullptr;
			Runner->StoreImage(Op, Base);
			return false;
		}

		// Fix input data
		InitialFormat = Base->GetFormat();
		check(InitialFormat < EImageFormat::Count);

		if (Args.mask && Mask)
		{
			FImageOperator ImOp = MakeImageOperator(Runner);

			if (Mask->GetFormat() != EImageFormat::L_UByte &&
				static_cast<EBlendType>(Args.blendType) == EBlendType::BT_NORMAL_COMBINE) // TODO Optimize. BT_NORMAL_COMBINE (ImageLayerCombineColor with mask) does not support formats such as RLE. Hence why we are changing the format here.
			{
				MUTABLE_CPUPROFILER_SCOPE(EmergencyFix_Format);
				TManagedPtr<FImage> Formatted = Runner->CreateImage(Mask->GetSizeX(), Mask->GetSizeY(), Mask->GetLODCount(), EImageFormat::L_UByte, EInitializationType::NotInitialized);

				bool bSuccess = false;
				ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Mask.Get());
				check(bSuccess); // Decompression cannot fail

				Mask = Formatted;
			}
			
			if (Base->GetSize() != Mask->GetSize())
			{
				MUTABLE_CPUPROFILER_SCOPE(EmergencyFix_Size);
				TManagedPtr<FImage> Resized = Runner->CreateImage(Base->GetSizeX(), Base->GetSizeY(), 1, Mask->GetFormat(), EInitializationType::NotInitialized);
				ImOp.ImageResizeLinear(Resized.Get(), ImageCompressionQuality, Mask.Get() );
				Mask = Resized;
			}

			if (Mask->GetLODCount() < Base->GetLODCount())
			{
				MUTABLE_CPUPROFILER_SCOPE(EmergencyFix_LOD);
				int32 StartLevel = Mask->GetLODCount() - 1;
				int32 LevelCount = Base->GetLODCount();

				TManagedPtr<FImage> MaskFix = Runner->CloneOrTakeOver(MoveTemp(Mask));
				MaskFix->DataStorage.SetNumLODs(LevelCount);

				FMipmapGenerationSettings Settings{};
				ImOp.ImageMipmap(ImageCompressionQuality, MaskFix.Get(), MaskFix.Get(), StartLevel, LevelCount, Settings);

				Mask = MaskFix;
			}
		}

		// Create destination data
		Result = Runner->CloneOrTakeOver(MoveTemp(Base));
		return true;
	}


	void FImageLayerColorTask::DoWork()
	{
		// This runs in a worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageLayerColorTask);
		// This runs in a worker thread.
		bool bOnlyOneMip = false;

		const int32 LODBegin = 0;
		const int32 LODEnd = Result->GetLODCount();

		if (EBlendType(Args.blendType) != EBlendType::BT_NORMAL_COMBINE)
		{
			FBlendFuncType* ColorBlendFunc = SelectBlendFunc(EBlendType(Args.blendType));
			FBlendFuncType* AlphaBlendFunc = ColorBlendFunc;
			
			bool bApplyColorBlendToAlpha       = (Args.flags & FOperation::ImageLayerArgs::F_APPLY_TO_ALPHA        ) != 0;
			bool bUseBaseSourceFromBaseAlpha   = (Args.flags & FOperation::ImageLayerArgs::F_BASE_RGB_FROM_ALPHA   ) != 0;
			bool bUseBlendSourceFromBlendAlpha = (Args.flags & FOperation::ImageLayerArgs::F_BLENDED_RGB_FROM_ALPHA) != 0;
			bool bUseMaskFromBlendAlpha = false; 

			if (!bApplyColorBlendToAlpha)
			{
				AlphaBlendFunc = SelectBlendFunc(EBlendType(Args.blendTypeAlpha));
			}

			ImageLayerBlendConstant(
					Result.Get(), Result.Get(), Mask.Get(), Color, 
					ColorBlendFunc, AlphaBlendFunc, 
					LODBegin, LODEnd, 
					bApplyColorBlendToAlpha, 
					bUseMaskFromBlendAlpha, 
					bUseBaseSourceFromBaseAlpha, 
					bUseBlendSourceFromBlendAlpha,
					Args.BlendAlphaSourceChannel);
		}
		else
		{
			ImageLayerCombineConstant(
				Result.Get(), Result.Get(), Mask.Get(), Color,
				OpImageLayerCombineOps::NormalCombine,
				LODBegin, LODEnd);
		}

		// Reset relevancy map.
		Result->Flags &= ~FImage::EImageFlags::IF_HAS_RELEVANCY_MAP;
	}


	bool FImageLayerColorTask::Complete(CodeRunner* Runner)
	{
		// This runs in the Runner thread
		Mask = nullptr;

		// If no shortcut was taken
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImagePixelFormatTask : public CodeRunner::FIssuedTask
	{
	public:
		FImagePixelFormatTask(const FScheduledOp& InOp, const FOperation::ImagePixelFormatArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		int ImageCompressionQuality = 0;
		FOperation::ImagePixelFormatArgs Args;
		EImageFormat TargetFormat = EImageFormat::None;
		TManagedPtr<const FImage> Base;
		TManagedPtr<FImage> Result;
		FImageOperator::FImagePixelFormatFunc ImagePixelFormatFunc;
	};


	bool FImagePixelFormatTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageLayerPixelFormatTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		Base = Runner->LoadImage({ Args.source, Op.ExecutionIndex, Op.ExecutionOptions });
		check(!Base || Base->GetFormat() < EImageFormat::Count);		

		// Shortcuts
		if (!Base)
		{
			Runner->StoreImage(Op, Base);
			Base = nullptr;
			return false;
		}

		bool bValid = Base->GetSizeX() > 0 && Base->GetSizeY() > 0;
		if (!bValid)
		{
			Runner->StoreImage(Op, Base);
			Base = nullptr;
			return false;
		}

		TargetFormat = Args.format;
		if (Args.formatIfAlpha != EImageFormat::None
			&&
			GetImageFormatData(Base->GetFormat()).Channels > 3)
		{
			TargetFormat = Args.formatIfAlpha;
		}

		if (TargetFormat == EImageFormat::None || TargetFormat == Base->GetFormat())
		{
			Runner->StoreImage(Op, Base);
			Base = nullptr;
			return false;
		}

		ImagePixelFormatFunc = Runner->LiveInstance->PixelFormatOverride;

		// Create destination data
		Result = Runner->CreateImage(Base->GetSizeX(), Base->GetSizeY(), Base->GetLODCount(), TargetFormat, EInitializationType::NotInitialized);
		return true;
	}


	void FImagePixelFormatTask::DoWork()
	{
		// This runs in a worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImagePixelFormatTask);
		
		bool bSuccess = false;
		FImageOperator ImOp = FImageOperator::GetDefault(ImagePixelFormatFunc);
		ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Result.Get(), Base.Get(), -1);
		check(bSuccess);
	}


	bool FImagePixelFormatTask::Complete(CodeRunner* Runner)
	{
		// This runs in the Runner thread
		Base = nullptr;

		// If no shortcut was taken
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}

	class FImageMipmapTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageMipmapTask(const FScheduledOp& InOp, const FOperation::ImageMipmapArgs& InArgs)
			:FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		int32 ImageCompressionQuality = 0;
		int32 StartLevel = -1;
		FOperation::ImageMipmapArgs Args;
		TManagedPtr<const FImage> Base;
		TManagedPtr<FImage> Result;
		FImageOperator::FScratchImageMipmap Scratch;
		FImageOperator::FImagePixelFormatFunc ImagePixelFormatFunc;
	};


	bool FImageMipmapTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageMipmapTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		Base = Runner->LoadImage({ Args.source, Op.ExecutionIndex, Op.ExecutionOptions });
		check(!Base || Base->GetFormat() < EImageFormat::Count);

		// Shortcuts
		if (!Base)
		{
			Runner->StoreImage(Op, Base);
			Base = nullptr;
			return false;
		}

		bool bValid = Base->GetSizeX() > 0 && Base->GetSizeY() > 0;
		if (!bValid)
		{
			Runner->StoreImage(Op, Base);
			Base = nullptr;
			return false;
		}

		int32 LevelCount = Args.levels;
		int32 MaxLevelCount = FImage::GetMipmapCount(Base->GetSizeX(), Base->GetSizeY());
		if (LevelCount == 0)
		{
			LevelCount = MaxLevelCount;
		}
		else if (LevelCount > MaxLevelCount)
		{
			// If code generation is smart enough, this should never happen.
			// \todo But apparently it does, sometimes.
			LevelCount = MaxLevelCount;
		}

		// At least keep the levels we already have.
		LevelCount = FMath::Max(Base->GetLODCount(), LevelCount);
		
		if (LevelCount == Base->GetLODCount())
		{
			Runner->StoreImage(Op, Base);
			Base = nullptr;
			return false;
		}

		StartLevel = Base->GetLODCount() - 1;
		
		Result = Runner->CloneOrTakeOver(MoveTemp(Base));

		Result->DataStorage.SetNumLODs(LevelCount);

		FImageOperator ImOp = MakeImageOperator(Runner);
		ImOp.ImageMipmap_PrepareScratch(Result.Get(), StartLevel, LevelCount, Scratch);

		ImagePixelFormatFunc = Runner->LiveInstance->PixelFormatOverride;

		return true;
	}


	void FImageMipmapTask::DoWork()
	{
		// This runs in a worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageMipmapTask);

		check(StartLevel >= 0);

		FMipmapGenerationSettings Settings{Args.FilterType, Args.AddressMode};
		FImageOperator ImOp = FImageOperator::GetDefault(ImagePixelFormatFunc);
		ImOp.ImageMipmap(Scratch, ImageCompressionQuality, Result.Get(), Result.Get(), StartLevel, Result->GetLODCount(), Settings);
	}


	bool FImageMipmapTask::Complete(CodeRunner* Runner)
	{
		FImageOperator ImOp = MakeImageOperator(Runner);

		Scratch.Uncompressed = nullptr;
		Scratch.UncompressedMips = nullptr;
		Scratch.CompressedMips = nullptr;
		Base = nullptr;

		// If no shortcut was taken
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	class FImageSwizzleTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageSwizzleTask(const FScheduledOp& InOp, const FOperation::ImageSwizzleArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed);
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		FOperation::ImageSwizzleArgs Args;
		TManagedPtr<const FImage> Sources[MUTABLE_OP_MAX_SWIZZLE_CHANNELS];
		TManagedPtr<FImage> Result;
	};


	bool FImageSwizzleTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageSwizzleTask_Prepare);
		bOutFailed = false;

		int32 FirstValidSourceIndex = -1;
		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Args.sources[SourceIndex])
			{
				FirstValidSourceIndex = SourceIndex;
				break;
			}
		}

		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Args.sources[SourceIndex])
			{
				Sources[SourceIndex] = Runner->LoadImage({ Args.sources[SourceIndex], Op.ExecutionIndex, Op.ExecutionOptions });
			}
		}

		// Shortcuts
		if (FirstValidSourceIndex < 0 || !Sources[FirstValidSourceIndex])
		{
			for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
			{
				Sources[SourceIndex] = nullptr;
			}

			Runner->StoreImage(Op, nullptr);

			return false;
		}

		// Create destination data
		EImageFormat format = (EImageFormat)Args.format;

		FImageOperator ImOp = MakeImageOperator(Runner);

		int32 ResultLODs = Sources[FirstValidSourceIndex]->GetLODCount();

		// Be defensive: ensure formats are uncompressed
		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Sources[SourceIndex] && Sources[SourceIndex]->GetFormat() != GetUncompressedFormat(Sources[SourceIndex]->GetFormat()))
			{
				MUTABLE_CPUPROFILER_SCOPE(ImageFormat_ForSwizzle);

				EImageFormat UncompressedFormat = GetUncompressedFormat(Sources[SourceIndex]->GetFormat());
				TManagedPtr<FImage> Formatted = Runner->CreateImage(Sources[SourceIndex]->GetSizeX(), Sources[SourceIndex]->GetSizeY(), 1, UncompressedFormat, EInitializationType::NotInitialized);
				bool bSuccess = false;
				int32 ImageCompressionQuality = 4; // TODO
				ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Sources[SourceIndex].Get());
				check(bSuccess); // Decompression cannot fail
				Sources[SourceIndex] = Formatted;
				ResultLODs = 1;
			}
		}

		const FImageSize ResultSize = Sources[FirstValidSourceIndex]->GetSize();

		// Be defensive: ensure image sizes match.
		for (int32 SourceIndex = FirstValidSourceIndex + 1; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Sources[SourceIndex] && ResultSize != Sources[SourceIndex]->GetSize())
			{
				MUTABLE_CPUPROFILER_SCOPE(ImageResize_ForSwizzle);
				TManagedPtr<FImage> Resized = Runner->CreateImage(ResultSize.X, ResultSize.Y, 1, Sources[SourceIndex]->GetFormat(), EInitializationType::NotInitialized);
				ImOp.ImageResizeLinear(Resized.Get(), 0, Sources[SourceIndex].Get());
				Sources[SourceIndex] = Resized;
				ResultLODs = 1;
			}
		}

		// If any source has only 1 LOD, then the result has to have 1 LOD and the rest be regenerated later on
		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Sources[SourceIndex] && Sources[SourceIndex]->GetLODCount() == 1)
			{
				ResultLODs = 1;
			}
		}

		Result = Runner->CreateImage(ResultSize.X, ResultSize.Y, ResultLODs, Args.format, EInitializationType::Black);
		return true;
	}


	void FImageSwizzleTask::DoWork()
	{
		// This runs in a worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageSwizzleTask);

		ImageSwizzle(Result.Get(), Sources, Args.sourceChannels);	
	}


	bool FImageSwizzleTask::Complete(CodeRunner* Runner)
	{
		// This runs in the Runner thread

		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			Sources[SourceIndex] = nullptr;
		}

		// \TODO: If Result LODs differ from Source[0]'s, rebuild mips?

		// If no shortcut was taken
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageSaturateTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageSaturateTask(const FScheduledOp&, const FOperation::ImageSaturateArgs&);

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		const FOperation::ImageSaturateArgs Args;
		TManagedPtr<FImage> Result;
		float Factor;
	};


	//---------------------------------------------------------------------------------------------
	FImageSaturateTask::FImageSaturateTask(const FScheduledOp& InOp, const FOperation::ImageSaturateArgs& InArgs)
		: FIssuedTask(InOp)
		, Args(InArgs)
	{
	}

	//---------------------------------------------------------------------------------------------
	bool FImageSaturateTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageSaturateTask_Prepare);
		bOutFailed = false;

		TManagedPtr<const FImage> Source = Runner->LoadImage(FCacheAddress(Args.Base, Op));
		Factor = Runner->LoadScalar(FScheduledOp::FromOpAndOptions(Args.Factor, Op, 0));

		if (!Source)
		{
			Runner->StoreImage(Op, Source);
			return false;
		}

		const bool bOptimizeUnchanged = FMath::IsNearlyEqual(Factor, 1.0f);
		if (bOptimizeUnchanged)
		{
			Runner->StoreImage(Op, Source);
			return false;
		}

		Result = Runner->CloneOrTakeOver(MoveTemp(Source));
		return true;
	}

	//---------------------------------------------------------------------------------------------
	void FImageSaturateTask::DoWork()
	{
		// This runs on a random worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageSaturateTask);

		constexpr bool bUseVectorIntrinsics = true;
		ImageSaturate<bUseVectorIntrinsics>(Result.Get(), Factor);
	}

	//---------------------------------------------------------------------------------------------
	bool FImageSaturateTask::Complete(CodeRunner* Runner)
	{
		// This runs in the mutable Runner thread

		// If we didn't take a shortcut
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageResizeTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageResizeTask(const FScheduledOp& InOp, const FOperation::ImageResizeArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner* Runner, bool& bFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		int32 ImageCompressionQuality = 0;
		FOperation::ImageResizeArgs Args;
		TManagedPtr<const FImage> Base;
		TManagedPtr<FImage> Result;
		FImageOperator::FImagePixelFormatFunc ImagePixelFormatFunc;
	};


	bool FImageResizeTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageResizeTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		FImageSize destSize = FImageSize(Args.Size[0], Args.Size[1]);

		// Apply the mips-to-skip to the dest size
		int32 MipsToSkip = Op.ExecutionOptions;
		destSize[0] = FMath::Max(destSize[0] >> MipsToSkip, 1);
		destSize[1] = FMath::Max(destSize[1] >> MipsToSkip, 1);

		Base = Runner->LoadImage(FCacheAddress(Args.Source, Op));
		if (!Base || (Base->GetSizeX() == destSize[0] && Base->GetSizeY() == destSize[1]))
		{
			Runner->StoreImage(Op, Base);
			return false;
		}

		int32 Lods = 1;

		// If the source image had mips, generate them as well for the resized image.
		// This shouldn't happen often since it should be usually optimised  during model compilation. 
		// The mipmap generation below is not very precise with the number of mips that are needed and 
		// will probably generate too many
		bool bSourceHasMips = Base->GetLODCount() > 1;
		if (bSourceHasMips)
		{
			Lods = FImage::GetMipmapCount(destSize[0], destSize[1]);
		}

		if (Base->IsReference())
		{
			// We are trying to resize an external reference. This shouldn't happen, but be deffensive.
			Runner->StoreImage(Op, Base);
			return false;
		}

		Result = Runner->CreateImage( destSize[0], destSize[1], Lods, Base->GetFormat(), EInitializationType::NotInitialized );

		ImagePixelFormatFunc = Runner->LiveInstance->PixelFormatOverride;

		return true;
	}


	//---------------------------------------------------------------------------------------------
	void FImageResizeTask::DoWork()
	{
		// This runs on a random worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageResizeTask);

		FImageSize destSize = FImageSize( Args.Size[0], Args.Size[1]);

		// Apply the mips-to-skip to the dest size
		int32 MipsToSkip = Op.ExecutionOptions;
		destSize[0] = FMath::Max(destSize[0] >> MipsToSkip, 1);
		destSize[1] = FMath::Max(destSize[1] >> MipsToSkip, 1);

		// Warning: This will actually allocate temp memory that may exceed the budget.
		// \TODO: Fix it.
		FImageOperator ImOp = FImageOperator::GetDefault(ImagePixelFormatFunc);
		ImOp.ImageResizeLinear( Result.Get(), ImageCompressionQuality, Base.Get());

		int32 LodCount = Result->GetLODCount();
		if (LodCount>1)
		{
			FMipmapGenerationSettings mipSettings{};
			ImageMipmapInPlace( ImageCompressionQuality, Result.Get(), mipSettings );
		}
	}


	//---------------------------------------------------------------------------------------------
	bool FImageResizeTask::Complete(CodeRunner* Runner)
	{
		// This runs in the Runner thread
		Base = nullptr;

		// If didn't take a shortcut and set it already
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageResizeRelTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageResizeRelTask(const FScheduledOp& InOp, const FOperation::ImageResizeRelArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner* Runner, bool& bFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		FOperation::ImageResizeRelArgs Args;
		int32 ImageCompressionQuality=0;
		TManagedPtr<const FImage> Base;
		TManagedPtr<FImage> Result;
		FImageSize DestSize;
		FImageOperator::FImagePixelFormatFunc ImagePixelFormatFunc;
	};


	bool FImageResizeRelTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageResizeRelTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		Base = Runner->LoadImage(FCacheAddress(Args.Source, Op));
		if (!Base->DataStorage.NumLODs)
		{
			Runner->StoreImage(Op, Base.ToStrongRef());
			return false;
		}

		DestSize = FImageSize(
			uint16(FMath::Max(1.0, Base->GetSizeX() * Args.Factor[0] + 0.5f)),
			uint16(FMath::Max(1.0, Base->GetSizeY() * Args.Factor[1] + 0.5f)));

		if (Base->GetSizeX() == DestSize[0] && Base->GetSizeY() == DestSize[1])
		{
			Runner->StoreImage(Op, Base);
			return false;
		}

		int32 Lods = 1;

		// If the source image had mips, generate them as well for the resized image.
		// This shouldn't happen often since it should be usually optimised  during model compilation. 
		// The mipmap generation below is not very precise with the number of mips that are needed and 
		// will probably generate too many
		bool bSourceHasMips = Base->GetLODCount() > 1;
		if (bSourceHasMips)
		{
			Lods = FImage::GetMipmapCount(DestSize[0], DestSize[1]);
		}

		if (Base->IsReference())
		{
			// We are trying to resize an external reference. This shouldn't happen, but be deffensive.
			Runner->StoreImage(Op, Base.ToStrongRef());
			return false;
		}

		Result = Runner->CreateImage(DestSize[0], DestSize[1], Lods, Base->GetFormat(), EInitializationType::NotInitialized);

		ImagePixelFormatFunc = Runner->LiveInstance->PixelFormatOverride;

		return true;
	}


	void FImageResizeRelTask::DoWork()
	{
		// This runs on a random worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageResizeRelTask);

		// \TODO: Track allocs
		FImageOperator ImOp = FImageOperator::GetDefault(ImagePixelFormatFunc);
		ImOp.ImageResizeLinear(Result.Get(), ImageCompressionQuality, Base.Get());

		int32 LodCount = Result->GetLODCount();
		if (LodCount > 1)
		{
			FMipmapGenerationSettings mipSettings{};
			ImageMipmapInPlace(ImageCompressionQuality, Result.Get(), mipSettings);
		}
	}


	bool FImageResizeRelTask::Complete(CodeRunner* Runner)
	{
		// This runs in the Runner thread
		Base = nullptr;

		// If didn't take a shortcut and set it already
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageInvertTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageInvertTask(const FScheduledOp& InOp, const FOperation::ImageInvertArgs& InArgs)
			: FIssuedTask(InOp), Args(InArgs)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner* Runner) override;

	private:
		TManagedPtr<FImage> Result;
		FOperation::ImageInvertArgs Args;
	};


	bool FImageInvertTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageInvertTask_Prepare);
		bOutFailed = false;

		TManagedPtr<const FImage> Source = Runner->LoadImage({ Args.Base, Op.ExecutionIndex, Op.ExecutionOptions });

		if (Source)
		{
			// Create destination data
			Result = Runner->CloneOrTakeOver(MoveTemp(Source));
		}

		return true;
	}


	void FImageInvertTask::DoWork()
	{
		// This runs on a random worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageInvertTask);

		ImageInvert(Result.Get());
	}


	bool FImageInvertTask::Complete(CodeRunner* Runner)
	{
		// If we didn't take a shortcut
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	class FImageComposeTask : public CodeRunner::FIssuedTask
	{
	public:
		FImageComposeTask(const FScheduledOp& InOp, const FOperation::ImageComposeArgs& InArgs, const TManagedPtr<const FLayout>& InLayout)
			: FIssuedTask(InOp), Args(InArgs), Layout(InLayout)
		{}

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual void DoWork() override;
		virtual bool Complete(CodeRunner*) override;

	private:
		int32 ImageCompressionQuality = 0;
		FOperation::ImageComposeArgs Args;
		TManagedPtr<const FLayout> Layout;
		TManagedPtr<const FImage> Block;
		TManagedPtr<const FImage> Mask;
		TManagedPtr<FImage> Result;
		box<FIntVector2> Rect;
		FImageOperator::FImagePixelFormatFunc ImagePixelFormatFunc;
	};


	bool FImageComposeTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageComposeTask_Prepare);
		bOutFailed = false;

		ImageCompressionQuality = Runner->Settings.ImageCompressionQuality;

		TManagedPtr<const FImage> Base = Runner->LoadImage({ Args.base, Op.ExecutionIndex, Op.ExecutionOptions });

		int32 RelBlockIndex = Layout->FindBlock(Args.BlockId);

		// Shortcuts
		if (RelBlockIndex < 0)
		{
			Runner->StoreImage(Op, Base);
			return false;
		}

		// Only load the image is RelBlockIndex is valid, otherwise, we won't have requested it.
		Block = Runner->LoadImage({ Args.blockImage, Op.ExecutionIndex, Op.ExecutionOptions });
		if (Args.mask)
		{
			Mask = Runner->LoadImage({ Args.mask, Op.ExecutionIndex, Op.ExecutionOptions });
		}

		box<FIntVector2> RectInblocks;
		RectInblocks.min = Layout->Blocks[RelBlockIndex].Min;
		RectInblocks.size = Layout->Blocks[RelBlockIndex].Size;

		// Convert the rect from blocks to pixels
		FIntPoint Grid = Layout->GetGridSize();
		int32 BlockSizeX = Base->GetSizeX() / Grid[0];
		int32 BlockSizeY = Base->GetSizeY() / Grid[1];
		Rect = RectInblocks;
		Rect.min[0] *= BlockSizeX;
		Rect.min[1] *= BlockSizeY;
		Rect.size[0] *= BlockSizeX;
		Rect.size[1] *= BlockSizeY;

		if (!(Block && Rect.size[0] && Rect.size[1] && Block->GetSizeX() && Block->GetSizeY()))
		{
			Block = nullptr;
			Mask = nullptr;
			Runner->StoreImage(Op, Base);
			return false;
		}

		// Create destination data
		Result = Runner->CloneOrTakeOver(MoveTemp(Base));
		Result->Flags = 0;

		bool useMask = Args.mask != 0;
		if (!useMask)
		{
			MUTABLE_CPUPROFILER_SCOPE(ImageComposeWithoutMask);

			FImageOperator ImOp = MakeImageOperator(Runner);

			EImageFormat Format = GetMostGenericFormat(Result->GetFormat(), Block->GetFormat());

			// Resize image if it doesn't fit in the new block size
			if (FIntVector2(Block->GetSize()) != Rect.size)
			{
				MUTABLE_CPUPROFILER_SCOPE(ImageComposeWithoutMask_BlockResize);

				// This now happens more often since the generation of specific mips on request. For this reason
				// this warning is usually acceptable.
				TManagedPtr<FImage> Resized = Runner->CreateImage(Rect.size[0], Rect.size[1], 1, Block->GetFormat(), EInitializationType::NotInitialized );
				ImOp.ImageResizeLinear(Resized.Get(), ImageCompressionQuality, Block.Get());
				Block = Resized;
			}

			// Change the block image format if it doesn't match the composed image
			// This is usually enforced at object compilation time.
			if (Result->GetFormat() != Block->GetFormat())
			{
				MUTABLE_CPUPROFILER_SCOPE(ImageComposeReformat);

				if (Result->GetFormat() != Format)
				{
					TManagedPtr<FImage> Formatted = Runner->CreateImage(Result->GetSizeX(), Result->GetSizeY(), Result->GetLODCount(), Format, EInitializationType::NotInitialized);
					bool bSuccess = false;
					ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Result.Get());
					check(bSuccess); // Decompression cannot fail
					Result = Formatted;
				}
				if (Block->GetFormat() != Format)
				{
					TManagedPtr<FImage> Formatted = Runner->CreateImage(Block->GetSizeX(), Block->GetSizeY(), Block->GetLODCount(), Format, EInitializationType::NotInitialized);
					bool bSuccess = false;
					ImOp.ImagePixelFormat(bSuccess, ImageCompressionQuality, Formatted.Get(), Block.Get());
					check(bSuccess); // Decompression cannot fail
					Block = Formatted;
				}
			}
		}

		ImagePixelFormatFunc = Runner->LiveInstance->PixelFormatOverride;

		return true;
	}


	void FImageComposeTask::DoWork()
	{
		// This runs on a random worker thread
		MUTABLE_CPUPROFILER_SCOPE(FImageComposeTask);

		bool useMask = Args.mask != 0;
		if (!useMask)
		{
			MUTABLE_CPUPROFILER_SCOPE(ImageComposeWithoutMask);

			// Compose without a mask
			// \TODO: track allocs
			FImageOperator ImOp = FImageOperator::GetDefault(ImagePixelFormatFunc);
			ImOp.ImageCompose(Result.Get(), Block.Get(), Rect);
		}
		else
		{
			MUTABLE_CPUPROFILER_SCOPE(ImageComposeWithMask);

			// Compose with a mask
			ImageBlendOnBaseNoAlpha(Result.Get(), Mask.Get(), Block.Get(), Rect);
		}

		Layout = nullptr;
	}


	bool FImageComposeTask::Complete(CodeRunner* Runner)
	{
		// This runs in the mutable Runner thread
		Block = nullptr;
		Mask = nullptr;

		// If we didn't take a shortcut
		if (Result)
		{
			Runner->StoreImage(Op, Result);
		}

		bool bSuccess = true;
		return bSuccess;
	}

	
	bool CodeRunner::FLoadMeshRomTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FLoadMeshRomTask_Prepare);

		if (!Runner)
		{
			return false;
		}

#if 0 // Enable load failure test.
		{
			float RomLoadFailureRate = CVarRomLoadFailureRate.GetValueOnAnyThread();	

			if (!FMath::IsNearlyZero(RomLoadFailureRate))
			{
				if (FMath::FRand() < RomLoadFailureRate)
				{
					UE_LOGF(LogMutableCore, Warning, "Mesh rom was forced to faile with CVar.");
					bOutFailed = true;
					return false;
				}
			}
		}
#endif
		bOutFailed = false;

		const FProgram& LocalProgram = Runner->Program;
		
		TArray<UE::Tasks::FTask, TInlineAllocator<4>> ReadCompleteEvents;
		ReadCompleteEvents.Reserve(4); 

		TArray<int32, TInlineAllocator<4>> RomsToLoad;

		// Rom indices are sorted by flag value
		static_assert(EMeshContentFlags::GeometryData < EMeshContentFlags::PoseData);
		static_assert(EMeshContentFlags::PoseData     < EMeshContentFlags::PhysicsData);
		static_assert(EMeshContentFlags::PhysicsData  < EMeshContentFlags::MetaData);

		int32 RomContentIndex = 0;

		if (EnumHasAnyFlags(ExecutionContentFlags, EMeshContentFlags::GeometryData) && 
			EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::GeometryData))
		{
			RomsToLoad.Add(RomContentIndex);
		}
		RomContentIndex += EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::GeometryData);

		if (EnumHasAnyFlags(ExecutionContentFlags, EMeshContentFlags::PoseData) && 
			EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::PoseData))
		{
			RomsToLoad.Add(RomContentIndex);
		}
		RomContentIndex += EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::PoseData);
		
		if (EnumHasAnyFlags(ExecutionContentFlags, EMeshContentFlags::PhysicsData) && 
			EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::PhysicsData))
		{
			RomsToLoad.Add(RomContentIndex);
		}
		RomContentIndex += EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::PhysicsData);

		if (EnumHasAnyFlags(ExecutionContentFlags, EMeshContentFlags::MetaData) && 
			EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::MetaData))
		{
			RomsToLoad.Add(RomContentIndex);
		}
		RomContentIndex += EnumHasAnyFlags(RomContentFlags, EMeshContentFlags::MetaData);

		check(RomContentIndex == FMath::CountBits((uint32)RomContentFlags));

		for (const int32 MeshContentRomIndex : RomsToLoad)
		{
			FConstantResourceIndex CurrentIndex = LocalProgram.ConstantMeshContentIndices[MeshContentRomIndex + FirstIndex];

			if (!CurrentIndex.Streamable)
			{
				// This data is always resident.
				continue;
			}

			const int32 RomIndex = CurrentIndex.Index;

			Tasks::TTask<TManagedPtr<const FMesh>> Task = Runner->Model.RomManager.GetRom<FMesh>(RomIndex, Runner->System.StreamInterface.ToSharedRef());
			ReadCompleteEvents.Add(Task);
			RomsStreamed.Add(RomIndex, Task);

			const uint32 RomSize = LocalProgram.Roms[RomIndex].Size;
			check(RomSize > 0);

			Runner->EnsureBudgetBelow(RomSize);
		}

		// Wait for all read operations to end
		Tasks::FTaskEvent GatherReadsCompletionEvent(TEXT("FLoadMeshRomsTask"));
		GatherReadsCompletionEvent.AddPrerequisites(ReadCompleteEvents);
		GatherReadsCompletionEvent.Trigger();

		Event = GatherReadsCompletionEvent;
			
		return false; // No worker thread work
	}	

	//---------------------------------------------------------------------------------------------
	bool CodeRunner::FLoadMeshRomTask::Complete(CodeRunner* Runner)
	{
		// This runs in the Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FLoadMeshRomTask_Complete);
		
		FOperation::MeshConstantArgs Args = Runner->Program.GetOpArgs<FOperation::MeshConstantArgs>(Op.At);
		
		const TPassthroughObjectPtr<UClothingAssetBase> ClothAsset = Runner->FindPassthrough<UClothingAssetBase>(Args.ClothID);

		const EMeshContentFlags MeshContentFlags = static_cast<EMeshContentFlags>(Op.ExecutionOptions);
		TManagedPtr<const FMesh> Source;
		Runner->Model.GetConstant(Args.Value, Args.Skeleton, ClothAsset, Source, MeshContentFlags,
		[this](int32 RomIndex)
		{
			Tasks::TTask<TManagedPtr<const FMesh>>* Found = RomsStreamed.Find(RomIndex);
			return Found ? Found->GetResult() : nullptr;
		},
		[Runner](int32 BudgetReserve)
		{
			return Runner->CreateMesh(BudgetReserve);
		});
			
		if (!Source)
		{
			return false;
		}

		Runner->StoreMesh(Op, Source);

		return true;
	}
	
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	bool CodeRunner::FLoadImageRomsTask::Prepare(CodeRunner* Runner, bool& bOutFailed )
	{
		if (!Runner)
		{
			return false;
		}

		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FLoadImageRomsTask_Prepare);


#if 0 // Enable load failure test.
		{
			float RomLoadFailureRate = CVarRomLoadFailureRate.GetValueOnAnyThread();	

			if (!FMath::IsNearlyZero(RomLoadFailureRate))
			{
				if (FMath::FRand() < RomLoadFailureRate)
				{
					UE_LOGF(LogMutableCore, Warning, "Image rom was forced to fail with CVar.");
					bOutFailed = true;
					return false;
				}
			}
		}
#endif

		bOutFailed = false;

		const FProgram& LocalProgram = Runner->Program;
		
		TArray<UE::Tasks::FTask> ReadCompleteEvents;

		ReadCompleteEvents.Reserve(FMath::Max(0, ImageIndexEnd - ImageIndexBegin));

		for (int32 ImageIndex = ImageIndexBegin; ImageIndex < ImageIndexEnd; ++ImageIndex)
		{
			const FConstantResourceIndex ImageConstantResourceIndex = LocalProgram.ConstantImageLODIndices[ImageIndex];

			if (!ImageConstantResourceIndex.Streamable)
			{
				// This data is always resident.
				continue;
			}

			int32 RomIndex = ImageConstantResourceIndex.Index;
			
			Tasks::TTask<TManagedPtr<const FImage>> Task = Runner->Model.RomManager.GetRom<FImage>(RomIndex, Runner->System.StreamInterface.ToSharedRef());
			RomsStreamed.Add(RomIndex, Task);
			ReadCompleteEvents.Add(Task);
			
			const uint32 RomSize = LocalProgram.Roms[RomIndex].Size;
			check(RomSize > 0);

			Runner->EnsureBudgetBelow(RomSize);
		}

		// Wait for all read operations to end
		Tasks::FTaskEvent GatherReadsCompletionEvent(TEXT("FLoadImageRomsTask"));
		GatherReadsCompletionEvent.AddPrerequisites(ReadCompleteEvents);
		GatherReadsCompletionEvent.Trigger();

		Event = GatherReadsCompletionEvent;
			
		return false; // No worker thread work
	}
	
	
	//---------------------------------------------------------------------------------------------
	bool CodeRunner::FLoadImageRomsTask::Complete(CodeRunner* Runner)
	{
		FOperation::ResourceConstantArgs Args = Runner->Model.GetProgram().GetOpArgs<FOperation::ResourceConstantArgs>(Op.At);

		const int32 MipsToSkip = Op.ExecutionOptions;
		TManagedPtr<const FImage> Source;
		Runner->Model.GetConstant(Args.value, Source, MipsToSkip,
		[this](int32 RomIndex)
		{
			Tasks::TTask<TManagedPtr<const FImage>>* Found = RomsStreamed.Find(RomIndex);
			return Found ? Found->GetResult() : nullptr;
		},
		[Runner](int32 SizeX, int32 SizeY, int32 NumLODs, EImageFormat Format, EInitializationType InitPolicy)
		{
			return Runner->CreateImage(SizeX, SizeY, NumLODs, Format, InitPolicy);
		});

		// Assume the ROM has been loaded previously in a task generated at IssueOp
		if (!Source)
		{
			return false;
		}

		Runner->StoreImage(Op, Source);
		
		return true;
	}


	/** This task is used to load an image parameter (by its FName) or an image reference (from its ID).
	*/
	class FImageExternalLoadTask : public CodeRunner::FIssuedTask
	{
	public:

		FImageExternalLoadTask(const FScheduledOp& InItem, uint8 InMipmapsToSkip, CodeRunner::FExternalResourceId InId);

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual bool Complete(CodeRunner* Runner) override;
		
	private:
		uint8 MipmapsToSkip;
		CodeRunner::FExternalResourceId Id;

		TManagedPtr<FImage> Result;
		
		TFunction<void()> ExternalCleanUpFunc;
	};


	FImageExternalLoadTask::FImageExternalLoadTask(const FScheduledOp& InOp, uint8 InMipmapsToSkip, CodeRunner::FExternalResourceId InId)
		: FIssuedTask(InOp)
	{
		MipmapsToSkip = InMipmapsToSkip;
		Id = InId;
	}


	bool FImageExternalLoadTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FImageExternalLoadTask_Prepare);

		// LoadExternalImageAsync will always generate some image even if it is a dummy one.
		bOutFailed = false;

		// Capturing this here should not be a problem. The lifetime of the callback lambda is tied to 
		// the task and the later will always outlive the former. 
		
		// Probably we could simply pass a reference to the result image. 
		TFunction<void (TManagedPtr<FImage>)> ResultCallback = [this](TManagedPtr<FImage> InResult)
		{
			Result = InResult;
		};

		Tie(Event, ExternalCleanUpFunc) = Runner->LoadExternalImageAsync(Id, MipmapsToSkip, ResultCallback);

		// return false indicating there is no work to do so Event is not overriden by a DoWork task.
		return false;
	}


	bool FImageExternalLoadTask::Complete(CodeRunner* Runner)
	{
		if (ExternalCleanUpFunc)
		{
			Invoke(ExternalCleanUpFunc);
		}

		Runner->StoreImage(Op, Result);

		bool bSuccess = true;
		return bSuccess;
	}	


	/** This task is used to load a mesh parameter (by its FName) or a mesh reference (from its ID).
	*/
	class FMeshExternalLoadTask : public CodeRunner::FIssuedTask
	{
	public:

		FMeshExternalLoadTask(const FScheduledOp&, CodeRunner::FExternalResourceId, int32 InBaseLODIndex, int32 InBaseSectionIndex, int32 InLODBias, uint8 ConversionFlags, uint32 MeshID);

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual bool Complete(CodeRunner* Runner) override;

	private:
		uint8 MipmapsToSkip;
		CodeRunner::FExternalResourceId Id;
		int32 BaseLODIndex = 0;
		int32 BaseSectionIndex = 0;
		int32 LODBias = 0;
		uint8 ConversionFlags = 0;
		uint32 MeshID = 0;

		TManagedPtr<FMesh> Result;

		TFunction<void()> ExternalCleanUpFunc;
	};


	FMeshExternalLoadTask::FMeshExternalLoadTask(const FScheduledOp& InOp, CodeRunner::FExternalResourceId InId, int32 InBaseLODIndex, int32 InBaseSectionIndex, int32 InLODBias, uint8 InConversionFlags, uint32 InMeshID)
		: FIssuedTask(InOp)
		, Id(InId)
		, BaseLODIndex(InBaseLODIndex)
		, BaseSectionIndex(InBaseSectionIndex)
		, LODBias(InLODBias)
		, ConversionFlags(InConversionFlags)
		, MeshID(InMeshID)
	{
	}


	bool FMeshExternalLoadTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FMeshExternalLoadTask_Prepare);

		// LoadExternalMeshAsync will always generate some mesh even if it is a dummy one.
		bOutFailed = false;

		// Capturing this here should not be a problem. The lifetime of the callback lambda is tied to 
		// the task and the later will always outlive the former. 

		// Probably we could simply pass a reference to the result mesh. 
		TFunction<void(TManagedPtr<FMesh>)> ResultCallback = [this](TManagedPtr<FMesh> InResult)
			{
				Result = InResult;
			};

		Tie(Event, ExternalCleanUpFunc) = Runner->LoadExternalMeshAsync(Id, BaseLODIndex, BaseSectionIndex, LODBias, ConversionFlags, ResultCallback);

		// return false indicating there is no work to do so Event is not overriden by a DoWork task.
		return false;
	}


	bool FMeshExternalLoadTask::Complete(CodeRunner* Runner)
	{
		if (ExternalCleanUpFunc)
		{
			Invoke(ExternalCleanUpFunc);
		}

		Result->MeshIDPrefix = MeshID;
		Runner->StoreMesh(Op, Result);
		
		bool bSuccess = true;
		return bSuccess;
	}

	/** This task is used to load a SkeletalMesh parameter (by its FName) or a mesh reference (from its ID).
	*/
	class FSkeletalMeshExternalLoadTask : public CodeRunner::FIssuedTask
	{
	public:

		FSkeletalMeshExternalLoadTask(const FScheduledOp&, CodeRunner::FExternalResourceId, uint16 SkeletalMeshOptions, uint32 MeshID, uint8 ConversionFlags);

		// FIssuedTask interface
		virtual bool Prepare(CodeRunner*, bool& bOutFailed) override;
		virtual bool Complete(CodeRunner* Runner) override;

	private:

		CodeRunner::FExternalResourceId Id;
	
		const uint16 SkeletalMeshOptions = 0;
		uint32 MeshID;
		uint8 ConversionFlags;

		TManagedPtr<FSkeletalMesh> Result;

		TFunction<void()> ExternalCleanUpFunc;
	};


	FSkeletalMeshExternalLoadTask::FSkeletalMeshExternalLoadTask(const FScheduledOp& InOp, CodeRunner::FExternalResourceId InId, uint16 InSkeletalMeshOptions, uint32 InMeshID, uint8 InConversionFlags)
		: FIssuedTask(InOp)
		, Id(InId)
		, SkeletalMeshOptions(InSkeletalMeshOptions)
		, MeshID(InMeshID)
		, ConversionFlags(InConversionFlags)
	{
	}


	bool FSkeletalMeshExternalLoadTask::Prepare(CodeRunner* Runner, bool& bOutFailed)
	{
		// This runs in the mutable Runner thread
		MUTABLE_CPUPROFILER_SCOPE(FMeshExternalLoadTask_Prepare);

		// LoadExternalMeshAsync will always generate some mesh even if it is a dummy one.
		bOutFailed = false;

		// Capturing this here should not be a problem. The lifetime of the callback lambda is tied to 
		// the task and the later will always outlive the former. 

		bool bInitialGeneration = false;
		uint8 LODIndex = 0;
		uint8 FirstLODResident = 0;
		uint8 FirstLODAvailable = 0;
		uint8 NumLODs = 0;

		SkeletalMeshOptionsUnpack(SkeletalMeshOptions, bInitialGeneration, LODIndex, FirstLODResident, FirstLODAvailable, NumLODs);

		int32 LODBegin = 0;
		int32 LODEnd   = 0;

		int32 GeometryLODBegin = 0;
		int32 GeometryLODEnd   = 0;

		if (bInitialGeneration)
		{
			LODBegin = FirstLODAvailable;
			LODEnd = NumLODs;
			GeometryLODBegin = FirstLODResident;
			GeometryLODEnd   = NumLODs;
		}
		else
		{
			LODBegin = LODIndex;
			LODEnd = LODIndex + 1;
			
			GeometryLODBegin = LODBegin;
			GeometryLODEnd = LODEnd;
		}

		// Probably we could simply pass a reference to the result mesh. 
		TFunction<void(TManagedPtr<FSkeletalMesh>)> ResultCallback = [this](TManagedPtr<FSkeletalMesh> InResult)
		{
			Result = InResult;
		};

		Tie(Event, ExternalCleanUpFunc) = Runner->LoadExternalSkeletalMeshAsync(
					Id, LODBegin, LODEnd, GeometryLODBegin, GeometryLODEnd, ConversionFlags, ResultCallback);

		// return false indicating there is no work to do so Event is not overriden by a DoWork task.
		return false;
	}


	bool FSkeletalMeshExternalLoadTask::Complete(CodeRunner* Runner)
	{
		if (ExternalCleanUpFunc)
		{
			Invoke(ExternalCleanUpFunc);
		}

		Runner->StoreSkeletalMesh(Op, Result);
		
		bool bSuccess = true;
		return bSuccess;
	}

	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	TSharedPtr<CodeRunner::FIssuedTask> CodeRunner::IssueOp(FScheduledOp Item)
	{
		if (Item.Type != static_cast<uint16>(FScheduledOp::EType::Full))
		{
			return nullptr;
		}

		TSharedPtr<FIssuedTask> Issued;
		
		EOpType type = Program.GetOpType(Item.At);

		switch (type)
		{
		case EOpType::ME_CONSTANT:
		{
			FOperation::MeshConstantArgs Args = Program.GetOpArgs<FOperation::MeshConstantArgs>(Item.At);
		
			const FMeshContentRange MeshContentRange = Program.ConstantMeshes[Args.Value];
			const EMeshContentFlags ContentFilterFlags = static_cast<EMeshContentFlags>(Item.ExecutionOptions);
			
			Issued = MakeShared<FLoadMeshRomTask>(Item, 
					MeshContentRange.GetFirstIndex(), MeshContentRange.GetContentFlags(), ContentFilterFlags);
			break;
		}

		case EOpType::IM_CONSTANT:
		{
			FOperation::ResourceConstantArgs args = Program.GetOpArgs<FOperation::ResourceConstantArgs>(Item.At);
			int32 MipsToSkip = Item.ExecutionOptions;
			int32 ImageIndex = args.value;
			int32 ReallySkip = FMath::Min(MipsToSkip, Program.ConstantImages[ImageIndex].LODCount - 1);

			const int32 NumLODs         = Program.ConstantImages[ImageIndex].LODCount;
			const int32 NumLODsInTail   = Program.ConstantImages[ImageIndex].NumLODsInTail;
			const int32 FirstImageIndex = Program.ConstantImages[ImageIndex].FirstIndex;
			const int32 FirstLODInTailIndex  = FirstImageIndex + NumLODs - NumLODsInTail;

			check(NumLODs >= NumLODsInTail);

			int32 ImageIndexBegin = FMath::Min(FirstImageIndex + ReallySkip, FirstLODInTailIndex);
			int32 ImageIndexEnd   = FirstLODInTailIndex + 1; 
			check(NumLODs > 0);

			// We always need to follow this path, or roms may not be protected for long enough and might be unloaded 
			// because of memory budget contraints.
			bool bAnyMissing = true;
			//bool bAnyMissing = false;
			//for (int32 i=0; i<LODIndexCount; ++i)
			//{
			//	uint32 LODIndex = Program.ConstantImageLODIndices[LODIndexIndex+i];
			//	if ( !Program.ConstantImageLODs[LODIndex].Value )
			//	{
			//		bAnyMissing = true;
			//		break;
			//	}
			//}

			if (bAnyMissing)
			{
				Issued = MakeShared<FLoadImageRomsTask>(Item, ImageIndexBegin, ImageIndexEnd);

				if (DebugRom && (DebugRomAll || ImageIndex == DebugImageIndex))
					UE_LOGF(LogMutableCore, Log, "Issuing image %d skipping %d .", ImageIndex, ReallySkip);
			}
			else
			{
				// If already available, the rest of the constant code will run right away.
				if (DebugRom && (DebugRomAll || ImageIndex == DebugImageIndex))
					UE_LOGF(LogMutableCore, Log, "Image %d skipping %d is already loaded.", ImageIndex, ReallySkip);
			}
			break;
		}		

		case EOpType::IM_PARAMETER_CONVERT:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageParameterConvertArgs Args = Program.GetOpArgs<FOperation::ImageParameterConvertArgs>(Item.At);
			
				TManagedPtr<const FImage> Image = LoadImage(FCacheAddress(Args.ImageParameter, Item));

				const uint8 MipmapsToSkip = Item.ExecutionOptions;

				FExternalResourceId FullId;
				FullId.ImageParameter = Image->PassthroughObject.Get();
				Issued = MakeShared<FImageExternalLoadTask>(Item, MipmapsToSkip, FullId);
			}

			break;
		}

		case EOpType::SK_CONVERT:
		{
			if (Item.Stage == 1)
			{
				FOperation::FSkeletalMeshConvertArgs Args = Program.GetOpArgs<FOperation::FSkeletalMeshConvertArgs>(Item.At);

				TManagedPtr<const FSkeletalMesh> SkeletalMesh = LoadSkeletalMesh(FCacheAddress(Args.SkeletalMeshObject, Item));

				CodeRunner::FExternalResourceId FullId;
				FullId.MeshParameter = SkeletalMesh->PassthroughObject.Get();

				uint16 SkeletalMeshExecutionOptions = Item.ExecutionOptions;
				Issued = MakeShared<FSkeletalMeshExternalLoadTask>(Item, FullId, SkeletalMeshExecutionOptions, Args.MeshID, Args.ConversionFlags);
			}

			break;
		}
		case EOpType::IM_PARAMETER_FROM_MATERIAL:
		{
			const FOperation::MaterialBreakImageParameterArgs Args = Program.GetOpArgs<FOperation::MaterialBreakImageParameterArgs>(Item.At);
			TSharedPtr<FRangeIndex> Index = BuildCurrentOpRangeIndex(Item, Args.MaterialParameter);

			// Get material parameter from the array of parameters
			UMaterialInterface* Material = Parameters.GetMaterialValue(Args.MaterialParameter, Index.Get());

			// Get the parameter texture from the UMaterial
			if (Material)
			{
				// Get the texture parameter name
				const FName& ParameterName = Program.ConstantNames[Args.ParameterName];

				FHashedMaterialParameterInfo ParameterInfo;
				ParameterInfo.Name = FScriptName(ParameterName);
				ParameterInfo.Index = (int32)Args.LayerIndex;
				ParameterInfo.Association = ParameterInfo.Index == INDEX_NONE ? GlobalParameter : LayerParameter;

				UTexture* Texture = nullptr;
				bool bParameterFound = Material->GetTextureParameterValue(ParameterInfo, Texture);

				if (bParameterFound && Texture)
				{
					const uint8 MipmapsToSkip = Item.ExecutionOptions;

					CodeRunner::FExternalResourceId FullId;
					FullId.ImageParameter = Texture;
					Issued = MakeShared<FImageExternalLoadTask>(Item, MipmapsToSkip, FullId);
				}
			}

			break;
		}

		case EOpType::IM_REFERENCE:
		{
			FOperation::ResourceReferenceArgs Args = Program.GetOpArgs<FOperation::ResourceReferenceArgs>(Item.At);

			// We only convert references to images if indicated in the operation.
			if (Args.ForceLoad)
			{
				check(Item.Stage==0);

				const uint8 MipmapsToSkip = Item.ExecutionOptions;

				FExternalResourceId FullId;
				FullId.ReferenceResourceId = Args.ID;
				Issued = MakeShared<FImageExternalLoadTask>(Item, MipmapsToSkip, FullId);
			}

			break;
		}

		case EOpType::ME_SKELETALMESH_BREAK:
		{
			if (Item.Stage == 1)
			{
				FOperation::MeshSkeletalMeshBreakArgs Args = Program.GetOpArgs<FOperation::MeshSkeletalMeshBreakArgs>(Item.At);

				TManagedPtr<const FSkeletalMesh> SkeletalMesh = LoadSkeletalMesh(FCacheAddress(Args.SkeletalMeshParameter, Item));

				CodeRunner::FExternalResourceId FullId;
				FullId.MeshParameter = SkeletalMesh->PassthroughObject.Get();
				Issued = MakeShared<FMeshExternalLoadTask>(Item, FullId, Args.BaseLOD, Args.BaseSection, Args.LODBias, Args.Flags, Args.MeshID);
			}

			break;
		}

		case EOpType::IM_PIXELFORMAT:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImagePixelFormatArgs Args = Program.GetOpArgs<FOperation::ImagePixelFormatArgs>(Item.At);
				Issued = MakeShared<FImagePixelFormatTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_LAYERCOLOR:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageLayerColorArgs Args = Program.GetOpArgs<FOperation::ImageLayerColorArgs>(Item.At);
				Issued = MakeShared<FImageLayerColorTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_LAYER:
		{
			if ((ExecutionStrategy == EExecutionStrategy::MinimizeMemory && Item.Stage == 2)
				||
				(ExecutionStrategy != EExecutionStrategy::MinimizeMemory && Item.Stage == 1)
				)
			{
				FOperation::ImageLayerArgs Args = Program.GetOpArgs<FOperation::ImageLayerArgs>(Item.At);
				Issued = MakeShared<FImageLayerTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_MIPMAP:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageMipmapArgs Args = Program.GetOpArgs<FOperation::ImageMipmapArgs>(Item.At);
				Issued = MakeShared<FImageMipmapTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_SWIZZLE:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageSwizzleArgs Args = Program.GetOpArgs<FOperation::ImageSwizzleArgs>(Item.At);
				Issued = MakeShared<FImageSwizzleTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_SATURATE:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageSaturateArgs Args = Program.GetOpArgs<FOperation::ImageSaturateArgs>(Item.At);
				Issued = MakeShared<FImageSaturateTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_INVERT:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageInvertArgs Args = Program.GetOpArgs<FOperation::ImageInvertArgs>(Item.At);
				Issued = MakeShared<FImageInvertTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_RESIZE:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageResizeArgs Args = Program.GetOpArgs<FOperation::ImageResizeArgs>(Item.At);
				Issued = MakeShared<FImageResizeTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_RESIZEREL:
		{
			if (Item.Stage == 1)
			{
				FOperation::ImageResizeRelArgs Args = Program.GetOpArgs<FOperation::ImageResizeRelArgs>(Item.At);
				Issued = MakeShared<FImageResizeRelTask>(Item, Args);
			}
			break;
		}

		case EOpType::IM_COMPOSE:
		{
			if ((ExecutionStrategy == EExecutionStrategy::MinimizeMemory && Item.Stage == 3) ||
				(ExecutionStrategy != EExecutionStrategy::MinimizeMemory && Item.Stage == 2))
			{
				FOperation::ImageComposeArgs Args = Program.GetOpArgs<FOperation::ImageComposeArgs>(Item.At);
				TManagedPtr<const FLayout> ComposeLayout = StaticCastManagedPtr<const FLayout>( HeapData[Item.CustomState].Resource);
				Issued = MakeShared<FImageComposeTask>(Item, Args, ComposeLayout);
			}
			break;
		}

		default:
			break;
		}

		return Issued;
	}
} // namespace UE::Mutable::Private
