// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//! A config file may have been provided with a define in the command-linewith something like
//! -D'MUTABLE_CONFIG="ConfigAndroid.h"'

#ifdef MUTABLE_CONFIG

#include MUTABLE_CONFIG

//! otherwise, use this one
#else

// This file contains all the possible configuration flags that affect the build of the runtime

//! This flag enables collection of statistics in the System objects. If using the waf build system,
//! the statistics are only collected in the debug and profile versions of the runtime.
//#define MUTABLE_PROFILE

//! Exclude support for BC(DXTC) image formats
//! This may also be set by the build system for some target platforms.
//#define MUTABLE_EXCLUDE_BC

//! Exclude support for ASTC image formats
//! This may also be set by the build system for some target platforms.
//#define MUTABLE_EXCLUDE_ASTC

//! Exclude support for higher quality compression image conversion (see Miro.h)
//! This may also be set by the build system for some target platforms.
//#define MUTABLE_EXCLUDE_IMAGE_COMPRESSION_QUALITY 4

#endif
