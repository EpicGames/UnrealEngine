// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/FloatAdapter.h"


namespace UE::Mutable
{
	FFloatAdapter::FFloatAdapter(const FFloatAdapter& Other)
	{
		Value = Other.Value;
	}

	
	FFloatAdapter& FFloatAdapter::operator=(const FFloatAdapter& Other)
	{
		Value = Other.Value;
		return *this;
	}


	float FFloatAdapter::GetValue() const
	{
		return Value;
	}


	void FFloatAdapter::SetValue(float InValue)
	{
		Value = InValue;
	}
}
