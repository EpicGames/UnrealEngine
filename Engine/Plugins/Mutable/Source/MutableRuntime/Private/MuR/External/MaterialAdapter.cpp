// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/MaterialAdapter.h"

#include "MuR/Material.h"
#include "MuR/Image.h"
#include "MuR/External/FloatAdapter.h"
#include "MuR/External/TextureAdapter.h"
#include "MuR/External/VectorAdapter.h"


namespace UE::Mutable
{
	FMaterialAdapter::FMaterialAdapter()
	{
		Material = Private::MakeManaged<Private::FMaterial>();
	}
	

	FMaterialAdapter::FMaterialAdapter(const FMaterialAdapter& Other)
	{
		Material = Private::MakeManaged<Private::FMaterial>();
		*Material = *Other.Material;
	}

	
	FMaterialAdapter& FMaterialAdapter::operator=(const FMaterialAdapter& Other)
	{
		*Material = *Other.Material;
		return *this;
	}


	TOptional<TValueConst<FFloatAdapter>> FMaterialAdapter::GetFloat(const Private::FParameterKey& ParameterKey) const
	{
		float* Result = Material->ScalarParameters.Find(ParameterKey);
		if (!Result)
		{
			return {};
		}

		TValue<FFloatAdapter> Value;
		Value.Ptr = Private::MakeManaged<FInstancedStruct>();
		Value.Ptr->InitializeAs(FFloatAdapter::StaticStruct());
		Value.Ptr->GetMutable<FFloatAdapter>().Value = *Result;
		
		return TOptional<TValueConst<FFloatAdapter>>(MoveTemp(Value));
	}


	bool FMaterialAdapter::SetFloat(const Private::FParameterKey& ParameterKey, const TValueConst<FFloatAdapter>* Value)
	{
		if (!Material->ScalarParameters.Contains(ParameterKey))
		{
			return false;
		}

		if (Value)
		{
			Material->ScalarParameters.Add(ParameterKey, Value->Get().GetValue());
		}
		else
		{
			Material->ScalarParameters.Remove(ParameterKey);
		}
		
		return true;
	}


	TOptional<TValueConst<FVectorAdapter>> FMaterialAdapter::GetVector(const Private::FParameterKey& ParameterKey) const
	{
		FVector4f* Result = Material->ColorParameters.Find(ParameterKey);
		if (!Result)
		{
			return {};
		}

		TValue<FVectorAdapter> Value;
		Value.Ptr = Private::MakeManaged<FInstancedStruct>();
		Value.Ptr->InitializeAs(FVectorAdapter::StaticStruct());
		Value.Ptr->GetMutable<FVectorAdapter>().Value = *Result;
		
		return TOptional<TValueConst<FVectorAdapter>>(MoveTemp(Value));
	}


	bool FMaterialAdapter::SetVector(const Private::FParameterKey& ParameterKey, const TValueConst<FVectorAdapter>* Value)
	{
		if (!Material->ColorParameters.Contains(ParameterKey))
		{
			return false;
		}

		if (Value)
		{
			Material->ColorParameters.Add(ParameterKey, Value->Get().GetValue());
		}
		else
		{
			Material->ColorParameters.Remove(ParameterKey);
		}
		
		return true;
	}


	TOptional<TValueConst<FTextureAdapter>> FMaterialAdapter::GetTexture(const Private::FParameterKey& ParameterKey) const
	{
		UE::Mutable::Private::FMaterial::FImageParameterData* Result = Material->ImageParameters.Find(ParameterKey);
		if (!Result)
		{
			return {};
		}

		TVariant<Private::FOperation::ADDRESS, Private::TManagedPtr<const Private::FImage>> Variant = Result->ImageParameter;

		if (ensure(Variant.IsType<Private::TManagedPtr<const Private::FImage>>()))
		{
			TValue<FTextureAdapter> Value;
			Value.Ptr = Private::MakeManaged<FInstancedStruct>();
			Value.Ptr->InitializeAs(FTextureAdapter::StaticStruct());
			Value.Ptr->GetMutable<FTextureAdapter>().Image = ConstCastManagedPtr<Private::FImage>(Variant.Get<Private::TManagedPtr<const Private::FImage>>()); // TODO GMT If lazy wait
			
			return TOptional<TValueConst<FTextureAdapter>>(MoveTemp(Value));
		}
		else
		{
			return {};
		}
	}

	bool FMaterialAdapter::SetTexture(const Private::FParameterKey& ParameterKey, const TValueConst<FTextureAdapter>* Value)
	{
		UE::Mutable::Private::FMaterial::FImageParameterData* ImageData = Material->ImageParameters.Find(ParameterKey);

		if (!ImageData)
		{
			return false;
		}

		if (Value)
		{
			TVariant<Private::FOperation::ADDRESS, Private::TManagedPtr<const Private::FImage>> Image;
			Image.Set<Private::TManagedPtr<const Private::FImage>>(Value->Get().Image);
			
			Material->ImageParameters.Add(ParameterKey, { Image, ImageData->ImagePropertyIndex });
		}
		else
		{
			Material->ImageParameters.Remove(ParameterKey);
		}
		
		return true;
	}
}
