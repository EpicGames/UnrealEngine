// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/MeshAdapter.h"

#include "MuR/Mesh.h"
#include "MuR/MeshPrivate.h"
#include "MuR/OpMeshRemove.h"


namespace UE::Mutable
{
	FMeshAdapter::FMeshAdapter()
	{
		Mesh = Private::MakeManaged<Private::FMesh>();
	}
	

	FMeshAdapter::FMeshAdapter(const FMeshAdapter& Other)
	{
		Mesh = Private::MakeManaged<Private::FMesh>();
		Mesh->CopyFrom(*Other.Mesh);
	}

	
	FMeshAdapter& FMeshAdapter::operator=(const FMeshAdapter& Other)
	{
		Mesh->CopyFrom(*Other.Mesh);
		return *this;
	}


	int32 FMeshAdapter::GetNumVertices() const
	{
		return Mesh->GetVertexCount();
	}

	FVector3f FMeshAdapter::GetVertex(int32 Index) const
	{
		Private::MeshBufferIterator<Private::EMeshBufferFormat::Float32, float, 3> BasePositionIter(Mesh->VertexBuffers, Private::EMeshBufferSemantic::Position, 0);
		BasePositionIter += Index;

		return BasePositionIter.GetAsVec3f();
	}

	
	void FMeshAdapter::SetVertex(int32 Index, const FVector3f& Vertex)
	{
		Private::MeshBufferIterator<Private::EMeshBufferFormat::Float32, float, 3> BasePositionIter(Mesh->VertexBuffers, Private::EMeshBufferSemantic::Position, 0);
		BasePositionIter += Index;

		BasePositionIter.SetFromVec3f(Vertex);
	}
	

	void FMeshAdapter::RemoveVertices(const TBitArray<>& Vertices)
	{
		Private::MeshRemoveVerticesWithCullSet(Mesh.Get(), Vertices, false);
	}


	Private::TManagedPtr<Private::FMesh> FMeshAdapter::GetPrivate()
	{
		return Mesh;
	}


	Private::TManagedPtr<const Private::FMesh> FMeshAdapter::GetPrivate() const
	{
		return Mesh;
	}
}
