// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/Operation.h"


namespace UE::Mutable
{
	FValueConst FContext::GetInput(const FText& Name)
	{
		return Inputs.FindAndRemoveChecked(Name.ToString());
	}

	void FContext::SetOutput(FValueConst&& Value)
	{
		Output = MoveTemp(Value);
	}
}
