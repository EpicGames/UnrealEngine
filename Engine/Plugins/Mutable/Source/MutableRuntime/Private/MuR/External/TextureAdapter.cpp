// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/TextureAdapter.h"

#include "MuR/Image.h"

namespace UE::Mutable
{
	FTextureAdapter::FTextureAdapter()
	{
		Image = Private::MakeManaged<Private::FImage>();		
	}


	FTextureAdapter::FTextureAdapter(const FTextureAdapter& Other)
	{
		Image = Private::MakeManaged<Private::FImage>();
		Image->Copy(Other.Image.Get());
	}


	FTextureAdapter& FTextureAdapter::operator=(const FTextureAdapter& Other)
	{
		Image->Copy(Other.Image.Get());
		return *this;
	}
}
