// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/Value.h"

#include "StructUtils/InstancedStruct.h"


namespace UE::Mutable
{
	FValue CopyOrMove(FValueConst&& ValueConst)
	{
		if (ValueConst.Ptr.IsUniqueReference())
		{
			FValue Value;
			Value.Ptr = Private::ConstCastManagedPtr<FInstancedStruct>(ValueConst.Ptr); 

			return MoveTemp(Value);
		}
		else
		{
			FValue Value;
			Value.Ptr = Private::MakeManaged<FInstancedStruct>(*ValueConst.Ptr); 

			return MoveTemp(Value);
		}
	}

	
	FValueConst::FValueConst(FValue&& Value)
	{
		Ptr = MoveTemp(Value.Ptr);
	}
}
