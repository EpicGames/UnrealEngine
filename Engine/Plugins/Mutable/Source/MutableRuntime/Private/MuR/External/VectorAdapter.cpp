// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/External/VectorAdapter.h"


namespace UE::Mutable
{
	FVectorAdapter::FVectorAdapter(const FVectorAdapter& Other)
	{
		Value = Other.Value;
	}

	
	FVectorAdapter& FVectorAdapter::operator=(const FVectorAdapter& Other)
	{
		Value = Other.Value;
		return *this;
	}


	FVector4f FVectorAdapter::GetValue() const
	{
		return Value;
	}


	void FVectorAdapter::SetValue(const FVector4f& InValue)
	{
		Value = InValue;
	}
}
