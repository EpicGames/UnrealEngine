// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/GeometryUtils.h"

#include "MuR/MutableTrace.h"
#include "MuR/MeshPrivate.h"

namespace UE::Mutable::Private
{

namespace GeometryUtils
{

	void GetMergedGeometryFromMeshes(TArrayView<TNotNull<const FMesh*>> Meshes, bool bNeedsNormals, FMeshGeometry& OutResult)
	{
		MUTABLE_CPUPROFILER_SCOPE(GetMergedGeometryFromMeshes);

		if (Meshes.Num() == 0)
		{
			return;
		}


		// Special case where we only have one mesh, if it's compatible we just take a view to the Mesh data.
		if (Meshes.Num() == 1)
		{
			const UntypedMeshBufferIteratorConst IndicesIter(Meshes[0]->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
			const UntypedMeshBufferIteratorConst NormalsIter(Meshes[0]->GetVertexBuffers(), EMeshBufferSemantic::Normal);
			MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> PositionIter(Meshes[0]->GetVertexBuffers(), EMeshBufferSemantic::Position);
	
			check(PositionIter.ptr());
			check(IndicesIter.ptr());

			int32 NumVertices = Meshes[0]->GetVertexCount(); 
			int32 NumIndices = Meshes[0]->GetIndexCount(); 

			OutResult.Positions = TConstArrayView<FVector3f>(reinterpret_cast<const FVector3f*>(PositionIter.ptr()), NumVertices);

			if (bNeedsNormals)
			{
				if (NormalsIter.ptr())
				{ 
					if (NormalsIter.GetFormat() == EMeshBufferFormat::Float32)
					{
						OutResult.Normals = TConstArrayView<FVector3f>(reinterpret_cast<const FVector3f*>(NormalsIter.ptr()), Meshes[0]->GetVertexCount());
					}
					else
					{
						OutResult.OptionalNormalsStorage.SetNumUninitialized(NumVertices);
						for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
						{
							OutResult.OptionalNormalsStorage[VertexIndex] = (NormalsIter + VertexIndex).GetAsVec3f();
						}
						
						OutResult.Normals = TConstArrayView<FVector3f>(OutResult.OptionalNormalsStorage);
					}
				}	
			}


			const int32 NumTriangles = NumIndices / 3;
			if (IndicesIter.GetFormat() == EMeshBufferFormat::UInt32 || IndicesIter.GetFormat() == EMeshBufferFormat::Int32)
			{
				OutResult.Triangles = TConstArrayView<UE::Geometry::FIndex3i>(reinterpret_cast<const UE::Geometry::FIndex3i*>(IndicesIter.ptr()), NumTriangles);
			}
			else
			{
				OutResult.OptionalTrianglesStorage.SetNumUninitialized(NumTriangles);
				for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					UE::Geometry::FIndex3i Triangle;
					Triangle.A = int32((IndicesIter + TriangleIndex * 3 + 0).GetAsUINT32());
					Triangle.B = int32((IndicesIter + TriangleIndex * 3 + 1).GetAsUINT32());
					Triangle.C = int32((IndicesIter + TriangleIndex * 3 + 2).GetAsUINT32());

					OutResult.OptionalTrianglesStorage[TriangleIndex] = Triangle;
				}

				OutResult.Triangles = TConstArrayView<UE::Geometry::FIndex3i>(OutResult.OptionalTrianglesStorage);
			}

		}
		else
		{
			// Otherwise we need to merge or convert the meshes.

			int32 NumVerticesToAllocate = 0;
			int32 NumIndicesToAllocate = 0;

			bool bAllMeshesHaveNormals = true;

			for (TNotNull<const FMesh*> Mesh : Meshes)
			{
				NumVerticesToAllocate += Mesh->GetVertexCount();		
				NumIndicesToAllocate += Mesh->GetIndexCount();
				
				const UntypedMeshBufferIteratorConst NormalsIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);
			
				bAllMeshesHaveNormals &= !!NormalsIter.ptr();
			}

			OutResult.OptionalPositionsStorage.SetNumUninitialized(NumVerticesToAllocate);
			OutResult.OptionalTrianglesStorage.SetNumUninitialized(NumIndicesToAllocate / 3);
			
			if (bNeedsNormals && bAllMeshesHaveNormals)
			{
				OutResult.OptionalNormalsStorage.SetNumUninitialized(NumVerticesToAllocate);
			}

			int32 VerticesOffset = 0;
			int32 TrianglesOffset = 0;
			for (TNotNull<const FMesh*> Mesh : Meshes)
			{
				int32 NumVertices = Mesh->GetVertexCount();

				MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> PositionIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
				check(PositionIter.ptr());

				TArrayView<const FVector3f> PositionsView = TArrayView<const FVector3f>(reinterpret_cast<const FVector3f*>(PositionIter.ptr()), NumVertices);
				FMemory::Memcpy(
						OutResult.OptionalPositionsStorage.GetData() + VerticesOffset,
						PositionsView.GetData(),
						PositionsView.NumBytes());

				const UntypedMeshBufferIteratorConst NormalsIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);
				if (bNeedsNormals && bAllMeshesHaveNormals)
				{
					if (NormalsIter.GetFormat() == EMeshBufferFormat::Float32)
					{
						TConstArrayView<FVector3f> NormalsView(reinterpret_cast<const FVector3f*>(NormalsIter.ptr()), NumVertices);
						FMemory::Memcpy(
							OutResult.OptionalNormalsStorage.GetData() + VerticesOffset,
							NormalsView.GetData(),
							NormalsView.NumBytes());
					}
					else
					{
						for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
						{
							OutResult.OptionalNormalsStorage[VertexIndex + VerticesOffset] = (NormalsIter + VertexIndex).GetAsVec3f();
						}
					}
				}

				bool bNeedsIndicesFixup = VerticesOffset > 0;

				const int32 NumTriangles = Mesh->GetIndexCount() / 3;

				const UntypedMeshBufferIteratorConst IndicesIter(Mesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
				check(IndicesIter.ptr());

				if (IndicesIter.GetFormat() == EMeshBufferFormat::UInt32 || IndicesIter.GetFormat() == EMeshBufferFormat::Int32)
				{
					TConstArrayView<UE::Geometry::FIndex3i> TrianglesView(reinterpret_cast<const UE::Geometry::FIndex3i*>(IndicesIter.ptr()), NumTriangles);

					FMemory::Memcpy(
						OutResult.OptionalTrianglesStorage.GetData() + TrianglesOffset,
						TrianglesView.GetData(),
						TrianglesView.NumBytes());
				}
				else
				{
					for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
					{
						UE::Geometry::FIndex3i Triangle;
						Triangle.A = int32((IndicesIter + TriangleIndex * 3 + 0).GetAsUINT32()) + VerticesOffset;
						Triangle.B = int32((IndicesIter + TriangleIndex * 3 + 1).GetAsUINT32()) + VerticesOffset;
						Triangle.C = int32((IndicesIter + TriangleIndex * 3 + 2).GetAsUINT32()) + VerticesOffset;

						OutResult.OptionalTrianglesStorage[TrianglesOffset + TriangleIndex] = Triangle;
					}
		
					bNeedsIndicesFixup = false;
				}

				if (bNeedsIndicesFixup)
				{
					TArrayView<UE::Geometry::FIndex3i> TrianglesToFixUp = TArrayView<UE::Geometry::FIndex3i>(
							OutResult.OptionalTrianglesStorage.GetData() + TrianglesOffset,
							NumTriangles);

					for (UE::Geometry::FIndex3i& Triangle : TrianglesToFixUp)
					{
						Triangle.A += VerticesOffset;
						Triangle.B += VerticesOffset;
						Triangle.C += VerticesOffset;
					}
				}

				VerticesOffset += NumVertices;
				TrianglesOffset += NumTriangles;
			}

			OutResult.Positions = TConstArrayView<FVector3f>(OutResult.OptionalPositionsStorage);
			OutResult.Triangles = TConstArrayView<UE::Geometry::FIndex3i>(OutResult.OptionalTrianglesStorage);
			
			if (bNeedsNormals)
			{
				OutResult.Normals = TConstArrayView<FVector3f>(OutResult.OptionalNormalsStorage);
			}
		}
	}

} // namespace GeometryUtils
} // namespace UE::Mutable::Private
