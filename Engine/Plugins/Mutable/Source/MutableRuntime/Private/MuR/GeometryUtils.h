// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "HAL/Platform.h"
#include "Math/Vector.h"
#include "IndexTypes.h"
#include "Spatial/MeshAABBTree3.h"

namespace UE::Mutable::Private
{
	class FMesh;

namespace GeometryUtils
{
	struct FMeshGeometry
	{
		UE_NONCOPYABLE(FMeshGeometry);

		FMeshGeometry() = default;
		
		TConstArrayView<FVector3f> Positions;
		TConstArrayView<UE::Geometry::FIndex3i> Triangles;
		
		// Optional, no elements indicate the data is not avaliable.
		TConstArrayView<FVector3f> Normals;

		// It can be used for storing the a format conversion. 
		TArray<FVector3f> OptionalPositionsStorage;
		TArray<FVector3f> OptionalNormalsStorage;
		TArray<UE::Geometry::FIndex3i> OptionalTrianglesStorage;
	};

	struct FMeshAdapter
	{
		FMeshGeometry& Mesh;
		
		FMeshAdapter(FMeshGeometry& InMesh)
			: Mesh(InMesh)
		{
		}

		int32 MaxTriangleID() const
		{
			return Mesh.Triangles.Num();
		}

		int32 MaxVertexID() const
		{
			return Mesh.Positions.Num();
		}

		bool IsTriangle(int32 Index) const
		{
			return Mesh.Triangles.IsValidIndex(Index);
		}

		bool IsVertex(int32 Index) const
		{
			return Mesh.Positions.IsValidIndex(Index);
		}

		int32 TriangleCount() const
		{
			return Mesh.Triangles.Num();
		}

		FORCEINLINE int32 VertexCount() const
		{
			return Mesh.Positions.Num();
		}

		FORCEINLINE uint64 GetChangeStamp() const
		{
			return 1;
		}

		FORCEINLINE UE::Geometry::FIndex3i GetTriangle(int32 Index) const
		{
			return Mesh.Triangles[Index];
		}

		FORCEINLINE FVector3d GetVertex(int32 Index) const
		{
			return (FVector3d)Mesh.Positions[Index];
		}

		FORCEINLINE void GetTriVertices(int32 TriIndex, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
		{
			UE::Geometry::FIndex3i Triangle = Mesh.Triangles[TriIndex];

			V0 = (FVector3d)Mesh.Positions[Triangle.A];
			V1 = (FVector3d)Mesh.Positions[Triangle.B];
			V2 = (FVector3d)Mesh.Positions[Triangle.C];
		}
	};

	void GetMergedGeometryFromMeshes(TArrayView<TNotNull<const FMesh*>> Meshes, bool bNeedsNormals, FMeshGeometry& OutResult);

} // namespace GeometryUtils
} // namespace UE::Mutable::Private
