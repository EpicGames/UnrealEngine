// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/Image.h"

#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Math/NumericLimits.h"
#include "MuR/ImagePrivate.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "ImageCore.h"

namespace UE::Mutable::Private
{

	FImage::FImage()
	{
	}

	FImage::FImage(uint32 SizeX, uint32 SizeY, uint32 Lods, EImageFormat Format, EInitializationType InitType)
	{
		Init(SizeX, SizeY, Lods, Format, InitType);
	}
	
	void FImage::Init(uint32 SizeX, uint32 SizeY, uint32 Lods, EImageFormat Format, EInitializationType InitType)
	{
		MUTABLE_CPUPROFILER_SCOPE(NewImage)
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		const FImageDesc ImageDesc {FImageSize(uint16(SizeX), uint16(SizeY)), Format, EImageFormat::None, static_cast<uint8>(Lods)};
		DataStorage.Init(ImageDesc, InitType);
	}


	void FImage::InitToBlack()
	{
		const FImageDesc Desc = DataStorage.MakeImageDesc();
		DataStorage.Init(Desc, EInitializationType::Black);

		Flags = 0;
		RelevancyMinY = 0;
		RelevancyMaxY = 0;
	}


	TManagedRef<FImage> FImage::CreateAsReference(const TPassthroughObjectPtr<UTexture>& ExternalObject, const FImageDesc& Desc, bool bForceLoad)
	{
		TManagedRef<FImage> Result = MakeManaged<FImage>();
		Result->PassthroughObject = ExternalObject;
	
		Result->DataStorage.InitVoid(Desc);

		if (bForceLoad)
		{
			Result->Flags = Result->Flags | EImageFlags::IF_IS_FORCELOAD;
		}

		return Result;
	}


	void FImage::Serialise(const FImage* ValuePtr, FOutputArchive& Arch)
	{
		Arch << *ValuePtr;
	}

	void FImage::Serialise(FOutputArchive& Arch) const
	{
		Arch << DataStorage;

		// Remove non-persistent flags.
		uint8 flags = Flags & ~IF_HAS_RELEVANCY_MAP;
		Arch << flags;
	}

	void FImage::Unserialise(FInputArchive& Arch)
	{
		Arch >> DataStorage;
		Arch >> Flags;
	}


	TManagedRef<FImage> FImage::StaticUnserialise(FInputArchive& Arch)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		TManagedRef<FImage> ResultPtr = MakeManaged<FImage>();
		Arch >> *ResultPtr;
		
		return ResultPtr;
	}

	int32 FImage::GetDataSize() const
	{
		return DataStorage.GetAllocatedSize();
	}

	TManagedPtr<FImage> FImageOperator::ExtractMip(const FImage* This, int32 Mip)
	{
		MUTABLE_CPUPROFILER_SCOPE(ExtractMip);
		
		if (Mip == 0 && This->GetLODCount() == 1)
		{
			return CloneImage(This);
		}

		FIntVector2 MipSize = This->CalculateMipSize(Mip);
		
		if (Mip < This->GetLODCount())
		{
			TManagedPtr<FImage> Result = CreateImage(MipSize.X, MipSize.Y, 1, This->GetFormat(), EInitializationType::NotInitialized);
			Result->Flags = This->Flags;

			// RLE formats will not allocate memory when created. Resize manually.
			if (Result->DataStorage.IsEmpty())
			{
				Result->DataStorage.ResizeLOD(0, This->DataStorage.GetLOD(Mip).Num());
			}

			TArrayView<const uint8> SrcView = This->DataStorage.GetLOD(Mip);
			TArrayView<uint8> DstView = Result->DataStorage.GetLOD(0);
			
			check(SrcView.Num() == DstView.Num());
			FMemory::Memcpy(DstView.GetData(), SrcView.GetData(), DstView.Num());

			return Result;	
		}

		TManagedPtr<FImage> Resized = CreateImage(MipSize.X, MipSize.Y, 1, This->GetFormat(), EInitializationType::NotInitialized);
		
		::FImageView SourceView;
		::FImageView DestView;

		// We don't know the gamma space here, set to linear so no conversions are done to be consistent
		// with old behaviour.  
		switch(This->GetFormat())
		{
		case EImageFormat::L_UByte:
		{
			SourceView = ::FImageView((void*)This->DataStorage.GetLOD(0).GetData(),
					This->GetSizeX(), This->GetSizeY(), 1, ERawImageFormat::Type::G8, EGammaSpace::Linear);

			DestView = ::FImageView((void*)Resized->DataStorage.GetLOD(0).GetData(),
					Resized->GetSizeX(), Resized->GetSizeY(), 1, ERawImageFormat::Type::G8, EGammaSpace::Linear);

			break;
		}
		case EImageFormat::RGBA_UByte:
		case EImageFormat::BGRA_UByte:
		{
			SourceView = ::FImageView((void*)This->DataStorage.GetLOD(0).GetData(),
					This->GetSizeX(), This->GetSizeY(), 1, ERawImageFormat::Type::BGRA8, EGammaSpace::Linear);

			DestView = ::FImageView((void*)Resized->DataStorage.GetLOD(0).GetData(),
					Resized->GetSizeX(), Resized->GetSizeY(), 1, ERawImageFormat::Type::BGRA8, EGammaSpace::Linear);

			break;
		}
		default:
			
			checkf(false, TEXT("Image format not supported."));
		}		

		if (SourceView.RawData && DestView.RawData)
		{
			// TODO: Consider using a better resize filter.
			MUTABLE_CPUPROFILER_SCOPE(ImageResize);
			FImageCore::ResizeImage(SourceView, DestView, FImageCore::EResizeImageFilter::Bilinear);
		}

		return Resized;
	}
	
	uint16 FImage::GetSizeX() const
	{
		return GetSize().X;
	}

	uint16 FImage::GetSizeY() const
	{
		return GetSize().Y;
	}

	const FImageSize& FImage::GetSize() const
	{
		return DataStorage.ImageSize;
	}

	EImageFormat FImage::GetFormat() const
	{
		return DataStorage.ImageFormat;
	}

	int32 FImage::GetLODCount() const
	{
		return FMath::Max(1, (int32)DataStorage.NumLODs);
	}

	const uint8* FImage::GetLODData(int32 LOD) const
	{
		return DataStorage.GetLOD(LOD).GetData();
	}

	uint8* FImage::GetLODData(int32 LOD)
	{
		return DataStorage.GetLOD(LOD).GetData();
	}

	int32 FImage::GetLODDataSize(int32 LOD) const
	{
		return DataStorage.GetLOD(LOD).Num();
	}

	bool FImage::IsReference() const
	{
		return PassthroughObject.IsValid();
	}

	bool FImage::IsForceLoad() const
	{
		return Flags & EImageFlags::IF_IS_FORCELOAD;
	}
	
	int32 FImage::CalculateDataSize(int32 SizeX, int32 SizeY, int32 LODCount, EImageFormat Format)
	{
		int32 Result = 0;

		const FImageFormatData& FormatData = GetImageFormatData(Format);
		if (FormatData.BytesPerBlock)
		{
			for (int32 LODIndex = 0; LODIndex < FMath::Max(1, LODCount); ++LODIndex)
			{
				int32 BlocksX = FMath::DivideAndRoundUp(SizeX, int32(FormatData.PixelsPerBlockX));
				int32 BlocksY = FMath::DivideAndRoundUp(SizeY, int32(FormatData.PixelsPerBlockY));

				Result += (BlocksX * BlocksY) * FormatData.BytesPerBlock;

				SizeX = FMath::DivideAndRoundUp(SizeX, 2);
				SizeY = FMath::DivideAndRoundUp(SizeY, 2);
			}
		}

		return Result;
	}


	//---------------------------------------------------------------------------------------------
	FIntVector2 FImage::CalculateMipSize(int32 Mip) const
	{
		FIntVector2 Result = FIntVector2(GetSizeX(), GetSizeY());

		for (int32 L = 0; L < Mip + 1; ++L)
		{
			if (L == Mip)
			{
				return Result;
			}

			Result[0] = FMath::DivideAndRoundUp(Result[0], 2);
			Result[1] = FMath::DivideAndRoundUp(Result[1], 2);
		}

		return FIntVector2(0, 0);
	}

	
	//---------------------------------------------------------------------------------------------
	int32 FImage::GetMipmapCount(int32 SizeX, int32 SizeY)
	{
		if (SizeX <= 0 || SizeY <= 0)
		{
			return 0;
		}

		int32 MaxLevel = FMath::CeilLogTwo(FMath::Max(SizeX, SizeY)) + 1;
		return MaxLevel;
	}


	const uint8* FImage::GetMipData(int32 Mip) const
	{
		return DataStorage.GetLOD(Mip).GetData();
	}


	uint8* FImage::GetMipData(int32 Mip)
	{
		return DataStorage.GetLOD(Mip).GetData();
	}

	int32 FImage::GetMipsDataSize() const
	{
		return DataStorage.GetDataSize();
	}

	FVector4f FImage::Sample(FVector2f Coords) const
	{
		FVector4f Result = FVector4f(0.0f);

		if (GetSizeX() == 0 || GetSizeY() == 0) 
		{ 
			return Result; 
		}

		const FImageFormatData& FormatData = GetImageFormatData(GetFormat());

		FImageSize Size = GetSize();
		EImageFormat Format = GetFormat();

		// TODO: This looks like it should support block compression, but it is not implemented.
		// Decide if it's worth implement it or simplify the ByteOffset computation assuming no
		// compression is allowed.

		int32 PixelX = FMath::Max(0, FMath::Min(Size.X - 1, (int32)(Coords.X * Size.X)));
		int32 BlockX = PixelX / FormatData.PixelsPerBlockX;
		int32 BlockPixelX = PixelX % FormatData.PixelsPerBlockX;

		int32 PixelY = FMath::Max(0, FMath::Min(Size.Y - 1, (int32)(Coords.Y * Size.Y)));
		int32 BlockY = PixelY / FormatData.PixelsPerBlockY;
		int32 BlockPixelY = PixelY % FormatData.PixelsPerBlockY;

		int32 BlocksPerRow = FMath::DivideAndRoundUp(int32(Size.X), int32(FormatData.PixelsPerBlockX));
		int32 BlockOffset = BlockX + BlockY * BlocksPerRow;

		const TArrayView<const uint8> Data = DataStorage.GetLOD(0);

		// Non-generic part
		if (Format == EImageFormat::RGB_UByte)
		{
			int32 ByteOffset = BlockOffset * FormatData.BytesPerBlock
					+ (BlockPixelY * FormatData.PixelsPerBlockX + BlockPixelX) * 3;

			Result[0] = Data[ByteOffset + 0] / 255.0f;
			Result[1] = Data[ByteOffset + 1] / 255.0f;
			Result[2] = Data[ByteOffset + 2] / 255.0f;
			Result[3] = 1.0f;
		}
		else if (Format == EImageFormat::RGBA_UByte)
		{
			int32 ByteOffset = BlockOffset * FormatData.BytesPerBlock
					+ (BlockPixelY * FormatData.PixelsPerBlockX + BlockPixelX) * 4;

			Result[0] = Data[ByteOffset + 0] / 255.0f;
			Result[1] = Data[ByteOffset + 1] / 255.0f;
			Result[2] = Data[ByteOffset + 2] / 255.0f;
			Result[3] = Data[ByteOffset + 3] / 255.0f;
		}
		else if (Format == EImageFormat::BGRA_UByte)
		{
			int32 ByteOffset = BlockOffset * FormatData.BytesPerBlock
					+ (BlockPixelY * FormatData.PixelsPerBlockX + BlockPixelX) * 4;

			Result[0] = Data[ByteOffset + 2] / 255.0f;
			Result[1] = Data[ByteOffset + 1] / 255.0f;
			Result[2] = Data[ByteOffset + 0] / 255.0f;
			Result[3] = Data[ByteOffset + 3] / 255.0f;
		}
		else if (Format== EImageFormat::L_UByte)
		{
			int32 ByteOffset = BlockOffset * FormatData.BytesPerBlock
					+ (BlockPixelY * FormatData.PixelsPerBlockX + BlockPixelX) * 1;

			Result[0] = Data[ByteOffset] / 255.0f;
			Result[1] = Data[ByteOffset] / 255.0f;
			Result[2] = Data[ByteOffset] / 255.0f;
			Result[3] = 1.0f;
		}
		else
		{
			check(false);
		}

		return Result;
	}

	//---------------------------------------------------------------------------------------------
	bool FImage::IsPlainColor(FVector4f& OutColor) const
	{
		bool Result = true;

		const TArrayView<const uint8> DataView = DataStorage.GetLOD(0);
		
		if (DataView.Num())
		{
			switch(GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				uint8 const * const DataPtr = DataView.GetData();

				const int32 NumElems = DataView.Num();
				int32 I = 0;
				for (; I < NumElems; ++I)
				{
					if (FMemory::Memcmp(DataPtr, DataPtr + I*1, 1) != 0)
					{
						break;
					}
				}

				Result = I >= NumElems;
				OutColor = FVector4f(*DataPtr, *DataPtr, *DataPtr, 255) / 255.0f;

				break;
			}

			case EImageFormat::RGB_UByte:
			{
				uint8 const * const DataPtr = DataView.GetData();

				const int32 NumElems = DataView.Num() / 3;
				int32 I = 0;
				for (; I < NumElems; ++I)
				{
					if (FMemory::Memcmp(DataPtr, DataPtr + I*3, 3) != 0)
					{
						break;
					}
				}

				Result = I >= NumElems;
				OutColor = FVector4f(DataPtr[0], DataPtr[1], DataPtr[2], 255) / 255.0f;

				break;
			}

			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				uint8 const * const DataPtr = DataView.GetData();

				const int32 NumElems = DataView.Num() / 4;
				int32 I = 0;
				for (; I < NumElems; ++I)
				{
					if (FMemory::Memcmp(DataPtr, DataPtr + I*4, 4) != 0)
					{
						break;
					}
				}

				Result = I >= NumElems;
				if (GetFormat() == EImageFormat::RGBA_UByte)
				{
					OutColor = FVector4f(DataPtr[0], DataPtr[1], DataPtr[2], DataPtr[3]) / 255.0f;
				}
				else if (GetFormat() == EImageFormat::BGRA_UByte)
				{
					OutColor = FVector4f(DataPtr[2], DataPtr[1], DataPtr[0], DataPtr[3]) / 255.0f;
				}
				else
				{
					check(false);
				}


				break;
			}

			// TODO: Other formats could also be implemented. For compressed types,
			// the compressed block could be compared and only uncompress if all are the same to
			// check if all pixels in the block are also equal.
			default:
				Result = false;
				break;
			}
		}

		return Result;
	}


	bool FImage::IsFullAlpha() const
	{
		const TArrayView<const uint8> DataView = DataStorage.GetLOD(0);
		
		if (DataView.Num())
		{
			switch(GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				const uint8* DataPtr = DataView.GetData() + 3;

				const int32 NumElems = DataView.Num() / 4;
				for (int32 I = 0; I < NumElems; ++I)
				{
					if (*(DataPtr + I*4) != 255) 
					{
						return false;
					}
				}

				return true;
			}

			case EImageFormat::RGB_UByte:
			{
				return true;
			}

			default:
				return false;
			}
		}

		return true;
	}


	namespace
	{
		bool IsZero(const uint8* Buff, SIZE_T Size)
		{
			if (Size == 0)
			{
				return true;
			}
			return (*Buff == 0) && (FMemory::Memcmp(Buff, Buff + 1, Size - 1) == 0);
		}
	}


	void FImage::GetNonBlackRect_Reference(FImageRect& Rect) const
	{
		const FImageSize Size = GetSize();
		Rect.min[0] = Rect.min[1] = 0;
		Rect.size[0] = Size.X;
		Rect.size[1] = Size.Y;

		if (!Rect.size[0] || !Rect.size[1])
		{
			return;
		}

		bool bFirst = true;

		// Slow reference implementation
		uint16 Top = 0;
		uint16 Left = 0;
		uint16 Right = Size.X - 1;
		uint16 Bottom = Size.Y - 1;

		const EImageFormat Format = GetFormat();

		const TArrayView<const uint8> DataView = DataStorage.GetLOD(0);

		const uint8 * const DataPtr = DataView.GetData();

		for (uint16 Y = 0; Y < Size.Y; ++Y)
		{
			for (uint16 X = 0; X < Size.X; ++X)
			{
				bool bIsBlack = false;

				switch (Format)
				{
				case EImageFormat::L_UByte:
				{
					bIsBlack = DataPtr[Y*Size.X + X] == 0;
					break;
				}

				case EImageFormat::RGB_UByte:
				{
					constexpr uint32 ZeroValue = 0;
					bIsBlack = FMemory::Memcmp(&ZeroValue, DataPtr + (Y*Size.X + X)*3, 3) == 0;

					break;
				}

				case EImageFormat::RGBA_UByte:
				case EImageFormat::BGRA_UByte:
				{
					constexpr uint32 ZeroValue = 0;
					bIsBlack = FMemory::Memcmp(&ZeroValue, DataPtr + (Y*Size.X + X)*4, 4) == 0;
					break;
				 }

				 default:
					 check(false);
					 break;
				}

				 if (!bIsBlack)
				 {
					 if (bFirst)
					 {
						 bFirst = false;
						 Left = Right = X;
						 Top = Bottom = Y;
					 }
					 else
					 {
						 Left = FMath::Min(Left, X);
						 Right = FMath::Max(Right, X);
						 Top = FMath::Min(Top, Y);
						 Bottom = FMath::Max(Bottom, Y);
					 }
				 }
			 }
		 }

		 Rect.min[0] = Left;
		 Rect.min[1] = Top;
		 Rect.size[0] = Right - Left + 1;
		 Rect.size[1] = Bottom - Top + 1;
	 }
	

    //---------------------------------------------------------------------------------------------
    void FImage::GetNonBlackRect(FImageRect& rect) const
    {            
		// TODO: There is a bug here with cyborg-windows.
		// Meanwhile do this.
		GetNonBlackRect_Reference(rect);
		return;

  //      rect.min[0] = rect.min[1] = 0;
  //      rect.size = m_size;

		//check(rect.size[0] > 0);
		//check(rect.size[1] > 0);

  //      if ( !rect.size[0] || !rect.size[1] )
  //          return;

  //      uint16 sx = m_size[0];
  //      uint16 sy = m_size[1];

  //      // Somewhat faster implementation
  //      uint16 top = 0;
  //      uint16 left = 0;
  //      uint16 right = sx - 1;
  //      uint16 bottom = sy - 1;

  //      switch ( m_format )
  //      {
  //      case L_UByte:
  //      {
  //          size_t rowStride = sx;

  //          // Find top
  //          const uint8_t* pRow = m_data.GetData();
  //          while ( top < sy )
  //          {
  //              if ( !is_zero( pRow, rowStride ) )
  //                  break;
  //              pRow += rowStride;
  //              ++top;
  //          }

  //          // Find bottom
  //          pRow = m_data.GetData() + rowStride * bottom;
  //          while ( bottom > top )
  //          {
  //              if ( !is_zero( pRow, rowStride ) )
  //                  break;
  //              pRow -= rowStride;
  //              --bottom;
  //          }

  //          // Find left and right
  //          left = sx - 1;
  //          right = 0;
  //          int16_t currentRow = top;
  //          pRow = m_data.GetData() + rowStride * currentRow;
  //          while ( currentRow <= bottom && ( left > 0 || right < ( sx - 1 ) ) )
  //          {
  //              for ( uint16 x = 0; x < left; ++x )
  //              {
  //                  if ( pRow[x] )
  //                  {
  //                      left = x;
  //                      break;
  //                  }
  //              }
  //              for ( uint16 x = sx - 1; x > right; --x )
  //              {
  //                  if ( pRow[x] )
  //                  {
  //                      right = x;
  //                      break;
  //                  }
  //              }
  //              pRow += rowStride;
  //              ++currentRow;
  //          }

  //          break;
  //      }

  //      case RGB_UByte:
  //      case RGBA_UByte:
  //      case BGRA_UByte:
  //      {
  //          size_t bytesPerPixel = GetImageFormatData( m_format ).BytesPerBlock;
  //          size_t rowStride = sx * bytesPerPixel;

  //          // Find top
  //          const uint8_t* pRow = m_data.GetData();
  //          while(top<sy)
  //          {
  //              if ( !is_zero( pRow, rowStride ) )
  //                  break;
  //              pRow += rowStride;
  //              ++top;
  //          }

  //          // Find bottom
  //          pRow = m_data.GetData() + rowStride * bottom;
  //          while ( bottom > top )
  //          {
  //              if ( !is_zero( pRow, rowStride ) )
  //                  break;
  //              pRow -= rowStride;
  //              --bottom;
  //          }

  //          // Find left and right
  //          left = sx - 1;
  //          right = 0;
  //          int16_t currentRow = top;
  //          pRow = m_data.GetData() + rowStride * currentRow;
  //          uint8_t zeroPixel[16] = { 0 };
  //          check(bytesPerPixel<16);
  //          while ( currentRow <= bottom && (left > 0 || right<(sx-1)) )
  //          {
  //              for ( uint16 x = 0; x < left; ++x )
  //              {
  //                  if ( memcmp( pRow + x * bytesPerPixel, zeroPixel, bytesPerPixel ) )
  //                  {
  //                      left = x;
  //                      break;
  //                  }
  //              }
  //              for ( uint16 x = sx - 1; x > right; --x )
  //              {
  //                  if ( memcmp( pRow + x * bytesPerPixel, zeroPixel, bytesPerPixel ) )
  //                  {
  //                      right = x;
  //                      break;
  //                  }
  //              }
  //              pRow += rowStride;
  //              ++currentRow;
  //          }

  //          break;
  //      }

  //      default:
  //          check(false);
  //          break;
  //      }

  //      rect.min[0] = left;
  //      rect.min[1] = top;
  //      rect.size[0] = right - left + 1;
  //      rect.size[1] = bottom - top + 1;

  //      // debug
  //       FImageRect debugRect;
  //       GetNonBlackRect_Reference( debugRect );
  //       if (!(rect==debugRect))
  //       {
  //           UE::Mutable::Private::Halt();
  //       }
    }

	void FImage::ReduceLODsTo(int32 NewLODCount)
	{
		DataStorage.SetNumLODs(NewLODCount);
	}

	void FImage::ReduceLODs(int32 LODsToSkip)
	{
		DataStorage.DropLODs(LODsToSkip);
	}

	void FImageOperator::FillColor(FImage* Target, FVector4f Color)
	{
		const EImageFormat Format = Target->GetFormat();

		switch (Format)
		{
		case EImageFormat::RGB_UByte:
		{
			constexpr int32 BatchSizeInElems = 1 << 14; 
			constexpr int32 ElemSizeInBytes  = 3;
			const int32 NumBatches = Target->DataStorage.GetNumBatches(BatchSizeInElems, ElemSizeInBytes);

			for (int32 I = 0; I < NumBatches; ++I)
			{
				TArrayView<uint8> DataView = Target->DataStorage.GetBatch(I, BatchSizeInElems, ElemSizeInBytes);

				const int32 PixelCount = DataView.Num()/ElemSizeInBytes; 
				uint8* DataPtr = DataView.GetData();
				uint32 R = uint32(FMath::Clamp(255.0f * Color[0], 0.0f, 255.0f));
				uint32 G = uint32(FMath::Clamp(255.0f * Color[1], 0.0f, 255.0f));
				uint32 B = uint32(FMath::Clamp(255.0f * Color[2], 0.0f, 255.0f));
				
				const uint32 PixelData = R | (G << 8) | (B << 16);

				if (PixelData == 0)
				{
					FMemory::Memzero(DataPtr, PixelCount*3);
					continue;
				}

				if ((R == G) & (R == B))
				{
					FMemory::Memset(DataPtr, static_cast<uint8>(R), PixelCount*3);
					continue;
				}

				const uint64 TwoPixelsData = (uint64(PixelData) << 24) | PixelData; 

				for (int32 P = 0; P < PixelCount >> 1; ++P)
				{
					FMemory::Memcpy(&DataPtr[P * 6], &TwoPixelsData, 6);
				}

				if (PixelCount & 1)
				{
					FMemory::Memcpy(&DataPtr[(PixelCount >> 1) * 6], &PixelData, 3);
				}
			}
			break;
		}

		case EImageFormat::RGBA_UByte:
		case EImageFormat::BGRA_UByte:
		{
			constexpr int32 BatchSizeInElems = 1 << 14; 
			constexpr int32 ElemSizeInBytes  = 4;
			const int32 NumBatches = Target->DataStorage.GetNumBatches(BatchSizeInElems, ElemSizeInBytes);

			uint32 R = uint32(FMath::Clamp(255.0f * Color[0], 0.0f, 255.0f));
			uint32 G = uint32(FMath::Clamp(255.0f * Color[1], 0.0f, 255.0f));
			uint32 B = uint32(FMath::Clamp(255.0f * Color[2], 0.0f, 255.0f));
			uint32 A = uint32(FMath::Clamp(255.0f * Color[3], 0.0f, 255.0f));
		
			const uint32 PixelData = Format == EImageFormat::RGBA_UByte
					? R | (G << 8) | (B << 16) | (A << 24)
					: B | (G << 8) | (R << 16) | (A << 24);

			for (int32 I = 0; I < NumBatches; ++I)
			{
				TArrayView<uint8> DataView = Target->DataStorage.GetBatch(I, BatchSizeInElems, ElemSizeInBytes);

				const int32 PixelCount = DataView.Num()/ElemSizeInBytes;
				uint8* DataPtr = DataView.GetData();

				if (PixelData == 0)
				{
					FMemory::Memzero(DataPtr, PixelCount*4);
					continue;
				}

				if ((R == G) & (R == B) & (R == A))
				{
					FMemory::Memset(DataPtr, static_cast<uint8>(R), PixelCount*4);
					continue;
				}

				const uint64 TwoPixelsData = (uint64(PixelData) << 32) | PixelData; 

				for (int32 P = 0; P < (PixelCount >> 1); ++P)
				{
					FMemory::Memcpy(&DataPtr[P * 8], &TwoPixelsData, 8);
				}

				if (PixelCount & 1)
				{
					FMemory::Memcpy(&DataPtr[(PixelCount >> 1) * 8], &PixelData, 4);
				}
			}

			break;
		}

		case EImageFormat::L_UByte:
		{
			constexpr int32 BatchSizeInElems = 1 << 14; 
			constexpr int32 ElemSizeInBytes  = 1;
			const int32 NumBatches = Target->DataStorage.GetNumBatches(BatchSizeInElems, ElemSizeInBytes);

			uint32 R = uint32(FMath::Clamp(255.0f * Color[0], 0.0f, 255.0f));
		
			for (int32 I = 0; I < NumBatches; ++I)
			{
				TArrayView<uint8> DataView = Target->DataStorage.GetBatch(I, BatchSizeInElems, ElemSizeInBytes);

				const int32 PixelCount = DataView.Num()/ElemSizeInBytes;
				uint8* DataPtr = DataView.GetData();

				if (R == 0)
				{
					FMemory::Memzero(DataPtr, PixelCount*1);
					continue;
				}

				FMemory::Memset(DataPtr, static_cast<uint8>(R), PixelCount*1);
			}

			break;
		}

		default:
		{
			// Generic case that supports compressed formats.
			const FImageFormatData& FormatData = GetImageFormatData(Format);
			TManagedPtr<FImage> BlockImage = CreateImage(
						FormatData.PixelsPerBlockX, FormatData.PixelsPerBlockY, 1, EImageFormat::RGBA_UByte, EInitializationType::NotInitialized);

			const uint32 PixelData = 
				  (uint32(FMath::Clamp(255.0f * Color[0], 0.0f, 255.0f)) << 0 ) |
				  (uint32(FMath::Clamp(255.0f * Color[1], 0.0f, 255.0f)) << 8 ) |
				  (uint32(FMath::Clamp(255.0f * Color[2], 0.0f, 255.0f)) << 16) |
				  (uint32(FMath::Clamp(255.0f * Color[3], 0.0f, 255.0f)) << 24);

			uint8 * const UncompressedBlockData = BlockImage->GetLODData(0);
			const int32 UncompressedPixelCount = FormatData.PixelsPerBlockX * FormatData.PixelsPerBlockY;
			for (int32 I = 0; I < UncompressedPixelCount; ++I)
			{
				FMemory::Memcpy(UncompressedBlockData + I*4, &PixelData, 4);
			}

			TManagedPtr<FImage> Converted = ImagePixelFormat(0, BlockImage.Get(), Format);

			const TArrayView<const uint8> BlockView = Converted->DataStorage.GetLOD(0); 

			constexpr int32 BatchSizeInElems = 1 << 14;
			const int32 ElemSizeInBytes = FormatData.BytesPerBlock;
			check(BlockView.Num() == ElemSizeInBytes);
			const int32 NumBatches = Target->DataStorage.GetNumBatches(BatchSizeInElems, ElemSizeInBytes);

			uint8 const * const BlockData = BlockView.GetData();
			for (int32 I = 0; I < NumBatches; ++I)
			{
				TArrayView<uint8> DataView = Target->DataStorage.GetBatch(I, BatchSizeInElems, ElemSizeInBytes);

				check(DataView.Num() % ElemSizeInBytes == 0);
				
				const int32 BatchNumBlocks = DataView.Num() / ElemSizeInBytes;
				uint8* DataPtr = DataView.GetData();


				for (int32 BlockIndex = 0; BlockIndex < BatchNumBlocks; ++BlockIndex)
				{
					FMemory::Memcpy(DataPtr + BlockIndex*ElemSizeInBytes, BlockData, ElemSizeInBytes);
				}
			}

			break;	
		}
		}
	}
}

