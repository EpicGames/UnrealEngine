// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImageComputeUtils.h"

#include "Math/VectorRegister.h"

namespace UE::Mutable::Private
{
namespace ImageComputeUtils
{

	struct alignas(4) FVec4Mask_U8
	{
		uint8 X;
		uint8 Y;
		uint8 Z;
		uint8 W;
	};

#if PLATFORM_ENABLE_VECTORINTRINSICS && !PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	using VectorRegisterInt8x16 = __m128i;

  	FORCEINLINE VectorRegisterInt8x16 MakeShuffle_Int8x16(uint8 V0 = 0, uint8 V1 = 1, uint8 V2 = 2, uint8 V3 = 3, uint8 V4 = 4, uint8 V5 = 5, uint8 V6 = 6, uint8 V7 = 7, uint8 V8 = 8, uint8 V9 = 9, uint8 V10 = 10, uint8 V11 = 11, uint8 V12 = 12, uint8 V13 = 13, uint8 V14 = 14, uint8 V15 = 15)
	{
		return _mm_setr_epi8(V0, V1, V2, V3, V4, V5, V6, V7, V8, V9, V10, V11, V12, V13, V14, V15);
	}

	FORCEINLINE void StoreUnaligned_Int8x16(uint8* Ptr, const VectorRegisterInt8x16 Reg)
	{
		_mm_storeu_si128((__m128i*)Ptr, Reg);
	}

	FORCEINLINE VectorRegisterInt8x16 LoadUnaligned_Int8x16(const uint8* Ptr)
	{
		return _mm_loadu_si128((__m128i*)Ptr);
	}

	FORCEINLINE bool TestZero_Int8x16(VectorRegisterInt8x16 Reg)
	{
		return _mm_testz_si128(Reg, Reg) != 0; 
	}

	FORCEINLINE VectorRegisterInt8x16 Shuffle_Int8x16(const VectorRegisterInt8x16 V, const VectorRegisterInt8x16 Shuffle)
	{
		return _mm_shuffle_epi8(V, Shuffle);
	}

	FORCEINLINE VectorRegisterInt8x16 MakeMask_Int8x16(FVec4Mask_U8 Mask)
	{
		return _mm_setr_epi8(Mask.X, Mask.Y, Mask.Z, Mask.W, Mask.X, Mask.Y, Mask.Z, Mask.W, Mask.X, Mask.Y, Mask.Z, Mask.W, Mask.X, Mask.Y, Mask.Z, Mask.W);	
	}

	FORCEINLINE VectorRegisterInt8x16 Select_Int8x16(const VectorRegisterInt8x16 A, const VectorRegisterInt8x16 B, const VectorRegisterInt8x16 Mask)
	{
		return _mm_blendv_epi8(B, A, Mask);
	}


	FORCEINLINE VectorRegisterInt8x16 Unorm255Lerp_Int8x16(const VectorRegisterInt8x16 A, const VectorRegisterInt8x16 B, const VectorRegisterInt8x16 Factor)
	{
		__m128i OneMinusFactor = _mm_xor_si128(Factor, _mm_set1_epi8((uint8)0xFF)); 
		__m128i OneMinusFactorLo = _mm_cvtepu8_epi16(OneMinusFactor);
		__m128i OneMinusFactorHi = _mm_cvtepu8_epi16(_mm_srli_si128(OneMinusFactor, 8));

		__m128i FactorLo = _mm_cvtepu8_epi16(Factor);
		__m128i FactorHi = _mm_cvtepu8_epi16(_mm_srli_si128(Factor, 8));

		__m128i ALo = _mm_mullo_epi16(_mm_cvtepu8_epi16(A), OneMinusFactorLo);
		__m128i AHi = _mm_mullo_epi16(_mm_cvtepu8_epi16(_mm_srli_si128(A, 8)), OneMinusFactorHi);

		__m128i BLo = _mm_mullo_epi16(_mm_cvtepu8_epi16(B), FactorLo);
		__m128i BHi = _mm_mullo_epi16(_mm_cvtepu8_epi16(_mm_srli_si128(B, 8)), FactorHi);

		__m128i Int128 = _mm_set1_epi16(128);
		__m128i Int257 = _mm_set1_epi16(257);

		__m128i ResultLo = _mm_mulhi_epu16(_mm_add_epi16(_mm_add_epi16(ALo, BLo), Int128), Int257);
		__m128i ResultHi = _mm_mulhi_epu16(_mm_add_epi16(_mm_add_epi16(AHi, BHi), Int128), Int257);

		return _mm_packus_epi16(ResultLo, ResultHi);
	}

#endif

#if PLATFORM_ENABLE_VECTORINTRINSICS && PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	using VectorRegisterInt8x16 = uint8x16_t;

  	FORCEINLINE VectorRegisterInt8x16 MakeShuffle_Int8x16(uint8 V0 = 0, uint8 V1 = 1, uint8 V2 = 2, uint8 V3 = 3, uint8 V4 = 4, uint8 V5 = 5, uint8 V6 = 6, uint8 V7 = 7, uint8 V8 = 8, uint8 V9 = 9, uint8 V10 = 10, uint8 V11 = 11, uint8 V12 = 12, uint8 V13 = 13, uint8 V14 = 14, uint8 V15 = 15)
	{
		const uint8 Data[16] = { V0, V1, V2, V3, V4, V5, V6, V7, V8, V9, V10, V11, V12, V13, V14, V15 };
		return vld1q_u8(Data);
	}

	FORCEINLINE void StoreUnaligned_Int8x16(uint8* Ptr, VectorRegisterInt8x16 Reg)
	{
		vst1q_u8(Ptr, Reg);
	}

	FORCEINLINE VectorRegisterInt8x16 LoadUnaligned_Int8x16(const uint8* Ptr)
	{
		return vld1q_u8(Ptr);
	}

	FORCEINLINE bool TestZero_Int8x16(VectorRegisterInt8x16 Reg)
	{
		uint64x2_t Val = vreinterpretq_u64_u8(Reg);
		return (vgetq_lane_u64(Val, 0) | vgetq_lane_u64(Val, 1)) == 0;
	}

	FORCEINLINE VectorRegisterInt8x16 Shuffle_Int8x16(VectorRegisterInt8x16 V, VectorRegisterInt8x16 Shuffle)
	{
		return vqtbl1q_u8(V, Shuffle);
	}

	FORCEINLINE VectorRegisterInt8x16 MakeMask_Int8x16(FVec4Mask_U8 Mask)
	{
		const uint8 Data[16] = { Mask.X, Mask.Y, Mask.Z, Mask.W, Mask.X, Mask.Y, Mask.Z, Mask.W, Mask.X, Mask.Y, Mask.Z, Mask.W, Mask.X, Mask.Y, Mask.Z, Mask.W };
		return vld1q_u8(Data);
	}

	FORCEINLINE VectorRegisterInt8x16 Select_Int8x16(VectorRegisterInt8x16 A, VectorRegisterInt8x16 B, VectorRegisterInt8x16 Mask)
	{
		return vbslq_u8(Mask, A, B);
	}

	FORCEINLINE VectorRegisterInt8x16 Unorm255Lerp_Int8x16(VectorRegisterInt8x16 A, VectorRegisterInt8x16 B, VectorRegisterInt8x16 Factor)
	{
		uint8x16_t OneMinusFactor = veorq_u8(Factor, vdupq_n_u8(0xFF)); 

		uint16x8_t BTimesFactorLo = vmull_u8(vget_low_u8(B), vget_low_u8(Factor));
		uint16x8_t BTimesFactorHi = vmull_u8(vget_high_u8(B), vget_high_u8(Factor));

		uint16x8_t AddLo = vmlal_u8(BTimesFactorLo, vget_low_u8(A), vget_low_u8(OneMinusFactor));
		uint16x8_t AddHi = vmlal_u8(BTimesFactorHi, vget_high_u8(A), vget_high_u8(OneMinusFactor));

		uint8x8_t ResultLo = vqrshrn_n_u16(vrsraq_n_u16(AddLo, AddLo, 8), 8);
		uint8x8_t ResultHi = vqrshrn_n_u16(vrsraq_n_u16(AddHi, AddHi, 8), 8);

		return vcombine_u8(ResultLo, ResultHi);
	}
#endif


#ifndef UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
	#define UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL (PLATFORM_ENABLE_VECTORINTRINSICS || PLATFORM_ENABLE_VECTORINTRINSICS_NEON)
#endif

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
	
	struct FConvertDescriptor
	{
		uint32 SrcBytesPerIter = 0;
		uint32 DstBytesPerIter = 0;

		VectorRegisterInt8x16 Shuffle;
	};

	// Applies the shuffle described by ConvDesc, it is the responsibility of the calling code to ensure NumIters
	// does not overflow Dest or Src with the values in ConvDesc.
	UE_REWRITE void VecConvertImpl(uint8* Dest, const uint8* Src, int32 NumIters, const FConvertDescriptor ConvDesc)
	{
		for (int32 I = 0; I < NumIters; ++I)
		{
			VectorRegisterInt8x16 SrcRegister = LoadUnaligned_Int8x16(Src);
			VectorRegisterInt8x16 DstRegister = Shuffle_Int8x16(SrcRegister, ConvDesc.Shuffle);
			StoreUnaligned_Int8x16(Dest, DstRegister);

			Src  += ConvDesc.SrcBytesPerIter;
			Dest += ConvDesc.DstBytesPerIter;
		}
	}

#endif

	void Vec4ToVec1_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,
			.DstBytesPerIter = 4,

			.Shuffle = MakeShuffle_Int8x16(0, 4, 8, 12),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 3);
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Src[ElemIndex*4 + 0];
		}
	}

	void Vec4ToVec3_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{	
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,	
			.DstBytesPerIter = 12,

			.Shuffle = MakeShuffle_Int8x16(0, 1, 2, 4, 5, 6, 8, 9, 10, 12, 13, 14),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 1);
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);
		
		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*3 + 0] = Src[ElemIndex*4 + 0];
			Dest[ElemIndex*3 + 1] = Src[ElemIndex*4 + 1];
			Dest[ElemIndex*3 + 2] = Src[ElemIndex*4 + 2];
		}
	}

	void Vec3ToVec1_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 15,	
			.DstBytesPerIter = 5,

			.Shuffle = MakeShuffle_Int8x16(0, 3, 6, 9, 12),
		};

		constexpr int32 ElemsPerIter = 5;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 3);
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Src[ElemIndex*3 + 0];
		}
	}

	void Vec3ToVec4_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 12,	
			.DstBytesPerIter = 16,

			.Shuffle = MakeShuffle_Int8x16(0, 1, 2, 0x80, 3, 4, 5, 0x80, 6, 7, 8, 0x80, 9, 10, 11, 0x80),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 1);
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);
		
		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = Src[ElemIndex*3 + 0];
			Dest[ElemIndex*4 + 1] = Src[ElemIndex*3 + 1];
			Dest[ElemIndex*4 + 2] = Src[ElemIndex*3 + 2];
			Dest[ElemIndex*4 + 3] = 0;
		}
	}
	
	void Vec1ToVec3_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 5,	
			.DstBytesPerIter = 15,

			.Shuffle = MakeShuffle_Int8x16(0, 0, 0, 1, 1, 1, 2, 2, 2, 3, 3, 3, 4, 4, 4),
		};

		constexpr int32 ElemsPerIter = 5;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 3);
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);
		
		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*3 + 0] = Src[ElemIndex];
			Dest[ElemIndex*3 + 1] = Src[ElemIndex];
			Dest[ElemIndex*3 + 2] = Src[ElemIndex];
		}
	}

	void Vec1ToVec4_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 4,	
			.DstBytesPerIter = 16,

			.Shuffle = MakeShuffle_Int8x16(0, 0, 0, 0, 1, 1, 1, 1, 2, 2, 2, 2, 3, 3, 3, 3),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 3);
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = Src[ElemIndex];
			Dest[ElemIndex*4 + 1] = Src[ElemIndex];
			Dest[ElemIndex*4 + 2] = Src[ElemIndex];
			Dest[ElemIndex*4 + 3] = Src[ElemIndex];
		}
	}

	void Vec4SwizzleZYXW_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,	
			.DstBytesPerIter = 16,

			.Shuffle = MakeShuffle_Int8x16(2, 1, 0, 3, 6, 5, 4, 7, 10, 9, 8, 11, 14, 13, 12, 15),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = NumElems/ElemsPerIter;
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);
		
		ElemIndex = NumIters*ElemsPerIter; 
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = Src[ElemIndex*4 + 2]; 
			Dest[ElemIndex*4 + 1] = Src[ElemIndex*4 + 1]; 
			Dest[ElemIndex*4 + 2] = Src[ElemIndex*4 + 0]; 
			Dest[ElemIndex*4 + 3] = Src[ElemIndex*4 + 3]; 
		}
	}

	void Vec4SwizzleWWWWAndConvertToVec1_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,	
			.DstBytesPerIter = 4,

			.Shuffle = MakeShuffle_Int8x16(3, 7, 11, 15),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 3); 
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Src[ElemIndex*4 + 3];
		}
	}

	void Vec4SwizzleWWWWAndConvertToVec3_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,	
			.DstBytesPerIter = 12,

			.Shuffle = MakeShuffle_Int8x16(3,3,3, 7,7,7, 11,11,11, 15,15,15),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = FMath::Max<int32>(0, NumElems/ElemsPerIter - 1); 
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*3 + 0] = Src[ElemIndex*4 + 3];
			Dest[ElemIndex*3 + 1] = Src[ElemIndex*4 + 3];
			Dest[ElemIndex*3 + 2] = Src[ElemIndex*4 + 3];
		}
	}


	void Vec4SwizzleXXXX_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,	
			.DstBytesPerIter = 16,

			.Shuffle = MakeShuffle_Int8x16(0,0,0,0, 4,4,4,4, 8,8,8,8, 12,12,12,12),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = NumElems/ElemsPerIter; 
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = Src[ElemIndex*4 + 0];
			Dest[ElemIndex*4 + 1] = Src[ElemIndex*4 + 0];
			Dest[ElemIndex*4 + 2] = Src[ElemIndex*4 + 0];
			Dest[ElemIndex*4 + 3] = Src[ElemIndex*4 + 0];
		}
	}

	void Vec4SwizzleWWWW_U8(uint8* Dest, const uint8* Src, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		FConvertDescriptor ConvDesc
		{
			.SrcBytesPerIter = 16,	
			.DstBytesPerIter = 16,

			.Shuffle = MakeShuffle_Int8x16(3,3,3,3, 7,7,7,7, 11,11,11,11, 15,15,15,15),
		};

		constexpr int32 ElemsPerIter = 4;

		int32 NumIters = NumElems/ElemsPerIter; 
		VecConvertImpl(Dest, Src, NumIters, ConvDesc);

		ElemIndex = NumIters*ElemsPerIter;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = Src[ElemIndex*4 + 3];
			Dest[ElemIndex*4 + 1] = Src[ElemIndex*4 + 3];
			Dest[ElemIndex*4 + 2] = Src[ElemIndex*4 + 3];
			Dest[ElemIndex*4 + 3] = Src[ElemIndex*4 + 3];
		}
	}

	void Vec4SelectMask_U8(uint8* Dest, const uint8* A, const uint8* B, FVec4Mask_U8 Mask, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		VectorRegisterInt8x16 VecMask = MakeMask_Int8x16(Mask);
		
		constexpr int32 ElemsPerIter = 4;
		int32 NumIters = NumElems / ElemsPerIter;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterInt8x16 VecA = LoadUnaligned_Int8x16(A + Iter*16);	
			VectorRegisterInt8x16 VecB = LoadUnaligned_Int8x16(B + Iter*16);	
			VectorRegisterInt8x16 Result = Select_Int8x16(VecA, VecB, VecMask); 

			StoreUnaligned_Int8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*ElemsPerIter; 
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = (A[ElemIndex*4 + 0] & Mask.X) | (B[ElemIndex*4 + 0] & ~Mask.X);
			Dest[ElemIndex*4 + 1] = (A[ElemIndex*4 + 1] & Mask.Y) | (B[ElemIndex*4 + 1] & ~Mask.Y);
			Dest[ElemIndex*4 + 2] = (A[ElemIndex*4 + 2] & Mask.Z) | (B[ElemIndex*4 + 2] & ~Mask.Z);
			Dest[ElemIndex*4 + 3] = (A[ElemIndex*4 + 3] & Mask.W) | (B[ElemIndex*4 + 3] & ~Mask.W);
		}
	}

	void Vec4SelectX_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems)
	{
		constexpr FVec4Mask_U8 Mask = FVec4Mask_U8{0x00, 0xFF, 0xFF, 0xFF};
		Vec4SelectMask_U8(Dest, A, B, Mask, NumElems);
	}

	void Vec4SelectY_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems)
	{
		constexpr FVec4Mask_U8 Mask = FVec4Mask_U8{0xFF, 0x00, 0xFF, 0xFF};	
		Vec4SelectMask_U8(Dest, A, B, Mask, NumElems);
	}

	void Vec4SelectZ_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems)
	{
		constexpr FVec4Mask_U8 Mask = FVec4Mask_U8{0xFF, 0xFF, 0x00, 0xFF};	
		Vec4SelectMask_U8(Dest, A, B, Mask, NumElems);
	}

	void Vec4SelectW_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems)
	{
		constexpr FVec4Mask_U8 Mask = FVec4Mask_U8{0xFF, 0xFF, 0xFF, 0x00};	
		Vec4SelectMask_U8(Dest, A, B, Mask, NumElems);
	}

	void Vec4Fill_U8(uint8* Dest, FColor Color, int32 NumElems)
	{
		uint32 ColorValue = Color.ToPackedABGR();

		for (int32 I = 0; I < NumElems; ++I)
		{
			FMemory::Memcpy(Dest + I*4, &ColorValue, sizeof(uint32));
		}
	}

	void Vec3Fill_U8(uint8* Dest, FColor Color, int32 NumElems)
	{
		for (int32 ElemIndex = 0; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*3 + 0] = Color.R;
			Dest[ElemIndex*3 + 1] = Color.G;
			Dest[ElemIndex*3 + 2] = Color.B;
		}
	}
	
	void Vec1Fill_U8(uint8* Dest, FColor Color, int32 NumElems)
	{
		FMemory::Memset(Dest, Color.R, NumElems);
	}

	void Vec4Fill_U16(uint16* Dest, Math::TIntVector4<uint16> Value, int32 NumElems)
	{
		for (int32 ElemIndex = 0; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*4 + 0] = Value.X;
			Dest[ElemIndex*4 + 1] = Value.Y;
			Dest[ElemIndex*4 + 2] = Value.Z;
			Dest[ElemIndex*4 + 3] = Value.W;
		}
	}

	void Vec3Fill_U16(uint16* Dest, Math::TIntVector4<uint16> Value, int32 NumElems)
	{
		for (int32 ElemIndex = 0; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*3 + 0] = Value.X;
			Dest[ElemIndex*3 + 1] = Value.Y;
			Dest[ElemIndex*3 + 2] = Value.Z;
		}
	}

	void Vec1Fill_U16(uint16* Dest, Math::TIntVector4<uint16> Value, int32 NumElems)
	{
		for (int32 ElemIndex = 0; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex*1 + 0] = Value.X;
		}
	}

	void Lerp_U8(uint8* Dest, const uint8* A, const uint8* B, const uint8* Factor, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		constexpr int32 ElemsPerIter = 16;
		int32 NumIters = NumElems / ElemsPerIter;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterInt8x16 VecA = LoadUnaligned_Int8x16(A + Iter*ElemsPerIter);	
			VectorRegisterInt8x16 VecB = LoadUnaligned_Int8x16(B + Iter*ElemsPerIter);	
			VectorRegisterInt8x16 VecFactor = LoadUnaligned_Int8x16(Factor + Iter*ElemsPerIter);	
	
			VectorRegisterInt8x16 Result = Unorm255Lerp_Int8x16(VecA, VecB, VecFactor);

			StoreUnaligned_Int8x16(Dest + Iter*ElemsPerIter, Result);
		}

		ElemIndex = NumIters*ElemsPerIter; 
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			uint8 FactorValue = Factor[ElemIndex];
			uint16 Value = (255 - FactorValue)*A[ElemIndex] + FactorValue*B[ElemIndex] + 128;

			Dest[ElemIndex] = static_cast<uint8>((Value * 257) >> 16);
		}
	}


	bool TestZero_U8(const uint8* Data, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_COMPUTE_UTILS_USE_VECTOR_IMPL
		constexpr int32 ElemsPerIter = 16;
		int32 NumIters = NumElems / ElemsPerIter;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{	
			if (!TestZero_Int8x16(LoadUnaligned_Int8x16(Data + Iter*ElemsPerIter)))
			{
				return false;
			}
		}

		ElemIndex = NumIters*ElemsPerIter; 
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			if (Data[ElemIndex] != 0)
			{
				return false;
			}
		}

		return true;
	}
} // namespace ImageComputeUtils
} // namespace UE::Mutable::Private
