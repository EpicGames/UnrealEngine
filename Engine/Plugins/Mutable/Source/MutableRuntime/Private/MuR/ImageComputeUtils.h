// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Math/IntVector.h"

namespace UE::Mutable::Private
{
namespace ImageComputeUtils
{
	void Lerp_U8(uint8* Dest, const uint8* A, const uint8* B, const uint8* Factor, int32 NumElems);

	void Vec4ToVec1_U8(uint8* Dest, const uint8* Src, int32 NumElems);	
	void Vec4ToVec3_U8(uint8* Dest, const uint8* Src, int32 NumElems);	
	void Vec3ToVec1_U8(uint8* Dest, const uint8* Src, int32 NumElems);	
	void Vec3ToVec4_U8(uint8* Dest, const uint8* Src, int32 NumElems);
	void Vec1ToVec3_U8(uint8* Dest, const uint8* Src, int32 NumElems);
	void Vec1ToVec4_U8(uint8* Dest, const uint8* Src, int32 NumElems);

	void Vec4SelectX_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems);
	void Vec4SelectY_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems);
	void Vec4SelectZ_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems);
	void Vec4SelectW_U8(uint8* Dest, const uint8* A, const uint8* B, int32 NumElems);
	
	void Vec4SwizzleWWWWAndConvertToVec1_U8(uint8* Dest, const uint8* Src, int32 NumElems);
	void Vec4SwizzleWWWWAndConvertToVec3_U8(uint8* Dest, const uint8* Src, int32 NumElems);
	void Vec4SwizzleXXXX_U8(uint8* Dest, const uint8* Src, int32 NumElems);
	void Vec4SwizzleWWWW_U8(uint8* Dest, const uint8* Src, int32 NumElems);
	void Vec4SwizzleZYXW_U8(uint8* Dest, const uint8* Src, int32 NumElems);

	void Vec4Fill_U8(uint8* Dest, FColor Value, int32 NumElems);
	void Vec3Fill_U8(uint8* Dest, FColor Value, int32 NumElems);	
	void Vec1Fill_U8(uint8* Dest, FColor Value, int32 NumElems);

	void Vec4Fill_U16(uint16* Dest, Math::TIntVector4<uint16> Value, int32 NumElems);
	void Vec3Fill_U16(uint16* Dest, Math::TIntVector4<uint16> Value, int32 NumElems);	
	void Vec1Fill_U16(uint16* Dest, Math::TIntVector4<uint16> Value, int32 NumElems);

	bool TestZero_U8(const uint8* Data, int32 NumElems);
} // namespace ImageComputeUtils
} // namespace UE::Mutable::Private
