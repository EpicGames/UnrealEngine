// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImageDataStorage.h"
#include "MuR/Serialisation.h"

namespace UE::Mutable::Private::FImageDataStorageInternal
{
	FORCEINLINE FImageSize ComputeLODSize(FImageSize BaseSize, int32 LOD)
	{
		for (int32 L = 0; L < LOD; ++L)
		{
			BaseSize = FImageSize(
					FMath::DivideAndRoundUp<uint16>(BaseSize.X, 2),
					FMath::DivideAndRoundUp<uint16>(BaseSize.Y, 2));
		}

		return BaseSize;
	}

	FORCEINLINE void InitViewToBlack(const TArrayView<uint8>& View, EImageFormat Format)
	{
		const FImageFormatData FormatData = GetImageFormatData(Format);
		constexpr uint8 ZeroBuffer[FImageFormatData::MAX_BYTES_PER_BLOCK] = {0};		

		const bool bIsFormatBlackBlockZeroed = 
				FMemory::Memcmp(ZeroBuffer, FormatData.BlackBlock, FImageFormatData::MAX_BYTES_PER_BLOCK) == 0;
		if (bIsFormatBlackBlockZeroed)
		{
			FMemory::Memzero(View.GetData(), View.Num());
		}
		else
		{
			const int32 FormatBlockSize = FormatData.BytesPerBlock;
			
			if (FormatData.BytesPerBlock == 0)
			{
				return;
			}

			check(FormatData.BytesPerBlock != 0);
			check(FormatBlockSize <= FImageFormatData::MAX_BYTES_PER_BLOCK);
			check(View.Num() % FormatBlockSize == 0);

			uint8* const BufferBegin = View.GetData();
			const int32 DataSize = View.Num();

			for (int32 BlockDataOffset = 0; BlockDataOffset < DataSize; BlockDataOffset += FormatBlockSize)
			{
				FMemory::Memcpy(BufferBegin + BlockDataOffset, FormatData.BlackBlock, FormatBlockSize);
			}
		}
	}
}

namespace UE::Mutable::Private::MemoryCounters
{
	std::atomic<SSIZE_T>& FImageMemoryCounter::Get()
	{
		static std::atomic<SSIZE_T> Counter{0};
		return Counter;
	}
}

namespace UE::Mutable::Private
{
	FImageDataStorage::FImageDataStorage()
	{
	}

	FImageDataStorage::FImageDataStorage(const FImageDesc& Desc)
	{
		Init(Desc, EInitializationType::NotInitialized);
	}

	FImageDataStorage::FImageDataStorage(const FImageDataStorage& Other)
	{
		CopyFrom(Other);
	}

	FImageDataStorage& FImageDataStorage::operator=(const FImageDataStorage& Other)
	{
		CopyFrom(Other);
		return *this;
	}

	bool FImageDataStorage::operator==(const FImageDataStorage& Other) const
	{
		const bool bSameMetadata =
				ImageSize == Other.ImageSize &&
				ImageFormat == Other.ImageFormat &&
				NumLODs == Other.NumLODs;

		if (!bSameMetadata)
		{
			return false;
		}

		if (Buffers.Num() != Other.Buffers.Num())
		{
			return false;
		}

		// Buffers are sorted from large to small, but mips usually have information about the whole image.
		// Process the small ones first.
		for (int32 BufferIndex = Buffers.Num() - 1; BufferIndex >= 0; --BufferIndex)
		{
			if (Buffers[BufferIndex] != Other.Buffers[BufferIndex])
			{
				return false;
			}
		}

		return true;
	}

	void FImageDataStorage::InitInternalArray(int32 Index, int32 Size, EInitializationType InitType)
	{
		using namespace FImageDataStorageInternal;

		check(Index < Buffers.Num());

		Buffers[Index].SetNumUninitialized(Size);

		if (InitType == EInitializationType::Black)
		{
			const TArrayView<uint8> BufferView = MakeArrayView(Buffers[Index].GetData(), Buffers[Index].Num());
			InitViewToBlack(BufferView, ImageFormat);
		}

		check(InitType == EInitializationType::Black || InitType == EInitializationType::NotInitialized);
	}

	void FImageDataStorage::Init(const FImageDesc& ImageDesc, EInitializationType InitType)
	{
		using namespace FImageDataStorageInternal;

		if (ImageFormat != ImageDesc.m_format || ImageSize != ImageDesc.m_size)
		{
			NumLODs = 0;
			Buffers.Empty();
		}

		ImageSize = ImageDesc.m_size;
		ImageFormat = ImageDesc.m_format;

		SetNumLODs(ImageDesc.m_lods, EInitializationType::NotInitialized);

		if (InitType == EInitializationType::Black)
		{
			for (FImageArray& Buffer : Buffers)
			{
				InitViewToBlack(MakeArrayView(Buffer.GetData(), Buffer.Num()), ImageFormat);
			}
		}
	}

	void FImageDataStorage::CopyFrom(const FImageDataStorage& Other)
	{
		// Copy images that have different format is not allowed.
		check(ImageFormat == EImageFormat::None || Other.ImageFormat == ImageFormat);

		ImageSize = Other.ImageSize;
		ImageFormat = Other.ImageFormat;
		NumLODs = Other.NumLODs;

		Buffers = Other.Buffers;
		CompactedTailOffsets = Other.CompactedTailOffsets;
	}

	void FImageDataStorage::SetNumLODs(int32 InNumLODs, EInitializationType InitType)
	{
		using namespace FImageDataStorageInternal;

		check(!IsVoid());

		if (InNumLODs == NumLODs)
		{
			return;
		}

		const int32 FirstCompactedTailLOD = ComputeFirstCompactedTailLOD();
		const int32 ValidNumLODsToSet = FMath::Min(InNumLODs, ComputeNumLODsForSize(ImageSize)); 

		const int32 NumOldLODsInTail = FMath::Max(0, NumLODs - FirstCompactedTailLOD);
		const int32 NumOldLODsNotInTail = FMath::Max(0, NumLODs - NumOldLODsInTail);

		const int32 NumNewLODsInTail = FMath::Max(0, ValidNumLODsToSet - FirstCompactedTailLOD);
		const int32 NumNewLODsNotInTail = FMath::Max(0, ValidNumLODsToSet - NumNewLODsInTail);

		Buffers.SetNum(NumNewLODsNotInTail + static_cast<int32>(NumNewLODsInTail > 0));

		const int32 BlockSizeX = GetImageFormatData(ImageFormat).PixelsPerBlockX;
		const int32 BlockSizeY = GetImageFormatData(ImageFormat).PixelsPerBlockY;
		const int32 BytesPerBlock = GetImageFormatData(ImageFormat).BytesPerBlock;

		// Do not allocate memory for formats like RLE that are not block based.
		if (BlockSizeX == 0 || BlockSizeY == 0 || BytesPerBlock == 0)
		{
			NumLODs = ValidNumLODsToSet;
			return;
		}

		FImageSize LODDims = ComputeLODSize(ImageSize, NumOldLODsNotInTail);
		// Update non tail buffers.
		{		
			const int32 NumAddedLODsNotInTail = FMath::Max(0, NumNewLODsNotInTail - NumOldLODsNotInTail);
			
			for (int32 L = 0; L < NumAddedLODsNotInTail; ++L)
			{
				const int32 BlocksX = FMath::DivideAndRoundUp<int32>(LODDims.X, BlockSizeX);
				const int32 BlocksY = FMath::DivideAndRoundUp<int32>(LODDims.Y, BlockSizeY);

				const int32 DataSize = BlocksX * BlocksY * BytesPerBlock;
				InitInternalArray(L + NumOldLODsNotInTail, DataSize, InitType);

				LODDims = FImageSize(
						FMath::DivideAndRoundUp<uint16>(LODDims.X, 2),
						FMath::DivideAndRoundUp<uint16>(LODDims.Y, 2));
			}
		}
		// Update tail buffer.
		if (NumNewLODsInTail > 0)
		{	
			const int32 NumAddedLODsInTail = FMath::Max(0, NumNewLODsInTail - NumOldLODsInTail);
			
			int32 TailBufferSize = 0;
			for (int32 L = 0; L < NumNewLODsInTail; ++L)
			{
				const int32 BlocksX = FMath::DivideAndRoundUp<int32>(LODDims.X, BlockSizeX);
				const int32 BlocksY = FMath::DivideAndRoundUp<int32>(LODDims.Y, BlockSizeY);

				TailBufferSize += BlocksX * BlocksY * BytesPerBlock;
				
				CompactedTailOffsets[L] = TailBufferSize;

				LODDims = FImageSize(
						FMath::DivideAndRoundUp<uint16>(LODDims.X, 2),
						FMath::DivideAndRoundUp<uint16>(LODDims.Y, 2));
			}

			FImageArray& TailBuffer = Buffers.Last();
			TailBuffer.SetNumUninitialized(TailBufferSize);

			if (NumAddedLODsInTail > 0 && InitType == EInitializationType::Black)
			{
				const int32 FirstAddedLODInTailOffset = NumOldLODsInTail == 0 ? 0 : CompactedTailOffsets[NumOldLODsInTail - 1];
				InitViewToBlack(MakeArrayView(TailBuffer.GetData() + FirstAddedLODInTailOffset, TailBuffer.Num()), ImageFormat);
			}

			// Fix the unused tail offsets, this is important to make sure serialized data is deterministic.
			const int32 UnusedTailOffsetValue = CompactedTailOffsets[NumNewLODsInTail - 1];  
			for (int32 I = NumNewLODsInTail; I < NumLODsInCompactedTail; ++I)
			{
				CompactedTailOffsets[I] = UnusedTailOffsetValue;
			}
		}

		NumLODs = ValidNumLODsToSet;
	}

	void FImageDataStorage::DropLODs(int32 NumLODsToDrop)
	{
		check(!IsVoid());
		
		if (NumLODsToDrop >= NumLODs)
		{
			//check(false);
			Buffers.Empty();
			ImageSize = FImageSize(0, 0);
			NumLODs = 0;

			return;
		}

		const int32 FirstCompactedTailLOD = ComputeFirstCompactedTailLOD();
		const int32 NumLODsToDropInTail = FMath::Max(0, NumLODsToDrop - FirstCompactedTailLOD); 

		for (int32 DestBufferIndex = 0, BufferIndex = NumLODsToDrop - NumLODsToDropInTail; 
			 BufferIndex < FirstCompactedTailLOD; 
			 ++DestBufferIndex, ++BufferIndex)
		{
			Buffers[DestBufferIndex] = MoveTemp(Buffers[BufferIndex]);
		}
		
		if (NumLODsToDropInTail)
		{
			int32 TailBufferOffset = NumLODsToDropInTail == 0 ? 0 : CompactedTailOffsets[NumLODsToDropInTail - 1];

			FImageArray& TailBuffer = Buffers.Last();

			int32 NewBufferSize = TailBuffer.Num() - TailBufferOffset; 
			FMemory::Memmove(TailBuffer.GetData(), TailBuffer.GetData() + TailBufferOffset, NewBufferSize);
			TailBuffer.SetNum(NewBufferSize);

			int32 NumLODsInTail = FMath::Max(0, NumLODs - FirstCompactedTailLOD);
			int32 OffsetIndex = NumLODsToDropInTail;

			for (int32 DestOffsetsIndex = 0; OffsetIndex < NumLODsInTail; ++DestOffsetsIndex, ++OffsetIndex)
			{
				CompactedTailOffsets[DestOffsetsIndex] = CompactedTailOffsets[OffsetIndex]; 
			}

			// Fix the unused tail offsets, this is important to make sure serialized data is deterministic.
			const uint32 UnusedTailValue = NumLODsInTail - 1 <= 0 ? 0 : CompactedTailOffsets[NumLODsInTail - 1];
			for (; OffsetIndex < NumLODsInCompactedTail; ++OffsetIndex)
			{
				CompactedTailOffsets[OffsetIndex] = UnusedTailValue;
			}
		}

		// Update metadata.
		NumLODs = FMath::Max(0, NumLODs - NumLODsToDrop);
		
		for (int32 I = 0; I < NumLODsToDrop; ++I)
		{
			ImageSize = FImageSize(
					FMath::DivideAndRoundUp<uint16>(ImageSize.X, 2),
					FMath::DivideAndRoundUp<uint16>(ImageSize.Y, 2));
		}
	}

	void FImageDataStorage::ResizeLOD(int32 LODIndex, int32 NewSizeBytes, EAllowShrinking AllowShrinking)
	{
		check(!IsVoid());
		check(NewSizeBytes >= 0);

		const int32 FirstCompactedTailLOD = ComputeFirstCompactedTailLOD();
		FImageArray& Buffer = GetInternalArray(LODIndex);

		if (LODIndex < FirstCompactedTailLOD)
		{
			Buffer.SetNum(NewSizeBytes, AllowShrinking);
			return;
		}

		check(&Buffer == &Buffers.Last());

		// Compacted tail resize.
		const int32 LODSize = GetLOD(LODIndex).Num();
		const int32 OldBufferSize = Buffer.Num();

		const int32 SizeDifference = NewSizeBytes - LODSize;
		const int32 NewBufferSize = OldBufferSize + SizeDifference;

		// If the buffer is growing we need to resize before moving the content. 
		if (SizeDifference > 0)
		{
			Buffer.SetNum(NewBufferSize, AllowShrinking);
		}

		// Move content and update tail offsets.
		const int32 LODIndexInTail = LODIndex - FirstCompactedTailLOD;
		
		// Last LOD does not need to move the content.
		if (LODIndex < NumLODs - 1) 
		{
			const int32 LODEndOffset = CompactedTailOffsets[LODIndexInTail];

			FMemory::Memmove(
					Buffer.GetData() + LODEndOffset + SizeDifference, 
					Buffer.GetData() + LODEndOffset,
					OldBufferSize - LODEndOffset);
		}

		// If the buffer is shrinking we need to resize after moving the content. 
		if (SizeDifference < 0)
		{
			Buffer.SetNum(NewBufferSize, AllowShrinking);
		}

		// Tail buffer offsets are represented with uint16 for compactness, check any overflow.
		check(Buffer.Num() < TNumericLimits<uint16>::Max());

		// Keep offset updated even if there is no LOD so that serialization can be deterministic.
		for (int32 I = LODIndexInTail; I < NumLODsInCompactedTail; ++I)
		{
			check(int32(CompactedTailOffsets[I]) + SizeDifference >= 0); 
			CompactedTailOffsets[I] += SizeDifference; 	
		}
	}

	TArrayView<const uint8> FImageDataStorage::GetLOD(int32 LODIndex) const
	{
		check(!IsVoid());
		if (LODIndex >= NumLODs)
		{
			//check(false);
			return TArrayView<const uint8>();
		}

		const int32 FirstTailLOD = ComputeFirstCompactedTailLOD();
		if (LODIndex >= FirstTailLOD)
		{
			const int32 LODIndexInTail = FMath::Max(0, LODIndex - FirstTailLOD);
			
			const int32 LODInTailBegin = LODIndexInTail == 0 ? 0 : CompactedTailOffsets[LODIndexInTail - 1];
			const int32 LODInTailEnd = CompactedTailOffsets[LODIndexInTail];

			const FImageArray& TailBuffer = Buffers.Last();
			return MakeArrayView(TailBuffer.GetData() + LODInTailBegin, LODInTailEnd - LODInTailBegin);
		}

		return MakeArrayView(Buffers[LODIndex].GetData(), Buffers[LODIndex].Num());
	}

	TArrayView<uint8> FImageDataStorage::GetLOD(int32 LODIndex)
	{	
		const TArrayView<const uint8> ConstLODView = const_cast<const FImageDataStorage*>(this)->GetLOD(LODIndex);
		return TArrayView<uint8>(const_cast<uint8*>(ConstLODView.GetData()), ConstLODView.Num());
	}

	int32 FImageDataStorage::GetNumBatchesLODRange(int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 LODBegin, int32 LODEnd) const
	{
		check(!IsVoid());
		check(LODBegin < LODEnd);
		check(LODBegin >= 0 && LODEnd <= NumLODs);

		const int32 BatchNumBytes = BatchSizeInElems*BatchElemSizeInBytes;
		
		const int32 FirstCompactedTailLOD = ComputeFirstCompactedTailLOD();
		const int32 NumLODsInTail = FMath::Max(0, NumLODs - FirstCompactedTailLOD);
		const int32 NumLODsNotInTail = FMath::Max(0, NumLODs - NumLODsInTail);

		const int32 LODsNotInTailLODRangeEnd = FMath::Min(LODEnd, NumLODsNotInTail);

		int32 NumBatches = 0;
		for (int32 BufferIndex = LODBegin; BufferIndex < LODsNotInTailLODRangeEnd; ++BufferIndex)
		{	
			NumBatches += FMath::DivideAndRoundUp(Buffers[BufferIndex].Num(), BatchNumBytes);
		}

		if (FirstCompactedTailLOD < LODEnd)
		{
			TArrayView<const uint8> FirstLODInRangeView = GetLOD(FMath::Max(FirstCompactedTailLOD, LODBegin));
			TArrayView<const uint8> LastLODInRangeView = GetLOD(LODEnd - 1);

			const int32 TailInLODRangeNumBytes = LastLODInRangeView.GetData() - FirstLODInRangeView.GetData() + LastLODInRangeView.Num();
			NumBatches += FMath::DivideAndRoundUp(TailInLODRangeNumBytes, BatchNumBytes);
		}
		return NumBatches;
	}

	int32 FImageDataStorage::GetNumBatches(int32 BatchSizeInElems, int32 BatchElemSizeInBytes) const
	{	
		return GetNumBatchesLODRange(BatchSizeInElems, BatchElemSizeInBytes, 0, NumLODs);
	}

	TArrayView<const uint8> FImageDataStorage::GetBatch(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes) const
	{
		return GetBatchLODRange(BatchId, BatchSizeInElems, BatchElemSizeInBytes, 0, NumLODs);
	}

	TArrayView<uint8> FImageDataStorage::GetBatch(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes)
	{
		// Use the const_cast idiom to generate the non-const operation. 
		const TArrayView<const uint8> ConstBatchView = 
				const_cast<const FImageDataStorage*>(this)->GetBatch(BatchId, BatchSizeInElems, BatchElemSizeInBytes);
		return TArrayView<uint8>(const_cast<uint8*>(ConstBatchView.GetData()), ConstBatchView.Num());
	}

	int32 FImageDataStorage::GetNumBatchesFirstLODOffset(int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 OffsetInBytes) const
	{
		check(!IsVoid());
		TArrayView<const uint8> FirstLODView = GetLOD(0);
		
		check(OffsetInBytes % BatchElemSizeInBytes == 0);
		check(FirstLODView.Num() % BatchElemSizeInBytes == 0);
		check(FirstLODView.Num() > OffsetInBytes); 

		return FMath::DivideAndRoundUp(FirstLODView.Num() - OffsetInBytes, BatchSizeInElems*BatchElemSizeInBytes);
	}

	TArrayView<const uint8> FImageDataStorage::GetBatchFirstLODOffset(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 OffsetInBytes) const
	{
		check(!IsVoid());
		TArrayView<const uint8> FirstLODView = GetLOD(0);
		
		check(OffsetInBytes < FirstLODView.Num());
		check(FirstLODView.Num() % BatchElemSizeInBytes == 0);
		check(OffsetInBytes % BatchElemSizeInBytes == 0);
	
		TArrayView<const uint8> OffsetedLODView = MakeArrayView(FirstLODView.GetData() + OffsetInBytes, FirstLODView.Num() - OffsetInBytes);

		const int32 BatchSizeInBytes = BatchSizeInElems * BatchElemSizeInBytes;
		const int32 BatchOffset = BatchId * BatchSizeInBytes;

		if (BatchOffset >= OffsetedLODView.Num())
		{
			return TArrayView<const uint8>();
		}

		return MakeArrayView(OffsetedLODView.GetData() + BatchOffset, FMath::Min(BatchSizeInBytes, OffsetedLODView.Num() - BatchOffset));
	}

	TArrayView<uint8> FImageDataStorage::GetBatchFirstLODOffset(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 OffsetInBytes)
	{
		const TArrayView<const uint8> ConstBatchView = 
				const_cast<const FImageDataStorage*>(this)->GetBatchFirstLODOffset(BatchId, BatchSizeInElems, BatchElemSizeInBytes, OffsetInBytes);

		return TArrayView<uint8>(const_cast<uint8*>(ConstBatchView.GetData()), ConstBatchView.Num());
	}

	TArrayView<const uint8> FImageDataStorage::GetBatchLODRange(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 LODBegin, int32 LODEnd) const
	{
		check(!IsVoid());
		check(LODBegin < LODEnd);
		check(LODBegin >= 0 && LODEnd <= NumLODs);

		const int32 BatchNumBytes = BatchSizeInElems*BatchElemSizeInBytes;

		const int32 FirstCompactedTailLOD = ComputeFirstCompactedTailLOD();
		const int32 NonTailBuffersEnd = FMath::Min(LODEnd, FirstCompactedTailLOD);

		int32 NumBatches = 0;	
		for (int32 BufferIndex = LODBegin; BufferIndex < NonTailBuffersEnd; ++BufferIndex)
		{
			const TArrayView<const uint8> BufferView = MakeArrayView(Buffers[BufferIndex].GetData(), Buffers[BufferIndex].Num());
			const int32 BufferNumBatches = FMath::DivideAndRoundUp(BufferView.Num(), BatchNumBytes); 
			
			check(BatchId >= NumBatches);
			int32 BufferBatchId = BatchId - NumBatches;

			if (BufferBatchId < BufferNumBatches)
			{ 
				const int32 BufferBatchBeginOffset = BufferBatchId * BatchNumBytes;

				return TArrayView<const uint8>(
						BufferView.GetData() + BufferBatchBeginOffset,
						FMath::Min(BufferView.Num() - BufferBatchBeginOffset, BatchNumBytes));
			}

			NumBatches += BufferNumBatches;
		}

		if (FirstCompactedTailLOD < LODEnd)
		{
			TArrayView<const uint8> FirstLODInRangeView = GetLOD(FMath::Max(FirstCompactedTailLOD, LODBegin));
			TArrayView<const uint8> LastLODInRangeView = GetLOD(LODEnd - 1);
			
			const int32 TailInLODRangeNumBytes = LastLODInRangeView.GetData() - FirstLODInRangeView.GetData() + LastLODInRangeView.Num();
			TArrayView<const uint8> TailInLODRangeView = MakeArrayView(FirstLODInRangeView.GetData(), TailInLODRangeNumBytes);

			check(BatchId >= NumBatches);
			const int32 TailInLODRangeBatchId = BatchId - NumBatches;
			const int32 TailInLODRangeNumBatches = FMath::DivideAndRoundUp(TailInLODRangeView.Num(), BatchNumBytes);

			if (TailInLODRangeBatchId < TailInLODRangeNumBatches)
			{
				const int32 BufferBatchBeginOffset = TailInLODRangeBatchId * BatchNumBytes;
				
				return TArrayView<const uint8>(
							TailInLODRangeView.GetData() + BufferBatchBeginOffset,
							FMath::Min(TailInLODRangeView.Num() - BufferBatchBeginOffset, BatchNumBytes));
			}
		}
		// If the BatchId is not found, return an empty view.
		return TArrayView<const uint8>();
	}

	TArrayView<uint8> FImageDataStorage::GetBatchLODRange(int32 BatchId, int32 BatchSizeInElems, int32 BatchElemSizeInBytes, int32 LODBegin, int32 LODEnd)
	{
		const TArrayView<const uint8> ConstBatchView = 
			const_cast<const FImageDataStorage*>(this)->GetBatchLODRange(
					BatchId, BatchSizeInElems, BatchElemSizeInBytes, LODBegin, LODEnd);

		return TArrayView<uint8>(const_cast<uint8*>(ConstBatchView.GetData()), ConstBatchView.Num());
	}

	int32 FImageDataStorage::GetAllocatedSize() const
	{ 
		SIZE_T Result = 0;
		for (const FImageArray& Buffer : Buffers)
		{
			Result += Buffer.GetAllocatedSize();
		}

		check(Result < TNumericLimits<int32>::Max())
		return (int32)Result;
	}

	int32 FImageDataStorage::GetDataSize() const
	{
		SIZE_T Result = 0;
		for (const FImageArray& Buffer : Buffers)
		{
			Result += Buffer.Num();
		}

		check(Result < TNumericLimits<int32>::Max())
		return (int32)Result;
	}

	bool FImageDataStorage::IsEmpty() const
	{
		check(!IsVoid());

		for (const FImageArray& Buffer : Buffers)
		{
			if (!Buffer.IsEmpty())
			{
				return false;
			}
		}

		return true;
	}

	void FImageDataStorage::Serialise(FOutputArchive& Arch) const
	{
		Arch << ImageSize.X;
		Arch << ImageSize.Y;
		Arch << ImageFormat;
		Arch << NumLODs;

		Arch << Buffers.Num();

		for (const FImageArray& Buffer : Buffers)
		{
			Arch << Buffer;
		}
	
		Arch << CompactedTailOffsets.Num();
		Arch << CompactedTailOffsets;
	}

	void FImageDataStorage::Unserialise(FInputArchive& Arch)
	{
		Arch >> ImageSize.X;
		Arch >> ImageSize.Y;
		Arch >> ImageFormat;
		Arch >> NumLODs;

		int32 BuffersNum = 0;
		Arch >> BuffersNum;

		Buffers.SetNum(BuffersNum);

		for (int32 I = 0; I < BuffersNum; ++I)
		{
			Arch >> Buffers[I];
		}

		int32 NumTailOffsets = 0;
		Arch >> NumTailOffsets;
		check(NumTailOffsets == CompactedTailOffsets.Num());
		
		Arch >> CompactedTailOffsets;
	}
}
