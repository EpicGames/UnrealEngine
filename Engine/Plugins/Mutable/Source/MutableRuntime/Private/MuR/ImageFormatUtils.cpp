// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImageFormatUtils.h"
#include "MuR/Image.h"

namespace UE::Mutable::Private
{

namespace SubImageFormat
{
namespace
{
	void DecompressionFuncNoOp(miro::FImageSize, miro::FImageSize, miro::FImageSize, const uint8*, uint8*)
	{
		checkf(false, TEXT("Formats not supported by SelectDecompressionFunction"));
	}

	void CompressionFuncNoOp(miro::FImageSize, miro::FImageSize, miro::FImageSize, const uint8*, uint8*, int32 Quality)
	{
		checkf(false, TEXT("Formats not supported by SelectCompressionFunction"));
	}

	void UncompressedFormatFuncNoOp(FImageSize, FImageSize, FImageSize, const uint8*, uint8*)
	{
		checkf(false, TEXT("Formats not supported by SelectUncompressedFormat"));
	}
} // namespace

	miro::SubImageDecompression::FuncRefType SelectDecompressionFunction(EImageFormat DestFormat, EImageFormat SrcFormat)
	{
		using DecompressionFuncRefType = miro::SubImageDecompression::FuncRefType;
		
		check(DestFormat != EImageFormat::None);
		check(SrcFormat != EImageFormat::None);

		switch(SrcFormat)
		{
			case EImageFormat::BC1:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC1_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC1_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::BC2:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC2_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC2_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::BC3:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC3_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC3_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::BC4:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC4_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC4_To_RGBSubImage);
				}
				if (DestFormat == EImageFormat::L_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC4_To_LSubImage);
				}
				break;
			}

			case EImageFormat::BC5:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC5_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::BC5_To_RGBSubImage);
				}
				break;
			}
	
			case EImageFormat::ASTC_4x4_RGBA_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC4x4RGBAL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC4x4RGBAL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_4x4_RGB_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC4x4RGBL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC4x4RGBL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_4x4_RG_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC4x4RGL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC4x4RGL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_6x6_RGBA_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC6x6RGBAL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC6x6RGBAL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_6x6_RGB_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC6x6RGBL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC6x6RGBL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_6x6_RG_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC6x6RGL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC6x6RGL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_8x8_RGBA_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC8x8RGBAL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC8x8RGBAL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_8x8_RGB_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC8x8RGBL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC8x8RGBL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_8x8_RG_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC8x8RGL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC8x8RGL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_10x10_RGBA_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC10x10RGBAL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC10x10RGBAL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_10x10_RGB_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC10x10RGBL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC10x10RGBL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_10x10_RG_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC10x10RGL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC10x10RGL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_12x12_RGBA_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC12x12RGBAL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC12x12RGBAL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_12x12_RGB_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC12x12RGBL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC12x12RGBL_To_RGBSubImage);
				}
				break;
			}

			case EImageFormat::ASTC_12x12_RG_LDR:
			{
				if (DestFormat == EImageFormat::RGBA_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC12x12RGL_To_RGBASubImage);
				}
				else if (DestFormat == EImageFormat::RGB_UByte)
				{
					return DecompressionFuncRefType(miro::SubImageDecompression::ASTC12x12RGL_To_RGBSubImage);
				}
				break;
			}
		}

		return DecompressionFuncRefType(DecompressionFuncNoOp);
	}

	miro::SubImageCompression::FuncRefType SelectCompressionFunction(EImageFormat DestFormat, EImageFormat SrcFormat)
	{
		using CompressionFuncRefType = miro::SubImageCompression::FuncRefType;

		check(DestFormat != EImageFormat::None);
		check(SrcFormat != EImageFormat::None);
		
		switch (SrcFormat)
		{
			case EImageFormat::L_UByte:
			{
				switch (DestFormat)
				{
					case EImageFormat::BC4:
					{
						return CompressionFuncRefType(miro::SubImageCompression::L_To_BC4SubImage);
					}
					case EImageFormat::ASTC_4x4_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::L_To_ASTC4x4RGBLSubImage);
					}	
				}
				break;
			}

			case EImageFormat::RGB_UByte:
			{
				switch (DestFormat)
				{
					case EImageFormat::BC1:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_BC1SubImage);
					}
					case EImageFormat::BC2:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_BC2SubImage);
					}
					case EImageFormat::BC3:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_BC3SubImage);
					}
					case EImageFormat::BC5:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_BC5SubImage);
					}
					case EImageFormat::ASTC_4x4_RGBA_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC4x4RGBALSubImage);
					}
					case EImageFormat::ASTC_4x4_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC4x4RGBLSubImage);
					}
					case EImageFormat::ASTC_4x4_RG_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC4x4RGLSubImage);
					}

					case EImageFormat::ASTC_6x6_RGBA_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC6x6RGBALSubImage);
					}
					case EImageFormat::ASTC_6x6_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC6x6RGBLSubImage);
					}
					case EImageFormat::ASTC_6x6_RG_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC6x6RGLSubImage);
					}

					case EImageFormat::ASTC_8x8_RGBA_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC8x8RGBALSubImage);
					}
					case EImageFormat::ASTC_8x8_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC8x8RGBLSubImage);
					}
					case EImageFormat::ASTC_8x8_RG_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGB_To_ASTC8x8RGLSubImage);
					}
				}

				break;
			}

			case EImageFormat::RGBA_UByte:
			{
				switch (DestFormat)
				{
					case EImageFormat::BC1:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_BC1SubImage);
					}
					case EImageFormat::BC2:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_BC2SubImage);
					}
					case EImageFormat::BC3:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_BC3SubImage);
					}

					case EImageFormat::BC5:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_BC5SubImage);
					}

					case EImageFormat::ASTC_4x4_RGBA_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC4x4RGBALSubImage);
					}
					case EImageFormat::ASTC_4x4_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC4x4RGBLSubImage);
					}
					case EImageFormat::ASTC_4x4_RG_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC4x4RGLSubImage);
					}

					case EImageFormat::ASTC_6x6_RGBA_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC6x6RGBALSubImage);
					}
					case EImageFormat::ASTC_6x6_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC6x6RGBLSubImage);
					}
					case EImageFormat::ASTC_6x6_RG_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC6x6RGLSubImage);
					}

					case EImageFormat::ASTC_8x8_RGBA_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC8x8RGBALSubImage);
					}
					case EImageFormat::ASTC_8x8_RGB_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC8x8RGBLSubImage);
					}
					case EImageFormat::ASTC_8x8_RG_LDR:
					{
						return CompressionFuncRefType(miro::SubImageCompression::RGBA_To_ASTC8x8RGLSubImage);
					}

				}
				break;
			}
		}

		//check(false);
		return CompressionFuncRefType(CompressionFuncNoOp);
	}

	void SubImageVec3ToVec4_U8(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		for (uint16 Y = 0; Y < SubSize.Y; ++Y)
		{
			for (uint16 X = 0; X < SubSize.X; ++X)
			{
				uint8* ToPtr         = To   + (Y*ToSize.X   + X)*4;
				const uint8* FromPtr = From + (Y*FromSize.X + X)*3;

				// TODO: Optimize.
				ToPtr[0] = FromPtr[0];	
				ToPtr[1] = FromPtr[1];	
				ToPtr[2] = FromPtr[2];
				ToPtr[3] = 255;
			}
		}
	}

	void SubImageVec4ToVec3_U8(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		for (uint16 Y = 0; Y < SubSize.Y; ++Y)
		{
			for (uint16 X = 0; X < SubSize.X; ++X)
			{
				uint8* ToPtr         = To   + (Y*ToSize.X   + X)*3;
				const uint8* FromPtr = From + (Y*FromSize.X + X)*4;

				// TODO: Optimize.
				ToPtr[0] = FromPtr[0];	
				ToPtr[1] = FromPtr[1];	
				ToPtr[2] = FromPtr[2];
			}
		}
	}
	
	void SubImageVec1ToVec4_U8(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		for (uint16 Y = 0; Y < SubSize.Y; ++Y)
		{
			for (uint16 X = 0; X < SubSize.X; ++X)
			{
				uint8* ToPtr         = To   + (Y*ToSize.X   + X)*4;
				const uint8* FromPtr = From + (Y*FromSize.X + X)*1;

				// TODO: Optimize.
				ToPtr[0] = FromPtr[0];	
				ToPtr[1] = FromPtr[0];	
				ToPtr[2] = FromPtr[0];
				ToPtr[3] = 255;
			}
		}
	}

	void SubImageVec4ToVec1_U8(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* RESTRICT From, uint8* RESTRICT To)
	{
		for (uint16 Y = 0; Y < SubSize.Y; ++Y)
		{
			for (uint16 X = 0; X < SubSize.X; ++X)
			{
				uint8* ToPtr         = To   + (Y*ToSize.X   + X)*1;
				const uint8* FromPtr = From + (Y*FromSize.X + X)*4;

				// TODO: Optimize.
				ToPtr[0] = FromPtr[0];	
			}
		}
	}


	UncompressedFormatFuncType* SelectUncompressedFormatFunction(EImageFormat DstFormat, EImageFormat SrcFormat)
	{
		check(GetUncompressedFormat(DstFormat) == DstFormat);
		check(GetUncompressedFormat(SrcFormat) == SrcFormat);

		switch (SrcFormat)
		{
		case EImageFormat::RGBA_UByte:
		{
			switch (DstFormat)
			{
				case EImageFormat::RGB_UByte:
				{
					return SubImageVec4ToVec3_U8;
				}
				case EImageFormat::L_UByte:
				{
					return SubImageVec4ToVec1_U8;
				}
			}
			break;
		}
		case EImageFormat::RGB_UByte:
		{
			switch (DstFormat)
			{
				case EImageFormat::RGBA_UByte:
				{
					return SubImageVec3ToVec4_U8;
				}
			}
			break;
		}
		case EImageFormat::L_UByte:
		{
			switch (DstFormat)
			{
				case EImageFormat::RGBA_UByte:
				{
					return SubImageVec1ToVec4_U8;
				}
			}
		}
		}
		return UncompressedFormatFuncNoOp;
	}

} // namespace SubImageFormat
} // namespace UE::Mutable::Private
