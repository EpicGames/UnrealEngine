// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/BlockCompression/Miro/Miro.h"
#include "MuR/ImageTypes.h"

namespace UE::Mutable::Private
{

namespace SubImageFormat
{	

	miro::SubImageDecompression::FuncRefType SelectDecompressionFunction(EImageFormat DestFormat, EImageFormat SrcFormat);
	miro::SubImageCompression::FuncRefType SelectCompressionFunction(EImageFormat DestFormat, EImageFormat SrcFormat);	

	using UncompressedFormatFuncType = void(FImageSize, FImageSize, FImageSize, const uint8*, uint8*); 
	UncompressedFormatFuncType* SelectUncompressedFormatFunction(EImageFormat DestFormat, EImageFormat SrcFormat);
	
	void SubImageVec3ToVec4_U8(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To);
	void SubImageVec4ToVec3_U8(FImageSize FromSize, FImageSize ToSize, FImageSize SubSize, const uint8* From, uint8* To);

} // namespace SubImageFormat
} // namespace UE::Mutable::Private
