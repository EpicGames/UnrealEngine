// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImageRLE.h"

#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeCompatibleBytes.h"

namespace UE::Mutable::Private
{

    //---------------------------------------------------------------------------------------------
    void CompressRLE_L( uint32& OutCompressedSize, int32 width, int32 rows, const uint8* pBaseData, uint8* destData, uint32 destDataSize )
    {
		const uint8* InitialBaseData = pBaseData;

        uint8* rle = destData;

        uint32 maxSize = destDataSize;

        // The first uint32 will be the total mip size.
        // Then there is an offset from the initial data pointer for each line.
        uint32 offset = sizeof( uint32 ) * ( rows + 1 );

        // Could happen in an image of 1x100, for example.
        if ( offset >= maxSize )
        {
			OutCompressedSize = 0;
            return;
        }

        for ( int32 r = 0; r < rows; ++r )
        {
            uint32* pOffset = (uint32*)&rle[sizeof( uint32 ) * ( r + 1 )];
            FMemory::Memmove( pOffset, &offset, sizeof( uint32 ) );

            const uint8* pBaseRowEnd = pBaseData + width;
            while ( pBaseData != pBaseRowEnd )
            {
                // Count equal pixels
                uint8 equalPixel = *pBaseData;

                uint16 equal = 0;
                while ( pBaseData != pBaseRowEnd && equal < 65535 && pBaseData[0] == equalPixel )
                {
                    pBaseData++;
                    equal++;
                }

                // Count different pixels
                uint8 different = 0;
                const uint8* pDifferentPixels = pBaseData;
                while ( pBaseData < pBaseRowEnd - 1 && different < 255 &&
                        // Last in the row, or different from next
                        ( pBaseData == pBaseRowEnd || pBaseData[0] != pBaseData[1] ) )
                {
                    pBaseData++;
                    different++;
                }

                // Copy header
                if ( maxSize < offset + 4 )
                {
					OutCompressedSize = 0;
                    return;
                }
                FMemory::Memmove( &rle[offset], &equal, sizeof( uint16 ) );
                offset += 2;
                rle[offset] = different;
                ++offset;

                // Copy the equal pixel
                rle[offset] = equalPixel;
                ++offset;

                // Copy the different pixels
                if ( different )
                {
                    if ( maxSize < offset + different )
                    {
						OutCompressedSize = 0;
						return;
                    }
                    FMemory::Memmove( &rle[offset], pDifferentPixels, different );
                    offset += different;
                }
            }
        }

        uint32* pTotalSize = (uint32*)rle;
        *pTotalSize = offset;

#ifdef MUTABLE_DEBUG_RLE		
		{
			TArray<uint8> Temp;
			Temp.SetNum(width*rows);
			UncompressRLE_L(width,rows,destData,Temp.GetData());
			int32 Difference = FMemory::Memcmp(Temp.GetData(), InitialBaseData, width * rows);
			if (Difference)
			{
				// Different pos.
				SIZE_T Delta = 0;
				for ( ; Delta<width*rows; ++Delta )
				{
					if (Temp[Delta] != InitialBaseData[Delta])
					{
						break;
					}
				}

				UncompressRLE_L(width, rows, destData, Temp.GetData());
				int32 OutSize = 0;
                CompressRLE_L(OutSize, width, rows, InitialBaseData, destData, destDataSize);
                check(OutSize > 0)
				check(false);
			}
		}
#endif

        // succeded
		OutCompressedSize = offset;
    }

	struct FIterativeDecodeTokenRLE_L
	{
		uint32 SrcDataOffset = 0;
		uint16 NumEqual      = 0;
		uint8 NumDifferent   = 0;
		uint8 EqualValue     = 0;
	};
	static_assert(sizeof(FIterativeDecodeTokenRLE_L) == sizeof(uint64));

	struct FSequenceHeaderRLE_L
	{
		uint16 NumEqual;
		uint8 NumDifferent;
		uint8 EqualValue;
	};
	static_assert(sizeof(FSequenceHeaderRLE_L) == 4);

	uint64 GetBeginTokenIterativeDecodeRLE_L(const uint8* Src, int32 NumRows)
	{
		// Skip RLE size and rows header.
		Src += (NumRows + 1) * sizeof(uint32);

		// Read first header.
		FSequenceHeaderRLE_L FirstHeader;
		FMemory::Memcpy(&FirstHeader, Src, sizeof(FSequenceHeaderRLE_L));

		FIterativeDecodeTokenRLE_L Token
		{
			.SrcDataOffset = (NumRows + 1)*sizeof(uint32) + sizeof(FSequenceHeaderRLE_L),
			.NumEqual      = FirstHeader.NumEqual,
			.NumDifferent  = FirstHeader.NumDifferent,
			.EqualValue    = FirstHeader.EqualValue,
		};

		return BitCast<uint64>(Token);
	}

	void IterativeDecodeRLE_L(uint8* Dest, const uint8* Src, uint64& InOutDecodeToken, int32 NumElems)
	{
		FIterativeDecodeTokenRLE_L DecodeToken = BitCast<FIterativeDecodeTokenRLE_L>(InOutDecodeToken);

		// DecodeToken has the information about the previous data header and points to the first differnt value.
		Src = Src + DecodeToken.SrcDataOffset;

		FSequenceHeaderRLE_L SeqHeader
		{
			.NumEqual     = DecodeToken.NumEqual,
			.NumDifferent = DecodeToken.NumDifferent,
			.EqualValue   = DecodeToken.EqualValue,
		};

		for (int32 NumElemsDecoded = 0;;)
		{
			int32 NumEqualToDecode = SeqHeader.NumEqual; 
			int32 NumDifferentToDecode = SeqHeader.NumDifferent;

			if (NumElemsDecoded + NumEqualToDecode > NumElems)
			{
				NumEqualToDecode = NumElems - NumElemsDecoded;
				NumDifferentToDecode = 0;
			}
			else if (NumElemsDecoded + NumEqualToDecode + NumDifferentToDecode > NumElems)
			{
				NumDifferentToDecode = NumElems - (NumElemsDecoded + NumEqualToDecode);
			}

			FMemory::Memset(Dest + NumElemsDecoded, SeqHeader.EqualValue, NumEqualToDecode);
			FMemory::Memcpy(Dest + NumElemsDecoded + NumEqualToDecode, Src, NumDifferentToDecode);

			DecodeToken.NumEqual = SeqHeader.NumEqual - NumEqualToDecode; 
			DecodeToken.NumDifferent = SeqHeader.NumDifferent - NumDifferentToDecode;
			DecodeToken.SrcDataOffset += NumDifferentToDecode;

			Src += NumDifferentToDecode;
			NumElemsDecoded += NumEqualToDecode + NumDifferentToDecode;

			if (NumElemsDecoded >= NumElems)
			{
				break;
			}
		
            // Read next header, at this point Src is already pointing to the next sequence header as all
            // sequence data has been consumed.
            FMemory::Memcpy(&SeqHeader, Src, sizeof(FSequenceHeaderRLE_L));
       
            // We don't allow empty sequences. Stop decoding.
            if (SeqHeader.NumEqual + SeqHeader.NumDifferent == 0)
            {
                check(false);
                break;
            }

            // Point Src to the sequence data.
			Src += sizeof(FSequenceHeaderRLE_L);

			DecodeToken.SrcDataOffset += sizeof(FSequenceHeaderRLE_L);
			DecodeToken.EqualValue = SeqHeader.EqualValue;
			
		}

		InOutDecodeToken = BitCast<uint64>(DecodeToken);
	}

    uint32 UncompressRLE_L( int32 width, int32 rows, const uint8* pStartBaseData, uint8* pStartDestData )
    {
		// Empty data. 
		if (!pStartBaseData)
		{
			FMemory::Memset(pStartDestData, 0, width * rows);
			return 0;
		}

		const uint8* pBaseData = pStartBaseData;
		uint8* pDestData = pStartDestData;
		pBaseData += sizeof(uint32); // Total mip size
        pBaseData += rows*sizeof(uint32); // Size of each row.

        for ( int32 r=0; r<rows; ++r )
        {
            const uint8* pDestRowEnd = pDestData + width;
            while ( pDestData!=pDestRowEnd )
            {
                // Decode header
				//check(pBaseData + 4 <= pStartBaseData + BaseDataSize);
				
				uint16 equal = 0;
                FMemory::Memmove(&equal, pBaseData, sizeof(uint16));
                pBaseData += 2;

                uint8 different = *pBaseData;
                ++pBaseData;

                uint8 equalPixel = *pBaseData;
                ++pBaseData;

                if (equal)
                {
					check(pDestData + equal <= pStartDestData + width * rows);
					FMemory::Memset(pDestData, equalPixel, equal);
                    pDestData += equal;
                }

                if (different)
                {
					check(pDestData + different <= pStartDestData + width * rows);
					//check(pBaseData + different <= pStartBaseData + BaseDataSize);
					FMemory::Memmove( pDestData, pBaseData, different );
                    pDestData += different;
                    pBaseData += different;
                }
            }
        }

		uint32 TotalSize = uint32(pBaseData-pStartBaseData);
        check( TotalSize==*(uint32*)pStartBaseData );

        return TotalSize;
    }

    void CompressRLE_L1(uint32& OutCompressedSize, int32 width, int32 rows,
                             const uint8* pBaseData,
                             uint8* destData,
                             uint32 destDataSize )
    {
        TArray<int8_t> rle;
        rle.Reserve(  (width * rows) / 2 );

        uint32 offset = sizeof(uint32)*(rows+1);
        rle.SetNum( offset, EAllowShrinking::No );

        for ( int32 r=0; r<rows; ++r )
        {
            uint32* pOffset = (uint32*) &rle[ sizeof(uint32)*(r+1) ];
            *pOffset = offset;

            const uint8* pBaseRowEnd = pBaseData + width;
            while ( pBaseData!=pBaseRowEnd )
            {
                // Count 0 pixels
                uint16 zeroPixels = 0;
                while ( pBaseData!=pBaseRowEnd
                        && !*pBaseData )
                {
                    pBaseData++;
                    zeroPixels++;
                }

                // Count 1 pixels
                uint16 onePixels = 0;
                while ( pBaseData!=pBaseRowEnd
                        && *pBaseData )
                {
                    pBaseData++;
                    onePixels++;
                }

                // Copy block
                rle.SetNum( rle.Num()+4, EAllowShrinking::No);
                FMemory::Memmove(&rle[ offset ], &zeroPixels, sizeof(uint16));
                offset += 2;

                FMemory::Memmove(&rle[ offset ], &onePixels, sizeof(uint16));
                offset += 2;
            }
        }

        if (destDataSize<(uint32)rle.Num())
        {
            // Failed
			OutCompressedSize = 0;
            return;
        }

        uint32* pTotalSize = (uint32*) &rle[ 0 ];
        *pTotalSize = offset;

        FMemory::Memmove( destData, &rle[0], offset );

        // succeded
		OutCompressedSize = offset;
    }


    uint32 UncompressRLE_L1(int32 width, int32 rows, const uint8* pStartBaseData, uint8* pDestData)
    {
        const uint8* pBaseData = pStartBaseData;
        pBaseData += sizeof(uint32); // Total mip size
        pBaseData += rows*sizeof(uint32);

        for ( int32 r=0; r<rows; ++r )
        {
            const uint8* pDestRowEnd = pDestData + width;
            while ( pDestData!=pDestRowEnd )
            {
                // Decode header
                uint16 zeroPixels = 0;
                FMemory::Memmove(&zeroPixels, pBaseData, sizeof(uint16));
                pBaseData += 2;

                uint16 onePixels = 0;
                FMemory::Memmove(&onePixels, pBaseData, sizeof(uint16));
                pBaseData += 2;

                if (zeroPixels)
                {
                    FMemory::Memzero( pDestData, zeroPixels );
                    pDestData += zeroPixels;
                }

                if (onePixels)
                {
                    FMemory::Memset( pDestData, 255, onePixels );
                    pDestData += onePixels;
                }
            }
        }

        size_t totalSize = pBaseData-pStartBaseData;
        check( totalSize==*(uint32*)pStartBaseData );

        return (uint32)totalSize;
    }


    void CompressRLE_RGBA(uint32& OutCompressedSize, int32 width, int32 rows,
                           const uint8* pBaseDataByte,
                           uint8* DestData, uint32 DestDataSize)
    {
        // TODO: Support for compression from compressed data size, like L_RLE formats.
        TArray<int8_t> rle;
        rle.Reserve(  (width*rows) );

        const uint32* pBaseData = (const uint32*)pBaseDataByte;
        rle.SetNum( rows*4, EAllowShrinking::No);
        uint32 offset = sizeof(uint32)*rows;
        for ( int32 r=0; r<rows; ++r )
        {
            uint32* pOffset = (uint32*) &rle[ sizeof(uint32)*r ];
            FMemory::Memmove(pOffset, &offset, sizeof(uint32));

            const uint32* pBaseRowEnd = pBaseData + width;
            while ( pBaseData!=pBaseRowEnd )
            {
                // Count equal pixels
                uint32 equalPixel = *pBaseData;

                uint16 equal = 0;
                while ( pBaseData<pBaseRowEnd-3 && equal<65535
                        && pBaseData[0]==equalPixel && pBaseData[1]==equalPixel
                        && pBaseData[2]==equalPixel && pBaseData[3]==equalPixel )
                {
                    pBaseData+=4;
                    equal++;
                }

                // Count different pixels
                uint16 different = 0;
                const uint32* pDifferentPixels = pBaseData;
                while ( pBaseData!=pBaseRowEnd
                        &&
                        different<65535
                        &&
                        // Last in the row, or different from next
                        ( pBaseData>pBaseRowEnd-4
                          || pBaseData[0]!=pBaseData[1]
                          || pBaseData[0]!=pBaseData[2]
                          || pBaseData[0]!=pBaseData[3]
                          )
                        )
                {
					pBaseData += FMath::Min(int64(4), int64(pBaseRowEnd - pBaseData));
                    different++;
                }

                // Copy header
                rle.SetNum( rle.Num()+8, EAllowShrinking::No);
                FMemory::Memmove(&rle[ offset ], &equal, sizeof(uint16));
                offset += 2;
                FMemory::Memmove(&rle[ offset ], &different, sizeof(uint16));
                offset += 2;
                FMemory::Memmove(&rle[ offset ], &equalPixel, sizeof(uint32));
                offset += 4;

                // Copy the different pixels
				if (different)
				{
					// If we are at the end of a row, maybe there isn't a block of 4 pixels
					uint16 BytesToCopy = FMath::Min(different * 4 * 4, uint16(pBaseRowEnd - pDifferentPixels) * 4);

					rle.SetNum( rle.Num()+ BytesToCopy, EAllowShrinking::No);
                    FMemory::Memmove( &rle[offset], pDifferentPixels, BytesToCopy);
					offset += BytesToCopy;
				}
            }

            if (offset > DestDataSize)
            {
                break;
            }
        }

        OutCompressedSize = DestDataSize >= offset ? offset : 0; 
        
        if (OutCompressedSize > 0)
        {
            FMemory::Memmove(DestData, rle.GetData(), offset);
        }
    }


    void UncompressRLE_RGBA( int32 width, int32 rows, const uint8* pBaseData, uint8* pDestDataB )
    {
        uint32* pDestData = reinterpret_cast<uint32*>( pDestDataB );

        pBaseData += rows*sizeof(uint32);

        int32 pendingPixels = width*rows;

        for ( int32 r=0; r<rows; ++r )
        {
            const uint32* pDestRowEnd = pDestData + width;
            while ( pDestData!=pDestRowEnd )
            {
                // Decode header
                uint16 equal = 0;
                FMemory::Memmove(&equal, pBaseData, sizeof(uint16));
                pBaseData += 2;

                uint16 different = 0;
                FMemory::Memmove(&different, pBaseData, sizeof(uint16));
                pBaseData += 2;

                uint32 equalPixel = 0;
                FMemory::Memmove(&equalPixel, pBaseData, sizeof(uint32));
                pBaseData += 4;

                check((equal+different)*4<=pendingPixels);

                for ( int32 e=0; e<equal*4; ++e )
                {
                    FMemory::Memmove( pDestData, &equalPixel, 4 );
                    ++pDestData;
                    pendingPixels--;
                }

				if (different)
				{
					// If we are at the end of a row, maybe there isn't a block of 4 pixels
					uint16 PixelsToCopy = FMath::Min(uint16(different * 4), uint16(pDestRowEnd - pDestData));

					FMemory::Memmove( pDestData, pBaseData, PixelsToCopy *4 );
					pDestData += PixelsToCopy;
					pBaseData += PixelsToCopy *4;
                    pendingPixels-= PixelsToCopy;
                }
            }
        }

        check(pendingPixels==0);
    }


    struct UINT24
    {
        uint8 d[3];

        bool operator==( const UINT24& o ) const
        {
            return d[0]==o.d[0] && d[1]==o.d[1] && d[2]==o.d[2];
        }

        bool operator!=( const UINT24& o ) const
        {
            return d[0]!=o.d[0] || d[1]!=o.d[1] || d[2]!=o.d[2];
        }
    };
    static_assert( sizeof(UINT24)==3, "Uint24SizeCheck" );


    void CompressRLE_RGB(uint32& OutCompressedSize, int32 width, int32 rows,
                         const uint8* pBaseDataByte,
                         uint8* DestData, uint32 DestDataSize)
    {
        // TODO: Optimize so no extra memory is allocated at this stage.

        TArray<int8_t> rle;
        rle.Reserve(width*rows);

        const UINT24* pBaseData = (const UINT24*)pBaseDataByte;

        rle.SetNum( rows*4, EAllowShrinking::No );

        uint32 offset = sizeof(uint32)*rows;
        for ( int32 r=0; r<rows; ++r )
        {
            uint32* pOffset = (uint32*) &rle[ sizeof(uint32)*r ];
            *pOffset = offset;

            const UINT24* pBaseRowEnd = pBaseData + width;
            while ( pBaseData!=pBaseRowEnd )
            {
                // Count equal pixels
                UINT24 equalPixel = *pBaseData;

                uint16 equal = 0;
                while ( pBaseData<pBaseRowEnd-3 && equal<65535
                        && pBaseData[0]==equalPixel && pBaseData[1]==equalPixel
                        && pBaseData[2]==equalPixel && pBaseData[3]==equalPixel )
                {
                    pBaseData+=4;
                    equal++;
                }

                // Count different pixels
                uint16 different = 0;
                const UINT24* pDifferentPixels = pBaseData;
                while ( pBaseData!=pBaseRowEnd
                        &&
                        different<65535
                        &&
                        // Last pixels in the row, or different from next
                        ( pBaseData>pBaseRowEnd-4
                          || pBaseData[0]!=pBaseData[1]
                          || pBaseData[0]!=pBaseData[2]
                          || pBaseData[0]!=pBaseData[3]
                          )
                        )
                {
                    pBaseData+=FMath::Min(int64(4), int64(pBaseRowEnd-pBaseData));
                    different++;
                }

                // Copy header
                rle.SetNum( rle.Num()+8, EAllowShrinking::No);
                FMemory::Memmove( &rle[offset], &equal, sizeof(uint16) );
                offset += 2;
                FMemory::Memmove( &rle[offset], &different, sizeof(uint16) );
                offset += 2;
                FMemory::Memmove( &rle[offset], &equalPixel, sizeof(UINT24) );
                offset += 4;

                // Copy the different pixels
				if (different)
				{
					// If we are at the end of a row, maybe there isn't a block of 4 pixels
					uint16 BytesToCopy = FMath::Min(different * 4 * 3, uint16(pBaseRowEnd- pDifferentPixels)*3 );

					rle.SetNum( rle.Num()+BytesToCopy, EAllowShrinking::No );
                    FMemory::Memmove( &rle[offset], pDifferentPixels, BytesToCopy );
					offset += BytesToCopy;
				}
            }

            if (offset > DestDataSize)
            {
                break;
            }
        }

        OutCompressedSize = DestDataSize >= offset ? offset : 0; 
        
        if (OutCompressedSize > 0)
        {
            FMemory::Memmove(DestData, rle.GetData(), offset);
        }

    }

    void UncompressRLE_RGB(int32 width, int32 rows, const uint8* pBaseData, uint8* pDestDataB)
    {
		check(pBaseData && pDestDataB);

        UINT24* pDestData = reinterpret_cast<UINT24*>( pDestDataB );

        pBaseData += rows*sizeof(uint32);

        for ( int32 r=0; r<rows; ++r )
        {
            const UINT24* pDestRowEnd = pDestData + width;
            while ( pDestData!=pDestRowEnd )
            {
                // Decode header
                uint16 equal = *(const uint16*)pBaseData;
                pBaseData += 2;

                uint16 different = *(const uint16*)pBaseData;
                pBaseData += 2;

                UINT24 equalPixel = *(const UINT24*)pBaseData;
                pBaseData += 4;

                for ( int32 e=0; e<equal*4; ++e )
                {
                    FMemory::Memmove( pDestData, &equalPixel, 3 );
                    ++pDestData;
                }

				if (different)
				{
					// If we are at the end of a row, maybe there isn't a block of 4 pixels
					uint16 PixelsToCopy = FMath::Min(uint16(different * 4), uint16(pDestRowEnd - pDestData));

					FMemory::Memmove( pDestData, pBaseData, PixelsToCopy*3);
					pDestData += PixelsToCopy;
					pBaseData += PixelsToCopy*3;
				}
            }
        }
    }



}

