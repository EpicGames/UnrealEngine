// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/UnrealMathSSE.h"
#include "MuR/Image.h"
/** This define enabled additional checks when using RLE compression. These checks have a lot of
* of overhead so it should usually be disabled.
*/
//#define MUTABLE_DEBUG_RLE


namespace UE::Mutable::Private
{
    //---------------------------------------------------------------------------------------------
    //! Compress to RLE if the compressed image fits in destData, which should be preallocated.
    //! If it doesn't fit, it returns 0, and the content of destData is undefined.
    //! This method doesn't allocate any memory.
    //!
    //! This RLE format compresses the data per row. At the beginning of the data there is the
    //! total compressed data size (uint32_t) followed by the row offsets for direct access.
    //! Every offset is a uint32_t starting at the begining of the compressed data.
    //! Every row consists of a series of header + data. The headers is:
    //!     - a UINT16 with how many equal pixels are there
    //!     - a UINT8 of how many different pixels follow the equal pixels
    //!		- the value of the "equal" pixels
    //! After the header there are as many pixel values as different pixels.
    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern void CompressRLE_L(uint32& OutCompressedSize, int32 Width, int32 Rows,
                                   const uint8* BaseData,
                                   uint8* DestData, uint32 DestDataSize);

    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern uint32 UncompressRLE_L(int32 Width, int32 Rows,
                                     const uint8* BaseData,
                                     uint8* pDestData);

    MUTABLERUNTIME_API extern uint64 GetBeginTokenIterativeDecodeRLE_L(const uint8* Src, int32 NumRows);
    MUTABLERUNTIME_API extern void IterativeDecodeRLE_L(uint8* Dest, const uint8* Src, uint64& InOutDecodeToken, int32 NumElems);
    //---------------------------------------------------------------------------------------------
    //! This RLE format compresses the data per row. At the beginning of the data there are the row
    //! offsets for direct access.
    //! Every row consists of a series of blocks. The blocks are:
    //!     - a UINT16 with how many 0 pixels are there
    //!     - a UINT16 with how many 1 pixels follow the 0 pixels
    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern void CompressRLE_L1(uint32& OutCompressedSize, int32 Width, int32 Rows,
                                    const uint8* BaseData,
                                    uint8* DestData,
                                    uint32 DestDataSize);

    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern uint32 UncompressRLE_L1(int32 Width, int32 Rows,
                                      const uint8* pBaseData,
                                      uint8* pDestData);


    //---------------------------------------------------------------------------------------------
    //! This RLE format compresses the data per row. At the beginning of the data there are the row
    //! offsets for direct access. Every offset is a uint32_t.
    //! Every row consists of a series of header + data. The headers is:
    //!     - a UINT16 with how many equal pixels / 4
    //!     - a UINT16 with how many different pixels / 4
    //!     - 4 UINT8 with the value of the "equal" pixels
    //! After the header there are as many pixel values as different pixels.
    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern void CompressRLE_RGBA(uint32& OutCompressedSize, int32 Width, int32 Rows,
									const uint8* pBaseDataByte,
									uint8* DestData, uint32 DestDataSize);

    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern void UncompressRLE_RGBA(int32 Width, int32 Rows,
                                    const uint8* pBaseData,
                                    uint8* pDestDataB);


    //---------------------------------------------------------------------------------------------
    //! This RLE format compresses the data per row. At the beginning of the data there are the row
    //! offsets for direct access. Every offset is a uint32_t.
    //! Every row consists of a series of header + data. The headers is:
    //!     - a UINT16 with how many equal pixels / 4
    //!     - a UINT16 with how many different pixels / 4
    //!     - 4 UINT8 with the value of the "equal" pixels
    //! After the header there are as many pixel values as different pixels.
    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern void CompressRLE_RGB(uint32& OutCompressedSize, int32 Width, int32 Rows,
									const uint8* pBaseDataByte,
									uint8* DestData, uint32 DestDataSize);

    //---------------------------------------------------------------------------------------------
	MUTABLERUNTIME_API extern void UncompressRLE_RGB(int32 Width, int32 Rows,
                                   const uint8* pBaseData,
                                   uint8* pDestDataB);


	inline void UncompressRLE_L(const FImage* BaseImage, FImage* DestImage)
	{
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());
		
		int32 SizeX = BaseImage->GetSizeX();
		int32 SizeY = BaseImage->GetSizeY();
		
        const int32 NumLODs = DestImage->GetLODCount();
		for (int32 L = 0; L < NumLODs; ++L)
		{
			uint32 CompressedSize = UncompressRLE_L(SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L));
	
            SizeX = FMath::Max(1, FMath::DivideAndRoundUp(SizeX, 2));
			SizeY = FMath::Max(1, FMath::DivideAndRoundUp(SizeY, 2));
		}
	}

    inline void CompressRLE_L(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());
		
		int32 SizeX = BaseImage->GetSizeX();
		int32 SizeY = BaseImage->GetSizeY();

        const int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
			bool bSuccess = false;
			while (!bSuccess)
			{
				uint32 CompressedSize = 0;
				CompressRLE_L(CompressedSize, SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L), DestImage->GetLODDataSize(L));
			
				bSuccess = CompressedSize > 0;
				if (bSuccess)
				{
					DestImage->DataStorage.ResizeLOD(L, CompressedSize);
				}
				else
				{
					const int32 MipMemorySize = DestImage->DataStorage.GetLOD(L).Num();
					DestImage->DataStorage.ResizeLOD(L, FMath::Max(4, MipMemorySize * 2));
				}
			}

			SizeX = FMath::DivideAndRoundUp(SizeX, 2);
			SizeY = FMath::DivideAndRoundUp(SizeY, 2);
        }
    }

    inline void CompressRLE_L1(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());
		
		int32 SizeX = BaseImage->GetSizeX();
		int32 SizeY = BaseImage->GetSizeY();

        const int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
			bool bSuccess = false;
			while (!bSuccess)
			{
				uint32 CompressedSize = 0;
				CompressRLE_L1(CompressedSize, SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L), DestImage->GetLODDataSize(L));
			
				bSuccess = CompressedSize > 0;
				if (bSuccess)
				{
					DestImage->DataStorage.ResizeLOD(L, CompressedSize);
				}
				else
				{
					const int32 MipMemorySize = DestImage->DataStorage.GetLOD(L).Num();
					DestImage->DataStorage.ResizeLOD(L, FMath::Max(4, MipMemorySize * 2));
				}
			}

			SizeX = FMath::DivideAndRoundUp(SizeX, 2);
			SizeY = FMath::DivideAndRoundUp(SizeY, 2);
        }
    }

    inline void UncompressRLE_L1(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());
        
		int32 SizeX = BaseImage->GetSizeX();
        int32 SizeY = BaseImage->GetSizeY();

        const int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
            uint32 CompressedSize = UncompressRLE_L1(SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L));

			SizeX = FMath::Max(1, FMath::DivideAndRoundUp(SizeX, 2));
			SizeY = FMath::Max(1, FMath::DivideAndRoundUp(SizeY, 2));
		}
    }

    inline void CompressRLE_RGB(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());
		
		int32 SizeX = BaseImage->GetSizeX();
		int32 SizeY = BaseImage->GetSizeY();

        const int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
			bool bSuccess = false;
			while (!bSuccess)
			{
				uint32 CompressedSize = 0;
				CompressRLE_RGB(CompressedSize, SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L), DestImage->GetLODDataSize(L));
			
				bSuccess = CompressedSize > 0;
				if (bSuccess)
				{
					DestImage->DataStorage.ResizeLOD(L, CompressedSize);
				}
				else
				{
					const int32 MipMemorySize = DestImage->DataStorage.GetLOD(L).Num();
					DestImage->DataStorage.ResizeLOD(L, FMath::Max(4, MipMemorySize * 2));
				}
			}

			SizeX = FMath::DivideAndRoundUp(SizeX, 2);
			SizeY = FMath::DivideAndRoundUp(SizeY, 2);
        }
    }

    inline void UncompressRLE_RGB(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());
        
		int32 SizeX = BaseImage->GetSizeX();
        int32 SizeY = BaseImage->GetSizeY();
       
		int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
            UncompressRLE_RGB(SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L));

			SizeX = FMath::Max(1, FMath::DivideAndRoundUp(SizeX, 2));
			SizeY = FMath::Max(1, FMath::DivideAndRoundUp(SizeY, 2));
        }
    }

    inline void CompressRLE_RGBA(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());

		int32 SizeX = BaseImage->GetSizeX();
		int32 SizeY = BaseImage->GetSizeY();

        const int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
			bool bSuccess = false;
			while (!bSuccess)
			{
				uint32 CompressedSize = 0;
				CompressRLE_RGBA(CompressedSize, SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L), DestImage->GetLODDataSize(L));
			
				bSuccess = CompressedSize > 0;
				if (bSuccess)
				{
					DestImage->DataStorage.ResizeLOD(L, CompressedSize);
				}
				else
				{
					const int32 MipMemorySize = DestImage->DataStorage.GetLOD(L).Num();
					DestImage->DataStorage.ResizeLOD(L, FMath::Max(4, MipMemorySize * 2));
				}
			}

			SizeX = FMath::DivideAndRoundUp(SizeX, 2);
			SizeY = FMath::DivideAndRoundUp(SizeY, 2);
        }
    }

    inline void UncompressRLE_RGBA(const FImage* BaseImage, FImage* DestImage)
    {
		check(DestImage->GetLODCount() <= BaseImage->GetLODCount());
		check(DestImage->GetSize() == BaseImage->GetSize());

        int32 SizeX = BaseImage->GetSizeX();
        int32 SizeY = BaseImage->GetSizeY();
        
        const int32 NumLODs = DestImage->GetLODCount();
        for (int32 L = 0; L < NumLODs; ++L)
        {
            UncompressRLE_RGBA(SizeX, SizeY, BaseImage->GetLODData(L), DestImage->GetLODData(L));

			SizeX = FMath::Max(1, FMath::DivideAndRoundUp(SizeX, 2));
			SizeY = FMath::Max(1, FMath::DivideAndRoundUp(SizeY, 2));
        }
    }

}
