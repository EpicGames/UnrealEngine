// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImageResizeUtils.h"
#include "HAL/Platform.h"

namespace UE::Mutable::Private
{

namespace SubImageResize
{
	template<uint32 NumChannels>
	FORCENOINLINE void ImageResizeLinearByLessThanTwoSubImageVectorImpl(
			FVector2f InOneOverScalingFactor, FVector2f InSrcSubPixelOffset,
			FImageSize DestSize, FImageSize DestSubSize, FImageSize SrcSize, FImageSize SrcSubSize,
		  	uint8* RESTRICT SubDest, const uint8* RESTRICT SubSrc)
	{	
		static_assert(NumChannels > 0 && NumChannels <= 4);

		check(InSrcSubPixelOffset.X >= 0.0f && InSrcSubPixelOffset.Y >= 0.0f);
		check(InOneOverScalingFactor.X > 0.0f && InOneOverScalingFactor.Y > 0.0f);

		// For some reason (probably cache related), doing this copy yields marginally but consistently 
		// better performance than using the function argument. 
		const FVector2f OneOverScalingFactor = InOneOverScalingFactor;
		const FVector2f SrcSubPixelOffset = InSrcSubPixelOffset;
		
		check(OneOverScalingFactor.X <= 2.0f && OneOverScalingFactor.Y <= 2.0f);

		for (uint16 Y = 0; Y < DestSubSize.Y; ++Y)
		{
			for (uint16 X = 0; X < DestSubSize.X; ++X)
			{
				auto LoadPixel = [](const uint8* Ptr) -> VectorRegister4Float
				{	
					alignas(4) uint8 PixelData[4] = {0};
					if constexpr (NumChannels == 4)
					{
						FMemory::Memcpy(&(PixelData[0]), Ptr, 4);
					}
					else
					{
						for (uint32 C = 0; C < NumChannels; ++C)
						{
							PixelData[C] = Ptr[C];
						}
					}

					return VectorIntToFloat(MakeVectorRegisterInt(PixelData[0], PixelData[1], PixelData[2], PixelData[3]));
				};
				
				const FVector2f CoordsF = FVector2f(X, Y)*OneOverScalingFactor + SrcSubPixelOffset;
	
				const FIntVector2 Coords = FIntVector2(
						FMath::Clamp<int32>(FMath::FloorToInt(CoordsF.X), 0, SrcSize.X - 1), 
						FMath::Clamp<int32>(FMath::FloorToInt(CoordsF.Y), 0, SrcSize.Y - 1));

				const FIntVector2 CoordsPlusOne = FIntVector2(
						FMath::Clamp<int32>(Coords.X + 1, 0, SrcSize.X - 1),
						FMath::Clamp<int32>(Coords.Y + 1, 0, SrcSize.Y - 1));

				const VectorRegister4Float Pixel00 = LoadPixel(SubSrc + (Coords.Y*SrcSize.X + Coords.X) * NumChannels);
				const VectorRegister4Float Pixel10 = LoadPixel(SubSrc + (Coords.Y*SrcSize.X + CoordsPlusOne.X) * NumChannels);
				const VectorRegister4Float Pixel01 = LoadPixel(SubSrc + (CoordsPlusOne.Y*SrcSize.X + Coords.X) * NumChannels);
				const VectorRegister4Float Pixel11 = LoadPixel(SubSrc + (CoordsPlusOne.Y*SrcSize.X + CoordsPlusOne.X) * NumChannels);

				const VectorRegister4Float CoordsFVector = MakeVectorRegister(CoordsF.X, CoordsF.Y, 0.0f, 0.0f);
				const VectorRegister4Float Frac = VectorSubtract(CoordsFVector, VectorFloor(CoordsFVector));

				const VectorRegister4Float FracX = VectorReplicate(Frac, 0);
				const VectorRegister4Float FracY = VectorReplicate(Frac, 1);
				
				VectorRegister4Float Result = VectorMultiplyAdd(FracX, VectorSubtract(Pixel10, Pixel00), Pixel00);
				Result = VectorMultiplyAdd(
						FracY,
						VectorSubtract(VectorMultiplyAdd(FracX, VectorSubtract(Pixel11, Pixel01), Pixel01), Result),
						Result);

				uint8* Dest = SubDest + (Y*DestSize.X + X)*NumChannels;  

				if constexpr (NumChannels == 4)
				{
					VectorStoreByte4(Result, Dest);
					VectorResetFloatRegisters();
				}
				else
				{
					alignas(4) uint8 ResultData[4];
					VectorStoreByte4(Result, ResultData);
					VectorResetFloatRegisters();
				
					for (uint32 C = 0; C < NumChannels; ++C)
					{
						Dest[C] = ResultData[C];
					}
				}
			}
		}
	}

	template<int32 NumChannels>
	UE_REWRITE void ImageResizeLinearByLessThanTwoSubImage(
			FVector2f OneOverScalingFactor, FVector2f SrcSubPixelOffset,
			FImageSize DestSize, FImageSize DestSubSize, FImageSize SrcSize, FImageSize SrcSubSize,
			uint8* RESTRICT SubDest, const uint8* RESTRICT SubSrc)
	{
			ImageResizeLinearByLessThanTwoSubImageVectorImpl<NumChannels>(
					OneOverScalingFactor, SrcSubPixelOffset,
					DestSize, DestSubSize, SrcSize, SrcSubSize, SubDest, SubSrc);
	}

	void SubImageResizeLinearByLessThanTwoVec1_U8(
				FVector2f OneOverScalingFactor, 
				FVector2f SrcSubPixelOffset,
				FImageSize DestSize, FImageSize DestSubSize, 
				FImageSize SrcSize, FImageSize SrcSubSize,
				uint8* RESTRICT SubDest, 
				const uint8* RESTRICT SubSrc)
	{
		ImageResizeLinearByLessThanTwoSubImage<1>(OneOverScalingFactor, SrcSubPixelOffset, DestSize, DestSubSize, SrcSize, SrcSubSize, SubDest, SubSrc); 
	}

	void SubImageResizeLinearByLessThanTwoVec3_U8(
				FVector2f OneOverScalingFactor, 
				FVector2f SrcSubPixelOffset,
				FImageSize DestSize, FImageSize DestSubSize, 
				FImageSize SrcSize, FImageSize SrcSubSize,
				uint8* RESTRICT SubDest, 
				const uint8* RESTRICT SubSrc)
	{
		ImageResizeLinearByLessThanTwoSubImage<3>(OneOverScalingFactor, SrcSubPixelOffset, DestSize, DestSubSize, SrcSize, SrcSubSize, SubDest, SubSrc); 
	}

	void SubImageResizeLinearByLessThanTwoVec4_U8(
				FVector2f OneOverScalingFactor, 
				FVector2f SrcSubPixelOffset,
				FImageSize DestSize, FImageSize DestSubSize, 
				FImageSize SrcSize, FImageSize SrcSubSize,
				uint8* RESTRICT SubDest, 
				const uint8* RESTRICT SubSrc)
	{
		ImageResizeLinearByLessThanTwoSubImage<4>(OneOverScalingFactor, SrcSubPixelOffset, DestSize, DestSubSize, SrcSize, SrcSubSize, SubDest, SubSrc); 
	}

	void SubImageResizeLinearByLessThanTwoNoOp(FVector2f, FVector2f, FImageSize, FImageSize, FImageSize, FImageSize, uint8*, const uint8*)
	{
		checkf(false, TEXT("The format passed to SelectSubImageResizeFunc is not supported."));
	}

	SubImageResizeFuncType* SelectSubImageResizeFunc(EImageFormat Format)
	{
		switch (Format)
		{
			case EImageFormat::L_UByte    : return SubImageResizeLinearByLessThanTwoVec1_U8;
			case EImageFormat::RGB_UByte  : return SubImageResizeLinearByLessThanTwoVec3_U8;
			case EImageFormat::RGBA_UByte : return SubImageResizeLinearByLessThanTwoVec4_U8;
			case EImageFormat::BGRA_UByte : return SubImageResizeLinearByLessThanTwoVec4_U8;
			default: return SubImageResizeLinearByLessThanTwoNoOp;
		}
	}

} // namespace SubImageResize
} // namespace UE::Mutable::Private
