// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "MuR/ImageTypes.h"

namespace UE::Mutable::Private
{

namespace SubImageResize
{
	using SubImageResizeFuncType = void(FVector2f, FVector2f, FImageSize, FImageSize, FImageSize, FImageSize, uint8*, const uint8*); 

	SubImageResizeFuncType* SelectSubImageResizeFunc(EImageFormat Format);

	void SubImageResizeLinearByLessThanTwoVec1_U8(
				FVector2f OneOverScalingFactor, 
				FVector2f SrcSubPixelOffset,
				FImageSize DestSize, FImageSize DestSubSize, 
				FImageSize SrcSize, FImageSize SrcSubSize,
				uint8* RESTRICT SubDest, 
				const uint8* RESTRICT SubSrc);

	void SubImageResizeLinearByLessThanTwoVec3_U8(
				FVector2f OneOverScalingFactor, 
				FVector2f SrcSubPixelOffset,
				FImageSize DestSize, FImageSize DestSubSize, 
				FImageSize SrcSize, FImageSize SrcSubSize,
				uint8* RESTRICT SubDest, 
				const uint8* RESTRICT SubSrc);

	void SubImageResizeLinearByLessThanTwoVec4_U8(
				FVector2f OneOverScalingFactor, 
				FVector2f SrcSubPixelOffset,
				FImageSize DestSize, FImageSize DestSubSize, 
				FImageSize SrcSize, FImageSize SrcSubSize,
				uint8* RESTRICT SubDest, 
				const uint8* RESTRICT SubSrc);

} // namespace SubImageResize
} // namespace UE::Mutable::Private
