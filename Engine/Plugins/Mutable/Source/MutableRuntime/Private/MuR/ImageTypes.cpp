// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImageTypes.h"
#include "MuR/Serialisation.h"

namespace UE::Mutable::Private
{
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EBlendType);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EMipmapFilterType);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EAddressMode);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(ECompositeImageMode);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(ESamplingMethod);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EMinFilterMethod);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EImageFormat);

    static FImageFormatData s_imageFormatData[uint32(EImageFormat::Count)] =
    {
		FImageFormatData(0, 0, 0, 0),	// None
		FImageFormatData(1, 1, 3, 3),	// RGB_UByte
		FImageFormatData(1, 1, 4, 4),	// RGBA_UByte
		FImageFormatData(1, 1, 1, 1),	// IF_U_UBYTE

		FImageFormatData(0, 0, 0, 0),	// IF_PVRTC2 (deprecated)
        FImageFormatData(0, 0, 0, 0),	// IF_PVRTC4 (deprecated)
        FImageFormatData(0, 0, 0, 0),	// IF_ETC1 (deprecated)
        FImageFormatData(0, 0, 0, 0),	// IF_ETC2 (deprecated)

        FImageFormatData(0, 0, 0, 1),	// L_UByteRLE
        FImageFormatData(0, 0, 0, 3),	// RGB_UByteRLE
        FImageFormatData(0, 0, 0, 4),	// RGBA_UByteRLE
        FImageFormatData(0, 0, 0, 1),	// L_UBitRLE

        FImageFormatData(4, 4, 8,  4),	// BC1
        FImageFormatData(4, 4, 16, 4),	// BC2
        FImageFormatData(4, 4, 16, 4),	// BC3
        FImageFormatData(4, 4, 8,  1),	// BC4
        FImageFormatData(4, 4, 16, 2),	// BC5
        FImageFormatData(4, 4, 16, 3),	// BC6
        FImageFormatData(4, 4, 16, 4),	// BC7

        FImageFormatData(1, 1, 4, 4 ),		// BGRA_UByte

		FImageFormatData(4, 4, 16, 3, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 255, 255}), // ASTC_4x4_RGB_LDR
		FImageFormatData(4, 4, 16, 4, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 0, 0}),     // ASTC_4x4_RGBA_LDR
        FImageFormatData(4, 4, 16, 2),	// ASTC_4x4_RG_LDR // TODO: check black block for RG.
		
		FImageFormatData(8, 8, 16, 3, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 255, 255}),	// ASTC_8x8_RGB_LDR,
		FImageFormatData(8, 8, 16, 4, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 0, 0}),		// ASTC_8x8_RGBA_LDR,
		FImageFormatData(8, 8, 16, 2),		// ASTC_8x8_RG_LDR,
		FImageFormatData(12, 12, 16, 3, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 255, 255}),	// ASTC_12x12_RGB_LDR
		FImageFormatData(12, 12, 16, 4, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 0, 0}),		// ASTC_12x12_RGBA_LDR
		FImageFormatData(12, 12, 16, 2),	// ASTC_12x12_RG_LDR
		FImageFormatData(6, 6, 16, 3, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 255, 255}),	// ASTC_6x6_RGB_LDR,
		FImageFormatData(6, 6, 16, 4, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 0, 0}),		// ASTC_6x6_RGBA_LDR,
		FImageFormatData(6, 6, 16, 2),		// ASTC_6x6_RG_LDR,
		FImageFormatData(10, 10, 16, 3, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 255, 255}),	// ASTC_6x6_RGB_LDR,
		FImageFormatData(10, 10, 16, 4, {252, 253, 255, 255, 255, 255, 255, 255, 0, 0, 0, 0, 0, 0, 0, 0}),		// ASTC_6x6_RGBA_LDR,
		FImageFormatData(10, 10, 16, 2),	// ASTC_6x6_RG_LDR,
    };

    const FImageFormatData& GetImageFormatData( EImageFormat format )
    {
        check( format < EImageFormat::Count );
        return s_imageFormatData[ uint8(format) ];
    }

	void FMipmapGenerationSettings::Serialise(FOutputArchive& Arch) const
	{
		Arch << FilterType;
		Arch << AddressMode;
	}

	void FMipmapGenerationSettings::Unserialise(FInputArchive& Arch)
	{
		Arch >> FilterType;
		Arch >> AddressMode;
	}
}

