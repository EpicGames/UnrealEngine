// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Instance.h"

#include "HAL/LowLevelMemTracker.h"
#include "Misc/AssertionMacros.h"
#include "MuR/MutableMath.h"

namespace UE::Mutable::Private
{
    TManagedPtr<FInstance> FInstance::Clone() const
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		TManagedPtr<FInstance> Result = MakeManaged<FInstance>();

        *Result = *this;

        return Result;
    }
	
	
	int32 FInstance::GetDataSize() const
	{
		return 16 + sizeof(*this) + Components.GetAllocatedSize() + ExtensionData.GetAllocatedSize();
	}
	
    
    int32 FInstance::GetComponentCount() const
    {
		return Components.Num();
    }

	
	FComponentId FInstance::GetComponentId( int32 ComponentIndex ) const
	{
		if (Components.IsValidIndex(ComponentIndex))
		{
			return Components[ComponentIndex].Id;
		}
		else
		{
			check(false);
		}

		return INDEX_NONE;
	}
	
	
	FSkeletalMeshId FInstance::GetSkeletalMeshId(int32 ComponentIndex) const
	{
    	return Components[ComponentIndex].SkeletalMeshId;
	}

	
	FMaterialId FInstance::GetOverlayMaterialId(int32 ComponentIndex) const
	{
		return Components[ComponentIndex].OverlayMaterialId;
	}


	int32 FInstance::GetOverrideMaterialCount(int32 ComponentIndex) const
	{
    	return Components[ComponentIndex].OverrideMaterials.Num();
	}


	FName FInstance::GetOverrideMaterialSlotSlotName(int32 ComponentIndex, int32 MaterialIndex) const
	{
    	return Components[ComponentIndex].OverrideMaterials[MaterialIndex].SlotName;
	}
	
	
	FMaterialId FInstance::GetOverrideMaterialId(int32 ComponentIndex, int32 MaterialIndex) const
	{
    	return Components[ComponentIndex].OverrideMaterials[MaterialIndex].MaterialId;
	}

	
	int32 FInstance::GetOverlayMaterialCount(int32 ComponentIndex) const
	{
    	return Components[ComponentIndex].OverlayMaterials.Num();
	}

	
	FName FInstance::GetOverlayMaterialSlotSlotName(int32 ComponentIndex, int32 MaterialIndex) const
	{
    	return Components[ComponentIndex].OverlayMaterials[MaterialIndex].SlotName;
	}

	
	FMaterialId FInstance::GetOverlayMaterialId(int32 ComponentIndex, int32 MaterialIndex) const
	{
    	return Components[ComponentIndex].OverlayMaterials[MaterialIndex].MaterialId;
	}

    
	int32 FInstance::GetExtensionDataCount() const
	{
		return ExtensionData.Num();
	}
	
    
	int32 FInstance::AddComponent()
	{
		int32 result = Components.Emplace();
		return result;
	}
	
	
	void FInstance::SetSkeletalMeshId(int32 ComponentIndex, const FSkeletalMeshId& SkeletalMeshId)
    {
    	while (ComponentIndex >= Components.Num())
    	{
    		AddComponent();
    	}

    	Components[ComponentIndex].SkeletalMeshId = SkeletalMeshId;
    }


	void FInstance::SetOverlayMaterialId(int32 ComponentIndex, const FMaterialId& MaterialId)
	{
		while (ComponentIndex >= Components.Num())
		{
			AddComponent();
		}

		Components[ComponentIndex].OverlayMaterialId = MaterialId;
	}


	void FInstance::AddOverrideMaterial(int32 ComponentIndex, const FName& SlotName, const FMaterialId& MaterialId)
	{
    	while (ComponentIndex >= Components.Num())
    	{
    		AddComponent();
    	}

    	Components[ComponentIndex].OverrideMaterials.Add({ SlotName, MaterialId });
	}

	void FInstance::AddOverlayMaterial(int32 ComponentIndex, const FName& SlotName, const FMaterialId& MaterialId)
	{
    	while (ComponentIndex >= Components.Num())
    	{
    		AddComponent();
    	}

    	Components[ComponentIndex].OverlayMaterials.Add({ SlotName, MaterialId });
	}
}

