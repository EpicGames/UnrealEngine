// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Layout.h"

#include "HAL/LowLevelMemTracker.h"
#include "Math/IntPoint.h"
#include "MuR/MutableMath.h"
#include "MuR/Image.h"
#include "MuR/Serialisation.h"


namespace UE::Mutable::Private {

	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FLayoutBlock);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FLayoutBlock);


	void FLayout::Serialise( const FLayout* In, FOutputArchive& Arch )
	{
		Arch << *In;
	}


	TManagedRef<FLayout> FLayout::StaticUnserialise( FInputArchive& Arch )
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		TManagedRef<FLayout> Result = MakeManaged<FLayout>();
		Arch >> *Result;
		return Result;
	}


	TManagedPtr<FLayout> FLayout::Clone() const
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		TManagedPtr<FLayout> Result = MakeManaged<FLayout>();
		Result->Size = Size;
		Result->MaxSize = MaxSize;
		Result->Blocks = Blocks;
		Result->Strategy = Strategy;
		Result->ReductionMethod = ReductionMethod;
		Result->Masks = Masks;
		return Result;
	}


	bool FLayout::operator==( const FLayout& Other ) const
	{
		return (Size == Other.Size) &&
			(MaxSize == Other.MaxSize) &&
			(Blocks == Other.Blocks) &&
			(Strategy == Other.Strategy) &&
			// maybe this is not needed
			(ReductionMethod == Other.ReductionMethod);
	}


	int32 FLayout::GetDataSize() const
	{
		return sizeof(FLayout) + Blocks.GetAllocatedSize();
	}


	FIntPoint FLayout::GetGridSize() const
	{
		return FIntPoint(Size[0], Size[1]);
	}


	void FLayout::SetGridSize( int32 SizeX, int32 SizeY )
	{
		check( SizeX>=0 && SizeY>=0 );
		Size[0] = (uint16)SizeX;
		Size[1] = (uint16)SizeY;
	}


	void FLayout::GetMaxGridSize(int32* SizeX, int32* SizeY) const
	{
		check(SizeX && SizeY);

		if (SizeX && SizeY)
		{
			*SizeX = MaxSize[0];
			*SizeY = MaxSize[1];
		}
	}


	void FLayout::SetMaxGridSize(int32 SizeX, int32 SizeY)
	{
		check(SizeX >= 0 && SizeY >= 0);
		MaxSize[0] = (uint16)SizeX;
		MaxSize[1] = (uint16)SizeY;
	}


	int32 FLayout::GetBlockCount() const
	{
		return Blocks.Num();
	}


	void FLayout::SetBlockCount( int32 Count )
	{
		check(Count >=0 );
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		Blocks.SetNum(Count);
	}


	void FLayout::SetLayoutPackingStrategy(EPackStrategy InStrategy)
	{
		Strategy = InStrategy;
	}


	EPackStrategy FLayout::GetLayoutPackingStrategy() const
	{
		return Strategy;
	}


	void FLayout::Serialise(FOutputArchive& arch) const
	{
		arch << Size;
		arch << MaxSize;
		arch << uint32(Strategy);
		arch << uint32(ReductionMethod);
		arch << Blocks;
		arch << Masks;
	}

	
	void FLayout::Unserialise(FInputArchive& arch)
	{
		arch >> Size;
		arch >> MaxSize;

		uint32 Temp;
		arch >> Temp;
		Strategy = EPackStrategy(Temp);

		arch >> Temp;
		ReductionMethod = EReductionMethod(Temp);

		arch >> Blocks;
		arch >> Masks;
	}


	bool FLayout::IsSimilar(const FLayout& Other) const
	{
		if (Size != Other.Size || MaxSize != Other.MaxSize ||
			Blocks.Num() != Other.Blocks.Num() || Strategy != Other.Strategy)
			return false;

		for (int32 i = 0; i < Blocks.Num(); ++i)
		{
			if (!Blocks[i].IsSimilar(Other.Blocks[i])) return false;
		}

		return true;

	}


	int32 FLayout::FindBlock(uint64 Id) const
	{
		for (int32 i = 0; i < Blocks.Num(); ++i)
		{
			if (Blocks[i].Id == Id)
			{
				return i;
			}
		}

		return -1;
	}


	bool FLayout::IsSingleBlockAndFull() const
	{
		if (Blocks.Num() == 1
			&& Blocks[0].Min == FIntVector2(0, 0)
			&& Blocks[0].Size == Size)
		{
			return true;
		}
		return false;
	}

}

