// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Material.h"

#include "HAL/LowLevelMemTracker.h"
#include "MuR/MutableTrace.h"
#include "MuR/Serialisation.h"
#include "MuR/Serialisation.h"
#include "MuR/Image.h"


namespace UE::Mutable::Private
{
	void FMaterial::Serialise(const FMaterial* MaterialPtr, FOutputArchive& Arch)
	{
		Arch << *MaterialPtr;
	}


	TManagedPtr<FMaterial> FMaterial::StaticUnserialise(FInputArchive& Arch)
	{
		MUTABLE_CPUPROFILER_SCOPE(MaterialUnserialise)
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		TManagedPtr<FMaterial> Result = MakeManaged<FMaterial>();
		Arch >> *Result;

		return Result;
	}


	void FMaterial::Serialise(FOutputArchive& Arch) const
	{
		Arch << ImageParameters;
		Arch << ColorParameters;
		Arch << ScalarParameters;
	}


	void FMaterial::Unserialise(FInputArchive& Arch)
	{
		Arch >> ImageParameters;
		Arch >> ColorParameters;
		Arch >> ScalarParameters;
	}


	TManagedPtr<FMaterial> FMaterial::Clone() const
	{
		TManagedPtr<FMaterial> Result = MakeManaged<FMaterial>();

		Result->PassthroughObject = PassthroughObject;
		Result->ImageParameters = ImageParameters;
		Result->ColorParameters = ColorParameters;
		Result->ScalarParameters = ScalarParameters;

		return Result;
	}


	bool FMaterial::operator==(const FMaterial& Other) const
	{
		return PassthroughObject == Other.PassthroughObject &&
			ImageParameters == Other.ImageParameters &&
			ColorParameters == Other.ColorParameters &&
			ScalarParameters == Other.ScalarParameters;
	};


	void FMaterial::FImageParameterData::Serialise(FOutputArchive& arch) const
	{
		arch << ImageParameter;
		arch << ImagePropertyIndex;
	}


	void FMaterial::FImageParameterData::Unserialise(FInputArchive& arch)
	{
		arch >> ImageParameter;
		arch >> ImagePropertyIndex;
	}


	void FParameterKey::Serialise(FOutputArchive& arch) const
	{
		arch << ParameterName;
		arch << LayerIndex;
	}


	void FParameterKey::Unserialise(FInputArchive& arch)
	{
		arch >> ParameterName;
		arch >> LayerIndex;
	}
}
