// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/MemoryTrackingUtils.h"


#include "MuR/Platform.h"
#include "MuR/MutableRuntimeModule.h"
#include "Logging/LogMacros.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/IConsoleManager.h"

namespace UE::Mutable::Private::Private
{
	// Call LLM system tick (we do this to simulate a frame since the LLM system is not entirelly designed to run over a program)
	FORCEINLINE void UpdateLLMStats()
	{
		// This code will only be compiled if the global definition to enable LLM tracking is set to 1 for the host program
		// E.g.: GlobalDefinitions.Add("LLM_ENABLED_IN_CONFIG=1");
#if ENABLE_LOW_LEVEL_MEM_TRACKER && IS_PROGRAM
		FLowLevelMemTracker& MemTracker = FLowLevelMemTracker::Get();
		if (MemTracker.IsEnabled())
		{
			MemTracker.UpdateStatsPerFrame();
		}
#endif
	}
}

namespace UE::Mutable::Private
{

	struct FMemoryTrackingConsoleFlags
	{
		static inline bool bEnableGlobalMemoryTracking = true;
		static inline FAutoConsoleVariableRef CVarEnableGlobalMemoryTracking = FAutoConsoleVariableRef(
			TEXT("mutable.EnableGlobalMemoryTracking"),
			bEnableGlobalMemoryTracking,
			TEXT("Enables Mutable plugin global memory tracking, peak and total."),
			ECVF_Default);

#if ENABLE_LOW_LEVEL_MEM_TRACKER && IS_PROGRAM
		static inline bool bEnableUpdateLLMStats = false;
		static inline FAutoConsoleVariableRef CVarEnableUpdateLLMStats = FAutoConsoleVariableRef(
			TEXT("mutable.EnableUpdateLLMStats"),
			bEnableUpdateLLMStats,
			TEXT("Enables LLM updates at every tracked allocation, only enabled on program builds."),
			ECVF_Default);
#endif
	};

#if UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK
	void FGlobalMemoryCounter::Update(SSIZE_T Differential)
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return;
		}

		FScopeLock Lock(&Mutex);

		Counter += Differential;
		PeakValue = FMath::Max(PeakValue, Counter);

		AbsoluteCounter += Differential;
		AbsolutePeakValue = FMath::Max(AbsolutePeakValue, AbsoluteCounter);

#if ENABLE_LOW_LEVEL_MEM_TRACKER && IS_PROGRAM
		if (UNLIKELY(FMemoryTrackingConsoleFlags::bEnableUpdateLLMStats))
		{
			Private::UpdateLLMStats();
		}
#endif
	}

	void FGlobalMemoryCounter::Zero()
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return;
		}

		FScopeLock Lock(&Mutex);

		Counter = 0;	
		PeakValue = 0;
	}

	void FGlobalMemoryCounter::Restore()
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return;
		}

		FScopeLock Lock(&Mutex);

		Counter = AbsoluteCounter;
		PeakValue = AbsolutePeakValue;
	}

	SSIZE_T FGlobalMemoryCounter::GetPeak()
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return 0;
		}

		FScopeLock Lock(&Mutex);

		const volatile SSIZE_T Result = PeakValue;
		return Result;
	}

	SSIZE_T FGlobalMemoryCounter::GetCounter()
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return 0;
		}

		FScopeLock Lock(&Mutex);

		const volatile SSIZE_T Result = Counter;
		return Result;
	}

	SSIZE_T FGlobalMemoryCounter::GetAbsolutePeak()
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return 0;
		}

		FScopeLock Lock(&Mutex);

		const volatile SSIZE_T Result = AbsolutePeakValue;
		return Result;
	}

	SSIZE_T FGlobalMemoryCounter::GetAbsoluteCounter()
	{
		if (LIKELY(!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking))
		{
			return 0;
		}

		FScopeLock Lock(&Mutex);

		const volatile SSIZE_T Result = AbsoluteCounter;
		return Result;
	}

#else
	void FGlobalMemoryCounter::Zero()
	{
	}

	void FGlobalMemoryCounter::Restore()
	{
	}

	SSIZE_T FGlobalMemoryCounter::GetPeak()
	{
		return 0;
	}

	SSIZE_T FGlobalMemoryCounter::GetCounter()
	{
		return 0;
	}

	SSIZE_T FGlobalMemoryCounter::GetAbsolutePeak()
	{
		return 0;
	}

	SSIZE_T FGlobalMemoryCounter::GetAbsoluteCounter()
	{
		return 0;
	}

#endif


	struct FMemoryTrackingCommands
	{
		static inline FAutoConsoleCommand CmdDumpMutableGlobalMemoryCounter = FAutoConsoleCommand(
			TEXT("mutable.DumpMutableGlobalMemoryCounter"),
			TEXT("Dump Mutable plugin global tracked memory counter."),
			FConsoleCommandDelegate::CreateStatic([]()
			{
				if (!UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK)
				{
					UE_LOGF(LogMutableCore, Display, "The build does not have UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK enabled, no counting performed.");
					return;
				}

				if (!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking)
				{
					UE_LOGF(LogMutableCore, Display, "mutable.EnableGlobalMemoryTracking is set to false, no counting performed.");
					return;
				}

				UE_LOGF(LogMutableCore, 
					   Display, 
					   "Mutable Memory: Current %2.f MiB, Peak %2.f MiB", 
					   double(UE::Mutable::Private::FGlobalMemoryCounter::GetCounter()) / double(1024*1024),
					   double(UE::Mutable::Private::FGlobalMemoryCounter::GetPeak())    / double(1024*1024)
				);
			})
		);

		static inline FAutoConsoleCommand CmdDumpMutableGlobalMemoryAbsoluteCounter = FAutoConsoleCommand(
			TEXT("mutable.DumpMutableGlobalMemoryAbsoluteCounter"),
			TEXT("Dump Mutable plugin global tracked memory absolute counter."),
			FConsoleCommandDelegate::CreateStatic([]()
			{
				if (!UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK)
				{
					UE_LOGF(LogMutableCore, Display, "The build does not have UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK enabled, no counting performed.");
					return;
				}

				if (!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking)
				{
					UE_LOGF(LogMutableCore, Display, "mutable.EnableGlobalMemoryTracking is set to false, no counting performed.");
					return;
				}

				UE_LOGF(LogMutableCore, 
					   Display, 
					   "Mutable Memory: AbsoluteCurrent %2.f MiB, AbsolutePeak %2.f MiB", 
					   double(UE::Mutable::Private::FGlobalMemoryCounter::GetAbsoluteCounter()) / double(1024*1024),
					   double(UE::Mutable::Private::FGlobalMemoryCounter::GetAbsolutePeak())    / double(1024*1024)
				);
			})
		);


		static inline FAutoConsoleCommand CmdZeroMutableGlobalMemoryCounter = FAutoConsoleCommand(
			TEXT("mutable.ZeroMutableGlobalMemoryCounter"),
			TEXT("Zero Mutable plugin global tracked memory counter."),
			FConsoleCommandDelegate::CreateStatic([]()
			{
				if (!UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK)
				{
					UE_LOGF(LogMutableCore, Display, "The build does not have UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK enabled, no counting performed.");
					return;
				}

				if (!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking)
				{
					UE_LOGF(LogMutableCore, Display, "mutable.EnableGlobalMemoryTracking is set to false, no counting performed.");
					return;
				}

				FGlobalMemoryCounter::Zero();	
			})
		);

		static inline FAutoConsoleCommand CmdRestoreMutableGlobalMemoryCounter = FAutoConsoleCommand(
			TEXT("mutable.RestoreMutableGlobalMemoryCounter"),
			TEXT("Restore Mutable plugin global tracked memory counter."),
			FConsoleCommandDelegate::CreateStatic([]()
			{
				if (!UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK)
				{
					UE_LOGF(LogMutableCore, Display, "The build does not have UE_MUTABLE_TRACK_ALLOCATOR_MEMORY_PEAK enabled, no counting performed.");
					return;
				}

				if (!FMemoryTrackingConsoleFlags::bEnableGlobalMemoryTracking)
				{
					UE_LOGF(LogMutableCore, Display, "mutable.EnableGlobalMemoryTracking is set to false, no counting performed.");
					return;
				}

				FGlobalMemoryCounter::Restore();	
			})
		);
	};

}
