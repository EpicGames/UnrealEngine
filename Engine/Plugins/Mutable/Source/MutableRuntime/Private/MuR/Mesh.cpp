// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Mesh.h"

#include "Containers/UnrealString.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"
#include "MuR/Serialisation.h"
#include "MuR/OpMeshClipWithMesh.h"
#include "Spatial/PointHashGrid3.h"
#include "Engine/SkeletalMesh.h"

namespace UE::Mutable::Private
{

MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EBoneUsageFlags);
MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EMeshBufferType);
MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EShapeBindingMethod);
MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EVertexColorUsage);


void FMesh::Serialise(const FMesh* MeshPtr, FOutputArchive& Arch)
{
    //MeshPtr->m_pD->CheckIntegrity();
    Arch << *MeshPtr;
}


TManagedPtr<FMesh> FMesh::StaticUnserialise(FInputArchive& Arch)
{
    MUTABLE_CPUPROFILER_SCOPE(MeshUnserialise)
	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

    TManagedPtr<FMesh> Result = MakeManaged<FMesh>();
    Arch >> *Result;

    //Result->m_pD->CheckIntegrity();

    return Result;
}


TManagedRef<FMesh> FMesh::CreateAsReference(const TPassthroughObjectPtr<USkeletalMesh>& PasstroughObjectPtr)
{
	TManagedRef<FMesh> Result = MakeManaged<FMesh>();
	Result->PassthroughObject = PasstroughObjectPtr;
	
	return Result;
}


bool FMesh::IsReference() const
{
	return PassthroughObject.IsValid();
}
	

void FMesh::SetReferencedMorph(const FString& MorphName)
{
	ReferencedMorph = MorphName;
}


const FString& FMesh::GetReferencedMorph() const
{
	return ReferencedMorph;
}


TManagedPtr<FMesh> FMesh::Clone() const
{
    //MUTABLE_CPUPROFILER_SCOPE(MeshClone);
	return Clone(EMeshCopyFlags::AllFlags);
}


TManagedPtr<FMesh> FMesh::Clone(EMeshCopyFlags InFlags) const
{
    //MUTABLE_CPUPROFILER_SCOPE(MeshClone);
	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

	TManagedPtr<FMesh> Result = MakeManaged<FMesh>();

	Result->CopyFrom(*this, InFlags);

	return Result;
}


void FMesh::CopyFrom(const FMesh& From, EMeshCopyFlags InFlags)
{
    //MUTABLE_CPUPROFILER_SCOPE(CopyFrom);

    InternalId = From.InternalId;
	Flags = From.Flags;
	PassthroughObject = From.PassthroughObject;

	MeshIDPrefix = From.MeshIDPrefix;

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithSurfaces))
	{
		Surfaces = From.Surfaces;
		ClothSections.SetNum(From.Surfaces.Num());
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithSkeleton))
	{
		Skeleton = From.Skeleton;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithPhysicsBody))
	{
		PhysicsBody = From.PhysicsBody;
		PhysicsAssets = From.PhysicsAssets;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithGameplayTags))
	{
		GameplayTags = From.GameplayTags;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithAssetUserData))
	{
		AssetUserData = From.AssetUserData;
	}
	
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithAnimSlots))
	{
		AnimationSlots = From.AnimationSlots;
	}

    // Copy the main buffers
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithVertexBuffers))
    {
		VertexBuffers = From.VertexBuffers;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithIndexBuffers))
    {
		IndexBuffers = From.IndexBuffers;
	}

	// Copy additional buffers
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithSockets))
	{
		Sockets = From.Sockets;
	}

	// Copy additional buffers
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithAdditionalBuffers))
	{
		AdditionalBuffers = From.AdditionalBuffers;
	}

    // Copy the layout	
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithLayouts))
	{
		Layouts = From.Layouts;
	}
	
	// Copy bone poses
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithPoses))
	{
		BonePoses = From.BonePoses;
	}

	// Copy BoneMap
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithBoneMap))
	{
		BoneMap = From.BoneMap;
	}

	// Copy SkeletonIDs
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithSkeletonIDs))
	{
		SkeletonObjects = From.SkeletonObjects;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithAdditionalPhysics))
	{
		AdditionalPhysicsBodies = From.AdditionalPhysicsBodies;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithMorphMetaData))
	{
		Morph = From.Morph;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithMorphData))
	{
		MorphDataBuffer = From.MorphDataBuffer;	
	}
	
	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithClothData))
	{
		ClothSections = From.ClothSections;
	}

	if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithSkinWeightProfileData))
	{
		SkinWeightProfiles = From.SkinWeightProfiles;
	}
	else if (EnumHasAnyFlags(InFlags, EMeshCopyFlags::WithSkinWeightProfileMetaData))
	{
		const int32 NumSkinWeightProfiles = From.SkinWeightProfiles.Num();
		SkinWeightProfiles.SetNum(NumSkinWeightProfiles);

		for (int32 Index = 0; Index < NumSkinWeightProfiles; ++Index)
		{
			SkinWeightProfiles[Index].CopyMetadata(From.SkinWeightProfiles[Index]);
		}
	}
}


uint32 FMesh::GetId() const
{
    return InternalId;
}


bool FMesh::HasMorphs() const
{
	return Morph.NumTotalBatches != 0;
}


bool FMesh::HasClothing() const
{
	for (const FCloth& Cloth : ClothSections)
	{
		if (Cloth.IsValid())
		{
			return true;
		}
	}
	
	return false;
}


int FMesh::GetVertexCount() const
{
    return GetVertexBuffers().GetElementCount();
}


FMeshBufferSet& FMesh::GetVertexBuffers()
{
    return VertexBuffers;
}


const FMeshBufferSet& FMesh::GetVertexBuffers() const
{
    return VertexBuffers;
}


bool FMesh::AreVertexIdsImplicit() const
{
	// Is there a buffer for vertex ids?
	int32 BufferIndex = -1;
	int32 ChannelIndex = -1;
	VertexBuffers.FindChannel(EMeshBufferSemantic::VertexIndex, 0, &BufferIndex, &ChannelIndex);

	return (MeshIDPrefix != 0) && (BufferIndex < 0) && (ChannelIndex < 0);
}


bool FMesh::AreVertexIdsExplicit() const
{
	// Is there a buffer for vertex ids?
	int32 BufferIndex = -1;
	int32 ChannelIndex = -1;
	VertexBuffers.FindChannel(EMeshBufferSemantic::VertexIndex, 0, &BufferIndex, &ChannelIndex);

	bool bExplicit = 
			(BufferIndex >= 0) && (ChannelIndex >= 0) && 
			(VertexBuffers.Buffers[BufferIndex].Channels[ChannelIndex].Format == EMeshBufferFormat::UInt64);

	if (bExplicit)
	{
		check(MeshIDPrefix == 0);
	}

	return bExplicit;
}


void FMesh::MakeVertexIdsRelative()
{
	check(AreVertexIdsImplicit());

	int32 NewBuffer = VertexBuffers.GetBufferCount();
	VertexBuffers.SetBufferCount(NewBuffer + 1);
	EMeshBufferSemantic Semantic = EMeshBufferSemantic::VertexIndex;
	int32 SemanticIndex = 0;
	EMeshBufferFormat Format = EMeshBufferFormat::UInt32;
	int32 Components = 1;
	int32 Offset = 0;
	VertexBuffers.SetBuffer(NewBuffer, sizeof(uint32), 1, &Semantic, &SemanticIndex, &Format, &Components, &Offset);
	uint32* IdDataPtr = reinterpret_cast<uint32*>(VertexBuffers.GetBufferData(NewBuffer));

	int32 VertexCount = GetVertexCount();
	for (int32 Index = 0; Index < VertexCount; ++Index)
	{
		(*IdDataPtr++) = Index;
	}
}


void FMesh::MakeIdsExplicit()
{
	MUTABLE_CPUPROFILER_SCOPE(Mesh_MakeIdsExplicit);

	int32 VertexCount = GetVertexCount();
	check(VertexCount == 0);

	// Vertex IDs
	{
		bool bHasRelativeVertexIndices = false;

		int32 OldBufferIndex = -1;
		int32 OldChannelIndex = -1;
		VertexBuffers.FindChannel(EMeshBufferSemantic::VertexIndex, 0, &OldBufferIndex, &OldChannelIndex);
		bool bHasVertexIndices = (OldBufferIndex >= 0 && OldChannelIndex >= 0);
		if (bHasVertexIndices)
		{
			check(OldChannelIndex == 0 && VertexBuffers.Buffers[OldBufferIndex].Channels.Num() == 1);

			FMeshBuffer& Buffer = VertexBuffers.Buffers[OldBufferIndex];
			Buffer.Channels[0].Format = EMeshBufferFormat::UInt64;
			Buffer.ElementSize = sizeof(uint64);
		}

		else
		{
			// The mesh has implicit Ids
			// Create a new buffer with explicit ids
			FMeshBuffer& Buffer = VertexBuffers.Buffers.Emplace_GetRef();
			Buffer.ElementSize = sizeof(uint64);
			FMeshBufferChannel& Channel = Buffer.Channels.Emplace_GetRef();
			Channel.Semantic = EMeshBufferSemantic::VertexIndex;
			Channel.SemanticIndex= 0;
			Channel.Format = EMeshBufferFormat::UInt64;
			Channel.ComponentCount = 1;
			Channel.Offset = 0;
		}
	}

	// Layout block IDs
	{
		for (FMeshBuffer& Buffer : VertexBuffers.Buffers)
		{
			for (FMeshBufferChannel& Channel : Buffer.Channels)
			{
				if (Channel.Semantic != EMeshBufferSemantic::LayoutBlock)
				{
					continue;
				}

				check(Buffer.Channels.Num() == 1);
				check(Buffer.Channels[0].Offset == 0);

				Buffer.Channels[0].Format = EMeshBufferFormat::UInt64;
				Buffer.ElementSize = sizeof(uint64);
			}
		}
	}

	// Final cleanup
	MeshIDPrefix = 0;
}


TManagedPtr<const FSkeleton> FMesh::GetSkeleton() const
{
    return Skeleton;
}


void FMesh::SetSkeleton(const TManagedPtr<const FSkeleton>& InSkeleton)
{
    Skeleton = InSkeleton;
}


void FMesh::ConvertBoneIdsToIndices()
{
	if (ensure(Skeleton))
	{
		// Convert BoneMaps' BoneIds to BoneIndices
		const int32 NumBonesBoneMap = BoneMap.Num();
		for (int32 Index = 0; Index < NumBonesBoneMap; ++Index)
		{
			BoneMap[Index].Index = Skeleton->BoneIds.Find(BoneMap[Index].Id);
		}

		// Convert BonePoses' BoneIds to BoneIndices
		const int32 NumBones = Skeleton->GetNumBones();
		const int32 NumBonePoses = BonePoses.Num();

		int32 AuxIndex = 0;
		for (int32 Index = 0; Index < NumBonePoses; ++Index)
		{
			FBoneIdOrIndex& BoneId = BonePoses[Index].BoneId;

			// Bone Poses are sorted to match the skeleton hierarchy. 
			for (; AuxIndex < NumBones; ++AuxIndex)
			{
				if (Skeleton->BoneIds[AuxIndex] == BoneId.Id)
				{
					BoneId.Index = AuxIndex;
					break;
				}
			}

			check(Skeleton->BoneIds.IsValidIndex(BoneId.Index));
		}
	}
}


TManagedPtr<const FPhysicsBody> FMesh::GetPhysicsBody() const
{
    return PhysicsBody;
}


void FMesh::SetPhysicsBody(const TManagedPtr<const FPhysicsBody>& InPhysicsBody)
{
    PhysicsBody = InPhysicsBody;
}


int32 FMesh::AddAdditionalPhysicsBody(const TManagedPtr<const FPhysicsBody>& Body)
{
	return AdditionalPhysicsBodies.Add(Body);
}


TManagedPtr<const FPhysicsBody> FMesh::GetAdditionalPhysicsBody(int32 Index) const
{
	check(AdditionalPhysicsBodies.IsValidIndex(Index));

	return AdditionalPhysicsBodies[Index];
}


int32 FMesh::GetFaceCount() const
{
    return GetIndexBuffers().GetElementCount() / 3;
}


int32 FMesh::GetIndexCount() const
{
    return GetIndexBuffers().GetElementCount();
}


FMeshBufferSet& FMesh::GetIndexBuffers()
{
    return IndexBuffers;
}


const FMeshBufferSet& FMesh::GetIndexBuffers() const
{
    return IndexBuffers;
}


int32 FMesh::GetSurfaceCount() const
{
    return Surfaces.Num();
}


void FMesh::GetSurface(
		int32 SurfaceIndex,
        int32& OutFirstVertex, int32& OutVertexCount,
    	int32& OutFirstIndex, int32& OutIndexCount,
		int32& OutBoneIndex, int32& OutBoneCount) const
{
    int32 Count = GetSurfaceCount();

    if (SurfaceIndex >= 0 && SurfaceIndex < Count)
    {
        if (SurfaceIndex < Surfaces.Num())
        {
            const FMeshSurface& Surf = Surfaces[SurfaceIndex];

			check(Surf.SubMeshes.Num());

			// Surfaces submeshes are sorted and have no gaps. 
            OutFirstVertex = Surf.SubMeshes[0].VertexBegin;
            OutVertexCount = Surf.SubMeshes.Last().VertexEnd - OutFirstVertex;
            OutFirstIndex = Surf.SubMeshes[0].IndexBegin;
            OutIndexCount = Surf.SubMeshes.Last().IndexEnd - OutFirstIndex;
            OutBoneIndex = Surf.BoneMapIndex;
            OutBoneCount = Surf.BoneMapCount;
        }
        else if (!Surfaces.Num())
        {
            // No surfaces defined, means only one surface using all the mesh
            OutFirstVertex = 0;
            OutVertexCount = GetVertexCount();
            OutFirstIndex = 0;
            OutIndexCount = GetIndexCount();
			OutBoneIndex = 0;
			OutBoneCount = BoneMap.Num();
        }
		else
		{
			check(false);
		}
    }
    else
    {
        check(false);
        OutFirstVertex = 0;
        OutVertexCount = 0;
        OutFirstIndex = 0;
        OutIndexCount = 0;
		OutBoneIndex = 0;
		OutBoneCount = 0;
    }
}


uint32 FMesh::GetSurfaceId(int32 SurfaceIndex) const
{
    if (SurfaceIndex >= 0 && SurfaceIndex < Surfaces.Num())
    {
        const FMeshSurface& Surf = Surfaces[SurfaceIndex];
        return Surf.Id;
    }

    return 0;
}


void FMesh::AddLayout(TManagedPtr<const FLayout> InLayout)
{
	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
    Layouts.Add(InLayout);
}


int32 FMesh::GetLayoutCount() const
{
    return Layouts.Num();
}


TManagedPtr<const FLayout> FMesh::GetLayout(int32 LayoutIndex) const
{
    check(Layouts.IsValidIndex(LayoutIndex));

    return Layouts[LayoutIndex];
}


void FMesh::SetLayout(int32 LayoutIndex, TManagedPtr<const FLayout> InLayout)
{
    check(Layouts.IsValidIndex(LayoutIndex));

    Layouts[LayoutIndex] = InLayout;
}


int32 FMesh::FindBonePoseByBoneIndex(int32 Index) const
{
	return BonePoses.IndexOfByPredicate([Index](const FBonePose& Pose) { return Pose.BoneId.Index == Index; });
}


void UE::Mutable::Private::FMesh::SetBonePoseCount(int32 count)
{
	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
	BonePoses.SetNum(count);
}


int32 UE::Mutable::Private::FMesh::GetBonePoseCount() const
{
	return BonePoses.Num();
}


void UE::Mutable::Private::FMesh::SetBonePose(int32 Index, FBoneIdOrIndex BoneId, FTransform3f Transform, EBoneUsageFlags BoneUsageFlags, int8 Priority)
{
	check(BonePoses.IsValidIndex(Index));
	if (BonePoses.IsValidIndex(Index))
	{
		BonePoses[Index] = FBonePose{ BoneId, BoneUsageFlags, Priority, Transform };
	}
}


FBoneIdOrIndex FMesh::GetBonePoseId(int32 Index) const
{
	check(BonePoses.IsValidIndex(Index));
	return BonePoses[Index].BoneId;
}


void UE::Mutable::Private::FMesh::GetBonePoseTransform(int32 BoneIndex, FTransform3f& Transform) const
{
	check(BoneIndex >= 0 && BoneIndex < BonePoses.Num());
	Transform = BoneIndex > INDEX_NONE ? BonePoses[BoneIndex].BoneTransform : FTransform3f::Identity;
}


EBoneUsageFlags FMesh::GetBoneUsageFlags(int32 BoneIndex) const
{
	check(BoneIndex >= 0 && BoneIndex < BonePoses.Num());
	return BoneIndex > INDEX_NONE ? BonePoses[BoneIndex].BoneUsageFlags : EBoneUsageFlags::None;
}


void FMesh::SetBoneMap(const TArray<FBoneIdOrIndex>& InBoneMap)
{
	BoneMap = InBoneMap;
}


const TArray<FBoneIdOrIndex>& FMesh::GetBoneMap() const
{
	return BoneMap;
}


int32 FMesh::GetDataSize() const
{
	// TODO: review if other mesh fields like additional physics assets
	// are relevant and add them to the count.

	// Should be allocation sizes used for this?
	int32 AdditionalBuffersSize = 0;
	for (const TPair<EMeshBufferType, FMeshBufferSet>&  AdditionalBuffer : AdditionalBuffers)
	{
		AdditionalBuffersSize += AdditionalBuffer.Value.GetDataSize();
	}

	for (const FCloth& ClothSection : ClothSections)
	{
		AdditionalBuffersSize += ClothSection.GetDataSize();
	}
	
	return sizeof(FMesh)
		+ IndexBuffers.GetDataSize()
		+ VertexBuffers.GetDataSize()
		+ BonePoses.Num() * sizeof(FBonePose)
		+ AdditionalBuffersSize;
}


bool FMesh::HasCompatibleFormat(const FMesh* Other) const
{
    bool bCompatible = true;

    bCompatible = Layouts.Num() == Other->Layouts.Num();
    bCompatible = bCompatible && VertexBuffers.GetBufferCount() == Other->VertexBuffers.GetBufferCount();

    // Indices
    //-----------------
    if (IndexBuffers.GetElementCount() > 0 && Other->GetIndexCount() > 0)
    {
        check(IndexBuffers.Buffers.Num() == 1);
        check(Other->GetIndexBuffers().Buffers.Num() == 1);
        check(IndexBuffers.GetBufferChannelCount(0) == 1);
        check(Other->GetIndexBuffers().GetBufferChannelCount(0) == 1);

        const FMeshBuffer& Dest = IndexBuffers.Buffers[0];
        const FMeshBuffer& Source = Other->GetIndexBuffers().Buffers[0];

        bCompatible = bCompatible && Dest.Channels[0].Format == Source.Channels[0].Format;
    }


    // Layouts
    //-----------------
    // TODO?


    // Vertices
    //-----------------
	const int32 NumVertexBuffers =  VertexBuffers.GetBufferCount();
    for (int32 VertexBufferIndex = 0; VertexBufferIndex < NumVertexBuffers; ++VertexBufferIndex)
    {
        const FMeshBuffer& Dest = VertexBuffers.Buffers[VertexBufferIndex];
        const FMeshBuffer& Source = Other->GetVertexBuffers().Buffers[VertexBufferIndex];

        // TODO: More checks about channels formats and semantics
        //bCompatible = bCompatible && GetVertexBufferElementSize(VertexBufferIndex) == Other->GetVertexBufferElementSize(VertexBufferIndex);
        bCompatible = bCompatible && Dest.Channels.Num() == Source.Channels.Num();
    }

    return bCompatible;
}


UE::Math::TIntVector3<uint32> FMesh::GetFaceVertexIndices(int32 FaceIndex) const
{
	UE::Math::TIntVector3<uint32> Result;

    MeshBufferIteratorConst<EMeshBufferFormat::UInt32, uint32, 1> Iter(IndexBuffers, EMeshBufferSemantic::VertexIndex);
    Iter += FaceIndex*3;

    Result[0] = (*Iter)[0];
    ++Iter;

    Result[1] = (*Iter)[0];
    ++Iter;

    Result[2] = (*Iter)[0];
    ++Iter;

    return Result;
}


bool FMesh::FVertexMatchMap::DoMatch(int32 Vertex, int32 OtherVertex) const
{
	if (Vertex >= 0 && Vertex < FirstMatch.Num())
	{
		int32 Start = FirstMatch[Vertex];
		int32 End = Vertex + 1 < FirstMatch.Num() ? FirstMatch[Vertex + 1] : Matches.Num();

		bool bResult = false;
		while (!bResult && Start < End)
		{
			if (Matches[Start] == OtherVertex)
			{
				bResult = true;
			}

			++Start;
		}

		return bResult;
	}

	return false;
}


void FMesh::GetVertexMap(const FMesh& Other, FVertexMatchMap& VertexMap, float Tolerance) const
{
    int32 VertexCount = VertexBuffers.GetElementCount();
    VertexMap.FirstMatch.SetNum(VertexCount);
    VertexMap.Matches.SetNum(VertexCount + (VertexCount >> 2));

    int32 OtherVertexCount = Other.VertexBuffers.GetElementCount();

    if (!VertexCount || !OtherVertexCount)
    {
        return;
    }


    MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> ItPosition(VertexBuffers, EMeshBufferSemantic::Position);
    MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> ItOtherPositionBegin(Other.VertexBuffers, EMeshBufferSemantic::Position);


    // Bucket the other mesh
#define MUTABLE_NUM_BUCKETS 256
#define MUTABLE_BUCKET_CHANNEL 0

    float RangeMin = TNumericLimits<float>::Max();
    float RangeMax = -TNumericLimits<float>::Max();
    MeshBufferIteratorConst< EMeshBufferFormat::Float32, float, 3 >  ItOtherPosition = ItOtherPositionBegin;
    
	for (int32 OtherVertex = 0; OtherVertex < OtherVertexCount; ++OtherVertex)
    {
        float V = (*ItOtherPosition)[MUTABLE_BUCKET_CHANNEL];
        RangeMin = FMath::Min(RangeMin, V);
        RangeMax = FMath::Max(RangeMax, V);
        ++ItOtherPosition;
    }
    RangeMin -= Tolerance;
    RangeMax += Tolerance;

    TArray<int32> Buckets[MUTABLE_NUM_BUCKETS];
    for (int32 BucketIndex = 0; BucketIndex < MUTABLE_NUM_BUCKETS; ++BucketIndex)
    {
        Buckets[BucketIndex].Reserve(OtherVertexCount/MUTABLE_NUM_BUCKETS*2);
    }

    float BucketSize = (RangeMax-RangeMin)/float(MUTABLE_NUM_BUCKETS);
    ItOtherPosition = ItOtherPositionBegin;
    for (int32 OtherVertex = 0; OtherVertex < OtherVertexCount; ++OtherVertex)
    {
        float V = (*ItOtherPosition)[MUTABLE_BUCKET_CHANNEL];

        int32 Bucket0 = FMath::FloorToInt((V-Tolerance-RangeMin)/BucketSize);
        Bucket0 = FMath::Min(MUTABLE_NUM_BUCKETS-1, FMath::Max(0, Bucket0));
        Buckets[Bucket0].Add(OtherVertex);

        int32 Bucket1 = FMath::FloorToInt((V + Tolerance - RangeMin)/BucketSize);
        Bucket1 = FMath::Min(MUTABLE_NUM_BUCKETS-1, FMath::Max(0, Bucket1));

        if (Bucket1 != Bucket0)
        {
            Buckets[Bucket1].Add(OtherVertex);
        }

        ++ItOtherPosition;
    }

    // TODO Compare only positions?

    // Use buckets
    for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
    {
        VertexMap.FirstMatch[VertexIndex] = VertexMap.Matches.Num();

        float VBucket = (*ItPosition)[MUTABLE_BUCKET_CHANNEL];
        int32 Bucket = FMath::FloorToInt((VBucket - RangeMin)/BucketSize);

        if (Bucket >= 0 && Bucket < MUTABLE_NUM_BUCKETS)
        {
            int32 BucketVertexCount = Buckets[Bucket].Num();
            for (int32 OtherVertex = 0; OtherVertex < BucketVertexCount; ++OtherVertex)
            {
                int32 OtherVertexIndex = Buckets[Bucket][OtherVertex];
                FVector3f Position = (ItOtherPositionBegin + OtherVertexIndex).GetAsVec3f();

                bool bSame = true;
                for (int32 Dim = 0; bSame && Dim < 3; ++Dim)
                {
                    float Diff = FMath::Abs((*ItPosition)[Dim] - Position[Dim]);
                    bSame = Diff <= Tolerance;
                }

                if (bSame)
                {
                    VertexMap.Matches.Add(OtherVertexIndex);
                }
            }
        }

        ++ItPosition;
    }

}


void FMesh::EnsureSurfaceData()
{
	if (!Surfaces.Num() && VertexBuffers.GetElementCount())
	{
		FMeshSurface& NewSurface = Surfaces.Emplace_GetRef();
		ClothSections.AddDefaulted();

		FSurfaceSubMesh& SubMesh = NewSurface.SubMeshes.Emplace_GetRef();
		SubMesh.VertexBegin = 0;
		SubMesh.VertexEnd = VertexBuffers.GetElementCount();
		SubMesh.IndexBegin = 0;
		SubMesh.IndexEnd = IndexBuffers.GetElementCount();

		NewSurface.BoneMapCount = BoneMap.Num();
	}
}


void FMesh::ResetBufferIndices()
{
	VertexBuffers.ResetBufferIndices();
	IndexBuffers.ResetBufferIndices();
}

MUTABLE_IMPLEMENT_POD_SERIALISABLE(FSurfaceSubMesh);
MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FSurfaceSubMesh);

MUTABLE_IMPLEMENT_POD_SERIALISABLE(EMorphUsageFlags);
MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(EMorphUsageFlags);

void FMeshSurface::Serialise(FOutputArchive& Arch) const
{
	Arch << SubMeshes;

	Arch << BoneMapIndex;
	Arch << BoneMapCount;
	Arch << Id;
	Arch << bCastShadow;
	Arch << bRecomputeTangent;
}


void FMeshSurface::Unserialise(FInputArchive& Arch)
{
	Arch >> SubMeshes;

	Arch >> BoneMapIndex;
	Arch >> BoneMapCount;
	Arch >> Id;
	Arch >> bCastShadow;
	Arch >> bRecomputeTangent;
}


void FMeshMorph::Serialise(FOutputArchive& Arch) const
{
	Arch << Names;
	Arch << MaximumValuePerMorph;
	Arch << MinimumValuePerMorph;
	Arch << BatchStartOffsetPerMorph;
	Arch << BatchesPerMorph;

	Arch << SurfacesInUsePerMorph;
	Arch << UsageFlagsPerMorph;

	Arch << NumTotalBatches;
	Arch << PositionPrecision;
	Arch << TangentZPrecision;
}


void FMeshMorph::Unserialise(FInputArchive& Arch)
{
	Arch >> Names;
	Arch >> MaximumValuePerMorph;
	Arch >> MinimumValuePerMorph;
	Arch >> BatchStartOffsetPerMorph;
	Arch >> BatchesPerMorph;

	Arch >> SurfacesInUsePerMorph;
	Arch >> UsageFlagsPerMorph;

	Arch >> NumTotalBatches;
	Arch >> PositionPrecision;
	Arch >> TangentZPrecision;
}


void FSkinWeightProfile::CopyMetadata(const FSkinWeightProfile& From)
{
	Name = From.Name;
	bDefaultProfile = From.bDefaultProfile;
	DefaultProfileFromLODIndex = From.DefaultProfileFromLODIndex;
	NumBoneInfluences = From.NumBoneInfluences;
	bUse16BitBoneIndex = From.bUse16BitBoneIndex;
	bUse16BitBoneWeight = From.bUse16BitBoneWeight;
}


void FSkinWeightProfile::Serialise(FOutputArchive& Arch) const
{
	Arch << Name;
	Arch << bDefaultProfile;
	Arch << DefaultProfileFromLODIndex;

	Arch << NumBoneInfluences;
	Arch << bUse16BitBoneIndex;
	Arch << bUse16BitBoneWeight;

	Arch << BoneIDs;
	Arch << BoneWeights;
	Arch << VertexIndexToInfluenceOffset;
}


void FSkinWeightProfile::Unserialise(FInputArchive& Arch)
{
	Arch >> Name;
	Arch >> bDefaultProfile;
	Arch >> DefaultProfileFromLODIndex;

	Arch >> NumBoneInfluences;
	Arch >> bUse16BitBoneIndex;
	Arch >> bUse16BitBoneWeight;

	Arch >> BoneIDs;
	Arch >> BoneWeights;
	Arch >> VertexIndexToInfluenceOffset;
}

MUTABLE_IMPLEMENT_POD_SERIALISABLE(FSkinWeightProfile::FVertexInfo);
MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FSkinWeightProfile::FVertexInfo);

void FMeshSocket::Serialise(FOutputArchive& Arch) const
{
	Arch << SocketName;
	Arch << BoneName;
	Arch << RelativeLocation;
	Arch << RelativeRotation;
	Arch << RelativeScale;
	Arch << bForceAlwaysAnimated;
}


void FMeshSocket::Unserialise(FInputArchive& Arch)
{
	Arch >> SocketName;
	Arch >> BoneName;
	Arch >> RelativeLocation;
	Arch >> RelativeRotation;
	Arch >> RelativeScale;
	Arch >> bForceAlwaysAnimated;
}


void FMesh::FBonePose::Serialise(FOutputArchive& Arch) const
{
	Arch << BoneId;
	Arch << BoneUsageFlags;
	Arch << BoneTransform;
}


void FMesh::FBonePose::Unserialise(FInputArchive& Arch)
{
	Arch >> BoneId;
	Arch >> BoneUsageFlags;
	Arch >> BoneTransform;
}


void FMesh::Serialise(FOutputArchive& Arch) const
{
	check(!IsReference());
	check(ClothSections.Num() <= 1);
	
	Arch << IndexBuffers;
	Arch << VertexBuffers;
	Arch << AdditionalBuffers;
	Arch << Layouts;

	Arch << PhysicsBody;

	Arch << uint32(Flags);
	Arch << Surfaces;

	Arch << BonePoses;
	Arch << BoneMap;

	Arch << AdditionalPhysicsBodies;

	Arch << MeshIDPrefix;
	
	Arch << ClothSections;

	Arch << Morph;
	Arch << MorphDataBuffer;

	Arch << SkinWeightProfiles;
}


void FMesh::Unserialise(FInputArchive& Arch)
{
	Arch >> IndexBuffers;
	Arch >> VertexBuffers;
	Arch >> AdditionalBuffers;
	Arch >> Layouts;

	Arch >> PhysicsBody;

	uint32 Temp;
	Arch >> Temp;
	Flags = static_cast<EMeshFlags>(Temp);

	Arch >> Surfaces;

	Arch >> BonePoses;
	Arch >> BoneMap;

	Arch >> AdditionalPhysicsBodies;

	Arch >> MeshIDPrefix;
	
	Arch >> ClothSections;

	Arch >> Morph;
	Arch >> MorphDataBuffer;
	
	Arch >> SkinWeightProfiles;
}


bool FMesh::IsSimilar(const FMesh& Other) const
{
	// Some meshes are just vertex indices (masks) we don't consider them for similarity,
	// because the kind of vertex channel data they store is the kind that is ignored.
	if (IndexBuffers.GetElementCount() == 0)
	{
		return false;
	}

	bool bEqual = IndexBuffers == Other.IndexBuffers;
	bEqual = bEqual && (PassthroughObject == Other.PassthroughObject);

	bEqual = bEqual && (SkeletonObjects == Other.SkeletonObjects);

	if (bEqual && Skeleton != Other.Skeleton)
	{
		if (Skeleton && Other.Skeleton)
		{
			bEqual = (*Skeleton == *Other.Skeleton);
		}
		else
		{
			bEqual = false;
		}
	}
	
	if (bEqual && PhysicsBody != Other.PhysicsBody)
	{
		if (PhysicsBody && Other.PhysicsBody)
		{
			bEqual = (*PhysicsBody == *Other.PhysicsBody);
		}
		else
		{
			bEqual = false;
		}
	}

	bEqual = bEqual && (Surfaces == Other.Surfaces);

	bEqual = bEqual && (Morph == Other.Morph);
	bEqual = bEqual && (MorphDataBuffer == Other.MorphDataBuffer);

	// Special comparison for vertex buffers
	if (bEqual)
	{
		bEqual = VertexBuffers.IsSimilarRobust(Other.VertexBuffers, false);
	}
	
	bEqual = bEqual && (ClothSections == Other.ClothSections);

	bEqual = bEqual && (SkinWeightProfiles == Other.SkinWeightProfiles);

	return bEqual;
}


void FMesh::CheckIntegrity() const
{
#ifdef MUTABLE_DEBUG

	{
		int32 BufferIndex = -1;
		int32 ChannelIndex = -1;
		VertexBuffers.FindChannel(EMeshBufferSemantic::VertexIndex, 0, &BufferIndex, &ChannelIndex);
		if (BufferIndex >= 0 && ChannelIndex >= 0)
		{
			EMeshBufferFormat IdFormat = VertexBuffers.m_buffers[BufferIndex].m_channels[ChannelIndex].m_format;
			if (IdFormat == EMeshBufferFormat::UInt64)
			{
				check(MeshIDPrefix==0);
			}
			else if (IdFormat == EMeshBufferFormat::UInt32)
			{
				check(MeshIDPrefix != 0);
			}
			else
			{
				check(false);
			}
		}
	}

    // Check vertex indices
    {
        for ( int32 b=0; b<IndexBuffers.GetBufferCount(); ++b )
        {
            int32 elemSize = IndexBuffers.GetElementSize( b );

            for ( int32 c=0; c<IndexBuffers.GetBufferChannelCount(b); ++c )
            {
                EMeshBufferSemantic semantic;
                int32 semanticIndex = 0;
                EMeshBufferFormat format;
                int32 components;
                int32 offset = 0;
                IndexBuffers.GetChannel( b, c, &semantic, &semanticIndex, &format, &components, &offset );

                if ( semantic==EMeshBufferSemantic::VertexIndex )
                {
                    int32 icount = IndexBuffers.GetElementCount();
                    int32 elemCount = VertexBuffers.GetElementCount();
                    for ( int32 indexIndex = 0; indexIndex < icount; ++indexIndex )
                    {
                        const uint8* pData = IndexBuffers.GetBufferData( b ) + elemSize*indexIndex + offset;

                        switch (format)
                        {
                        case EMeshBufferFormat::UInt32:
                        {
                            uint32 index = *(const uint32*)pData;
                            check( index < uint32( elemCount ) );
                            break;
                        }
                        case EMeshBufferFormat::UInt16:
                        {
                            uint16 index = *(const uint16*)pData;
                            check( index < uint16( elemCount ) );
                            break;
                        }
                        case EMeshBufferFormat::UInt8:
                        {
                            uint8 index = *(const uint8*)pData;
                            check( index < uint8( elemCount ) );
                            break;
                        }
                        default:
                            check(false);
                            break;
                        }
                    }
                }
            }
        }
    }


    // Check bone indices, if there are bones. Bones could have been removed for later addition as an optimisation.
    // For all the attributes in this mesh
    int32 boneCount = Skeleton ? Skeleton->GetBoneCount() : 0;
    if ( Skeleton && boneCount )
    {
        for ( int32 b = 0; b < VertexBuffers.GetBufferCount(); ++b )
        {
            int32 channelCount = VertexBuffers.GetBufferChannelCount( b );
            for ( int32 c = 0; c < channelCount; ++c )
            {
                EMeshBufferSemantic semantic;
                int32 semanticIndex = 0;
                EMeshBufferFormat format;
                int32 components;
                int32 offset = 0;
                VertexBuffers.GetChannel( b, c, &semantic, &semanticIndex, &format, &components, &offset );

                // If it is not one of the relevant semantics
                if (
                        //semantic!=EMeshBufferSemantic::Position &&
                        //semantic!=EMeshBufferSemantic::TexCoords &&
                        //semantic!=EMeshBufferSemantic::Normal &&
                        //semantic!=EMeshBufferSemantic::Tangent &&
                        //semantic!=EMeshBufferSemantic::Binormal &&
                        semantic!=EMeshBufferSemantic::BoneIndices
                        // && semantic!=EMeshBufferSemantic::BoneWeights
                        )
                {
                    continue;
                }

                int32 elemCount = VertexBuffers.GetElementCount();
                int32 elemSize = VertexBuffers.GetElementSize( b );

                for (int32 vertexIndex = 0; vertexIndex < elemCount; ++vertexIndex )
                {
                    const uint8* pData = VertexBuffers.GetBufferData( b ) + elemSize*vertexIndex + offset;

                    switch (format)
                    {
                    case EMeshBufferFormat::UInt8:
                    {
                        for ( int32 d = 0; d < components; ++d )
                        {
                            uint8 index = *pData;
                            check( index < uint64(boneCount) );
                            ++pData;
                        }
                        break;
                    }

                    case EMeshBufferFormat::UInt16:
                    {
                        for ( int32 d = 0; d < components; ++d )
                        {
                            uint16 index = *(uint16*)pData;
                            check( index < uint64( boneCount ) );
                            pData+=2;
                        }
                        break;
                    }

                    case EMeshBufferFormat::UInt32:
                    {
                        for ( int32 d = 0; d < components; ++d )
                        {
                            uint32 index = *(uint32*)pData;
                            check( index < uint64( boneCount ) );
                            pData+=4;
                        }
                        break;
                    }

                    default:
                        check( false );
                    }
                }
            }
        }
    }
#endif
}


bool FMesh::IsClosed() const
{
	return IsMeshClosed(this);
}


static bool StaticMeshFormatIdentify_Project(const FMesh* InMesh)
{
    // This format is used internally for the mesh project
    bool bResult = true;

    // The first vertex buffer must be texcoords(2f), position(3f), normal(3f)
    // all tightly packed
    bResult &= InMesh->VertexBuffers.GetBufferCount() >= 1;

    if (bResult)
    {
        bResult &= InMesh->VertexBuffers.Buffers[0].Channels.Num() == 3;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[0].Channels[0];

        bResult &= Channel.Semantic == EMeshBufferSemantic::TexCoords;
        bResult &= Channel.Format == EMeshBufferFormat::Float32;
        bResult &= Channel.ComponentCount == 2;
        //we don't really care about the semantic index
        //bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 0;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[0].Channels[1];

        bResult &= Channel.Semantic == EMeshBufferSemantic::Position;
        bResult &= Channel.Format == EMeshBufferFormat::Float32;
        bResult &= Channel.ComponentCount == 3;
        bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 8;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[0].Channels[2];

        bResult &= Channel.Semantic == EMeshBufferSemantic::Normal;
        bResult &= Channel.Format == EMeshBufferFormat::Float32;
        bResult &= Channel.ComponentCount == 3;
        bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 20;
    }

    // The first index buffer must be just index buffers u32
	if (!InMesh->IndexBuffers.Buffers.IsEmpty())
	{
		if (bResult)
		{
			bResult &= InMesh->IndexBuffers.Buffers[0].Channels.Num() >= 1;
		}

		if (bResult)
		{
			const FMeshBufferChannel& Channel = InMesh->IndexBuffers.Buffers[0].Channels[0];

			bResult &= Channel.Semantic == EMeshBufferSemantic::VertexIndex;
			bResult &= Channel.Format == EMeshBufferFormat::UInt32;
			bResult &= Channel.ComponentCount == 1;
			bResult &= Channel.SemanticIndex == 0;
			bResult &= Channel.Offset == 0;
		}
	}
	else
	{
		bResult = false;
	}

    return bResult;
}


static bool StaticMeshFormatIdentify_ProjectWrapping(const FMesh* InMesh)
{
    // This format is used internally for the mesh project
    bool bResult = true;

    // The first vertex buffer must be texcoords(2f), position(3f), normal(3f), layoutBlock(uint32_t)
    // all tightly packed
    bResult &= InMesh->VertexBuffers.GetBufferCount() >= 2;

    if (bResult)
    {
        bResult &= InMesh->VertexBuffers.Buffers[0].Channels.Num() == 3;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[0].Channels[0];

        bResult &= Channel.Semantic == EMeshBufferSemantic::TexCoords;
        bResult &= Channel.Format == EMeshBufferFormat::Float32;
        bResult &= Channel.ComponentCount == 2;
        // we don't really care about the semantic index as long as there is only one
        // bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 0;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[0].Channels[1];

        bResult &= Channel.Semantic == EMeshBufferSemantic::Position;
        bResult &= Channel.Format == EMeshBufferFormat::Float32;
        bResult &= Channel.ComponentCount == 3;
        bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 8;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[0].Channels[2];

        bResult &= Channel.Semantic == EMeshBufferSemantic::Normal;
        bResult &= Channel.Format == EMeshBufferFormat::Float32;
        bResult &= Channel.ComponentCount == 3;
        bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 20;
    }

	// Block IDs
    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->VertexBuffers.Buffers[1].Channels[0];

        bResult &= Channel.Semantic == EMeshBufferSemantic::LayoutBlock;
		// We don't care about the layout block id format. We need to support them all.
        //bResult &= Channel.Format == EMeshBufferFormat::UInt64;
        bResult &= Channel.ComponentCount == 1;
        // we don't really care about the semantic index as long as there is only one
        // bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 0;
    }

    // The first index buffer must be just index buffers u32
    if (bResult)
    {
        bResult &= InMesh->IndexBuffers.Buffers[0].Channels.Num() >= 1;
    }

    if (bResult)
    {
        const FMeshBufferChannel& Channel = InMesh->IndexBuffers.Buffers[0].Channels[0];

        bResult &= Channel.Semantic == EMeshBufferSemantic::VertexIndex;
        bResult &= Channel.Format == EMeshBufferFormat::UInt32;
        bResult &= Channel.ComponentCount == 1;
        bResult &= Channel.SemanticIndex == 0;
        bResult &= Channel.Offset == 0;
    }

    return bResult;
}


void FMesh::ResetStaticFormatFlags() const
{
	EnumRemoveFlags(Flags, EMeshFlags::ProjectFormat);
	EnumRemoveFlags(Flags, EMeshFlags::ProjectWrappingFormat);

	if (StaticMeshFormatIdentify_Project(this))
	{
		EnumAddFlags(Flags, EMeshFlags::ProjectFormat);
	}

	if (StaticMeshFormatIdentify_ProjectWrapping(this))
	{
		EnumAddFlags(Flags, EMeshFlags::ProjectWrappingFormat);
	}
}

namespace
{
    void LogBuffer(FString& Out, const FMeshBufferSet& BufferSet, int32 BufferElementLimit)
    {
		uint32 ElemCount = BufferSet.ElementCount;
        Out += FString::Printf(TEXT("  Set with %d buffers and %d elements\n"), BufferSet.Buffers.Num(), ElemCount);

        for(const FMeshBuffer& Buffer : BufferSet.Buffers)
        {
            Out += FString::Printf(TEXT("    Buffer with %d channels and %d elementsize\n"), Buffer.Channels.Num(), Buffer.ElementSize);
            
			const uint8* DataPtr = Buffer.Data.GetData();
			if (!DataPtr)
			{
				continue;
			}

            for (const FMeshBufferChannel& Channel : Buffer.Channels)
            {
				Out += FString::Printf(TEXT("      Channel with format: %d semantic: %d %d, components: %d, offset: %d\n"),
							EnumToUnderlyingType(Channel.Format), EnumToUnderlyingType(Channel.Semantic), Channel.SemanticIndex, Channel.ComponentCount, Channel.Offset);
                
				for (int32 ElemIndex = 0; uint32(ElemIndex) < ElemCount && ElemIndex < BufferElementLimit; ++ElemIndex)
                {
                    Out += "        ";
                    
                    const uint8* ChanDataPtr = DataPtr + Buffer.ElementSize*ElemIndex + Channel.Offset;
					for (uint32 CompIndex = 0; CompIndex < Channel.ComponentCount; ++CompIndex)
                    {
                        Out += "\t";
                        
						switch (Channel.Format)
                        {
                        case EMeshBufferFormat::UInt32:
                        case EMeshBufferFormat::NUInt32:
						{
							Out += FString::Printf(TEXT("%d"), *(const uint32*)ChanDataPtr); 
							ChanDataPtr += 4; 
							break;
						}
                        case EMeshBufferFormat::UInt16:
                        case EMeshBufferFormat::NUInt16: 
						{
							Out += FString::Printf(TEXT("%d"), *(const uint16*)ChanDataPtr); 
							ChanDataPtr += 2; 
							break;
						}
                        case EMeshBufferFormat::UInt8:
                        case EMeshBufferFormat::NUInt8: 
						{
							Out += FString::Printf(TEXT("%d"), *(const uint8*)ChanDataPtr); 
							ChanDataPtr += 1; 
							break;
						}
						case EMeshBufferFormat::Float32: 
						{
							Out += FString::Printf(TEXT("%.3f"), *(const float*)ChanDataPtr); 
							ChanDataPtr += 4; 
							break;
						}
                        case EMeshBufferFormat::Float16: 
						{
							float Converted = *((const FFloat16*)ChanDataPtr);
							Out += FString::Printf(TEXT("%.3f"), Converted); 
							ChanDataPtr += 2; 
							break;
						}
                        default: 
							break;
                        }

                        Out += ",";
                    }
                    Out += "\n";
                }
            }
        }
    }
}


void FMesh::Log( FString& out, int32 BufferElementLimit) const
{
    out += "Mesh:\n";

    out += "Indices:\n";
    LogBuffer( out, IndexBuffers, BufferElementLimit);

    out += "Vertices:\n";
    LogBuffer( out, VertexBuffers, BufferElementLimit);
}


void GetUVIsland(TArray<FTriangleInfo>& InTriangles,
	const uint32 InFirstTriangle,
	TArray<uint32>& OutTriangleIndices,
	const TArray<FVector2f>& InUVs,
	const TMultiMap<int32, uint32>& InVertexToTriangleMap)
{
	MUTABLE_CPUPROFILER_SCOPE(GetUVIsland);

	const uint32 NumTriangles = (uint32)InTriangles.Num();

	OutTriangleIndices.Reserve(NumTriangles);
	OutTriangleIndices.Add(InFirstTriangle);

	TArray<bool> SkipTriangles;
	SkipTriangles.Init(false, NumTriangles);

	TArray<uint32> PendingTriangles;
	PendingTriangles.Reserve(NumTriangles / 64);
	PendingTriangles.Add(InFirstTriangle);

	while (!PendingTriangles.IsEmpty())
	{
		const uint32 TriangleIndex = PendingTriangles.Pop();

		// Triangle about to be proccessed, mark as skip;
		SkipTriangles[TriangleIndex] = true;

		bool ConnectedEdges[3] = { false, false, false };

		const FTriangleInfo& Triangle = InTriangles[TriangleIndex];

		// Find Triangles connected to edges 0 and 2
		int32 CollapsedVertex1 = Triangle.CollapsedIndices[1];
		int32 CollapsedVertex2 = Triangle.CollapsedIndices[2];

		TArray<uint32> FoundTriangleIndices;
		InVertexToTriangleMap.MultiFind(Triangle.CollapsedIndices[0], FoundTriangleIndices);

		for (uint32 OtherTriangleIndex : FoundTriangleIndices)
		{
			const FTriangleInfo& OtherTriangle = InTriangles[OtherTriangleIndex];

			for (int32 OtherIndex = 0; OtherIndex < 3; ++OtherIndex)
			{
				const int32 OtherCollapsedIndex = OtherTriangle.CollapsedIndices[OtherIndex];
				if (OtherCollapsedIndex == CollapsedVertex1)
				{
					// Check if the vertex is in the same UV Island 
					if (!SkipTriangles[OtherTriangleIndex]
						&& InUVs[Triangle.Indices[1]].Equals(InUVs[OtherTriangle.Indices[OtherIndex]], 0.00001f))
					{
						OutTriangleIndices.Add(OtherTriangleIndex);
						PendingTriangles.Add(OtherTriangleIndex);
						SkipTriangles[OtherTriangleIndex] = true;
					}

					// Connected but already processed or in another island
					break;
				}

				if (OtherCollapsedIndex == CollapsedVertex2)
				{
					// Check if the vertex is in the same UV Island 
					if (!SkipTriangles[OtherTriangleIndex]
						&& InUVs[Triangle.Indices[2]].Equals(InUVs[OtherTriangle.Indices[OtherIndex]], 0.00001f))
					{
						OutTriangleIndices.Add(OtherTriangleIndex);
						PendingTriangles.Add(OtherTriangleIndex);
						SkipTriangles[OtherTriangleIndex] = true;
					}

					// Connected but already processed or in another UV Island
					break;
				}
			}

		}

		// Find the triangle connected to edge 1
		FoundTriangleIndices.Reset();
		InVertexToTriangleMap.MultiFind(CollapsedVertex1, FoundTriangleIndices);

		for (uint32 OtherTriangleIndex : FoundTriangleIndices)
		{
			const FTriangleInfo& OtherTriangle = InTriangles[OtherTriangleIndex];

			for (int32 OtherIndex = 0; OtherIndex < 3; ++OtherIndex)
			{
				const int32 OtherCollapsedIndex = OtherTriangle.CollapsedIndices[OtherIndex];
				if (OtherCollapsedIndex == CollapsedVertex2)
				{
					// Check if the vertex belong to the same UV island
					if (!SkipTriangles[OtherTriangleIndex]
						&& InUVs[Triangle.Indices[2]].Equals(InUVs[OtherTriangle.Indices[OtherIndex]], 0.00001f))
					{
						OutTriangleIndices.Add(OtherTriangleIndex);
						PendingTriangles.Add(OtherTriangleIndex);
						SkipTriangles[OtherTriangleIndex] = true;
					}

					// Connected but already processed or in another island
					break;
				}
			}
		}
	}
}


void MeshCreateCollapsedVertexMap(const UE::Mutable::Private::FMesh* Mesh, TArray<int32>& CollapsedVertices)
{
	MUTABLE_CPUPROFILER_SCOPE(LayoutUV_CreateCollapsedVertexMap);

	const int32 NumVertices = Mesh->GetVertexCount();
	CollapsedVertices.Reserve(NumVertices);

	UE::Geometry::TPointHashGrid3f<int32> VertHash(0.01f, INDEX_NONE);
	VertHash.Reserve(NumVertices);

	TArray<FVector3f> Vertices;
	Vertices.SetNumUninitialized(NumVertices);

	UE::Mutable::Private::UntypedMeshBufferIteratorConst ItPosition = UE::Mutable::Private::UntypedMeshBufferIteratorConst(Mesh->GetVertexBuffers(), UE::Mutable::Private::EMeshBufferSemantic::Position);

	FVector3f* VertexData = Vertices.GetData();

	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		*VertexData = ItPosition.GetAsVec3f();
		VertHash.InsertPointUnsafe(VertexIndex, *VertexData);

		++ItPosition;
		++VertexData;
	}

	// Find unique vertices
	CollapsedVertices.Init(INDEX_NONE, NumVertices);

	TArray<int32> NearbyVertices;
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		if (CollapsedVertices[VertexIndex] != INDEX_NONE)
		{
			continue;
		}

		const FVector3f& Vertex = Vertices[VertexIndex];

		NearbyVertices.Reset();
		VertHash.FindPointsInBall(Vertex, 0.00001f,
			[&Vertex, &Vertices](const int32& Other) -> float {return FVector3f::DistSquared(Vertices[Other], Vertex); },
			NearbyVertices);

		// Find equals
		for (int32 NearbyVertexIndex : NearbyVertices)
		{
			CollapsedVertices[NearbyVertexIndex] = VertexIndex;
		}
	}
}

	
}
