// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/MeshBufferSet.h"

#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/MutableTrace.h"
#include "MuR/Serialisation.h"


namespace UE::Mutable::Private::MemoryCounters
{
	std::atomic<SSIZE_T>& FMeshMemoryCounter::Get()
	{
		static std::atomic<SSIZE_T> Counter{0};
		return Counter;
	}
}

namespace UE::Mutable::Private
{
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FMeshBufferChannel);
	MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FMeshBufferChannel);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EMeshBufferFormat);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EMeshBufferSemantic);
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EMeshBufferSetFlags);

	
	static FMeshBufferFormatData s_meshBufferFormatData[] = // EMeshBufferFormat::Count entries
	{
		{ 0, 0 },
		{ 2, 0 },
		{ 4, 0 },

		{ 1, 8 },
		{ 2, 16 },
		{ 4, 32 },
		{ 1, 7 },
		{ 2, 15 },
		{ 4, 31 },

		{ 1, 0 },
		{ 2, 0 },
		{ 4, 0 },
		{ 1, 0 },
		{ 2, 0 },
		{ 4, 0 },

		{ 1, 0 },
		{ 1, 0 },
		{ 1, 0 },
		{ 1, 0 },

		{ 8, 0 },

		{ 8, 64 },
		{ 8, 63 },
		{ 8, 0 },
		{ 8, 0 }
	};

	static_assert(sizeof(s_meshBufferFormatData) / sizeof(FMeshBufferFormatData) == int32(EMeshBufferFormat::Count));

	const FMeshBufferFormatData& GetMeshFormatData(EMeshBufferFormat Format)
	{
		check(uint32(Format) >= 0);
		check(uint32(Format) < uint32(EMeshBufferFormat::Count));
		return s_meshBufferFormatData[uint32(Format)];
	}

	void FMeshBufferSet::Serialise(FOutputArchive& Arch) const
	{
		Arch << ElementCount;
		Arch << Buffers;
		Arch << Flags;
	}

	void FMeshBufferSet::Unserialise(FInputArchive& Arch)
	{
		Arch >> ElementCount;
		Arch >> Buffers;
		Arch >> Flags;
	}


	int32 FMeshBufferSet::GetElementCount() const
	{
		return ElementCount;
	}

	void FMeshBufferSet::SetElementCount(int32 Count, EMemoryInitPolicy MemoryInitPolicy)
	{
		check(Count >= 0);
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		const bool bAllocateMemory = !IsDescriptor();
		if (bAllocateMemory)
		{
			// If the new size is 0, allow shrinking.
			// TODO: Add better shrink policy or let the user decide. Denying shrinking
			// unconditionally could mean having small meshes that use lots of memory. For now
			// allow it if no other allocation will be done.  
			
			const EAllowShrinking AllowShrinking = (Count == 0) ? EAllowShrinking::Yes : EAllowShrinking::No;
			for (FMeshBuffer& Buffer : Buffers)
			{
				if (MemoryInitPolicy == EMemoryInitPolicy::Uninitialized)
				{
					Buffer.Data.SetNumUninitialized(Buffer.ElementSize * Count, AllowShrinking);
				}
				else if (MemoryInitPolicy == EMemoryInitPolicy::Zeroed)
				{
					Buffer.Data.SetNumZeroed(Buffer.ElementSize * Count, AllowShrinking);
				}
				else
				{
					check(false);
				}
			}
		}
		
		ElementCount = Count;
	}


	int32 FMeshBufferSet::GetBufferCount() const
	{
		return Buffers.Num();
	}

	void FMeshBufferSet::SetBufferCount(int32 Count)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		Buffers.SetNum(Count);
	}

	int32 FMeshBufferSet::GetBufferChannelCount(int32 BufferIndex) const
	{
		check(Buffers.IsValidIndex(BufferIndex));
		
		if (Buffers.IsValidIndex(BufferIndex))
		{
			return Buffers[BufferIndex].Channels.Num();
		}

		return 0;
	}

	void FMeshBufferSet::GetChannel(
			int32 BufferIndex,
			int32 ChannelIndex,
			EMeshBufferSemantic* SemanticPtr,
			int32* SemanticIndexPtr,
			EMeshBufferFormat* FormatPtr,
			int32* ComponentCountPtr,
			int32* OffsetPtr) const
	{
		check(Buffers.IsValidIndex(BufferIndex));
		check(Buffers[BufferIndex].Channels.IsValidIndex(ChannelIndex));

		const FMeshBufferChannel& Channel = Buffers[BufferIndex].Channels[ChannelIndex];

		if (SemanticPtr)
		{
			*SemanticPtr = Channel.Semantic;
		}

		if (SemanticIndexPtr)
		{
			*SemanticIndexPtr = Channel.SemanticIndex;
		}

		if (FormatPtr)
		{
			*FormatPtr = Channel.Format;
		}

		if (ComponentCountPtr)
		{
			*ComponentCountPtr = Channel.ComponentCount;
		}

		if (OffsetPtr)
		{
			*OffsetPtr = Channel.Offset;
		}
	}


	void FMeshBufferSet::SetBuffer(
		int32 BufferIndex,
		int32 ElementSize,
		int32 ChannelCount,
		const EMeshBufferSemantic* SemanticsPtr,
		const int32* SemanticIndicesPtr,
		const EMeshBufferFormat* FormatsPtr,
		const int32* ComponentCountPtr,
		const int32* OffsetsPtr,
		EMemoryInitPolicy MemoryInitPolicy)
	{
		check(Buffers.IsValidIndex(BufferIndex));
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		FMeshBuffer& Buffer = Buffers[BufferIndex];


		int32 MinElemSize = 0;
		Buffer.Channels.SetNum(ChannelCount);
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
		{
			FMeshBufferChannel& Channel = Buffer.Channels[ChannelIndex];
			Channel.Semantic = SemanticsPtr ? SemanticsPtr[ChannelIndex] : EMeshBufferSemantic::None;
			Channel.SemanticIndex = SemanticIndicesPtr ? SemanticIndicesPtr[ChannelIndex] : 0;
			Channel.Format = FormatsPtr ? FormatsPtr[ChannelIndex] : EMeshBufferFormat::None;
			Channel.ComponentCount = ComponentCountPtr ? ((uint16)ComponentCountPtr[ChannelIndex]) : 0;
			Channel.Offset = OffsetsPtr ? ((uint8)OffsetsPtr[ChannelIndex]) : 0;

			int32 ThisChannelMinElemSize = Channel.Offset + Channel.ComponentCount * GetMeshFormatData(Channel.Format).SizeInBytes;			
			MinElemSize = FMath::Max(MinElemSize, ThisChannelMinElemSize);
		}

		// Set the user specified element size, or enlarge it if it was too small
		Buffer.ElementSize = FMath::Max(ElementSize, MinElemSize);

		bool bAllocateMemory = !IsDescriptor(); 

		// Update the buffer data
		if (bAllocateMemory)
		{
			if (MemoryInitPolicy == EMemoryInitPolicy::Uninitialized)
			{
				Buffer.Data.SetNumUninitialized(Buffer.ElementSize * ElementCount, EAllowShrinking::No);
			}
			else if (MemoryInitPolicy == EMemoryInitPolicy::Zeroed)
			{
				Buffer.Data.SetNumZeroed(Buffer.ElementSize * ElementCount, EAllowShrinking::No);
			}
			else
			{
				check(false);
			}
		}
	}

	void FMeshBufferSet::SetBufferChannel(
		int32 BufferIndex,
		int32 ChannelIndex,
		EMeshBufferSemantic Semantic,
		int32 SemanticIndex,
		EMeshBufferFormat Format,
		int32 ComponentCount,
		int32 Offset)
	{
		if (!Buffers.IsValidIndex(BufferIndex))
		{
			check(false);
			return;
		}

		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		FMeshBuffer& Buffer = Buffers[BufferIndex];

		if (!Buffer.Channels.IsValidIndex(ChannelIndex))
		{
			check(false);
			return;
		}

		FMeshBufferChannel& Channel = Buffer.Channels[ChannelIndex];
		Channel.Semantic = Semantic;
		Channel.SemanticIndex = SemanticIndex;
		Channel.Format = Format;
		Channel.ComponentCount = uint16(ComponentCount);
		Channel.Offset = uint8(Offset);
	}


	uint8* FMeshBufferSet::GetBufferData(int32 BufferIndex)
	{
		check(!IsDescriptor());

		check(Buffers.IsValidIndex(BufferIndex));
		uint8* ResultPtr = Buffers[BufferIndex].Data.GetData();
		return ResultPtr;
	}

	const uint8* FMeshBufferSet::GetBufferData(int32 BufferIndex) const
	{
		check(Buffers.IsValidIndex(BufferIndex));
		const uint8* ResultPtr = Buffers[BufferIndex].Data.GetData();
		return ResultPtr;
	}

	uint32 FMeshBufferSet::GetBufferDataSize(int32 BufferIndex) const
	{
		check(Buffers.IsValidIndex(BufferIndex));
		const uint32 Result = Buffers[BufferIndex].Data.Num();

#if WITH_EDITOR
		int32 Size = Buffers[BufferIndex].ElementSize * ElementCount;
		ensure(Size == Result);
#endif

		return Result;
	}

	void FMeshBufferSet::FindChannel(EMeshBufferSemantic Semantic, int32 SemanticIndex, int32* BufferPtr, int32* ChannelPtr) const
	{
		check(BufferPtr && ChannelPtr);

		*BufferPtr = -1;
		*ChannelPtr = -1;

		const int32 NumBuffers = Buffers.Num();
		for (int32 BufferIndex = 0; BufferIndex < NumBuffers; ++BufferIndex)
		{
			const int32 NumChannels = Buffers[BufferIndex].Channels.Num();
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				if (Buffers[BufferIndex].Channels[ChannelIndex].Semantic == Semantic &&
					Buffers[BufferIndex].Channels[ChannelIndex].SemanticIndex == SemanticIndex)
				{
					*BufferPtr = BufferIndex;
					*ChannelPtr = ChannelIndex;

					return;
				}
			}
		}
	}

	int32 FMeshBufferSet::GetElementSize(int32 BufferIndex) const
	{
		check(Buffers.IsValidIndex(BufferIndex));

		return Buffers[BufferIndex].ElementSize;
	}

	int32 FMeshBufferSet::GetChannelOffset(int32 BufferIndex, int32 ChannelIndex) const
	{
		check(Buffers.IsValidIndex(BufferIndex));
		check(Buffers[BufferIndex].Channels.IsValidIndex(ChannelIndex));

		return Buffers[BufferIndex].Channels[ChannelIndex].Offset;
	}

	void FMeshBufferSet::AddBuffer(const FMeshBufferSet& Other, int32 BufferIndex)
	{
		check(GetElementCount() == Other.GetElementCount());

		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		Buffers.Add(Other.Buffers[BufferIndex]);
	}

	void FMeshBufferSet::RemoveBuffer(int32 BufferIndex)
	{
		if (Buffers.IsValidIndex(BufferIndex))
		{
			Buffers.RemoveAt(BufferIndex);
		}
		else
		{
			check(false);
		}
	}

	bool FMeshBufferSet::HasSameFormat(const FMeshBufferSet& Other) const
	{
		const int32 BufferCount = GetBufferCount();
		if (Other.GetBufferCount() != BufferCount)
		{
			return false;
		}

		for (int32 BufferIndex = 0; BufferIndex < BufferCount; ++BufferIndex)
		{
			if (!HasSameFormat(BufferIndex, Other, BufferIndex))
			{
				return false;
			}
		}

		return true;
	}

	bool FMeshBufferSet::HasSameFormat(int32 ThisBufferIndex, const FMeshBufferSet& Other, int32 OtherBufferIndex) const
	{
		return Buffers[ThisBufferIndex].HasSameFormat(Other.Buffers[OtherBufferIndex]);
	}

	int32 FMeshBufferSet::GetDataSize() const
	{
		int32 Result = 0;

		const bool bIsDescriptor = IsDescriptor();

		const int32 NumBuffers = Buffers.Num();
		for (int32 BufferIndex = 0; BufferIndex < NumBuffers; ++BufferIndex)
		{
			Result += sizeof(FMeshBufferChannel) * Buffers[BufferIndex].Channels.Num();

			if (!bIsDescriptor)
			{
				Result += Buffers[BufferIndex].ElementSize * ElementCount;
			}
		}

		return Result;
	}

	int32 FMeshBufferSet::GetAllocatedSize() const
	{
		int32 ByteCount = 0;

		const int32 NumBuffers = Buffers.Num();
		for (int32 BufferIndex = 0; BufferIndex < NumBuffers; ++BufferIndex)
		{
			ByteCount += Buffers[BufferIndex].Data.GetAllocatedSize();
		}

		return ByteCount;
	}

	void FMeshBufferSet::CopyElement(uint32 FromIndex, uint32 ToIndex)
	{
		check(!IsDescriptor())
		check(FromIndex < ElementCount);
		check(ToIndex < ElementCount);

		if (FromIndex != ToIndex)
		{
			for (FMeshBuffer& Buffer : Buffers)
			{
				FMemory::Memcpy(
						&Buffer.Data[Buffer.ElementSize * ToIndex],
						&Buffer.Data[Buffer.ElementSize * FromIndex],
						Buffer.ElementSize);
			}
		}
	}

	bool FMeshBufferSet::IsSpecialBufferToIgnoreInSimilar(const FMeshBuffer& Buffer) const
	{
		if (Buffer.Channels.Num() == 1 && Buffer.Channels[0].Semantic == EMeshBufferSemantic::VertexIndex)
		{
			return true;
		}

		if (Buffer.Channels.Num() == 1 && Buffer.Channels[0].Semantic == EMeshBufferSemantic::LayoutBlock)
		{
			return true;
		}

		return false;
	}

	bool FMeshBufferSet::IsSimilarRobust(const FMeshBufferSet& Other, bool bCompareUVs) const
	{
		MUTABLE_CPUPROFILER_SCOPE(FMeshBufferSet::IsSimilarRobust);

		if (ElementCount != Other.ElementCount)
		{
			return false;
		}

		// IsSimilar is mush faster but can give false negatives if the buffer data description 
		// omits parts of the data (e.g, memory layout paddings). It cannot have false positives. 
		if (IsSimilar(Other))
		{
			return true;
		}

		const int32 ThisNumBuffers = Buffers.Num();
		const int32 OtherNumBuffers = Other.Buffers.Num();

		int32 I = 0, J = 0;
		while (I < ThisNumBuffers && J < OtherNumBuffers)
		{
			if (IsSpecialBufferToIgnoreInSimilar(Buffers[I]))
			{
				++I;
				continue;
			}

			if (IsSpecialBufferToIgnoreInSimilar(Other.Buffers[J]))
			{
				++J;
				continue;
			}

			const int32 ThisNumChannels = Buffers[I].Channels.Num();
			const int32 OtherNumChannels = Buffers[J].Channels.Num();

			const FMeshBuffer& ThisBuffer = Buffers[I];
			const FMeshBuffer& OtherBuffer = Other.Buffers[J];

			if (!(ThisBuffer.Channels == OtherBuffer.Channels && ThisBuffer.ElementSize == OtherBuffer.ElementSize))
			{
				return false;
			}

			bool bBufferHasTexCoords = ThisBuffer.HasSemantic(EMeshBufferSemantic::TexCoords);
			bool bCanBeFullCompared = !ThisBuffer.HasPadding() && (!bBufferHasTexCoords || bCompareUVs);
			if (bCanBeFullCompared)
			{
				MUTABLE_CPUPROFILER_SCOPE(FastCompare);

				// This buffer can be directly compared
				if (FMemory::Memcmp(ThisBuffer.Data.GetData(), OtherBuffer.Data.GetData(), ThisBuffer.Data.Num()) != 0)
				{
					return false;
				}
			}
			else
			{
				MUTABLE_CPUPROFILER_SCOPE(SlowCompare);

				for (uint32 Elem = 0; Elem < ElementCount; ++Elem)
				{
					for (int32 C = 0; C < ThisNumChannels; ++C)
					{
						if (!bCompareUVs && ThisBuffer.Channels[C].Semantic == EMeshBufferSemantic::TexCoords)
						{
							continue;
						}

						const SIZE_T SizeA = GetMeshFormatData(ThisBuffer.Channels[C].Format).SizeInBytes * ThisBuffer.Channels[C].ComponentCount;
						const SIZE_T SizeB = GetMeshFormatData(OtherBuffer.Channels[C].Format).SizeInBytes * OtherBuffer.Channels[C].ComponentCount;
						check(SizeA == SizeB);

						const uint8* BuffA = ThisBuffer.Data.GetData() + Elem * ThisBuffer.ElementSize + ThisBuffer.Channels[C].Offset;
						const uint8* BuffB = OtherBuffer.Data.GetData() + Elem * OtherBuffer.ElementSize + OtherBuffer.Channels[C].Offset;

						if (FMemory::Memcmp(BuffA, BuffB, SizeA) != 0)
						{
							return false;
						}
					}
				}
			}

			++J;
			++I;
		}

		// Whatever buffers are left should be irrelevant
		while (I < ThisNumBuffers)
		{
			if (!IsSpecialBufferToIgnoreInSimilar(Buffers[I]))
			{
				return false;
			}
			++I;
		}

		while (J < OtherNumBuffers)
		{
			if (!IsSpecialBufferToIgnoreInSimilar(Other.Buffers[J]))
			{
				return false;
			}
			++J;
		}

		return true;
	}

	bool FMeshBufferSet::IsSimilar(const FMeshBufferSet& Other) const
	{
		MUTABLE_CPUPROFILER_SCOPE(FMeshBufferSet::IsSimilar);

		if (ElementCount != Other.ElementCount)
		{
			return false;
		}

		// Compare all buffers except the vertex index channel, which should always be alone in
		// the last buffer
		int32 Index = 0;
		int32 OtherIndex = 0;

		const int32 ThisNumBuffers = Buffers.Num();
		const int32 OtherNumBuffers = Other.Buffers.Num();

		while (Index < ThisNumBuffers && OtherIndex < OtherNumBuffers)
		{
			// Is it a special buffer that we should ignore?
			if (IsSpecialBufferToIgnoreInSimilar(Buffers[Index]))
			{
				++Index;
				continue;
			}

			if (IsSpecialBufferToIgnoreInSimilar(Other.Buffers[OtherIndex]))
			{
				++OtherIndex;
				continue;
			}

			if (!(Buffers[Index] == Other.Buffers[OtherIndex]))
			{
				return false;
			}
			++Index;
			++OtherIndex;
		}

		// Whatever buffers are left should be irrelevant
		while (Index < ThisNumBuffers)
		{
			if (!IsSpecialBufferToIgnoreInSimilar(Buffers[Index]))
			{
				return false;
			}
			++Index;
		}

		while (OtherIndex < OtherNumBuffers)
		{
			if (!IsSpecialBufferToIgnoreInSimilar(Other.Buffers[OtherIndex]))
			{
				return false;
			}
			++OtherIndex;
		}

		return true;
	}

	void FMeshBufferSet::ResetBufferIndices()
	{
		int32 CurrentIndices[int32(EMeshBufferSemantic::Count)] = {0};
		for (FMeshBuffer& Buffer : Buffers)
		{
			for (FMeshBufferChannel& Channel : Buffer.Channels)
			{
				Channel.SemanticIndex = CurrentIndices[int32(Channel.Semantic)];
				CurrentIndices[int32(Channel.Semantic)]++;
			}
		}
	}

	void FMeshBufferSet::UpdateOffsets(int32 BufferIndex)
	{
		checkf(Buffers[BufferIndex].Data.IsEmpty(), TEXT("UpdateOffsets called on a non empty buffer set. This is not supported."));

		uint32 Offset = 0;
		for (FMeshBufferChannel& Channel : Buffers[BufferIndex].Channels)
		{
			if (Channel.Offset < Offset)
			{
				Channel.Offset = Offset;
			}
			else
			{
				Offset = Channel.Offset;
			}

			Offset += Channel.ComponentCount * GetMeshFormatData(Channel.Format).SizeInBytes;
		}

		if (Buffers[BufferIndex].ElementSize < Offset)
		{
			Buffers[BufferIndex].ElementSize = Offset;
		}
	}

	bool FMeshBufferSet::HasAnySemanticWithDifferentFormat(EMeshBufferSemantic Semantic, EMeshBufferFormat ExpectedFormat) const
	{
		for (const FMeshBuffer& Buffer : Buffers)
		{
			const int32 NumChannels = Buffer.Channels.Num();
			for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
			{
				if (Buffer.Channels[ChannelIndex].Semantic == Semantic)
				{
					if (Buffer.Channels[ChannelIndex].Format != ExpectedFormat)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	bool FMeshBufferSet::IsDescriptor() const
	{
		return EnumHasAnyFlags(Flags, EMeshBufferSetFlags::IsDescriptor);
	}

	void FMeshBuffer::Serialise(FOutputArchive& Arch) const
	{
		Arch << Channels;
		Arch << Data;
		Arch << ElementSize;
	}

	void FMeshBuffer::Unserialise(FInputArchive& Arch)
	{
		Arch >> Channels;
		Arch >> Data;
		Arch >> ElementSize;
	}
}
