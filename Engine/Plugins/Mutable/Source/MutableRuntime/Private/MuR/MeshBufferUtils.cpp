// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/MeshBufferUtils.h"

#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"

namespace UE::Mutable::Private
{
namespace MeshBufferUtils
{
	void SetupVertexPositionsBuffer(const int32& InCurrentVertexBuffer, FMeshBufferSet& OutTargetVertexBuffers)
	{
		const int32 ElementSize = sizeof(FPositionVertex);
		constexpr int32 ChannelCount = 1;
		const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::Position };
		const int32 SemanticIndices[ChannelCount] = { 0 };
		const EMeshBufferFormat Formats[ChannelCount] = { EMeshBufferFormat::Float32 };
		const int32 Components[ChannelCount] = { 3 };
		const int32 Offsets[ChannelCount] =
		{
			STRUCT_OFFSET(FPositionVertex, Position)
		};

		OutTargetVertexBuffers.SetBuffer(InCurrentVertexBuffer, ElementSize, ChannelCount, Semantics, SemanticIndices,
			Formats, Components, Offsets);
	}


	void SetupTangentBuffer(const int32& InCurrentVertexBuffer,
		FMeshBufferSet& OutTargetVertexBuffers)
	{
		// \todo: support for high precision?
		typedef TStaticMeshVertexTangentDatum<TStaticMeshVertexTangentTypeSelector<
			EStaticMeshVertexTangentBasisType::Default>::TangentTypeT> TangentType;

		const int32 ElementSize = sizeof(TangentType);
		constexpr int32 ChannelCount = 2;
		const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::Tangent, EMeshBufferSemantic::Normal };
		const int32 SemanticIndices[ChannelCount] = { 0, 0 };
		const EMeshBufferFormat Formats[ChannelCount] = { EMeshBufferFormat::PackedDirS8, EMeshBufferFormat::PackedDirS8_W_TangentSign };
		const int32 Components[ChannelCount] = { 4, 4 };
		const int32 Offsets[ChannelCount] =
		{
			STRUCT_OFFSET(TangentType, TangentX),
			STRUCT_OFFSET(TangentType, TangentZ)
		};

		OutTargetVertexBuffers.SetBuffer(InCurrentVertexBuffer, ElementSize, ChannelCount, Semantics, SemanticIndices,
			Formats, Components, Offsets);
	}


	void SetupTexCoordinatesBuffer(const int32& InCurrentVertexBuffer, const int32& InChannelCount, bool bHighPrecision,
		FMeshBufferSet& OutTargetVertexBuffers, const int32* InTextureSemanticIndicesOverride /* = nullptr */)
	{
		int32 UVSize = bHighPrecision
			? sizeof(TStaticMeshVertexUVsDatum<TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::HighPrecision>::UVsTypeT>)
			: sizeof(TStaticMeshVertexUVsDatum<TStaticMeshVertexUVsTypeSelector<EStaticMeshVertexUVType::Default>::UVsTypeT>);

		const int32 ElementSize = UVSize * InChannelCount;
		constexpr int32 MaxChannelCount = MaxTexCordChannelCount;
		const EMeshBufferSemantic Semantics[MaxChannelCount] = { EMeshBufferSemantic::TexCoords, EMeshBufferSemantic::TexCoords, EMeshBufferSemantic::TexCoords, EMeshBufferSemantic::TexCoords };
		const int32 Components[MaxChannelCount] = { 2, 2, 2, 2 };
		const int32 Offsets[MaxChannelCount] = { 0 * UVSize, 1 * UVSize, 2 * UVSize, 3 * UVSize, };

		const int32 StandardSemanticIndices[MaxChannelCount] = { 0, 1, 2, 3 };
		const int32* SemanticIndices = InTextureSemanticIndicesOverride ? InTextureSemanticIndicesOverride : StandardSemanticIndices;

		const EMeshBufferFormat HighFormats[MaxChannelCount] = { EMeshBufferFormat::Float32, EMeshBufferFormat::Float32, EMeshBufferFormat::Float32, EMeshBufferFormat::Float32 };
		const EMeshBufferFormat DefaultFormats[MaxChannelCount] = { EMeshBufferFormat::Float16, EMeshBufferFormat::Float16, EMeshBufferFormat::Float16, EMeshBufferFormat::Float16 };
		const EMeshBufferFormat* Formats = bHighPrecision ? HighFormats : DefaultFormats;

		OutTargetVertexBuffers.SetBuffer(InCurrentVertexBuffer, ElementSize, InChannelCount, Semantics, SemanticIndices, Formats, Components, Offsets);
	}


	void SetupSkinBuffer(const int32& InCurrentVertexBuffer,
		const int32& MaxBoneIndexTypeSizeBytes,
		const int32& MaxBoneWeightTypeSizeBytes,
		const int32& MaxNumBonesPerVertex,
		FMeshBufferSet& OutTargetVertexBuffers)
	{
		const int32 ElementSize = (MaxBoneWeightTypeSizeBytes + MaxBoneIndexTypeSizeBytes) * MaxNumBonesPerVertex;
		constexpr int32 ChannelCount = 2;
		const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::BoneIndices, EMeshBufferSemantic::BoneWeights };
		const int32 SemanticIndices[ChannelCount] = { 0, 0 };

		EMeshBufferFormat Formats[ChannelCount] = { EMeshBufferFormat::UInt8, EMeshBufferFormat::NUInt8 };

		switch (MaxBoneIndexTypeSizeBytes)
		{
		case 0: // Fallback to something in this case.
		case 1: Formats[0] = EMeshBufferFormat::UInt8;
			break;
		case 2: Formats[0] = EMeshBufferFormat::UInt16;
			break;
		case 4: Formats[0] = EMeshBufferFormat::UInt32;
			break;
		default:
			// unsupported bone index type
			check(false);
			Formats[0] = EMeshBufferFormat::None;
			break;
		}

		switch (MaxBoneWeightTypeSizeBytes)
		{
		case 0: // Fallback to something in this case.
		case 1: Formats[1] = EMeshBufferFormat::NUInt8;
			break;
		case 2: Formats[1] = EMeshBufferFormat::NUInt16;
			break;
		case 4: Formats[1] = EMeshBufferFormat::NUInt32;
			break;
		default:
			// unsupported bone weight type
			check(false);
			Formats[1] = EMeshBufferFormat::None;
			break;
		}

		int32 Components[ChannelCount];
		Components[0] = Components[1] = MaxNumBonesPerVertex;

		int32 Offsets[ChannelCount];
		Offsets[0] = 0;
		Offsets[1] = MaxBoneIndexTypeSizeBytes * MaxNumBonesPerVertex;

		OutTargetVertexBuffers.SetBuffer(InCurrentVertexBuffer, ElementSize, ChannelCount, Semantics, SemanticIndices,
			Formats, Components, Offsets);
	}


	void SetupVertexColorBuffer(const int32& InCurrentVertexBuffer, FMeshBufferSet& OutTargetVertexBuffers)
	{
		const int32 ElementSize = sizeof(FColor);
		constexpr int32 ChannelCount = 1;
		const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::Color };
		const int32 SemanticIndices[ChannelCount] = { 0 };
		const EMeshBufferFormat Formats[ChannelCount] = { EMeshBufferFormat::NUInt8 };
		const int32 Components[ChannelCount] = { 4 };
		const int32 Offsets[ChannelCount] = { 0 };
		check(ElementSize == 4);

		OutTargetVertexBuffers.SetBuffer(InCurrentVertexBuffer, ElementSize, ChannelCount, Semantics, SemanticIndices,
			Formats, Components, Offsets);
	}


	void SetupIndexBuffer(FMeshBufferSet& OutTargetIndexBuffers, EMeshBufferFormat InFormat)
	{
		check(InFormat == EMeshBufferFormat::UInt16 || InFormat == EMeshBufferFormat::UInt32);

		OutTargetIndexBuffers.SetBufferCount(1);

		const int32 ElementSize = InFormat == EMeshBufferFormat::UInt32 ? sizeof(uint32) : sizeof(uint16);
		//SkeletalMesh->GetImportedResource()->LODModels[LOD].MultiSizeIndexContainer.GetDataTypeSize();
		constexpr int32 ChannelCount = 1;
		const EMeshBufferSemantic Semantics[ChannelCount] = { EMeshBufferSemantic::VertexIndex };
		const int32 SemanticIndices[ChannelCount] = { 0 };
		// We force 32 bit indices, since merging meshes may create vertex buffers bigger than the initial mesh
		// and for now the mutable runtime doesn't handle it.
		// \TODO: go back to 16-bit indices when possible.
		EMeshBufferFormat Formats[ChannelCount] = { InFormat };
		const int32 Components[ChannelCount] = { 1 };
		const int32 Offsets[ChannelCount] = { 0 };

		OutTargetIndexBuffers.SetBuffer(0, ElementSize, ChannelCount, Semantics, SemanticIndices, Formats, Components,
			Offsets);
	}
}
}
