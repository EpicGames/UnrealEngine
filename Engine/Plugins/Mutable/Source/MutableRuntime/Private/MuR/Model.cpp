// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Model.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Serialisation.h"
#include "MuR/Types.h"

#if WITH_EDITOR
#include "Hash/xxhash.h"
#include "Serialization/MemoryWriter.h"
#endif

class UTexture;
class USkeletalMesh;
class UMaterialInterface;


namespace UE::Mutable::Private
{
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FRomDataRuntime);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FRomDataCompile);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FImageLODRange);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FMeshContentRange);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FConstantResourceIndex);

	
	FProgram::FProgram()
	{
		// Add the null instruction at address 0.
		// TODO: Will do it in the linker
		AppendCode( ByteCode, EOpType::NONE );
	}


	void FProgram::FState::Serialise(FOutputArchive& Arch) const
	{
		Arch << Name;
		Arch << Root;
		Arch << m_runtimeParameters;
		Arch << m_updateCache;
		Arch << m_dynamicResources;
	}


	void FProgram::FState::Unserialise(FInputArchive& Arch)
	{
		Arch >> Name;
		Arch >> Root;
		Arch >> m_runtimeParameters;
		Arch >> m_updateCache;
		Arch >> m_dynamicResources;
	}


	uint64 FProgram::FState::IsDynamic(FOperation::ADDRESS At) const
	{
		uint64 Result = 0;

		for (int32 Index = 0; Index < m_dynamicResources.Num(); ++Index)
		{
			if (m_dynamicResources[Index].Key == At)
			{
				Result = m_dynamicResources[Index].Value;
				break;
			}
		}

		return Result;
	}


	bool FProgram::FState::IsUpdateCache(FOperation::ADDRESS At) const
	{
		bool Result = false;

		for (int32 i = 0; !Result && i < m_updateCache.Num(); ++i)
		{
			if (m_updateCache[i] == At)
			{
				Result = true;
			}
		}

		return Result;
	}


	void FProgram::FState::AddUpdateCache(FOperation::ADDRESS At)
	{
		if (!IsUpdateCache(At))
		{
			m_updateCache.Add(At);
		}
	}


	void FProgram::Serialise(FOutputArchive& Arch) const
	{
		Arch << ByteCode;
		Arch << States;
		Arch << Roms;
		Arch << RomsCompileData;
		Arch << ConstantImageLODsPermanent;
		Arch << ConstantImageLODIndices;
		Arch << ConstantImages;
		Arch << ConstantMeshesPermanent;
		Arch << ConstantMeshContentIndices;
		Arch << ConstantMeshes;
		Arch << ConstantStrings;
		Arch << ConstantUInt32Lists;
		Arch << ConstantInt32Lists;
		Arch << ConstantUInt64Lists;
		Arch << ConstantFloatLists;
		Arch << ConstantBoolLists;
		Arch << ConstantLayouts;
		Arch << ConstantProjectors;
		Arch << ConstantMatrices;
		Arch << ConstantShapes;
		Arch << ConstantCurves;
		Arch << ConstantSkeletons;
		Arch << Parameters;
		Arch << Ranges;
		Arch << ParameterLists;
		Arch << RelevantParameterList;
		Arch << ConstantMaterials;
		Arch << ConstantNames;
		Arch << ConstantSockets;
	}


	void FProgram::Unserialise(FInputArchive& Arch)
	{
		Arch >> ByteCode;
		Arch >> States;
		Arch >> Roms;
		Arch >> RomsCompileData;
		Arch >> ConstantImageLODsPermanent;
		Arch >> ConstantImageLODIndices;
		Arch >> ConstantImages;
		Arch >> ConstantMeshesPermanent;
		Arch >> ConstantMeshContentIndices;
		Arch >> ConstantMeshes;
		Arch >> ConstantStrings;
		Arch >> ConstantUInt32Lists;
		Arch >> ConstantInt32Lists;
		Arch >> ConstantUInt64Lists;
		Arch >> ConstantFloatLists;
		Arch >> ConstantBoolLists;
		Arch >> ConstantLayouts;
		Arch >> ConstantProjectors;
		Arch >> ConstantMatrices;
		Arch >> ConstantShapes;
		Arch >> ConstantCurves;
		Arch >> ConstantSkeletons;
		Arch >> Parameters;
		Arch >> Ranges;
		Arch >> ParameterLists;
		Arch >> RelevantParameterList;
		Arch >> ConstantMaterials;
		Arch >> ConstantNames;
		Arch >> ConstantSockets;

		PostLoad();
	}
	
	void FProgram::PostLoad()
	{
		// Convert Skeletons' BoneIds to FNames for runtime operations
		const int32 NumSkeletons = ConstantSkeletons.Num();
		for (int32 Index = 0; Index < NumSkeletons; ++Index)
		{
			TManagedPtr<FSkeleton> Skeleton = ConstCastManagedPtr<FSkeleton>(ConstantSkeletons[Index]);

			const TArray<uint32>& BoneIds = Skeleton->BoneIds;

			TArray<FName>& BoneNames = Skeleton->BoneNames;
			BoneNames.Reset(BoneIds.Num());

			for (uint32 BoneId : BoneIds)
			{
				BoneNames.Add(ConstantNames[BoneId]);
			}
		}
	}

	void FProgram::LogHistogram() const
    {
#if 0
        uint64 countPerType[(int32)EOpType::COUNT];
        mutable_memset(countPerType,0,sizeof(countPerType));

        for ( const uint32& o: OpAddress )
        {
            EOpType type = GetOpType(o);
            countPerType[(int32)type]++;
        }

		TArray< TPair<uint64,EOpType> > sorted((int32)EOpType::COUNT);
        for (int32 i=0; i<(int32)EOpType::COUNT; ++i)
        {
            sorted[i].second = (EOpType)i;
            sorted[i].first = countPerType[i];
        }

        std::sort(sorted.begin(),sorted.end(), []( const pair<uint64,EOpType>& a, const pair<uint64,EOpType>& b )
        {
            return a.first>b.first;
        });

        UE_LOGF(LogMutableCore,Log, "Op histogram (%llu ops):", OpAddress.Num());
        for(int32 i=0; i<8; ++i)
        {
            float p = sorted[i].first/float(OpAddress.Num())*100.0f;
            UE_LOGF(LogMutableCore,Log, "  %3.2f%% : %d", p, (int32)sorted[i].second );
        }
#endif
    }
	
#if WITH_EDITOR
    FOperation::ADDRESS FProgram::AddConstant(const TManagedPtr<const FLayout>& Layout)
    {
	    // Ensure unique
	    for (int32 Index = 0; Index < ConstantLayouts.Num(); ++Index)
	    {
		    if (ConstantLayouts[Index] == Layout)
		    {
			    return (FOperation::ADDRESS)Index;
		    }
	    }

	    FOperation::ADDRESS Index = FOperation::ADDRESS(ConstantLayouts.Num());
	    ConstantLayouts.Add(Layout);
	    return Index;
    }


	FOperation::ADDRESS FProgram::AddConstant(const TManagedPtr<const FSkeleton>& InSkeleton)
	{
		// Ensure unique
		for (int32 i = 0; i < ConstantSkeletons.Num(); ++i)
		{
			if (ConstantSkeletons[i] == InSkeleton
				||
				*ConstantSkeletons[i] == *InSkeleton // Does not compare BoneIds
				)
			{
				return (FOperation::ADDRESS)i;
			}
		}

		// Convert FNames to FBoneIds
		TManagedPtr<FSkeleton> Skeleton = InSkeleton->Clone();

		const TArray<FName>& BoneNames = Skeleton->BoneNames;
		const int32 NumBoneNames = BoneNames.Num();

		TArray<uint32>& BoneIds = Skeleton->BoneIds;
		BoneIds.SetNumUninitialized(NumBoneNames);

		for (int32 Index = 0; Index < NumBoneNames; ++Index)
		{
			BoneIds[Index] = AddConstant(BoneNames[Index]);
		}

		return FOperation::ADDRESS(ConstantSkeletons.Add(Skeleton));
	}


    FOperation::ADDRESS FProgram::AddConstant(const TManagedPtr<const FPhysicsBody>& PhysicsBody)
    {
	    // Ensure unique
	    for (int32 Index = 0; Index < ConstantPhysicsBodies.Num(); ++Index)
	    {
		    if (ConstantPhysicsBodies[Index] == PhysicsBody || *ConstantPhysicsBodies[Index] == *PhysicsBody )
		    {
			    return (FOperation::ADDRESS)Index;
		    }
	    }

	    FOperation::ADDRESS index = FOperation::ADDRESS(ConstantPhysicsBodies.Num());
	    ConstantPhysicsBodies.Add(PhysicsBody);
	    return index;
    }


	uint32 AddSocketToProgram(TMap<uint32, FMeshSocket>& ConstantSockets, FMeshSocket Socket, TArray<uint8>& AuxData)
	{
		AuxData.SetNum(0, EAllowShrinking::No);
		FMemoryWriter Writer(AuxData);

		FString SocketName = Socket.SocketName.ToString().ToLower();
		Writer << SocketName;

		FString BoneName = Socket.BoneName.ToString().ToLower();
		Writer << BoneName;

		Writer << Socket.RelativeLocation;
		Writer << Socket.RelativeRotation;
		Writer << Socket.RelativeScale;
		Writer << Socket.bForceAlwaysAnimated;

		uint32 SocketDataHash = static_cast<uint32>(FXxHash64::HashBuffer(AuxData.GetData(), AuxData.Num()).Hash);
		
		for (uint32 NumTries = 0; NumTries < TNumericLimits<uint32>::Max(); ++NumTries)
		{
			FMeshSocket& FoundData = ConstantSockets.FindOrAdd(SocketDataHash, Socket);
			if (FoundData == Socket)
			{
				break;
			}

			SocketDataHash++;
		}

		return SocketDataHash;
	}


	uint32 FProgram::AddConstant(const FMeshSocket& Socket)
	{
		TArray<uint8> SocketData;
		return AddSocketToProgram(ConstantSockets, Socket, SocketData);
	}


	FOperation::CONSTANT_LIST_UINT32 FProgram::AddConstant(const TArray<FMeshSocket>& Sockets)
	{
		TArray<uint32> SocketIds;
		SocketIds.Reset(Sockets.Num());

		TArray<uint8> SocketData;
		for (const FMeshSocket& Socket : Sockets)
		{
			SocketIds.Add(AddSocketToProgram(ConstantSockets, Socket, SocketData));
		}

		return AddConstant(SocketIds);
	}

    FOperation::CONSTANT_STRING FProgram::AddConstant(const FString& Str)
    {     
	    return (FOperation::CONSTANT_STRING)ConstantStrings.AddUnique(Str);
    }


	FOperation::CONSTANT_NAME FProgram::AddConstant(const FName& InName)
	{
		const FString NameString = InName.ToString().ToLower();
		uint32 NameId = CityHash32(reinterpret_cast<const char*>(*NameString), NameString.Len() * sizeof(FString::ElementType));

		FName& ConstantName = ConstantNames.FindOrAdd(NameId, InName);
		while (ConstantName != InName)
		{
			// Increase Id in an attempt to make it unique again.
			++NameId;
			ConstantName = ConstantNames.FindOrAdd(NameId, InName);
		}

		return NameId;
	}


	FOperation::CONSTANT_LIST_UINT32 FProgram::AddConstant(const TArray<FName>& NameList)
	{
		const int32 NumNames = NameList.Num();

		TArray<uint32> NameAddrs;
		NameAddrs.SetNumUninitialized(NumNames);

		for (int32 NameIndex = 0; NameIndex < NumNames; ++NameIndex)
		{
			NameAddrs[NameIndex] = AddConstant(NameList[NameIndex]);
		}

		return (FOperation::ADDRESS)AddConstant(NameAddrs);
	}


    FOperation::ADDRESS FProgram::AddConstant(const TArray<uint32>& UInt32List)
    {
	    return (FOperation::ADDRESS)ConstantUInt32Lists.AddUnique(UInt32List);
    }

	
	FOperation::ADDRESS FProgram::AddConstant(const TArray<int32>& Int32List)
	{
		return (FOperation::ADDRESS)ConstantInt32Lists.AddUnique(Int32List);
	}

	
    FOperation::CONSTANT_LIST_UINT64 FProgram::AddConstant(const TArray<uint64>& UInt64List)
    {
	    return (FOperation::CONSTANT_LIST_UINT64)ConstantUInt64Lists.AddUnique(UInt64List);
    }


    FOperation::ADDRESS FProgram::AddConstant(const TArray<FString>& StringList)
    {
	    const int32 NumStrings = StringList.Num();

	    TArray<uint32> StringAddrs;
	    StringAddrs.SetNum(NumStrings);

	    for (int32 StringIndex = 0; StringIndex < NumStrings; ++StringIndex)
	    {
		    StringAddrs[StringIndex] = AddConstant(StringList[StringIndex]);
	    }

	    return (FOperation::ADDRESS)AddConstant(StringAddrs);
    }


    FOperation::CONSTANT_MATRIX FProgram::AddConstant(const FMatrix44f& Constant)
    {
	    // Ensure unique
	    for (int32 i = 0; i < ConstantMatrices.Num(); ++i)
	    {
		    if (ConstantMatrices[i] == Constant)
		    {
			    return (FOperation::CONSTANT_MATRIX)i;
		    }
	    }

	    FOperation::CONSTANT_MATRIX Index = FOperation::CONSTANT_MATRIX(ConstantMatrices.Num());
	    ConstantMatrices.Add(Constant);
	    return Index;
    }


    FOperation::CONSTANT_SHAPE FProgram::AddConstant(const FShape& Constant)
    {
	    // Ensure unique
	    for (int32 Index = 0; Index<ConstantShapes.Num(); ++Index)
	    {
		    if (ConstantShapes[Index] == Constant)
		    {
			    return (FOperation::CONSTANT_SHAPE)Index;
		    }
	    }

	    FOperation::CONSTANT_SHAPE Index = FOperation::CONSTANT_SHAPE(ConstantShapes.Num());
	    ConstantShapes.Add(Constant);
	    return Index;
    }


    FOperation::ADDRESS FProgram::AddConstant(const FProjector& Constant)
    {
	    // Ensure unique
	    for (int32 Index = 0; Index < ConstantProjectors.Num(); ++Index)
	    {
		    if (ConstantProjectors[Index] == Constant)
		    {
			    return (FOperation::ADDRESS)Index;
		    }
	    }

	    FOperation::ADDRESS Index = FOperation::ADDRESS(ConstantProjectors.Num());
	    ConstantProjectors.Add(Constant);
	    return Index;
    }


    FOperation::ADDRESS FProgram::AddConstant(const FRichCurve& Constant)
    {
	    FOperation::ADDRESS Index = FOperation::ADDRESS(ConstantCurves.AddUnique(Constant ));
	    return Index;
    }


    FOperation::ADDRESS FProgram::AddConstant(const TManagedPtr<const FMaterial>& Material)
    {
	    for (int32 Index=0; Index<ConstantMaterials.Num(); ++Index)
	    {
		    if (ConstantMaterials[Index] == Material || *ConstantMaterials[Index] == *Material)
		    {
			    return static_cast<FOperation::ADDRESS>(Index);
		    }
	    }

	    FOperation::ADDRESS Index = static_cast<FOperation::ADDRESS>(ConstantMaterials.Num());
	    ConstantMaterials.Add(Material);
	    return Index;
    }

    FOperation::ADDRESS FProgram::AddConstant(const TArray<float>& FloatList)
    {
		return (FOperation::ADDRESS)ConstantFloatLists.AddUnique(FloatList);
    }

    FOperation::ADDRESS FProgram::AddConstant(const TArray<bool>& BoolList)
    {
		return (FOperation::ADDRESS)ConstantBoolLists.AddUnique(BoolList);
    }
#endif // WITH_EDITOR
	

    EOpType FProgram::GetOpType(FOperation::ADDRESS Address) const
    {
		uint32 ByteCodeOffset = GetAddressByteCodeOffset(Address);

		if ((ByteCodeOffset >= (uint32)ByteCode.Num()) | 
			(ByteCodeOffset >= FOperation::MaxAddressByteCodeOffset)) 
		{
			return EOpType::NONE;
		}

		EOpType Result;
		FMemory::Memcpy(&Result, &ByteCode[ByteCodeOffset], sizeof(EOpType));

		check(Result < EOpType::COUNT);

		return Result;
    }


    const uint8* FProgram::GetOpArgsPointer(FOperation::ADDRESS Address) const
    {
	    uint32 ByteCodeOffset = GetAddressByteCodeOffset(Address) + sizeof(EOpType);
	    const uint8* DataPtr = (const uint8*)&ByteCode[ByteCodeOffset];
	    return DataPtr;
    }


    uint8* FProgram::GetOpArgsPointer(FOperation::ADDRESS Address)
    {
	    uint32 ByteCodeOffset = GetAddressByteCodeOffset(Address) + sizeof(EOpType);
	    uint8* DataPtr = &ByteCode[ByteCodeOffset];
	    return DataPtr;
    }

	
    FModel::FModel(): RomManager(*this)
    {
    }


    void FModel::Serialise(const FModel* Model, FOutputArchive& Arch)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

        Arch << *Model;
    }

   
    class FOutputModelStream : public FOutputStream
    {
    public:

        // Life cycle
        FOutputModelStream(FModelWriter* InStreamer)
            : Streamer(InStreamer)
        {
        }

        // FInputStream interface
        void Write(const void* Data, uint64 Size) override
        {
            Streamer->Write(Data, Size);
        }

    private:
        FModelWriter* Streamer;
    };


    void FModel::Serialise(FModel* Model, FModelWriter& Streamer, bool bDropData)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
    	
    	Model->RomManager.Serialise(Streamer, bDropData);

		// Store the main data of the model
		{
			Streamer.OpenWriteFile(0, false);
			FOutputModelStream stream(&Streamer);
			FOutputArchive arch(&stream);

			arch << *Model;

			Streamer.CloseWriteFile();
		}
	}

	
#if WITH_EDITOR
	bool FModel::IsValid() const
	{
		return Program.bIsValid;
	}


	void FModel::Invalidate()
    {
		Program.bIsValid = false;
    }
#endif


	TSharedPtr<FModel> FModel::StaticUnserialise(FInputArchive& Arch)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		TSharedPtr<FModel> pResult = MakeShared<FModel>();
        Arch >> *pResult;
        return pResult;
    }


	void FModel::Serialise(FOutputArchive& Arch) const
	{
		Arch << Program;
	}


	void FModel::Unserialise(FInputArchive& Arch)
	{
		Arch >> Program;
	}


	bool FModel::GetBoolDefaultValue(int32 Index) const
    {
    	check(Program.Parameters.IsValidIndex(Index));
		check(Program.Parameters[Index].Type == EParameterType::Bool);

        // Early out in case of invalid parameters
        if (!Program.Parameters.IsValidIndex(Index) ||
            Program.Parameters[Index].Type != EParameterType::Bool)
        {
            return false;
        }
		
        return Program.Parameters[Index].DefaultValue.Get<FParamBoolType>();
    }


	int32 FModel::GetIntDefaultValue(int32 Index) const
	{
		check(Program.Parameters.IsValidIndex(Index));
		check(Program.Parameters[Index].Type == EParameterType::Int);

        // Early out in case of invalid parameters
        if (!Program.Parameters.IsValidIndex(Index) ||
            Program.Parameters[Index].Type != EParameterType::Int)
        {
            return 0;
        }
		
        return Program.Parameters[Index].DefaultValue.Get<FParamIntType>();
	}


	float FModel::GetFloatDefaultValue(int32 Index) const
	{
    	check(Program.Parameters.IsValidIndex(Index));
		check(Program.Parameters[Index].Type == EParameterType::Float);

        // Early out in case of invalid parameters
        if (!Program.Parameters.IsValidIndex(Index) ||
            Program.Parameters[Index].Type != EParameterType::Float)
        {
            return 0.0f;
        }
		
        return Program.Parameters[Index].DefaultValue.Get<FParamFloatType>();
	}


	void FModel::GetColorDefaultValue(int32 Index, FVector4f& OutValue) const
    {
    	check(Program.Parameters.IsValidIndex(Index));
		check(Program.Parameters[Index].Type == EParameterType::Color);

        // Early out in case of invalid parameters
        if (!Program.Parameters.IsValidIndex(Index) ||
            Program.Parameters[Index].Type != EParameterType::Color)
        {
            return;
        }

        const FParamColorType& Color = Program.Parameters[Index].DefaultValue.Get<FParamColorType>();
		
		OutValue = Color;
    }


	FMatrix44f FModel::GetMatrixDefaultValue(int32 Index) const
	{
    	check(Program.Parameters.IsValidIndex(Index));
    	check(Program.Parameters[Index].Type == EParameterType::Matrix);

    	// Early out in case of invalid parameters
    	if (!Program.Parameters.IsValidIndex(Index) ||
			Program.Parameters[Index].Type != EParameterType::Matrix)
    	{
    		return FMatrix44f::Identity;
    	}

    	return Program.Parameters[Index].DefaultValue.Get<FParamMatrixType>();
	}


	void FModel::GetProjectorDefaultValue(int32 Index, EProjectorType* OutProjectionType, FVector3f* OutPos,
	                                     FVector3f* OutDir, FVector3f* OutUp, FVector3f* OutScale, float* OutProjectionAngle) const
	{
    	check(Program.Parameters.IsValidIndex(Index));
		check(Program.Parameters[Index].Type == EParameterType::Projector);

        // Early out in case of invalid parameters
        if (!Program.Parameters.IsValidIndex(Index) ||
            Program.Parameters[Index].Type != EParameterType::Projector)
        {
            return;
        }

        const FProjector& Projector = Program.Parameters[Index].DefaultValue.Get<FParamProjectorType>();
        if (OutProjectionType) *OutProjectionType = Projector.type;
    	if (OutPos) *OutPos = Projector.position;
		if (OutDir) *OutDir = Projector.direction;
    	if (OutUp) *OutUp = Projector.up;
    	if (OutScale) *OutScale = Projector.scale;
    	if (OutProjectionAngle) *OutProjectionAngle = Projector.projectionAngle;
	}

	
    int32 FProgram::GetRomCount() const
    {
    	return Roms.Num();
    }

	
#if WITH_EDITOR
	uint32 FProgram::GetRomSourceId(int32 Index) const
	{
		return RomsCompileData[Index].SourceId;
	}
#endif

	
    uint32 FProgram::GetRomSize(int32 Index) const
    {
    	return Roms[Index].Size;
    }

	
    ERomDataType FProgram::GetRomDataType(int32 Index) const
	{
		return static_cast<ERomDataType>(Roms[Index].ResourceType);
	}

	
	bool FProgram::IsRomHighRes(int32 Index) const
	{
		return Roms[Index].IsHighRes == 1;
	}

	
    int32 FProgram::GetConstantImageRomId(int32 ConstantImageIndex, int32 LODIndex) const
    {
        check(ConstantImages.IsValidIndex(ConstantImageIndex));
        
        FImageLODRange LODRange = ConstantImages[ConstantImageIndex];

        if (LODIndex >= LODRange.LODCount) 
        {
            return -1; 
        }

        FConstantResourceIndex ResourceIndex = ConstantImageLODIndices[LODRange.FirstIndex + LODIndex];
        if (!ResourceIndex.Streamable)
        {
            return -1;
        }

        return ResourceIndex.Index;
    }
	

    TSharedPtr<FParameters> FModel::NewParameters(TSharedPtr<const FModel> Model, const FParameters* OldParameters)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

        TSharedPtr<FParameters> pRes = MakeShared<FParameters>();

        pRes->Model = Model;

		const FProgram& Program = Model->Program;
        pRes->Values.SetNum(Program.Parameters.Num());
        for ( int32 p=0; p< Program.Parameters.Num(); ++p )
        {
            pRes->Values[p] = Program.Parameters[p].DefaultValue;
        }

        // Copy old values
        if ( OldParameters )
        {
            for ( int32 p=0; p<OldParameters->GetCount(); ++p )
            {
                int32 thisP = pRes->Find( OldParameters->GetName(p) );

                if ( thisP>=0 )
                {
                    if ( OldParameters->GetType(p)==pRes->GetType(thisP) )
                    {
                        switch ( pRes->GetType(thisP) )
                        {
						case EParameterType::Bool:
                            pRes->SetBoolValue( thisP, OldParameters->GetBoolValue(p) );
                            break;

                        case EParameterType::Int:
                            pRes->SetIntValue( thisP, OldParameters->GetIntValue(p) );
                            break;

                        case EParameterType::Float:
                            pRes->SetFloatValue( thisP, OldParameters->GetFloatValue(p) );
                            break;

                        case EParameterType::Color:
                        {
                            FVector4f V;
                            OldParameters->GetColorValue( p, V );
                            pRes->SetColorValue( thisP, V );
                            break;
                        }

                        case EParameterType::Projector:
                        {
//							float m[16];
//							Parameters->GetProjectorValue( p, m );
                            pRes->Values[thisP].Set<FParamProjectorType>(OldParameters->Values[p].Get<FParamProjectorType>());
                            break;
                        }

                        case EParameterType::Texture:
                            pRes->SetTextureValue(thisP, OldParameters->GetTextureValue(p));
                            break;
                        	
                        case EParameterType::SkeletalMesh:
                        	pRes->SetSkeletalMeshValue(thisP, OldParameters->GetSkeletalMeshValue(p));
                        	break;
                        	
	        			case EParameterType::Material:
	        				pRes->SetMaterialValue(thisP, OldParameters->GetMaterialValue(p));
	        				break;
                        	
	        			case EParameterType::InstancedStruct:
	        				pRes->SetInstancedStructValue(thisP, OldParameters->GetInstancedStructValue(p));
	        				break;

                        default:
                            check(false);
                            break;
                        }
                    }
                }
            }
        }

        return pRes;
    }


    bool FModel::IsParameterMultidimensional(const int32 ParamIndex) const
    {
		if (Program.Parameters.IsValidIndex(ParamIndex))
		{
			return Program.Parameters[ParamIndex].Ranges.Num() > 0;
		}

		return false;
    }
	

    int32 FModel::GetStateCount() const
    {
        return (int32)Program.States.Num();
    }


    const FString& FModel::GetStateName( int32 index ) const
    {
        const char* strRes = 0;

        if ( index>=0 && index<(int32)Program.States.Num() )
        {
            return Program.States[index].Name;
        }

		static FString None;
        return None;
    }


    int32 FModel::FindState( const FString& Name ) const
    {
        int32 res = -1;

        for ( int32 i=0; res<0 && i<(int32)Program.States.Num(); ++i )
        {
            if ( Program.States[i].Name == Name )
            {
                res = i;
            }
        }

        return res;
    }


    int32 FModel::GetStateParameterCount( int32 stateIndex ) const
    {
        int32 res = -1;

        if ( stateIndex>=0 && stateIndex<(int32)Program.States.Num() )
        {
            res = (int32)Program.States[stateIndex].m_runtimeParameters.Num();
        }

        return res;
    }


    int32 FModel::GetStateParameterIndex( int32 stateIndex, int32 paramIndex ) const
    {
        int32 res = -1;

        if ( stateIndex>=0 && stateIndex<(int32)Program.States.Num() )
        {
            const FProgram::FState& state = Program.States[stateIndex];
            if ( paramIndex>=0 && paramIndex<(int32)state.m_runtimeParameters.Num() )
            {
                res = (int32)state.m_runtimeParameters[paramIndex];
            }
        }

        return res;
    }
}

