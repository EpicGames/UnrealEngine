// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/MutableRuntimeModule.h"

#include "Modules/ModuleManager.h"
#include "MuR/Platform.h"
#include "MuR/BlockCompression/Miro/Miro.h"

IMPLEMENT_MODULE(FMutableRuntimeModule, MutableRuntime);

DEFINE_LOG_CATEGORY(LogMutableCore);

namespace
{
	static int32 SInitialized = 0;
	static int32 SFinalized = 0;
}


void FMutableRuntimeModule::StartupModule()
{
	if (!SInitialized)
	{
		SInitialized = 1;
		SFinalized = 0;

		miro::initialize();
	}
}


void FMutableRuntimeModule::ShutdownModule()
{
	if (SInitialized && !SFinalized)
	{
		miro::finalize();

		SFinalized = 1;
		SInitialized = 0;
	}
}

