// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/MutableString.h"


namespace UE::Mutable::Private 
{

	String::String( const FString& InValue)
	{
        Value = InValue;
	}


	TSharedPtr<String> String::Clone() const
	{
		TSharedPtr<String> Result = MakeShared<String>(Value);
		return Result;
	}


	int32 String::GetDataSize() const
	{
		return sizeof(String) + Value.GetAllocatedSize();
	}


	const FString& String::GetValue() const
	{
        return Value;
	}

}

