// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpImageApplyComposite.h"

#include "MuR/ImageComputeUtils.h"
#include "MuR/ParallelExecutionUtils.h"


namespace UE::Mutable::Private
{
namespace OpImageApplyCompositeInternal
{
	struct FBatchViews
	{
		int32 NumElems = 0;

		TArrayView<uint8> Result;
		TArrayView<const uint8> Base;
		TArrayView<const uint8> Normal;
	};

	struct FBatchArgs
	{
		int32 BatchNumElems  = 0;
		int32 LODBegin       = 0;
		int32 LODEnd         = 0;
		int32 FirstLODOffsetInElems = 0;

		FImage* Result       = nullptr;
		const FImage* Base   = nullptr;
		const FImage* Normal = nullptr;

		uint32 ResultBytesPerElem = 0;
		uint32 BaseBytesPerElem   = 0;
		uint32 NormalBytesPerElem = 0;
	};

	struct FNormalCompositeKernel
	{
		using FFormatTypeFunc = void(uint8*, const uint8*, int32);
		using FComputeRoughnessTypeFunc = void(uint8*, const uint8*, const uint8*, float, int32); 
		using FSelectTypeFunc = void(uint8*, const uint8*, const uint8*, int32);

		FFormatTypeFunc* BaseFormat                 = nullptr;
		FFormatTypeFunc* NormalFormat               = nullptr;
		FComputeRoughnessTypeFunc* ComputeRoughness = nullptr;
		FSelectTypeFunc* SelectChannel              = nullptr;
		FFormatTypeFunc* ResultFormat               = nullptr;

		float Power =  1.0f;
	};

	struct FMakeNormalCompositeKernelArgs
	{
		const FImage* ResultImage = nullptr;
		const FImage* BaseImage   = nullptr;
		const FImage* NormalImage = nullptr;
		
		int32 NormalRoughnessOutputChannel = 0;
		float Power = 1.0f;
	};

	void Transpose(VectorRegister4Float& InOutA, VectorRegister4Float& InOutB, VectorRegister4Float& InOutC, VectorRegister4Float& InOutD)
	{
		VectorRegister4Float Temp0;
		VectorRegister4Float Temp1;
		VectorRegister4Float Temp2;
		VectorRegister4Float Temp3;

		VectorDeinterleave(Temp0, Temp1, InOutA, InOutB);
		VectorDeinterleave(Temp2, Temp3, InOutC, InOutD);

		// Temp0 = [XA,ZA,XB,ZB]
		// Temp1 = [YA,WA,YB,WB]
		// Temp2 = [XC,ZC,XD,ZD]
		// Temp3 = [YC,WC,YD,WD]

		VectorDeinterleave(InOutA, InOutC, Temp0, Temp2);
		VectorDeinterleave(InOutB, InOutD, Temp1, Temp3);
	}

	void ComputeRoughnessFromVarianceImpl(uint8* Result, const uint8* Base, const VectorRegister4Float& TwoTimesVariance)
	{
		constexpr VectorRegister4Float FloatOne        = GlobalVectorConstants::FloatOne;
		constexpr VectorRegister4Float Float255        = GlobalVectorConstants::Float255;
		constexpr VectorRegister4Float FloatOneOver255 = MakeVectorRegisterFloatConstant(1.0f/255.0f, 1.0f/255.0f,  1.0f/255.0f, 1.0f/255.0f);

		VectorRegister4Float Value = VectorMultiply(VectorLoadByte4(Base), FloatOneOver255);

		VectorRegister4Float ValueSquared = VectorMultiply(Value, Value);
		Value = VectorMultiply(ValueSquared, ValueSquared);

		VectorRegister4Float B = VectorMultiply(VectorSubtract(Value, FloatOne), TwoTimesVariance);

		Value = VectorDivide(VectorSubtract(B, Value), VectorSubtract(B, FloatOne));
		Value = VectorSqrt(VectorSqrt(Value));

		VectorStoreByte4(VectorMultiply(Value, Float255), Result);
	}

	/** 
	 * It computes the Toksvig estimation of the normal variance for all channels similar to
	 * TextureCompressorModule.cpp::ApplyCompositeTexture.
	 *
	 * Base, Normal and Result are expected to be 4 channel wide. 
	 *
	 * This implementation has not been validated.
	 */
	void ComputeRoughness(uint8* Result, const uint8* Base, const uint8* Normal, float PowerValue, int32 NumElems)
	{
		constexpr VectorRegister4Float FloatOne = GlobalVectorConstants::FloatOne;
		constexpr VectorRegister4Float Float255 = GlobalVectorConstants::Float255;

		constexpr VectorRegister4Float FloatVarianceTolerance = MakeVectorRegisterFloatConstant(0.00004f, 0.00004f, 0.00004f, 0.00004f);

		VectorRegister4Float Power = VectorSetFloat1(PowerValue);

		int32 I = 0;
		for (; I < NumElems - 4; I += 4)
		{
			// Compute variance.
			VectorRegister4Float Vec0 = VectorLoadByte4((Normal + I*4 + 0 ));
			VectorRegister4Float Vec1 = VectorLoadByte4((Normal + I*4 + 4 ));
			VectorRegister4Float Vec2 = VectorLoadByte4((Normal + I*4 + 8 ));
			VectorRegister4Float Vec3 = VectorLoadByte4((Normal + I*4 + 12));

			Transpose(Vec0, Vec1, Vec2, Vec3);
		
			VectorRegister4Float SquaredLength =
					VectorMultiplyAdd(Vec0, Vec0, VectorMultiplyAdd(Vec1, Vec1, VectorMultiply(Vec2, Vec2)));	

			VectorRegister4Float OneOverLength = VectorMultiply(VectorReciprocalSqrt(SquaredLength), Float255);

			VectorRegister4Float Variance = VectorSubtract(OneOverLength, FloatOne);

			Variance = VectorMax(FloatVarianceTolerance, VectorSubtract(Variance, FloatVarianceTolerance));
			Variance = VectorMultiply(Variance, Power);

			VectorRegister4Float TwoTimesVariance = VectorAdd(Variance, Variance);
			
			ComputeRoughnessFromVarianceImpl(Result + I*4 + 0 , Base + I*4 + 0 , VectorReplicate(TwoTimesVariance, 0));
			ComputeRoughnessFromVarianceImpl(Result + I*4 + 4 , Base + I*4 + 4 , VectorReplicate(TwoTimesVariance, 1));
			ComputeRoughnessFromVarianceImpl(Result + I*4 + 8 , Base + I*4 + 8 , VectorReplicate(TwoTimesVariance, 2));
			ComputeRoughnessFromVarianceImpl(Result + I*4 + 12, Base + I*4 + 12, VectorReplicate(TwoTimesVariance, 3));
		}

		for (; I < NumElems; ++I)
		{
			VectorRegister4Float Vec = VectorLoadByte4((Normal + I*4 + 0));
			VectorRegister4Float OneOverLength = VectorMultiply(VectorReciprocalSqrt(VectorDot3(Vec, Vec)), Float255);

			VectorRegister4Float Variance = VectorSubtract(OneOverLength, FloatOne);

			Variance = VectorMax(FloatVarianceTolerance, VectorSubtract(Variance, FloatVarianceTolerance));
			Variance = VectorMultiply(Variance, Power);

			VectorRegister4Float TwoTimesVariance = VectorAdd(Variance, Variance);

			ComputeRoughnessFromVarianceImpl(Result + I*4 + 0, Base + I*4 + 0, TwoTimesVariance);
		}

	}

	FNormalCompositeKernel MakeNormalCompositeKernel(FMakeNormalCompositeKernelArgs Args)
	{
		using namespace ImageComputeUtils;

		FNormalCompositeKernel Kernel;
		Kernel.Power = Args.Power;
		Kernel.ComputeRoughness = ComputeRoughness;

		if (Args.BaseImage->GetFormat() == EImageFormat::RGB_UByte)
		{
			Kernel.BaseFormat = Vec3ToVec4_U8;
		}

		if (Args.NormalImage->GetFormat() == EImageFormat::RGB_UByte)
		{
			Kernel.NormalFormat = Vec3ToVec4_U8;
		}

		switch (Args.NormalRoughnessOutputChannel)
		{
			case 0: Kernel.SelectChannel = Vec4SelectX_U8; break;
			case 1: Kernel.SelectChannel = Vec4SelectY_U8; break;
			case 2: Kernel.SelectChannel = Vec4SelectZ_U8; break;
			case 3: Kernel.SelectChannel = Vec4SelectW_U8; break;
			default: Kernel.SelectChannel = nullptr; break;
		}

		if (Args.ResultImage->GetFormat() == EImageFormat::RGB_UByte)
		{
			Kernel.ResultFormat = Vec4ToVec3_U8;
		}

		return Kernel;
	}

	void RunKernelOnBatch(FNormalCompositeKernel Kernel, FBatchViews BatchViews)
	{
		MUTABLE_CPUPROFILER_SCOPE(OpImageNormalComposite_RunKernelOnBatch);

		constexpr int32 MaxElemSize = 4;
		constexpr int32 NumCacheLinesPerWave = 8;
		constexpr int32 WaveSizeInElems = (PLATFORM_CACHE_LINE_SIZE * NumCacheLinesPerWave)/MaxElemSize;

		struct alignas(PLATFORM_CACHE_LINE_SIZE) FTemporalRegister
		{
			uint8 Data[WaveSizeInElems*MaxElemSize];	
		};

		FTemporalRegister Register0;
		FTemporalRegister Register1;
		FTemporalRegister Register2;
		//FTemporalRegister Register3;
		//FTemporalRegister Register4;

		if (BatchViews.NumElems == 0)
		{
			return;
		}

		int32 BaseBytesPerElem = BatchViews.Base.Num() / BatchViews.NumElems;
		int32 NormalBytesPerElem = BatchViews.Normal.Num() / BatchViews.NumElems;

		check(BaseBytesPerElem <= MaxElemSize);
		check(NormalBytesPerElem <= MaxElemSize);

		uint8* ResultBuffer = BatchViews.Result.GetData();
		const uint8* BaseBuffer = BatchViews.Base.GetData();
		const uint8* NormalBuffer = BatchViews.Normal.GetData();

		const int32 NumWaves = FMath::DivideAndRoundUp<int32>(BatchViews.NumElems, WaveSizeInElems);
		for (int32 Idx = 0; Idx < NumWaves; ++Idx)
		{
			int32 WaveBeginInElems = Idx * WaveSizeInElems;
			int32 WaveEndInElems   = FMath::Min(WaveBeginInElems + WaveSizeInElems, BatchViews.NumElems); 

			int32 NumWaveElems = WaveEndInElems - WaveBeginInElems;

			uint8* ResultDataBuffer       = ResultBuffer + WaveBeginInElems*BaseBytesPerElem;
			const uint8* BaseDataBuffer   = BaseBuffer + WaveBeginInElems*BaseBytesPerElem;
			const uint8* NormalDataBuffer = NormalBuffer + WaveBeginInElems*NormalBytesPerElem;

			if (Kernel.BaseFormat)
			{
				Kernel.BaseFormat(Register0.Data, BaseDataBuffer, NumWaveElems);
				BaseDataBuffer = Register0.Data;
			}

			if (Kernel.NormalFormat)
			{
				Kernel.NormalFormat(Register1.Data, NormalDataBuffer, NumWaveElems);
				NormalDataBuffer = Register1.Data;
			}

			const uint8* RoughnessDataBuffer = BaseDataBuffer;
			if (Kernel.ComputeRoughness)
			{
				Kernel.ComputeRoughness(Register2.Data, BaseDataBuffer, NormalDataBuffer, Kernel.Power, NumWaveElems);
				RoughnessDataBuffer = Register2.Data;
			}

			const uint8* SelectDataBuffer = BaseDataBuffer;
			if (Kernel.SelectChannel)
			{
				uint8* MutableSelectDataBuffer = Kernel.ResultFormat ? Register1.Data : ResultDataBuffer;
				SelectDataBuffer = MutableSelectDataBuffer;
				
				Kernel.SelectChannel(MutableSelectDataBuffer, BaseDataBuffer, RoughnessDataBuffer, NumWaveElems); 
			}

			if (Kernel.ResultFormat)
			{
				Kernel.ResultFormat(ResultDataBuffer, SelectDataBuffer, NumWaveElems); 
			}
			else if (!Kernel.SelectChannel && ResultDataBuffer != BaseDataBuffer)
			{
				FMemory::Memcpy(ResultDataBuffer, BaseDataBuffer, NumWaveElems*BaseBytesPerElem);
			}
		}
	}

	FORCENOINLINE int32 GetNumBatchesLODRangeOffsetViews(const FBatchArgs& Args)
	{
		const bool bOnlyFirstLOD = Args.FirstLODOffsetInElems >= 0;

		check(Args.Result);
		const int32 NumBatches = bOnlyFirstLOD
				? Args.Result->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.ResultBytesPerElem, Args.FirstLODOffsetInElems*Args.ResultBytesPerElem)
				: Args.Result->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.ResultBytesPerElem, Args.LODBegin, Args.LODEnd);

#if DO_CHECK
		if (Args.Base)
		{
			const int32 BaseNumBatches = bOnlyFirstLOD
					? Args.Base->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.BaseBytesPerElem, Args.FirstLODOffsetInElems*Args.BaseBytesPerElem)
					: Args.Base->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.BaseBytesPerElem, Args.LODBegin, Args.LODEnd);

			check(NumBatches == BaseNumBatches);
		}
	
		if (Args.Normal)
		{
			const int32 BlendNumBatches = bOnlyFirstLOD
					? Args.Normal->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.NormalBytesPerElem, Args.FirstLODOffsetInElems*Args.NormalBytesPerElem)
					: Args.Normal->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.NormalBytesPerElem, Args.LODBegin, Args.LODEnd);

			check(NumBatches == BlendNumBatches);
		}

#endif
		return NumBatches;
	}

	FORCENOINLINE FBatchViews GetBatchLODRangeOffsetViews(int32 BatchId, const FBatchArgs& Args)
	{
		FBatchViews BatchViewsResult;
		
		const bool bOnlyFirstLOD = Args.FirstLODOffsetInElems >= 0;

		check(Args.Result);
		BatchViewsResult.Result = bOnlyFirstLOD
				? Args.Result->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.ResultBytesPerElem, Args.FirstLODOffsetInElems*Args.ResultBytesPerElem) 
				: Args.Result->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.ResultBytesPerElem, Args.LODBegin, Args.LODEnd);

		BatchViewsResult.NumElems = BatchViewsResult.Result.Num() / Args.ResultBytesPerElem;
		
		if (Args.Base)
		{
			BatchViewsResult.Base = bOnlyFirstLOD
					? Args.Base->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.BaseBytesPerElem, Args.FirstLODOffsetInElems*Args.BaseBytesPerElem) 
					: Args.Base->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.BaseBytesPerElem, Args.LODBegin, Args.LODEnd); 

			check(BatchViewsResult.NumElems == BatchViewsResult.Base.Num() / Args.BaseBytesPerElem);
		}

		if (Args.Normal)
		{
			BatchViewsResult.Normal = bOnlyFirstLOD
					? Args.Normal->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.NormalBytesPerElem, Args.FirstLODOffsetInElems*Args.NormalBytesPerElem) 
					: Args.Normal->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.NormalBytesPerElem, Args.LODBegin, Args.LODEnd); 

			check(BatchViewsResult.NumElems == BatchViewsResult.Normal.Num() / Args.NormalBytesPerElem);
		}

		return BatchViewsResult;
	}

} // namespace OpImageApplyCompositeInternal

	void ImageNormalComposite(FImage* ResultImage, const FImage* BaseImage, const FImage* NormalImage, int32 NormalRoughnessOutputChannel, float Power)
	{
		check(BaseImage);
		check(ResultImage);
		check(NormalImage);

		check(BaseImage->GetSize() == ResultImage->GetSize());
		check(BaseImage->GetSize() == NormalImage->GetSize());

		check(
			NormalImage->GetFormat() == EImageFormat::RGB_UByte || 
			NormalImage->GetFormat() == EImageFormat::RGBA_UByte);
		check(
			BaseImage->GetFormat() == EImageFormat::RGB_UByte || 
			BaseImage->GetFormat() == EImageFormat::RGBA_UByte);
		
		check(BaseImage->GetFormat() == ResultImage->GetFormat());

		OpImageApplyCompositeInternal::FMakeNormalCompositeKernelArgs Args =
		{
			.ResultImage = ResultImage,
			.BaseImage   = BaseImage,
			.NormalImage = NormalImage,

			.NormalRoughnessOutputChannel = NormalRoughnessOutputChannel,
			.Power = Power,
		};

		OpImageApplyCompositeInternal::FNormalCompositeKernel Kernel = 
				OpImageApplyCompositeInternal::MakeNormalCompositeKernel(Args);


		const uint32 BaseBytesPerElem = GetImageFormatData(BaseImage->GetFormat()).BytesPerBlock;
		const uint32 NormalBytesPerElem = GetImageFormatData(NormalImage->GetFormat()).BytesPerBlock;

		OpImageApplyCompositeInternal::FBatchArgs BatchArgs
		{
			.BatchNumElems = 1 << 12,
			.LODBegin      = 0,
			.LODEnd        = BaseImage->GetLODCount(),
			.FirstLODOffsetInElems = -1,

			.Result = ResultImage,
			.Base   = BaseImage  ,
			.Normal = NormalImage,

		    .ResultBytesPerElem = BaseBytesPerElem,
		    .BaseBytesPerElem   = BaseBytesPerElem,
		    .NormalBytesPerElem = NormalBytesPerElem,
		};

		const int32 NumBatches = OpImageApplyCompositeInternal::GetNumBatchesLODRangeOffsetViews(BatchArgs);
		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches, 
		[
			&BatchArgs, &Kernel
		] (int32 BatchId)
		{
			OpImageApplyCompositeInternal::FBatchViews BatchViews = 
					OpImageApplyCompositeInternal::GetBatchLODRangeOffsetViews(BatchId, BatchArgs);

			OpImageApplyCompositeInternal::RunKernelOnBatch(Kernel, BatchViews);
		});
	}
} // namespace UE::Mutable::Private
