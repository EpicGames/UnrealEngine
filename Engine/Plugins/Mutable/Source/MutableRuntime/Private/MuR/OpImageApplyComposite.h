// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"

namespace UE::Mutable::Private
{
	void ImageNormalComposite(FImage* ResultImage, const FImage* BaseImage, const FImage* NormalImage, int32 NormalRoughnessOutputChannel, float Power);
}
