// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"

#include "MuR/ParallelExecutionUtils.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	inline void ImageBinarise(FImage* DestImage, const FImage* AImage, float Threshold)
	{
		if (!AImage || !DestImage)
		{
			return;
		}

		check(DestImage->GetFormat() == EImageFormat::L_UByte);

		// Generic implementation
        const uint16 ThresholdUNorm = static_cast<uint16>(FMath::Clamp(Threshold, 0.0f, 1.0f) * 255.0f);

		constexpr int32 BatchNumElems = 1 << 14;
		const int32 BytesPerElem = GetImageFormatData(AImage->GetFormat()).BytesPerBlock;
		
		const int32 NumBatches = AImage->DataStorage.GetNumBatches(BatchNumElems, BytesPerElem);
		check(DestImage->DataStorage.GetNumBatches(BatchNumElems, 1) == NumBatches);

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[
			DestImage, AImage, ThresholdUNorm, BatchNumElems, BytesPerElem
		](int32 BatchId)
		{
			TArrayView<uint8> DestView = DestImage->DataStorage.GetBatch(BatchId, BatchNumElems, 1);	
			TArrayView<const uint8> AView = AImage->DataStorage.GetBatch(BatchId, BatchNumElems, BytesPerElem);	

			const int32 NumElems = DestView.Num();
			check(AView.Num() / BytesPerElem == NumElems);

			uint8* DestBuf = DestView.GetData();
			const uint8* ABuf = AView.GetData();

			switch(AImage->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					DestBuf[I] = ABuf[I] >= ThresholdUNorm ? 255 : 0;
				}
				break;
			}

			case EImageFormat::RGB_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					uint16 Value = ABuf[3*I + 0];
					Value += ABuf[3*I + 1];
					Value += ABuf[3*I + 2];
					DestBuf[I] = (Value / 3) >= ThresholdUNorm ? 255 : 0;
				}
				break;
			}

			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					uint16 Value = ABuf[4*I + 0];
					Value += ABuf[4*I + 1];
					Value += ABuf[4*I + 2];
					DestBuf[I] = (Value / 3) >= ThresholdUNorm ? 255 : 0;
				}
				break;
			}

			default:
				check(false);
			}
		});
	}

}
