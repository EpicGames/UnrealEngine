// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "Math/VectorRegister.h"

namespace UE::Mutable::Private
{

	inline uint32 BlendChannel( uint32 // base
		, uint32 blended )
	{
		return blended;
	}

	inline uint32 BlendChannelMasked( uint32 base, uint32 blended, uint32 mask )
	{
		uint32 blend = blended;
		uint32 masked = ( ( ( 255 - mask ) * base ) + ( mask * blend ) ) / 255;
		return masked;
	}


	//---------------------------------------------------------------------------------------------
	inline uint32 ScreenChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		// R = 1 - (1-Base) × (1-Blend)
		uint32 screen = 255 - (((255 - base) * (255 - blended)) >> 8);
		uint32 masked = (((255 - mask) * base) + (mask * screen)) >> 8;
		return masked;
	}

	inline uint32 ScreenChannel(uint32 base, uint32 blended)
	{
		// R = 1 - (1-Base) × (1-Blend)
		uint32 screen = 255 - (((255 - base) * (255 - blended)) >> 8);
		return screen;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 SoftLightChannel(uint32 base, uint32 blended)
	{
		// gimp-like
		uint32 mix = (base * blended) >> 8;
		uint32 screen = 255 - (((255 - base) * (255 - blended)) >> 8);
		uint32 softlight = (((255 - base) * mix) + (base * screen)) >> 8;
		return softlight;
	}


	inline uint32 SoftLightChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		uint32 softlight = SoftLightChannel(base, blended);
		uint32 masked = (((255 - mask) * base) + (mask * softlight)) >> 8;
		return masked;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 HardLightChannel(uint32 base, uint32 blended)
	{
		// gimp-like


		// photoshop-like
		// if (Blend > ½) R = 1 - (1-Base) × (1-2×(Blend-½))
		// if (Blend <= ½) R = Base × (2×Blend)
		uint32 hardlight = blended > 128
			? 255 - (((255 - base) * (255 - 2 * (blended - 128))) >> 8)
			: (base * (2 * blended)) >> 8;

		hardlight = FMath::Min(255u, hardlight);

		return hardlight;
	}

	inline uint32 HardLightChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		uint32 hardlight = HardLightChannel(base, blended);

		uint32 masked = (((255 - mask) * base) + (mask * hardlight)) >> 8;
		return masked;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 BurnChannel(uint32 base, uint32 blended)
	{
		// R = 1 - (1-Base) / Blend
		uint32 burn =
			FMath::Min(255,
				FMath::Max(0,
					255 - (((255 - (int)base) << 8) / ((int)blended + 1))
				)
			);
		return burn;
	}

	inline uint32 BurnChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		uint32 burn = BurnChannel(base, blended);
		uint32 masked = (((255 - mask) * base) + (mask * burn)) >> 8;
		return masked;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 DodgeChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		// R = Base / (1-Blend)
		uint32 dodge = (base << 8) / (256 - blended);
		uint32 masked = (((255 - mask) * base) + (mask * dodge)) >> 8;
		return masked;
	}

	inline uint32 DodgeChannel(uint32 base, uint32 blended)
	{
		// R = Base / (1-Blend)
		uint32 dodge = (base << 8) / (256 - blended);
		return dodge;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 LightenChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		uint32 overlay = base + (blended * uint32(255 - base) >> 8);
		uint32 masked = (((255 - mask) * base) + (mask * overlay)) >> 8;
		return masked;
	}	

	inline uint32 LightenChannel(uint32 base, uint32 blended)
	{
		uint32 overlay = base + (blended * uint32(255 - base) >> 8);
		return overlay;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 MultiplyChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		uint32 multiply = (base * blended) / 255;
		uint32 masked = (((255 - mask) * base) + (mask * multiply)) / 255;
		return masked;
	}

	inline uint32 MultiplyChannel(uint32 base, uint32 blended)
	{
		uint32 multiply = (base * blended) / 255;
		return multiply;
	}

	//---------------------------------------------------------------------------------------------
	inline uint32 OverlayChannelMasked(uint32 base, uint32 blended, uint32 mask)
	{
		uint32 overlay = (base * (base + ((2 * blended * (255 - base)) >> 8))) >> 8;
		uint32 masked = (((255 - mask) * base) + (mask * overlay)) >> 8;
		return masked;
	}


	inline uint32 OverlayChannel(uint32 base, uint32 blended)
	{
		uint32 overlay = (base * (base + ((2 * blended * (255 - base)) >> 8))) >> 8;
		return overlay;
	}

	//---------------------------------------------------------------------------------------------
	FORCEINLINE VectorRegister4Int VectorBlendChannelMasked(
		const VectorRegister4Int& Base, const VectorRegister4Int& Blended, const VectorRegister4Int& Mask)
	{
		const VectorRegister4Int Value = VectorIntAdd(
				VectorIntMultiply(Base, VectorIntSubtract(MakeVectorRegisterIntConstant(255, 255, 255, 255), Mask)),
				VectorIntMultiply(Blended, Mask));

		// fast division by 255 assuming Value is in the range [0, (1 << 16)]
		return VectorShiftRightImmLogical(
				VectorIntMultiply(Value, MakeVectorRegisterIntConstant(32897, 32897, 32897, 32897)),
				23);
	}

	FORCEINLINE int32 VectorLightenChannel(int32 Base, int32 Blended)
	{
		return Base + (Blended * (255 - Base) >> 8);
	}

	// Combine layer oprations get packed color.
	inline uint32 CombineNormal(uint32 base, uint32 blended)
	{
		// See /Engine/Functions/Engine_MaterialFunctions02/Utility/BlendAngleCorrectedNormals.BlendAngleCorrectedNormals
		// for the source of the effect.

		const float baseRf = (static_cast<float>((base >> 0)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float baseGf = (static_cast<float>((base >> 8)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float baseBf = (static_cast<float>((base >> 16) & 0xFF) / 255.0f) * 2.0f; //One added to the b channel.

		const float blendedRf = (static_cast<float>((blended >> 0)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float blendedGf = (static_cast<float>((blended >> 8)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float blendedBf = (static_cast<float>((blended >> 16) & 0xFF) / 255.0f) * 2.0f - 1.0f;

		FVector3f a = FVector3f(baseRf, baseGf, baseBf);
		FVector3f b = FVector3f(-blendedRf, -blendedGf, blendedBf);

		// The original does not normalize but if not done results don't look good due to signal clipping.
		FVector3f n = (a* FVector3f::DotProduct(a, b) - b*a.Z);
		n.Normalize();

		return 
			(FMath::Clamp(static_cast<uint32>((n.X + 1.0f) * 0.5f * 255.0f), 0u, 255u) << 0) |
			(FMath::Clamp(static_cast<uint32>((n.Y + 1.0f) * 0.5f * 255.0f), 0u, 255u) << 8) |
			(FMath::Clamp(static_cast<uint32>((n.Z + 1.0f) * 0.5f * 255.0f), 0u, 255u) << 16);
	}

	inline uint32 CombineNormalMasked(uint32 base, uint32 blended, uint32 mask)
	{
		// See /Engine/Functions/Engine_MaterialFunctions02/Utility/BlendAngleCorrectedNormals.BlendAngleCorrectedNormals
		// for the source of the effect.

		const float baseRf = (static_cast<float>((base >> 0)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float baseGf = (static_cast<float>((base >> 8)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float baseBf = (static_cast<float>((base >> 16) & 0xFF) / 255.0f) * 2.0f - 1.0f;

		const float blendedRf = (static_cast<float>((blended >> 0)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float blendedGf = (static_cast<float>((blended >> 8)  & 0xFF) / 255.0f) * 2.0f - 1.0f;
		const float blendedBf = (static_cast<float>((blended >> 16) & 0xFF) / 255.0f) * 2.0f - 1.0f;

		const float maskRf = (static_cast<float>(mask & 0xFF) / 255.0f);

		FVector3f a = FVector3f(baseRf, baseGf, baseBf + 1.0f);
		FVector3f b = FVector3f(-blendedRf, -blendedGf, blendedBf);

		// The original does not normalize but if not done results don't look good due to signal clipping.
		FVector3f n = ( FMath::Lerp( FVector3f(baseRf, baseGf, baseBf), a*FVector3f::DotProduct(a, b) - b*a.Z, maskRf ) );
		n.Normalize();

		return 
			(FMath::Clamp(static_cast<uint32>((n.X + 1.0f) * 0.5f * 255.0f), 0u, 255u) << 0) |
			(FMath::Clamp(static_cast<uint32>((n.Y + 1.0f) * 0.5f * 255.0f), 0u, 255u) << 8) |
			(FMath::Clamp(static_cast<uint32>((n.Z + 1.0f) * 0.5f * 255.0f), 0u, 255u) << 16);
	}


	// Combine layer oprations get packed color.
	inline uint32 NormalComposite(
			const uint32 Base, 
			const uint32 Normal, 
			const uint8 Channel, 
			const float Power )
	{
		const int32 ChannelShift = Channel << 3;

		float ReferencValue = static_cast<float>(Base >> ChannelShift & 0xFF) / 255.0f;

		// Rougness is stored as Unorm8
		float Roughness = static_cast<float>( (Base >> ChannelShift) & 0xFF ) / 255.0f;

		FVector3f N = ( FVector3f( static_cast<float>((Normal >> 0 ) & 0xFF) / 255.0f,
							       static_cast<float>((Normal >> 8 ) & 0xFF) / 255.0f,
							       static_cast<float>((Normal >> 16) & 0xFF) / 255.0f ) * 2.0f - 1.0f );	
	
		// See TexturecCompressorModule.cpp::ApplyCompositeTexture
		// Toksvig estimation of variance
		float LengthN = FMath::Min( N.Size(), 1.0f );
		float Variance = ( 1.0f - LengthN ) / LengthN;
		Variance = FMath::Max( 0.0f, Variance - 0.00004f );

		Variance *= Power;
		
		float a = Roughness * Roughness;
		float a2 = a * a;
		float B = 2.0f * Variance * (a2 - 1.0f);
		a2 = ( B - a2 ) / ( B - 1.0f );
		Roughness = FMath::Pow( a2, 0.25f );

		const uint32 ResMask = ~(0xFF << ChannelShift);
		const uint32 Value = FMath::Clamp<uint32>( static_cast<uint32>(Roughness * 255.0f), 0u, 255u );

		return  (Value << ChannelShift) | (Base & ResMask); 
	}

	struct FNormalCompositeIdentityFunctor
	{
		uint32 operator()(uint32 Base, uint32) const
		{
			return Base;		
		}
	};


	struct FNormalCompositeFunctor
	{
		const uint8 Channel;
		const float Power;

		uint32 operator()(uint32 Base, uint32 Normal) const
		{
			return NormalComposite(Base, Normal, Channel, Power);
		}
	};
}
