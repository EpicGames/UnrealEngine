// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"

#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{

	inline void ImageColorMap(FImage* DestImage, const FImage* SourceImage, const FImage* MaskImage, const FImage* MapImage, bool bOnlyOneMip)
	{
		check(SourceImage->GetSizeX() == MaskImage->GetSizeX());
		check(SourceImage->GetSizeY() == MaskImage->GetSizeY());
		check(SourceImage->GetLODCount() == MaskImage->GetLODCount() || bOnlyOneMip);
		check(SourceImage->GetFormat() == DestImage->GetFormat());
		check(SourceImage->GetSize() == DestImage->GetSize());
		check(MaskImage->GetFormat() == EImageFormat::L_UByte);

		// Generic implementation

		// Make a palette for faster conversion
        uint8 Palette[256][4];
		for (int32 I = 0; I < 256; ++I)
		{
			FVector4f Color = MapImage->Sample(FVector2f(float(I)/255.0f, 0.0f));
            Palette[I][0] = (uint8)FMath::Max(0, FMath::Min(255, int32(Color[0]*255.0f)));
            Palette[I][1] = (uint8)FMath::Max(0, FMath::Min(255, int32(Color[1]*255.0f)));
            Palette[I][2] = (uint8)FMath::Max(0, FMath::Min(255, int32(Color[2]*255.0f)));
            Palette[I][3] = (uint8)FMath::Max(0, FMath::Min(255, int32(Color[3]*255.0f)));
		}

		const int32 LODBegin = 0;
		const int32 LODEnd = bOnlyOneMip ? 1 : DestImage->GetLODCount();
	
		constexpr int32 NumBatchElems = 1 << 14;
		const int32 BytesPerElem = GetImageFormatData(DestImage->GetFormat()).BytesPerBlock;
		check(GetImageFormatData(SourceImage->GetFormat()).BytesPerBlock == BytesPerElem);

		const int32 NumBatches = DestImage->DataStorage.GetNumBatchesLODRange(NumBatchElems, BytesPerElem, LODBegin, LODEnd); 
		check(NumBatches == SourceImage->DataStorage.GetNumBatchesLODRange(NumBatchElems, BytesPerElem, LODBegin, LODEnd));

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[
			DestImage, SourceImage, MaskImage, Palette, NumBatchElems, BytesPerElem, LODBegin, LODEnd
		](int32 BatchId)
		{
			TArrayView<uint8> DestView = DestImage->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, BytesPerElem, LODBegin, LODEnd); 
			TArrayView<const uint8> SourceView = SourceImage->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, BytesPerElem, LODBegin, LODEnd); 
			TArrayView<const uint8> MaskView = MaskImage->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, 1, LODBegin, LODEnd); 

			const int32 NumElems = DestView.Num() / BytesPerElem;
			check(SourceView.Num() / BytesPerElem == NumElems);
			check(MaskView.Num() / 1 == NumElems);

			uint8* DestBuf = DestView.GetData();
			const uint8* SourceBuf = SourceView.GetData();
			const uint8* MaskBuf = MaskView.GetData();

			switch (SourceImage->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					if (MaskBuf[I] > 127)
					{
						DestBuf[I] = Palette[SourceBuf[I]][0];
					}
					else
					{
						DestBuf[I] = SourceBuf[I];
					}
				}
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					if (MaskBuf[I])
					{
						DestBuf[3*I + 0] = Palette[SourceBuf[3*I + 0]][0];
						DestBuf[3*I + 1] = Palette[SourceBuf[3*I + 1]][1];
						DestBuf[3*I + 2] = Palette[SourceBuf[3*I + 2]][2];
					}
					else
					{
						DestBuf[3*I + 0] = SourceBuf[3*I + 0];
						DestBuf[3*I + 1] = SourceBuf[3*I + 1];
						DestBuf[3*I + 2] = SourceBuf[3*I + 2];
					}
				}
				break;
			}

			case EImageFormat::RGBA_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					if (MaskBuf[I] > 127)
					{
						DestBuf[4*I + 0] = Palette[SourceBuf[4*I + 0]][0];
						DestBuf[4*I + 1] = Palette[SourceBuf[4*I + 1]][1];
						DestBuf[4*I + 2] = Palette[SourceBuf[4*I + 2]][2];
						DestBuf[4*I + 3] = Palette[SourceBuf[4*I + 3]][3];
					}
					else
					{
						DestBuf[4*I + 0] = SourceBuf[4*I + 0];
						DestBuf[4*I + 1] = SourceBuf[4*I + 1];
						DestBuf[4*I + 2] = SourceBuf[4*I + 2];
						DestBuf[4*I + 3] = SourceBuf[4*I + 3];
					}
				}
				break;
			}

			case EImageFormat::BGRA_UByte:
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					if (MaskBuf[I] > 127)
					{
						DestBuf[4*I + 0] = Palette[SourceBuf[4*I + 0]][2];
						DestBuf[4*I + 1] = Palette[SourceBuf[4*I + 1]][1];
						DestBuf[4*I + 2] = Palette[SourceBuf[4*I + 2]][0];
						DestBuf[4*I + 3] = Palette[SourceBuf[4*I + 3]][3];
					}
					else
					{
						DestBuf[4*I + 0] = SourceBuf[4*I + 0];
						DestBuf[4*I + 1] = SourceBuf[4*I + 1];
						DestBuf[4*I + 2] = SourceBuf[4*I + 2];
						DestBuf[4*I + 3] = SourceBuf[4*I + 3];
					}
				}
				break;
			}

			default:
				check(false);
			}
		});
	}

}
