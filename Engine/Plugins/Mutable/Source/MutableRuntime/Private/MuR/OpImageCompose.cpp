// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImagePrivate.h"
#include "MuR/MutableMath.h"


namespace UE::Mutable::Private
{

	void FImageOperator::ImageCompose(FImage* Base, const FImage* Block, const box< FIntVector2 >& Rect)
	{
		check(Base && Block);
		check(Base != Block);
		check(Rect.min[0] >= 0 && Rect.min[1] >= 0);
		check(Rect.size[0] >= 0 && Rect.size[1] >= 0);
		check(Rect.min[0] + Rect.size[0] <= Base->GetSizeX());
		check(Rect.min[1] + Rect.size[1] <= Base->GetSizeY());
		check(Rect.size[0] == Block->GetSizeX());
		check(Rect.size[1] == Block->GetSizeY());
		check(Base->GetFormat() == Block->GetFormat());

		// This assert may be acceptable if the missing lods are due to the image block format size
		// limiting it, or disadjustment due to image scaling down. It is handled below.
		//check( pBase->GetLODCount()<=pBlock->GetLODCount() );

		EImageFormat OriginalFormat = Base->GetFormat();
		const FImageFormatData& FormatInfo = GetImageFormatData(OriginalFormat);

		// If this is true, we are composing coordinates or sizes that don't fall on pixel format
		// block borders.
		bool bSubPixelBlockCompose = 
			Rect.min[0] % FormatInfo.PixelsPerBlockX != 0
			|| Rect.min[1] % FormatInfo.PixelsPerBlockY != 0
			|| Rect.size[0] % FormatInfo.PixelsPerBlockX != 0
			|| Rect.size[1] % FormatInfo.PixelsPerBlockY != 0;

		if (bSubPixelBlockCompose)
		{
			// Bad performance case, but likely a very small image anyway.
			MUTABLE_CPUPROFILER_SCOPE(ImageCompose_SubBlock_Fix);
			EImageFormat UncompressedFormat = GetUncompressedFormat(OriginalFormat);
			constexpr int32 Quality = 0;
			TManagedPtr<FImage> TempBase = ImagePixelFormat(Quality, Base, UncompressedFormat);
			TManagedPtr<FImage> TempBlock = ImagePixelFormat(Quality, Block, UncompressedFormat);
			ImageCompose(TempBase.Get(), TempBlock.Get(), Rect);
			TManagedPtr<FImage> NewBase = ImagePixelFormat(Quality, TempBase.Get(), OriginalFormat);
			// ImageCompose above may have reduced the LODs because of the block lods.
			Base->CopyMove(NewBase.Get());
			return;
		}
		else
		{
			// Sizes in blocks of the current mips
			UE::Math::TIntVector2<uint16> BaseMipSize(
				FMath::DivideAndRoundUp(Base->GetSizeX(), uint16(FormatInfo.PixelsPerBlockX)),
				FMath::DivideAndRoundUp(Base->GetSizeY(), uint16(FormatInfo.PixelsPerBlockY)));

			UE::Math::TIntVector2<uint16> BlockMipPos(
				Rect.min[0] / FormatInfo.PixelsPerBlockX,
				Rect.min[1] / FormatInfo.PixelsPerBlockY);

			UE::Math::TIntVector2<uint16> BlockMipSize(
				FMath::DivideAndRoundUp(Rect.size[0], int32(FormatInfo.PixelsPerBlockX)),
				FMath::DivideAndRoundUp(Rect.size[1], int32(FormatInfo.PixelsPerBlockY)));

			int32 DoneMips = 0;
			for (; DoneMips < Base->GetLODCount() && DoneMips < Block->GetLODCount(); ++DoneMips)
			{
				//UE_LOGF(LogMutableCore, Warning, "Block Mip Pos : %d, %d", blockMipPos[0], blockMipPos[1]);

				// Sizes of a row in bytes
				int32 BaseRowSize = FormatInfo.BytesPerBlock * BaseMipSize[0];
				int32 BlockRowSize = FormatInfo.BytesPerBlock * BlockMipSize[0];

				TArrayView<uint8> BaseLODView = Base->DataStorage.GetLOD(DoneMips);
				TArrayView<const uint8> BlockLODView = Block->DataStorage.GetLOD(DoneMips);

				uint8* BaseBuf = BaseLODView.GetData();
				const uint8* BlockBuf = BlockLODView.GetData();

				const int32 SkipBytes = BaseRowSize * BlockMipPos[1] + BlockMipPos[0] * FormatInfo.BytesPerBlock;

				BaseBuf += SkipBytes;
				for (int32 Y = 0; Y < BlockMipSize.Y; ++Y)
				{
					check(BaseBuf + BlockRowSize <= BaseLODView.GetData() + BaseLODView.Num());
					check(BlockBuf + BlockRowSize <= BlockLODView.GetData() + BlockLODView.Num());
					FMemory::Memcpy(BaseBuf, BlockBuf, BlockRowSize);

					BlockBuf += BlockRowSize;
					BaseBuf += BaseRowSize;
				}

				// We may not proceed to the next mip, if the layout block size is smaller than the
				// pixel format block size.
				if (BlockMipPos[0] % 2 || BlockMipPos[1] % 2 || BlockMipSize[0] % 2 || BlockMipSize[1] % 2)
				{
					++DoneMips;
					break;
				}

				BaseMipSize[0] =  FMath::DivideAndRoundUp(BaseMipSize[0], uint16(2));
				BaseMipSize[1] =  FMath::DivideAndRoundUp(BaseMipSize[1], uint16(2));
				BlockMipPos[0] =  FMath::DivideAndRoundUp(BlockMipPos[0], uint16(2));
				BlockMipPos[1] =  FMath::DivideAndRoundUp(BlockMipPos[1], uint16(2));
				BlockMipSize[0] = FMath::DivideAndRoundUp(BlockMipSize[0], uint16(2));
				BlockMipSize[1] = FMath::DivideAndRoundUp(BlockMipSize[1], uint16(2));
			}

			// Adjust the actually valid mips.
			Base->DataStorage.SetNumLODs(DoneMips);
		}
	}

}

