// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImagePrivate.h"
#include "MuR/MutableMath.h"
#include "MuR/Platform.h"

#include "Math/IntRect.h"

namespace UE::Mutable::Private
{
	bool FImageOperator::ImageCrop(TManagedPtr<FImage>& OutCropped, int32 CompressionQuality, const FImage* InBase, const box< FIntVector2 >& Rect)
	{
		check(InBase);

		TManagedPtr<FImage> BaseReformat;
		TManagedPtr<FImage> Cropped = OutCropped;

		EImageFormat BaseFormat = InBase->GetFormat();
		EImageFormat UncompressedFormat = GetUncompressedFormat(BaseFormat);

		if (BaseFormat != UncompressedFormat)
		{
			// Compressed formats need decompression + compression after crop			
			// \TODO: This may use some additional untracked memory locally in this function.
			BaseReformat = ImagePixelFormat(CompressionQuality, InBase, UncompressedFormat);
			InBase = BaseReformat.Get();
			Cropped = CreateImage(OutCropped->GetSizeX(), OutCropped->GetSizeY(), OutCropped->GetLODCount(), UncompressedFormat, EInitializationType::NotInitialized);
        }

		// In case the base is smaller than the crop extension, make it as if the base extends indefinitely
		// with black pixels.
		if (Rect.min.X + Rect.size.X > InBase->GetSizeX() || Rect.min.Y + Rect.size.Y > InBase->GetSizeY())
		{
			Cropped->InitToBlack();			
		}

		const FImageFormatData& finfo = GetImageFormatData(UncompressedFormat);

		check(Cropped);
		check(Cropped->GetSizeX() == Rect.size[0]);
		check(Cropped->GetSizeY() == Rect.size[1]);

		// TODO: better error control. This happens if some layouts are corrupt.
		bool bCorrect =
				(Rect.min[0] >= 0 && Rect.min[1] >= 0) &&
				(Rect.size[0] >= 0 && Rect.size[1] >= 0) &&
				(Rect.min[0] + Rect.size[0] <= InBase->GetSizeX()) &&
				(Rect.min[1] + Rect.size[1] <= InBase->GetSizeY());

		if (!bCorrect)
		{
			return false;
		}

		// Block images are not supported for now
		check(finfo.PixelsPerBlockX == 1);
		check(finfo.PixelsPerBlockY == 1);

		checkf(Rect.min[0] % finfo.PixelsPerBlockX == 0, TEXT("Rect must snap to blocks."));
		checkf(Rect.min[1] % finfo.PixelsPerBlockY == 0, TEXT("Rect must snap to blocks."));
		checkf(Rect.size[0] % finfo.PixelsPerBlockX == 0, TEXT("Rect must snap to blocks."));
		checkf(Rect.size[1] % finfo.PixelsPerBlockY == 0, TEXT("Rect must snap to blocks."));
	
		const uint32 BytesPerPixel = finfo.BytesPerBlock;

        const uint8* BaseBuf = InBase->GetLODData(0);
        uint8* CropBuf = Cropped->GetLODData(0);

		FImageSize BaseSize = InBase->GetSize();
		FImageSize CroppedSize = Cropped->GetSize();

		FIntRect BaseRect = FIntRect(
				FMath::Min<int32>(BaseSize.X, Rect.min[0]),
				FMath::Min<int32>(BaseSize.Y, Rect.min[1]),
				FMath::Min<int32>(BaseSize.X, Rect.min[0] + Rect.size[0]),
				FMath::Min<int32>(BaseSize.Y, Rect.min[1] + Rect.size[1]));

		const uint32 NumBytesPerRow = FMath::Max<uint32>(0, (BaseRect.Max.X - BaseRect.Min.X) * BytesPerPixel);
		for (int32 Y = 0; Y < BaseRect.Max.Y - BaseRect.Min.Y; ++Y)
		{
			FMemory::Memcpy(
				CropBuf + CroppedSize.X*Y*BytesPerPixel, 
				BaseBuf + (BaseSize.X*(Y + BaseRect.Min.Y) + BaseRect.Min.X)*BytesPerPixel, 
				NumBytesPerRow);
		}

		if (BaseFormat != UncompressedFormat)
		{
			bool bSuccess = false;
			ImagePixelFormat(bSuccess, CompressionQuality, OutCropped.Get(), Cropped.Get());
			check(bSuccess);
		}

		return true;
	}
}
