// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/Platform.h"
#include "MuR/ConvertData.h"
#include "Async/ParallelFor.h"
#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{

    inline uint8 MutableEncodeOffset(int32 X, int32 Y)
	{
        uint8 C = uint8((7 << 4) | 7);
		if ((X < 8) & (X > -8) & (Y < 8) & (Y > -8))
		{
            C = uint8(((X + 7) << 4) | (Y + 7));
		}

		return C;
	}

    inline void MutableDecodeOffset(uint8 C, int32& X, int32& Y)
	{
		X = int32(C >> 4) - 7;
		Y = int32(C & 0xF) - 7;
	}

	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	inline void ImageMakeGrowMap(FImage* ResultImage, FImage* MaskImage, int32 InBorder)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageMakeGrowMap);

		check(MaskImage->GetFormat() == EImageFormat::L_UByte);
		check(ResultImage->GetFormat() == EImageFormat::L_UByte);
		check(ResultImage->GetSizeX() == MaskImage->GetSizeX());
		check(ResultImage->GetSizeY() == MaskImage->GetSizeY());
		check(ResultImage->GetLODCount() == MaskImage->GetLODCount());

		int32 MipCount = ResultImage->GetLODCount();
		int32 SizeX = ResultImage->GetSizeX();
		int32 SizeY = ResultImage->GetSizeY();
		
		if (SizeX <= 0 || SizeY <= 0)
		{
			return;
		}

		const uint8 ZeroOffset = MutableEncodeOffset(0, 0);
		for (FImageArray& Buffer : ResultImage->DataStorage.Buffers)
		{
			FMemory::Memset(Buffer.GetData(), ZeroOffset, Buffer.Num());
		}

		TManagedPtr<FImage> NextMask = MaskImage->Clone();

		for (int32 B = 0; B < InBorder; ++B)
		{
			if (B > 0)
			{
				MaskImage->DataStorage = NextMask->DataStorage;
			}

			// TODO: Replace with a single parallel for.
			ParallelExecutionUtils::InvokeBatchParallelFor(MipCount, [MaskImage, NextMask, ResultImage](int32 MipIndex)
				{
					uint8* ResultData = ResultImage->GetMipData(MipIndex);
					const uint8* MaskData = MaskImage->GetMipData(MipIndex);
					uint8* NextMaskData = NextMask->GetMipData(MipIndex);

					FIntVector2 MipSize = ResultImage->CalculateMipSize(MipIndex);

					const int32 NumRowsPerBatch = FMath::Min(FMath::DivideAndRoundUp((1 << 14), MipSize.X), MipSize.Y);
					const int32 NumBatches = FMath::DivideAndRoundUp<int32>(MipSize.Y, NumRowsPerBatch);

					ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches, [
						MaskData, NextMaskData, MipSize, ResultData, NumBatches, NumRowsPerBatch
					] (int32 BatchIndex)
						{
							const int32 BatchBeginRowY = BatchIndex * NumRowsPerBatch;
							const int32 BatchEndRowY = FMath::Min(BatchBeginRowY + NumRowsPerBatch, MipSize.Y);

							uint8* NextData = NextMaskData + MipSize.X * BatchBeginRowY;

							for (int32 Y = BatchBeginRowY; Y < BatchEndRowY; ++Y)
							{
								const uint8* S0 = Y > 0 ? MaskData + (Y - 1) * MipSize.X : nullptr;
								const uint8* S1 = MaskData + Y * MipSize.X;
								bool bNotLastRow = Y < MipSize.Y - 1;
								const uint8* S2 = bNotLastRow ? MaskData + (Y + 1) * MipSize.X : nullptr;

								const uint8* R0 = Y > 0 ? ResultData + (Y - 1) * MipSize.X : 0;
								uint8* R1 = ResultData + Y * MipSize.X;

								const uint8* R2 = bNotLastRow ? ResultData + (Y + 1) * MipSize.X : nullptr;

								for (int32 X = 0; X < MipSize.X; ++X)
								{
									int32 Dx, Dy;
									if (S1[X])
									{
										*NextData = 255;
									}
									else if (Y > 0 && S0[X])
									{
										MutableDecodeOffset(R0[X], Dx, Dy);
										R1[X] = MutableEncodeOffset(Dx, Dy - 1);
										*NextData = 255;
									}
									else if (bNotLastRow && S2 && R2 && S2[X])
									{
										MutableDecodeOffset(R2[X], Dx, Dy);
										R1[X] = MutableEncodeOffset(Dx, Dy + 1);
										*NextData = 255;
									}
									else if (X < MipSize.X - 1 && S1[X + 1])
									{
										MutableDecodeOffset(R1[X + 1], Dx, Dy);
										R1[X] = MutableEncodeOffset(Dx + 1, Dy);
										*NextData = 255;
									}
									else if (X > 0 && S1[X - 1])
									{
										MutableDecodeOffset(R1[X - 1], Dx, Dy);
										R1[X] = MutableEncodeOffset(Dx - 1, Dy);
										*NextData = 255;
									}

									++NextData;
								}
							}

						});
				});
		}
	}


	inline void ImageDisplace(FImage* ResultImage, const FImage* SourceImage, const FImage* MapImage)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageDisplace)

		check(ResultImage->GetFormat() == SourceImage->GetFormat());
		check(MapImage->GetFormat() == EImageFormat::L_UByte || MapImage->GetFormat() == EImageFormat::L_UByteRLE);
		check(ResultImage->GetSizeX() == SourceImage->GetSizeX());
		check(ResultImage->GetSizeY() == SourceImage->GetSizeY());
		check(ResultImage->GetSizeX() == MapImage->GetSizeX());
		check(ResultImage->GetSizeY() == MapImage->GetSizeY());

		int32 LODCount = ResultImage->GetLODCount();
		check(LODCount <= MapImage->GetLODCount());
		check(LODCount <= SourceImage->GetLODCount());

		int32 SizeX = ResultImage->GetSizeX();
		int32 SizeY = ResultImage->GetSizeY();

        //if (SizeX < 4 || SizeY < 4)
        //{
        //    return;
        //}
        
		EImageFormat MapFormat = MapImage->GetFormat();
        bool bIsUncompressed = MapFormat == EImageFormat::L_UByte;

        if (bIsUncompressed)
		{
			switch (ResultImage->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				const uint8* SourceData = SourceImage->GetLODData(0);
				const uint8* MapBuf = MapImage->GetLODData(0);
				uint8* ResultBuf = ResultImage->GetLODData(0);

				//for (int32 Y = 0; Y < SizeY; ++Y)
				const auto ProcessRow = [
					SourceData, MapBuf, ResultBuf, SizeX, SizeY
				] (int32 Y)
				{
					const uint8* MapData = MapBuf + Y * SizeX;
					uint8* ResultData = ResultBuf + Y * SizeX;

					for (int32 X = 0; X < SizeX; ++X)
					{
						int32 Dx, Dy;
						MutableDecodeOffset(*MapData, Dx, Dy);

						// This could actually happen since we enable the crop+displace optimization
						//check(X + Dx >= 0 && X + Dx < SizeX);
						//check(Y + Dy >= 0 && Y + Dy < SizeY);
						if ((X + Dx >= 0 && X + Dx < SizeX) && (Y + Dy >= 0 && Y + Dy < SizeY))
						{
							int32 Offset = ((Y + Dy) * SizeX + (X + Dx));
							FMemory::Memcpy(ResultData, SourceData + Offset, 1);
						}

						++ResultData;
						++MapData;
					}

				};

				ParallelFor(SizeY, ProcessRow);
				break;
			}

			case EImageFormat::RGB_UByte:
			{
        		const uint8* SourceData = SourceImage->GetLODData(0);
				const uint8* MapBuf = MapImage->GetLODData(0);
				uint8* ResultBuf = ResultImage->GetLODData(0);
				//for (int y = 0; y < SizeY; ++y)
				const auto ProcessRow = [SourceData, MapBuf, ResultBuf, SizeX, SizeY] (int32 Y)
				{
					const uint8* MapData = MapBuf + Y * SizeX;
					uint8* ResultData = ResultBuf + Y * SizeX*3;

					for (int32 X = 0; X < SizeX; ++X)
					{
						int32 Dx, Dy;
						MutableDecodeOffset(*MapData, Dx, Dy);
						// This could actually happen since we enable the crop+displace optimization
						//check(X + Dx >= 0 && X + Dx < SizeX);
						//check(Y + Dy >= 0 && Y + Dy < SizeY);
						if ((X + Dx >= 0 && X + Dx < SizeX) && (Y + Dy >= 0 && Y + Dy < SizeY))
						{
							int32 Offset = ((Y + Dy) * SizeX + (X + Dx)) * 3;
							FMemory::Memcpy(ResultData, SourceData + Offset, 3);
						}

						ResultData += 3;
						++MapData;
					}

				};

				ParallelFor(SizeY, ProcessRow);
				break;
			}

			default:
			{
				// Generic
				int32 PixelSize = GetImageFormatData(ResultImage->GetFormat()).BytesPerBlock;
				
				const uint8* SourceData = SourceImage->GetLODData(0);
				const uint8* MapBuf = MapImage->GetLODData(0);
				uint8* ResultBuf = ResultImage->GetLODData(0);
				
				//for (int32 Y = 0; Y < SizeY; ++Y)
				const auto ProcessRow = [
					SourceData, MapBuf, ResultBuf, SizeX, SizeY, PixelSize
				] (int32 Y)
				{
					const uint8* MapData = MapBuf + Y * SizeX;
					uint8* ResultData = ResultBuf + Y * SizeX * PixelSize;

					for (int32 X = 0; X < SizeX; ++X)
					{
						int32 Dx, Dy;
						MutableDecodeOffset(*MapData, Dx, Dy);
						// This could actually happen since we enable the crop+displace optimization
						//check(X + Dx >= 0 && X + Dx < SizeX);
						//check(Y + Dy >= 0 && Y + Dy < SizeY);
						if ((X + Dx >= 0 && X + Dx < SizeX) && (Y + Dy >= 0 && Y + Dy < SizeY))
						{
							int32 Offset = ((Y + Dy) * SizeX + (X + Dx)) * PixelSize;
							FMemory::Memcpy(ResultData, SourceData + Offset, PixelSize);
						}

						ResultData += PixelSize;
						++MapData;
					}

				};

				ParallelFor(SizeY, ProcessRow);
			}

			}
		}

        else if (MapFormat == EImageFormat::L_UByteRLE)
		{
            int32 PixelSize = GetImageFormatData(ResultImage->GetFormat()).BytesPerBlock;
			
			// TODO: ParallelFor this
            for (int32 LOD = 0; LOD < LODCount; ++LOD)
            {
				const uint8* SourceData = SourceImage->GetLODData(LOD);
                const uint8* MapData = MapImage->GetLODData(LOD);
				uint8* ResultData = ResultImage->GetLODData(LOD);

				// Skip RLE header, total size and row sizes.
				MapData += sizeof(uint32) + SizeY*sizeof(uint32);

                for (int32 Y = 0; Y < SizeY; ++Y)
                {
                    int32 X = 0;
                    while (X < SizeX)
                    {
                        // Decode header
                        uint16 Equal = *(const uint16*)MapData;
                        MapData += 2;

                        uint8 Different = *MapData;
                        ++MapData;

                        uint8 EqualPixel = *MapData;
                        ++MapData;

                        // Equal pixels
                        {
                            int32 Dx, Dy;
                            MutableDecodeOffset(EqualPixel, Dx, Dy);
                            Dx += X;
                            Dy += Y;

							if (Dx >= 0 && Dx < SizeX && Dy >= 0 && Dy < SizeY)
							{
								int32 Offset = (Dy*SizeX + Dx) * PixelSize;
								FMemory::Memcpy(ResultData, SourceData + Offset, Equal*PixelSize);
							}

                            ResultData += Equal*PixelSize;
                            X += Equal;
                        }

                        // Different pixels
                        for (int32 I = 0; I < Different; ++I)
                        {
                            int32 Dx, Dy;
                            MutableDecodeOffset(MapData[I], Dx, Dy);
                            Dx += X;
                            Dy += Y;

							if (Dx >= 0 && Dx < SizeX && Dy >= 0 && Dy < SizeY)
							{
								int32 Offset = (Dy * SizeX + Dx) * PixelSize;
								FMemory::Memcpy(ResultData, &SourceData[Offset], PixelSize);
							}

                            ResultData += PixelSize;
                            ++X;
                        }

                        MapData += Different;
                    }
                }

                SizeX = FMath::DivideAndRoundUp(SizeX, 2);
                SizeY = FMath::DivideAndRoundUp(SizeY, 2);
            }
		}

		else
		{
			checkf( false, TEXT("Unsupported mask format.") );
		}


		// Update the relevancy data of the image for the worst case. 
		if (ResultImage->Flags & FImage::EImageFlags::IF_HAS_RELEVANCY_MAP)
		{
			// Displace can encode at max MUTABLE_GROW_BORDER_VALUE pixels away according to "border" in MakeGrowMap.
			ResultImage->RelevancyMinY = FMath::Max(int32(ResultImage->RelevancyMinY) - MUTABLE_GROW_BORDER_VALUE, 0);
			ResultImage->RelevancyMaxY = FMath::Min(ResultImage->RelevancyMaxY + MUTABLE_GROW_BORDER_VALUE, ResultImage->GetSizeY() - 1);
		}
	}

}


