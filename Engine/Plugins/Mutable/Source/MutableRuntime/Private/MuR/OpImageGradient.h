// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"


namespace UE::Mutable::Private
{

	inline void ImageGradient( FImage* pDest, const FVector4f& c0, const FVector4f& c1 )
	{
		check(pDest && pDest->GetFormat()==EImageFormat::RGBA_UByte);

        uint8* pDestBuf = pDest->GetLODData(0);

		int32 sizeX = pDest->GetSizeX();
		int32 sizeY = pDest->GetSizeX();

		for ( int i=0; i< sizeX; ++i )
		{
			float delta = float(i)/float(sizeX -1);
			FVector3f c = c0 * (1.0f-delta) + c1 * delta;

            uint8 color[4];
            color[0] = (uint8)FMath::Max( 0, FMath::Min( 255, int(c[0]*255.0f) ) );
            color[1] = (uint8)FMath::Max( 0, FMath::Min( 255, int(c[1]*255.0f) ) );
            color[2] = (uint8)FMath::Max( 0, FMath::Min( 255, int(c[2]*255.0f) ) );
			color[3] = 255;

			for ( int j=0; j< sizeY; ++j )
			{
				pDestBuf[ (sizeX*j+i)*3 + 0 ] = color[0];
				pDestBuf[ (sizeX*j+i)*3 + 1 ] = color[1];
				pDestBuf[ (sizeX*j+i)*3 + 2 ] = color[2];
			}
		}
	}

}
