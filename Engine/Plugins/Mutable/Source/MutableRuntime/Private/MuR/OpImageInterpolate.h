// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"

#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{

    inline void ImageInterpolate(FImage* DestImage, const FImage* BImage, float Factor)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageInterpolate)

		check(DestImage && BImage);
        check(DestImage->GetSizeX() == BImage->GetSizeX());
        check(DestImage->GetSizeY() == BImage->GetSizeY());
        check(DestImage->GetFormat() == BImage->GetFormat());

		// Clamp the factor
		Factor = FMath::Clamp(Factor, 0.0f, 1.0f);
		const uint16 FactorUNorm = static_cast<uint16>(Factor * 255.0f);

		const int32 BytesPerElem = GetImageFormatData(DestImage->GetFormat()).BytesPerBlock;

		constexpr int32 NumBatchElems = 1 << 14;

		const int32 NumBatches = DestImage->DataStorage.GetNumBatches(NumBatchElems, BytesPerElem);
		check(BImage->DataStorage.GetNumBatches(NumBatchElems, BytesPerElem) == NumBatches);

		// Generic implementation	
		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[
			DestImage, BImage, NumBatchElems, BytesPerElem, FactorUNorm
		](int32 BatchId)
		{
			TArrayView<uint8> DestView = DestImage->DataStorage.GetBatch(BatchId, NumBatchElems, BytesPerElem);
			TArrayView<const uint8> BView = BImage->DataStorage.GetBatch(BatchId, NumBatchElems, BytesPerElem);

			const uint8* BBuf = BView.GetData();
			uint8* DestBuf = DestView.GetData();

			const int32 NumBytes = DestView.Num();
			check(BView.Num() == NumBytes);

			switch (DestImage->GetFormat())
			{
			case EImageFormat::L_UByte:
			case EImageFormat::RGB_UByte:
			case EImageFormat::BGRA_UByte:
			case EImageFormat::RGBA_UByte:
			{
				for (int32 I = 0; I < NumBytes; ++I)
				{
					uint16 AValue = DestBuf[I];
					uint16 BValue = BBuf[I];
					DestBuf[I] = static_cast<uint8>((AValue * (255 - FactorUNorm) + BValue*FactorUNorm) / 255);
				}
				break;
			}
			default:
				check(false);
			}
		});
	}

}
