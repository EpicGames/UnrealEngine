// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"
#include "MuR/ParallelExecutionUtils.h"

#include "Math/VectorRegister.h"

namespace UE::Mutable::Private
{
	/** Invert the color (RGB or L) components of an image. Leave alpha untouched. */
	template<bool bUseVectorImpl = false>
	inline void ImageInvert(FImage* AImage)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageInvertInPlace);
			
		if (!AImage)
		{
			return;
		}

		//Generic implementation
		constexpr int32 NumBatchElems = 1 << 14;

		const int32 BytesPerElem = GetImageFormatData(AImage->GetFormat()).BytesPerBlock;
		const int32 NumBatches = AImage->DataStorage.GetNumBatches(NumBatchElems, BytesPerElem);

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[
			AImage, NumBatchElems, BytesPerElem
		](int32 BatchId)
		{
			TArrayView<uint8> AView = AImage->DataStorage.GetBatch(BatchId, NumBatchElems, BytesPerElem);
			const int32 NumBytes = AView.Num();
			
			uint8* ABuf = AView.GetData();

			switch (AImage->GetFormat())
			{
			case EImageFormat::L_UByte:
			case EImageFormat::RGB_UByte:
			{
				if constexpr (!bUseVectorImpl)
				{
					for (int32 I = 0; I < NumBytes; ++I)
					{
						ABuf[I] = 255 - ABuf[I];
					}
				}
				else
				{
					const int32 RemBytes = NumBytes % 4;

					constexpr VectorRegister4Int One255 = MakeVectorRegisterIntConstant(255, 255, 255, 255);
					for (int32 I = 0; I < NumBytes - RemBytes; I += 4)
					{
						VectorRegister4Int Value = MakeVectorRegisterInt(ABuf[I], ABuf[I + 1], ABuf[I + 2], ABuf[I + 3]);
						Value = VectorIntSubtract(One255, Value);

						alignas(alignof(VectorRegister4Int)) int32 IndexableValue[4];
						VectorIntStoreAligned(Value, IndexableValue);

						ABuf[I + 0] = static_cast<uint8>(IndexableValue[0]);
						ABuf[I + 1] = static_cast<uint8>(IndexableValue[1]);
						ABuf[I + 2] = static_cast<uint8>(IndexableValue[2]);
						ABuf[I + 3] = static_cast<uint8>(IndexableValue[3]);
					}

					for (int32 I = NumBytes - RemBytes; I < NumBytes; ++I)
					{
						ABuf[I] = 255 - ABuf[I];
					}
				}
				break;
			}
			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				if constexpr (!bUseVectorImpl)
				{
					const int32 NumElems = NumBytes/4;
					for (int32 I = 0; I < NumElems; ++I)
					{
						ABuf[I*4 + 0] = 255 - ABuf[I*4 + 0];
						ABuf[I*4 + 1] = 255 - ABuf[I*4 + 1];
						ABuf[I*4 + 2] = 255 - ABuf[I*4 + 2];
					}
				}
				else
				{
					constexpr VectorRegister4Int One255 = MakeVectorRegisterIntConstant(255, 255, 255, 255);

					for (int32 I = 0; I < NumBytes; I += 4)
					{
						VectorRegister4Int Value = MakeVectorRegisterInt(ABuf[I], ABuf[I + 1], ABuf[I + 2], ABuf[I + 3]);
						Value = VectorIntSubtract(One255, Value);

						alignas(alignof(VectorRegister4Int)) int32 IndexableValue[4];
						VectorIntStoreAligned(Value, IndexableValue);

						ABuf[I + 0] = static_cast<uint8>(IndexableValue[0]);
						ABuf[I + 1] = static_cast<uint8>(IndexableValue[1]);
						ABuf[I + 2] = static_cast<uint8>(IndexableValue[2]);
					}
				}
				break;
			}
			default:
				check(false);
			}
		});
	}
}
