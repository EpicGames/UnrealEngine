// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpImageLayer.h"

#include "MuR/ImageRLE.h"
#include "MuR/ImageComputeUtils.h"
#include "MuR/ParallelExecutionUtils.h"

#include "MuR/OpImageBlend.h"

#include "Math/UnrealMathUtility.h"
#include "Math/VectorRegister.h"


namespace UE::Mutable::Private
{

namespace OpImageLayerBlendOps
{

#if PLATFORM_ENABLE_VECTORINTRINSICS && !PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	using VectorRegisterUInt8x16 = __m128i;
	using VectorRegisterUInt16x8 = __m128i;

	FORCEINLINE void StoreUnaligned_UInt8x16(uint8* Ptr, const VectorRegisterUInt8x16 Reg)
	{
		_mm_storeu_si128((__m128i*)Ptr, Reg);
	}

	FORCEINLINE VectorRegisterUInt8x16 LoadUnaligned_UInt8x16(const uint8* Ptr)
	{
		return _mm_loadu_si128((__m128i*)Ptr);
	}

	FORCEINLINE VectorRegisterUInt8x16 LoadUnaligned_UInt16x8(const uint16* Ptr)
	{
		return _mm_loadu_si128((__m128i*)Ptr);
	}

	FORCEINLINE VectorRegisterUInt16x8 GetLow16_UInt8x16(VectorRegisterUInt8x16 Reg)
	{
		return _mm_cvtepu8_epi16(Reg);
	}	

	FORCEINLINE VectorRegisterUInt16x8 GetHigh16_UInt8x16(VectorRegisterUInt8x16 Reg)
	{
		return _mm_cvtepu8_epi16(_mm_srli_si128(Reg, 8));
	}

	FORCEINLINE VectorRegisterUInt8x16 Saturate_UInt16x8(VectorRegisterUInt16x8 Lo, VectorRegisterUInt16x8 Hi)
	{
		return _mm_packus_epi16(Lo, Hi); 
	}

	/* Multply 16 bit 255 normalized values and saturate */
	FORCEINLINE VectorRegisterUInt8x16 Mult255Unorm_UInt16x8(VectorRegisterUInt16x8 ALo, VectorRegisterUInt16x8 AHi, VectorRegisterUInt16x8 BLo, VectorRegisterUInt16x8 BHi)
	{
		unimplemented();
		return Saturate_UInt16x8(ALo, AHi);
	}

	FORCEINLINE VectorRegisterUInt8x16 Mult255Unorm_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		__m128i MultLo = _mm_add_epi16(_mm_mullo_epi16(GetLow16_UInt8x16(A), GetLow16_UInt8x16(B)), _mm_set1_epi16(0x80));
		__m128i MultHi = _mm_add_epi16(_mm_mullo_epi16(GetHigh16_UInt8x16(A), GetHigh16_UInt8x16(B)), _mm_set1_epi16(0x80));

		MultLo = _mm_mulhi_epu16(MultLo, _mm_set1_epi16(257));
		MultHi = _mm_mulhi_epu16(MultHi, _mm_set1_epi16(257));

		return Saturate_UInt16x8(MultLo, MultHi);
	}

	FORCEINLINE VectorRegisterUInt8x16 Screen_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		__m128i ALo = GetLow16_UInt8x16(A);
		__m128i AHi = GetHigh16_UInt8x16(A);
		
		__m128i BLo = GetLow16_UInt8x16(B);
		__m128i BHi = GetHigh16_UInt8x16(B);

		__m128i MultLo = _mm_mullo_epi16(ALo, BLo);
		__m128i MultHi = _mm_mullo_epi16(AHi, BHi);

		MultLo = _mm_mulhi_epu16(_mm_add_epi16(MultLo, _mm_set1_epi16(0x80)), _mm_set1_epi16(257));
		MultHi = _mm_mulhi_epu16(_mm_add_epi16(MultHi, _mm_set1_epi16(0x80)), _mm_set1_epi16(257));

		__m128i ResultLo = _mm_sub_epi16(_mm_add_epi16(ALo, BLo), MultLo);
		__m128i ResultHi = _mm_sub_epi16(_mm_add_epi16(AHi, BHi), MultHi);

		return Saturate_UInt16x8(ResultLo, ResultHi);
	}

	FORCEINLINE VectorRegisterUInt8x16 SoftLight_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		__m128i ALo = GetLow16_UInt8x16(A);
		__m128i AHi = GetHigh16_UInt8x16(A);
		
		__m128i BLo = GetLow16_UInt8x16(B);
		__m128i BHi = GetHigh16_UInt8x16(B);

		__m128i MultLo = _mm_mullo_epi16(ALo, BLo);
		__m128i MultHi = _mm_mullo_epi16(AHi, BHi);

		MultLo = _mm_mulhi_epu16(_mm_add_epi16(MultLo, _mm_set1_epi16(0x80)), _mm_set1_epi16(257));
		MultHi = _mm_mulhi_epu16(_mm_add_epi16(MultHi, _mm_set1_epi16(0x80)), _mm_set1_epi16(257));

		__m128i ScreenLo = _mm_sub_epi16(_mm_add_epi16(ALo, BLo), MultLo);
		__m128i ScreenHi = _mm_sub_epi16(_mm_add_epi16(AHi, BHi), MultHi);

		__m128i LerpLo = _mm_mullo_epi16(MultLo, _mm_xor_si128(ALo, _mm_set1_epi16(0xFF)));  
		__m128i LerpHi = _mm_mullo_epi16(MultHi, _mm_xor_si128(AHi, _mm_set1_epi16(0xFF)));

		LerpLo = _mm_add_epi16(LerpLo, _mm_add_epi16(_mm_mullo_epi16(ScreenLo, ALo), _mm_set1_epi16(0x80)));  
		LerpHi = _mm_add_epi16(LerpHi, _mm_add_epi16(_mm_mullo_epi16(ScreenHi, AHi), _mm_set1_epi16(0x80)));

		LerpLo = _mm_mulhi_epu16(LerpLo, _mm_set1_epi16(257));
		LerpHi = _mm_mulhi_epu16(LerpHi, _mm_set1_epi16(257));

		return Saturate_UInt16x8(LerpLo, LerpHi);
	}

	FORCEINLINE VectorRegisterUInt8x16 HardLight_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		__m128i Xor = _mm_cmpgt_epi8(_mm_setzero_si128(), B);

		__m128i AXor = _mm_xor_si128(A, Xor);

		__m128i TwoTimesBMod255Xor = _mm_xor_si128(_mm_add_epi8(B, B), Xor);

		__m128i MultLo = _mm_add_epi16(_mm_mullo_epi16(GetLow16_UInt8x16(AXor), GetLow16_UInt8x16(TwoTimesBMod255Xor)), _mm_set1_epi16(0x80));
		__m128i MultHi = _mm_add_epi16(_mm_mullo_epi16(GetHigh16_UInt8x16(AXor), GetHigh16_UInt8x16(TwoTimesBMod255Xor)), _mm_set1_epi16(0x80));

		MultLo = _mm_mulhi_epu16(MultLo, _mm_set1_epi16(257));
		MultHi = _mm_mulhi_epu16(MultHi, _mm_set1_epi16(257));

		return _mm_xor_si128(Saturate_UInt16x8(MultLo, MultHi), Xor);
	}

	FORCEINLINE VectorRegisterUInt8x16 Overlay_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		__m128i Xor = _mm_cmpgt_epi8(_mm_setzero_si128(), A);

		__m128i TwoTimesAMod255Xor = _mm_xor_si128(_mm_add_epi8(A, A), Xor);
		__m128i BXor = _mm_xor_si128(B, Xor);

		__m128i MultLo = _mm_add_epi16(_mm_mullo_epi16(GetLow16_UInt8x16(TwoTimesAMod255Xor), GetLow16_UInt8x16(BXor)), _mm_set1_epi16(0x80));
		__m128i MultHi = _mm_add_epi16(_mm_mullo_epi16(GetHigh16_UInt8x16(TwoTimesAMod255Xor), GetHigh16_UInt8x16(BXor)), _mm_set1_epi16(0x80));

		MultLo = _mm_mulhi_epu16(MultLo, _mm_set1_epi16(257));
		MultHi = _mm_mulhi_epu16(MultHi, _mm_set1_epi16(257));

		return _mm_xor_si128(Saturate_UInt16x8(MultLo, MultHi), Xor);
	}

	FORCEINLINE VectorRegisterUInt8x16 Dodge_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		//__m128i A0 = _mm_slli_epi16(GetLow16_UInt8x16(A), 8);
		//__m128i A1 = _mm_slli_epi16(GetHigh16_UInt8x16(A), 8);

		//__m128i B0 = _mm_sub_epi16(_mm_set1_epi16(256), GetLow16_UInt8x16(B));
		//__m128i B1 = _mm_sub_epi16(_mm_set1_epi16(256), GetHigh16_UInt8x16(B));
		//
		//__m128 FloatA0 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(A0));
		//__m128 FloatA1 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(A0, 8)));
		//__m128 FloatA2 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(A1));
		//__m128 FloatA3 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(A1, 8)));

		//__m128 FloatB0 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(B0));
		//__m128 FloatB1 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(B0, 8)));
		//__m128 FloatB2 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(B1));
		//__m128 FloatB3 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(B1, 8)));

		//__m128 FloatResult0 = _mm_div_ps(FloatA0, FloatB0);
		//__m128 FloatResult1 = _mm_div_ps(FloatA1, FloatB1);
		//__m128 FloatResult2 = _mm_div_ps(FloatA2, FloatB2);
		//__m128 FloatResult3 = _mm_div_ps(FloatA3, FloatB3);

		//__m128i ResultLo = _mm_packus_epi32(_mm_cvtps_epi32(FloatResult0), _mm_cvtps_epi32(FloatResult1));
		//__m128i ResultHi = _mm_packus_epi32(_mm_cvtps_epi32(FloatResult2), _mm_cvtps_epi32(FloatResult3));

		//return Saturate_UInt16x8(ResultLo, ResultHi);

		unimplemented();
		return A;
	}

	FORCEINLINE VectorRegisterUInt8x16 Burn_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		//A = _mm_xor_si128(A, _mm_set1_epi8(0xFF));
		//
		//__m128i A0 = _mm_slli_epi16(GetLow16_UInt8x16(A), 8);
		//__m128i A1 = _mm_slli_epi16(GetHigh16_UInt8x16(A), 8);

		//__m128 FloatA0 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(A0));
		//__m128 FloatA1 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(A0, 8)));
		//__m128 FloatA2 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(A1));
		//__m128 FloatA3 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(A1, 8)));

		//__m128i B0 = _mm_add_epi16(GetLow16_UInt8x16(B), _mm_set1_epi16(1));
		//__m128i B1 = _mm_add_epi16(GetHigh16_UInt8x16(B), _mm_set1_epi16(1));

		//__m128 FloatB0 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(B0));
		//__m128 FloatB1 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(B0, 8)));
		//__m128 FloatB2 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(B1));
		//__m128 FloatB3 = _mm_cvtepi32_ps(_mm_cvtepu16_epi32(_mm_srli_si128(B1, 8)));

		//__m128 FloatResult0 = _mm_div_ps(FloatA0, FloatB0);
		//__m128 FloatResult1 = _mm_div_ps(FloatA1, FloatB1);
		//__m128 FloatResult2 = _mm_div_ps(FloatA2, FloatB2);
		//__m128 FloatResult3 = _mm_div_ps(FloatA3, FloatB3);

		//__m128i ResultLo = _mm_packus_epi32(_mm_cvtps_epi32(FloatResult0), _mm_cvtps_epi32(FloatResult1));
		//__m128i ResultHi = _mm_packus_epi32(_mm_cvtps_epi32(FloatResult2), _mm_cvtps_epi32(FloatResult3));

		//return _mm_xor_si128(Saturate_UInt16x8(ResultLo, ResultHi), _mm_set1_epi8(0xFF));

		unimplemented();
		return A;
	}


#endif
	
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	using VectorRegisterUInt8x16 = int8x16_t;
	using VectorRegisterUInt16x8 = int16x8_t;


	FORCEINLINE void StoreUnaligned_UInt8x16(uint8* Ptr, VectorRegisterUInt8x16 Reg)
	{
		vst1q_u8(Ptr, Reg);
	}

	FORCEINLINE VectorRegisterUInt8x16 LoadUnaligned_UInt8x16(const uint8* Ptr)
	{
		return vld1q_u8(Ptr);
	}	

	FORCEINLINE VectorRegisterUInt16x8 LoadUnaligned_UInt16x8(const uint16* Ptr)
	{
		return vld1q_u16(Ptr);
	}	

	FORCEINLINE VectorRegisterUInt16x8 GetLow16_UInt8x16(VectorRegisterUInt8x16 Reg)
	{
		return vmovl_u8(vget_low_u8(Reg));
	}	

	FORCEINLINE VectorRegisterUInt16x8 GetHigh16_UInt8x16(VectorRegisterUInt8x16 Reg)
	{
		return vmovl_u8(vget_high_u8(Reg));
	}

	FORCEINLINE VectorRegisterUInt8x16 Saturate_UInt16x8(VectorRegisterUInt16x8 Lo, VectorRegisterUInt16x8 Hi)
	{
		return vcombine_u8(vqmovn_u16(Lo), vqmovn_u16(Hi)); 
	}

	FORCEINLINE VectorRegisterUInt8x16 Mult255Unorm_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		uint16x8_t MultLo = vmull_u8(vget_low_u8(A), vget_low_u8(B));
		uint16x8_t MultHi = vmull_u8(vget_high_u8(A), vget_high_u8(B));

		uint8x8_t ResultLo = vqrshrn_n_u16(vrsraq_n_u16(MultLo, MultLo, 8), 8);
		uint8x8_t ResultHi = vqrshrn_n_u16(vrsraq_n_u16(MultHi, MultHi, 8), 8);

		return vcombine_u8(ResultLo, ResultHi);
	}

	FORCEINLINE VectorRegisterUInt8x16 Mult255Unorm_UInt16x8(VectorRegisterUInt16x8 A0, VectorRegisterUInt16x8 A1, VectorRegisterUInt16x8 B0, VectorRegisterUInt16x8 B1)
	{
		unimplemented();
		return Saturate_UInt16x8(A0, A1);
	}

	FORCEINLINE VectorRegisterUInt8x16 Screen_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		uint8x8_t ALo = vget_low_u8(A);
		uint8x8_t AHi = vget_high_u8(A);

		uint8x8_t BLo = vget_low_u8(B);
		uint8x8_t BHi = vget_high_u8(B);

		uint16x8_t MultLo = vmull_u8(ALo, BLo);
		uint16x8_t MultHi = vmull_u8(AHi, BHi);

		uint16x8_t ProdLo = vmovl_u8(vqrshrn_n_u16(vrsraq_n_u16(MultLo, MultLo, 8), 8));
		uint16x8_t ProdHi = vmovl_u8(vqrshrn_n_u16(vrsraq_n_u16(MultHi, MultHi, 8), 8));
		
		uint16x8_t ResultLo = vsubq_u16(vaddl_u8(ALo, BLo), ProdLo);
		uint16x8_t ResultHi = vsubq_u16(vaddl_u8(AHi, BHi), ProdHi);

		return vcombine_u8(vmovn_u16(ResultLo), vmovn_u16(ResultHi));
	}

	FORCEINLINE VectorRegisterUInt8x16 SoftLight_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		uint8x8_t ALo = vget_low_u8(A);
		uint8x8_t AHi = vget_high_u8(A);

		uint8x8_t BLo = vget_low_u8(B);
		uint8x8_t BHi = vget_high_u8(B);

		uint16x8_t MultLo = vmull_u8(ALo, BLo);
		uint16x8_t MultHi = vmull_u8(AHi, BHi);

		uint8x8_t ProdLo = vqrshrn_n_u16(vrsraq_n_u16(MultLo, MultLo, 8), 8);
		uint8x8_t ProdHi = vqrshrn_n_u16(vrsraq_n_u16(MultHi, MultHi, 8), 8);
		
		uint8x8_t ScreenLo = vmovn_u16(vsubq_u16(vaddl_u8(ALo, BLo), vmovl_u8(ProdLo)));
		uint8x8_t ScreenHi = vmovn_u16(vsubq_u16(vaddl_u8(AHi, BHi), vmovl_u8(ProdHi)));

		uint8x16_t OneMinusA = veorq_u8(A, vdupq_n_u8(0xFF)); 

		uint16x8_t ScreenTimesALo = vmull_u8(ScreenLo, ALo);
		uint16x8_t ScreenTimesAHi = vmull_u8(ScreenHi, AHi);

		uint16x8_t AddLo = vmlal_u8(ScreenTimesALo, ProdLo, vget_low_u8(OneMinusA));
		uint16x8_t AddHi = vmlal_u8(ScreenTimesAHi, ProdHi, vget_high_u8(OneMinusA));

		uint8x8_t ResultLo = vqrshrn_n_u16(vrsraq_n_u16(AddLo, AddLo, 8), 8);
		uint8x8_t ResultHi = vqrshrn_n_u16(vrsraq_n_u16(AddHi, AddHi, 8), 8);

		return vcombine_u8(ResultLo, ResultHi);
	}

	FORCEINLINE VectorRegisterUInt8x16 HardLight_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		uint8x16_t Xor = vcltzq_s8(B);

		uint8x16_t AXor = veorq_u8(A, Xor);
		uint8x16_t TwoTimesBMod255Xor = veorq_u8(vaddq_u8(B, B), Xor);

		uint16x8_t MultLo = vmull_u8(vget_low_u8(AXor), vget_low_u8(TwoTimesBMod255Xor));
		uint16x8_t MultHi = vmull_u8(vget_high_u8(AXor), vget_high_u8(TwoTimesBMod255Xor));

		uint8x8_t ResultLo = vqrshrn_n_u16(vrsraq_n_u16(MultLo, MultLo, 8), 8);
		uint8x8_t ResultHi = vqrshrn_n_u16(vrsraq_n_u16(MultHi, MultHi, 8), 8);

		return veorq_u8(vcombine_u8(ResultLo, ResultHi), Xor);
	}

	FORCEINLINE VectorRegisterUInt8x16 Overlay_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		uint8x16_t Xor = vcltzq_s8(A);

		uint8x16_t BXor = veorq_u8(B, Xor);
		uint8x16_t TwoTimesAMod255Xor = veorq_u8(vaddq_u8(A, A), Xor);

		uint16x8_t MultLo = vmull_u8(vget_low_u8(BXor), vget_low_u8(TwoTimesAMod255Xor));
		uint16x8_t MultHi = vmull_u8(vget_high_u8(BXor), vget_high_u8(TwoTimesAMod255Xor));

		uint8x8_t ResultLo = vqrshrn_n_u16(vrsraq_n_u16(MultLo, MultLo, 8), 8);
		uint8x8_t ResultHi = vqrshrn_n_u16(vrsraq_n_u16(MultHi, MultHi, 8), 8);

		return veorq_u8(vcombine_u8(ResultLo, ResultHi), Xor);
	}

	FORCEINLINE VectorRegisterUInt8x16 Dodge_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		unimplemented();
		return A;
	}

	FORCEINLINE VectorRegisterUInt8x16 Burn_UInt8x16(VectorRegisterUInt8x16 A, VectorRegisterUInt8x16 B)
	{
		unimplemented();
		return A;
	}

#endif

	FORCEINLINE uint8 Saturate_U16(uint16 V)
	{
		return (uint8)FMath::Min<uint16>(V, 255u);
	}

	FORCEINLINE uint8 Mult255Unorm_U8(uint8 A, uint8 B)
	{
		uint32 Value = A * B + 128;
		return (Value + (Value >> 8)) >> 8;
	}

	/* Multply 16 bit 255 normalized values and saturate */
	FORCEINLINE uint8 Mult255Unorm_U16(uint16 A, uint16 B)
	{
		uint32 Value = A * B + 128;
		return Saturate_U16(uint16((Value + (Value >> 8)) >> 8));
	}

	FORCEINLINE uint8 Lerp_U8(uint8 A, uint8 B, uint8 Factor)
	{
		uint32 Value = (255 - Factor)*A + Factor*B + 128;
		return (Value + (Value >> 8)) >> 8; 
	}

	FORCEINLINE uint8 Screen_U8(uint8 A, uint8 B)
	{
		return (A + B) - Mult255Unorm_U8(A, B);
	}

	FORCEINLINE uint8 SoftLight_U8(uint8 A, uint8 B)
	{
		uint8 Prod = Mult255Unorm_U8(A, B);
		return Lerp_U8(Prod, (A + B) - Prod, A);
	}

	FORCEINLINE uint8 HardLight_U8(uint8 A, uint8 B)
	{
		uint8 Xor = (B >= 128) ? 255 : 0;
		return Xor ^ Mult255Unorm_U8(A ^ Xor, ((B << 1) & 0xFF) ^ Xor);
	}

	FORCEINLINE uint8 Overlay_U8(uint8 A, uint8 B)
	{
		uint8 Xor = (A >= 128) ? 255 : 0;
		return Xor ^ Mult255Unorm_U8(((A << 1) & 0xFF) ^ Xor, B ^ Xor);
	}

	FORCEINLINE uint8 Dodge_U8(uint8 A, uint8 B)
	{
		return Saturate_U16(uint16((A << 8) / (256 - B)));
	}

	FORCEINLINE uint8 Burn_U8(uint8 A, uint8 B)
	{
		return 255 - Saturate_U16(uint16(((255 - A) << 8) / (B + 1)));
	}
//#define UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL 0
#ifndef UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
	#define UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL (PLATFORM_ENABLE_VECTORINTRINSICS || PLATFORM_ENABLE_VECTORINTRINSICS_NEON)
#endif

	void Blend(uint8* Dest, const uint8*, const uint8* Blen, int32 NumElems)
	{
		FMemory::Memcpy(Dest, Blen, NumElems);
	}

	void BlendWithWord(uint8* Dest, const uint8*, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 Result = Saturate_UInt16x8(BLo, BHi);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Saturate_U16(Blen[ElemIndex]);
		}
	}

	void Screen(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);

			VectorRegisterUInt8x16 Result = Screen_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Screen_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void ScreenWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = Saturate_UInt16x8(BLo, BHi);

			VectorRegisterUInt8x16 Result = Screen_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Screen_U8(Base[ElemIndex], Saturate_U16(Blen[ElemIndex]));
		}
	}

	void SoftLight(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);

			VectorRegisterUInt8x16 Result = SoftLight_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = SoftLight_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void SoftLightWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = Saturate_UInt16x8(BLo, BHi);

			VectorRegisterUInt8x16 Result = SoftLight_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = SoftLight_U8(Base[ElemIndex], Saturate_U16(Blen[ElemIndex]));
		}
	}

	void HardLight(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);

			VectorRegisterUInt8x16 Result = HardLight_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = HardLight_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void HardLightWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = Saturate_UInt16x8(BLo, BHi);

			VectorRegisterUInt8x16 Result = HardLight_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = HardLight_U8(Base[ElemIndex], Saturate_U16(Blen[ElemIndex]));
		}
	}

	void Dodge(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if 0 //UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);

			VectorRegisterUInt8x16 Result = Dodge_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Dodge_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void DodgeWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if 0 //UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = Saturate_UInt16x8(BLo, BHi);

			VectorRegisterUInt8x16 Result = Dodge_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Dodge_U8(Base[ElemIndex], Saturate_U16(Blen[ElemIndex]));
		}
	}

	void Lighten(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		Screen(Dest, Base, Blen, NumElems);
	}

	void LightenWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		ScreenWithWord(Dest, Base, Blen, NumElems);
	}

	void Burn(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if 0 //UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);

			VectorRegisterUInt8x16 Result = Burn_UInt8x16(A, B);

			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Burn_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void BurnWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if 0 //UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = Saturate_UInt16x8(BLo, BHi);

			VectorRegisterUInt8x16 Result = Burn_UInt8x16(A, B);
			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Burn_U8(Base[ElemIndex], Saturate_U16(Blen[ElemIndex]));
		}
	}

	void Overlay(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{	
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);
			VectorRegisterUInt8x16 Result = Overlay_UInt8x16(A, B);
			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Overlay_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void OverlayWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt8x16 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = Saturate_UInt16x8(BLo, BHi);

			VectorRegisterUInt8x16 Result = Overlay_UInt8x16(A, B);
			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Overlay_U8(Base[ElemIndex], Saturate_U16(Blen[ElemIndex]));
		}
	}

	void Multiply(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);
			VectorRegisterUInt8x16 B = LoadUnaligned_UInt8x16(Blen + Iter*16);
			VectorRegisterUInt8x16 Result = Mult255Unorm_UInt8x16(A, B);
			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Mult255Unorm_U8(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void MultiplyWithWord(uint8* Dest, const uint8* Base, const uint16* Blen, int32 NumElems)
	{
		int32 ElemIndex = 0;

#if 0 //UE_MUTABLE_IMAGE_LAYER_USE_VECTOR_IMPL
		int32 NumIters = NumElems / 16;

		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			VectorRegisterUInt16x8 BLo = LoadUnaligned_UInt16x8(Blen + Iter*16);
			VectorRegisterUInt16x8 BHi = LoadUnaligned_UInt16x8(Blen + Iter*16 + 8);

			VectorRegisterUInt8x16 A = LoadUnaligned_UInt8x16(Base + Iter*16);

			VectorRegisterUInt16x8 ALo = GetLow16_UInt8x16(A);
			VectorRegisterUInt16x8 AHi = GetHigh16_UInt8x16(A);

			VectorRegisterUInt8x16 Result = Mult255Unorm_UInt16x8(ALo, AHi, BLo, BHi);
			StoreUnaligned_UInt8x16(Dest + Iter*16, Result);
		}

		ElemIndex = NumIters*16;
#endif

		for (; ElemIndex < NumElems; ++ElemIndex)
		{
			Dest[ElemIndex] = Mult255Unorm_U16(Base[ElemIndex], Blen[ElemIndex]);
		}
	}

	void Vec1FillOnes(uint8* Dest, const uint8*, int32 NumElems)
	{
		FMemory::Memset(Dest, 255, NumElems);
	}

	void Vec3FillOnes(uint8* Dest, const uint8*, int32 NumElems)
	{
		FMemory::Memset(Dest, 255, NumElems*3);
	}

	void Vec4FillOnes(uint8* Dest, const uint8*, int32 NumElems)
	{
		FMemory::Memset(Dest, 255, NumElems*4);
	}
} // namespace OpImageLayerBlendOps

namespace OpImageLayerCombineOps
{

	void Transpose(VectorRegister4Float& InOutA, VectorRegister4Float& InOutB, VectorRegister4Float& InOutC, VectorRegister4Float& InOutD)
	{
		VectorRegister4Float Temp0;
		VectorRegister4Float Temp1;
		VectorRegister4Float Temp2;
		VectorRegister4Float Temp3;

		VectorDeinterleave(Temp0, Temp1, InOutA, InOutB);
		VectorDeinterleave(Temp2, Temp3, InOutC, InOutD);

		// Temp0 = [XA,ZA,XB,ZB]
		// Temp1 = [YA,WA,YB,WB]
		// Temp2 = [XC,ZC,XD,ZD]
		// Temp3 = [YC,WC,YD,WD]

		VectorDeinterleave(InOutA, InOutC, Temp0, Temp2);
		VectorDeinterleave(InOutB, InOutD, Temp1, Temp3);
	}

	void NormalCombine(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems)
	{
		constexpr VectorRegister4Float Float127AndAHalf = MakeVectorRegisterFloatConstant(127.5f, 127.5f, 127.5f, 127.5f);
		constexpr VectorRegister4Float FloatOneOver127AndAHalf = MakeVectorRegisterFloatConstant(1.0f/127.5f, 1.0f/127.5f, 1.0f/127.5f, 1.0f/127.5f);
		constexpr VectorRegister4Float FloatMinusOneOver127AndAHalf = MakeVectorRegisterFloatConstant(-1.0f/127.5f, -1.0f/127.5f, -1.0f/127.5f, -1.0f/127.5f);
		constexpr VectorRegister4Float FloatMinusOne = GlobalVectorConstants::FloatMinusOne;
		constexpr VectorRegister4Float FloatOne = GlobalVectorConstants::FloatOne;
		constexpr VectorRegister4Float FloatZero = GlobalVectorConstants::FloatZero;
		
		int32 I = 0;
		for (; I < NumElems - 4; I += 4)
		{
			VectorRegister4Float A0 = VectorLoadByte4((Base + I*4 + 0 ));
			VectorRegister4Float A1 = VectorLoadByte4((Base + I*4 + 4 ));
			VectorRegister4Float A2 = VectorLoadByte4((Base + I*4 + 8 ));
			VectorRegister4Float A3 = VectorLoadByte4((Base + I*4 + 12));

			Transpose(A0, A1, A2, A3);

			VectorRegister4Float B0 = VectorLoadByte4((Blen + I*4 + 0 ));
			VectorRegister4Float B1 = VectorLoadByte4((Blen + I*4 + 4 ));
			VectorRegister4Float B2 = VectorLoadByte4((Blen + I*4 + 8 ));
			VectorRegister4Float B3 = VectorLoadByte4((Blen + I*4 + 12));
			
			Transpose(B0, B1, B2, B3);

			A0 = VectorMultiplyAdd(A0, FloatOneOver127AndAHalf, FloatMinusOne);
			A1 = VectorMultiplyAdd(A1, FloatOneOver127AndAHalf, FloatMinusOne);
			A2 = VectorMultiply   (A2, FloatOneOver127AndAHalf               );
			
			VectorRegister4Float NegateA2 = VectorNegate(A2);

			B0 = VectorMultiplyAdd(B0, FloatMinusOneOver127AndAHalf, FloatOne);
			B1 = VectorMultiplyAdd(B1, FloatMinusOneOver127AndAHalf, FloatOne);
			B2 = VectorMultiplyAdd(B2, FloatOneOver127AndAHalf, FloatMinusOne);

			VectorRegister4Float DotAB =
					VectorMultiplyAdd(A0, B0, VectorMultiplyAdd(A1, B1, VectorMultiply(A2, B2)));

			VectorRegister4Float N0 = VectorMultiplyAdd(A0, DotAB, VectorMultiply(B0, NegateA2));
			VectorRegister4Float N1 = VectorMultiplyAdd(A1, DotAB, VectorMultiply(B1, NegateA2));
			VectorRegister4Float N2 = VectorMultiplyAdd(A2, DotAB, VectorMultiply(B2, NegateA2));
			VectorRegister4Float N3 = FloatZero;

			VectorRegister4Float ReciprocalLen = VectorReciprocalSqrt(
					VectorMultiplyAdd(N0, N0, VectorMultiplyAdd(N1, N1, VectorMultiply(N2, N2))));
		
			N0 = VectorMultiplyAdd(VectorMultiply(N0, ReciprocalLen), Float127AndAHalf, Float127AndAHalf);
			N1 = VectorMultiplyAdd(VectorMultiply(N1, ReciprocalLen), Float127AndAHalf, Float127AndAHalf);
			N2 = VectorMultiplyAdd(VectorMultiply(N2, ReciprocalLen), Float127AndAHalf, Float127AndAHalf);

			Transpose(N0, N1, N2, N3);

			VectorStoreByte4(N0, Dest + I*4 + 0 );
			VectorStoreByte4(N1, Dest + I*4 + 4 );
			VectorStoreByte4(N2, Dest + I*4 + 8 );
			VectorStoreByte4(N3, Dest + I*4 + 12);
		}

		constexpr VectorRegister4Float AOffset = MakeVectorRegisterFloatConstant(-1.0f, -1.0f, 0.0f, 0.0f);
		constexpr VectorRegister4Float BOffset = MakeVectorRegisterFloatConstant(1.0f, 1.0f, -1.0f, 0.0f);
		constexpr VectorRegister4Float AScale = MakeVectorRegisterFloatConstant(1.0f/127.5f, 1.0f/127.5f, 1.0f/127.5f, 0.0f);
		constexpr VectorRegister4Float BScale = MakeVectorRegisterFloatConstant(-1.0f/127.5f, -1.0f/127.5f, 1.0f/127.5f, 0.0f);

		for (; I < NumElems; ++I)
		{
			VectorRegister4Float A = VectorLoadByte4((Base + I*4));
			VectorRegister4Float B = VectorLoadByte4((Blen + I*4));

			A = VectorMultiplyAdd(A, AScale, AOffset);
			B = VectorMultiplyAdd(B, BScale, BOffset);

			VectorRegister4Float NegateA2 = VectorNegate(VectorReplicate(A, 2));
		
			VectorRegister4Float N = VectorMultiplyAdd(A, VectorDot3(A, B), VectorMultiply(B, NegateA2));
			N = VectorMultiplyAdd(VectorMultiply(N, VectorReciprocalLen(N)), Float127AndAHalf, Float127AndAHalf);

			VectorStoreByte4(N, Dest + I*4);
		}
	}
} //namespace OpImageLayerCombineOps

namespace OpImageLayerInternal
{
	struct FOpLayerBatchArgs
	{
		int32 BatchNumElems         = 0;
		int32 LODBegin              = 0;
		int32 LODEnd                = 0;
		int32 FirstLODOffsetInElems = 0;

		FImage* Result      = nullptr;
		const FImage* Base  = nullptr;
		const FImage* Blend = nullptr;
		const FImage* Mask  = nullptr;

		uint32 ResultBytesPerElem = 0;
		uint32 BaseBytesPerElem   = 0;
		uint32 BlendBytesPerElem  = 0;
		uint32 MaskBytesPerElem   = 0;
	};

	struct FOpLayerBatchViews
	{
		int32 NumElems = 0;

		TArrayView<uint8> Result;
		TArrayView<const uint8> Base;
		TArrayView<const uint8> Blend;
		TArrayView<const uint8> Mask;

		uint64 DecodeTokenRLE = 0;
	};

	struct FOpBlendKernel
	{
		using FConvertFuncType         = void(uint8*, const uint8*, int32);
		using FMixFuncType             = void(uint8*, const uint8*, const uint8*, const uint8*, int32);
		using FMakeConstantU8FuncType  = void(uint8*, FColor, int32);
		using FMakeConstantU16FuncType = void(uint16*, Math::TIntVector4<uint16>, int32);
		using FMaskDecodeFuncType      = void(uint8*, const uint8*, uint64&, int32);
		using FBlendWithWordFuncType   = void(uint8*, const uint8*, const uint16*, int32);

		FColor ConstantU8;
		Math::TIntVector4<uint16> ConstantU16;
		FMakeConstantU8FuncType* MakeConstantU8   = nullptr;
		FMakeConstantU16FuncType* MakeConstantU16 = nullptr;

		FConvertFuncType* BaseFormat  = nullptr;
		FConvertFuncType* BlendFormat = nullptr;
		FBlendFuncType* ColorBlend    = nullptr;
		FBlendFuncType* AlphaBlend    = nullptr;

		FBlendWithWordFuncType* ColorBlendWithWord = nullptr;
		FBlendWithWordFuncType* AlphaBlendWithWord = nullptr;

		FConvertFuncType* ExtractMaskFromBlendAlpha = nullptr; 
		FConvertFuncType* MaskFormat                = nullptr;
		FMixFuncType* Mix                           = nullptr;
		FBlendFuncType* SelectAlpha                 = nullptr;

		FMaskDecodeFuncType* MaskDecode = nullptr;
	};

	struct FMakeOpBlendKernelArgs
	{
		FImage* Dest  = nullptr;
		const FImage* Base  = nullptr;
		const FImage* Blend = nullptr;
		const FImage* Mask  = nullptr;

		FBlendFuncType* ColorBlendFunc = nullptr;
		FBlendFuncType* AlphaBlendFunc = nullptr;

		bool bApplyColorBlendToAlpha       = false;
		bool bUseBaseSourceFromBaseAlpha   = false;
		bool bUseBlendSourceFromBlendAlpha = false;
		bool bUseMaskFromBlendAlpha        = false;
		uint8 BlendAlphaSourceChannel      = 3;

		bool bBlendWithConstant = false;
		FVector4f RealConstant  = FVector4f{};
	};

	struct FOpCombineKernel
	{
		using FConvertFuncType      = void(uint8*, const uint8*, int32);
		using FMixFuncType          = void(uint8*, const uint8*, const uint8*, const uint8*, int32);
		using FMakeConstantFuncType = void(uint8*, FColor, int32);
		using FMaskDecodeFuncType   = void(uint8*, const uint8*, uint64&, int32);

		FMakeConstantFuncType* MakeConstant = nullptr;
		FColor Constant;

		FConvertFuncType* BaseFormat  = nullptr;
		FConvertFuncType* BlendFormat = nullptr;
		FConvertFuncType* MaskFormat  = nullptr;

		FCombineFuncType* Combine     = nullptr;
		FMixFuncType* Mix             = nullptr;

		FConvertFuncType* ResultFormat = nullptr;
		
		FMaskDecodeFuncType* MaskDecode = nullptr;
		
		uint32 CombineBytesPerElem = 4;
	};

	struct FMakeOpCombineKernelArgs
	{
		FImage* Dest  = nullptr;
		const FImage* Base  = nullptr;
		const FImage* Blend = nullptr;
		const FImage* Mask  = nullptr;

		FCombineFuncType* CombineFunc = nullptr;
		uint32 CombineBytesPerElem = 4;

		bool bCombineWithConstant = false;
		FColor Constant           = FColor::White;
	};

	FOpBlendKernel::FBlendWithWordFuncType* ConvertBlendFuncToWordVariant(FBlendFuncType* Func)
	{
		if (Func == nullptr)
		{
			return nullptr;
		}

		if (Func == OpImageLayerBlendOps::Blend)
		{
			return OpImageLayerBlendOps::BlendWithWord;
		}

		if (Func == OpImageLayerBlendOps::Screen)
		{
			return OpImageLayerBlendOps::ScreenWithWord;
		}

		if (Func == OpImageLayerBlendOps::SoftLight)
		{
			return OpImageLayerBlendOps::BlendWithWord;
		}

		if (Func == OpImageLayerBlendOps::HardLight)
		{
			return OpImageLayerBlendOps::HardLightWithWord;
		}

		if (Func == OpImageLayerBlendOps::Dodge)
		{
			return OpImageLayerBlendOps::DodgeWithWord;
		}

		if (Func == OpImageLayerBlendOps::Lighten)
		{
			return OpImageLayerBlendOps::LightenWithWord;
		}

		if (Func == OpImageLayerBlendOps::Burn)
		{
			return OpImageLayerBlendOps::BurnWithWord;
		}

		if (Func == OpImageLayerBlendOps::Multiply)
		{
			return OpImageLayerBlendOps::MultiplyWithWord;
		}

		if (Func == OpImageLayerBlendOps::Overlay)
		{
			return OpImageLayerBlendOps::OverlayWithWord;
		}

		check(false);
		return nullptr;
	}

	FOpBlendKernel MakeKernel(const FMakeOpBlendKernelArgs& Args)
	{
		using namespace ImageComputeUtils;
		
		FOpBlendKernel Kernel;

		Kernel.ColorBlend = Args.ColorBlendFunc;
		Kernel.AlphaBlend = Args.AlphaBlendFunc;

		if (Args.bBlendWithConstant)
		{
			FVector4f Constant = Args.RealConstant;
			if (Args.bUseBlendSourceFromBlendAlpha)
			{
				Constant = FVector4f{Constant.W, Constant.W, Constant.W, Constant.W};
			}

			Kernel.ConstantU8  = FLinearColor(Constant).QuantizeRound();
			Kernel.ConstantU16 = Math::TIntVector4<uint16>(
					FMath::Clamp(Constant.X * 255.0f + 0.5f, 0.0f, 65535.0f),
					FMath::Clamp(Constant.Y * 255.0f + 0.5f, 0.0f, 65535.0f),
					FMath::Clamp(Constant.Z * 255.0f + 0.5f, 0.0f, 65535.0f),
					FMath::Clamp(Constant.W * 255.0f + 0.5f, 0.0f, 65535.0f));

			bool bAllLessThanEqOne = 
					(Constant.X <= 1.0f) && (Constant.Y <= 1.0f) && (Constant.Z <= 1.0f) && (Constant.W <= 1.0f);

			bool bUseU16Constant = !bAllLessThanEqOne && 
					(Kernel.ColorBlend == OpImageLayerBlendOps::Multiply || Kernel.AlphaBlend == OpImageLayerBlendOps::Multiply);
			
			switch (Args.Base->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				if (!bUseU16Constant)
				{
					Kernel.MakeConstantU8 = Vec1Fill_U8;
				}
				else
				{
					Kernel.MakeConstantU16 = Vec1Fill_U16;
				}

				break;
			}
			case EImageFormat::RGB_UByte:
			{
				if (!bUseU16Constant)
				{
					Kernel.MakeConstantU8 = Vec3Fill_U8;
				}
				else
				{
					Kernel.MakeConstantU16 = Vec3Fill_U16;
				}
				break;
			}
			case EImageFormat::RGBA_UByte:
			{
				if (!bUseU16Constant)
				{
					Kernel.MakeConstantU8 = Vec4Fill_U8;
				}
				else
				{
					Kernel.MakeConstantU16 = Vec4Fill_U16;
				}
				break;
			}
			case EImageFormat::BGRA_UByte:
			{
				if (!bUseU16Constant)
				{
					Kernel.ConstantU8 = FColor(Kernel.ConstantU8.B, Kernel.ConstantU8.G, Kernel.ConstantU8.R, Kernel.ConstantU8.A);
					Kernel.MakeConstantU8 = Vec4Fill_U8;
				}
				else
				{
					Kernel.ConstantU16 = Math::TIntVector4<uint16>(Kernel.ConstantU16.X, Kernel.ConstantU16.Y, Kernel.ConstantU16.Z, Kernel.ConstantU16.W);
					Kernel.MakeConstantU16 = Vec4Fill_U16;
				}

				break;
			}
			default: check(false);
			}

			if (bUseU16Constant)
			{
				Kernel.ColorBlendWithWord = ConvertBlendFuncToWordVariant(Kernel.ColorBlend);  
				Kernel.AlphaBlendWithWord = ConvertBlendFuncToWordVariant(Kernel.AlphaBlend);  
			}
		}

		if (Args.bUseBaseSourceFromBaseAlpha)
		{
			switch (Args.Base->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				Kernel.BaseFormat = OpImageLayerBlendOps::Vec1FillOnes;
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				Kernel.BaseFormat = OpImageLayerBlendOps::Vec3FillOnes;
				break;
			}
			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				Kernel.BaseFormat = Vec4SwizzleWWWW_U8;
				break;
			}
			}
		}

		if (!Args.bBlendWithConstant && Args.Base->GetFormat() != Args.Blend->GetFormat())
		{
			switch (Args.Base->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				switch (Args.Blend->GetFormat())
				{
				case EImageFormat::RGB_UByte:
				{
					if (Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.BlendFormat = OpImageLayerBlendOps::Vec1FillOnes;
					}
					else
					{
						Kernel.BlendFormat = Vec3ToVec1_U8;
					}
					break;
				}
				case EImageFormat::RGBA_UByte:
				case EImageFormat::BGRA_UByte:
				{
					if (Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.BlendFormat = Vec4SwizzleWWWWAndConvertToVec1_U8;
					}
					else
					{
						Kernel.BlendFormat = Vec4ToVec1_U8;
					}
					break;
				}
				default: check(Args.Blend->GetFormat() == EImageFormat::L_UByte); break;
				}

				break;
			}
			case EImageFormat::RGB_UByte:
			{
				switch (Args.Blend->GetFormat())
				{
				case EImageFormat::L_UByte:
				{
					if (Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.BlendFormat = OpImageLayerBlendOps::Vec3FillOnes;
					}
					else
					{
						Kernel.BlendFormat = Vec1ToVec3_U8;
					}
					break;
				}
				case EImageFormat::RGBA_UByte:
				case EImageFormat::BGRA_UByte:
				{
					if (Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.BlendFormat = Vec4SwizzleWWWWAndConvertToVec3_U8;
					}
					else
					{
						Kernel.BlendFormat = Vec4ToVec3_U8;
					}
					break;
				}
				default: check(Args.Blend->GetFormat() == EImageFormat::RGB_UByte); break;
				}
				break;
			}
			case EImageFormat::BGRA_UByte:
			case EImageFormat::RGBA_UByte:
			{
				switch (Args.Blend->GetFormat())
				{
				case EImageFormat::L_UByte:
				{
					if (Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.BlendFormat = OpImageLayerBlendOps::Vec4FillOnes;
					}
					else
					{
						Kernel.BlendFormat = Vec1ToVec4_U8;
					}
					break;
				}
				case EImageFormat::RGB_UByte:
				{
					if (Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.BlendFormat = OpImageLayerBlendOps::Vec4FillOnes;
					}
					else
					{
						Kernel.BlendFormat = Vec3ToVec4_U8;
					}
					break;
				}
				default: check(Args.Blend->GetFormat() == EImageFormat::RGBA_UByte || Args.Blend->GetFormat() == EImageFormat::BGRA_UByte); break;
				}
				break;
			}
			default: check(Args.Base->GetFormat() == EImageFormat::RGBA_UByte || Args.Base->GetFormat() == EImageFormat::BGRA_UByte); break;
			}
		}

		if (Args.Mask || Args.bUseMaskFromBlendAlpha)
		{
			if (Args.bUseMaskFromBlendAlpha)
			{
				check(Args.BlendAlphaSourceChannel == 3 || Args.BlendAlphaSourceChannel == 0);

				if (Args.BlendAlphaSourceChannel == 3)
				{
					Kernel.ExtractMaskFromBlendAlpha = Vec4SwizzleWWWWAndConvertToVec1_U8;
				}
				else
				{
					if (!Args.bUseBlendSourceFromBlendAlpha)
					{
						Kernel.ExtractMaskFromBlendAlpha = Vec4ToVec1_U8;
					}
					else
					{
						switch(Args.Blend->GetFormat())
						{
							case EImageFormat::L_UByte:
							case EImageFormat::RGB_UByte:
							{
								Kernel.ExtractMaskFromBlendAlpha = OpImageLayerBlendOps::Vec1FillOnes;
								break;
							}
							case EImageFormat::RGBA_UByte:
							case EImageFormat::BGRA_UByte:
							{
								Kernel.ExtractMaskFromBlendAlpha = Vec4SwizzleWWWWAndConvertToVec1_U8;
								break;
							}
							default: check(false);
						}
					}
				}
			}

			if (Args.Mask)
			{
				check(Args.Mask->GetFormat() == EImageFormat::L_UByteRLE || 
					  Args.Mask->GetFormat() == EImageFormat::L_UByte)

				if (Args.Mask->GetFormat() == EImageFormat::L_UByteRLE)
				{
					Kernel.MaskDecode = IterativeDecodeRLE_L;
				}
			}

			if (Args.Mask || Args.bUseMaskFromBlendAlpha)
			{
				switch (Args.Base->GetFormat())
				{
				case EImageFormat::RGB_UByte:
				{
					Kernel.MaskFormat = Vec1ToVec3_U8;
					break;
				}
				case EImageFormat::BGRA_UByte:
				case EImageFormat::RGBA_UByte:
				{
					if (Kernel.ExtractMaskFromBlendAlpha == Vec4SwizzleWWWWAndConvertToVec1_U8)
					{
						Kernel.ExtractMaskFromBlendAlpha = Vec4SwizzleWWWW_U8;
					}
					else if (Kernel.ExtractMaskFromBlendAlpha == Vec4ToVec1_U8)
					{
						Kernel.ExtractMaskFromBlendAlpha = Vec4SwizzleXXXX_U8;
					}
					else
					{
						Kernel.MaskFormat = Vec1ToVec4_U8;
					}
					break;
				}
				}
				
				Kernel.Mix = Lerp_U8;
			}
		}

		if (Args.bApplyColorBlendToAlpha)
		{
			Kernel.AlphaBlend = nullptr;
			Kernel.SelectAlpha = nullptr;
		}
		else
		{
			if (Args.Base->GetFormat() == EImageFormat::RGBA_UByte || Args.Base->GetFormat() == EImageFormat::BGRA_UByte)
			{
				Kernel.SelectAlpha = Vec4SelectW_U8;
			}
		}

		return Kernel;
	}

	FOpCombineKernel MakeKernel(const FMakeOpCombineKernelArgs& Args)
	{
		using namespace ImageComputeUtils;
		
		FOpCombineKernel Kernel;

		Kernel.Combine = Args.CombineFunc;
		Kernel.CombineBytesPerElem = Args.CombineBytesPerElem;

		switch (Args.Base->GetFormat())
		{
			case EImageFormat::L_UByte:
			{
				Kernel.BaseFormat = Vec1ToVec4_U8;
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				Kernel.BaseFormat = Vec3ToVec4_U8;
				break;
			}
			case EImageFormat::BGRA_UByte:
			{
				Kernel.BaseFormat = Vec4SwizzleZYXW_U8;
				break;
			}
			default: check(Args.Base->GetFormat() == EImageFormat::RGBA_UByte); break;
		}

		if (Args.bCombineWithConstant)
		{
			Kernel.Constant = Args.Constant;
			Kernel.MakeConstant = Vec4Fill_U8;
		}
		else
		{
			switch (Args.Blend->GetFormat())
			{
				case EImageFormat::L_UByte:
				{
					Kernel.BlendFormat = Vec1ToVec4_U8;
					break;
				}
				case EImageFormat::RGB_UByte:
				{
					Kernel.BlendFormat = Vec3ToVec4_U8;
					break;
				}
				case EImageFormat::BGRA_UByte:
				{
					Kernel.BlendFormat = Vec4SwizzleZYXW_U8;
					break;
				}
				default: check(Args.Blend->GetFormat() == EImageFormat::RGBA_UByte); break;
			}
		}

		if (Args.Mask)
		{
			Kernel.MaskFormat = Vec1ToVec4_U8;
			Kernel.Mix = Lerp_U8;
		}

		switch (Args.Base->GetFormat())
		{
			case EImageFormat::L_UByte:
			{
				Kernel.ResultFormat = Vec4ToVec1_U8;
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				Kernel.ResultFormat = Vec4ToVec3_U8;
				break;
			}
			case EImageFormat::BGRA_UByte:
			{
				Kernel.ResultFormat = Vec4SwizzleZYXW_U8;
				break;
			}
			default: check(Args.Base->GetFormat() == EImageFormat::RGBA_UByte); break;
		}

		return Kernel;
	}

	void RunKernelOnBatch(const FOpBlendKernel& Kernel, const FOpLayerBatchViews BatchViews)
	{
		//MUTABLE_CPUPROFILER_SCOPE(OpImageLayer_RunKernelOnBatch_Blend);

		constexpr int32 MaxElemSize = 4;
		constexpr int32 NumCacheLinesPerWave = 8;
		constexpr int32 WaveSizeInByteElems = (PLATFORM_CACHE_LINE_SIZE * NumCacheLinesPerWave)/MaxElemSize;

		struct alignas(PLATFORM_CACHE_LINE_SIZE) FTemporalRegister
		{
			uint8 Data[WaveSizeInByteElems*MaxElemSize];
		};

		FTemporalRegister Register0;
		FTemporalRegister Register1;
		FTemporalRegister Register2;
		FTemporalRegister Register3;
		FTemporalRegister Register4;

		int32 BaseBytesPerElem = BatchViews.Base.Num() / BatchViews.NumElems;
		int32 BlenBytesPerElem = BatchViews.Blend.Num() / BatchViews.NumElems;

		check(BaseBytesPerElem <= MaxElemSize);
		check(BlenBytesPerElem <= MaxElemSize);

		uint8* ResultBuffer = BatchViews.Result.GetData();
		const uint8* BaseBuffer = BatchViews.Base.GetData();
		const uint8* BlenBuffer = BatchViews.Blend.GetData();
		const uint8* MaskBuffer = BatchViews.Mask.GetData();

		uint64 InOutDecodeToken = BatchViews.DecodeTokenRLE;

		const int32 WaveSizeInElems = Kernel.MakeConstantU16 != nullptr ? WaveSizeInByteElems / 2 : WaveSizeInByteElems; 
		
		if (Kernel.MakeConstantU16)
		{
			Kernel.MakeConstantU16((uint16*)Register0.Data, Kernel.ConstantU16, WaveSizeInElems);
		}
		else if (Kernel.MakeConstantU8)
		{
			Kernel.MakeConstantU8(Register0.Data, Kernel.ConstantU8, WaveSizeInElems);
		}

		const int32 NumWaves = FMath::DivideAndRoundUp<int32>(BatchViews.NumElems, WaveSizeInElems);
		for (int32 Idx = 0; Idx < NumWaves; ++Idx)
		{
			int32 WaveBeginInElems = Idx * WaveSizeInElems;
			int32 WaveEndInElems   = FMath::Min(WaveBeginInElems + WaveSizeInElems, BatchViews.NumElems);

			int32 NumWaveElems = WaveEndInElems - WaveBeginInElems;

			uint8* ResultDataBuffer     = ResultBuffer + WaveBeginInElems*BaseBytesPerElem;
			const uint8* BaseDataBuffer = BaseBuffer + WaveBeginInElems*BaseBytesPerElem;
			const uint8* MaskDataBuffer = MaskBuffer + WaveBeginInElems;
			const uint8* BlenDataBuffer = BlenBuffer + WaveBeginInElems*BlenBytesPerElem;

			// Zero mask optimization
			bool bSkipColorBlend = false;
			if (Kernel.Mix)
			{
				if (Kernel.ExtractMaskFromBlendAlpha)
				{
					Kernel.ExtractMaskFromBlendAlpha(Register3.Data, BlenDataBuffer, NumWaveElems);
					MaskDataBuffer = Register3.Data;
				}
				else if (Kernel.MaskDecode)
				{
					check(InOutDecodeToken != 0);
					//NOTE: Decode operation, unlike the other operations, use the batch buffer and not the wave buffer 
					//      because the data offset is carried by the DecodeToken.
					Kernel.MaskDecode(Register3.Data, MaskBuffer, InOutDecodeToken, NumWaveElems);

					MaskDataBuffer = Register3.Data;
				}

				if (Kernel.MaskFormat)
				{
					Kernel.MaskFormat(Register4.Data, MaskDataBuffer, NumWaveElems);
					MaskDataBuffer = Register4.Data;
				}

				bSkipColorBlend = ImageComputeUtils::TestZero_U8(MaskDataBuffer, NumWaveElems*BaseBytesPerElem);
			}
			
			if (Kernel.BaseFormat)
			{
				Kernel.BaseFormat(ResultDataBuffer, BaseDataBuffer, NumWaveElems);
				BaseDataBuffer = ResultDataBuffer;
			}

			if (Kernel.MakeConstantU8 || Kernel.MakeConstantU16)
			{
				BlenDataBuffer = Register0.Data;
			}
			else if (Kernel.BlendFormat)
			{
				Kernel.BlendFormat(Register0.Data, BlenDataBuffer, NumWaveElems);
				BlenDataBuffer = Register0.Data;
			}

			const uint8* BlendedResultBuffer = BaseDataBuffer; 
			if (Kernel.ColorBlend && !bSkipColorBlend)
			{
				uint8* MutableColorBlendResultBuffer = Kernel.Mix || Kernel.SelectAlpha ? Register1.Data : ResultDataBuffer;
				BlendedResultBuffer = MutableColorBlendResultBuffer;

				if (Kernel.ColorBlend == OpImageLayerBlendOps::Blend)
				{
					BlendedResultBuffer = BlenDataBuffer;
				}
				else if (Kernel.ColorBlendWithWord)
				{
					check(Kernel.MakeConstantU16);
					Kernel.ColorBlendWithWord(MutableColorBlendResultBuffer, BaseDataBuffer, (uint16*)BlenDataBuffer, BaseBytesPerElem*NumWaveElems);
				}
				else
				{
					Kernel.ColorBlend(MutableColorBlendResultBuffer, BaseDataBuffer, BlenDataBuffer, BaseBytesPerElem*NumWaveElems);
				}
			}

			const uint8* AlphaBlendedResultBuffer = BaseDataBuffer;
			if (Kernel.AlphaBlend)
			{
				uint8* MutableAlphaBlendedResult = Register2.Data;
				AlphaBlendedResultBuffer = MutableAlphaBlendedResult;

				if (Kernel.AlphaBlend == OpImageLayerBlendOps::Blend)
				{
					AlphaBlendedResultBuffer = BlenDataBuffer;
				}
				else if (Kernel.AlphaBlendWithWord)
				{
					check(Kernel.MakeConstantU16);
					Kernel.AlphaBlendWithWord(MutableAlphaBlendedResult, BaseDataBuffer, (uint16*)BlenDataBuffer, BaseBytesPerElem*NumWaveElems);
				}
				else
				{
					Kernel.AlphaBlend(MutableAlphaBlendedResult, BaseDataBuffer, BlenDataBuffer, BaseBytesPerElem*NumWaveElems);
				}
			}

			const uint8* MixResultBuffer = BlendedResultBuffer;
			if (Kernel.Mix && !bSkipColorBlend)
			{
				uint8* MutableMixResultBuffer = Kernel.SelectAlpha ? Register3.Data : ResultDataBuffer;
				MixResultBuffer = MutableMixResultBuffer;

				Kernel.Mix(MutableMixResultBuffer, BaseDataBuffer, BlendedResultBuffer, MaskDataBuffer, NumWaveElems*BaseBytesPerElem);
			}

			if (Kernel.SelectAlpha)
			{
				Kernel.SelectAlpha(ResultDataBuffer, MixResultBuffer, AlphaBlendedResultBuffer, NumWaveElems); 
			}
			else if (MixResultBuffer != ResultDataBuffer)
			{
				FMemory::Memcpy(ResultDataBuffer, MixResultBuffer, NumWaveElems*BaseBytesPerElem);
			}
		}
	}

	void RunKernelOnBatch(const FOpCombineKernel& Kernel, const FOpLayerBatchViews& BatchViews)
	{
		constexpr int32 MaxElemSize = 4;
		constexpr int32 NumCacheLinesPerWave = 8;
		constexpr int32 WaveSizeInElems = (PLATFORM_CACHE_LINE_SIZE * NumCacheLinesPerWave)/MaxElemSize;

		struct alignas(PLATFORM_CACHE_LINE_SIZE) FTemporalRegister
		{
			uint8 Data[WaveSizeInElems*MaxElemSize];	
		};

		FTemporalRegister Register0;
		FTemporalRegister Register1;
		FTemporalRegister Register2;
		FTemporalRegister Register3;
		FTemporalRegister Register4;

		int32 BaseBytesPerElem = BatchViews.Base.Num() / BatchViews.NumElems;
		int32 BlenBytesPerElem = BatchViews.Blend.Num() / BatchViews.NumElems;

		check(BaseBytesPerElem <= MaxElemSize);
		check(BlenBytesPerElem <= MaxElemSize);

		uint8* ResultBuffer = BatchViews.Result.GetData();
		const uint8* BaseBuffer = BatchViews.Base.GetData();
		const uint8* BlenBuffer = BatchViews.Blend.GetData();
		const uint8* MaskBuffer = BatchViews.Mask.GetData();

		uint64 InOutDecodeToken = BatchViews.DecodeTokenRLE;

		if (Kernel.MakeConstant)
		{
			Kernel.MakeConstant(Register1.Data, Kernel.Constant, WaveSizeInElems);
		}

		const int32 NumWaves = FMath::DivideAndRoundUp<int32>(BatchViews.NumElems, WaveSizeInElems);
		for (int32 Idx = 0; Idx < NumWaves; ++Idx)
		{
			int32 WaveBeginInElems = Idx * WaveSizeInElems;
			int32 WaveEndInElems   = FMath::Min(WaveBeginInElems + WaveSizeInElems, BatchViews.NumElems); 

			int32 NumWaveElems = WaveEndInElems - WaveBeginInElems;

			uint8* ResultDataBuffer     = ResultBuffer + WaveBeginInElems*BaseBytesPerElem;
			const uint8* BaseDataBuffer = BaseBuffer + WaveBeginInElems*BaseBytesPerElem;
			const uint8* MaskDataBuffer = MaskBuffer + WaveBeginInElems;
			const uint8* BlenDataBuffer = BlenBuffer + WaveBeginInElems*BlenBytesPerElem;

			const uint8* FormatedBaseBuffer = BaseDataBuffer;
			if (Kernel.BaseFormat)
			{
				Kernel.BaseFormat(Register0.Data, BaseDataBuffer, NumWaveElems);
				FormatedBaseBuffer = Register0.Data;
			}

			const uint8* FormatedBlenBuffer = Kernel.MakeConstant ? Register1.Data : BlenDataBuffer;
			if (Kernel.BlendFormat)
			{
				Kernel.BlendFormat(Register1.Data, BlenDataBuffer, NumWaveElems);
				FormatedBlenBuffer = Register1.Data;
			}

			const uint8* CombineBuffer = FormatedBaseBuffer;
			if (Kernel.Combine)
			{
				Kernel.Combine(Register2.Data, FormatedBaseBuffer, FormatedBlenBuffer, NumWaveElems);
				CombineBuffer = Register2.Data;
			}

			const uint8* FormatedMaskBuffer = MaskDataBuffer;
			if (Kernel.MaskDecode)
			{
				Kernel.MaskDecode(Register3.Data, MaskDataBuffer, InOutDecodeToken, NumWaveElems);
				FormatedMaskBuffer = Register3.Data;
			}

			if (Kernel.MaskFormat)
			{
				Kernel.MaskFormat(Register4.Data, FormatedMaskBuffer, NumWaveElems);
				FormatedMaskBuffer = Register4.Data;
			}

			const uint8* MixedBuffer = CombineBuffer;
			if (Kernel.Mix)
			{
				uint8* MutableMixedBuffer = Kernel.ResultFormat ? Register1.Data : ResultDataBuffer;
				MixedBuffer = MutableMixedBuffer;

				Kernel.Mix(MutableMixedBuffer, FormatedBaseBuffer, CombineBuffer, FormatedMaskBuffer, Kernel.CombineBytesPerElem*NumWaveElems);
			}

			if (Kernel.ResultFormat)
			{
				Kernel.ResultFormat(ResultDataBuffer, MixedBuffer, NumWaveElems);
			}
		}
	}


	int32 GetOpLayerNumBathesLODRangeForMaskRLE(const FOpLayerBatchArgs& Args)
	{
		// For masks with L_UByteRLE format, Parallel execution is not supported. One batch per lod.
		check(Args.Result);
		check(Args.Result->GetLODCount() >= Args.LODEnd);
		return Args.LODEnd - Args.LODBegin;
	}

	FOpLayerBatchViews GetOpLayerBatchViewsLODRangeForMaskRLE(int32 BatchId, const FOpLayerBatchArgs& Args)
	{
		FOpLayerBatchViews BatchViewsResult;

		check(Args.Result);
		BatchViewsResult.Result = Args.Result->DataStorage.GetLOD(BatchId);

		BatchViewsResult.NumElems = BatchViewsResult.Result.Num() / Args.ResultBytesPerElem;

		if (Args.Base)
		{
			BatchViewsResult.Base = Args.Base->DataStorage.GetLOD(BatchId);
			check(BatchViewsResult.NumElems == BatchViewsResult.Base.Num() / Args.BaseBytesPerElem);
		}

		if (Args.Blend)
		{
			BatchViewsResult.Blend = Args.Blend->DataStorage.GetLOD(BatchId);
			check(BatchViewsResult.NumElems == BatchViewsResult.Blend.Num() / Args.BlendBytesPerElem);
		}

		check(Args.Mask)
		{
			BatchViewsResult.Mask = Args.Mask->DataStorage.GetLOD(BatchId);

			FIntVector2 LODSize = Args.Mask->CalculateMipSize(BatchId);	

			BatchViewsResult.DecodeTokenRLE = GetBeginTokenIterativeDecodeRLE_L(
					Args.Mask->GetLODData(BatchId), LODSize.Y);
		}

		return BatchViewsResult;
	}

	FORCENOINLINE int32 GetOpLayerNumBatchesLODRange(const FOpLayerBatchArgs& Args)
	{
		check(Args.Result);

		if (Args.Mask && Args.Mask->GetFormat() == EImageFormat::L_UByteRLE)
		{
			return GetOpLayerNumBathesLODRangeForMaskRLE(Args);
		}

		const int32 NumBatches = Args.Result->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.ResultBytesPerElem, Args.LODBegin, Args.LODEnd);

#if DO_CHECK
		if (Args.Base)
		{
			check(NumBatches == Args.Base->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.BaseBytesPerElem, Args.LODBegin, Args.LODEnd));
		}
	
		if (Args.Blend)
		{
			check(NumBatches == Args.Blend->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.BlendBytesPerElem, Args.LODBegin, Args.LODEnd));
		}

		if (Args.Mask)
		{
			check(NumBatches == Args.Mask->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.MaskBytesPerElem, Args.LODBegin, Args.LODEnd));
		}
#endif
		return NumBatches;
	}
	
	FORCENOINLINE int32 GetOpLayerNumBatchesLODRangeOffsetViews(const FOpLayerBatchArgs& Args)
	{
		const bool bOnlyFirstLOD = Args.FirstLODOffsetInElems >= 0;

		check(Args.Result);
		const int32 NumBatches = bOnlyFirstLOD
				? Args.Result->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.ResultBytesPerElem, Args.FirstLODOffsetInElems*Args.ResultBytesPerElem)
				: Args.Result->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.ResultBytesPerElem, Args.LODBegin, Args.LODEnd);

#if DO_CHECK
		if (Args.Base)
		{
			const int32 BaseNumBatches = bOnlyFirstLOD
					? Args.Base->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.BaseBytesPerElem, Args.FirstLODOffsetInElems*Args.BaseBytesPerElem)
					: Args.Base->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.BaseBytesPerElem, Args.LODBegin, Args.LODEnd);

			check(NumBatches == BaseNumBatches);
		}
	
		if (Args.Blend)
		{
			const int32 BlendNumBatches = bOnlyFirstLOD
					? Args.Blend->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.BlendBytesPerElem, Args.FirstLODOffsetInElems*Args.BlendBytesPerElem)
					: Args.Blend->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.BlendBytesPerElem, Args.LODBegin, Args.LODEnd);

			check(NumBatches == BlendNumBatches);
		}

		if (Args.Mask)
		{
			const int32 MaskNumBatches = bOnlyFirstLOD
					? Args.Mask->DataStorage.GetNumBatchesFirstLODOffset(Args.BatchNumElems, Args.MaskBytesPerElem, Args.FirstLODOffsetInElems*Args.MaskBytesPerElem)
					: Args.Mask->DataStorage.GetNumBatchesLODRange(Args.BatchNumElems, Args.MaskBytesPerElem, Args.LODBegin, Args.LODEnd);

			check(NumBatches == MaskNumBatches);
		}
#endif
		return NumBatches;
	}


	FORCENOINLINE FOpLayerBatchViews GetOpLayerBatchViews(int32 BatchId, const FOpLayerBatchArgs& Args)
	{
		FOpLayerBatchViews BatchViewsResult;

		check(Args.Result);
		BatchViewsResult.Result = Args.Result->DataStorage.GetBatch(BatchId, Args.BatchNumElems, Args.ResultBytesPerElem);

		BatchViewsResult.NumElems = BatchViewsResult.Result.Num() / Args.ResultBytesPerElem;

		if (Args.Base)
		{
			BatchViewsResult.Base = Args.Base->DataStorage.GetBatch(BatchId, Args.BatchNumElems, Args.BaseBytesPerElem);
			check(BatchViewsResult.NumElems == BatchViewsResult.Base.Num() / Args.BaseBytesPerElem);
		}

		if (Args.Blend)
		{
			BatchViewsResult.Blend = Args.Blend->DataStorage.GetBatch(BatchId, Args.BatchNumElems, Args.BlendBytesPerElem);
			check(BatchViewsResult.NumElems == BatchViewsResult.Blend.Num() / Args.BlendBytesPerElem);
		}

		if (Args.Mask)
		{
			BatchViewsResult.Mask = Args.Mask->DataStorage.GetBatch(BatchId, Args.BatchNumElems, Args.MaskBytesPerElem);
			check(BatchViewsResult.NumElems == BatchViewsResult.Mask.Num() / Args.MaskBytesPerElem);
		}

		return BatchViewsResult;
	}

	FORCENOINLINE FOpLayerBatchViews GetOpLayerBatchLODRangeViews(int32 BatchId, const FOpLayerBatchArgs& Args)
	{
		FOpLayerBatchViews BatchViewsResult;
		
		if (Args.Mask)
		{
			if (Args.Mask->GetFormat() == EImageFormat::L_UByteRLE)
			{
				return GetOpLayerBatchViewsLODRangeForMaskRLE(BatchId, Args);
			}
		}

		check(Args.Result);
		BatchViewsResult.Result = 
					Args.Result->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.ResultBytesPerElem, Args.LODBegin, Args.LODEnd);

		BatchViewsResult.NumElems = BatchViewsResult.Result.Num() / Args.ResultBytesPerElem;

		if (Args.Base)
		{
			BatchViewsResult.Base = 
					Args.Base->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.BaseBytesPerElem, Args.LODBegin, Args.LODEnd);
			check(BatchViewsResult.NumElems == BatchViewsResult.Base.Num() / Args.BaseBytesPerElem);
		}

		if (Args.Blend)
		{
			BatchViewsResult.Blend = 
					Args.Blend->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.BlendBytesPerElem, Args.LODBegin, Args.LODEnd);
			check(BatchViewsResult.NumElems == BatchViewsResult.Blend.Num() / Args.BlendBytesPerElem);
		}

		if (Args.Mask)
		{
			BatchViewsResult.Mask = 
					Args.Mask->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.MaskBytesPerElem, Args.LODBegin, Args.LODEnd);
			check(BatchViewsResult.NumElems == BatchViewsResult.Mask.Num() / Args.MaskBytesPerElem);
		}

		return BatchViewsResult;
	}

	FORCENOINLINE FOpLayerBatchViews GetOpLayerBatchLODRangeOffsetViews(int32 BatchId, const FOpLayerBatchArgs& Args)
	{
		FOpLayerBatchViews BatchViewsResult;
		
		const bool bOnlyFirstLOD = Args.FirstLODOffsetInElems >= 0;

		check(Args.Result);
		BatchViewsResult.Result = bOnlyFirstLOD
				? Args.Result->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.ResultBytesPerElem, Args.FirstLODOffsetInElems*Args.ResultBytesPerElem) 
				: Args.Result->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.ResultBytesPerElem, Args.LODBegin, Args.LODEnd);

		BatchViewsResult.NumElems = BatchViewsResult.Result.Num() / Args.ResultBytesPerElem;
		
		if (Args.Base)
		{
			BatchViewsResult.Base = bOnlyFirstLOD
					? Args.Base->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.BaseBytesPerElem, Args.FirstLODOffsetInElems*Args.BaseBytesPerElem) 
					: Args.Base->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.BaseBytesPerElem, Args.LODBegin, Args.LODEnd); 

			check(BatchViewsResult.NumElems == BatchViewsResult.Base.Num() / Args.BaseBytesPerElem);
		}

		if (Args.Blend)
		{
			BatchViewsResult.Blend = bOnlyFirstLOD
					? Args.Blend->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.BlendBytesPerElem, Args.FirstLODOffsetInElems*Args.BlendBytesPerElem) 
					: Args.Blend->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.BlendBytesPerElem, Args.LODBegin, Args.LODEnd); 

			check(BatchViewsResult.NumElems == BatchViewsResult.Blend.Num() / Args.BlendBytesPerElem);
		}

		if (Args.Mask)
		{
			BatchViewsResult.Mask = bOnlyFirstLOD
					? Args.Mask->DataStorage.GetBatchFirstLODOffset(BatchId, Args.BatchNumElems, Args.MaskBytesPerElem, Args.FirstLODOffsetInElems*Args.MaskBytesPerElem) 
					: Args.Mask->DataStorage.GetBatchLODRange(BatchId, Args.BatchNumElems, Args.MaskBytesPerElem, Args.LODBegin, Args.LODEnd); 

			check(BatchViewsResult.NumElems == BatchViewsResult.Mask.Num() / Args.MaskBytesPerElem);
		}

		return BatchViewsResult;
	}


	void ImageLayerImpl(
			FImage* DestImage, const FImage* BaseImage, const FImage* BlenImage, const FImage* MaskImage, 
			FBlendFuncType* ColorBlendFunc, FBlendFuncType* AlphaBlendFunc,
			bool bBlendWithConstant, const FVector4f& BlendConstant,
			int32 LODBegin, int32 LODEnd, 
			bool bApplyColorBlendToAlpha, 
			bool bUseMaskFromBlendAlpha, 
			bool bUseBaseSourceFromBaseAlpha, 
			bool bUseBlendSourceFromBlendAlpha, 
			uint8 BlendAlphaSourceChannel)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageLayer)

		check(DestImage && DestImage->GetFormat() == GetUncompressedFormat(DestImage->GetFormat()));
		check(BaseImage && BaseImage->GetFormat() == GetUncompressedFormat(BaseImage->GetFormat()));
		
		check(LODEnd <= BaseImage->GetLODCount());

		if (!bBlendWithConstant)
		{
			check(BlenImage);
			check(BlenImage->GetFormat() == GetUncompressedFormat(BlenImage->GetFormat()));
			check(BlenImage->GetLODCount() >= LODEnd);

			check(BlenImage->GetSize() == BaseImage->GetSize());
		}

		check(BaseImage->GetFormat() == DestImage->GetFormat());
		check(BaseImage->GetSize() == DestImage->GetSize());

		if (MaskImage)
		{
			check(MaskImage->GetFormat() == EImageFormat::L_UByte || MaskImage->GetFormat() == EImageFormat::L_UByteRLE);
		}

		const int32 BaseBytesPerElem = GetImageFormatData(BaseImage->GetFormat()).BytesPerBlock;
		const int32 BlenBytesPerElem = BlenImage ? GetImageFormatData(BlenImage->GetFormat()).BytesPerBlock : 0;

		bool bOnlyFirstLOD = (LODBegin == 0 || LODEnd == 1) || BaseImage->GetLODCount() == 1;
		
		int32 FirstLODOffsetInElems = -1;	
		int32 NumRelevantElems = -1;

		if (BlenImage && BlenImage->Flags & FImage::IF_HAS_RELEVANCY_MAP && bOnlyFirstLOD)
		{
			check(BlenImage->RelevancyMaxY < BaseImage->GetSizeY());
			check(BlenImage->RelevancyMaxY >= BlenImage->RelevancyMinY);

			uint16 SizeX = BaseImage->GetSizeX();
			NumRelevantElems = (BlenImage->RelevancyMaxY - BlenImage->RelevancyMinY + 1) * SizeX;
			
			FirstLODOffsetInElems = BlenImage->RelevancyMinY * SizeX;
		}
	
		constexpr int32 BatchNumElems = 4096*2;

		OpImageLayerInternal::FOpLayerBatchArgs BatchArgs;
		BatchArgs.BatchNumElems         = BatchNumElems;
		BatchArgs.LODBegin              = LODBegin;
		BatchArgs.LODEnd                = LODEnd;
		BatchArgs.FirstLODOffsetInElems = bOnlyFirstLOD ? FirstLODOffsetInElems : -1;
		BatchArgs.Result                = DestImage;
		BatchArgs.Base                  = BaseImage;
		BatchArgs.Blend                 = BlenImage;
		BatchArgs.Mask                  = MaskImage;
		BatchArgs.ResultBytesPerElem    = BaseBytesPerElem;
		BatchArgs.BaseBytesPerElem      = BaseBytesPerElem;
		BatchArgs.BlendBytesPerElem     = BlenBytesPerElem;
		BatchArgs.MaskBytesPerElem      = 1;

		OpImageLayerInternal::FMakeOpBlendKernelArgs MakeKernelArgs
		{
			.Dest  = DestImage,
			.Base  = BaseImage,
			.Blend = BlenImage,
			.Mask  = MaskImage,

			.ColorBlendFunc = ColorBlendFunc,
			.AlphaBlendFunc = AlphaBlendFunc,

			.bApplyColorBlendToAlpha       = bApplyColorBlendToAlpha,
			.bUseBaseSourceFromBaseAlpha   = bUseBaseSourceFromBaseAlpha,
			.bUseBlendSourceFromBlendAlpha = bUseBlendSourceFromBlendAlpha,
			.bUseMaskFromBlendAlpha        = bUseMaskFromBlendAlpha,
			.BlendAlphaSourceChannel       = BlendAlphaSourceChannel,

			.bBlendWithConstant = bBlendWithConstant, 
			.RealConstant       = BlendConstant,
		};

		int32 NumBatches = 0;
		if (BatchArgs.Mask && BatchArgs.Mask->GetFormat() == EImageFormat::L_UByteRLE)
		{
			NumBatches = OpImageLayerInternal::GetOpLayerNumBatchesLODRange(BatchArgs);
		}
		else
		{
			NumBatches = OpImageLayerInternal::GetOpLayerNumBatchesLODRangeOffsetViews(BatchArgs);
		}

		bool bUseRelevancy = NumRelevantElems >= 0;
		if (BatchArgs.Mask)
		{
			bUseRelevancy &= BatchArgs.Mask->GetFormat() != EImageFormat::L_UByteRLE;
		}

		// This will always be an upper-bound for bOnlyFirtsLODs, check if it performs as expected or it needs more fine tune.
		const int32 NumRelevantBatches = bUseRelevancy
				? FMath::DivideAndRoundUp(NumRelevantElems, BatchNumElems)
				: NumBatches;

		const OpImageLayerInternal::FOpBlendKernel Kernel = OpImageLayerInternal::MakeKernel(MakeKernelArgs);
		ParallelExecutionUtils::InvokeBatchParallelFor(NumRelevantBatches, 
		[
			&BatchArgs, &Kernel
		] (int32 BatchId)
		{

			OpImageLayerInternal::FOpLayerBatchViews BatchViews;
			
			if (BatchArgs.Mask && BatchArgs.Mask->GetFormat() == EImageFormat::L_UByteRLE)
			{
				BatchViews = OpImageLayerInternal::GetOpLayerBatchLODRangeViews(BatchId, BatchArgs);
			}
			else
			{
				BatchViews = OpImageLayerInternal::GetOpLayerBatchLODRangeOffsetViews(BatchId, BatchArgs);
			}

			OpImageLayerInternal::RunKernelOnBatch(Kernel, BatchViews);
		});

	}

	void ImageLayerCombineImpl(
			FImage* DestImage, const FImage* BaseImage, const FImage* BlenImage, const FImage* MaskImage, 
			FCombineFuncType* CombineFunc,
			bool bCombineWithConstant, const FVector4f& Constant,
			int32 LODBegin, int32 LODEnd)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageLayerNormalCombine)

		check(BaseImage->GetFormat() == DestImage->GetFormat());
		check(DestImage->GetFormat() == GetUncompressedFormat(DestImage->GetFormat()));
		check(BaseImage->GetFormat() == GetUncompressedFormat(BaseImage->GetFormat()));
		
		check(BaseImage->GetLODCount() >= LODEnd);

		if (!bCombineWithConstant)
		{
			check(BlenImage)
			check(BlenImage->GetFormat() == GetUncompressedFormat(BlenImage->GetFormat()));
			check(BlenImage->GetLODCount() >= LODEnd);
		}

		if (MaskImage)
		{
			check(MaskImage->GetFormat() == GetUncompressedFormat(MaskImage->GetFormat()));
			check(MaskImage->GetFormat() == EImageFormat::L_UByte || MaskImage->GetFormat() == EImageFormat::L_UByteRLE);
		}


		const int32 BaseBytesPerElem = GetImageFormatData(BaseImage->GetFormat()).BytesPerBlock;
		const int32 BlenBytesPerElem = BlenImage ? GetImageFormatData(BlenImage->GetFormat()).BytesPerBlock : 0;

		bool bOnlyFirstLOD = (LODBegin == 0 && LODEnd == 1) || BaseImage->GetLODCount() == 1;
		
		int32 FirstLODDataOffset = -1;	
		int32 NumRelevantElems = -1;

		if (BlenImage && BlenImage->Flags & FImage::IF_HAS_RELEVANCY_MAP && bOnlyFirstLOD)
		{
			check(BlenImage->RelevancyMaxY < BaseImage->GetSizeY());
			check(BlenImage->RelevancyMaxY >= BlenImage->RelevancyMinY);

			uint16 SizeX = BaseImage->GetSizeX();
			NumRelevantElems = (BlenImage->RelevancyMaxY - BlenImage->RelevancyMinY + 1) * SizeX;
			
			FirstLODDataOffset = BlenImage->RelevancyMinY * SizeX * BlenBytesPerElem;
		}
	
		constexpr int32 BatchNumElems = 4096*2;

		OpImageLayerInternal::FOpLayerBatchArgs BatchArgs;
		BatchArgs.BatchNumElems         = BatchNumElems;
		BatchArgs.LODBegin              = LODBegin;
		BatchArgs.LODEnd                = LODEnd;
		BatchArgs.FirstLODOffsetInElems = bOnlyFirstLOD ? FirstLODDataOffset : -1;
		BatchArgs.Result                = DestImage;
		BatchArgs.Base                  = BaseImage;
		BatchArgs.Blend                 = BlenImage;
		BatchArgs.Mask                  = MaskImage;
		BatchArgs.ResultBytesPerElem    = BaseBytesPerElem;
		BatchArgs.BaseBytesPerElem      = BaseBytesPerElem;
		BatchArgs.BlendBytesPerElem     = BlenBytesPerElem;
		BatchArgs.MaskBytesPerElem      = 1;

		OpImageLayerInternal::FMakeOpCombineKernelArgs MakeKernelArgs
		{
			.Dest  = DestImage,
			.Base  = BaseImage,
			.Blend = BlenImage,
			.Mask  = MaskImage,

			.CombineFunc = CombineFunc,

			.bCombineWithConstant = bCombineWithConstant, 
			.Constant             = FLinearColor(Constant).QuantizeRound(),
		};

		int32 NumBatches = 0;
		if (BatchArgs.Mask && BatchArgs.Mask->GetFormat() == EImageFormat::L_UByteRLE)
		{
			NumBatches = OpImageLayerInternal::GetOpLayerNumBatchesLODRange(BatchArgs);
		}
		else
		{
			NumBatches = OpImageLayerInternal::GetOpLayerNumBatchesLODRangeOffsetViews(BatchArgs);
		}

		// This will always be an upper-bound for bOnlyFirtsLODs, check if it performs as expected or it needs more fine tune.
		const int32 NumRelevantBatches = NumRelevantElems != -1
				? FMath::DivideAndRoundUp(NumRelevantElems, BatchNumElems)
				: NumBatches;

		const OpImageLayerInternal::FOpCombineKernel Kernel = OpImageLayerInternal::MakeKernel(MakeKernelArgs);
		ParallelExecutionUtils::InvokeBatchParallelFor(NumRelevantBatches, 
		[
			&BatchArgs, &Kernel
		] (int32 BatchId)
		{

			OpImageLayerInternal::FOpLayerBatchViews BatchViews;
			
			if (BatchArgs.Mask && BatchArgs.Mask->GetFormat() == EImageFormat::L_UByteRLE)
			{
				BatchViews = OpImageLayerInternal::GetOpLayerBatchLODRangeViews(BatchId, BatchArgs);
			}
			else
			{
				BatchViews = OpImageLayerInternal::GetOpLayerBatchLODRangeOffsetViews(BatchId, BatchArgs);
			}

			OpImageLayerInternal::RunKernelOnBatch(Kernel, BatchViews);
		});

	}
} // namespece OpImageLayerInternal;

	//TODO: Remove this, use the kernel execution aproatch. Needs some masking to be implemented. 
	template< 
		uint32 (*BLEND_FUNC)(uint32, uint32),
		bool CLAMP,
		uint32 NC>
    void BufferLayerFormatStrideNoAlpha(
		FImage* DestImage,
		int32 DestOffset,
		int32 Stride,
		const FImage* MaskImage,
		const FImage* BlendImage/*, int32 LODCount*/)
	{
        const uint8* MaskBuf = MaskImage->GetLODData(0);
        const uint8* BlendedBuf = BlendImage->GetLODData(0);
		uint8* DestBuf = DestImage->GetLODData(0) + DestOffset;

		// This could happen in case of missing data files.
		if (!BlendedBuf || !MaskBuf || !DestImage->GetLODData(0))
		{
			return;
		}

		EImageFormat MaskFormat = MaskImage->GetFormat();
        bool bIsUncompressed = (MaskFormat == EImageFormat::L_UByte);

        if (bIsUncompressed)
		{
			int32 RowCount = BlendImage->GetSizeY();
			int32 PixelCount = BlendImage->GetSizeX();
			for (int32 RowIndex = 0; RowIndex < RowCount; ++RowIndex)
			{
				for (int32 PixelIndex = 0; PixelIndex < PixelCount; ++PixelIndex)
				{
					uint32 Mask = *MaskBuf;
					if (Mask)
					{
						for (int32 C = 0; C < NC; ++C)
						{
							uint32 Base = *DestBuf;
							uint32 Blended = *BlendedBuf;
							uint32 Result = BLEND_FUNC(Base, Blended);
							if constexpr (CLAMP)
							{
								*DestBuf = (uint8)FMath::Min(255u, Result);
							}
							else
							{
								*DestBuf = (uint8)Result;
							}
							++DestBuf;
							++BlendedBuf;
						}
					}
					else
					{
						DestBuf += NC;
						BlendedBuf += NC;
					}
					++MaskBuf;
				}

				DestBuf += Stride;
			}
		}
        else if (MaskFormat == EImageFormat::L_UBitRLE)
		{
			int32 Rows = MaskImage->GetSizeY();
			int32 Width = MaskImage->GetSizeX();

            //for (int32 lod = 0; lod < LODCount; ++lod)
            //{
				// Remove RLE header.
                MaskBuf += 4 + Rows*sizeof(uint32);

                for (int32 RowIndex = 0; RowIndex < Rows; ++RowIndex)
                {
                    const uint8* DestRowEnd = DestBuf + Width*NC;
                    while (DestBuf != DestRowEnd)
                    {
                        // Decode header
                        uint16 Zeros = *(const uint16*)MaskBuf;
                        MaskBuf += 2;

                        uint16 Ones = *(const uint16*)MaskBuf;
                        MaskBuf += 2;

                        // Skip
                        DestBuf += Zeros*NC;
                        BlendedBuf += Zeros*NC;

                        // Copy
                        FMemory::Memmove(DestBuf, BlendedBuf, Ones*NC);

                        DestBuf += NC*Ones;
                        BlendedBuf += NC*Ones;
                    }

                    DestBuf += Stride;
                }

                //Rows = FMath::DivideAndRoundUp(Rows, 2);
                //Width = FMath::DivideAndRoundUp(Width, 2);
            //}
		}
		else
		{
			checkf( false, TEXT("Unsupported mask format.") );
		}
	}


	template<uint32 (*BLEND_FUNC)(uint32,uint32),
			 bool CLAMP>
    void BufferLayerStrideNoAlpha(FImage* DestImage, int32 DestOffset, int32 Stride, const FImage* MaskImage, const FImage* BlendImage/*, int32 LODCount*/)
	{
		if (BlendImage->GetFormat() == EImageFormat::RGB_UByte)
		{
			BufferLayerFormatStrideNoAlpha<BLEND_FUNC, CLAMP, 3>
					(DestImage, DestOffset, Stride, MaskImage, BlendImage/*, LODCount*/);
		}
        else if (BlendImage->GetFormat() == EImageFormat::RGBA_UByte || 
				 BlendImage->GetFormat() == EImageFormat::BGRA_UByte)
		{
			BufferLayerFormatStrideNoAlpha<BLEND_FUNC, CLAMP, 4>
					(DestImage, DestOffset, Stride, MaskImage, BlendImage/*, LODCount*/);
		}
		else if (BlendImage->GetFormat() == EImageFormat::L_UByte)
		{
			BufferLayerFormatStrideNoAlpha<BLEND_FUNC, CLAMP, 1>
					(DestImage, DestOffset, Stride, MaskImage, BlendImage/*, LODCount*/);
		}
		else
		{
			checkf( false, TEXT("Unsupported format.") );
		}
	}


	/**
	* Apply a blending function to an image with another image as blending layer, on a subrect of
	* the base image.
	* \warning this method applies the blending function to the alpha channel too
	* \warning this method uses the mask as a binary mask (>0)
	*/
	template<uint32 (*BLEND_FUNC)(uint32, uint32), bool CLAMP>
	void ImageLayerOnBaseNoAlpha(
			FImage* BaseImage,
			const FImage* MaskImage,
			const FImage* BlendedImage,
			const box<FIntVector2>& Rect)
	{
		check(BaseImage->GetSizeX() >= Rect.min[0] + Rect.size[0]);
		check(BaseImage->GetSizeY() >= Rect.min[1] + Rect.size[1]);
		check(MaskImage->GetSizeX() == BlendedImage->GetSizeX());
		check(MaskImage->GetSizeY() == BlendedImage->GetSizeY());
		check(BaseImage->GetFormat() == BlendedImage->GetFormat());
		check(MaskImage->GetFormat() == EImageFormat::L_UByte ||
			  //UBYTE_RLE does not look to be supported.
			  //MaskImage->GetFormat() == EImageFormat::L_UByteRLE || 
			  MaskImage->GetFormat() == EImageFormat::L_UBitRLE);
        check(BaseImage->GetLODCount() <= MaskImage->GetLODCount());
        check(BaseImage->GetLODCount() <= BlendedImage->GetLODCount());

		int32 PixelSize = GetImageFormatData(BaseImage->GetFormat()).BytesPerBlock;

		int32 Start = (BaseImage->GetSizeX() * Rect.min[1] + Rect.min[0]) * PixelSize;
		int32 Stride = (BaseImage->GetSizeX() - Rect.size[0]) * PixelSize;

		// Stride is only valid for LOD 0, BufferLayerStride variants cannot operate on multiple lods.
		// TODO: review if this needs to be supported, and implement using a rect lod reducction at this level.
		BufferLayerStrideNoAlpha<BLEND_FUNC, CLAMP>(BaseImage, Start, Stride, MaskImage, BlendedImage/*, BaseImage->GetLODCount()*/);
	}

	//! Blend a subimage on the base using a mask.
	void ImageBlendOnBaseNoAlpha(FImage* BaseImage, const FImage* MaskImage, const FImage* BlendedImage, const box<FIntVector2>& Rect)
	{
		ImageLayerOnBaseNoAlpha<BlendChannel, false>(BaseImage, MaskImage, BlendedImage, Rect);
	}

	void ImageLayerBlend(
			FImage* DestImage, const FImage* BaseImage, const FImage* BlenImage, const FImage* MaskImage, 
			FBlendFuncType* ColorBlendFunc, FBlendFuncType* AlphaBlendFunc,
			int32 LODBegin, int32 LODEnd,
			bool bApplyColorBlendToAlpha, 
			bool bUseMaskFromBlendAlpha, 
			bool bUseBaseSourceFromBaseAlpha, 
			bool bUseBlendSourceFromBlendAlpha, 
			uint8 BlendAlphaSourceChannel)
	{
		constexpr bool bBlendWithConstant = false;
		OpImageLayerInternal::ImageLayerImpl(
				DestImage, BaseImage, BlenImage, MaskImage, 
				ColorBlendFunc, AlphaBlendFunc, 
				bBlendWithConstant, FVector4f{}, 
				LODBegin, LODEnd,  
				bApplyColorBlendToAlpha, 
				bUseMaskFromBlendAlpha, 
				bUseBaseSourceFromBaseAlpha,
				bUseBlendSourceFromBlendAlpha, 
				BlendAlphaSourceChannel);
	}

	void ImageLayerBlendConstant(
			FImage* DestImage, const FImage* BaseImage, const FImage* MaskImage, const FVector4f& Constant,  
			FBlendFuncType* ColorBlendFunc, FBlendFuncType* AlphaBlendFunc,
			int32 LODBegin, int32 LODEnd, 
			bool bApplyColorBlendToAlpha, 
			bool bUseMaskFromBlendAlpha, 
			bool bUseBaseSourceFromBaseAlpha, 
			bool bUseBlendSourceFromBlendAlpha, 
			uint8 BlendAlphaSourceChannel)
	{
		constexpr bool bBlendWithConstant = true;
		OpImageLayerInternal::ImageLayerImpl(
				DestImage, BaseImage, nullptr, MaskImage, 
				ColorBlendFunc, AlphaBlendFunc, 
				bBlendWithConstant, Constant, 
				LODBegin, LODEnd,
				bApplyColorBlendToAlpha, 
				bUseMaskFromBlendAlpha, 
				bUseBaseSourceFromBaseAlpha,
				bUseBlendSourceFromBlendAlpha, 
				BlendAlphaSourceChannel);
	}

	void ImageLayerCombine(
			FImage* DestImage, const FImage* BaseImage, const FImage* BlendImage, const FImage* MaskImage,
			FCombineFuncType* CombineFunc,
			int32 LODBegin, int32 LODEnd)
	{
		constexpr bool bCombineWithConstant = false;
		OpImageLayerInternal::ImageLayerCombineImpl(
				DestImage, BaseImage, BlendImage, MaskImage, 
				CombineFunc, 
				bCombineWithConstant, FVector4f{},
				LODBegin, LODEnd);
	}

	void ImageLayerCombineConstant(
			FImage* DestImage, const FImage* BaseImage, const FImage* MaskImage, const FVector4f& Constant,
			FCombineFuncType* CombineFunc,
			int32 LODBegin, int32 LODEnd)
	{
		constexpr bool bCombineWithConstant = true;
		OpImageLayerInternal::ImageLayerCombineImpl(
				DestImage, BaseImage, nullptr, MaskImage, 
				CombineFunc, 
				bCombineWithConstant, Constant,
				LODBegin, LODEnd);
	}

	FBlendFuncType* SelectBlendFunc(EBlendType BlendType) 
	{
		switch (BlendType)
		{
			case EBlendType::BT_SOFTLIGHT: return OpImageLayerBlendOps::SoftLight;
			case EBlendType::BT_HARDLIGHT: return OpImageLayerBlendOps::HardLight;
			case EBlendType::BT_BURN     : return OpImageLayerBlendOps::Burn;
			case EBlendType::BT_DODGE    : return OpImageLayerBlendOps::Dodge;
			case EBlendType::BT_SCREEN   : return OpImageLayerBlendOps::Screen;
			case EBlendType::BT_OVERLAY  : return OpImageLayerBlendOps::Overlay;
			case EBlendType::BT_LIGHTEN  : return OpImageLayerBlendOps::Lighten;
			case EBlendType::BT_MULTIPLY : return OpImageLayerBlendOps::Multiply;
			case EBlendType::BT_BLEND    : return OpImageLayerBlendOps::Blend;
			case EBlendType::BT_NONE     : return nullptr;
			default                      : check(false); return nullptr;
		}
	}
}

