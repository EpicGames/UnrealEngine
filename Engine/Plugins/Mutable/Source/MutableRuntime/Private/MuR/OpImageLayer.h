// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/ImageTypes.h"

namespace UE::Mutable::Private
{

using FBlendFuncType = void(uint8*, const uint8*, const uint8*, int32);

namespace OpImageLayerBlendOps
{
	void Blend    (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void Screen   (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void SoftLight(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void HardLight(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void Dodge    (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void Lighten  (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void Burn     (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void Multiply (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
	void Overlay  (uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
} // namespace OpImageLayerBlendOps
	
using FCombineFuncType = void(uint8*, const uint8*, const uint8*, int32);
namespace OpImageLayerCombineOps
{
	void NormalCombine(uint8* Dest, const uint8* Base, const uint8* Blen, int32 NumElems);
} // namespace OpImageLayerCombineOps


/** Blend a subimage on the base using a mask */
void ImageBlendOnBaseNoAlpha(FImage* BaseImage, const FImage* MaskImage, const FImage* BlendedImage, const box<FIntVector2>& Rect);

FBlendFuncType* SelectBlendFunc(EBlendType BlendType); 

void ImageLayerBlend(
		FImage* Dest, const FImage* Base, const FImage* Blend, const FImage* Mask, 
		FBlendFuncType* ColorBlendFunc, FBlendFuncType* AlphaBlendFunc,
		int32 LODBegin, int32 LODEnd, 
		bool bApplyColorBlendToAlpha, 
		bool bUseMaskFromBlendAlpha, 
		bool bUseBaseSourceFromBaseAlpha, 
		bool bUseBlendSourceFromBlendAlpha, 
		uint8 BlendAlphaSourceChannel);

void ImageLayerBlendConstant(
		FImage* Dest, const FImage* Base, const FImage* Mask, const FVector4f& Constant, 
		FBlendFuncType* ColorBlendFunc, FBlendFuncType* AlphaBlendFunc,
		int32 LODBegin, int32 LODEnd, 
		bool bApplyColorBlendToAlpha, 
		bool bUseMaskFromBlendAlpha, 
		bool bUseBaseSourceFromBaseAlpha, 
		bool bUseBlendSourceFromBlendAlpha, 
		uint8 BlendAlphaSourceChannel);

void ImageLayerCombine(
		FImage* Dest, const FImage* Base, const FImage* Blend, const FImage* Mask,
		FCombineFuncType* CombineFunc,
		int32 LODBegin, int32 LODEnd);

void ImageLayerCombineConstant(
		FImage* Dest, const FImage* Base, const FImage* Mask, const FVector4f& Constant,
		FCombineFuncType* CombineFunc,
		int32 LODBegin, int32 LODEnd);

}
