// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"
#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{

	inline void ImageLuminance(FImage* DestImage, const FImage* AImage)
	{
		check(DestImage && AImage && DestImage->GetFormat() == EImageFormat::L_UByte);

		const int32 BytesPerElem = GetImageFormatData(AImage->GetFormat()).BytesPerBlock;
		constexpr int32 NumBatchElems = 1 << 14;

		const int32 NumBatches = DestImage->DataStorage.GetNumBatches(NumBatchElems, 1); 
		check(NumBatches == AImage->DataStorage.GetNumBatches(NumBatchElems, BytesPerElem)); 

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
        [
            DestImage, AImage, NumBatchElems, BytesPerElem
        ](int32 BatchId)
        {
			TArrayView<uint8> DestView = DestImage->DataStorage.GetBatch(BatchId, NumBatchElems, 1);
			TArrayView<const uint8> AView = AImage->DataStorage.GetBatch(BatchId, NumBatchElems, BytesPerElem);

			uint8* DestBuf = DestView.GetData(); 
			const uint8* ABuf = AView.GetData(); 
			
            const int32 NumElems = DestView.Num() / 1;

            switch (AImage->GetFormat())
            {
            case EImageFormat::RGB_UByte:
            {
                for (int32 I = 0; I < NumElems; ++I)
                {
                    const uint16 L = (77*ABuf[3*I + 0] + 150*ABuf[3*I + 1] + 29*ABuf[3*I + 2]);
                    DestBuf[I] = uint8(L >> 8);
                }
                break;
            }

            case EImageFormat::RGBA_UByte:
            {
                for (int32 I = 0; I < NumElems; ++I)
                {
                    const uint16 L = (77*ABuf[4*I + 0] + 150*ABuf[4*I + 1] + 29*ABuf[4*I + 2]);
                    DestBuf[I] = uint8(L >> 8);
                }
                break;
            }

            case EImageFormat::BGRA_UByte:
            {
                for (int32 I = 0; I < NumElems; ++I)
                {
                    const uint16 L = (77*ABuf[4*I + 2] + 150*ABuf[4*I + 1] + 29*ABuf[4*I + 0]);
                    DestBuf[I] = uint8(L >> 8);
                }
                break;
            }

            default:
                check(false);
                break;
            }
        });
	}

}
