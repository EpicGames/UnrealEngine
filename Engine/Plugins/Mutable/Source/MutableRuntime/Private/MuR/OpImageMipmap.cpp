// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImagePrivate.h"
#include "MuR/ImageFormatUtils.h"
#include "MuR/ParallelExecutionUtils.h"
#include "HAL/UnrealMemory.h"
#include "Async/ParallelFor.h"

namespace
{

bool bEnableCompressedMipGenerationMemoryOptimizations = true;
static FAutoConsoleVariableRef CVarEnableCompressedMipGenerationMemoryOptimizations (
	TEXT("mutable.EnableCompressedMipGenerationMemoryOptimizations"),
	bEnableCompressedMipGenerationMemoryOptimizations,
	TEXT("If set to true, enables memory optimizations for mip generation on compressed images."),
	ECVF_Default);
}

namespace UE::Mutable::Private
{


namespace OpImageMipmapInternal
{

	template<int32 PIXEL_SIZE>
	inline void GenerateNextMipmapUint8Unfiltered(const uint8* SourceData, uint8* DestData, FIntVector2 SourceSize)
	{
		FIntVector2 DestSize = FIntVector2(
				FMath::DivideAndRoundUp(SourceSize.X, 2), 
				FMath::DivideAndRoundUp(SourceSize.Y, 2));

		for (int32 Y = 0; Y < DestSize.Y; ++Y)
		{
			for (int32 X = 0; X < DestSize.X; ++X)
			{
				uint8* DestPixel = DestData + (Y*DestSize.X + X) * PIXEL_SIZE;
				const uint8* SourcePixel = SourceData + ((Y << 1) * SourceSize.X + (X << 1)) * PIXEL_SIZE;

				FMemory::Memcpy(DestPixel, SourcePixel, PIXEL_SIZE);
			}
		}
	}

	template<int32 PIXEL_SIZE>
	UE_NO_PROFILE_SVE_BUG
	inline void GenerateNextMipmapUint8SimpleAverage(const uint8* SourceData, uint8* DestData, FIntVector2 SourceSize)
	{
		check(SourceSize[0] > 1 || SourceSize[1] > 1);

		FIntVector2 DestSize = FIntVector2(
				FMath::DivideAndRoundUp(SourceSize.X, 2),
				FMath::DivideAndRoundUp(SourceSize.Y, 2));

		int32 FullColumns = SourceSize.X / 2;
		int32 FullRows = SourceSize.Y / 2;
		bool bStrayColumn = (SourceSize.X % 2) != 0;
		bool bStrayRow = (SourceSize.Y % 2) != 0;

		int32 SourceStride = SourceSize.X * PIXEL_SIZE;
		int32 DestStride = DestSize.X * PIXEL_SIZE;

		const auto ProcessRow = [
			DestData, SourceData, FullColumns, bStrayColumn, SourceStride, DestStride
		] (uint32 Y)
		{
			const uint8* SourceRow0 = SourceData + 2*Y*SourceStride;
			const uint8* SourceRow1 = SourceRow0 + SourceStride;
			uint8* DestRow = DestData + Y * DestStride;

			for (int32 X = 0; X < FullColumns; ++X)
			{
				if constexpr (PIXEL_SIZE == 4)
				{
					// Use memcpy to avoid any possible but improbable UB. memcpy should be optimized away by the compiler.
					uint64 Row0Bits; 
					uint64 Row1Bits;

					FMemory::Memcpy(&Row0Bits, SourceRow0, sizeof(uint64));
					FMemory::Memcpy(&Row1Bits, SourceRow1, sizeof(uint64));
					
					// Average of 2 unsigned integers with rounding without overflow extended to work on multiple bytes.
					constexpr uint64 ShiftMask = 0x7F7F7F7F7F7F7F7F;
			
					const uint64 AvgLowBits = (Row0Bits | Row1Bits) - (((Row0Bits ^ Row1Bits) >> 1) & ShiftMask);
					const uint64 AvgHighBits = AvgLowBits >> 32;
					const uint32 Result = (AvgLowBits | AvgHighBits) - (((AvgLowBits ^ AvgHighBits) >> 1) & ShiftMask);

					FMemory::Memcpy(DestRow, &Result, sizeof(uint32));
				}
				else
				{
					for (int32 C = 0; C < PIXEL_SIZE; ++C)
					{
						int32 PixelSum = SourceRow0[C] + SourceRow0[PIXEL_SIZE + C] + SourceRow1[C] + SourceRow1[PIXEL_SIZE + C] + 2;
						DestRow[C] = (uint8)(PixelSum >> 2);
					}
				}

				SourceRow0 += 2*PIXEL_SIZE;
				SourceRow1 += 2*PIXEL_SIZE;
				DestRow += PIXEL_SIZE;
			}

			if (bStrayColumn)
			{
				if constexpr (PIXEL_SIZE == 4)
				{	
					uint32 Row0Bits; 
					uint32 Row1Bits;

					FMemory::Memcpy(&Row0Bits, SourceRow0, sizeof(uint32));
					FMemory::Memcpy(&Row1Bits, SourceRow1, sizeof(uint32));

					// Average of 2 unsigned integers with rounding without overflow extended to work on multiple bytes.
					constexpr uint32 ShiftMask = 0x7F7F7F7F;
					const uint32 Result = (Row0Bits | Row1Bits) - (((Row0Bits ^ Row1Bits) >> 1) & ShiftMask);

					FMemory::Memcpy(DestRow, &Result, sizeof(uint32));
				}
				else
				{
					for (int32 C = 0; C < PIXEL_SIZE; ++C)
					{
						int32 PixelSum = SourceRow0[C] + SourceRow1[C] + 1;
						DestRow[C] = (uint8)(PixelSum >> 1);
					}
				}
			}
		};

		constexpr int32 PixelConcurrencyThreshold = 0xffff;
		if (DestSize[0] * DestSize[1] < PixelConcurrencyThreshold)
		{
			for (int32 Y = 0; Y < FullRows; ++Y)
			{
				ProcessRow(Y);
			}
		}
		else
		{
			ParallelFor(FullRows, ProcessRow);
		}

		if (bStrayRow)
		{
			const uint8* SourceRow0 = SourceData + 2 * FullRows * SourceStride;
			const uint8* SourceRow1 = SourceRow0 + SourceStride;
			uint8* DestRow = DestData + FullRows * DestStride;

			for (int32 X = 0; X < FullColumns; ++X)
			{
				if constexpr (PIXEL_SIZE == 4)
				{
					uint32 Col0Bits; 
					uint32 Col1Bits;

					FMemory::Memcpy(&Col0Bits, SourceRow0, sizeof(uint32));
					FMemory::Memcpy(&Col1Bits, SourceRow0 + 4, sizeof(uint32));

					// Average of 2 unsigned integers with rounding without overflow extended to work on multiple bytes.
					constexpr uint32 ShiftMask = 0x7F7F7F7F;
					const uint32 Result = (Col0Bits | Col1Bits) - (((Col0Bits ^ Col1Bits) >> 1) & ShiftMask);

					FMemory::Memcpy(DestRow, &Result, sizeof(uint32));
				}
				else
				{
					for (int32 C = 0; C < PIXEL_SIZE; ++C)
					{
						int32 P = SourceRow0[C] + SourceRow0[PIXEL_SIZE + C] + 1;
						DestRow[C] = (uint8)(P >> 1);
					}
				}

				SourceRow0 += 2*PIXEL_SIZE;
				DestRow += PIXEL_SIZE;
			}

			if (bStrayColumn)
			{
				if constexpr (PIXEL_SIZE == 4)
				{
					FMemory::Memcpy(DestRow, SourceRow0, 4);
				}
				else
				{
					for (int32 C = 0; C < PIXEL_SIZE; ++C)
					{
						DestRow[C] = SourceRow0[C];
					}
				}
			}
		}
	
	}

	// Generate next mip decompressed form block compressed image. 
	template<int32 NumChannels, EMipmapFilterType Filter>
	inline void GenerateNextMipBlockCompressed(
		const uint8* Src, uint8* Dest, FIntVector2 SrcSize, EImageFormat SrcFormat, EImageFormat DestFormat)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateNextMipBlockCompressed);

		const FImageFormatData& DestFormatData = GetImageFormatData(DestFormat);
		const FImageFormatData& SrcFormatData = GetImageFormatData(SrcFormat);

		check(NumChannels == DestFormatData.Channels);
		check(DestFormatData.PixelsPerBlockX == 1 && DestFormatData.PixelsPerBlockY == 1);

		const int32 DestChannelCount = DestFormatData.Channels; 
		const FIntVector2 PixelsPerBlock = FIntVector2(SrcFormatData.PixelsPerBlockX, SrcFormatData.PixelsPerBlockY);
		const int32 BlockSizeInBytes = SrcFormatData.BytesPerBlock;

		const FIntVector2 DestSize = FIntVector2(
				FMath::DivideAndRoundUp(SrcSize.X, 2),
				FMath::DivideAndRoundUp(SrcSize.Y, 2));

		const FIntVector2 NumBlocks = FIntVector2(
				FMath::DivideAndRoundUp(SrcSize.X, PixelsPerBlock.X),
				FMath::DivideAndRoundUp(SrcSize.Y, PixelsPerBlock.Y));

		constexpr int32 BatchSizeInBlocksX = 1 << 5;
		constexpr int32 BatchSizeInBlocksY = 1 << 4;

		FIntVector2 NumBatches = FIntVector2(
				FMath::DivideAndRoundUp(NumBlocks.X, BatchSizeInBlocksX),
				FMath::DivideAndRoundUp(NumBlocks.Y, BatchSizeInBlocksY));


		// Limit the parallel job num based on actual num workers. Here we cannot rely on ParallelFor
		// balancing the load as we need to allocate memory for every job. Make sure there is always 1 job.
		// TODO: Consider balancing work on using a 2D grid.
		const int32 MaxParallelJobs = FMath::Max(1, FMath::Min(int32(LowLevelTasks::FScheduler::Get().GetNumWorkers()), 8));
		
		constexpr int32 MinRowBatchesPerJob = 1;

		const int32 NumRowBatchesPerJob = 
			 FMath::Min(NumBatches.Y, FMath::Max(MinRowBatchesPerJob, FMath::DivideAndRoundUp(NumBatches.Y, MaxParallelJobs)));

		const int32 NumParallelJobs = FMath::DivideAndRoundUp(NumBatches.Y, NumRowBatchesPerJob); 

		// Use the tracking allocator policy on the image counter, this will not count for preventing memory peaks 
		// but will show if it happens. This allocation should be small enough so it is not a problem to get over-budget
		// by this amount. 
		TArray<uint8, FDefaultMemoryTrackingAllocator<MemoryCounters::FImageMemoryCounter>> StagingMemory;

		const miro::FImageSize StagingSize = miro::FImageSize(
				uint16(BatchSizeInBlocksX*PixelsPerBlock.X), 
				uint16(BatchSizeInBlocksY*PixelsPerBlock.Y));
		
		// Allocate extra memory so the mip computation can work on all possible pixels sizes. 
		// Also add some extra padding so different threads do not share cache lines.
		const int32 PerJobStagingBytes = StagingSize.X*StagingSize.Y*NumChannels + 8 + 64;
		
		StagingMemory.SetNum(PerJobStagingBytes*NumParallelJobs);
		uint8 * const StagingMemoryData = StagingMemory.GetData();

		miro::SubImageDecompression::FuncRefType DecompressionFunc = SubImageFormat::SelectDecompressionFunction(DestFormat, SrcFormat);

		ParallelExecutionUtils::InvokeBatchParallelFor(NumParallelJobs, 
		[
			NumParallelJobs, NumRowBatchesPerJob, 
			StagingMemoryData, PerJobStagingBytes, 
			NumBatches, NumBlocks, PixelsPerBlock, BlockSizeInBytes, DecompressionFunc,
			Src, SrcSize, Dest, DestSize
		](int32 JobId)
		{
			const int32 JobRowBegin = JobId*NumRowBatchesPerJob;
			const int32 JobRowEnd   = FMath::Min(JobRowBegin + NumRowBatchesPerJob, NumBatches.Y);
			uint8 * const JobStagingMemoryData = StagingMemoryData + JobId*PerJobStagingBytes;

			for (int32 BatchY = JobRowBegin; BatchY < JobRowEnd; ++BatchY)
			{
				for (int32 BatchX = 0; BatchX < NumBatches.X; ++BatchX)
				{
					const FIntVector2 BatchBeginInBlocks = FIntVector2(BatchX*BatchSizeInBlocksX, BatchY*BatchSizeInBlocksY);
					const FIntVector2 BatchEndInBlocks = FIntVector2(
							FMath::Min(BatchBeginInBlocks.X + BatchSizeInBlocksX, NumBlocks.X),
							FMath::Min(BatchBeginInBlocks.Y + BatchSizeInBlocksY, NumBlocks.Y));

					const uint8* const SrcBatchData = Src + (BatchBeginInBlocks.Y * NumBlocks.X + BatchBeginInBlocks.X)*BlockSizeInBytes;

					// Assume the decompressed size is always multiple of the block size. Trim unused bytes when copying to 
					// the final destination.
					const FIntVector2 BatchDecSizeInPixels = FIntVector2(
							(BatchEndInBlocks.X - BatchBeginInBlocks.X)*PixelsPerBlock.X,
							(BatchEndInBlocks.Y - BatchBeginInBlocks.Y)*PixelsPerBlock.Y);

					const miro::FImageSize FromSize = miro::FImageSize(uint16(SrcSize.X), uint16(SrcSize.Y)); 
					const miro::FImageSize SubSize  = miro::FImageSize(uint16(BatchDecSizeInPixels.X), uint16(BatchDecSizeInPixels.Y));
					DecompressionFunc(FromSize, SubSize, SubSize, SrcBatchData, JobStagingMemoryData);

					const FIntVector2 BatchOutBeginInPixels = FIntVector2(
							(BatchBeginInBlocks.X*PixelsPerBlock.X) >> 1, 
							(BatchBeginInBlocks.Y*PixelsPerBlock.Y) >> 1);

					const FIntVector2 BatchOutEndInPixels = FIntVector2(
							FMath::Min(BatchOutBeginInPixels.X + ((BatchSizeInBlocksX*PixelsPerBlock.X) >> 1), DestSize.X), 
							FMath::Min(BatchOutBeginInPixels.Y + ((BatchSizeInBlocksY*PixelsPerBlock.Y) >> 1), DestSize.Y));
		
					// Generate partial next mip to dest.
					// This works for all pixel sizes because we have preallocated more memory than needed.
					for (int32 Y = BatchOutBeginInPixels.Y; Y < BatchOutEndInPixels.Y; ++Y)
					{
						for (int32 X = BatchOutBeginInPixels.X; X < BatchOutEndInPixels.X; ++X)
						{
							uint8* const DestPixel = Dest + (Y*DestSize.X + X) * NumChannels;

							const FIntVector2 Row0Offset = FIntVector2(
									(X - BatchOutBeginInPixels.X) << 1, (Y - BatchOutBeginInPixels.Y) << 1);

							uint8 const * const SrcRow0 = JobStagingMemoryData + (Row0Offset.Y*BatchDecSizeInPixels.X + Row0Offset.X) * NumChannels;

							if constexpr (Filter == EMipmapFilterType::SimpleAverage)
							{
								// Use memcpy to avoid any possible but improbable UB. memcpy should be optimized away by the compiler.
								uint64 Row0Bits; 
								FMemory::Memcpy(&Row0Bits, SrcRow0, sizeof(uint64));
								
								uint8 const * const SrcRow1 = JobStagingMemoryData + 
										(FMath::Min(Row0Offset.Y + 1, BatchDecSizeInPixels.Y - 1)*BatchDecSizeInPixels.X + Row0Offset.X) * NumChannels;

								uint64 Row1Bits;
								FMemory::Memcpy(&Row1Bits, SrcRow1, sizeof(uint64));
								
								const bool bOutOfBounds = Row0Offset.X + 1 >= BatchDecSizeInPixels.X;
									
								// Average of 2 unsigned integers with rounding without overflow extended to work on multiple bytes.
								constexpr uint64 ShiftMask = 0x7F7F7F7F7F7F7F7F;
						
								const uint64 AvgLowBits = (Row0Bits | Row1Bits) - (((Row0Bits ^ Row1Bits) >> 1) & ShiftMask);
								const uint64 AvgHighBits = bOutOfBounds ? AvgLowBits : (AvgLowBits >> NumChannels*8);
								const uint32 Result = (AvgLowBits | AvgHighBits) - (((AvgLowBits ^ AvgHighBits) >> 1) & ShiftMask);

								FMemory::Memcpy(DestPixel, &Result, NumChannels);
							}
							else // constexpr Filter == EMipmapFilterType::Unfiltered
							{
								FMemory::Memcpy(DestPixel, SrcRow0, NumChannels);	
							}
							static_assert(
								Filter == EMipmapFilterType::SimpleAverage || 
								Filter == EMipmapFilterType::Unfiltered);
						}
					}
				}
			}
		});
	}
} // namespace OpImageMipmapInternal


	/** Generate the mipmaps for byte-based images of whatever number of channels.
	* \param mips number of additional levels to build from the source.
	*/
	template<int32 PIXEL_SIZE>
	inline void GenerateMipmapUint8LODRange(
		int32 SrcLOD, int32 DestLODBegin, int32 DestLODEnd,
		const FImage* SourceImage, FImage* DestImage,
		const FMipmapGenerationSettings& Settings)
	{
		using namespace OpImageMipmapInternal;

		FIntVector2 SourceSize = SourceImage->CalculateMipSize(SrcLOD);

		check(Invoke([&]() -> bool
		{
			FIntVector2 DestBeginLODSize = DestImage->CalculateMipSize(DestLODBegin);

			FIntVector2 SrcNextLODSize = FIntVector2(
					FMath::DivideAndRoundUp(SourceSize.X, 2),
					FMath::DivideAndRoundUp(SourceSize.Y, 2));

			return DestBeginLODSize == SrcNextLODSize;
		}));

		switch (Settings.FilterType)
		{
		case EMipmapFilterType::SimpleAverage:
		{
			const uint8* SrcData = SourceImage->GetLODData(SrcLOD);

			for (int32 L = DestLODBegin; L < DestLODEnd; ++L)
			{
				uint8* DestData = DestImage->GetLODData(L);
				GenerateNextMipmapUint8SimpleAverage<PIXEL_SIZE>(SrcData, DestData, SourceSize);
				
				SrcData = DestData;
				SourceSize = FIntVector2(
						FMath::DivideAndRoundUp(SourceSize.X, 2),
						FMath::DivideAndRoundUp(SourceSize.Y, 2));
			}
			break;
		}
		case EMipmapFilterType::Unfiltered:
		{
			const uint8* SrcData = SourceImage->GetLODData(SrcLOD);

			for (int32 L = DestLODBegin; L < DestLODEnd; ++L)
			{
				uint8* DestData = DestImage->GetLODData(L);
				GenerateNextMipmapUint8Unfiltered<PIXEL_SIZE>(SrcData, DestData, SourceSize);
				
				SrcData = DestData;
				SourceSize = FIntVector2(
						FMath::DivideAndRoundUp(SourceSize.X, 2),
						FMath::DivideAndRoundUp(SourceSize.Y, 2));
			}
			break;
		}
		default:
		{
			check(false);
			break;
		}
		}
	}

	/** 
	 * Generate the mipmaps for Block Comporessed images of whatever number of channels.
	 * The result is a non compressed image of the next mip with its tail.
	 * \param mips number of additional levels to build from the source.
	 */
	template<int32 PixelSize>
	inline void GenerateMipmapsBlockCompressedLODRange(
			int32 SrcLOD, int32 DestLODBegin, int32 DestLODEnd,
			const FImage* SourceImage, FImage* DestImage,
			const FMipmapGenerationSettings& Settings)
	{
		using namespace OpImageMipmapInternal;
	
		const FIntVector2 SourceSize = SourceImage->CalculateMipSize(SrcLOD);

		check(DestImage->GetLODCount() >= DestLODEnd);
		check(Invoke([&]() -> bool
		{
			FIntVector2 DestBeginLODSize = DestImage->CalculateMipSize(DestLODBegin);

			FIntVector2 SrcNextLODSize = FIntVector2(
					FMath::DivideAndRoundUp(SourceSize.X, 2),
					FMath::DivideAndRoundUp(SourceSize.Y, 2));

			return DestBeginLODSize == SrcNextLODSize;
		}));

		const EImageFormat SrcFormat = SourceImage->GetFormat();
		const EImageFormat DestFormat = DestImage->GetFormat();

		switch (Settings.FilterType)
		{
		case EMipmapFilterType::SimpleAverage:
		{
			const uint8* SrcData = SourceImage->GetLODData(SrcLOD);
			uint8* DestData = DestImage->GetLODData(DestLODBegin);

			GenerateNextMipBlockCompressed<PixelSize, EMipmapFilterType::SimpleAverage>(
					SrcData, DestData, SourceSize, SrcFormat, DestFormat);

			SrcData = DestData;
			FIntVector2 CurrentMipSize = FIntVector2(
					FMath::DivideAndRoundUp(SourceSize[0], 2),
					FMath::DivideAndRoundUp(SourceSize[1], 2));
		
			if (CurrentMipSize.X > 1 || CurrentMipSize.Y > 1)
			{
				for (int32 L = DestLODBegin + 1; L < DestLODEnd; ++L)
				{
					DestData = DestImage->GetLODData(L);

					GenerateNextMipmapUint8SimpleAverage<PixelSize>(SrcData, DestData, CurrentMipSize);

					SrcData = DestData;
					CurrentMipSize = FIntVector2(
							FMath::DivideAndRoundUp(CurrentMipSize.X, 2),
							FMath::DivideAndRoundUp(CurrentMipSize.Y, 2));
				}
			}

			break;
		}
		case EMipmapFilterType::Unfiltered:
		{
			const uint8* SrcData = SourceImage->GetLODData(SrcLOD);
			uint8* DestData = DestImage->GetLODData(DestLODBegin);
			
			GenerateNextMipBlockCompressed<PixelSize, EMipmapFilterType::Unfiltered>(
					SrcData, DestData, SourceSize, SrcFormat, DestFormat);

			SrcData = DestData;
			FIntVector2 CurrentMipSize = FIntVector2(
					FMath::DivideAndRoundUp(SourceSize.X, 2), 
					FMath::DivideAndRoundUp(SourceSize.Y, 2));

			if (CurrentMipSize.X > 1 || CurrentMipSize.Y > 1)
			{
				for (int32 L = DestLODBegin + 1; L < DestLODEnd; ++L)
				{
					DestData = DestImage->GetLODData(L);

					GenerateNextMipmapUint8Unfiltered<PixelSize>(SrcData, DestData, CurrentMipSize);

					SrcData = DestData;
					CurrentMipSize = FIntVector2(
							FMath::DivideAndRoundUp(CurrentMipSize.X, 2),
							FMath::DivideAndRoundUp(CurrentMipSize.Y, 2));
				}
			}
			break;
		}
		default:
		{
			check(false);
			break;
		}
		}
	}


    void FImageOperator::ImageMipmap_PrepareScratch(const FImage* DestImage, int32 StartLevel, int32 LevelCount, FScratchImageMipmap& Scratch )
    {
        check(DestImage->GetLODCount() == LevelCount);

		EImageFormat DestFormat = DestImage->GetFormat();
		if (UE::Mutable::Private::IsCompressedFormat(DestFormat))
		{
			// Is it a block format?
			if (UE::Mutable::Private::GetImageFormatData(DestFormat).PixelsPerBlockX > 1)
			{
				if (!bEnableCompressedMipGenerationMemoryOptimizations)
				{
					// Uncompress the last mip that we already have
					FIntVector2 UncompressedSize = DestImage->CalculateMipSize(StartLevel);
					Scratch.Uncompressed = CreateImage(
						(uint16)UncompressedSize[0], (uint16)UncompressedSize[1],
						1,
						EImageFormat::RGBA_UByte, EInitializationType::NotInitialized);
				}

				FIntVector2 UncompressedMipsSize = DestImage->CalculateMipSize(StartLevel + 1);
				// Generate the mipmaps from there on
				Scratch.UncompressedMips = CreateImage(
					(uint16)UncompressedMipsSize[0], (uint16)UncompressedMipsSize[1],
					FMath::Max(1, LevelCount - StartLevel - 1),
					EImageFormat::RGBA_UByte, EInitializationType::NotInitialized);

				// Compress the mipmapped image
			//	Scratch.CompressedMips = CreateImage(
			//		(uint16)UncompressedMipsSize[0], (uint16)UncompressedMipsSize[1],
			//		Scratch.UncompressedMips->GetLODCount(),
			//		DestImage->GetFormat(), EInitializationType::NotInitialized);
			}
			else
			{
				// It's probably an RLE compressed format

				// Uncompress the last mip that we already have
				FIntVector2 UncompressedSize = DestImage->CalculateMipSize(StartLevel);
				Scratch.Uncompressed = CreateImage(
					(uint16)UncompressedSize[0], (uint16)UncompressedSize[1],
					1,
					EImageFormat::L_UByte, EInitializationType::NotInitialized);


				FIntVector2 UncompressedMipsSize = DestImage->CalculateMipSize(StartLevel + 1);
				// Generate the mipmaps from there on
				Scratch.UncompressedMips = CreateImage(
					(uint16)UncompressedMipsSize[0], (uint16)UncompressedMipsSize[1],
					FMath::Max(1, LevelCount - StartLevel - 1),
					EImageFormat::L_UByte, EInitializationType::NotInitialized);


				// Compress the mipmapped image
			//	Scratch.CompressedMips = CreateImage(
			//		(uint16)UncompressedMipsSize[0], (uint16)UncompressedMipsSize[1],
			//		Scratch.UncompressedMips->GetLODCount(),
			//		DestImage->GetFormat(), EInitializationType::NotInitialized);

				// Preallocate ample memory for the compressed data
				int32 UncompressedNumMips = Scratch.UncompressedMips->GetLODCount();
				for (int32 L = 0; L < UncompressedNumMips; ++L)
				{	
					// It could happen with missing data.
					if (!Scratch.CompressedMips)
					{
						break;
					}

					const FIntVector2 LODSize = Scratch.Uncompressed->CalculateMipSize(L);
					Scratch.CompressedMips->DataStorage.ResizeLOD(L, LODSize.X*LODSize.Y);
				}
			}
		}
    }


	void FImageOperator::ImageMipmap(FScratchImageMipmap& Scratch, int32 CompressionQuality, FImage* DestImage, const FImage* BaseImage,
		int32 StartLevel, int32 NumLODs, const FMipmapGenerationSettings& Settings, bool bGenerateOnlyTail)
	{
		check(!(BaseImage->Flags & FImage::IF_CANNOT_BE_SCALED));
		check(DestImage->GetFormat() == BaseImage->GetFormat());
		
		if (!bGenerateOnlyTail)
		{
			check(DestImage->GetLODCount() == NumLODs);
			check(DestImage->GetSizeX() == BaseImage->GetSizeX());
			check(DestImage->GetSizeY() == BaseImage->GetSizeY());
		}
		else
		{
			check(DestImage->GetLODCount() + BaseImage->GetLODCount() == NumLODs);

			checkCode
			(
				const FIntVector2 BaseNextMipSize = BaseImage->CalculateMipSize(StartLevel + 1);
				check(BaseNextMipSize.X == DestImage->GetSizeX() && BaseNextMipSize.Y == DestImage->GetSizeY());
			);
		}

		if (!bGenerateOnlyTail && DestImage != BaseImage)
		{
			for (int32 L = 0; L <= StartLevel; ++L)
			{
				TArrayView<uint8> DestView = DestImage->DataStorage.GetLOD(L);
				TArrayView<const uint8> SrcView = BaseImage->DataStorage.GetLOD(L);

				check(DestView.Num() == SrcView.Num());
				FMemory::Memcpy(DestView.GetData(), SrcView.GetData(), DestView.Num());
			}
		}

		EImageFormat BaseFormat = BaseImage->GetFormat();
		const bool bIsBlockCompressedFormat = UE::Mutable::Private::IsBlockCompressedFormat(BaseFormat);
		const bool bIsCompressedFormat = UE::Mutable::Private::IsCompressedFormat(BaseFormat);
		
		check(!bIsBlockCompressedFormat || bIsCompressedFormat);

		if (bIsBlockCompressedFormat && bEnableCompressedMipGenerationMemoryOptimizations)
		{
			const EImageFormat DestFormat = Scratch.UncompressedMips->GetFormat();
			FImage* UncompressedImage = Scratch.UncompressedMips.Get();
	
			const int32 UncompressedNumLODs = UncompressedImage->GetLODCount();

			switch (DestFormat)
			{
			case EImageFormat::L_UByte:
			{
				constexpr int32 PixelSize = 1;
				GenerateMipmapsBlockCompressedLODRange<PixelSize>(
						StartLevel, 0, UncompressedNumLODs, BaseImage, UncompressedImage, Settings);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				constexpr int32 PixelSize = 3;
				GenerateMipmapsBlockCompressedLODRange<PixelSize>(
						StartLevel, 0, UncompressedNumLODs, BaseImage, UncompressedImage, Settings);
				break;
			}
			case EImageFormat::RGBA_UByte:
			{
				constexpr int32 PixelSize = 4;
				GenerateMipmapsBlockCompressedLODRange<PixelSize>(
						StartLevel, 0, UncompressedNumLODs, BaseImage, UncompressedImage, Settings);
				break;
			}
			default: check(false);
			}

			int32 DestLODBegin = bGenerateOnlyTail ? 0 : StartLevel + 1;

			bool bSuccess = false;
			ImagePixelFormat(
					bSuccess, CompressionQuality, DestImage, UncompressedImage, DestLODBegin, 0, UncompressedNumLODs); 
			check(bSuccess);	
		}
		else if (bIsCompressedFormat)
		{
			const int32 DestLODBegin = bGenerateOnlyTail ? 0 : StartLevel + 1;
			const int32 DestLODEnd = DestImage->GetLODCount();

			// Bad case.
			// Uncompress the last mip that we already have
			bool bSuccess = false;
			ImagePixelFormat(bSuccess, CompressionQuality, Scratch.Uncompressed.Get(), BaseImage, StartLevel);
			check(bSuccess);

			// Generate the mipmaps from there on
			constexpr bool bGenerateOnlyTailForCompressed = true;

			const int32 NumScratchMips = Scratch.UncompressedMips->GetLODCount() + Scratch.Uncompressed->GetLODCount();
			ImageMipmap(Scratch, CompressionQuality, Scratch.UncompressedMips.Get(),
					Scratch.Uncompressed.Get(), 0, NumScratchMips, Settings, bGenerateOnlyTailForCompressed);

			// Compress the mipmapped image
			bSuccess = false;
			ImagePixelFormat(
					bSuccess, CompressionQuality, DestImage, Scratch.UncompressedMips.Get(), 
					StartLevel + 1, 0, Scratch.UncompressedMips->GetLODCount()); 
			check(bSuccess);
		}
		else
		{	
			const int32 DestLODBegin = bGenerateOnlyTail ? 0 : StartLevel + 1;
			const int32 DestLODEnd = DestImage->GetLODCount();

			switch (BaseImage->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				GenerateMipmapUint8LODRange<1>(StartLevel, DestLODBegin, DestLODEnd, BaseImage, DestImage, Settings);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				GenerateMipmapUint8LODRange<3>(StartLevel, DestLODBegin, DestLODEnd, BaseImage, DestImage, Settings);
				break;
			}
			case EImageFormat::BGRA_UByte:
			case EImageFormat::RGBA_UByte:
			{
				GenerateMipmapUint8LODRange<4>(StartLevel, DestLODBegin, DestLODEnd, BaseImage, DestImage, Settings);
				break;
			}
			default:
				checkf(false, TEXT("Format not implemented in mipmap generation."));
			}
		}
	}

	void FImageOperator::ImageMipmap(int32 CompressionQuality, FImage* Dest, const FImage* Base,
		int32 StartLevel, int32 LevelCount, const FMipmapGenerationSettings& Settings, bool bGenerateOnlyTail)
	{
		FScratchImageMipmap Scratch;

		ImageMipmap_PrepareScratch(Dest, StartLevel, LevelCount, Scratch);
		ImageMipmap(Scratch, CompressionQuality, Dest, Base, StartLevel, LevelCount, Settings, bGenerateOnlyTail);
	}

	/** 
	 * Update all the mipmaps in the image from the data in the base one. 
	 * Only the mipmaps already existing in the image are updated.
	 */
	void ImageMipmapInPlace(int32 InImageCompressionQuality, FImage* InBase, const FMipmapGenerationSettings& InSettings)
	{
		check(!(InBase->Flags & FImage::IF_CANNOT_BE_SCALED));

		int32 LevelCount = InBase->GetLODCount();
		
		if (LevelCount - 1 <= 0)
		{
			return;
		}

		switch (InBase->GetFormat())
		{
		case EImageFormat::L_UByte:
			GenerateMipmapUint8LODRange<1>(0, 1, LevelCount, InBase, InBase, InSettings);
			break;

		case EImageFormat::RGB_UByte:
			GenerateMipmapUint8LODRange<3>(0, 1, LevelCount, InBase, InBase, InSettings);
			break;

		case EImageFormat::BGRA_UByte:
		case EImageFormat::RGBA_UByte:
			GenerateMipmapUint8LODRange<4>(0, 1, LevelCount, InBase, InBase, InSettings);
			break;

		default:
			checkf(false, TEXT("Format not implemented in mipmap generation."));
		}
	}

}
