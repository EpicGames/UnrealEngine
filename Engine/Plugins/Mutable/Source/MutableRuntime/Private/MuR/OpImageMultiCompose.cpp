// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpImageMultiCompose.h"

#include "MuR/ImagePrivate.h"
#include "MuR/MutableMath.h"
#include "MuR/ImageFormatUtils.h"
#include "MuR/ImageResizeUtils.h"
#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{

namespace OpImageMultiComposeInternal
{
	enum class EActiveStages : uint32
	{
		None             = 0 << 0,
		SrcDecompression = 1 << 0,
		DstDecompression = 1 << 1,
		Resize           = 1 << 2,
		DstCompression   = 1 << 3,
	};
	ENUM_CLASS_FLAGS(EActiveStages);

	struct FImageFormatInfo
	{
		const FImageFormatData& Dst;
		const FImageFormatData& Src;
		const FImageFormatData& DstDecompressed;

		EImageFormat SrcDecompressedFormat;
		EImageFormat DstDecompressedFormat;
		EImageFormat MostGenericFormat;

		miro::SubImageDecompression::FuncRefType SrcDecompressionFunc;
		miro::SubImageDecompression::FuncRefType DstDecompressionFunc;
		miro::SubImageCompression::FuncRefType DstCompressionFunc;
		SubImageResize::SubImageResizeFuncType* ResizeFunc;
		SubImageFormat::UncompressedFormatFuncType* SrcUncompressedFormatFunc;
		SubImageFormat::UncompressedFormatFuncType* DstUncompressedFormatFunc;

		bool bIsUncompressedReformat;
	};

	template<typename T>
	struct TSubImage
	{
		TArrayView<T> DataView;

		FImageSize Size          = FImageSize{0, 0};
		FImageSize SubSize       = FImageSize{0, 0};
		uint16 BytesPerBlock     = 0;
		uint8  NumElemsPerBlockX = 0;
		uint8  NumElemsPerBlockY = 0;

		TSubImage<const T> ToConst()
		{
			return TSubImage<const T> 
			{
				.DataView          = DataView         ,

				.Size              = Size             ,
				.SubSize           = SubSize          ,
				.BytesPerBlock     = BytesPerBlock    ,
				.NumElemsPerBlockX = NumElemsPerBlockX,
				.NumElemsPerBlockY = NumElemsPerBlockY,
			};
		}
	};

	template<typename T>
	using TConstSubImage = TSubImage<const T>;

	struct FScratchMemoryAllocation
	{
		static constexpr int32 MaxAllowedJobs = 8; 

		FIntVector2 BatchSizeInBlocks;
		int32 MaxParallelJobs; 

		EActiveStages ActiveStages = EActiveStages::None; 
		
		TArray<uint8, FDefaultMemoryTrackingAllocator<MemoryCounters::FImageMemoryCounter>> Buffer;

		TArray<TArrayView<uint8>, TInlineAllocator<MaxAllowedJobs>> SrcDecompressedViews;
		TArray<TArrayView<uint8>, TInlineAllocator<MaxAllowedJobs>> DstDecompressedViews;		
	};

	FImageFormatInfo GatherImageFormatInfo(const FImage& Dst, const FImage& Src)
	{
		using namespace SubImageFormat;

		EImageFormat SrcDecompressedFormat = GetUncompressedFormat(Src.GetFormat());
		EImageFormat DstDecompressedFormat = GetUncompressedFormat(Dst.GetFormat());
		EImageFormat MostGenericFormat     = GetMostGenericFormat(SrcDecompressedFormat, DstDecompressedFormat);

		const bool bIsUncompressedReformat = 
				SrcDecompressedFormat != DstDecompressedFormat && 
				SrcDecompressedFormat == Src.GetFormat()       &&  
				DstDecompressedFormat == Dst.GetFormat();

		return FImageFormatInfo
		{
			.Dst                       = GetImageFormatData(Dst.GetFormat()),
			.Src                       = GetImageFormatData(Src.GetFormat()),
			.DstDecompressed           = GetImageFormatData(DstDecompressedFormat),
			.SrcDecompressedFormat     = SrcDecompressedFormat,
			.DstDecompressedFormat     = DstDecompressedFormat,
			.MostGenericFormat         = MostGenericFormat,
			.SrcDecompressionFunc      = SelectDecompressionFunction(DstDecompressedFormat, Src.GetFormat()),
			.DstDecompressionFunc      = SelectDecompressionFunction(DstDecompressedFormat, Dst.GetFormat()),
			.DstCompressionFunc        = SelectCompressionFunction(Dst.GetFormat(), DstDecompressedFormat),
			.ResizeFunc                = SubImageResize::SelectSubImageResizeFunc(DstDecompressedFormat),
			.SrcUncompressedFormatFunc = SubImageFormat::SelectUncompressedFormatFunction(DstDecompressedFormat, SrcDecompressedFormat),
			.DstUncompressedFormatFunc = SubImageFormat::SelectUncompressedFormatFunction(MostGenericFormat, DstDecompressedFormat),
			.bIsUncompressedReformat   = bIsUncompressedReformat,
		};
	}

	inline int32 AlignDown(int32 Value, int32 Align)
	{
		return FMath::DivideAndRoundDown(Value, Align) * Align;
	}

	inline int32 AlignUp(int32 Value, int32 Align)
	{
		return FMath::DivideAndRoundUp(Value, Align) * Align;
	}

	inline bool IsLogicallyAligned(int32 Value, int32 Align, int32 Size)
	{
		return (Value % Align == 0) || Value == Size;
	}

	EActiveStages ComputeActiveStages(const FImage& Base, const FImage& Source, FIntVector2 SrcLODSize, const FImageFormatInfo& FormatInfo, const FIntRect& DstRect, const FIntRect& SrcRect)
	{
		const FImageSize BaseSize = Base.GetSize();

		const bool bIsSrcCompressionRectAligned =
				IsLogicallyAligned(SrcRect.Min.X, FormatInfo.Src.PixelsPerBlockX, SrcLODSize.X) &&
				IsLogicallyAligned(SrcRect.Min.Y, FormatInfo.Src.PixelsPerBlockY, SrcLODSize.Y) &&
				IsLogicallyAligned(SrcRect.Max.X, FormatInfo.Src.PixelsPerBlockX, SrcLODSize.X) &&
				IsLogicallyAligned(SrcRect.Max.Y, FormatInfo.Src.PixelsPerBlockY, SrcLODSize.Y);

		const bool bIsDstCompressionRectAligned =
				IsLogicallyAligned(DstRect.Min.X, FormatInfo.Dst.PixelsPerBlockX, BaseSize.X) &&
				IsLogicallyAligned(DstRect.Min.Y, FormatInfo.Dst.PixelsPerBlockY, BaseSize.Y) &&
				IsLogicallyAligned(DstRect.Max.X, FormatInfo.Dst.PixelsPerBlockX, BaseSize.X) &&
				IsLogicallyAligned(DstRect.Max.Y, FormatInfo.Dst.PixelsPerBlockY, BaseSize.Y);

		const bool bUncompressedFormatMissmatch =
				FormatInfo.SrcDecompressedFormat != FormatInfo.DstDecompressedFormat &&
				FormatInfo.SrcDecompressedFormat == Source.GetFormat() && 
				FormatInfo.DstDecompressedFormat == Base.GetFormat();

		const bool bCompressedFormatMismatch = 
				Source.GetFormat() != Base.GetFormat()                 && 
				Source.GetFormat() != FormatInfo.SrcDecompressedFormat &&
				Base.GetFormat() != FormatInfo.DstDecompressedFormat;

		const bool bNeedsResize = 
				(DstRect.Width()  != SrcRect.Width()) ||
				(DstRect.Height() != SrcRect.Height());

		const bool bDirectDecompression = (!bNeedsResize)                              &&
				(FormatInfo.SrcDecompressedFormat == FormatInfo.DstDecompressedFormat) &&
				(FormatInfo.DstDecompressedFormat == Base.GetFormat())                 &&
				(FormatInfo.SrcDecompressedFormat != Source.GetFormat());

		const bool bNeedsSrcDecompression = 
				(bNeedsResize && (FormatInfo.SrcDecompressedFormat != Source.GetFormat())) ||
				(!(bIsSrcCompressionRectAligned && bIsDstCompressionRectAligned))          ||
				(bUncompressedFormatMissmatch)                                             ||
				(bDirectDecompression)                                                     ||
				(bCompressedFormatMismatch);

		const bool bNeedsDstDecompression =
				(bNeedsResize && (FormatInfo.DstDecompressedFormat != Base.GetFormat())) ||
				(!bNeedsResize && bCompressedFormatMismatch)                             ||
				(!(bIsSrcCompressionRectAligned && bIsDstCompressionRectAligned));

		const bool bNeedsDstCompression =
				(FormatInfo.DstDecompressedFormat != Base.GetFormat()) && bNeedsSrcDecompression;
		
		EActiveStages Result = EActiveStages::None;
		EnumAddFlags(Result, bNeedsResize           ? EActiveStages::Resize           : EActiveStages::None);
		EnumAddFlags(Result, bNeedsSrcDecompression ? EActiveStages::SrcDecompression : EActiveStages::None);
		EnumAddFlags(Result, bNeedsDstCompression   ? EActiveStages::DstCompression   : EActiveStages::None);
		EnumAddFlags(Result, bNeedsDstDecompression ? EActiveStages::DstDecompression : EActiveStages::None);

		return Result;
	}

	FScratchMemoryAllocation AllocateScratchMemory(const FImage& Dst, const FImage& Src, TConstArrayView<FIntRect> DstRects, TConstArrayView<FIntRect> SrcRects, const FImageFormatInfo& FormatInfo)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose_AllocateScratchMemory);
	
		// This values can be tuned to control the maximum amount of scratch memory it will be used.
		// Smaller batches may get better data locality but extra source block decompression from the 
		// batch edge will be done. The current values seem to give good results with the tested cases.
		constexpr int32 MaxNumBlocksPerBatchX = 1 << 4; 
		constexpr int32 MaxNumBlocksPerBatchY = 1 << 3; 

		FScratchMemoryAllocation Result;
		Result.BatchSizeInBlocks = FIntVector2(MaxNumBlocksPerBatchX, MaxNumBlocksPerBatchY);
	
		FIntVector2 BatchSizeInPixels = Result.BatchSizeInBlocks * FIntVector2(FormatInfo.Dst.PixelsPerBlockX, FormatInfo.Dst.PixelsPerBlockY);

		uint32 DstDecompressedAllocSize = BatchSizeInPixels.X * BatchSizeInPixels.Y * FormatInfo.DstDecompressed.BytesPerBlock;

		uint32 SrcDecompressedAllocSize = 0; 
		for (int32 RectIndex = 0, NumRects = DstRects.Num(); RectIndex < NumRects; ++RectIndex)
		{
			FIntRect DstRect = DstRects[RectIndex];
			FIntRect SrcRect = SrcRects[RectIndex];
	
			if (DstRect.Area() == 0)
			{
				continue;
			}

			// Resize Dst Batch to get the Src 
			FIntVector2 SrcDecBatchSize = BatchSizeInPixels * FIntVector2(SrcRect.Width(), SrcRect.Height());

			SrcDecBatchSize = FIntVector2(
					FMath::DivideAndRoundUp(SrcDecBatchSize.X, DstRect.Width()), 
					FMath::DivideAndRoundUp(SrcDecBatchSize.Y, DstRect.Height()));

			// Add 1 compression blocks per side to account for alignment. 
			SrcDecBatchSize += FIntVector2(FormatInfo.Src.PixelsPerBlockX*2, FormatInfo.Src.PixelsPerBlockY*2);
			
			uint32 SrcDecBatchBytes = SrcDecBatchSize.X * SrcDecBatchSize.Y * FormatInfo.DstDecompressed.BytesPerBlock;

			SrcDecompressedAllocSize = FMath::Max(SrcDecBatchBytes, SrcDecompressedAllocSize);
		}
		
		const int32 NumWorkers = LowLevelTasks::FScheduler::Get().GetNumWorkers();
		Result.MaxParallelJobs = FMath::Clamp(NumWorkers - 1, 1, FScratchMemoryAllocation::MaxAllowedJobs);  

		// Add a cache line of padding per job to reduce possible false sharing between parallel batches.
		uint32 AllocationSizeInBytes = (SrcDecompressedAllocSize + DstDecompressedAllocSize + PLATFORM_CACHE_LINE_SIZE) * Result.MaxParallelJobs;

		Result.Buffer.SetNumUninitialized(AllocationSizeInBytes);

		for (int32 JobId = 0; JobId < Result.MaxParallelJobs; ++JobId)
		{
			uint32 JobViewOffset = (SrcDecompressedAllocSize + DstDecompressedAllocSize + PLATFORM_CACHE_LINE_SIZE) * JobId;
			Result.SrcDecompressedViews.Emplace(Result.Buffer.GetData() + JobViewOffset, SrcDecompressedAllocSize);
			Result.DstDecompressedViews.Emplace(Result.Buffer.GetData() + JobViewOffset + SrcDecompressedAllocSize, DstDecompressedAllocSize);
		}

		// TODO: We are allocating for all possible stages but should only allocate for the used ones. 
		Result.ActiveStages = EActiveStages::None;
		EnumAddFlags(Result.ActiveStages, EActiveStages::Resize | EActiveStages::SrcDecompression | EActiveStages::DstCompression | EActiveStages::DstDecompression);

		return Result;
	}

	void CopySubImage(const TSubImage<uint8>& Dst, const TConstSubImage<uint8>& Src)
	{
		check(Dst.SubSize == Src.SubSize);
		check(Dst.BytesPerBlock == Src.BytesPerBlock);
		check(Dst.NumElemsPerBlockX == Src.NumElemsPerBlockX);
		check(Dst.NumElemsPerBlockY == Src.NumElemsPerBlockY);

		FIntVector2 DstSizeInBlocks = FIntVector2(
				FMath::DivideAndRoundUp<int32>(Dst.Size.X, Dst.NumElemsPerBlockX),
				FMath::DivideAndRoundUp<int32>(Dst.Size.Y, Dst.NumElemsPerBlockY));

		FIntVector2 SrcSizeInBlocks = FIntVector2(
				FMath::DivideAndRoundUp<int32>(Src.Size.X, Src.NumElemsPerBlockX),
				FMath::DivideAndRoundUp<int32>(Src.Size.Y, Src.NumElemsPerBlockY));

		FIntVector2 SubSizeInBlocks = FIntVector2(
				Dst.SubSize.X / Dst.NumElemsPerBlockX, Dst.SubSize.Y / Dst.NumElemsPerBlockY);

		const int32 DstPitchInBytes = DstSizeInBlocks.X*Dst.BytesPerBlock;
		const int32 SrcPitchInBytes = SrcSizeInBlocks.X*Src.BytesPerBlock;
		const int32 RowSizeInBytes = SubSizeInBlocks.X*Dst.BytesPerBlock;

		for (int32 Y = 0; Y < SubSizeInBlocks.Y; ++Y)
		{
			FMemory::Memcpy(Dst.DataView.GetData() + Y*DstPitchInBytes, Src.DataView.GetData() + Y*SrcPitchInBytes, RowSizeInBytes);
		}
	}

	FIntVector2 ComputeNumBatches(const FIntRect& DstRect, const FImageFormatInfo& FormatInfo, const FScratchMemoryAllocation& ScratchMemory)
	{ 
		FIntVector2 BatchSizeInBlocks = ScratchMemory.BatchSizeInBlocks;
		FIntVector2 DstBlockSize      = FIntVector2(FormatInfo.Dst.PixelsPerBlockX, FormatInfo.Dst.PixelsPerBlockY);

		FIntRect AlignedRect = FIntRect(
				AlignDown(DstRect.Min.X, DstBlockSize.X),
				AlignDown(DstRect.Min.Y, DstBlockSize.Y),
				AlignUp  (DstRect.Max.X, DstBlockSize.X),
				AlignUp  (DstRect.Max.Y, DstBlockSize.Y));

		FIntVector2 RectSizeInBlocks = FIntVector2(
				FMath::DivideAndRoundUp(AlignedRect.Width(),  DstBlockSize.X),
				FMath::DivideAndRoundUp(AlignedRect.Height(), DstBlockSize.Y));

		FIntVector2 NumBatches = FIntVector2(
				FMath::DivideAndRoundUp(RectSizeInBlocks.X, BatchSizeInBlocks.X),
				FMath::DivideAndRoundUp(RectSizeInBlocks.Y, BatchSizeInBlocks.Y));

		return NumBatches;
	}

	struct FBatchWorkingSubImages
	{
		TConstSubImage<uint8> Src;
		TSubImage<uint8> Dst;
		TSubImage<uint8> DecSrc;
		TSubImage<uint8> DecDst;
		TSubImage<uint8> DecSrcAligned;
		TSubImage<uint8> DecDstAligned;

		FImageSize DstCompressionSubSize = FImageSize{0, 0};

		FVector2f SrcSubPixelOffset = FVector2f(0, 0);

		EActiveStages ActiveStages;
	};

	FBatchWorkingSubImages ComputeWorkingSubImages(
			int32 JobId,
			FIntVector2 BatchId, 
			FIntRect DstRect, FIntRect SrcRect, 
			const TSubImage<uint8>& DstImage, const TConstSubImage<uint8>& SrcImage, 
			const FImageFormatInfo& FormatInfo, 
			const FScratchMemoryAllocation& ScratchMemory,
			EActiveStages ActiveStages)
	{
		FBatchWorkingSubImages Result;
		Result.ActiveStages =  ActiveStages;

		FIntVector2 BatchSizeInPixels = ScratchMemory.BatchSizeInBlocks * 
				FIntVector2(FormatInfo.Dst.PixelsPerBlockX, FormatInfo.Dst.PixelsPerBlockY);
	
		FIntRect AlignedDstRect = FIntRect(
				AlignDown(DstRect.Min.X, DstImage.NumElemsPerBlockX),
				AlignDown(DstRect.Min.Y, DstImage.NumElemsPerBlockY),
				AlignUp  (DstRect.Max.X, DstImage.NumElemsPerBlockX),
				AlignUp  (DstRect.Max.Y, DstImage.NumElemsPerBlockY));	

		// Dst batches are always aligned to the compression blocks.
		FIntRect DstBatchRect = FIntRect(
				BatchId.X*BatchSizeInPixels.X + AlignedDstRect.Min.X,
				BatchId.Y*BatchSizeInPixels.Y + AlignedDstRect.Min.Y,
				BatchId.X*BatchSizeInPixels.X + BatchSizeInPixels.X + AlignedDstRect.Min.X,
				BatchId.Y*BatchSizeInPixels.Y + BatchSizeInPixels.Y + AlignedDstRect.Min.Y);
		
		// Clip the batch rect to the original dst layout rect to get the actual region to process.
		FIntRect DstClippedBatchRect = DstBatchRect;
		DstClippedBatchRect.Clip(DstRect);

		// If all the batch area will be modified there is no need to decompress the dst image for this batch.
		// Remove the stage from this batch ActiveStages.
		if (DstBatchRect == DstClippedBatchRect)
		{
			EnumRemoveFlags(Result.ActiveStages, EActiveStages::DstDecompression);
		}

		// Compute the clipped dst rect coordinates respect the original layout rect to calculate the resizing
		// sizes and offsets.
		FBox2f DstRectForResize = FBox2f(
				FVector2f(DstClippedBatchRect.Min.X - DstRect.Min.X, DstClippedBatchRect.Min.Y - DstRect.Min.Y),
				FVector2f(DstClippedBatchRect.Max.X - DstRect.Min.X, DstClippedBatchRect.Max.Y - DstRect.Min.Y));

		FVector2f OneOverResizeFactor = FVector2f(SrcRect.Width(), SrcRect.Height()) / FVector2f(DstRect.Width(), DstRect.Height());
		FBox2f SrcRectResized = FBox2f(DstRectForResize.Min * OneOverResizeFactor, DstRectForResize.Max * OneOverResizeFactor);

		Result.SrcSubPixelOffset = FVector2f(FMath::Frac(SrcRectResized.Min.X), FMath::Frac(SrcRectResized.Min.Y));

		// Translate the resized batch to src image coordinates.
		FIntRect SrcBatchRect = FIntRect(
				FMath::FloorToInt32(SrcRectResized.Min.X), 
				FMath::FloorToInt32(SrcRectResized.Min.Y), 
				FMath::CeilToInt32(SrcRectResized.Max.X ), 
				FMath::CeilToInt32(SrcRectResized.Max.Y )) + SrcRect.Min; 

		FIntRect AlignedBatchSrcRect = FIntRect(
				AlignDown(SrcBatchRect.Min.X, SrcImage.NumElemsPerBlockX),
				AlignDown(SrcBatchRect.Min.Y, SrcImage.NumElemsPerBlockY),
				AlignUp  (SrcBatchRect.Max.X, SrcImage.NumElemsPerBlockX),
				AlignUp  (SrcBatchRect.Max.Y, SrcImage.NumElemsPerBlockY));

		FIntRect AlignedBatchDstRect = FIntRect(
				AlignDown(DstClippedBatchRect.Min.X, DstImage.NumElemsPerBlockX),
				AlignDown(DstClippedBatchRect.Min.Y, DstImage.NumElemsPerBlockY),
				AlignUp  (DstClippedBatchRect.Max.X, DstImage.NumElemsPerBlockX),
				AlignUp  (DstClippedBatchRect.Max.Y, DstImage.NumElemsPerBlockY));

		Result.DstCompressionSubSize = FImageSize(
				FMath::Min<int32>(AlignedBatchDstRect.Max.X, DstImage.Size.X) - AlignedBatchDstRect.Min.X,
				FMath::Min<int32>(AlignedBatchDstRect.Max.Y, DstImage.Size.Y) - AlignedBatchDstRect.Min.Y);

		int32 SrcDataOffsetInBlocks = (AlignedBatchSrcRect.Min.Y / SrcImage.NumElemsPerBlockY) * FMath::DivideAndRoundUp<int32>(SrcImage.Size.X, SrcImage.NumElemsPerBlockX) + (AlignedBatchSrcRect.Min.X / SrcImage.NumElemsPerBlockX);
		int32 SrcDataOffsetInBytes =  SrcDataOffsetInBlocks * SrcImage.BytesPerBlock;

		Result.Src = TConstSubImage<uint8> 
		{
			.DataView      = TConstArrayView<uint8>(SrcImage.DataView.GetData() + SrcDataOffsetInBytes, SrcImage.DataView.Num() - SrcDataOffsetInBytes),
			.Size          = SrcImage.Size,
			.SubSize       = FImageSize(AlignedBatchSrcRect.Width(), AlignedBatchSrcRect.Height()),
			.BytesPerBlock = SrcImage.BytesPerBlock,
			.NumElemsPerBlockX = SrcImage.NumElemsPerBlockX,
			.NumElemsPerBlockY = SrcImage.NumElemsPerBlockY,
		};

		int32 DstDataOffsetInBlocks = (AlignedBatchDstRect.Min.Y / DstImage.NumElemsPerBlockY) * FMath::DivideAndRoundUp<int32>(DstImage.Size.X, DstImage.NumElemsPerBlockX) + (AlignedBatchDstRect.Min.X / DstImage.NumElemsPerBlockX);
		int32 DstDataOffsetInBytes =  DstDataOffsetInBlocks * DstImage.BytesPerBlock;
		Result.Dst = TSubImage<uint8>
		{
			.DataView      = TArrayView(DstImage.DataView.GetData() + DstDataOffsetInBytes, DstImage.DataView.Num() - DstDataOffsetInBytes),
			.Size          = DstImage.Size,
			.SubSize       = FImageSize(AlignedBatchDstRect.Width(), AlignedBatchDstRect.Height()),
			.BytesPerBlock = DstImage.BytesPerBlock,
			.NumElemsPerBlockX = DstImage.NumElemsPerBlockX,
			.NumElemsPerBlockY = DstImage.NumElemsPerBlockY,
		};

		if (!ScratchMemory.SrcDecompressedViews[JobId].IsEmpty())
		{
			Result.DecSrcAligned = TSubImage<uint8>
			{
				.DataView      = TArrayView(ScratchMemory.SrcDecompressedViews[JobId]),
				.Size          = FImageSize(AlignedBatchSrcRect.Width(), AlignedBatchSrcRect.Height()),
				.SubSize       = FImageSize(AlignedBatchSrcRect.Width(), AlignedBatchSrcRect.Height()),
				.BytesPerBlock = FormatInfo.DstDecompressed.BytesPerBlock, 
				.NumElemsPerBlockX = 1,
				.NumElemsPerBlockY = 1,
			};
		}

		if (!ScratchMemory.DstDecompressedViews[JobId].IsEmpty())
		{
			Result.DecDstAligned = TSubImage<uint8>
			{
				.DataView      = TArrayView(ScratchMemory.DstDecompressedViews[JobId]),
				.Size          = FImageSize(AlignedBatchDstRect.Width(), AlignedBatchDstRect.Height()),
				.SubSize       = FImageSize(AlignedBatchDstRect.Width(), AlignedBatchDstRect.Height()),
				.BytesPerBlock = FormatInfo.DstDecompressed.BytesPerBlock, 
				.NumElemsPerBlockX = 1,
				.NumElemsPerBlockY = 1,
			};
		}

		if (!Result.DecDstAligned.DataView.IsEmpty())
		{
			FIntVector2 DstOffset = DstClippedBatchRect.Min - AlignedBatchDstRect.Min;
			int32 DecDstOffsetInBytes = (DstOffset.Y*Result.DecDstAligned.Size.X + DstOffset.X) * Result.DecDstAligned.BytesPerBlock;
			
			Result.DecDst = TSubImage<uint8>
			{
				
				.DataView      = TArrayView(Result.DecDstAligned.DataView.GetData() + DecDstOffsetInBytes, Result.DecDstAligned.DataView.Num() - DecDstOffsetInBytes),
				.Size          = Result.DecDstAligned.Size,
				.SubSize       = FImageSize(DstClippedBatchRect.Width(), DstClippedBatchRect.Height()),
				.BytesPerBlock = FormatInfo.DstDecompressed.BytesPerBlock,
				.NumElemsPerBlockX = 1,
				.NumElemsPerBlockY = 1,
			};
		}

		if (!Result.DecSrcAligned.DataView.IsEmpty())
		{
			FIntVector2 SrcOffset = SrcBatchRect.Min - AlignedBatchSrcRect.Min;
			int32 DecSrcOffsetInBytes = (SrcOffset.Y*Result.DecSrcAligned.Size.X + SrcOffset.X) * Result.DecSrcAligned.BytesPerBlock;

			Result.DecSrc = TSubImage<uint8>
			{
				.DataView      = TArrayView(Result.DecSrcAligned.DataView.GetData() + DecSrcOffsetInBytes, Result.DecSrcAligned.DataView.Num() - DecSrcOffsetInBytes),
				.Size          = Result.DecSrcAligned.Size,
				.SubSize       = FImageSize(SrcBatchRect.Width(), SrcBatchRect.Height()),
				.BytesPerBlock = FormatInfo.DstDecompressed.BytesPerBlock,
				.NumElemsPerBlockX = 1,
				.NumElemsPerBlockY = 1,
			};
		}

		return Result;
	}
} // namespace OpImageMultiComposeInternal

	void ImageMultiCompose(
			FImage& Base, 
			const FImage& Source, 
			TConstArrayView<FIntRect> DstRects, 
			TConstArrayView<FIntRect> SrcRects,
			TConstArrayView<int32> SourceMipPerRect,
			int32 CompressionQuality)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose);

		using namespace OpImageMultiComposeInternal;

		check(DstRects.Num() == SrcRects.Num());
		check(SourceMipPerRect.Num() == SrcRects.Num());

		if (!DstRects.Num())
		{
			return;
		}

		FImageFormatInfo FormatInfo = GatherImageFormatInfo(Base, Source);

		FScratchMemoryAllocation ScratchMemory = AllocateScratchMemory(Base, Source, DstRects, SrcRects, FormatInfo);

		for (int32 RectIndex = 0; RectIndex < SrcRects.Num(); ++RectIndex)
		{
			FIntRect DstRect = DstRects[RectIndex];
			FIntRect SrcRect = SrcRects[RectIndex];

			if (DstRect.Area() == 0 || SrcRect.Area() == 0)
			{
				continue;
			}

			uint32 SourceLODIndex = SourceMipPerRect[RectIndex];

			FIntVector2 SourceLODSize = Source.CalculateMipSize(SourceLODIndex); 

			EActiveStages ActiveStages = ComputeActiveStages(Base, Source, SourceLODSize, FormatInfo, DstRect, SrcRect);
			check(EnumHasAllFlags(ScratchMemory.ActiveStages, ActiveStages)); 

			TConstSubImage<uint8> SrcSubImage = TConstSubImage<uint8>
			{
				.DataView = Source.DataStorage.GetLOD(SourceLODIndex),
				.Size     = FImageSize(SourceLODSize.X, SourceLODSize.Y),
				.SubSize  = FImageSize(SourceLODSize.X, SourceLODSize.Y), 
				.BytesPerBlock     = FormatInfo.Src.BytesPerBlock,
				.NumElemsPerBlockX = FormatInfo.Src.PixelsPerBlockX,
				.NumElemsPerBlockY = FormatInfo.Src.PixelsPerBlockY,
			};

			TSubImage<uint8> BaseSubImage = TSubImage<uint8>
			{
				.DataView = Base.DataStorage.GetLOD(0),
				.Size     = Base.GetSize(),
				.SubSize  = Base.GetSize(),
				.BytesPerBlock     = FormatInfo.Dst.BytesPerBlock,
				.NumElemsPerBlockX = FormatInfo.Dst.PixelsPerBlockX,
				.NumElemsPerBlockY = FormatInfo.Dst.PixelsPerBlockY,
			};

			FIntVector2 NumBatches = ComputeNumBatches(DstRect, FormatInfo, ScratchMemory);

			// Split parallel work in rows, using a 2D grid approach could lead to better work distribution. 
			constexpr int32 MinRowBatchesPerJob = 1;
			const int32 NumRowBatchesPerJob = 
				 FMath::Min(NumBatches.Y, FMath::Max(MinRowBatchesPerJob, FMath::DivideAndRoundUp(NumBatches.Y, ScratchMemory.MaxParallelJobs)));

			const int32 NumParallelJobs = FMath::DivideAndRoundUp(NumBatches.Y, NumRowBatchesPerJob); 

			ParallelExecutionUtils::InvokeBatchParallelFor(NumParallelJobs, 
			[
				&ScratchMemory, &FormatInfo, 
				NumRowBatchesPerJob, NumBatches,
				BaseSubImage, SrcSubImage,
				ActiveStages,
				DstRect, SrcRect,
				CompressionQuality
			](int32 JobId)
			{
				const int32 JobRowBegin = JobId*NumRowBatchesPerJob;
				const int32 JobRowEnd   = FMath::Min(JobRowBegin + NumRowBatchesPerJob, NumBatches.Y);

				for (int32 BatchIdY = JobRowBegin; BatchIdY < JobRowEnd; ++BatchIdY)
				{
					for (int32 BatchIdX = 0; BatchIdX < NumBatches.X; ++BatchIdX)
					{
						FIntVector2 BatchId = FIntVector2(BatchIdX, BatchIdY);
						FBatchWorkingSubImages WorkingSubImages = ComputeWorkingSubImages(JobId, BatchId, DstRect, SrcRect, BaseSubImage, SrcSubImage, FormatInfo, ScratchMemory, ActiveStages);

						if (EnumHasAnyFlags(WorkingSubImages.ActiveStages, EActiveStages::DstDecompression))
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose_DstDecompression);

							const TSubImage<uint8>& StageSrc = WorkingSubImages.Dst;
							const TSubImage<uint8>& StageDst = WorkingSubImages.DecDstAligned;

							check(StageSrc.SubSize == StageDst.SubSize);

							const uint8* FromPtr = StageSrc.DataView.GetData();
							uint8* ToPtr         = StageDst.DataView.GetData();

							miro::FImageSize FromSize = miro::FImageSize(StageSrc.Size.X, StageSrc.Size.Y);
							miro::FImageSize ToSize   = miro::FImageSize(StageDst.Size.X, StageDst.Size.Y); 
							miro::FImageSize SubSize  = miro::FImageSize(StageSrc.SubSize.X, StageSrc.SubSize.Y);

							FormatInfo.DstDecompressionFunc(FromSize, ToSize, SubSize, FromPtr, ToPtr);	
						}

						bool bDstDataAlreadySet = false;
						if (EnumHasAnyFlags(WorkingSubImages.ActiveStages, EActiveStages::SrcDecompression))
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose_SrcDecompression);

							const TConstSubImage<uint8>& StageSrc = WorkingSubImages.Src;
							const TSubImage<uint8>& StageDst = EnumHasAnyFlags(ActiveStages, EActiveStages::DstCompression | EActiveStages::Resize)
									? WorkingSubImages.DecSrcAligned
									: WorkingSubImages.Dst;
				
							const uint8* FromPtr = StageSrc.DataView.GetData();
							uint8* ToPtr         = StageDst.DataView.GetData();

							if (FormatInfo.bIsUncompressedReformat)
							{
								FormatInfo.SrcUncompressedFormatFunc(StageSrc.Size, StageDst.Size, StageDst.SubSize, FromPtr, ToPtr); 
							}
							else
							{
								miro::FImageSize FromSize = miro::FImageSize(StageSrc.Size.X, StageSrc.Size.Y);
								miro::FImageSize ToSize   = miro::FImageSize(StageDst.Size.X, StageDst.Size.Y); 
								miro::FImageSize SubSize  = miro::FImageSize(StageDst.SubSize.X, StageDst.SubSize.Y);

								FormatInfo.SrcDecompressionFunc(FromSize, ToSize, SubSize, FromPtr, ToPtr);
							}

							bDstDataAlreadySet = !EnumHasAnyFlags(ActiveStages, EActiveStages::DstCompression | EActiveStages::Resize);
						}

						if (!bDstDataAlreadySet)
						{
							TConstSubImage<uint8> ConstDecSrc = WorkingSubImages.DecSrc.ToConst();

							const TConstSubImage<uint8>& StageSrc = EnumHasAnyFlags(ActiveStages, EActiveStages::SrcDecompression)
									? ConstDecSrc
									: WorkingSubImages.Src;

							const TSubImage<uint8>& StageDst = EnumHasAnyFlags(ActiveStages, EActiveStages::DstCompression)
									? WorkingSubImages.DecDst
									: WorkingSubImages.Dst;

							if (EnumHasAnyFlags(WorkingSubImages.ActiveStages, EActiveStages::Resize))
							{
								MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose_Resize);

								const uint8* FromPtr = StageSrc.DataView.GetData();
								uint8* ToPtr         = StageDst.DataView.GetData();

								FVector2f SrcRectDimsF = FVector2f(SrcRect.Width(), SrcRect.Height());
								FVector2f DstRectDimsF = FVector2f(DstRect.Width(), DstRect.Height());

								FVector2f OneOverScalingFactor = SrcRectDimsF / DstRectDimsF;

								FormatInfo.ResizeFunc(OneOverScalingFactor, WorkingSubImages.SrcSubPixelOffset, 
										StageDst.Size, StageDst.SubSize, StageSrc.Size, StageSrc.SubSize, ToPtr, FromPtr);	
							}
							else
							{
								MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose_Copy);
								CopySubImage(StageDst, StageSrc);
							}
						}

						if (EnumHasAnyFlags(WorkingSubImages.ActiveStages, EActiveStages::DstCompression))
						{
							MUTABLE_CPUPROFILER_SCOPE(ImageMultiCompose_DstCompression);

							const TSubImage<uint8>& StageSrc = WorkingSubImages.DecDstAligned;
							const TSubImage<uint8>& StageDst = WorkingSubImages.Dst;

							check(StageSrc.SubSize == StageDst.SubSize);

							const uint8* FromPtr = StageSrc.DataView.GetData();
							uint8* ToPtr         = StageDst.DataView.GetData();

							miro::FImageSize FromSize = miro::FImageSize(StageSrc.Size.X, StageSrc.Size.Y);
							miro::FImageSize ToSize   = miro::FImageSize(StageDst.Size.X, StageDst.Size.Y);
							miro::FImageSize SubSize  = miro::FImageSize(WorkingSubImages.DstCompressionSubSize.X, WorkingSubImages.DstCompressionSubSize.Y);

							FormatInfo.DstCompressionFunc(FromSize, ToSize, SubSize, FromPtr, ToPtr, CompressionQuality);
						}
					}
				}
			});

		}
	}

	float ImageMultiComposeComputeBestLODForSrcRect(const FIntRect& DstRectInPixels, const FIntRect& SrcRectInPixels)
	{
		FVector2f DstRectSize = FVector2f(DstRectInPixels.Width(), DstRectInPixels.Height());
		FVector2f SrcRectSize = FVector2f(SrcRectInPixels.Width(), SrcRectInPixels.Height());

		// Image multicompose only supports scaling down by less than 2 or scale up, for now pick the largest LOD 
		// calculated for both axis so we never get values outside the limitations.
		//
		// NOTE: Doing this we may loose resolution if both dimensions change very differently, using layout packing
		// strategies that maintain the original aspect ratio is advised to get the best results possible. 
		float BestLODForXAxis = FMath::Log2(FMath::Max(1.0f, SrcRectSize.X)) -  FMath::Log2(FMath::Max(1.0f, DstRectSize.X));
		float BestLODForYAxis = FMath::Log2(FMath::Max(1.0f, SrcRectSize.Y)) -  FMath::Log2(FMath::Max(1.0f, DstRectSize.Y));

		return FMath::Max(BestLODForXAxis, BestLODForYAxis); 
	}
} // namespace UE::Mutable::Private

