// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"
#include "Containers/ArrayView.h"

namespace UE::Mutable::Private
{
	class FImage;
	
	/**
	 * Composes an image using multiple rects from the same source image. 
	 * 
	 * @param Base                Base image where the composing Rects will be copied.
	 *
	 * @param Source              Source image from where the composing Rects data will be extracted. 
	 *
	 * @param DstRectsInPixels    Rects in pixels of regions of the Base image to be composed. 
	 *
	 * @param SrcRectsInPixels    Rects in pixels of regions of the Source image to compose, i-th SrcRectsInPixels 
	 * 						      region content will be copied to i-th DstRectsInPixels region. 
	 * 						      SrcRectsInPixels must have the same number of elements than DstRectsInPixels.
	 *
	 * @param SourceLODPerRect    Source LOD index where the i-th SrcRect is extracted from. Number of pixels in 
	 *                            SrcRect are expected to be based on the Source image's LOD index dimensions.
	 * 					          SourceLODPerRect must have the same number of elements than SrcRectsInPixels.
	 *
	 * @param CompressionQuality Quality level used by the block compression functions.
	 */
	void ImageMultiCompose(FImage& Base, const FImage& Source, TConstArrayView<FIntRect> DstRectsInPixels, TConstArrayView<FIntRect> SrcRectsInPixels, TConstArrayView<int32> SourceLODPerRect, int32 CompressionQuality);
	
	/**
	 * Computes the best LOD bias for an image going from SrcRectInPixels dimensions to DstRectInPixels dimensions. 
	 * 
	 * @param DstRectInPixels The destination rect used for getting the dimensions used for the computation.
	 *
	 * @param SrcRectInPixels The source rect used for getting the dimensions used for the computation.
	 *
	 * @return                LOD bias needed to apply to SrcRect to get the dimensions closest to DstRect. 
	 */
	float ImageMultiComposeComputeBestLODForSrcRect(const FIntRect& DstRectInPixels, const FIntRect& SrcRectInPixels);
}

