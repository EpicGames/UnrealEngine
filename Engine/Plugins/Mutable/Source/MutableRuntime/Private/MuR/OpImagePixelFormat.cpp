// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Image.h"

#include "Containers/Array.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/BlockCompression/Miro/Miro.h"
#include "MuR/ImageRLE.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/MutableRuntimeModule.h"
#include "MuR/Ptr.h"


namespace UE::Mutable::Private
{


TManagedPtr<FImage> FImageOperator::ImagePixelFormat(int32 CompressionQuality, const FImage* Base, EImageFormat TargetFormat, int32 OnlyLOD)
{
	MUTABLE_CPUPROFILER_SCOPE(ImagePixelFormat);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("From %d to %d"), int32(Base->GetFormat()), int32(TargetFormat)));

	FIntVector2 ResultSize;
	int32 ResultLODCount = 0;

	if (OnlyLOD == -1)
	{
		ResultSize[0] = Base->GetSizeX();
		ResultSize[1] = Base->GetSizeY();
		ResultLODCount = Base->GetLODCount();
	}
	else
	{
		ResultSize = Base->CalculateMipSize(OnlyLOD);
		ResultLODCount = 1;
	}

	TManagedPtr<FImage> Result = CreateImage(ResultSize[0], ResultSize[1], ResultLODCount, TargetFormat, EInitializationType::NotInitialized);
	Result->Flags = Base->Flags;

	if (Base->GetSizeX() <= 0 || Base->GetSizeY() <= 0)
	{
		return Result;
	}

	bool bSuccess = false;
	ImagePixelFormat(bSuccess, CompressionQuality, Result.Get(), Base, OnlyLOD);
	check(bSuccess);

	return Result;
}

namespace ImagePixelFormatInternal
{
	template<class PixelFormatFuncType>
	void ProcessBatchedHelper(
			FImage* Dest, const FImage* Src, 
			int32 DestElemSizeInBytes, int32 SrcElemSizeInBytes, 
			int32 DestLODBegin, int32 SrcLODBegin, int32 NumLODs, 
			PixelFormatFuncType&& BatchFunc)
	{
		check(Dest->GetLODCount() >= DestLODBegin + NumLODs);
		check(Src->GetLODCount() >= SrcLODBegin + NumLODs);
		
		const int32 DestLODEnd = DestLODBegin + NumLODs;
		const int32 SrcLODEnd = SrcLODBegin + NumLODs;

		constexpr int32 NumBatchElems = 1 << 15;

		const int32 NumBatches = Dest->DataStorage.GetNumBatchesLODRange(NumBatchElems, DestElemSizeInBytes, DestLODBegin, DestLODEnd);
		check(NumBatches == Src->DataStorage.GetNumBatchesLODRange(NumBatchElems, SrcElemSizeInBytes, SrcLODBegin, SrcLODEnd));

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			TArrayView<const uint8> SrcView = Src->DataStorage.GetBatchLODRange(BatchIndex, NumBatchElems, SrcElemSizeInBytes, SrcLODBegin, SrcLODEnd);
			TArrayView<uint8> DestView = Dest->DataStorage.GetBatchLODRange(BatchIndex, NumBatchElems, DestElemSizeInBytes, DestLODBegin, DestLODEnd);

			const int32 NumElems = DestView.Num() / DestElemSizeInBytes;
			check(DestView.Num() % DestElemSizeInBytes == 0);
			check(NumElems == SrcView.Num() / SrcElemSizeInBytes);

			BatchFunc(DestView.GetData(), SrcView.GetData(), NumElems);
		}
	}

	template<class PixelFormatFuncType>
	void ProcessBatchedHelper(
			FImage* Dest, int32 DestElemSizeInBytes, int32 DestLODBegin, int32 NumLODs, 
			PixelFormatFuncType&& BatchFunc)
	{
		check(Dest->GetLODCount() >= DestLODBegin + NumLODs);
		
		const int32 DestLODEnd = DestLODBegin + NumLODs;

		constexpr int32 NumBatchElems = 1 << 15;

		const int32 NumBatches = Dest->DataStorage.GetNumBatchesLODRange(NumBatchElems, DestElemSizeInBytes, DestLODBegin, DestLODEnd);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			TArrayView<uint8> DestView = Dest->DataStorage.GetBatchLODRange(BatchIndex, NumBatchElems, DestElemSizeInBytes, DestLODBegin, DestLODEnd);

			const int32 NumElems = DestView.Num() / DestElemSizeInBytes;
			check(DestView.Num() % DestElemSizeInBytes == 0);

			BatchFunc(DestView.GetData(), NumElems);
		}
	}

	template<class BlockDecompressionFuncType> 
	void BlockDecompressionLODRangeHelper(
		FImage* Result, const FImage* Base, 
		int32 ResultLODBegin, int32 BaseLODBegin, int32 NumLODs, 
		BlockDecompressionFuncType&& DecFunc)
	{
		check(Base->GetLODCount() >= BaseLODBegin + NumLODs);
		check(Result->GetLODCount() >= ResultLODBegin + NumLODs);
		for (int32 L = 0; L < NumLODs; ++L)
		{
			const int32 BaseLOD = BaseLODBegin + L;
			const int32 ResultLOD = ResultLODBegin + L;
			FIntVector2 MipSize = Result->CalculateMipSize(ResultLOD);
			
			check(Base->CalculateMipSize(BaseLOD) == MipSize);
			
			DecFunc(MipSize.X, MipSize.Y, Base->GetLODData(BaseLOD), Result->GetLODData(ResultLOD));
		}
	}

	template<class BlockCompressionFuncType> 
	void BlockCompressionLODRangeHelper(
		FImage* Result, const FImage* Base, 
		int32 ResultLODBegin, int32 BaseLODBegin, int32 NumLODs, int32 Quality,
		BlockCompressionFuncType&& CompFunc)
	{
		check(Base->GetLODCount() >= BaseLODBegin + NumLODs);
		check(Result->GetLODCount() >= ResultLODBegin + NumLODs);
		for (int32 L = 0; L < NumLODs; ++L)
		{
			const int32 BaseLOD = BaseLODBegin + L;
			const int32 ResultLOD = ResultLODBegin + L;
			FIntVector2 MipSize = Result->CalculateMipSize(ResultLOD);
		
			check(Base->CalculateMipSize(BaseLOD) == MipSize);

			CompFunc(MipSize.X, MipSize.Y, Base->GetLODData(BaseLOD), Result->GetLODData(ResultLOD), Quality);
		}
	}

}

void FImageOperator::ImagePixelFormat(bool& bOutSuccess, int32 CompressionQuality, FImage* Result, const FImage* Base, int32 OnlyLOD)
{
	if (FormatImageOverride)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImagePixelFormatOverride);
		FormatImageOverride(bOutSuccess, CompressionQuality, Result, Base, OnlyLOD);
		if (bOutSuccess)
		{
			return;
		}
	}
	
	bOutSuccess = true;
	
	const int32 BaseLODBegin = OnlyLOD != -1 ? OnlyLOD : 0;
	const int32 ResultLODBegin = 0;
	const int32 NumLODs = OnlyLOD != -1 ? 1 : Base->GetLODCount();

	ImagePixelFormat(bOutSuccess, CompressionQuality, Result, Base, ResultLODBegin, BaseLODBegin, NumLODs);
	check(bOutSuccess);
}

void FImageOperator::ImagePixelFormat(bool& bOutSuccess, int32 CompressionQuality, FImage* Result, const FImage* Base, 
		int32 ResultLODBegin, int32 BaseLODBegin, int32 NumLODs)
{
	MUTABLE_CPUPROFILER_SCOPE(ImagePixelFormatInPlace);

	check(Result->GetLODCount() >= ResultLODBegin + NumLODs);
	check(Base->GetLODCount() >= BaseLODBegin + NumLODs);
	check(Base->CalculateMipSize(BaseLODBegin) == Result->CalculateMipSize(ResultLODBegin));

	const bool bFormatAllLODs = Result->GetLODCount() == NumLODs && 
								BaseLODBegin == 0 && ResultLODBegin == 0;
	
	check(!bFormatAllLODs || Base->GetLODCount() >= NumLODs);

	bOutSuccess = true;

	const bool bIsPixelFormatNeeded = 
			Result->GetFormat()   == Base->GetFormat()   &&
			Result->GetLODCount() == Base->GetLODCount() &&
			Result->GetLODCount() == NumLODs;

	if (bIsPixelFormatNeeded)
	{
		// Shouldn't really happen
		Result->DataStorage = Base->DataStorage;
	}
	else
	{
		switch (Result->GetFormat())
		{
		case EImageFormat::L_UByte:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 1, 1, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					FMemory::Memcpy(DestBuf, BaseBuf, NumElems * 1);
				});

				break;
			}
			case EImageFormat::L_UByteRLE:
			{
				if (bFormatAllLODs)
				{
					UncompressRLE_L(Base, Result);
				}
				else
				{
					check(NumLODs == 1);
					FImageSize ResultSize = Result->GetSize();
					UncompressRLE_L(ResultSize.X, ResultSize.Y, Base->GetLODData(BaseLODBegin), Result->GetLODData(ResultLODBegin));
				}
				break;
			}
			case EImageFormat::L_UBitRLE:
			{
				if(bFormatAllLODs)
				{
					UncompressRLE_L1(Base, Result);
				}
				else
				{
					check(NumLODs == 1);
					FImageSize ResultSize = Result->GetSize();
					UncompressRLE_L1(ResultSize.X, ResultSize.Y, Base->GetLODData(BaseLODBegin), Result->GetLODData(ResultLODBegin));
				}
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 1, 3, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint32 ResultValue = 77 * BaseBuf[3*I + 0] + 150 * BaseBuf[3*I + 1] + 29 * BaseBuf[3*I + 2];
						DestBuf[I] = (uint8)FMath::Min(255u, ResultValue >> 8);
					}
				});

				break;
			}
			case EImageFormat::RGB_UByteRLE:
			{
				TManagedPtr<FImage> TempBase =
						ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGB_UByte);

				ImagePixelFormatInternal::ProcessBatchedHelper(Result, TempBase.Get(), 1, 3, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint32 ResultValue = 77 * BaseBuf[3*I + 0] + 150 * BaseBuf[3*I + 1] + 29 * BaseBuf[3*I + 2];
						DestBuf[I] = (uint8)FMath::Min(255u, ResultValue >> 8);
					}
				});


				break;
			}
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 1, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint32 ResultValue = 77 * BaseBuf[4*I + 0] + 150 * BaseBuf[4*I + 1] + 29 * BaseBuf[4*I + 2];
						DestBuf[I] = (uint8)FMath::Min(255u, ResultValue >> 8);
					}
				});

				break;
			}
			case EImageFormat::RGBA_UByteRLE:
			{
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, TempBase.Get(), 1, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint32 ResultValue = 77 * BaseBuf[4*I + 0] + 150 * BaseBuf[4*I + 1] + 29 * BaseBuf[4*I + 2];
						DestBuf[I] = uint8( ResultValue >> 8);
					}
				});


				break;
			}
			case EImageFormat::BGRA_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 1, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint32 ResultValue = 77 * BaseBuf[4*I + 2] + 150 * BaseBuf[4*I + 1] + 29 * BaseBuf[4*I + 0];
						DestBuf[I] = uint8(ResultValue >> 8);
					}
				});

				break;
			}
			case EImageFormat::BC1:
			case EImageFormat::BC2:
			case EImageFormat::BC3:
			{
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGB_UByte);
				
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, TempBase.Get(), 1, 3, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint32 ResultValue = 77 * BaseBuf[3*I + 0] + 150 * BaseBuf[3*I + 1] + 29 * BaseBuf[3*I + 2];
						DestBuf[I] = uint8(ResultValue >> 8);
					}
				});


				break;
			}
			case EImageFormat::BC4:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC4_to_L);
				break;
			}
			default:
				// Case not implemented
				check(false);
			}

			break;
		}

		case EImageFormat::L_UByteRLE:
		{
			switch (Base->GetFormat())
			{

			case EImageFormat::L_UByte:
			{
				// Try to compress
				if (Result->DataStorage.IsEmpty())
				{
					// Allocate memory for the compressed data. TODO: Smaller?	
					check(Base->DataStorage.Buffers.Num() == Result->DataStorage.Buffers.Num());

					for (int32 I = 0; I < Base->DataStorage.Buffers.Num(); ++I)
					{
						int32 BufferSize = Base->DataStorage.Buffers[I].Num();
						Result->DataStorage.Buffers[I].SetNumUninitialized(BufferSize);
					}
					Result->DataStorage.CompactedTailOffsets = Base->DataStorage.CompactedTailOffsets;
				}
				
				int32 SizeX = Base->GetSizeX();
				int32 SizeY = Base->GetSizeY();

				for (int32 L = 0; L < NumLODs; ++L)
				{
					bool bSuccess = false;
					while (!bSuccess)
					{
						uint32 CompressedSize = 0;
						CompressRLE_L(CompressedSize, SizeX, SizeY, Base->GetLODData(BaseLODBegin + L), Result->GetLODData(ResultLODBegin + L), Result->GetLODDataSize(ResultLODBegin + L));

						bSuccess = CompressedSize > 0;
						if (bSuccess)
						{
							Result->DataStorage.ResizeLOD(ResultLODBegin + L, CompressedSize);
						}
						else
						{
							const int32 MipMemorySize = Result->DataStorage.GetLOD(ResultLODBegin + L).Num();
							Result->DataStorage.ResizeLOD(ResultLODBegin + L, FMath::Max(4, MipMemorySize * 2));
						}
					}

					SizeX = FMath::DivideAndRoundUp(SizeX, 2);
					SizeY = FMath::DivideAndRoundUp(SizeY, 2);
				}

//#ifdef MUTABLE_DEBUG_RLE		
//				if (bOutSuccess)
//				{
//					// Verify
//					TManagedStrongPtr<FImage> Test = ImagePixelFormat(CompressionQuality, Result, EImageFormat::L_UByte);
//					check(Test->DataStorage == Base->DataStorage);
//				}
//#endif
				break;
			}

			case EImageFormat::RGB_UByte:
			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				check(bFormatAllLODs);
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::L_UByte, -1);
 
				// Try to compress
				if (Result->DataStorage.IsEmpty())
				{
					// Allocate memory for the compressed data. TODO: Smaller?	
					check(Base->DataStorage.Buffers.Num() == Result->DataStorage.Buffers.Num());

					for (int32 I = 0; I < Base->DataStorage.Buffers.Num(); ++I)
					{
						int32 BufferSize = Base->DataStorage.Buffers[I].Num();
						Result->DataStorage.Buffers[I].SetNumUninitialized(BufferSize);
					}
					Result->DataStorage.CompactedTailOffsets = Base->DataStorage.CompactedTailOffsets;
				}

				CompressRLE_L(TempBase.Get(), Result);

//#ifdef MUTABLE_DEBUG_RLE
//				if (bOutSuccess)
//				{
//					// Verify
//					TManagedStrongPtr<FImage> Test = ImagePixelFormat(CompressionQuality, Result, EImageFormat::L_UByte);
//					check(Test->DataStorage == TempBase->DataStorage);
//				}
//#endif

				break;
			}

			default:
				// Case not implemented
				check(false);

			}

			break;
		}

		case EImageFormat::L_UBitRLE:
		{
			switch (Base->GetFormat())
			{

			case EImageFormat::L_UByte:
			{
				check(bFormatAllLODs);

				// Try to compress
				if (Result->DataStorage.IsEmpty())
				{
					// Allocate memory for the compressed data. TODO: Smaller?	
					check(Base->DataStorage.Buffers.Num() == Result->DataStorage.Buffers.Num());

					for (int32 I = 0; I < Base->DataStorage.Buffers.Num(); ++I)
					{
						int32 BufferSize = Base->DataStorage.Buffers[I].Num();
						Result->DataStorage.Buffers[I].SetNumUninitialized(BufferSize);
					}
					Result->DataStorage.CompactedTailOffsets = Base->DataStorage.CompactedTailOffsets;
				}
				
				CompressRLE_L1(Base, Result);
				
//#ifdef MUTABLE_DEBUG_RLE		
//				if (bOutSuccess)
//				{
//					// Verify
//					TManagedStrongPtr<FImage> Test = ImagePixelFormat(CompressionQuality, Result, EImageFormat::L_UByte);
//					check(Test->DataStorage == Base->DataStorage);
//				}
//#endif

				break;
			}

			case EImageFormat::RGB_UByte:
			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			{
				check(bFormatAllLODs);
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::L_UByte, -1);
				CompressRLE_L1(TempBase.Get(), Result);

//#ifdef MUTABLE_DEBUG_RLE
//				if (bOutSuccess)
//				{
//					// Verify
//					TManagedStrongPtr<FImage> Test = ImagePixelFormat(CompressionQuality, Result, EImageFormat::L_UByte);
//					check(Test->DataStorage == TempBase->DataStorage);
//				}
//#endif
				break;
			}

			default:
				// Case not implemented
				check(false);

			}

			break;
		}

		case EImageFormat::RGB_UByte:
		{
			switch (Base->GetFormat())
			{

			case EImageFormat::L_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 3, 1, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*3 + 0] = BaseBuf[I];
						DestBuf[I*3 + 1] = BaseBuf[I];
						DestBuf[I*3 + 2] = BaseBuf[I];
					}
				});

				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 3, 3, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					FMemory::Memcpy(DestBuf, BaseBuf, 3*NumElems);
				});

				break;
			}
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 3, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*3 + 0] = BaseBuf[I*4 + 0];
						DestBuf[I*3 + 1] = BaseBuf[I*4 + 1];
						DestBuf[I*3 + 2] = BaseBuf[I*4 + 2];
					}
				});

				break;
			}
			case EImageFormat::BGRA_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 3, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*3 + 0] = BaseBuf[I*4 + 2];
						DestBuf[I*3 + 1] = BaseBuf[I*4 + 1];
						DestBuf[I*3 + 2] = BaseBuf[I*4 + 0];
					}
				});

				break;
			}
			case EImageFormat::RGB_UByteRLE:
			{
				UncompressRLE_RGB(Base, Result);
				break;
			}
			case EImageFormat::RGBA_UByteRLE:
			{
				check(bFormatAllLODs);

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);

				ImagePixelFormatInternal::ProcessBatchedHelper(Result, TempBase.Get(), 3, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*3 + 0] = BaseBuf[I*4 + 2];
						DestBuf[I*3 + 1] = BaseBuf[I*4 + 1];
						DestBuf[I*3 + 2] = BaseBuf[I*4 + 0];
					}
				});

				break;
			}
			case EImageFormat::BC1:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC1_to_RGB);
				break;
			}
			case EImageFormat::BC2:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC2_to_RGB);
				break;
			}
			case EImageFormat::BC3:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC3_to_RGB);
				break;
			}
			case EImageFormat::BC4:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC4_to_RGB);
				break;
			}
			case EImageFormat::BC5:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC5_to_RGB);
				break;
			}
			case EImageFormat::ASTC_4x4_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_4x4_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBAL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_4x4_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_6x6_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC6x6RGBL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_6x6_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC6x6RGBAL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_6x6_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC6x6RGL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_8x8_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC8x8RGBL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_8x8_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC8x8RGBAL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_8x8_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC8x8RGL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_10x10_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC10x10RGBL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_10x10_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC10x10RGBAL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_10x10_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC10x10RGL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_12x12_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC12x12RGBL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_12x12_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC12x12RGBAL_to_RGB);
				break;
			}
			case EImageFormat::ASTC_12x12_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC12x12RGL_to_RGB);
				break;
			}

			default:
				// Case not implemented
				check(false);
			}

			break;
		}

		case EImageFormat::BGRA_UByte:
		{
			// TODO: Optimise
			Result->DataStorage.ImageFormat = EImageFormat::RGBA_UByte;
			ImagePixelFormat(bOutSuccess, CompressionQuality, Result, Base, ResultLODBegin, BaseLODBegin, NumLODs);
			check(bOutSuccess);
			Result->DataStorage.ImageFormat = EImageFormat::BGRA_UByte;

			ImagePixelFormatInternal::ProcessBatchedHelper(Result, 4, ResultLODBegin, NumLODs,
			[](uint8* DestBuf, int32 NumElems)
			{
				for (int32 I = 0; I < NumElems; ++I)
				{
					uint8 Temp = DestBuf[I*4 + 0];
					DestBuf[I*4 + 0] = DestBuf[I*4 + 2];
					DestBuf[I*4 + 2] = Temp;
				}
			});

			break;
		}

		case EImageFormat::RGBA_UByte:
		{
			switch (Base->GetFormat())
			{

			case EImageFormat::RGBA_UByteRLE:
			{
				check(bFormatAllLODs);
				UncompressRLE_RGBA(Base, Result);
				break;
			}
			case EImageFormat::L_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 4, 1, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*4 + 0] = BaseBuf[I];
						DestBuf[I*4 + 1] = BaseBuf[I];
						DestBuf[I*4 + 2] = BaseBuf[I];
						DestBuf[I*4 + 3] = 255;
					}
				});

				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 4, 3, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*4 + 0] = BaseBuf[I*3 + 0];
						DestBuf[I*4 + 1] = BaseBuf[I*3 + 1];
						DestBuf[I*4 + 2] = BaseBuf[I*3 + 2];
						DestBuf[I*4 + 3] = 255;
					}
				});

				break;
			}
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 4, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					FMemory::Memcpy(DestBuf, BaseBuf, NumElems*4);
				});

				break;
			}
			case EImageFormat::BGRA_UByte:
			{
				ImagePixelFormatInternal::ProcessBatchedHelper(Result, Base, 4, 4, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*4 + 0] = BaseBuf[I*4 + 2];
						DestBuf[I*4 + 1] = BaseBuf[I*4 + 1];
						DestBuf[I*4 + 2] = BaseBuf[I*4 + 0];
						DestBuf[I*4 + 3] = BaseBuf[I*4 + 3];
					}
				});

				break;
			}
			case EImageFormat::BC1:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC1_to_RGBA);
				break;
			}
			case EImageFormat::BC2:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC2_to_RGBA);
				break;
			}
			case EImageFormat::BC3:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC3_to_RGBA);
				break;
			}
			case EImageFormat::BC4:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC4_to_RGBA);
				break;
			}
			case EImageFormat::BC5:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::BC5_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_4x4_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_4x4_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBAL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_4x4_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_6x6_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC6x6RGBL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_6x6_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC6x6RGBAL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_6x6_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC6x6RGL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_8x8_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC8x8RGBL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_8x8_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC8x8RGBAL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_8x8_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC8x8RGL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_10x10_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC10x10RGBL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_10x10_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC10x10RGBAL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_10x10_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC10x10RGL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_12x12_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC12x12RGBL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_12x12_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC12x12RGBAL_to_RGBA);
				break;
			}
			case EImageFormat::ASTC_12x12_RG_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC12x12RGL_to_RGBA);
				break;
			}
			case EImageFormat::RGB_UByteRLE:
			{
				check(bFormatAllLODs);
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGB_UByte);

				ImagePixelFormatInternal::ProcessBatchedHelper(Result, TempBase.Get(), 4, 3, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*4 + 0] = BaseBuf[I*3 + 0];
						DestBuf[I*4 + 1] = BaseBuf[I*3 + 1];
						DestBuf[I*4 + 2] = BaseBuf[I*3 + 2];
						DestBuf[I*4 + 3] = 255;
					}
				});

				break;
			}
			case EImageFormat::L_UByteRLE:
			{
				check(bFormatAllLODs);
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::L_UByte);

				ImagePixelFormatInternal::ProcessBatchedHelper(Result, TempBase.Get(), 4, 1, ResultLODBegin, BaseLODBegin, NumLODs,
				[](uint8* DestBuf, const uint8* BaseBuf, int32 NumElems)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I*4 + 0] = BaseBuf[I];
						DestBuf[I*4 + 1] = BaseBuf[I];
						DestBuf[I*4 + 2] = BaseBuf[I];
						DestBuf[I*4 + 3] = 255;
					}
				});

				break;
			}

			default:
				// Case not implemented
				check(false);
			}

			break;
		}

		case EImageFormat::RGBA_UByteRLE:
		{
			switch (Base->GetFormat())
			{

			case EImageFormat::RGBA_UByte:
			{
				check(bFormatAllLODs);
				CompressRLE_RGBA(Base, Result);

				break;
			}
			case EImageFormat::RGB_UByte:
			{
				check(bFormatAllLODs);

				// \todo: optimise
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				CompressRLE_RGBA(TempBase.Get(), Result);

				// Test
				//TManagedStrongPtr<FImage> Test = ImagePixelFormat(CompressionQuality, Result, EImageFormat::RGBA_UByte); 
				//check(Test->DataStorage == Base->DataStorage);


				break;
			}
			case EImageFormat::RGB_UByteRLE:
			{
				check(bFormatAllLODs);

				// \todo: optimise
				TManagedPtr<FImage> Temp1 = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGB_UByte);
				TManagedPtr<FImage> Temp2 = ImagePixelFormat(CompressionQuality, Temp1.Get(), EImageFormat::RGBA_UByte);
				CompressRLE_RGBA(Temp2.Get(), Result);

				break;
			}
			default:
				// Case not implemented
				check(false);

			}

			break;
		}

		case EImageFormat::RGB_UByteRLE:
		{
			switch (Base->GetFormat())
			{

			case EImageFormat::RGB_UByte:
			{
				check(bFormatAllLODs);
				CompressRLE_RGB(Base, Result);

				// Test
				// TManagedStrongPtr<FImage> Test = ImagePixelFormat(CompressionQuality, Result, EImageFormat::RGB_UByte); 
				// check(Test->DataStorage == Base->DataStorage);
				break;
			}

			case EImageFormat::RGBA_UByteRLE:
			{
				check(bFormatAllLODs);

				// \todo: optimise
				TManagedPtr<FImage> Temp1 = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				TManagedPtr<FImage> Temp2 = ImagePixelFormat(CompressionQuality, Temp1.Get(), EImageFormat::RGB_UByte);
				CompressRLE_RGB(Temp2.Get(), Result);

				break;
			}

			default:
				// Case not implemented
				check(false);
			}

			break;
		}

		case EImageFormat::BC1:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_BC1);
				break;
			}
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC1);
				break;
			}
			case EImageFormat::L_UByte:
			{
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGB_UByte);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_BC1);

				break;
			}
			case EImageFormat::BC3:
			{
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC1);

				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				// Case not implemented, try a generic more expensive operation.
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC1);
			}

			}

			break;
		}

		case EImageFormat::BC2:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC2);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used.");

				// Case not implemented, try a generic more expensive operatino.
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC2);
			}

			}

			break;
		}

		case EImageFormat::BC3:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC3);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_BC3);
				break;
			}
			case EImageFormat::BC1:
			{
				MUTABLE_CPUPROFILER_SCOPE(BC1toBC3);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::BC1_to_BC3);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used.");

				// Case not implemented, try a generic more expensive operation.
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC3);
			}

			}

			break;
		}

		case EImageFormat::BC4:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::L_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::L_to_BC4);
				break;
			}
			case EImageFormat::RGB_UByte:
			case EImageFormat::RGBA_UByte:
			case EImageFormat::BGRA_UByte:
			default:
			{	
				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::L_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::L_to_BC4);
				break;
			}
			}

			break;
		}

		case EImageFormat::BC5:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC5);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_BC5);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_BC5);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_4x4_RGB_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC4x4RGBL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC4x4RGBL);
				break;
			}
			case EImageFormat::ASTC_4x4_RGBA_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBAL_to_ASTC4x4RGBL);
				break;
			}
			case EImageFormat::L_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::L_to_ASTC4x4RGBL);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC4x4RGBL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_4x4_RGBA_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC4x4RGBAL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC4x4RGBAL);
				break;
			}
			case EImageFormat::ASTC_4x4_RGB_LDR:
			{
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBL_to_ASTC4x4RGBAL);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC4x4RGBAL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_4x4_RG_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC4x4RGL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC4x4RGL);
				break;
			}
			case EImageFormat::ASTC_4x4_RG_LDR:
			{
				// Hack that actually works because of block size.
				ImagePixelFormatInternal::BlockDecompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, miro::ASTC4x4RGBAL_to_ASTC4x4RGBL);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC4x4RGL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_6x6_RGB_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC6x6RGBL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC6x6RGBL);
				break;
			}
			//case EImageFormat::ASTC_6x6_RGBA_LDR:
			//{
			//  ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//  		Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::ASTC6x6RGBAL_to_ASTC6x6RGBL);
			//	break;
			//}
			//case EImageFormat::L_UByte:
			//{
			//  ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//  		Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::L_to_ASTC6x6RGBL);
			//	break;
			//}

			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC6x6RGBL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_6x6_RGBA_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC6x6RGBAL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC6x6RGBAL);
				break;
			}
			//case EImageFormat::ASTC_6x6_RGB_LDR:
			//{
			//	ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//			Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::ASTC6x6RGBL_to_ASTC6x6RGBAL);
			//	break;
			//}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC6x6RGBAL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_6x6_RG_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC6x6RGL);
				break;
			}

			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC6x6RGL);
				break;
			}
			//case EImageFormat::ASTC_6x6_RG_LDR:
			//{
			//	// Hack that actually works because of block size.
			//	ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//			Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::ASTC6x6RGBAL_to_ASTC6x6RGBL);
			//	break;
			//}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC6x6RGL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_8x8_RGB_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC8x8RGBL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC8x8RGBL);
				break;
			}
			//case EImageFormat::ASTC_8x8_RGBA_LDR:
			//{
			//	ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//			Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::ASTC8x8RGBAL_to_ASTC8x8RGBL);
			//	break;
			//}
			//case EImageFormat::L_UByte:
			//{
			//	ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//			Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::L_to_ASTC8x8RGBL);
			//	break;
			//}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC8x8RGBL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_8x8_RGBA_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC8x8RGBAL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC8x8RGBAL);
				break;
			}
			//case EImageFormat::ASTC_8x8_RGB_LDR:
			//{
			//	ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
			//			Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::ASTC8x8RGBL_to_ASTC8x8RGBAL);
			//	break;
			//}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC8x8RGBAL);
				break;
			}
			}

			break;
		}

		case EImageFormat::ASTC_8x8_RG_LDR:
		{
			switch (Base->GetFormat())
			{
			case EImageFormat::RGBA_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC8x8RGL);
				break;
			}
			case EImageFormat::RGB_UByte:
			{
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, Base, ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGB_to_ASTC8x8RGL);
				break;
			}
			default:
			{
				UE_LOGF(LogMutableCore, Log, "Image format conversion not implemented. Expensive generic one used. ");

				TManagedPtr<FImage> TempBase = ImagePixelFormat(CompressionQuality, Base, EImageFormat::RGBA_UByte, -1);
				ImagePixelFormatInternal::BlockCompressionLODRangeHelper(
						Result, TempBase.Get(), ResultLODBegin, BaseLODBegin, NumLODs, CompressionQuality, miro::RGBA_to_ASTC8x8RGL);
				break;
			}
			}
			
			break;
		}

		default:
			// Case not implemented
			check(false);
		}
	}
}

}
