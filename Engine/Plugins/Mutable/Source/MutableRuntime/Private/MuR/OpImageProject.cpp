// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpImageProject.h"
#include "MuR/ConvertData.h"
#include "MuR/Image.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Layout.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/OpMeshClipWithMesh.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/Raster.h"

#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Intersection/IntrRay3Triangle3.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"

#include "Spatial/PointHashGrid3.h"

#include "PackedNormal.h"

#include "MuR/Image.h"
#include "MuR/Mesh.h"


namespace UE::Mutable::Private
{

//-------------------------------------------------------------------------------------------------
//! This format is the one we assumee the meshes optimised for planar and cylindrical projection
//! will have.
//! See CreateMeshOptimisedForProjection
struct FOptimizedVertex
{
	FVector2f Uv;
	FVector3f Position;
	FVector3f Normal;
};

static_assert(sizeof(FOptimizedVertex)==32, "UNEXPECTED_STRUCT_SIZE" );



namespace Private
{
	struct FImageRasterInvokeArgs
	{
		const FMesh* MeshPtr;
		FImage* ImagePtr;
		const FImage* SourcePtr;
		const FImage* MaskPtr;
		bool bIsRGBFadingEnabled;
		bool bIsAlphaFadingEnabled;
		ESamplingMethod SamplingMethod;
		float FadeStart;
		float FadeEnd;
		float ProjectionAngle;
		float MipInterpolationFactor;
		int32 Layout;
		uint64 BlockId;
		UE::Math::TIntVector2<uint16> CropMin;
		UE::Math::TIntVector2<uint16> UncroppedSize;
		FScratchImageProject* Scratch;
	};

	enum class EPixelProcessorFeatures
	{
		None = 0,
		WithMask              = 1 << 1,
		SamplingPoint         = 1 << 2,
		SamplingLinear		  = 1 << 3, 
		ProjectionPlanar	  = 1 << 4,
		ProjectionCylindrical = 1 << 5,
		ProjectionWrap		  = 1 << 6,
		FormatRGBA            = 1 << 7,
		FormatRGB             = 1 << 8,
		FormatL               = 1 << 9,
		VectorizedImpl		  = 1 << 10    
	};

	ENUM_CLASS_FLAGS(EPixelProcessorFeatures);

	constexpr bool CheckExactlyOneFormatFlag(EPixelProcessorFeatures Features)
	{
		return 
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::FormatL)) +
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::FormatRGB)) +
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::FormatRGBA)) == 1;
	}

	constexpr bool CheckExactlyOneSamplingFlag(EPixelProcessorFeatures Features)
	{
		return
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingPoint)) +
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingLinear)) == 1;
	}

	constexpr bool CheckExactlyOneProjectionFlag(EPixelProcessorFeatures Features)
	{
		return
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionCylindrical)) +
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionPlanar)) +
			static_cast<int32>(EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionWrap)) == 1;
	}

	constexpr int32 GetFormatNumChannels(EPixelProcessorFeatures Features)
	{
		if (EnumHasAnyFlags(Features, EPixelProcessorFeatures::FormatL))
		{
			return 1;
		}
		else if (EnumHasAnyFlags(Features, EPixelProcessorFeatures::FormatRGB))
		{
			return 3;
		}
		else if (EnumHasAnyFlags(Features, EPixelProcessorFeatures::FormatRGBA))
		{
			return 4;
		}

		return -1;
	}

	// The processor constant data fits into a 64 bytes cache line, align to it so potentially have less cache 
	// misses.
	struct alignas(64) FProjectedPixelProcessorContext
	{
		const uint8* TargetData = nullptr;	
		const uint8* Source0Data = nullptr;
		
		uint16 Source0SizeX = 0;
		uint16 Source0SizeY = 0;
		
		uint16 Source1SizeX = 0;
		uint16 Source1SizeY = 0;

		const uint8* Source1Data = nullptr;
		const uint8* MaskData = nullptr;

        //! Cosine of the fading angle range
		float FadeEndCos = 0.0f;
		// precomputation of 255 / (FadeStartCos - FadeEndCos) 
		float OneOverFadeRangeTimes255 = 0.0f;

		// Only used for cylindrical projections.
		float  OneOverProjectionAngle = 0.0f;

		// Only used with linear sampling.
		float MipInterpolationFactor = 0.0f;

		// Those are the seed for a mask 0xFFFFFFFF (-1) mask for enabled and 0 otherwise.
		// so, the expected value here is -1 for enabled and 0 for disabled.
		int32 RGBFadingEnabledMask = 0;
		int32 AlphaFadingEnabledMask = 0;
	};

	static_assert(sizeof(FProjectedPixelProcessorContext) <= 64);

	template<EPixelProcessorFeatures Features>
	class TProjectedPixelProcessor
	{
		static_assert(CheckExactlyOneFormatFlag(Features));
		static_assert(CheckExactlyOneSamplingFlag(Features));
		static_assert(CheckExactlyOneProjectionFlag(Features));

		static constexpr int32 PIXEL_SIZE = GetFormatNumChannels(Features);
		static_assert(PIXEL_SIZE != -1 && PIXEL_SIZE > 0);

	public:
		static FProjectedPixelProcessorContext MakeContext(
			const FImage* Source,
			const uint8* InTargetData,
			const uint8* InMaskData,
			bool bInIsRGBFadingEnabled, bool bInIsAlphaFadingEnabled,
			float FadeStart, float FadeEnd, float InProjectionAngle, float InMipInterpolationFactor)
		{
			FProjectedPixelProcessorContext Context;

            Context.Source0SizeX = static_cast<uint16>(Source->GetSizeX());
            Context.Source0SizeY = static_cast<uint16>(Source->GetSizeY());
			Context.Source0Data = (Context.Source0SizeX > 0 && Context.Source0SizeY > 0) ? Source->GetLODData(0) : nullptr;

			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingLinear))
			{
				// If we don't have the next mip, use the first mip as a fallback.
				const bool bHasNextMip = Source->GetLODCount() >= 2;

				Context.Source1Data = bHasNextMip ? Source->GetMipData(1) : Context.Source0Data;

				FIntVector2 Source1Size = Source->CalculateMipSize(1);
				Context.Source1SizeX = bHasNextMip ? static_cast<uint16>(Source1Size.X) : Context.Source0SizeX;
				Context.Source1SizeY = bHasNextMip ? static_cast<uint16>(Source1Size.Y) : Context.Source0SizeY;

				// Invalidate SourceData if either of the two is not valid. 
				if (!(Context.Source0Data && Context.Source1Data))
				{
					Context.Source0Data = nullptr;
					Context.Source1Data = nullptr;
				}
			}

			Context.MipInterpolationFactor = InMipInterpolationFactor;

			Context.RGBFadingEnabledMask = bInIsRGBFadingEnabled ? -1 : 0;
			Context.AlphaFadingEnabledMask = bInIsAlphaFadingEnabled ? -1 : 0;

			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionPlanar))
			{
				const float FadeStartCos = FMath::Max(0.0f, FMath::Cos(FadeStart));
				Context.FadeEndCos = FMath::Max(0.0f, FMath::Cos(FadeEnd));

				const float FadeCosRangeSafeDiv =
					FMath::IsNearlyZero(FadeStartCos - Context.FadeEndCos, UE_KINDA_SMALL_NUMBER)
					? UE_KINDA_SMALL_NUMBER
					: FadeStartCos - Context.FadeEndCos;

				Context.OneOverFadeRangeTimes255 = 255.0f / FadeCosRangeSafeDiv;
			}
			else
			{
				Context.FadeEndCos = 0.0f;
				Context.OneOverFadeRangeTimes255 = 255.0f;
			}

			Context.OneOverProjectionAngle = FMath::IsNearlyZero(InProjectionAngle)
					? 1.0f / UE_KINDA_SMALL_NUMBER
					: 1.0f / InProjectionAngle;

            Context.TargetData = InTargetData;
            Context.MaskData = InMaskData;

            check(GetImageFormatData(Source->GetFormat()).BytesPerBlock == PIXEL_SIZE);

			return Context;
		}

		static void ProcessPixel(const FProjectedPixelProcessorContext& Context, uint8* BufferPos, float Varying[4])
		{
			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::VectorizedImpl))
			{
				ProcessPixelVectorImpl(Context, BufferPos, Varying);
			}
			else
			{
				ProcessPixelImpl(Context, BufferPos, Varying);
			}
		}

	private:
		static void ProcessPixelVectorImpl(const FProjectedPixelProcessorContext& Context, uint8* BufferPos, float Varying[4])
		{
			if (!Context.Source0Data)
			{
				return;
			}

			const bool bDepthClamp = Invoke([&]() -> bool
			{
				if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionCylindrical))
				{
					const FVector2f CylinderPolarCoords(Varying[1], Varying[2]);
					return FVector2f::DotProduct(CylinderPolarCoords, CylinderPolarCoords) > 1.0f;
				}
				else
				{
					return static_cast<bool>((Varying[2] < 0.0f) | (Varying[2] > 1.0f));
				}
			});

			float Factor = static_cast<float>(!bDepthClamp) * 255.0f;
			float MaskFactor = 255.0f;

			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionPlanar))
			{
				const float AngleCos = Varying[3];

				Factor = FMath::Min(Factor, (FMath::Clamp((AngleCos - Context.FadeEndCos) * Context.OneOverFadeRangeTimes255, 0.0f, 255.0f)));
				MaskFactor = AngleCos > Context.FadeEndCos ? 255.0f : 0.0f;
			}

			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::WithMask))
			{
				// Only read from memory if needed.
				if (Factor > UE_SMALL_NUMBER)
				{
					MaskFactor = ((MaskFactor > 0.0f) && Context.MaskData)
							? static_cast<float>(Context.MaskData[(BufferPos - Context.TargetData) / PIXEL_SIZE])
							: MaskFactor;
					Factor = (Factor * MaskFactor) * (1.0f/255.0f);
				}
			}

            if (Factor > UE_SMALL_NUMBER)
            {
				const FVector2f Uv = Invoke([&]()
				{ 
					if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionCylindrical))
					{
						return FVector2f{ 0.5f + FMath::Atan2(Varying[2], -Varying[1]) * Context.OneOverProjectionAngle, Varying[0] };
					}
					else
					{
						return FVector2f{ Varying[0], Varying[1] };
					}
				});

                if ((Uv.X >= 0.0f) & (Uv.X < 1.0f) & (Uv.Y >= 0.0f) & (Uv.Y < 1.0f))
                {
					using namespace GlobalVectorConstants;

					auto LoadPixel = [](const uint8* Ptr) -> VectorRegister4Float
					{
						constexpr VectorRegister4Float OneOver255 =
							MakeVectorRegisterFloatConstant(1.0f / 255.0f, 1.0f / 255.0f, 1.0f / 255.0f, 1.0f / 255.0f);

						if constexpr (PIXEL_SIZE == 4)
						{
							const uint32 PackedData = *reinterpret_cast<const uint32*>(Ptr);
							return VectorMultiply(MakeVectorRegister(
								static_cast<float>((PackedData >> (8 * 0)) & 0xFF),
								static_cast<float>((PackedData >> (8 * 1)) & 0xFF),
								static_cast<float>((PackedData >> (8 * 2)) & 0xFF),
								static_cast<float>((PackedData >> (8 * 3)) & 0xFF)), OneOver255);
						}
						else
						{
							alignas(VectorRegister4Float) float PixelData[4];
							for (int32 C = 0; C < PIXEL_SIZE; ++C)
							{
								PixelData[C] = static_cast<float>(Ptr[C]);
							}

							return VectorMultiply(VectorLoadAligned(&(PixelData[0])), OneOver255);
						}
					};

					VectorRegister4Float Result;
					if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingLinear))
					{
						const VectorRegister4Int SizeI =
							MakeVectorRegisterInt(Context.Source0SizeX, Context.Source0SizeY, Context.Source1SizeX, Context.Source1SizeY);

						const VectorRegister4Float SizeF = VectorIntToFloat(SizeI);

						const VectorRegister4Float CoordsF = VectorMin(VectorMax(
							VectorMultiplyAdd(MakeVectorRegister(Uv.X, Uv.Y, Uv.X, Uv.Y), SizeF, FloatMinusOneHalf),
							FloatZero),
							VectorSubtract(SizeF, FloatOne));

						const VectorRegister4Float Frac = VectorSubtract(CoordsF, VectorFloor(CoordsF));

						const VectorRegister4Int CoordsI = VectorFloatToInt(CoordsF);
						const VectorRegister4Int CoordsIPlusOne = VectorIntMin(
							VectorIntAdd(CoordsI, IntOne), VectorIntSubtract(SizeI, IntOne));

						alignas(VectorRegister4Int) int32 CoordsIData[4];
						VectorIntStoreAligned(CoordsI, CoordsIData);

						alignas(VectorRegister4Int) int32 CoordsIPlusOneData[4];
						VectorIntStoreAligned(CoordsIPlusOne, CoordsIPlusOneData);

						uint8 const* const Pixel000Ptr = Context.Source0Data + (CoordsIData[1] * Context.Source0SizeX + CoordsIData[0]) * PIXEL_SIZE;
						uint8 const* const Pixel010Ptr = Context.Source0Data + (CoordsIData[1] * Context.Source0SizeX + CoordsIPlusOneData[0]) * PIXEL_SIZE;
						uint8 const* const Pixel001Ptr = Context.Source0Data + (CoordsIPlusOneData[1] * Context.Source0SizeX + CoordsIData[0]) * PIXEL_SIZE;
						uint8 const* const Pixel011Ptr = Context.Source0Data + (CoordsIPlusOneData[1] * Context.Source0SizeX + CoordsIPlusOneData[0]) * PIXEL_SIZE;

						uint8 const* const Pixel100Ptr = Context.Source1Data + (CoordsIData[3] * Context.Source1SizeX + CoordsIData[2]) * PIXEL_SIZE;
						uint8 const* const Pixel110Ptr = Context.Source1Data + (CoordsIData[3] * Context.Source1SizeX + CoordsIPlusOneData[2]) * PIXEL_SIZE;
						uint8 const* const Pixel101Ptr = Context.Source1Data + (CoordsIPlusOneData[3] * Context.Source1SizeX + CoordsIData[2]) * PIXEL_SIZE;
						uint8 const* const Pixel111Ptr = Context.Source1Data + (CoordsIPlusOneData[3] * Context.Source1SizeX + CoordsIPlusOneData[2]) * PIXEL_SIZE;

						const VectorRegister4Float Pixel000 = LoadPixel(Pixel000Ptr);
						const VectorRegister4Float Pixel010 = LoadPixel(Pixel010Ptr);
						const VectorRegister4Float Pixel001 = LoadPixel(Pixel001Ptr);
						const VectorRegister4Float Pixel011 = LoadPixel(Pixel011Ptr);

						const VectorRegister4Float Pixel100 = LoadPixel(Pixel100Ptr);
						const VectorRegister4Float Pixel110 = LoadPixel(Pixel110Ptr);
						const VectorRegister4Float Pixel101 = LoadPixel(Pixel101Ptr);
						const VectorRegister4Float Pixel111 = LoadPixel(Pixel111Ptr);

						const VectorRegister4Float Frac0X = VectorReplicate(Frac, 0);
						const VectorRegister4Float Frac0Y = VectorReplicate(Frac, 1);

						// Bilerp image 0
						VectorRegister4Float Sample0 = VectorMultiplyAdd(Frac0X, VectorSubtract(Pixel010, Pixel000), Pixel000);
						Sample0 = VectorMultiplyAdd(
							Frac0Y,
							VectorSubtract(
								VectorMultiplyAdd(Frac0X, VectorSubtract(Pixel011, Pixel001), Pixel001),
								Sample0),
							Sample0);

						const VectorRegister4Float Frac1X = VectorReplicate(Frac, 2);
						const VectorRegister4Float Frac1Y = VectorReplicate(Frac, 3);

						// Bilerp image 1
						VectorRegister4Float Sample1 = VectorMultiplyAdd(Frac1X, VectorSubtract(Pixel110, Pixel100), Pixel100);
						Sample1 = VectorMultiplyAdd(
							Frac1Y,
							VectorSubtract(
								VectorMultiplyAdd(Frac1X, VectorSubtract(Pixel111, Pixel101), Pixel101),
								Sample1),
							Sample1);

						Result = VectorMultiplyAdd(VectorSetFloat1(Context.MipInterpolationFactor), VectorSubtract(Sample1, Sample0), Sample0);
					}
					else if (EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingPoint))
					{						
						const int32 CoordX = FMath::FloorToInt32(Context.Source0SizeX * Uv.X);
						const int32 CoordY = FMath::FloorToInt32(Context.Source0SizeY * Uv.Y);

						Result = LoadPixel(Context.Source0Data + (CoordY*Context.Source0SizeX + CoordX) * PIXEL_SIZE);
					}

					Result = VectorMultiply(
						Result,
						// Fadding factor
						VectorSelect(
							// The selction mask needs to be 0xFFFFFFFF if enabled 0 otherwise. FadingEnabledValue will be -1
							// if enabled and 0 otherwise.
							VectorCastIntToFloat(MakeVectorRegisterInt(
								static_cast<int32>(Context.RGBFadingEnabledMask),
								static_cast<int32>(Context.RGBFadingEnabledMask),
								static_cast<int32>(Context.RGBFadingEnabledMask),
								static_cast<int32>(Context.AlphaFadingEnabledMask))),
							VectorSetFloat1(Factor),
							VectorSetFloat1(MaskFactor)));

					if constexpr (PIXEL_SIZE == 4)
					{
						VectorStoreByte4(Result, BufferPos);
						VectorResetFloatRegisters();
					}
					else
					{
						const AlignedFloat4 ResultData(VectorMin(VectorMax(Result, FloatZero), Float255));
						for (int32 C = 0; C < PIXEL_SIZE; ++C)
						{
							BufferPos[C] = static_cast<uint8>(ResultData[C]);		
						}				
					}
                }
            }
		}

        static void ProcessPixelImpl(const FProjectedPixelProcessorContext& Context, uint8* BufferPos, float Varying[4])
        {
			if (!Context.Source0Data)
			{
				return;
			}

			const bool bDepthClamp = Invoke([&]() -> bool
			{
				if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionCylindrical))
				{
					const FVector2f CylinderPolarCoords(Varying[1], Varying[2]);
					return FVector2f::DotProduct(CylinderPolarCoords, CylinderPolarCoords) > 1.0f;
				}
				else
				{
					return static_cast<bool>((Varying[2] < 0.0f) | (Varying[2] > 1.0f));
				}
			});

			uint16 Factor = static_cast<uint16>(!bDepthClamp) * 255;
			uint16 MaskFactor = 255;

			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionPlanar))
			{
				const float AngleCos = Varying[3];

				Factor = FMath::Min(Factor, 
						static_cast<uint16>(FMath::Clamp((AngleCos - Context.FadeEndCos) * Context.OneOverFadeRangeTimes255, 0.0f, 255.0f)));

				MaskFactor = AngleCos > Context.FadeEndCos ? 255 : 0;
			}

			if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::WithMask))
			{
				// Only read from memory if needed.
				if (Factor > 0)
				{
					MaskFactor = ((MaskFactor > 0) && Context.MaskData) 
							? Context.MaskData[(BufferPos - Context.TargetData) / PIXEL_SIZE]
							: MaskFactor;
					Factor = (Factor * MaskFactor) / 255;
				}
			}

            if (Factor > 0)
            {
				const FVector2f Uv = Invoke([&]()
				{ 
					if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionCylindrical))
					{
						return FVector2f{ 0.5f + FMath::Atan2(Varying[2], -Varying[1])*Context.OneOverProjectionAngle, Varying[0] };
					}
					else
					{
						return FVector2f{ Varying[0], Varying[1] };
					}
				});

				const FVector2f Source0SizeF = FVector2f(Context.Source0SizeX, Context.Source0SizeY);
				const FVector2f Source1SizeF = FVector2f(Context.Source1SizeX, Context.Source1SizeY);

				if ((Uv.X >= 0.0f) & (Uv.X < 1.0f) & (Uv.Y >= 0.0f) & (Uv.Y < 1.0f))
				{
					using FUInt16Vector2 = UE::Math::TIntVector2<uint16>;
					struct FPixelData
					{
						alignas(8) uint16 Data[PIXEL_SIZE];
					};


					auto LoadPixel = [](const uint8* Ptr) -> FPixelData
					{
						FPixelData Result;
						if constexpr (PIXEL_SIZE == 4)
						{
							const uint32 PackedData = *reinterpret_cast<const uint32*>(Ptr);

							Result.Data[0] = static_cast<uint16>((PackedData >> (8 * 0)) & 0xFF);
							Result.Data[1] = static_cast<uint16>((PackedData >> (8 * 1)) & 0xFF);
							Result.Data[2] = static_cast<uint16>((PackedData >> (8 * 2)) & 0xFF);
							Result.Data[3] = static_cast<uint16>((PackedData >> (8 * 3)) & 0xFF);
						}
						else
						{
							for (int32 C = 0; C < PIXEL_SIZE; ++C)
							{
								Result.Data[C] = static_cast<uint16>(Ptr[C]);
							}
						}

						return Result;
					};

					FPixelData Result;
					FMemory::Memzero(Result);

					if constexpr (EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingLinear))
					{
						auto SampleImageBilinear = [&](FVector2f Uv, FUInt16Vector2 Size, const uint8* DataPtr)
						{
							const FVector2f SizeF(Size.X, Size.Y);

							const FVector2f CoordsF = FVector2f(
								FMath::Clamp(Uv.X * SizeF.X - 0.5f, 0.0f, SizeF.X - 1.0f),
								FMath::Clamp(Uv.Y * SizeF.Y - 0.5f, 0.0f, SizeF.Y - 1.0f));

							const FUInt16Vector2 Frac = FUInt16Vector2(
								static_cast<uint16>(FMath::Frac(CoordsF.X) * 255.0f),
								static_cast<uint16>(FMath::Frac(CoordsF.Y) * 255.0f));

							const FIntVector2 Coords = FIntVector2(CoordsF.X, CoordsF.Y);
							const FIntVector2 CoordsPlusOne = FIntVector2(
								FMath::Min(Size.X - 1, Coords.X + 1),
								FMath::Min(Size.Y - 1, Coords.Y + 1));

							uint8 const* const Pixel00Ptr = DataPtr + (Coords.Y * Size.X + Coords.X) * PIXEL_SIZE;
							uint8 const* const Pixel10Ptr = DataPtr + (Coords.Y * Size.X + CoordsPlusOne.X) * PIXEL_SIZE;
							uint8 const* const Pixel01Ptr = DataPtr + (CoordsPlusOne.Y * Size.X + Coords.X) * PIXEL_SIZE;
							uint8 const* const Pixel11Ptr = DataPtr + (CoordsPlusOne.Y * Size.X + CoordsPlusOne.X) * PIXEL_SIZE;

							FPixelData PixelData00 = LoadPixel(Pixel00Ptr);
							FPixelData PixelData10 = LoadPixel(Pixel10Ptr);
							FPixelData PixelData01 = LoadPixel(Pixel01Ptr);
							FPixelData PixelData11 = LoadPixel(Pixel11Ptr);

							FPixelData FilteredPixelData;

							for (int32 C = 0; C < PIXEL_SIZE; ++C)
							{
								const uint16 LerpY0 = ((PixelData10.Data[C] * Frac.X) + PixelData00.Data[C] * (255 - Frac.X)) / 255;
								const uint16 LerpY1 = ((PixelData11.Data[C] * Frac.X) + PixelData01.Data[C] * (255 - Frac.X)) / 255;
								FilteredPixelData.Data[C] = ((LerpY1 * Frac.Y) + LerpY0 * (255 - Frac.Y)) / 255;
							}

							return FilteredPixelData;
						};

						Result = SampleImageBilinear(Uv, FUInt16Vector2(Context.Source0SizeX, Context.Source0SizeY), Context.Source0Data);
						FPixelData Sample1 = SampleImageBilinear(Uv, FUInt16Vector2(Context.Source1SizeX, Context.Source1SizeY), Context.Source1Data);

						const uint16 MipFactor = static_cast<uint16>(Context.MipInterpolationFactor * 255.0f);

						for (int32 C = 0; C < PIXEL_SIZE; ++C)
						{
							Result.Data[C] = ((Sample1.Data[C] * MipFactor) + Result.Data[C] * (255 - MipFactor)) / 255;
						}
					}
					else if (EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingPoint))
					{
						const int32 CoordX = FMath::FloorToInt32(Context.Source0SizeX * Uv.X);
						const int32 CoordY = FMath::FloorToInt32(Context.Source0SizeY * Uv.Y);

						Result = LoadPixel(Context.Source0Data + (CoordY*Context.Source0SizeX + CoordX) * PIXEL_SIZE);
					}
					static_assert(EnumHasAnyFlags(Features, EPixelProcessorFeatures::SamplingPoint | EPixelProcessorFeatures::SamplingLinear));

					static_assert(PIXEL_SIZE <= 4);
					// FadingEnabledMaskSeed will be -1 if enabled and 0 otherwise.
					const int16 FadingMask[4] = {
						static_cast<int16>(Context.RGBFadingEnabledMask),
						static_cast<int16>(Context.RGBFadingEnabledMask),
						static_cast<int16>(Context.RGBFadingEnabledMask),
						static_cast<int16>(Context.AlphaFadingEnabledMask) };

					for (int32 I = 0; I < PIXEL_SIZE; ++I)
					{
						BufferPos[I] = static_cast<uint8>(
							(Result.Data[I] * ((MaskFactor & ~FadingMask[I]) + (Factor & FadingMask[I]))) / 255);
					}
				}
            }
        }
	};
	
	//-------------------------------------------------------------------------------------------------
	FORCENOINLINE void ImageRasterProjected_Optimised(const FMesh* pMesh, FImage* pImage,
		TTriangleRasterPixelProcRefType<4> PixelProc, 
		float FadeEnd,
		UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
		FScratchImageProject* Scratch)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageRasterProjected_Optimised);

		if (!pMesh || !pMesh->GetFaceCount())
		{
			return;
		}

		EImageFormat Format = pImage->GetFormat();
		int32 PixelSize = GetImageFormatData(Format).BytesPerBlock;

		int32 SizeX = pImage->GetSizeX();
		int32 SizeY = pImage->GetSizeY();

		bool bUseCropping = UncroppedSize[0] > 0;

		// Get the vertices
		int32 VertexCount = pMesh->GetVertexCount();

		check((int32)Scratch->Vertices.Num() == VertexCount);
		check((int32)Scratch->CulledVertex.Num() == VertexCount);

		check(pMesh->GetVertexBuffers().GetElementSize(0) == sizeof(FOptimizedVertex));
		const FOptimizedVertex* pVertices =  reinterpret_cast<const FOptimizedVertex*>(pMesh->GetVertexBuffers().GetBufferData(0));

		float FadeEndCos = FMath::Cos(FadeEnd);
		for (int32 V = 0; V < VertexCount; ++V)
		{
			if (bUseCropping)
			{
				Scratch->Vertices[V].x = pVertices[V].Uv[0] * UncroppedSize[0] - CropMin[0];
				Scratch->Vertices[V].y = pVertices[V].Uv[1] * UncroppedSize[1] - CropMin[1];
			}
			else
			{
				Scratch->Vertices[V].x = pVertices[V].Uv[0] * SizeX;
				Scratch->Vertices[V].y = pVertices[V].Uv[1] * SizeY;
			}

			// TODO: No need to copy all. use scratch for the rest only.
			Scratch->Vertices[V].interpolators[0] = pVertices[V].Position[0];
			Scratch->Vertices[V].interpolators[1] = pVertices[V].Position[1];
			Scratch->Vertices[V].interpolators[2] = pVertices[V].Position[2];
			Scratch->Vertices[V].interpolators[3] = pVertices[V].Normal[0];
			Scratch->CulledVertex[V] = pVertices[V].Normal[0] < FadeEndCos;
		}

		// Get the indices
		check(pMesh->GetIndexBuffers().GetElementSize(0) == 4);
		const uint32* pIndices = reinterpret_cast<const uint32*>(pMesh->GetIndexBuffers().GetBufferData(0));

		// The mesh is supposed to contain only the faces in the selected block
		int32 FaceCount = pMesh->GetFaceCount();
		//for (int32 Face = 0; Face < FaceCount; ++Face)
		ParallelFor(FaceCount, [pIndices, Scratch, SizeX, SizeY, PixelSize, pImage, PixelProc](int32 Face)
		{
			int32 Index0 = pIndices[Face*3 + 0];
			int32 Index1 = pIndices[Face*3 + 1];
			int32 Index2 = pIndices[Face*3 + 2];

			// TODO: This optimisation may remove projection in the center of the face, if the angle
			// range is small. Make it optional or more sophisticated (cross product may help).
			if (!Scratch->CulledVertex[Index0] ||
				!Scratch->CulledVertex[Index1] ||
				!Scratch->CulledVertex[Index2])
			{
				constexpr int32 NumInterpolators = 4;
				Triangle<NumInterpolators>(pImage->GetLODData(0), pImage->GetDataSize(),
					SizeX, SizeY,
					PixelSize,
					Scratch->Vertices[Index0],
					Scratch->Vertices[Index1],
					Scratch->Vertices[Index2],
					PixelProc,
					false);
			}
		});

		// Update the relevancy map
		// \TODO: fix it to work with cropping.
		if (!bUseCropping)
		{
			MUTABLE_CPUPROFILER_SCOPE(ImageRasterProjected_Optimised_Relevancy);

			float MinY = float(SizeY) - 1.0f;
			float MaxY = 0.0f;
			for (int32 F = 0; F < FaceCount; ++F)
			{
				int32 Index0 = pIndices[F*3 + 0];
				int32 Index1 = pIndices[F*3 + 1];
				int32 Index2 = pIndices[F*3 + 2];

				// A bit ugly, probably can be improved if integrated in above loops and made more precise
				// inside the pixel processor to account for masked out pixels?
				if (!Scratch->CulledVertex[Index0] ||
					!Scratch->CulledVertex[Index1] ||
					!Scratch->CulledVertex[Index2])
				{
					MinY = FMath::Min(MinY, Scratch->Vertices[Index0].y);
					MinY = FMath::Min(MinY, Scratch->Vertices[Index1].y);
					MinY = FMath::Min(MinY, Scratch->Vertices[Index2].y);
					MaxY = FMath::Max(MaxY, Scratch->Vertices[Index0].y);
					MaxY = FMath::Max(MaxY, Scratch->Vertices[Index1].y);
					MaxY = FMath::Max(MaxY, Scratch->Vertices[Index2].y);
				}
			}

			pImage->Flags |= FImage::IF_HAS_RELEVANCY_MAP;
			pImage->RelevancyMaxY = uint16(FMath::Min( int32(FMath::CeilToFloat(MaxY)), SizeY - 1));
			pImage->RelevancyMinY = FMath::Min(uint16(FMath::FloorToFloat(MinY)), pImage->RelevancyMaxY);
		}
	}


	//-------------------------------------------------------------------------------------------------
	FORCENOINLINE void ImageRasterProjected_OptimisedWrapping(const FMesh* pMesh, FImage* pImage, 
		TTriangleRasterPixelProcRefType<4> PixelProc, 
		float FadeEnd,
		uint64 BlockId,
		UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
		FScratchImageProject* Scratch)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageRasterProjected_OptimisedWrapping);

		if (!pMesh || !pMesh->GetFaceCount())
		{
			return;
		}

		EImageFormat Format = pImage->GetFormat();
		int32 PixelSize = GetImageFormatData(Format).BytesPerBlock;

		int32 SizeX = pImage->GetSizeX();
		int32 SizeY = pImage->GetSizeY();

		// Get the vertices
		int32 VertexCount = pMesh->GetVertexCount();

		check( Scratch->Vertices.Num() == VertexCount );
		check( Scratch->CulledVertex.Num() == VertexCount );

		check(pMesh->GetVertexBuffers().GetElementSize(0) == sizeof(FOptimizedVertex));
		const FOptimizedVertex* pVertices = reinterpret_cast<const FOptimizedVertex*>(pMesh->GetVertexBuffers().GetBufferData(0));

		float FadeEndCos = FMath::Cos(FadeEnd);

		// Calculate the culled flag
		EMeshBufferFormat LayoutBlockType = pMesh->VertexBuffers.Buffers[1].Channels[0].Format;
		switch (LayoutBlockType)
		{
		case EMeshBufferFormat::UInt64:
		{
			const uint64* LayoutBlockIds = reinterpret_cast<const uint64*>(pMesh->GetVertexBuffers().GetBufferData(1));
			for (int32 V = 0; V < VertexCount; ++V)
			{
				// Don't cull vertices based on Normal for wrapping projectors. 
				//Scratch->CulledVertex[V] = pVertices[V].Normal[0] < FadeEndCos;
				if (LayoutBlockIds[V] != BlockId)
				{
					Scratch->CulledVertex[V] = true;
				}
			}
			break;
		}

		case EMeshBufferFormat::UInt16:
		{
			const uint16* LayoutBlockIds = reinterpret_cast<const uint16*>(pMesh->GetVertexBuffers().GetBufferData(1));
			for (int32 V = 0; V < VertexCount; ++V)
			{
				//Scratch->CulledVertex[V] = pVertices[V].Normal[0] < FadeEndCos;
				if (LayoutBlockIds[V] != BlockId)
				{
					Scratch->CulledVertex[V] = true;
				}
			}
			break;
		}

		default:
			// Not implemented?
			check(false);
			break;
		}

		for (int32 V = 0; V < VertexCount; ++V)
		{
			bool bUseCropping = UncroppedSize[0] > 0;
			if (bUseCropping)
			{
				Scratch->Vertices[V].x = pVertices[V].Uv[0] * UncroppedSize[0] - CropMin[0];
				Scratch->Vertices[V].y = pVertices[V].Uv[1] * UncroppedSize[1] - CropMin[1];
			}
			else
			{
				Scratch->Vertices[V].x = pVertices[V].Uv[0] * SizeX;
				Scratch->Vertices[V].y = pVertices[V].Uv[1] * SizeY;
			}

			// TODO: No need to copy all. use scratch for the rest only.
			Scratch->Vertices[V].interpolators[0] = pVertices[V].Position[0];
			Scratch->Vertices[V].interpolators[1] = pVertices[V].Position[1];
			Scratch->Vertices[V].interpolators[2] = pVertices[V].Position[2];
			Scratch->Vertices[V].interpolators[3] = pVertices[V].Normal[0];
		}

		// Get the indices
		check(pMesh->GetIndexBuffers().GetElementSize(0) == 4);
		const uint32* pIndices = reinterpret_cast<const uint32*>(pMesh->GetIndexBuffers().GetBufferData(0));

		// The mesh is supposed to contain only the faces in the selected block
		int32 FaceCount = pMesh->GetFaceCount();
		//for (int32 Face = 0; Face < FaceCount; ++Face)
		ParallelFor(FaceCount, [pIndices, Scratch, SizeX, SizeY, PixelSize, pImage, PixelProc](int32 Face)
		{
			int32 Index0 = pIndices[Face*3 + 0];
			int32 Index1 = pIndices[Face*3 + 1];
			int32 Index2 = pIndices[Face*3 + 2];

			// TODO: This optimisation may remove projection in the center of the face, if the angle
			// range is small. Make it optional or more sophisticated (cross product may help).
			if (!Scratch->CulledVertex[Index0] ||
				!Scratch->CulledVertex[Index1] ||
				!Scratch->CulledVertex[Index2])
			{
				constexpr int32 NumInterpolators = 4;
				Triangle<NumInterpolators>(pImage->GetLODData(0), pImage->GetDataSize(),
					SizeX, SizeY,
					PixelSize,
					Scratch->Vertices[Index0], Scratch->Vertices[Index1], Scratch->Vertices[Index2],
					PixelProc,
					false);
			}
		});
	}

	template <EPixelProcessorFeatures Features> 
	FORCENOINLINE void InvokePlanarRasterizerImpl(const FImageRasterInvokeArgs& Args)
	{
		static_assert(EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionPlanar));

		const Private::FProjectedPixelProcessorContext Context = 
			Private::TProjectedPixelProcessor<Features>::MakeContext(
				Args.SourcePtr, Args.ImagePtr->GetLODData(0), Args.MaskPtr ? Args.MaskPtr->GetLODData(0) : nullptr,
				Args.bIsRGBFadingEnabled, Args.bIsAlphaFadingEnabled, Args.FadeStart, Args.FadeEnd, 
				Args.ProjectionAngle, Args.MipInterpolationFactor);

		auto PixelProc = [&Context](uint8* Buffer, float Varying[4])
		{
			Private::TProjectedPixelProcessor<Features>::ProcessPixel(Context, Buffer, Varying);
		};

		ImageRasterProjected_Optimised(Args.MeshPtr, Args.ImagePtr, PixelProc, Args.FadeEnd, Args.CropMin, Args.UncroppedSize, Args.Scratch);
	}

	template <EPixelProcessorFeatures Features> 
	FORCENOINLINE void InvokeCylindricalRasterizerImpl(const FImageRasterInvokeArgs& Args)
	{
		static_assert(EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionCylindrical));

		const Private::FProjectedPixelProcessorContext Context = 
			Private::TProjectedPixelProcessor<Features>::MakeContext(
				Args.SourcePtr, Args.ImagePtr->GetLODData(0), Args.MaskPtr ? Args.MaskPtr->GetLODData(0) : nullptr,
				Args.bIsRGBFadingEnabled, Args.bIsAlphaFadingEnabled, Args.FadeStart, Args.FadeEnd, 
				Args.ProjectionAngle, Args.MipInterpolationFactor);

		auto PixelProc = [&Context](uint8* Buffer, float Varying[4])
		{
			Private::TProjectedPixelProcessor<Features>::ProcessPixel(Context, Buffer, Varying);
		};

		ImageRasterProjected_Optimised(Args.MeshPtr, Args.ImagePtr, PixelProc, Args.FadeEnd, Args.CropMin, Args.UncroppedSize, Args.Scratch);
	}

	template <EPixelProcessorFeatures Features> 
	FORCENOINLINE void InvokeWrappingRasterizerImpl(const FImageRasterInvokeArgs& Args)
	{
		static_assert(EnumHasAnyFlags(Features, EPixelProcessorFeatures::ProjectionWrap));

		const Private::FProjectedPixelProcessorContext Context = 
			Private::TProjectedPixelProcessor<Features>::MakeContext(
				Args.SourcePtr, Args.ImagePtr->GetLODData(0), Args.MaskPtr ? Args.MaskPtr->GetLODData(0) : nullptr,
				Args.bIsRGBFadingEnabled, Args.bIsAlphaFadingEnabled, Args.FadeStart, Args.FadeEnd, 
				Args.ProjectionAngle, Args.MipInterpolationFactor);

		auto PixelProc = [&Context](uint8* Buffer, float Varying[4])
		{
			Private::TProjectedPixelProcessor<Features>::ProcessPixel(Context, Buffer, Varying);
		};

		ImageRasterProjected_OptimisedWrapping(Args.MeshPtr, Args.ImagePtr, PixelProc, Args.FadeEnd, Args.BlockId, Args.CropMin, Args.UncroppedSize, Args.Scratch);
	}
} // namespace Private



//-------------------------------------------------------------------------------------------------
void ImageRasterProjectedPlanar(const FMesh* pMesh, FImage* pImage,
	const FImage* pSource,
	const FImage* pMask,
	bool bIsRGBFadingEnabled, bool bIsAlphaFadingEnabled,
	ESamplingMethod SamplingMethod,
	float FadeStart, float FadeEnd, float MipInterpolationFactor,
	int32 Layout, uint64 BlockId,
	UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
	FScratchImageProject* Scratch, bool bUseVectorImplementation)
{
	using namespace UE::Mutable::Private::Private;

    check(!pMask || pMask->GetSizeX() == pImage->GetSizeX());
    check(!pMask || pMask->GetSizeY() == pImage->GetSizeY());
    check(!pMask || pMask->GetFormat()==EImageFormat::L_UByte);

	MUTABLE_CPUPROFILER_SCOPE(ImageProject);

	EPixelProcessorFeatures ProcessorFeatures = EPixelProcessorFeatures::ProjectionPlanar;

	// Disable vectorized implementation if Point sampling.
	if (bUseVectorImplementation && SamplingMethod != ESamplingMethod::Point)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::VectorizedImpl;
	}

	if (pSource->GetFormat() == EImageFormat::L_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatL;
	}
	else if (pSource->GetFormat() == EImageFormat::RGB_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatRGB;
	}
	else if (pSource->GetFormat() == EImageFormat::RGBA_UByte || pSource->GetFormat() == EImageFormat::BGRA_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatRGBA;
	}

	if (SamplingMethod == ESamplingMethod::Point)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::SamplingPoint;
	}
	else if (SamplingMethod == ESamplingMethod::BiLinear)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::SamplingLinear;
	}

	if (pMask)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::WithMask;
	}

	if (EnumHasAnyFlags(pMesh->Flags, EMeshFlags::ProjectFormat))
	{
		float UnusedProjectionAngle = 0;
		FImageRasterInvokeArgs RasterArgs = 
		{
			pMesh, pImage, pSource, pMask, 
			bIsRGBFadingEnabled, bIsAlphaFadingEnabled,
			SamplingMethod,
			FadeStart, FadeEnd,  UnusedProjectionAngle, MipInterpolationFactor, 
			Layout, BlockId, CropMin, UncroppedSize, Scratch
		};

		using EPPF = EPixelProcessorFeatures;

		// This contains all the permutations, this is way more than what it was supported before
		// but maybe not needed. Mask permutations can be removed safely
		switch (ProcessorFeatures)
		{
		case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
	
		//case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}

		case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}

		//case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}

		case (EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatL | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokePlanarRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionPlanar | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}

		default:
		{
			checkf(false, TEXT("Planar raster pixel processor permutation not implemented."));
		}
		};
	}

    else
    {
        check(false);
    }

}


//-------------------------------------------------------------------------------------------------
void ImageRasterProjectedWrapping( const FMesh* pMesh, FImage* pImage,
	const FImage* pSource, const FImage* pMask,
	bool bIsRGBFadingEnabled, bool bIsAlphaFadingEnabled,
	ESamplingMethod SamplingMethod,
	float FadeStart, float FadeEnd, float MipInterpolationFactor,
	int32 Layout, uint64 BlockId,
	UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
	FScratchImageProject* Scratch, bool bUseVectorImplementation)
{

	using namespace UE::Mutable::Private::Private;
    
	check(!pMask || pMask->GetSizeX() == pImage->GetSizeX());
    check(!pMask || pMask->GetSizeY() == pImage->GetSizeY());
    check(!pMask || pMask->GetFormat()==EImageFormat::L_UByte);

	MUTABLE_CPUPROFILER_SCOPE(ImageProjectWrapping);

	EPixelProcessorFeatures ProcessorFeatures = EPixelProcessorFeatures::ProjectionWrap;

	if (bUseVectorImplementation && SamplingMethod != ESamplingMethod::Point)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::VectorizedImpl;
	}

	if (pSource->GetFormat() == EImageFormat::L_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatL;
	}
	else if (pSource->GetFormat() == EImageFormat::RGB_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatRGB;
	}
	else if (pSource->GetFormat() == EImageFormat::RGBA_UByte || pSource->GetFormat() == EImageFormat::BGRA_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatRGBA;
	}

	if (SamplingMethod == ESamplingMethod::Point)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::SamplingPoint;
	}
	else if (SamplingMethod == ESamplingMethod::BiLinear)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::SamplingLinear;
	}

	if (pMask)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::WithMask;
	}

	if (EnumHasAnyFlags(pMesh->Flags, EMeshFlags::ProjectWrappingFormat))
    {
		float UnusedProjectionAngle = 0;

		FImageRasterInvokeArgs RasterArgs =
		{
			pMesh, pImage, pSource, pMask,
			bIsRGBFadingEnabled, bIsAlphaFadingEnabled,
			SamplingMethod,
			FadeStart, FadeEnd, UnusedProjectionAngle, MipInterpolationFactor,
			Layout, BlockId, CropMin, UncroppedSize, Scratch
		};

		using EPPF = EPixelProcessorFeatures;

		// This contains all the permutations, this is way more than what it was supported before
		// but maybe not needed. Mask permutations can be removed safely
		switch (ProcessorFeatures)
		{
		case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
	
		//case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}

		case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}

		//case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}

		case (EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatL | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokeWrappingRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionWrap | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		
		default:
		{
			checkf(false, TEXT("Wrapping projection raster pixel processor permutation not implemented."));
		}
		};
	}
    else
    {
        check(false);
    }

}

//-------------------------------------------------------------------------------------------------
void ImageRasterProjectedCylindrical( const FMesh* pMesh, FImage* pImage,
	const FImage* pSource, const FImage* pMask,
	bool bIsRGBFadingEnabled, bool bIsAlphaFadingEnabled,
	ESamplingMethod SamplingMethod,
	float FadeStart, float FadeEnd, float MipInterpolationFactor,
	int32 /*layout*/,
	float ProjectionAngle,
	UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize,
	FScratchImageProject* Scratch, bool bUseVectorImplementation)
{

	using namespace UE::Mutable::Private::Private;

    check(!pMask || pMask->GetSizeX() == pImage->GetSizeX());
    check(!pMask || pMask->GetSizeY() == pImage->GetSizeY());
    check(!pMask || pMask->GetFormat()==EImageFormat::L_UByte);

	MUTABLE_CPUPROFILER_SCOPE(ImageProjectCylindrical);

	EPixelProcessorFeatures ProcessorFeatures = EPixelProcessorFeatures::ProjectionCylindrical;

	if (bUseVectorImplementation && SamplingMethod != ESamplingMethod::Point)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::VectorizedImpl;
	}

	if (pSource->GetFormat() == EImageFormat::L_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatL;
	}
	else if (pSource->GetFormat() == EImageFormat::RGB_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatRGB;
	}
	else if (pSource->GetFormat() == EImageFormat::RGBA_UByte || pSource->GetFormat() == EImageFormat::BGRA_UByte)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::FormatRGBA;
	}

	if (SamplingMethod == ESamplingMethod::Point)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::SamplingPoint;
	}
	else if (SamplingMethod == ESamplingMethod::BiLinear)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::SamplingLinear;
	}

	if (pMask)
	{
		ProcessorFeatures |= EPixelProcessorFeatures::WithMask;
	}

	if (EnumHasAnyFlags(pMesh->Flags, EMeshFlags::ProjectFormat))
	{
		int32 UnusedLayout = 0;
		int32 UnusedBlock = 0;

		FImageRasterInvokeArgs RasterArgs =
		{
			pMesh, pImage, pSource, pMask, 
			bIsRGBFadingEnabled, bIsAlphaFadingEnabled,
			SamplingMethod,
			FadeStart, FadeEnd, ProjectionAngle, MipInterpolationFactor, 
			UnusedLayout, UnusedBlock, CropMin, UncroppedSize, Scratch
		};

		using EPPF = EPixelProcessorFeatures;

		// This contains all the permutations, this is way more than what it was supported before
		// but maybe not needed. Mask permutations can be removed safely
		switch (ProcessorFeatures)
		{
		case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint>(RasterArgs);
			break;
		}

		case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
	
		//case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl):
		//{
		//	InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}

		case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::VectorizedImpl):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}

		//case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}
		//case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::WithMask | EPPF::VectorizedImpl):
		//{
		//	InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingPoint | EPPF::VectorizedImpl>(RasterArgs);
		//	break;
		//}

		case (EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatL | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGB | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask>(RasterArgs);
			break;
		}
		case (EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl):
		{
			InvokeCylindricalRasterizerImpl<EPPF::FormatRGBA | EPPF::ProjectionCylindrical | EPPF::SamplingLinear | EPPF::WithMask | EPPF::VectorizedImpl>(RasterArgs);
			break;
		}
		default:
		{
			checkf(false, TEXT("Cylindrical projection raster pixel processor permutation not implemented."));
		}
		}
    }
    else
    {
        check(false);
    }
}

float ComputeProjectedFootprintBestMip(
	const FMesh* pMesh, const FProjector& Projector, const FVector2f& TargetSize, const FVector2f& SourceSize)
{
	// Compute projected mesh footprint on the traget image and extract a optimal mip for source image.
	const bool bIsPlanarProjection = Projector.type == EProjectorType::Planar;
	const bool bIsCylindicalProjection = Projector.type == EProjectorType::Cylindrical;
	const bool bIsWrappingProjection = Projector.type == EProjectorType::Wrapping;

	check(pMesh->GetIndexBuffers().GetElementSize(0) == sizeof(uint32));
	const uint32* IndicesPtr = reinterpret_cast<const uint32*>(pMesh->GetIndexBuffers().GetBufferData(0));

	const uint32 NumIndices = pMesh->GetIndexBuffers().GetElementCount();

	// Source area is the fraction of source image that is covered by the projected mesh.
	float SourceArea = UE_KINDA_SMALL_NUMBER;
	// Target area is the fraction of the target image that is covered by the projected mesh.
	float TargetArea = UE_KINDA_SMALL_NUMBER;

	auto ComputeTriangleArea = [](const FVector2f& A, const FVector2f& B, const FVector2f& C) -> float
	{
		return (A.X * (B.Y - C.Y) + B.X * (C.Y - A.Y) + C.X * (A.Y - B.Y)) * 0.5f;
	};

	check(NumIndices % 3 == 0);
	
	// TODO: evaluate other strategies to get a better mip estimate. Maybe use the median of area change?
	if (bIsPlanarProjection)
	{
		check(pMesh->GetVertexBuffers().GetElementSize(0) == sizeof(FOptimizedVertex));
		const FOptimizedVertex* VerticesPtr = 
			reinterpret_cast<const FOptimizedVertex*>(pMesh->GetVertexBuffers().GetBufferData(0));

		check(NumIndices % 3 == 0);
		for (uint32 I = 0; I < NumIndices; I += 3)
		{
			const FOptimizedVertex& A = VerticesPtr[IndicesPtr[I + 0]];
			const FOptimizedVertex& B = VerticesPtr[IndicesPtr[I + 1]];
			const FOptimizedVertex& C = VerticesPtr[IndicesPtr[I + 2]];

			const float TriangleSourceArea = ComputeTriangleArea(
				FVector2f(A.Position.X, A.Position.Y), FVector2f(B.Position.X, B.Position.Y), FVector2f(C.Position.X, C.Position.Y));

			const float TriangleTargetArea = ComputeTriangleArea(A.Uv, B.Uv, C.Uv);

			// Set weight to zero if source or target area are close to zero to remove outliers.
			float TriangleWeight = static_cast<float>(!FMath::IsNearlyZero(TriangleSourceArea)) *
								   static_cast<float>(!FMath::IsNearlyZero(TriangleTargetArea));
		
			TriangleWeight *= FMath::Clamp((A.Normal.X + B.Normal.X + C.Normal.X) * (1.0f / 3.0f), 0.0f, 1.0f);

			SourceArea += TriangleSourceArea * TriangleWeight;
			TargetArea += TriangleTargetArea * TriangleWeight;
		}
	}
	else if (bIsCylindicalProjection)
	{
		check(pMesh->GetVertexBuffers().GetElementSize(0) == sizeof(FOptimizedVertex));
		const FOptimizedVertex* VerticesPtr = 
			reinterpret_cast<const FOptimizedVertex*>(pMesh->GetVertexBuffers().GetBufferData(0));

		check(NumIndices % 3 == 0);
		for (uint32 I = 0; I < NumIndices; I += 3)
		{
			const FOptimizedVertex& A = VerticesPtr[IndicesPtr[I + 0]];
			const FOptimizedVertex& B = VerticesPtr[IndicesPtr[I + 1]];
			const FOptimizedVertex& C = VerticesPtr[IndicesPtr[I + 2]];

			const float ClampedProjectorAngleRad = FMath::Clamp(Projector.projectionAngle, 0.0f, UE_TWO_PI);
			const float OneOverProjectionAngleSafe = !FMath::IsNearlyZero(ClampedProjectorAngleRad) 
				? 1.0f / Projector.projectionAngle
				: 1.0f / KINDA_SMALL_NUMBER;

			const FVector2f PosA = FVector2f((FMath::Atan2(A.Position.Z, -A.Position.Y) + UE_PI) * OneOverProjectionAngleSafe, A.Position.X);
			const FVector2f PosB = FVector2f((FMath::Atan2(B.Position.Z, -B.Position.Y) + UE_PI) * OneOverProjectionAngleSafe, B.Position.X);
			const FVector2f PosC = FVector2f((FMath::Atan2(C.Position.Z, -C.Position.Y) + UE_PI) * OneOverProjectionAngleSafe, C.Position.X);

			const float TriangleSourceArea = FMath::Abs(ComputeTriangleArea(PosA, PosB, PosC));
			const float TriangleTargetArea = FMath::Abs(ComputeTriangleArea(A.Uv, B.Uv, C.Uv));

			// Set weight to zero if source or target area are close to zero to remove outliers.
			float TriangleWeight = static_cast<float>(!FMath::IsNearlyZero(TriangleSourceArea)) *
								   static_cast<float>(!FMath::IsNearlyZero(TriangleTargetArea));

			SourceArea += TriangleSourceArea * TriangleWeight;
			TargetArea += TriangleTargetArea * TriangleWeight;
		}
	}
	else if (bIsWrappingProjection)
	{
		check(pMesh->GetVertexBuffers().GetElementSize(0) == sizeof(FOptimizedVertex));
		const FOptimizedVertex* VerticesPtr =  reinterpret_cast<const FOptimizedVertex*>(pMesh->GetVertexBuffers().GetBufferData(0));

		check(NumIndices % 3 == 0);
		for (uint32 I = 0; I < NumIndices; I += 3)
		{
			const FOptimizedVertex& A = VerticesPtr[IndicesPtr[I + 0]];
			const FOptimizedVertex& B = VerticesPtr[IndicesPtr[I + 1]];
			const FOptimizedVertex& C = VerticesPtr[IndicesPtr[I + 2]];

			TargetArea += ComputeTriangleArea(A.Uv, B.Uv, C.Uv);
		}

		SourceArea = TargetArea;
	}
	else
	{
		check(false);
	}

	const float SourceFootprintAreaOnTarget = TargetArea * (1.0f / SourceArea);
	const float SourceFootprintAreaOnTargetInPixelsSquared = 
		SourceFootprintAreaOnTarget * (TargetSize.X * TargetSize.Y);

	// Find the mip that better adapts to the footprint.
	float BestMip = FMath::Log2(FMath::Max(1.0f, SourceSize.X * SourceSize.Y)) * 0.5f - 
				    FMath::Log2(FMath::Max(1.0f, SourceFootprintAreaOnTargetInPixelsSquared)) * 0.5f;

	return BestMip;
}

//#define DEBUG_PROJECTION 1

#ifdef DEBUG_PROJECTION
UE_DISABLE_OPTIMIZATION

#undef assert
int32* assert_aux = 0;
#define assert(x) if((x) == 0) assert_aux[0] = 1;
#endif

constexpr float vert_collapse_eps = 0.0001f;


//---------------------------------------------------------------------------------------------
//! Create a map from vertices into vertices, collapsing vertices that have the same position
//---------------------------------------------------------------------------------------------
inline void MeshCreateCollapsedVertexMap( TConstArrayView<FVector3f> Positions,
	TArray<int32>& CollapsedVertices,
	TMultiMap<int32, int32>& CollapsedVerticesMap)
{
	MUTABLE_CPUPROFILER_SCOPE(CreateCollapseMap);

	const int32 NumVertices = Positions.Num();

	// Used to speed up vertex comparison
	UE::Geometry::TPointHashGrid3f<int32> VertHash(0.01f, INDEX_NONE);
	VertHash.Reserve(NumVertices);

	// Info to collect: collapsed vertices and the unique-vertex to nearby-vertices map.
	CollapsedVertices.Init(INDEX_NONE, NumVertices);
	CollapsedVerticesMap.Reserve(NumVertices);

	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		VertHash.InsertPointUnsafe(VertexIndex, Positions[VertexIndex]);
	}


	// Find unique vertices
	TArray<int32> NearbyVertices;
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		if (CollapsedVertices[VertexIndex] != INDEX_NONE)
		{
			continue;
		}

		const FVector3f& Vertex = Positions[VertexIndex];

		NearbyVertices.Reset();
		VertHash.FindPointsInBall(Vertex, vert_collapse_eps,
			[&Vertex, &Positions](const int32& Other) -> float {return FVector3f::DistSquared(Positions[Other], Vertex); },
			NearbyVertices);

		for (int32 NearbyVertexIndex : NearbyVertices)
		{
			CollapsedVertices[NearbyVertexIndex] = VertexIndex;
			CollapsedVerticesMap.Add(VertexIndex, NearbyVertexIndex);
		}
	}
}


struct AdjacentFaces
{
	int32 faces[3];
	int32 newVertices[3];
	bool changesUVIsland[3];

	AdjacentFaces()
	{
		faces[0] = -1;
		faces[1] = -1;
		faces[2] = -1;

		newVertices[0] = -1;
		newVertices[1] = -1;
		newVertices[2] = -1;

		changesUVIsland[0] = false;
		changesUVIsland[1] = false;
		changesUVIsland[2] = false;
	}

	void addConnectedFace(int32 newConnectedFace, int32 newVertex, bool changesUVIslandParam)
	{
#ifdef DEBUG_PROJECTION
		assert( newConnectedFace >= 0 );
#endif

		for (int32 i = 0; i < 3; ++i)
		{
			if (faces[i] == newConnectedFace)
			{
				return;
			}
		}

		for (int32 i = 0; i < 3; ++i)
		{
			if (faces[i] == -1)
			{
				faces[i] = newConnectedFace;
				newVertices[i] = newVertex;
				changesUVIsland[i] = changesUVIslandParam;
				break;
			}
		}
	}
};


void PlanarlyProjectVertex(const FVector3f& unfoldedPosition, FVector4f& projectedPosition,
	const FProjector& projector, const FVector3f& projectorPosition, const FVector3f& projectorDirection, const FVector3f& s, const FVector3f& u)
{
	float x = FVector3f::DotProduct( unfoldedPosition - projectorPosition, s ) / projector.scale[0] + 0.5f;
	float y = FVector3f::DotProduct( unfoldedPosition - projectorPosition, u ) / projector.scale[1] + 0.5f;
	y = 1.0f - y;
	float z = FVector3f::DotProduct( unfoldedPosition - projectorPosition, projectorDirection ) / projector.scale[2];

	bool inside = x>=0.0f && x<=1.0f && y>=0.0f && y<=1.0 && z>=0.0f && z<=1.0f;
	projectedPosition[0] = x;
	projectedPosition[1] = y;
	projectedPosition[2] = z;
	projectedPosition[3] = inside ? 1.0f : 0.0f;
}


FVector2f ChangeBase2D(const FVector2f& origPosition, const FVector2f& newOrigin, const FVector2f& newBaseX, const FVector2f& newBaseY)
{
	float x = FVector2f::DotProduct(origPosition - newOrigin, newBaseX) / newBaseX.SquaredLength() + 0.5f;
	float y = FVector2f::DotProduct(origPosition - newOrigin, newBaseY) / newBaseY.SquaredLength() + 0.5f;
	//y = 1.0f - y;

	return FVector2f(x, y);
}


// Compute barycentric coordinates (u, v, w) for
// point p with respect to triangle (a, b, c)
void GetBarycentricCoords(FVector3f p, FVector3f a, FVector3f b, FVector3f c, float &u, float &v, float &w)
{
    FVector3f v0 = b - a, v1 = c - a, v2 = p - a;
    float d00 = FVector3f::DotProduct(v0, v0);
    float d01 = FVector3f::DotProduct(v0, v1);
    float d11 = FVector3f::DotProduct(v1, v1);
    float d20 = FVector3f::DotProduct(v2, v0);
    float d21 = FVector3f::DotProduct(v2, v1);
    float denom = d00 * d11 - d01 * d01;
    v = (d11 * d20 - d01 * d21) / denom;
    w = (d00 * d21 - d01 * d20) / denom;
    u = 1.0f - v - w;
}

void GetBarycentricCoords(FVector2f p, FVector2f a, FVector2f b, FVector2f c, float &u, float &v, float &w)
{
	FVector2f v0 = b - a, v1 = c - a, v2 = p - a;
    float den = v0.X * v1.Y - v1.X * v0.Y;
#ifdef DEBUG_PROJECTION
	assert(den != 0.f);
#endif
    v = (v2.X * v1.Y - v1.X * v2.Y) / den;
    w = (v0.X * v2.Y - v2.X * v0.Y) / den;
    u = 1.0f - v - w;
}


float getTriangleArea(FVector2f& a, FVector2f& b, FVector2f& c)
{
	float area = a.X * (b.Y - c.Y) + b.X * (c.Y - a.Y) + c.X * (a.Y - b.Y);
	return area / 2.f;
}


float getTriangleRatio(const FVector2f& a, const FVector2f& b, const FVector2f& c)
{
	float lenSide1 = (b - a).Length();
	float lenSide2 = (c - a).Length();
	float lenSide3 = (b - c).Length();

	float maxLen = FMath::Max3(lenSide1, lenSide2, lenSide3);
	float minLen = FMath::Min3(lenSide1, lenSide2, lenSide3);

	return maxLen / minLen;
}

struct NeighborFace
{
	int32 neighborFace;
	int32 newVertex;
	int32 previousFace;
	int32 numUVIslandChanges = 0;
	int32 step;
	bool changesUVIsland;

	friend bool operator <(const NeighborFace& a, const NeighborFace& b)
	{
		//if (a.numUVIslandChanges == b.numUVIslandChanges)
		//{
			return a.step < b.step;
		//}

		//return a.numUVIslandChanges > b.numUVIslandChanges;
	}
};


void getEdgeHorizontalLength( int32 edgeVert0, int32 edgeVert1, int32 opposedVert,
                              TConstArrayView<FVector3f> Positions, TConstArrayView<FVector2f> UVs,
                              float &out_uvSpaceLen, float &out_objSpaceLen,
                              float& out_midEdgePointFraction)
{
	const int32 oldVertices[2] = { edgeVert0, edgeVert1 };

	const FVector2f& edge0_oldUVSpace = UVs[oldVertices[0]];
	const FVector2f& edge1_oldUVSpace = UVs[oldVertices[1]];
	const FVector2f& opposedVert_oldUVSpace = UVs[opposedVert];

	FVector2f edgeVector_oldUVSpace = edge1_oldUVSpace - edge0_oldUVSpace;
	FVector2f sideVector_oldUVSpace = opposedVert_oldUVSpace - edge0_oldUVSpace;

	float edgeVector_oldUVSpaceLen = edgeVector_oldUVSpace.Length();
	float dotEdgeSideVectors_oldUVSpace = FVector2f::DotProduct(edgeVector_oldUVSpace, sideVector_oldUVSpace);
	float midEdgePointFraction_oldUVSpace = (dotEdgeSideVectors_oldUVSpace / powf(edgeVector_oldUVSpaceLen, 2));
	FVector2f edgeVectorProj_oldUVSpace = edgeVector_oldUVSpace * midEdgePointFraction_oldUVSpace;
	FVector2f midEdgePoint_oldUVSpace = edge0_oldUVSpace + edgeVectorProj_oldUVSpace;

	FVector2f perpEdgeVector_oldUVSpace = opposedVert_oldUVSpace - midEdgePoint_oldUVSpace;
#ifdef DEBUG_PROJECTION
	assert(fabs(dot(perpEdgeVector_oldUVSpace, edgeVector_oldUVSpace)) < 0.1f);
#endif
	float perpEdgeVector_oldUVSpace_Len = perpEdgeVector_oldUVSpace.Length();
	out_uvSpaceLen = perpEdgeVector_oldUVSpace_Len;
	out_midEdgePointFraction = midEdgePointFraction_oldUVSpace;

	// Do the same in object space to be able to extract the scale of the original uv space
	const FVector3f& edge0_objSpace = Positions[oldVertices[0]];
	const FVector3f& edge1_objSpace = Positions[oldVertices[1]];
	const FVector3f& opposedVert_objSpace = Positions[opposedVert]; // FIXME

	FVector3f edgeVector_objSpace = edge1_objSpace - edge0_objSpace;
	FVector3f sideVector_objSpace = opposedVert_objSpace - edge0_objSpace;

	float edgeVector_objSpaceLen = edgeVector_objSpace.Length();
	float dotEdgeSideVectors_objSpace = FVector3f::DotProduct(edgeVector_objSpace, sideVector_objSpace);
	float midEdgePointFraction_objSpace = (dotEdgeSideVectors_objSpace / powf(edgeVector_objSpaceLen, 2));
	FVector3f edgeVectorProj_objSpace = edgeVector_objSpace * midEdgePointFraction_objSpace;
	FVector3f midEdgePoint_objSpace = edge0_objSpace + edgeVectorProj_objSpace;

	FVector3f perpEdgeVector_objSpace = opposedVert_objSpace - midEdgePoint_objSpace;
#ifdef DEBUG_PROJECTION
	assert(fabs(dot(perpEdgeVector_objSpace, edgeVector_objSpace)) < 0.1f);
#endif
	float perpEdgeVector_objSpace_Len = perpEdgeVector_objSpace.Length();
	out_objSpaceLen = perpEdgeVector_objSpace_Len;
}


bool testPointsAreInOppositeSidesOfEdge( const FVector2f& pointA, const FVector2f& pointB,
                                         const FVector2f& edge0, const FVector2f& edge1 )
{
	FVector2f edge = edge1 - edge0;
	FVector2f perp_edge = FVector2f(-edge.Y, edge.X);
	FVector2f AVertexVector = pointA - edge0;
	FVector2f BVertexVector = pointB - edge0;
	float dotAVertexVector = FVector2f::DotProduct(AVertexVector, perp_edge);
	float dotBVertexVector = FVector2f::DotProduct(BVertexVector, perp_edge);
	
	return dotAVertexVector * dotBVertexVector < 0.f;
}


//-------------------------------------------------------------------------------------------------
#pragma pack(push,1)
struct PROJECTED_VERTEX
{   
    float pos0 = 0, pos1 = 0, pos2 = 0;
    uint32 mask3 = 0;

	inline FVector2f xy() const { return FVector2f(pos0,pos1); }
};
#pragma pack(pop)
static_assert( sizeof(PROJECTED_VERTEX)==16, "Unexpected struct size" );


//-------------------------------------------------------------------------------------------------
void MeshProject_Optimised_Planar(TConstArrayView<FVector3f> Positions,
                                  const UntypedMeshBufferIteratorConst& NormalIter,
                                  const UntypedMeshBufferIteratorConst& UVIter,
                                  TConstArrayView<uint32> Indices,
                                  const FVector3f& projectorPosition, const FVector3f& projectorDirection,
                                  const FVector3f& projectorSide, const FVector3f& projectorUp,
                                  const FVector3f& projectorScale,
                                  FOptimizedVertex* pResultVertices, int32& currentVertex,
                                  uint32* pResultIndices, int32& currentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshProject_Optimised_Planar)

	const int32 VertexCount = Positions.Num();
	const int32 FaceCount = Indices.Num() / 3;

	TArray<int32> oldToNewVertex;
	oldToNewVertex.Init(INDEX_NONE, VertexCount);

	TArray<PROJECTED_VERTEX> projectedPositions;
	projectedPositions.SetNumZeroed(VertexCount);

	// Runtime meshes use FPackedNormals
	const EMeshBufferFormat NormalFormat = NormalIter.GetFormat();
	const bool bNormalIsPackedNormal =
		NormalFormat == EMeshBufferFormat::PackedDirS8
		|| NormalFormat == EMeshBufferFormat::PackedDirS8_W_TangentSign;

    for ( int32 v=0; v<VertexCount; ++v )
    {
		const FVector3f& Position = Positions[v];

        float x = FVector3f::DotProduct(Position - projectorPosition, projectorSide) / projectorScale[0] + 0.5f;
        float y = FVector3f::DotProduct(Position - projectorPosition, projectorUp) / projectorScale[1] + 0.5f;
        y = 1.0f-y;
        float z = FVector3f::DotProduct(Position - projectorPosition, projectorDirection) / projectorScale[2];

        // Plane mask with bits for each plane discarding the vertex
        uint32 planeMask =
                ((x<0.0f)<<0) |
                ((x>1.0f)<<1) |
                ((y<0.0f)<<2) |
                ((y>1.0f)<<3) |
                ((z<0.0f)<<4) |
                ((z>1.0f)<<5);
        projectedPositions[v].pos0 = x;
        projectedPositions[v].pos1 = y;
        projectedPositions[v].pos2 = z;
        projectedPositions[v].mask3 = planeMask;
    }

    // Iterate the faces
    for ( int32 f=0; f<FaceCount; ++f )
    {
        int32 i0 = Indices[f*3+0];
        int32 i1 = Indices[f*3+1];
        int32 i2 = Indices[f*3+2];

        // Approximate test: discard the triangle if any of the 6 planes entirely discards all the vertices.
        // This will let some triangles through that could be discarded with a precise test.
        bool discarded = (
                projectedPositions[i0].mask3 &
                projectedPositions[i1].mask3 &
                projectedPositions[i2].mask3 )
                !=
                0;

        if ( !discarded )
        {
            // This face is required.
            for (int32 v=0;v<3;++v)
            {
                uint32 i = Indices[f*3+v];
                if (oldToNewVertex[i]<0)
                {
					FOptimizedVertex& OutVertex = pResultVertices[currentVertex];
					OutVertex.Uv = (UVIter + i).GetAsVec2f();
					OutVertex.Position = FVector3f(projectedPositions[i].pos0, projectedPositions[i].pos1, projectedPositions[i].pos2);

                    // Normal is actually the fade factor
					const FVector3f Normal = bNormalIsPackedNormal
						? reinterpret_cast<const FPackedNormal*>((NormalIter + i).ptr())->ToFVector3f()
						: (NormalIter + i).GetAsVec3f();

					const float angleCos = FVector3f::DotProduct(Normal, projectorDirection * -1.0f);
					OutVertex.Normal = FVector3f(angleCos, angleCos, angleCos);

                    oldToNewVertex[i] = currentVertex++;
                }

                pResultIndices[currentIndex++] = oldToNewVertex[i];
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
void MeshProject_Optimised_Cylindrical(TConstArrayView<FVector3f> Positions,
                                       const UntypedMeshBufferIteratorConst& UVIter,
                                       TConstArrayView<uint32> Indices,
                                       const FVector3f& projectorPosition, const FVector3f& projectorDirection,
                                       const FVector3f& projectorSide, const FVector3f& projectorUp,
                                       const FVector3f& projectorScale,
                                       FOptimizedVertex* pResultVertices, int32& currentVertex,
                                       uint32* pResultIndices, int32& currentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshProject_Optimised_Cylindrical)

	const int32 VertexCount = Positions.Num();
	const int32 FaceCount = Indices.Num() / 3;

	TArray<int32> oldToNewVertex;
	oldToNewVertex.Init( -1, VertexCount);

	TArray<PROJECTED_VERTEX> projectedPositions;
	projectedPositions.SetNumUninitialized(VertexCount);

    // TODO: support for non uniform scale?
    float radius = projectorScale[1];
    float height = projectorScale[0];
	FMatrix44f WorldToCylinder(projectorDirection, projectorSide, projectorUp,FVector3f(0,0,0));

	for ( int32 v=0; v<VertexCount; ++v )
    {
        // Cylinder is along the X axis

        // Project
		FVector3f RelPos = Positions[v] - projectorPosition;
		FVector3f VertexPos_Cylinder = WorldToCylinder.InverseTransformPosition(RelPos);

        // This final projection needs to be done per pixel
        float x = VertexPos_Cylinder.X / height;
        float r2 = VertexPos_Cylinder.Y * VertexPos_Cylinder.Y
                + VertexPos_Cylinder.Z * VertexPos_Cylinder.Z;
        projectedPositions[v].pos0 = x;
        projectedPositions[v].pos1 = VertexPos_Cylinder.Y / radius;
        projectedPositions[v].pos2 = VertexPos_Cylinder.Z / radius;
        uint32 planeMask =
                ((x<0.0f)<<0) |
                ((x>1.0f)<<1) |
                ((r2>=(radius*radius))<<2);
        projectedPositions[v].mask3 = planeMask;
    }

    // Iterate the faces
    for ( int32 f=0; f<FaceCount; ++f )
    {
        int32 i0 = Indices[f*3+0];
        int32 i1 = Indices[f*3+1];
        int32 i2 = Indices[f*3+2];

        // Approximate test: discard the triangle if any of the 3 "planes" (top, bottom and radius)
        // entirely discards all the vertices.
        // This will let some triangles through that could be discarded with a precise test.
        // Also, some big triangles can be wrongly discarded by the radius test, since it is not
        // really a plane.
        bool discarded = (
                projectedPositions[i0].mask3 &
                projectedPositions[i1].mask3 &
                projectedPositions[i2].mask3 )
                !=
                0;

        if ( !discarded )
        {
            // This face is required.
            for (int32 v=0;v<3;++v)
            {
                uint32 i = Indices[f*3+v];
                if (oldToNewVertex[i]<0)
                {
					FOptimizedVertex& OutVertex = pResultVertices[currentVertex];

					OutVertex.Uv = (UVIter + i).GetAsVec2f();
					OutVertex.Position = FVector3f(projectedPositions[i].pos0, projectedPositions[i].pos1, projectedPositions[i].pos2);

                    // Normal is actually the fade factor
                    //vec3 vertexCylinderDirection =
                    float angleCos = 1.0f; //dot( Normal, vertexCylinderDirection * -1.0f );
					OutVertex.Normal = FVector3f(angleCos, angleCos, angleCos);

                    oldToNewVertex[i] = currentVertex++;
                }

                pResultIndices[currentIndex++] = oldToNewVertex[i];
            }
        }
    }
}

//-------------------------------------------------------------------------------------------------
void MeshProject_Optimised_Wrapping(const FMesh* pMesh, int32 LayoutIndex,
                                    TConstArrayView<FVector3f> Positions, TConstArrayView<uint32> Indices,
                                    const FVector3f& projectorPosition, const FVector3f& projectorDirection,
                                    const FVector3f& projectorSide, const FVector3f& projectorUp,
                                    const FVector3f& projectorScale,
                                    FOptimizedVertex* pResultVertices, uint64* pResultLayoutBlockIds, int32& currentVertex,
                                    uint32* pResultIndices, int32& currentIndex)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshProject_Optimised_Wrapping)

	const int32 VertexCount = Positions.Num();
    const int32 FaceCount = Indices.Num() / 3;

	// UVs are converted once from the source TexCoords channel for the layout being projected.
	TArray<FVector2f> UVs;
	UVs.SetNumUninitialized(VertexCount);
	{
		UntypedMeshBufferIteratorConst UvIt(pMesh->GetVertexBuffers(), EMeshBufferSemantic::TexCoords, LayoutIndex);
		for (int32 v = 0; v < VertexCount; ++v)
		{
			UVs[v] = UvIt.GetAsVec2f();
			++UvIt;
		}
	}

	// Per-vertex layout block ids iterator.
	const UntypedMeshBufferIteratorConst LayoutBlockIter(pMesh->GetVertexBuffers(), EMeshBufferSemantic::LayoutBlock, LayoutIndex);
	const EMeshBufferFormat LayoutBlockType = LayoutBlockIter.GetFormat();

	const uint32* pIndices = Indices.GetData();

	TArray<PROJECTED_VERTEX> projectedPositions;
	projectedPositions.SetNum(VertexCount);

    // Iterate the faces and trace a ray to find the origin face of the projection
    const float maxDist = 100000.f; // TODO: This should be proportional to the mesh bounding box size.
    float min_t = maxDist;
    float rayLength = maxDist;
    int32 intersectedFace = -1;
	TBitArray<> processedVertices(false, VertexCount);
    FVector3f projectionPlaneNormal;
    FVector3f out_intersection;

	TArray<AdjacentFaces> faceConnectivity;
	faceConnectivity.SetNum(FaceCount);
    TBitArray<> processedFaces(false, FaceCount);
	TBitArray<> discardedWrapAroundFaces(false, FaceCount);
	TArray<int32> faceStep;
	faceStep.SetNum(FaceCount);

    // Map vertices to the one they are collapsed to because they are very similar, if they aren't collapsed then they are mapped to themselves
    TArray<int32> collapsedVertexMap;
    TMultiMap<int32, int32> collapsedVertsMap; // Maps a collapsed vertex to all the vertices that collapse to it
	MeshCreateCollapsedVertexMap(Positions, collapsedVertexMap, collapsedVertsMap);

    // Used to speed up connectivity building
	TMultiMap<int32, int32> vertToFacesMap;
	vertToFacesMap.Reserve(VertexCount);

    for (int32 f = 0; f < FaceCount; ++f)
    {
        int32 i0 = pIndices[f * 3 + 0];
        int32 i1 = pIndices[f * 3 + 1];
        int32 i2 = pIndices[f * 3 + 2];

        vertToFacesMap.Add(collapsedVertexMap[i0], f);
        vertToFacesMap.Add(collapsedVertexMap[i1], f);
        vertToFacesMap.Add(collapsedVertexMap[i2], f);
    }

	FRay3f Ray(projectorPosition, projectorDirection, false);
	UE::Geometry::FIntrRay3Triangle3f Intersector(Ray, UE::Geometry::FTriangle3f());

    // Trace a ray in the projection direction to find the face that will be projected planarly and be the root of the unfolding
    // Also build face connectivity information
    for (int32 f = 0; f < FaceCount; ++f)
    {
        int32 i0 = pIndices[f * 3 + 0];
        int32 i1 = pIndices[f * 3 + 1];
        int32 i2 = pIndices[f * 3 + 2];

		FVector3f rayStart = projectorPosition;
		FVector3f rayEnd = projectorPosition + projectorDirection * maxDist;

		Intersector.Triangle = UE::Geometry::FTriangle3f(Positions[i0], Positions[i1], Positions[i2]);


        rayLength = (rayEnd - rayStart).Length();

		bool bIntersects = Intersector.Find();
		float t = Intersector.RayParameter;
		FVector3f aux_out_intersection = Ray.PointAt(Intersector.RayParameter);

        if (bIntersects && t < min_t)
        {
            intersectedFace = f;
            min_t = t;
            out_intersection = aux_out_intersection;

            FVector3f v0 = Positions[i0];
			FVector3f v1 = Positions[i1];
			FVector3f v2 = Positions[i2];
            projectionPlaneNormal = FVector3f::CrossProduct(v1 - v0, v2 - v0).GetSafeNormal();
        }

        // Face connectivity info
        for (int32 i = 0; i < 3; ++i)
        {
            int32 v = collapsedVertexMap[pIndices[f * 3 + i]];
            //int32 v = pIndices[f * 3 + i];

			TArray<int32,TInlineAllocator<32>> FoundValues;
			vertToFacesMap.MultiFind(v, FoundValues);
            for (int32 f2 : FoundValues )
            {
                if (f != f2 && f2 >= 0)
                {
                    int32 commonVertices = 0;
                    int32 commonVerticesDifferentIsland = 0;
                    bool commonMask[3] = { false, false, false };

                    for (int32 ii = 0; ii < 3; ++ii)
                    {
                        int32 f_vertex_orig = pIndices[f * 3 + ii];
                        int32 f_vertex = collapsedVertexMap[f_vertex_orig];
                        //int32 f_vertex = pIndices[f * 3 + ii];

                        for (int32 j = 0; j < 3; ++j)
                        {
                            int32 f2_vertex_orig = pIndices[f2 * 3 + j];
                            int32 f2_vertex = collapsedVertexMap[f2_vertex_orig];
                            //int32 f2_vertex = pIndices[f2 * 3 + j];

                            if (f_vertex != -1 && f_vertex == f2_vertex)
                            {
                                commonVertices++;
                                commonMask[j] = true;

                                if (f_vertex_orig != f2_vertex_orig)
                                {
                                    commonVerticesDifferentIsland++;
                                }
                            }
                        }
                    }
#ifdef DEBUG_PROJECTION
                    assert(commonVerticesDifferentIsland <= commonVertices);
#endif
                    if (commonVertices == 2)
                    {
                        int32 newVertex = -1;

                        for (int32 j = 0; j < 3; ++j)
                        {
                            if (!commonMask[j])
                            {
                                newVertex = collapsedVertexMap[pIndices[f2 * 3 + j]];
                                //newVertex = pIndices[f2 * 3 + j];
                                break;
                            }
                        }
#ifdef DEBUG_PROJECTION
                        assert(newVertex >= 0);
#endif
                        faceConnectivity[f].addConnectedFace(f2, newVertex, commonVerticesDifferentIsland == 2);
                    }
                }
            }
        }
    }

    // New projector located perpendicularly up from the hit face
    FProjector projector2;
    projector2.direction[0] = projectionPlaneNormal[0];
    projector2.direction[1] = projectionPlaneNormal[1];
    projector2.direction[2] = projectionPlaneNormal[2];
    FVector3f auxPosition = out_intersection - projectionPlaneNormal * rayLength * min_t;
    //FVector3f auxPosition = FVector3f( projector.position[0], projector.position[1], projector.position[2] );
    projector2.position[0] = auxPosition[0];
    projector2.position[1] = auxPosition[1];
    projector2.position[2] = auxPosition[2];

    FVector3f auxUp = projectorUp;
    float test = fabs(FVector3f::DotProduct(auxUp, projectionPlaneNormal));

    if (test < 0.9f)
    {
        FVector3f auxSide = FVector3f::CrossProduct(projectionPlaneNormal, auxUp).GetSafeNormal();
        auxUp = FVector3f::CrossProduct(auxSide, projectionPlaneNormal);
    }
    else
    {
        auxUp = FVector3f::CrossProduct(projectionPlaneNormal, projectorSide);
    }

    projector2.up[0] = auxUp[0];
    projector2.up[1] = auxUp[1];
    projector2.up[2] = auxUp[2];

    projector2.scale[0] = projectorScale[0];
    projector2.scale[1] = projectorScale[1];
    projector2.scale[2] = projectorScale[2];
    FVector3f projectorDirection2, s2, u2;
    FVector3f projectorPosition2 = projector2.position;
    projector2.GetDirectionSideUp( projectorDirection2, s2, u2 );
    float maxDistSquared = powf(projector2.scale[0], 2.f) + powf(projector2.scale[1], 2.f);

    // Do a BFS walk of the mesh, unfolding a face at each step
    if (intersectedFace >= 0)
    {
        TArray<NeighborFace> pendingFaces; // Queue of pending face + new vertex
		pendingFaces.Reserve(FaceCount/64);
        
		TBitArray<> pendingFacesUnique(false, FaceCount); // Used to quickly check uniqueness in the pendingFaces queue

        //FVector3f hitFaceProjectedNormal;
        bool hitFaceHasPositiveArea = false;

        NeighborFace neighborFace;
        neighborFace.neighborFace = intersectedFace;
        neighborFace.newVertex = -1; // -1 because all the vertices of the face are new
        neighborFace.previousFace = -1;
        neighborFace.numUVIslandChanges = 0;
        neighborFace.step = 0;
        neighborFace.changesUVIsland = false;
        pendingFaces.HeapPush(neighborFace);
        pendingFacesUnique[intersectedFace] = true;

        int32 step = 0;
        float totalUVAreaCovered = 0.f;
        TArray<int32> oldVertices;
        oldVertices.Reserve(3);

        while (!pendingFaces.IsEmpty())
        {
			NeighborFace currentFaceStruct;
			pendingFaces.HeapPop(currentFaceStruct);
            int32 currentFace = currentFaceStruct.neighborFace;
#ifdef DEBUG_PROJECTION
            int32 newVertexFromQueue = currentFaceStruct.newVertex;
#endif
            pendingFacesUnique[currentFace] = false;

            processedFaces[currentFace] = true;
            faceStep[currentFace] = step;
            float currentTriangleArea = -2.f;

            // Process the face's vertices
            if (currentFace == intersectedFace)
            {
#ifdef DEBUG_PROJECTION
                assert(newVertexFromQueue == -1);
#endif

                FVector3f outIntersectionBaricentric;
                int32 i0 = pIndices[currentFace * 3 + 0];
                int32 i1 = pIndices[currentFace * 3 + 1];
                int32 i2 = pIndices[currentFace * 3 + 2];
				FVector3f v3_0 = Positions[i0];
				FVector3f v3_1 = Positions[i1];
				FVector3f v3_2 = Positions[i2];
                GetBarycentricCoords(out_intersection, v3_0, v3_1, v3_2, outIntersectionBaricentric[0], outIntersectionBaricentric[1], outIntersectionBaricentric[2]);
				FVector2f outIntersectionUV = UVs[i0] * outIntersectionBaricentric[0] + UVs[i1] * outIntersectionBaricentric[1] + UVs[i2] * outIntersectionBaricentric[2];

				FVector4f proj4D_v0, proj4D_v1, proj4D_v2;
                PlanarlyProjectVertex(Positions[i0], proj4D_v0, projector2, projectorPosition2, projectorDirection2, s2, u2);
                PlanarlyProjectVertex(Positions[i1], proj4D_v1, projector2, projectorPosition2, projectorDirection2, s2, u2);
                PlanarlyProjectVertex(Positions[i2], proj4D_v2, projector2, projectorPosition2, projectorDirection2, s2, u2);

				FVector2f proj_v0 = FVector2f(proj4D_v0[0], proj4D_v0[1]);
				FVector2f proj_v1 = FVector2f(proj4D_v1[0], proj4D_v1[1]);
				FVector2f proj_v2 = FVector2f(proj4D_v2[0], proj4D_v2[1]);

                float proj_TriangleArea = getTriangleArea(proj_v0, proj_v1, proj_v2);

                FVector3f BaricentricBaseU;
                FVector3f BaricentricBaseV;
                FVector3f BaricentricOrigin;

                GetBarycentricCoords(FVector2f(1.f, 0.f), proj_v0, proj_v1, proj_v2, BaricentricBaseU[0], BaricentricBaseU[1], BaricentricBaseU[2]);
                GetBarycentricCoords(FVector2f(0.f, 1.f), proj_v0, proj_v1, proj_v2, BaricentricBaseV[0], BaricentricBaseV[1], BaricentricBaseV[2]);
                GetBarycentricCoords(FVector2f(0.f, 0.f), proj_v0, proj_v1, proj_v2, BaricentricOrigin[0], BaricentricOrigin[1], BaricentricOrigin[2]);

                FVector2f OrigUVs_BaseU_EndPoint = UVs[i0] * BaricentricBaseU[0] + UVs[i1] * BaricentricBaseU[1] + UVs[i2] * BaricentricBaseU[2];
                FVector2f OrigUVs_BaseV_EndPoint = UVs[i0] * BaricentricBaseV[0] + UVs[i1] * BaricentricBaseV[1] + UVs[i2] * BaricentricBaseV[2];
                FVector2f OrigUVs_Origin = UVs[i0] * BaricentricOrigin[0] + UVs[i1] * BaricentricOrigin[1] + UVs[i2] * BaricentricOrigin[2];

                FVector2f OrigUVs_BaseU = OrigUVs_BaseU_EndPoint - OrigUVs_Origin;
                FVector2f OrigUVs_BaseV_Aux = OrigUVs_BaseV_EndPoint - OrigUVs_Origin;

                OrigUVs_BaseU = OrigUVs_BaseU.GetSafeNormal();
                OrigUVs_BaseV_Aux = OrigUVs_BaseV_Aux.GetSafeNormal();

                // Generate perp vector from OrigUVs_BaseU by flipping coords and one sign. Using OrigUVs_BaseV directly sometimes doesn't give an orthogonal basis
                FVector2f OrigUVs_BaseV = FVector2f(copysign(OrigUVs_BaseU.Y, OrigUVs_BaseV_Aux.X), copysign(OrigUVs_BaseU.X, OrigUVs_BaseV_Aux.Y));

#ifdef DEBUG_PROJECTION
                assert(fabs(dot(OrigUVs_BaseU, OrigUVs_BaseV)) < 0.3f);
#endif

                OrigUVs_Origin = outIntersectionUV;
                GetBarycentricCoords(outIntersectionUV, UVs[i0], UVs[i1], UVs[i2], outIntersectionBaricentric[0], outIntersectionBaricentric[1], outIntersectionBaricentric[2]);

                for (int32 i = 0; i < 3; ++i)
                {
                    int32 v = pIndices[currentFace * 3 + i];
#ifdef DEBUG_PROJECTION
                    assert(v >= 0 && v < VertexCount);
#endif

                    int32 collapsedVert = collapsedVertexMap[v];

                    processedVertices[collapsedVert] = true;

                    //PlanarlyProjectVertex(pVertices[collapsedVert].pos, projectedPositions[collapsedVert], projector2, projectorPosition2, projectorDirection2, s2, u2);
                    FVector2f projectedVertex = ChangeBase2D(UVs[v], OrigUVs_Origin, OrigUVs_BaseU, OrigUVs_BaseV);

                    projectedPositions[collapsedVert].pos0 = projectedVertex[0];
                    projectedPositions[collapsedVert].pos1 = projectedVertex[1];
                    projectedPositions[collapsedVert].pos2 = 0.f;
                    projectedPositions[collapsedVert].mask3 = 1;
                }

                FVector2f v0 = projectedPositions[collapsedVertexMap[pIndices[currentFace * 3 + 0]]].xy();
				FVector2f v1 = projectedPositions[collapsedVertexMap[pIndices[currentFace * 3 + 1]]].xy();
				FVector2f v2 = projectedPositions[collapsedVertexMap[pIndices[currentFace * 3 + 2]]].xy();
                currentTriangleArea = getTriangleArea(v0, v1, v2);
                float triangleAreaFactor = sqrt(fabs(proj_TriangleArea) / fabs(currentTriangleArea));

                FVector2f origin = v0 * outIntersectionBaricentric[0] + v1 * outIntersectionBaricentric[1] + v2 * outIntersectionBaricentric[2];
#ifdef DEBUG_PROJECTION
                assert(length(origin - FVector2f(0.5f, 0.5f)) < 0.001f);
#endif

                //hitFaceProjectedNormal = (cross<float>(v1 - v0, v2 - v0));
                hitFaceHasPositiveArea = currentTriangleArea >= 0.f;

                for (int32 i = 0; i < 3; ++i)
                {
                    int32 v = pIndices[currentFace * 3 + i];
                    int32 collapsedVert = collapsedVertexMap[v];

                    projectedPositions[collapsedVert].pos0 = ((projectedPositions[collapsedVert].pos0 - 0.5f) * triangleAreaFactor) + 0.5f;
                    projectedPositions[collapsedVert].pos1 = ((projectedPositions[collapsedVert].pos1 - 0.5f) * triangleAreaFactor) + 0.5f;
                    projectedPositions[collapsedVert].pos2 = 0.f;
                    projectedPositions[collapsedVert].mask3 = projectedPositions[collapsedVert].pos0 >= 0.0f && projectedPositions[collapsedVert].pos0 <= 1.0f &&
                                                           projectedPositions[collapsedVert].pos1 >= 0.0f && projectedPositions[collapsedVert].pos1 <= 1.0f;

                    // Copy the new info to all the vertices that collapse to the same vertex
					TArray<int32> FoundValues;
					collapsedVertsMap.MultiFind(collapsedVert,FoundValues);
                    for (int32 otherVert:FoundValues)
                    {
                        processedVertices[otherVert] = true;

                        projectedPositions[otherVert] = projectedPositions[collapsedVert];
                    }
                }

                v0 = projectedPositions[collapsedVertexMap[pIndices[currentFace * 3 + 0]]].xy();
                v1 = projectedPositions[collapsedVertexMap[pIndices[currentFace * 3 + 1]]].xy();
                v2 = projectedPositions[collapsedVertexMap[pIndices[currentFace * 3 + 2]]].xy();
                currentTriangleArea = getTriangleArea(v0, v1, v2);
#ifdef DEBUG_PROJECTION
                assert(fabs(proj_TriangleArea - currentTriangleArea) / proj_TriangleArea < 0.1f);
#endif
                origin = v0 * outIntersectionBaricentric[0] + v1 * outIntersectionBaricentric[1] + v2 * outIntersectionBaricentric[2];
#ifdef DEBUG_PROJECTION
                assert(length(origin - FVector2f(0.5f, 0.5f)) < 0.001f);
#endif
            }
            else
            {
                // Unwrap the rest of the faces
#ifdef DEBUG_PROJECTION
                assert(newVertexFromQueue >= 0);
#endif
                int32 newVertex = -1;
                //int32 oldVertices[2] = { -1, -1 };
                oldVertices.Empty();
                bool reverseOrder = false;

                for (int32 i = 0; i < 3; ++i)
                {
                    int32 v = pIndices[currentFace * 3 + i];

                    if (!processedVertices[collapsedVertexMap[v]])
                    {
                        newVertex = v;

                        if (i == 1)
                        {
                            reverseOrder = true;
                        }
                    }
                    else
                    {
                        //int32 index = oldVertices[0] == -1 ? 0 : 1;
                        //oldVertices[index] = v;
                        //oldVertices.push_back(collapsedVertexMap[v]);
                        oldVertices.Add(v);
                    }
                }

                if (reverseOrder && oldVertices.Num() == 2)
                {
                    int32 aux = oldVertices[0];
                    oldVertices[0] = oldVertices[1];
                    oldVertices[1] = aux;
                }

                if (newVertex >= 0)
                {
#ifdef DEBUG_PROJECTION
                    assert(newVertex >= 0);
                    //assert(collapsedVertexMap[newVertex] == newVertexFromQueue);
                    assert(oldVertices[0] >= 0 && oldVertices[1] >= 0);
                    assert(oldVertices.Num() == 2);
                    assert(newVertex < VertexCount && oldVertices[0] < VertexCount && oldVertices[1] < VertexCount);
#endif

                    int32 previousFace = currentFaceStruct.previousFace;
#ifdef DEBUG_PROJECTION
                    assert(previousFace >= 0);
                    assert(processedFaces[previousFace]);
#endif
                    // Previous face indices
                    int32 pi0 = pIndices[previousFace * 3 + 0];
                    int32 pi1 = pIndices[previousFace * 3 + 1];
                    int32 pi2 = pIndices[previousFace * 3 + 2];

                    // Previous face uv coords
					FVector2f pv0;
					FVector2f pv1;
					FVector2f pv2;

                    // Proj vertices from previous triangle
					FVector2f pv0_proj;
					FVector2f pv1_proj;
					FVector2f pv2_proj;

                    int32 oldVertices_previousFace[2] = { -1, -1 };
                    int32 oldVertex = -1;

                    for (int32 i = 0; i < 3; ++i)
                    {
                        int32 v = pIndices[previousFace * 3 + i];
                        int32 collapsed_v = collapsedVertexMap[v];

                        if (collapsed_v == collapsedVertexMap[oldVertices[0]])
                        {
                            oldVertices_previousFace[0] = v;
                        }
                        else if (collapsed_v == collapsedVertexMap[oldVertices[1]])
                        {
                            oldVertices_previousFace[1] = v;
                        }
                        else
                        {
                            oldVertex = v;
                        }
                    }

                    if (currentFaceStruct.changesUVIsland)
                    {
                        // It's crossing a uv island border, so the previous uvs need to be converted to the new islands uv space
#ifdef DEBUG_PROJECTION
                        assert(oldVertex >= 0 && oldVertices_previousFace[0] >= 0 && oldVertices_previousFace[1] >= 0);
                        assert(oldVertex != newVertex && oldVertices_previousFace[0] != oldVertices[0] && oldVertices_previousFace[1] != oldVertices[1]);
                        assert(collapsedVertexMap[oldVertices_previousFace[0]] == collapsedVertexMap[oldVertices[0]]);
                        assert(collapsedVertexMap[oldVertices_previousFace[1]] == collapsedVertexMap[oldVertices[1]]);
#endif
                        // Compute the horizontal lengths in the previous island old (original) uv space, and in obj space
                        float previousFace_uvSpaceLen, previousFace_objSpaceLen, midEdgePointFraction_previousFace;
                        getEdgeHorizontalLength(oldVertices_previousFace[0], oldVertices_previousFace[1], oldVertex, Positions, UVs, previousFace_uvSpaceLen, previousFace_objSpaceLen, midEdgePointFraction_previousFace);

                        float currentFace_uvSpaceLen, currentFace_objSpaceLen, midEdgePointFraction_currentFace;
                        getEdgeHorizontalLength(oldVertices[0], oldVertices[1], newVertex, Positions, UVs, currentFace_uvSpaceLen, currentFace_objSpaceLen, midEdgePointFraction_currentFace);

                        float objSpaceToCurrentUVSpaceConversion = currentFace_uvSpaceLen / currentFace_objSpaceLen;
                        float previousFaceWidthInCurrentUVspace = previousFace_objSpaceLen * objSpaceToCurrentUVSpaceConversion;

                        const FVector2f& edge0_currentFace_oldUVSpace = UVs[oldVertices[0]];
                        const FVector2f& edge1_currentFace_oldUVSpace = UVs[oldVertices[1]];
                        const FVector2f& newVertex_currentFace_oldUVSpace = UVs[newVertex];

                        FVector2f edge_currentFace_oldUVSpace = edge1_currentFace_oldUVSpace - edge0_currentFace_oldUVSpace;
                        FVector2f midpoint_currentFace_oldUVSpace = edge0_currentFace_oldUVSpace + edge_currentFace_oldUVSpace * midEdgePointFraction_currentFace;
                        FVector2f edgePerpVectorNorm = (midpoint_currentFace_oldUVSpace - newVertex_currentFace_oldUVSpace).GetSafeNormal();
                        FVector2f test_edge_currentFace_oldUVSpaceNorm = edge_currentFace_oldUVSpace.GetSafeNormal();
                        (void)test_edge_currentFace_oldUVSpaceNorm;

#ifdef DEBUG_PROJECTION
                        FVector2f sideVector_oldUVSpace = newVertex_currentFace_oldUVSpace - edge0_currentFace_oldUVSpace;
                        float edgeVector_oldUVSpaceLen = length(edge_currentFace_oldUVSpace);
                        float dotEdgeSideVectors_oldUVSpace = dot(edge_currentFace_oldUVSpace, sideVector_oldUVSpace);
                        float midEdgePointFraction_oldUVSpace = (dotEdgeSideVectors_oldUVSpace / powf(edgeVector_oldUVSpaceLen, 2));
                        assert(fabs(midEdgePointFraction_oldUVSpace - midEdgePointFraction_currentFace) < 0.01f);
                        assert(fabs(dot(edgePerpVectorNorm, test_edge_currentFace_oldUVSpaceNorm)) < 0.1f);
#endif

                        FVector2f midpoint_previousFace_oldUVSpace = edge0_currentFace_oldUVSpace + edge_currentFace_oldUVSpace * midEdgePointFraction_previousFace;
                        FVector2f oldVertex_currentFace_oldUVSpace = midpoint_previousFace_oldUVSpace + edgePerpVectorNorm * previousFaceWidthInCurrentUVspace;

                        pv0 = edge0_currentFace_oldUVSpace;
                        pv1 = edge1_currentFace_oldUVSpace;
                        pv2 = oldVertex_currentFace_oldUVSpace;

                        pv0_proj = projectedPositions[collapsedVertexMap[oldVertices[0]]].xy();
                        pv1_proj = projectedPositions[collapsedVertexMap[oldVertices[1]]].xy();
                        pv2_proj = projectedPositions[collapsedVertexMap[oldVertex]].xy();

#ifdef DEBUG_PROJECTION
                        assert(testPointsAreInOppositeSidesOfEdge(oldVertex_currentFace_oldUVSpace, newVertex_currentFace_oldUVSpace, edge0_currentFace_oldUVSpace, edge1_currentFace_oldUVSpace));
#endif
                    }
                    else
                    {
                        // If there's no UV island border, just use the previous uvs
                        pv0 = UVs[pi0];
                        pv1 = UVs[pi1];
                        pv2 = UVs[pi2];

                        pv0_proj = projectedPositions[pi0].xy();
                        pv1_proj = projectedPositions[pi1].xy();
                        pv2_proj = projectedPositions[pi2].xy();
                    }

					FVector2f v2 = UVs[newVertex];

                    // New vertex baricentric coords in respect to old triangle
                    float a, b, c;
                    GetBarycentricCoords(v2, pv0, pv1, pv2, a, b, c);
                    FVector2f newVertex_proj = pv0_proj * a + pv1_proj * b + pv2_proj * c;
#ifdef DEBUG_PROJECTION
                    FVector2f v2Test = pv0 * a + pv1 * b + pv2 * c;
                    float v2TestDist = length(v2 - v2Test);
                    assert(v2TestDist < 0.001f);

                    float a2, b2, c2;
                    GetBarycentricCoords(newVertex_proj, pv0_proj, pv1_proj, pv2_proj, a2, b2, c2);
                    assert(fabs(a - a2) < 0.5f);
                    assert(fabs(b - b2) < 0.5f);
                    assert(fabs(c - c2) < 0.5f);

                    assert(testPointsAreInOppositeSidesOfEdge(projectedPositions[collapsedVertexMap[oldVertex]].xy(), newVertex_proj,
                        projectedPositions[collapsedVertexMap[oldVertices[0]]].xy(), projectedPositions[collapsedVertexMap[oldVertices[1]]].xy()));
#endif

                    int32 collapsedNewVertex = collapsedVertexMap[newVertex];
                    processedVertices[collapsedNewVertex] = true;

                    projectedPositions[collapsedNewVertex].pos0 = newVertex_proj[0];
                    projectedPositions[collapsedNewVertex].pos1 = newVertex_proj[1];
                    projectedPositions[collapsedNewVertex].pos2 = projectedPositions[oldVertices[0]].pos2;
                    projectedPositions[collapsedNewVertex].mask3 = newVertex_proj[0] >= 0.0f && newVertex_proj[0] <= 1.0f &&
                                                                newVertex_proj[1] >= 0.0f && newVertex_proj[1] <= 1.0f;

                    // Copy the new info to all the vertices that collapse to the same vertex
					TArray<int32> FoundValues;
					collapsedVertsMap.MultiFind(collapsedNewVertex,FoundValues);
                    for (int32 otherVert : FoundValues)
                    {
                        processedVertices[otherVert] = true;

                        projectedPositions[otherVert] = projectedPositions[collapsedNewVertex];
                    }

                    //float oldRatio = getTriangleRatio(pVertices[newVertex].uv, pVertices[oldVertices[0]].uv, pVertices[oldVertices[1]].uv);
                    //float newRatio = getTriangleRatio(newVertex_proj, projectedPositions[collapsedVertexMap[oldVertices[0]]].xy(), projectedPositions[collapsedVertexMap[oldVertices[1]]].xy());

                    //if (fabs(newRatio - oldRatio) > 31.f)
                    //{
                    //	break;
                    //}
                }
                else
                {
#ifdef DEBUG_PROJECTION
                    assert(oldVertices.Num() == 3);
#endif
                }

                int32 i0 = collapsedVertexMap[pIndices[currentFace * 3 + 0]];
                int32 i1 = collapsedVertexMap[pIndices[currentFace * 3 + 1]];
                int32 i2 = collapsedVertexMap[pIndices[currentFace * 3 + 2]];

                FVector2f v0 = projectedPositions[i0].xy();
                FVector2f v1 = projectedPositions[i1].xy();
                FVector2f v2 = projectedPositions[i2].xy();
                currentTriangleArea = getTriangleArea(v0, v1, v2);
                bool currentTriangleHasPositiveArea = currentTriangleArea >= 0.f;
                //FVector3f currentFaceNormal = (cross<float>(v1 - v0, v2 - v0));

                //if (hitFaceProjectedNormal.z() * currentFaceNormal.z() < 0)
                if(hitFaceHasPositiveArea != currentTriangleHasPositiveArea) // Is the current face wound in the opposite direction?
                {
                    discardedWrapAroundFaces[currentFace] = true; // If so, discard it since it's probably a wrap-around face
                }
            }

            bool anyVertexInUVSpace = false;

            for (int32 i = 0; i < 3; ++i)
            {
                int32 v = collapsedVertexMap[pIndices[currentFace * 3 + i]];

                if (projectedPositions[v].mask3 == 1) // Is it inside?
                {
                    anyVertexInUVSpace = true;
                    break;
                }
            }

            bool anyVertexInObjSpaceRange = false;

            for (int32 i = 0; i < 3; ++i)
            {
                int32 v = pIndices[currentFace * 3 + i];
                FVector3f r = Positions[v] - out_intersection;
                float squaredDist = FVector3f::DotProduct(r, r);

                if (squaredDist <= maxDistSquared / 4.f) // Is it inside?
                {
                    anyVertexInObjSpaceRange = true;
                    break;
                }
            }

            if (anyVertexInUVSpace && anyVertexInObjSpaceRange && !discardedWrapAroundFaces[currentFace] )
            {
#ifdef DEBUG_PROJECTION
                assert(currentTriangleArea != -2.f);
#endif
                totalUVAreaCovered += fabs(currentTriangleArea);

                for(int32 i = 0; i < 3; ++i)
                {
                    int32 neighborFace2 = faceConnectivity[currentFace].faces[i];
                    int32 newVertex = faceConnectivity[currentFace].newVertices[i];
                    bool changesUVIsland = faceConnectivity[currentFace].changesUVIsland[i];

                    if (neighborFace2 >= 0 && !processedFaces[neighborFace2] && !pendingFacesUnique[neighborFace2])
                    {
                        NeighborFace neighborFaceStruct;
                        neighborFaceStruct.neighborFace = neighborFace2;
                        neighborFaceStruct.newVertex = newVertex;
                        neighborFaceStruct.previousFace = currentFace;
                        neighborFaceStruct.numUVIslandChanges =  currentFaceStruct.changesUVIsland ? currentFaceStruct.numUVIslandChanges + 1
                                                                                                   : currentFaceStruct.numUVIslandChanges;
                        neighborFaceStruct.step = step;
                        neighborFaceStruct.changesUVIsland = changesUVIsland;
                        pendingFaces.Add(neighborFaceStruct);
                        pendingFacesUnique[neighborFace2] = true;
                    }
                }
            }
            else
            {
                discardedWrapAroundFaces[currentFace] = true;
            }

            //if(step == 1000)
            //{
            //	break;
            //}

            if (totalUVAreaCovered > 1.5f)
            {
                break;
            }

            step++;
        }
    }

	TArray<int32> oldToNewVertex;
	oldToNewVertex.Init(-1,VertexCount);

    // Add the projected face
    for (int32 f = processedFaces.FindFrom(true, 0); f != INDEX_NONE; f = processedFaces.FindFrom(true, f + 1))
    {
        if (discardedWrapAroundFaces[f])
        {
            continue;
        }

#ifdef DEBUG_PROJECTION
        int32 i0 = pIndices[f * 3 + 0];
        int32 i1 = pIndices[f * 3 + 1];
        int32 i2 = pIndices[f * 3 + 2];

        assert(processedVertices[i0]);
        assert(processedVertices[i1]);
        assert(processedVertices[i2]);

        assert(i0 >= 0 && i0 < VertexCount);
        assert(i1 >= 0 && i1 < VertexCount);
        assert(i2 >= 0 && i2 < VertexCount);
#endif

        //// TODO: This test is wrong and may fail for big triangles! Do proper triangle-quad culling.
        //if ( projectedPositions[i0][3] > 0.0f ||
        //     projectedPositions[i1][3] > 0.0f ||
        //     projectedPositions[i2][3] > 0.0f )
        {
            // This face is required.
            for (int32 v = 0; v < 3; ++v)
            {
                int32 i = pIndices[f * 3 + v];
#ifdef DEBUG_PROJECTION
                assert(i >= 0 && i < VertexCount);
#endif
                if (oldToNewVertex[i] < 0)
                {
					FOptimizedVertex& OutVertex = pResultVertices[currentVertex];

					OutVertex.Uv = UVs[i];

					// Per-vertex layout block id: UInt64 ids are used as-is, UInt16 (relative) ids are folded
					// with the mesh id prefix. The channel is located by semantic, not a fixed buffer index.
					switch (LayoutBlockType)
					{
					case EMeshBufferFormat::UInt64:
						pResultLayoutBlockIds[currentVertex] = *reinterpret_cast<const uint64*>((LayoutBlockIter + i).ptr());
						break;

					case EMeshBufferFormat::UInt16:
						pResultLayoutBlockIds[currentVertex] = (uint64(pMesh->MeshIDPrefix) << 32) | uint64(*reinterpret_cast<const uint16*>((LayoutBlockIter + i).ptr()));
						break;

					case EMeshBufferFormat::None:
						// The mesh has no per-vertex layout block ids (e.g. an object mixing meshes with and
						// without layouts). The old mesh format filled these with zeros, so do the same.
						pResultLayoutBlockIds[currentVertex] = 0;
						break;

					default:
						// Not implemented?
						check(false);
						break;
					}

                    OutVertex.Position[0] = projectedPositions[i].pos0;
                    OutVertex.Position[1] = projectedPositions[i].pos1;
                    OutVertex.Position[2] = projectedPositions[i].pos2;

                    // Normal is actually the fade factor
                    constexpr float MaxGradient = 10.f;
                    const int32 Step = faceStep[f];
                    float StepGradient = FMath::Min(Step, MaxGradient) / MaxGradient;
                    float FadeFactor = 1.0f - StepGradient;

					OutVertex.Normal[0] = FadeFactor;
                    OutVertex.Normal[1] = FadeFactor;
                    OutVertex.Normal[2] = FadeFactor;

                    oldToNewVertex[i] = currentVertex++;
                }

                pResultIndices[currentIndex++] = oldToNewVertex[i];
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
void MeshProject_Optimised(FMesh* Result, const FMesh* pMesh, const FProjector& projector, int32 LayoutIndex, bool& bOutSuccess)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshProject_Optimised);

	bOutSuccess = true;

    FVector3f projectorPosition,projectorDirection,projectorSide,projectorUp;
    projectorPosition = FVector3f( projector.position[0], projector.position[1], projector.position[2] );
    projector.GetDirectionSideUp( projectorDirection, projectorSide, projectorUp );
    FVector3f projectorScale( projector.scale[0], projector.scale[1], projector.scale[2] );

    // Create with worse case, shrink later.
    const int32 VertexCount = pMesh->GetVertexCount();
    const int32 indexCount = pMesh->GetIndexCount();
    int32 currentVertex = 0;
    int32 currentIndex = 0;

    // The result mesh is always created with the projection-optimised format using layout 0.
    const int32 layout = 0;

	// Read the source mesh directly through its native buffer formats (no projection mesh format applied at
	// compile time). Positions live in a dedicated Float32x3 buffer, so they can be viewed as FVector3f.
	const MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> PositionIter(pMesh->GetVertexBuffers(), EMeshBufferSemantic::Position, 0);
	check(PositionIter.GetElementSize() == sizeof(FVector3f));
	const TConstArrayView<FVector3f> Positions(reinterpret_cast<const FVector3f*>(*PositionIter), VertexCount);

	// Triangle indices: viewed directly when already UInt32, otherwise converted once into an auxiliary array.
	UntypedMeshBufferIteratorConst IndexIter(pMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex, 0);
	TArray<uint32> ConvertedIndices;
	TConstArrayView<uint32> Indices;
	if (IndexIter.GetFormat() == EMeshBufferFormat::UInt32)
	{
		Indices = TConstArrayView<uint32>(reinterpret_cast<const uint32*>(IndexIter.ptr()), indexCount);
	}
	else
	{
		ConvertedIndices.SetNumUninitialized(indexCount);
		for (int32 IndexElement = 0; IndexElement < indexCount; ++IndexElement)
		{
			ConvertedIndices[IndexElement] = IndexIter.GetAsUINT32();
			++IndexIter;
		}
		Indices = ConvertedIndices;
	}

    switch (projector.type)
    {
    case EProjectorType::Planar:
    case EProjectorType::Cylindrical:
    {
        CreateMeshOptimisedForProjection(Result, layout);
        Result->GetVertexBuffers().SetElementCount(VertexCount);
        Result->GetIndexBuffers().SetElementCount(indexCount);
		uint32* pResultIndices = reinterpret_cast<uint32*>(Result->GetIndexBuffers().GetBufferData(0));
		FOptimizedVertex* pResultVertices = reinterpret_cast<FOptimizedVertex*>(Result->GetVertexBuffers().GetBufferData(0));

		// Normal (usually FPackedNormal) and UVs (Float16/Float32) are read through iterators; the FPackedNormal
		// fast path and ConvertData handle the conversions.
		const FMeshBufferSet& SourceVertexBuffers = pMesh->GetVertexBuffers();
		const UntypedMeshBufferIteratorConst NormalIter(SourceVertexBuffers, EMeshBufferSemantic::Normal, 0);
		const UntypedMeshBufferIteratorConst UVIter(SourceVertexBuffers, EMeshBufferSemantic::TexCoords, LayoutIndex);

        if (projector.type==EProjectorType::Planar)
        {
            MeshProject_Optimised_Planar(Positions, NormalIter, UVIter, Indices,
                                          projectorPosition, projectorDirection,
                                          projectorSide, projectorUp,
                                          projectorScale,
                                          pResultVertices, currentVertex,
                                          pResultIndices, currentIndex);
        }

        else
        {
            MeshProject_Optimised_Cylindrical(Positions, UVIter, Indices,
                                              projectorPosition, projectorDirection,
                                              projectorSide, projectorUp,
                                              projectorScale,
                                              pResultVertices, currentVertex,
                                              pResultIndices, currentIndex);
        }

        break;
    }

    case EProjectorType::Wrapping:
	{
        CreateMeshOptimisedForWrappingProjection(Result, layout);

		// Wrapping-projection also needs the layout block IDs
        Result->GetVertexBuffers().SetElementCount(VertexCount);
        Result->GetIndexBuffers().SetElementCount(indexCount);
		uint32* pResultIndices = reinterpret_cast<uint32*>(Result->GetIndexBuffers().GetBufferData(0));
		FOptimizedVertex* pResultVertices = reinterpret_cast<FOptimizedVertex*>(Result->GetVertexBuffers().GetBufferData(0));

		// \TODO: be more flexible with the block id formats?
		uint64* pResultLayoutBlockIds = reinterpret_cast<uint64*>(Result->GetVertexBuffers().GetBufferData(1));

        MeshProject_Optimised_Wrapping(pMesh, LayoutIndex, Positions, Indices,
                                       projectorPosition, projectorDirection,
                                       projectorSide, projectorUp,
                                       projectorScale,
                                       pResultVertices, pResultLayoutBlockIds, currentVertex,
                                       pResultIndices, currentIndex);

        break;
	}

    default:
        // Projector type not implemented.
        check(false);
		bOutSuccess = false;
        break;

    }

    // Shrink result mesh
    Result->GetVertexBuffers().SetElementCount(currentVertex);
    Result->GetIndexBuffers().SetElementCount(currentIndex);
}
#ifdef DEBUG_PROJECTION
UE_ENABLE_OPTIMIZATION
#endif



//-------------------------------------------------------------------------------------------------
void MeshProject(FMesh* Result, const FMesh* pMesh, const FProjector& projector, int32 LayoutIndex, bool& bOutSuccess)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshProject);

    MeshProject_Optimised(Result, pMesh, projector, LayoutIndex, bOutSuccess);

	if (bOutSuccess)
	{
		Result->Surfaces.Empty();
		Result->ClothSections.Empty();
		Result->EnsureSurfaceData();
		Result->ResetStaticFormatFlags();
	}
}



//-------------------------------------------------------------------------------------------------
void CreateMeshOptimisedForProjection(FMesh* Result, int32 layout)
{
    Result->GetVertexBuffers().SetBufferCount( 1 );
    Result->GetIndexBuffers().SetBufferCount( 1 );

    EMeshBufferSemantic semantics[3] =	{ EMeshBufferSemantic::TexCoords,	EMeshBufferSemantic::Position,	EMeshBufferSemantic::Normal };
    int32 semanticIndices[3] =			{ 0,				0,				0 };
    EMeshBufferFormat formats[3] =		{ EMeshBufferFormat::Float32,		EMeshBufferFormat::Float32,	EMeshBufferFormat::Float32 };
    int32 componentCounts[3] =			{ 2,				3,				3 };
    int32 offsets[3] =					{ 0,				8,				20 };
    semanticIndices[0] = layout;
    Result->GetVertexBuffers().SetBuffer
            ( 0, 32, 3,
              semantics, semanticIndices,
              formats, componentCounts,
              offsets );

    EMeshBufferSemantic isemantics[1] =	{ EMeshBufferSemantic::VertexIndex };
    int32 isemanticIndices[1] =			{ 0 };
    EMeshBufferFormat iformats[1] =		{ EMeshBufferFormat::UInt32 };
    int32 icomponentCounts[1] =			{ 1 };
    int32 ioffsets[1] =					{ 0 };
    Result->GetIndexBuffers().SetBuffer
            ( 0, 4, 1,
              isemantics, isemanticIndices,
              iformats, icomponentCounts,
              ioffsets );
}


//-------------------------------------------------------------------------------------------------
void CreateMeshOptimisedForWrappingProjection(FMesh* Result, int32 LayoutIndex)
{
    Result->GetVertexBuffers().SetBufferCount( 2 );
    Result->GetIndexBuffers().SetBufferCount( 1 );

    EMeshBufferSemantic semantics[3] =	{ EMeshBufferSemantic::TexCoords,	EMeshBufferSemantic::Position,	EMeshBufferSemantic::Normal };
    int32 semanticIndices[3] =			{ 0,				0,				0 };
    EMeshBufferFormat formats[3] =		{ EMeshBufferFormat::Float32,		EMeshBufferFormat::Float32,	EMeshBufferFormat::Float32 };
    int32 componentCounts[3] =			{ 2,				3,				3 };
    int32 offsets[3] =					{ 0,				8,				20 };
    semanticIndices[0] = LayoutIndex;
    Result->GetVertexBuffers().SetBuffer( 0, 32, 3, semantics, semanticIndices, formats, componentCounts, offsets );

	EMeshBufferSemantic LayoutSemantics[1] = { EMeshBufferSemantic::LayoutBlock };
	int32 LayoutSemanticIndices[1] = { 0 };
	EMeshBufferFormat LayoutFormats[1] = { EMeshBufferFormat::UInt64 };
	int32 LayoutComponentCounts[1] = { 1 };
	int32 LayoutOffsets[1] = { 0 };
	LayoutSemanticIndices[0] = LayoutIndex;
	Result->GetVertexBuffers().SetBuffer(1, sizeof(uint64), 1, LayoutSemantics, LayoutSemanticIndices, LayoutFormats, LayoutComponentCounts, LayoutOffsets);

    EMeshBufferSemantic isemantics[1] =	{ EMeshBufferSemantic::VertexIndex };
    int32 isemanticIndices[1] =			{ 0 };
    EMeshBufferFormat iformats[1] =		{ EMeshBufferFormat::UInt32 };
    int32 icomponentCounts[1] =			{ 1 };
    int32 ioffsets[1] =					{ 0 };
    Result->GetIndexBuffers().SetBuffer( 0, 4, 1, isemantics, isemanticIndices, iformats, icomponentCounts, ioffsets );
}

}
