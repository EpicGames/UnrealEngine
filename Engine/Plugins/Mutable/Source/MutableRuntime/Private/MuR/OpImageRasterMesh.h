// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ParallelExecutionUtils.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"
#include "MuR/MeshPrivate.h"
#include "MuR/Raster.h"
#include "MuR/ConvertData.h"

namespace UE::Mutable::Private
{

	class WhitePixelProcessor
	{
	public:
		inline void ProcessPixel(uint8* pBufferPos, float[1]) const
		{
			pBufferPos[0] = 255;
		}

		inline void operator()(uint8* BufferPos, float Interpolators[1]) const
		{
			ProcessPixel(BufferPos, Interpolators);
		}
	};


	inline void ImageRasterMesh( const FMesh* pMesh, FImage* pImage, int32 LayoutIndex, uint64 BlockId,
		UE::Math::TIntVector2<uint16> CropMin, UE::Math::TIntVector2<uint16> UncroppedSize )
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageRasterMesh);

		if (pMesh->GetVertexCount() == 0)
		{
			return;
		}

		check( pImage->GetFormat()== EImageFormat::L_UByte );

		int32 sizeX = pImage->GetSizeX();
		int32 sizeY = pImage->GetSizeY();

		// Get the vertices
		int32 vertexCount = pMesh->GetVertexCount();
		TArray< RasterVertex<1> > vertices;
		vertices.SetNumZeroed(vertexCount);

		UntypedMeshBufferIteratorConst texIt( pMesh->GetVertexBuffers(), EMeshBufferSemantic::TexCoords, 0 );
		if (!texIt.ptr())
		{
			ensure(false);
			return;
		}

		for ( int32 v=0; v<vertexCount; ++v )
		{
            float uv[2] = {0.0f,0.0f};
			ConvertData( 0, uv, EMeshBufferFormat::Float32, texIt.ptr(), texIt.GetFormat() );
			ConvertData( 1, uv, EMeshBufferFormat::Float32, texIt.ptr(), texIt.GetFormat() );

			bool bUseCropping = UncroppedSize[0] > 0;
			if (bUseCropping)
			{
				vertices[v].x = uv[0] * UncroppedSize[0] - CropMin[0];
				vertices[v].y = uv[1] * UncroppedSize[1] - CropMin[1];
			}
			else
			{
				vertices[v].x = uv[0] * sizeX;
				vertices[v].y = uv[1] * sizeY;
			}
			++texIt;
		}

		// Get the indices
		const int32 NumFaces = pMesh->GetFaceCount();
		
		TArrayView<const int32> IndicesView;
		TArray<int32> FormatedIndices;

		UntypedMeshBufferIteratorConst indIt(pMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex, 0);
		if (indIt.GetFormat() == EMeshBufferFormat::UInt32)
		{
			IndicesView = TArrayView<const int32>(reinterpret_cast<const int32*>(indIt.ptr()), NumFaces * 3);
		}
		else
		{
			FormatedIndices.SetNumZeroed(NumFaces * 3);

			for (int32 i = 0; i < NumFaces * 3; ++i)
			{
				uint32_t index = 0;
				ConvertData(0, &index, EMeshBufferFormat::UInt32, indIt.ptr(), indIt.GetFormat());

				FormatedIndices[i] = index;
				++indIt;
			}

			IndicesView = TArrayView<const int32>(FormatedIndices);
		}

        UntypedMeshBufferIteratorConst bloIt( pMesh->GetVertexBuffers(), EMeshBufferSemantic::LayoutBlock, LayoutIndex );

		constexpr int32 NumBatchElems = 1 << 5; // Small batches perform better.
		const int32 NumBatches = FMath::DivideAndRoundUp<int32>(NumFaces, NumBatchElems);

        if (BlockId== FLayoutBlock::InvalidBlockId || bloIt.GetElementSize()==0 )
		{
			// Raster all the faces
            WhitePixelProcessor pixelProc;
			const TArrayView<uint8> ImageData = pImage->DataStorage.GetLOD(0);

			ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches, [
				&vertices, IndicesView, ImageData, sizeX, sizeY, pixelProc, NumFaces, NumBatchElems
			] (int32 BatchId)
				{
					const int32 BatchBeginFaceIndex = BatchId * NumBatchElems;
					const int32 BatchEndFaceIndex = FMath::Min(BatchBeginFaceIndex + NumBatchElems, NumFaces);

					for (int32 FaceIndex = BatchBeginFaceIndex; FaceIndex < BatchEndFaceIndex; ++FaceIndex)
					{
						constexpr int32 NumInterpolators = 1;
						Triangle<NumInterpolators>(ImageData.GetData(), ImageData.Num(),
							sizeX, sizeY,
							1,
							vertices[IndicesView[FaceIndex * 3 + 0]],
							vertices[IndicesView[FaceIndex * 3 + 1]],
							vertices[IndicesView[FaceIndex * 3 + 2]],
							pixelProc,
							false);
					}
				});
		}
		else
		{
			// Raster only the faces in the selected block
			check(bloIt.GetComponents() == 1);

			// Get the block per vertex
			TArray<uint64> VertexBlockIds;
			VertexBlockIds.SetNumZeroed(vertexCount);

			if (bloIt.GetFormat() == EMeshBufferFormat::UInt16)
			{
				// Relative blocks.
				const uint16* SourceIds = reinterpret_cast<const uint16*>(bloIt.ptr());
				for (int32 i = 0; i < vertexCount; ++i)
				{
					uint64 Id = SourceIds[i];
					Id = Id | (uint64(pMesh->MeshIDPrefix)<<32);
					VertexBlockIds[i] = Id;
				}
			}
			else if (bloIt.GetFormat() == EMeshBufferFormat::UInt64)
			{
				// Absolute blocks.
				const uint64* SourceIds = reinterpret_cast<const uint64*>(bloIt.ptr());
				for (int32 i = 0; i < vertexCount; ++i)
				{
					uint64 Id = SourceIds[i];
					VertexBlockIds[i] = Id;
				}
			}
			else
			{
				// Format not supported
				check(false);
			}

            WhitePixelProcessor pixelProc;

			const TArrayView<uint8> ImageData = pImage->DataStorage.GetLOD(0); 

			ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches, [
				&vertices, IndicesView, VertexBlockIds, BlockId, ImageData, sizeX, sizeY, pixelProc, NumFaces, NumBatchElems
			] (int32 BatchId)
				{
					const int32 BatchBeginFaceIndex = BatchId * NumBatchElems;
					const int32 BatchEndFaceIndex = FMath::Min(BatchBeginFaceIndex + NumBatchElems, NumFaces);

					for (int32 FaceIndex = BatchBeginFaceIndex; FaceIndex < BatchEndFaceIndex; ++FaceIndex)
					{
						// TODO: Select faces outside for loop?
						if (VertexBlockIds[IndicesView[FaceIndex * 3 + 0]] == BlockId)
						{
							constexpr int32 NumInterpolators = 1;
							Triangle<NumInterpolators>(ImageData.GetData(), ImageData.Num(),
								sizeX, sizeY,
								1,
								vertices[IndicesView[FaceIndex * 3 + 0]],
								vertices[IndicesView[FaceIndex * 3 + 1]],
								vertices[IndicesView[FaceIndex * 3 + 2]],
								pixelProc,
								false);
						}
					}
				});
		}

	}

}
