// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImagePrivate.h"
#include "MuR/MutableTrace.h"
#include "MuR/ParallelExecutionUtils.h"
#include "MuR/ImageResizeUtils.h"
#include "MuR/ImageFormatUtils.h"

#include "Async/ParallelFor.h"
#include "HAL/Platform.h"

namespace
{

bool bEnableImageResizeMemoryOptimizations = true;
static FAutoConsoleVariableRef CVarEnableImageResizeMemoryOptimizations (
	TEXT("mutable.EnableImageResizeMemoryOptimizations"),
	bEnableImageResizeMemoryOptimizations,
	TEXT("If set to true, enables image resize optimizations form small resizes and block compressed images."),
	ECVF_Default);


bool bEnableVectorImplementationForSmallResizes = true;
static FAutoConsoleVariableRef CVarEnableVectorImplementationForSmallResizes (
	TEXT("mutable.EnableVectorImplementationForSmallResizes"),
	bEnableVectorImplementationForSmallResizes,
	TEXT("If set to true, enables image resize vector optimizations form small resizes. Will only have effect if mutable.EnableImageResizeMemoryOptimizations is enabled"),
	ECVF_Default);

}

namespace UE::Mutable::Private
{
	/** */
	template< int32 NC >
	void ImageMagnifyX(FImage* Dest, const FImage* Base)
	{
		int32 BaseSizeX = Base->GetSizeX();
		int32 DestSizeX = Dest->GetSizeX();
		int32 SizeY = Base->GetSizeY();

		uint32 dx_16 = (uint32(BaseSizeX) << 16) / DestSizeX;

		// Linear filtering
		//for ( int32 y=0; y<SizeY; ++y )
		const auto& ProcessLine = [
			Dest, Base, dx_16, BaseSizeX, DestSizeX
		] (uint32 y)
		{
			uint32 px_16 = 0;
			const uint8* BaseBuf = Base->GetLODData(0) + y * BaseSizeX * NC;
			uint8* DestBuf = Dest->GetLODData(0) + y * DestSizeX * NC;

			uint32 LastPX = BaseSizeX - 1;
			for (int32 x = 0; x < DestSizeX; ++x)
			{
				uint32 px = px_16 >> 16;
				uint32 epx_16 = px_16 + dx_16;

				uint32 NextPX = FMath::Min(px + 1, LastPX);
				
				// Two fractions
				uint32 frac1 = (px_16 & 0xffff);
				uint32 frac0 = 0x10000 - frac1;

				for (int32 c = 0; c < NC; ++c)
				{
					DestBuf[c] = uint8((BaseBuf[px * NC + c] * frac0 + BaseBuf[NextPX * NC + c] * frac1) >> 16);
				}

				px_16 = epx_16;
				DestBuf += NC;
			}
		};

		// TODO: ensure 64K-aligned batches?
		ParallelFor(SizeY, ProcessLine);
	}


	/** */
	void ImageMagnifyX(FImage* Dest, const FImage* Base)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageMagnifyX)

		check(Dest->GetSizeY() == Base->GetSizeY());
		check(Dest->GetSizeX() > Base->GetSizeX());

		switch (Base->GetFormat())
		{

		case EImageFormat::L_UByte:
		{
			ImageMagnifyX<1>(Dest, Base);
			break;
		}

		case EImageFormat::RGB_UByte:
		{
			ImageMagnifyX<3>(Dest, Base);
			break;
		}

		case EImageFormat::BGRA_UByte:
		case EImageFormat::RGBA_UByte:
		{
			ImageMagnifyX<4>(Dest, Base);
			break;
		}

		default:
			// Case not implemented
			check(false);
		}
	}


	/** General image minimisation. */
	template< int32 NC >
	void ImageMinifyX(FImage* Dest, const FImage* Base)
	{
		int32 BaseSizeX = Base->GetSizeX();
		int32 DestSizeX = Dest->GetSizeX();
		int32 SizeY = Base->GetSizeY();

		uint32 dx_16 = (uint32(BaseSizeX) << 16) / DestSizeX;

		// Linear filtering
		//for ( int32 y=0; y<SizeY; ++y )
		const auto& ProcessLine = [
			Dest, Base, dx_16, BaseSizeX, DestSizeX
		] (uint32 y)
		{
			const uint8* BaseBuf = Base->GetLODData(0) + y * BaseSizeX * NC;
			uint8* DestBuf = Dest->GetLODData(0) + y * DestSizeX * NC;

			uint32 px_16 = 0;
			for (int32 x = 0; x < DestSizeX; ++x)
			{
				uint32 r_16[NC];
				for (int32 c = 0; c < NC; ++c)
				{
					r_16[c] = 0;
				}

				uint32 epx_16 = px_16 + dx_16;
				uint32 px = px_16 >> 16;
				uint32 epx = epx_16 >> 16;

				// First fraction
				uint32 frac0 = px_16 & 0xffff;
				if (frac0)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r_16[c] += (0x10000 - frac0) * BaseBuf[px * NC + c];
					}

					++px;
				}

				// Whole pixels
				while (px < epx)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r_16[c] += uint32(BaseBuf[px * NC + c]) << 16;
					}

					++px;
				}

				// Second fraction
				uint32 frac1 = epx_16 & 0xffff;
				if (frac1)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r_16[c] += frac1 * BaseBuf[px * NC + c];
					}
				}

				for (int32 c = 0; c < NC; ++c)
				{
					DestBuf[c] = (uint8)(r_16[c] / dx_16);
				}

				px_16 = epx_16;
				DestBuf += NC;
			}
		};

		ParallelFor(SizeY, ProcessLine);
	}


	//---------------------------------------------------------------------------------------------
	//! Optimised for whole factors
	//---------------------------------------------------------------------------------------------
	template< int32 NC, int32 FACTOR >
	void ImageMinifyX_Exact(FImage* Dest, const FImage* Base)
	{
		int32 BaseSizeX = Base->GetSizeX();
		int32 DestSizeX = Dest->GetSizeX();
		int32 SizeY = Base->GetSizeY();

		const uint8* BaseBuf = Base->GetLODData(0);
		uint8* DestBuf = Dest->GetLODData(0);

		// Linear filtering
		const auto& ProcessLine = [
			Dest, Base, BaseSizeX, DestSizeX
		] (uint32 y)
		{
			const uint8* BaseBuf = Base->GetLODData(0) + y * BaseSizeX * NC;
			uint8* DestBuf = Dest->GetLODData(0) + y * DestSizeX * NC;

			uint32 r[NC];
			for (int32 x = 0; x < DestSizeX; ++x)
			{
				for (int32 c = 0; c < NC; ++c)
				{
					r[c] = 0;
					for (int32 a = 0; a < FACTOR; ++a)
					{
						r[c] += BaseBuf[a * NC + c];
					}
				}

				for (int32 c = 0; c < NC; ++c)
				{
					DestBuf[c] = (uint8)(r[c] / FACTOR);
				}

				DestBuf += NC;
				BaseBuf += NC * FACTOR;
			}
		};

		ParallelFor(SizeY, ProcessLine);
	}

	inline uint32 AverageChannel(uint32 A, uint32 B)
	{
		uint32 Result = (A + B) >> 1;
		return Result;
	}

	template<>
	void ImageMinifyX_Exact<4, 2>(FImage* Dest, const FImage* Base)
	{
		int32 BaseSizeX = Base->GetSizeX();
		int32 DestSizeX = Dest->GetSizeX();
		int32 SizeY = Base->GetSizeY();

		const uint8* BaseBuf = Base->GetLODData(0);
		uint8* DestBuf = Dest->GetLODData(0);

		int32 TotalBasePixels = BaseSizeX * SizeY;
		constexpr int32 BasePixelsPerBatch = 4096 * 2;
		int32 NumBatches = FMath::DivideAndRoundUp(TotalBasePixels, BasePixelsPerBatch);

		// Linear filtering
		const auto& ProcessBatchUnaligned =
			[DestBuf, BaseBuf, BaseSizeX, DestSizeX, BasePixelsPerBatch, TotalBasePixels]
		(int32 BatchIndex)
		{
			const uint8* pBatchBaseBuf = BaseBuf + BatchIndex * BasePixelsPerBatch * 4;
			uint8* pBatchDestBuf = DestBuf + BatchIndex * BasePixelsPerBatch * 4 / 2;

			int32 NumBasePixels = FMath::Min(BasePixelsPerBatch, TotalBasePixels - BatchIndex * BasePixelsPerBatch);

			uint16 r[4];
			for (int32 x = 0; x < NumBasePixels; x += 2)
			{
				r[0] = pBatchBaseBuf[0] + pBatchBaseBuf[0 + 4];
				r[1] = pBatchBaseBuf[1] + pBatchBaseBuf[1 + 4];
				r[2] = pBatchBaseBuf[2] + pBatchBaseBuf[2 + 4];
				r[3] = pBatchBaseBuf[3] + pBatchBaseBuf[3 + 4];

				pBatchDestBuf[0] = (uint8)(r[0] >> 1);
				pBatchDestBuf[1] = (uint8)(r[1] >> 1);
				pBatchDestBuf[2] = (uint8)(r[2] >> 1);
				pBatchDestBuf[3] = (uint8)(r[3] >> 1);

				pBatchBaseBuf += 4 * 2;
				pBatchDestBuf += 4;
			}
		};

		const auto& ProcessBatchAligned =
			[DestBuf, BaseBuf, BaseSizeX, DestSizeX, BasePixelsPerBatch, TotalBasePixels]
		(int32 BatchIndex)
		{
			const uint32* pBatchBaseBuf = reinterpret_cast<const uint32*>(BaseBuf) + BatchIndex * BasePixelsPerBatch;
			uint32* pBatchDestBuf = reinterpret_cast<uint32*>(DestBuf) + BatchIndex * (BasePixelsPerBatch >> 1);

			int32 NumBasePixels = FMath::Min(BasePixelsPerBatch, TotalBasePixels - BatchIndex * BasePixelsPerBatch);

			for (int32 p = 0; p < NumBasePixels; ++p)
			{
				uint32 FullSource0 = pBatchBaseBuf[p * 2 + 0];
				uint32 FullSource1 = pBatchBaseBuf[p * 2 + 1];

				uint32 FullResult = 0;

				FullResult |= AverageChannel((FullSource0 >> 0) & 0xff, (FullSource1 >> 0) & 0xff) << 0;
				FullResult |= AverageChannel((FullSource0 >> 8) & 0xff, (FullSource1 >> 8) & 0xff) << 8;
				FullResult |= AverageChannel((FullSource0 >> 16) & 0xff, (FullSource1 >> 16) & 0xff) << 16;
				FullResult |= AverageChannel((FullSource0 >> 24) & 0xff, (FullSource1 >> 24) & 0xff) << 24;

				pBatchDestBuf[p] = FullResult;
			}
		};


		ParallelFor(NumBatches, ProcessBatchUnaligned);
	}


	//---------------------------------------------------------------------------------------------
	//! Image minify X version hub.
	//---------------------------------------------------------------------------------------------
	void ImageMinifyX(FImage* Dest, const FImage* Base)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageMinifyX)

		check(Dest->GetSizeY() == Base->GetSizeY());
		check(Dest->GetSizeX() < Base->GetSizeX());

		switch (Base->GetFormat())
		{

		case EImageFormat::L_UByte:
		{
			if (2 * Dest->GetSizeX() == Base->GetSizeX())
			{
				// Optimised case
				ImageMinifyX_Exact<1, 2>(Dest, Base);
			}
			else
			{
				// Generic case
				ImageMinifyX<1>(Dest, Base);
			}
			break;
		}

		case EImageFormat::RGB_UByte:
		{
			if (2 * Dest->GetSizeX() == Base->GetSizeX())
			{
				// Optimised case
				ImageMinifyX_Exact<3, 2>(Dest, Base);
			}
			else
			{
				// Generic case
				ImageMinifyX<3>(Dest, Base);
			}
			break;
		}

		case EImageFormat::BGRA_UByte:
		case EImageFormat::RGBA_UByte:
		{
			if (2 * Dest->GetSizeX() == Base->GetSizeX())
			{
				// Optimised case
				ImageMinifyX_Exact<4, 2>(Dest, Base);
			}
			else if (4 * Dest->GetSizeX() == Base->GetSizeX())
			{
				// Optimised case
				ImageMinifyX_Exact<4, 4>(Dest, Base);
			}
			else
			{
				// Generic case
				ImageMinifyX<4>(Dest, Base);
			}
			break;
		}

		default:
			// Case not implemented
			check(false);
		}

	}


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	template< int32 NC >
	void ImageMagnifyY(FImage* Dest, const FImage* Base)
	{
		if (!Base || !Dest ||
			!Base->GetSizeX() || !Base->GetSizeY() || !Dest->GetSizeX() || !Dest->GetSizeY())
		{
			return;
		}

		int32 BaseSizeY = Base->GetSizeY();
		int32 DestSizeY = Dest->GetSizeY();
		int32 SizeX = Base->GetSizeX();

		SIZE_T rowSize = SizeX * NC;

		// Common case, optimised.
		if (DestSizeY == BaseSizeY * 2)
		{
			//for (int32 y = 0; y < BaseSizeY; ++y)
			const auto& ProcessLine = [
				Dest, Base, rowSize
			] (uint32 y)
			{
				uint8* DestBuf = Dest->GetLODData(0) + 2 * y * rowSize;
				const uint8* BaseBuf = Base->GetLODData(0) + y * rowSize;

				FMemory::Memcpy(DestBuf, BaseBuf, rowSize);
				DestBuf += rowSize;

				FMemory::Memcpy(DestBuf, BaseBuf, rowSize);
			};

				ParallelFor(BaseSizeY, ProcessLine);
		}
		else
		{
			uint32 dy_16 = (uint32(BaseSizeY) << 16) / DestSizeY;

			uint32 LastPY = BaseSizeY - 1;

			// Linear filtering
			// \todo: optimise: swap loops, etc.
			//for ( int32 x=0; x<SizeX; ++x )
			const auto& ProcessColumn = [
				Dest, Base, SizeX, DestSizeY, LastPY, dy_16
			] (uint32 x)
			{
				uint32 py_16 = 0;
				uint8* DestBuf = Dest->GetLODData(0) + x * NC;
				const uint8* BaseBuf = Base->GetLODData(0);

				for (int32 y = 0; y < DestSizeY; ++y)
				{
					uint32 py = py_16 >> 16;
					uint32 epy_16 = py_16 + dy_16;

					uint32 NextPY = FMath::Min(py + 1, LastPY);

					// Two fractions
					uint32 frac1 = (py_16 & 0xffff);
					uint32 frac0 = 0x10000 - frac1;

					for (int32 c = 0; c < NC; ++c)
					{
						DestBuf[c] = (uint8)((BaseBuf[(py * SizeX + x) * NC + c] * frac0 +
							BaseBuf[(NextPY * SizeX + x) * NC + c] * frac1
							) >> 16);
					}

					py_16 = epy_16;
					DestBuf += SizeX * NC;
				}
			};

			ParallelFor(SizeX, ProcessColumn);
		}
	}


	void ImageMagnifyY(FImage* Dest, const FImage* Base)
	{
		check(Dest->GetSizeY() > Base->GetSizeY());
		check(Dest->GetSizeX() == Base->GetSizeX());

		MUTABLE_CPUPROFILER_SCOPE(ImageMagnifyY)

		switch (Base->GetFormat())
		{

		case EImageFormat::L_UByte:
		{
			ImageMagnifyY<1>(Dest, Base);
			break;
		}

		case EImageFormat::RGB_UByte:
		{
			ImageMagnifyY<3>(Dest, Base);
			break;
		}

		case EImageFormat::RGBA_UByte:
		case EImageFormat::BGRA_UByte:
		{
			ImageMagnifyY<4>(Dest, Base);
			break;
		}

		default:
			// Case not implemented
			check(false);
		}
	}


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	template< int32 NC >
	void ImageMinifyY(FImage* Dest, const FImage* Base)
	{
		int32 BaseSizeY = Base->GetSizeY();
		int32 DestSizeY = Dest->GetSizeY();
		int32 SizeX = Base->GetSizeX();

		uint32 dy_16 = (uint32(BaseSizeY) << 16) / DestSizeY;

		const uint8* BaseBuf = Base->GetLODData(0);

		// Linear filtering
		//for (int32 x = 0; x < SizeX; ++x)
		const auto& ProcessColumn = [
			Dest, BaseBuf, SizeX, DestSizeY, dy_16
		] (uint32 x)
		{
			uint8* DestBuf = Dest->GetLODData(0) + x * NC;
			uint32 py_16 = 0;
			for (int32 y = 0; y < DestSizeY; ++y)
			{
				uint32 r_16[NC];
				for (int32 c = 0; c < NC; ++c)
				{
					r_16[c] = 0;
				}

				uint32 epy_16 = py_16 + dy_16;
				uint32 py = py_16 >> 16;
				uint32 epy = epy_16 >> 16;

				// First fraction
				uint32 frac0 = py_16 & 0xffff;
				if (frac0)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r_16[c] += (0x10000 - frac0) * BaseBuf[(py * SizeX + x) * NC + c];
					}

					++py;
				}

				// Whole pixels
				while (py < epy)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r_16[c] += uint32(BaseBuf[(py * SizeX + x) * NC + c]) << 16;
					}

					++py;
				}

				// Second fraction
				uint32 frac1 = epy_16 & 0xffff;
				if (frac1)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r_16[c] += frac1 * BaseBuf[(py * SizeX + x) * NC + c];
					}
				}

				for (int32 c = 0; c < NC; ++c)
				{
					DestBuf[c] = (uint8)(r_16[c] / dy_16);
				}

				py_16 = epy_16;
				DestBuf += SizeX * NC;
			}
		};

		ParallelFor(SizeX, ProcessColumn);

	}


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	template< int32 NC, int32 FACTOR >
	void ImageMinifyY_Exact(FImage* Dest, const FImage* Base)
	{
		int32 DestSizeY = Dest->GetSizeY();
		int32 SizeX = Base->GetSizeX();

		// Linear filtering
		//for ( int32 y=0; y<DestSizeY; ++y )
		const auto ProcessRow = [
			Dest, Base, SizeX, DestSizeY
		] (uint32 y)
		{
			uint8* DestBuf = Dest->GetLODData(0) + y * NC * SizeX;
			const uint8* BaseBuf = Base->GetLODData(0) + y * FACTOR * SizeX * NC;

			for (int32 x = 0; x < SizeX; ++x)
			{
				uint32 r[NC];
				for (int32 c = 0; c < NC; ++c)
				{
					r[c] = 0;
				}

				// Whole pixels
				for (int32 f = 0; f < FACTOR; ++f)
				{
					for (int32 c = 0; c < NC; ++c)
					{
						r[c] += BaseBuf[SizeX * NC * f + x * NC + c];
					}
				}

				for (int32 c = 0; c < NC; ++c)
				{
					DestBuf[c] = (uint8)(r[c] / FACTOR);
				}

				DestBuf += NC;
			}
			BaseBuf += FACTOR * SizeX * NC;
		};

		ParallelFor(DestSizeY, ProcessRow);
	}


	//---------------------------------------------------------------------------------------------
	//! Image minify Y version hub.
	//---------------------------------------------------------------------------------------------
	void ImageMinifyY(FImage* Dest, const FImage* Base)
	{
		check(Dest->GetSizeY() < Base->GetSizeY());
		check(Dest->GetSizeX() == Base->GetSizeX());

		MUTABLE_CPUPROFILER_SCOPE(ImageMinifyY)

		switch (Base->GetFormat())
		{

		case EImageFormat::L_UByte:
		{
			if (2 * Dest->GetSizeY() == Base->GetSizeY())
			{
				// Optimised case
				ImageMinifyY_Exact<1, 2>(Dest, Base);
			}
			else
			{
				// Generic case
				ImageMinifyY<1>(Dest, Base);
			}
			break;
		}

		case EImageFormat::RGB_UByte:
		{
			if (2 * Dest->GetSizeY() == Base->GetSizeY())
			{
				// Optimised case
				ImageMinifyY_Exact<3, 2>(Dest, Base);
			}
			else
			{
				// Generic case
				ImageMinifyY<3>(Dest, Base);
			}
			break;
		}

		case EImageFormat::RGBA_UByte:
		case EImageFormat::BGRA_UByte:
		{
			if (2 * Dest->GetSizeY() == Base->GetSizeY())
			{
				// Optimised case
				ImageMinifyY_Exact<4, 2>(Dest, Base);
			}
			else
			{
				// Generic case
				ImageMinifyY<4>(Dest, Base);
			}
			break;
		}

		default:
			// Case not implemented
			check(false);
			//UE::Mutable::Private::Halt();
		}

	}

namespace
{
	template<bool bUseVectorImpl>
	void ImageResizeLinearDownByLessThanTwo(FImage* Dest, const FImage* InBase)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageResizeLinearDownByLessThanTwo);

		const FImageSize DestSize = FImageSize(Dest->GetSizeX(), Dest->GetSizeY());
		const FImageSize SrcSize = FImageSize(InBase->GetSizeX(), InBase->GetSizeY());
		
		check(SrcSize.X >= DestSize.X && SrcSize.Y >= DestSize.Y && SrcSize.X <= DestSize.X*2 && SrcSize.Y <= DestSize.Y*2);

		const EImageFormat SrcFormat = InBase->GetFormat();
		const EImageFormat DestFormat = Dest->GetFormat();
	
		uint8* DestData = Dest->GetLODData(0);
		const uint8* SrcData = InBase->GetLODData(0);
		
		const FImageFormatData& DestFormatData = GetImageFormatData(Dest->GetFormat());
		const FImageFormatData& SrcFormatData = GetImageFormatData(InBase->GetFormat());

		const FVector2f OneOverScalingFactor = FVector2f(SrcSize.X, SrcSize.Y) / FVector2f(DestSize.X, DestSize.Y);

		if (SrcFormatData.PixelsPerBlockX == 1 && SrcFormatData.PixelsPerBlockY == 1)
		{
			MUTABLE_CPUPROFILER_SCOPE(ResizeUnCompressed);

			SubImageResize::SubImageResizeFuncType* SubImageResizeFunc = SubImageResize::SelectSubImageResizeFunc(SrcFormat);

			const FVector2f SrcSubPixelOffset = FVector2f(0.0f);
			// Uncompressed format, resize in one go directly to Dest.
			SubImageResizeFunc(
						OneOverScalingFactor, SrcSubPixelOffset,
						DestSize, DestSize, SrcSize, SrcSize, DestData, SrcData);
		}
		else if (SrcFormatData.PixelsPerBlockX > 0 && SrcFormatData.PixelsPerBlockY > 0)
		{
			MUTABLE_CPUPROFILER_SCOPE(ResizeBlockCompressed);

			check(DestFormatData.PixelsPerBlockX == 1 && DestFormatData.PixelsPerBlockY == 1);

			const int32 DestNumChannels = DestFormatData.Channels; 
			const FIntVector2 PixelsPerBlock = FIntVector2(SrcFormatData.PixelsPerBlockX, SrcFormatData.PixelsPerBlockY);
			const int32 BlockSizeInBytes = SrcFormatData.BytesPerBlock;

			const FIntVector2 NumBlocks = FIntVector2(
					FMath::DivideAndRoundUp(SrcSize.X, uint16(PixelsPerBlock.X)),
					FMath::DivideAndRoundUp(SrcSize.Y, uint16(PixelsPerBlock.Y)));

			constexpr int32 BatchSizeInBlocksX = 1 << 5;
			constexpr int32 BatchSizeInBlocksY = 1 << 4;

			const FIntVector2 NumBatches = FIntVector2(
					FMath::DivideAndRoundUp(NumBlocks.X, BatchSizeInBlocksX),
					FMath::DivideAndRoundUp(NumBlocks.Y, BatchSizeInBlocksY));

			// Limit the parallel job num based on actual num workers. Here we cannot rely on ParallelFor
			// balancing the load as we need to allocate memory for every job. Make sure there is always 1 job.
			// TODO: Consider balancing work on using a 2D division.
			const int32 MaxParallelJobs = FMath::Max(1, FMath::Min(int32(LowLevelTasks::FScheduler::Get().GetNumWorkers()), 8));
			
			constexpr int32 MinRowBatchesPerJob = 2;

			const int32 NumRowBatchesPerJob = 
				 FMath::Min(NumBatches.Y, FMath::Max(MinRowBatchesPerJob, FMath::DivideAndRoundUp(NumBatches.Y, MaxParallelJobs)));

			const int32 NumParallelJobs = FMath::DivideAndRoundUp(NumBatches.Y, NumRowBatchesPerJob); 

			// Allocate memory for 1 extra block, this is needed so that the data to compute the batch edges
			// is available. This implies extra work needs to be done, for simplicity re-decompress those blocks, if
			// that becomes a problem more convoluted memory cacheing schemes could be implemented.
			const miro::FImageSize StagingSize = miro::FImageSize(
					uint16((BatchSizeInBlocksX + 1)*PixelsPerBlock.X), 
					uint16((BatchSizeInBlocksY + 1)*PixelsPerBlock.Y));
		
			// Add some extra padding so different threads do not share cache lines.
			const int32 PerJobStagingBytes = StagingSize.X*StagingSize.Y*DestNumChannels + PLATFORM_CACHE_LINE_SIZE;

			TArray<uint8, FDefaultMemoryTrackingAllocator<MemoryCounters::FImageMemoryCounter>> StagingMemory;
			StagingMemory.SetNum(PerJobStagingBytes * NumParallelJobs);
			uint8 * const StagingMemoryData = StagingMemory.GetData();

			miro::SubImageDecompression::FuncRefType DecompressionFunc = SubImageFormat::SelectDecompressionFunction(DestFormat, SrcFormat);
			SubImageResize::SubImageResizeFuncType* SubImageResizeFunc = SubImageResize::SelectSubImageResizeFunc(DestFormat);

			ParallelExecutionUtils::InvokeBatchParallelFor(NumParallelJobs, 
			[
				NumParallelJobs, NumRowBatchesPerJob, 
				StagingMemoryData, PerJobStagingBytes, 
				NumBatches, NumBlocks, PixelsPerBlock, BlockSizeInBytes, DestNumChannels, DecompressionFunc,
				SubImageResizeFunc, SrcData, SrcSize, DestData, DestSize, OneOverScalingFactor
			](int32 JobId)
			{
				const int32 JobRowBegin = JobId*NumRowBatchesPerJob;
				const int32 JobRowEnd   = FMath::Min(JobRowBegin + NumRowBatchesPerJob, NumBatches.Y);
				uint8 * const JobStagingMemoryData = StagingMemoryData + JobId*PerJobStagingBytes;

				for (int32 BatchY = JobRowBegin; BatchY < JobRowEnd; ++BatchY)
				{
					for (int32 BatchX = 0; BatchX < NumBatches.X; ++BatchX)
					{
						const FIntVector2 BatchBeginInBlocks = FIntVector2(
								BatchX*BatchSizeInBlocksX, BatchY*BatchSizeInBlocksY);

						const FIntVector2 BatchDecEndInBlocks = FIntVector2(
								FMath::Min(BatchBeginInBlocks.X + BatchSizeInBlocksX + 1, NumBlocks.X),
								FMath::Min(BatchBeginInBlocks.Y + BatchSizeInBlocksY + 1, NumBlocks.Y));

						const FIntVector2 BatchEndInBlocks = FIntVector2(
								FMath::Min(BatchBeginInBlocks.X + BatchSizeInBlocksX, NumBlocks.X),
								FMath::Min(BatchBeginInBlocks.Y + BatchSizeInBlocksY, NumBlocks.Y));

						const uint8* const SrcBatchData = 
								SrcData + (BatchBeginInBlocks.Y*NumBlocks.X + BatchBeginInBlocks.X)*BlockSizeInBytes;

						// Assume the decompressed size is always multiple of the block size. Trim unused bytes when copying to 
						// the final destination.
						const FIntVector2 BatchDecSizeInPixels = FIntVector2(
								(BatchDecEndInBlocks.X - BatchBeginInBlocks.X)*PixelsPerBlock.X,
								(BatchDecEndInBlocks.Y - BatchBeginInBlocks.Y)*PixelsPerBlock.Y);

						const miro::FImageSize FromSize = miro::FImageSize(SrcSize.X, SrcSize.Y); 
						const miro::FImageSize DecSize  = miro::FImageSize(uint16(BatchDecSizeInPixels.X), uint16(BatchDecSizeInPixels.Y));

						DecompressionFunc(FromSize, DecSize, DecSize, SrcBatchData, JobStagingMemoryData);

						const FIntVector2 BatchOutBeginInPixels = FIntVector2( 
								FMath::DivideAndRoundUp(BatchBeginInBlocks.X*PixelsPerBlock.X*DestSize.X, int32(SrcSize.X)),
								FMath::DivideAndRoundUp(BatchBeginInBlocks.Y*PixelsPerBlock.Y*DestSize.Y, int32(SrcSize.Y)));

						const FVector2f SrcSubPixelOffset = FVector2f(
								FMath::Frac(float(BatchOutBeginInPixels.X) * OneOverScalingFactor.X),
								FMath::Frac(float(BatchOutBeginInPixels.Y) * OneOverScalingFactor.Y));

						const FIntVector2 BatchOutEndInPixels = FIntVector2(
								FMath::Min(
									FMath::DivideAndRoundUp(BatchEndInBlocks.X*PixelsPerBlock.X*DestSize.X, int32(SrcSize.X)), 
									int32(DestSize.X)),
								FMath::Min(
									FMath::DivideAndRoundUp(BatchEndInBlocks.Y*PixelsPerBlock.Y*DestSize.Y, int32(SrcSize.Y)), 
									int32(DestSize.Y))); 
							
						const FIntVector2 BatchSizeInPixels = (BatchEndInBlocks - BatchBeginInBlocks)*PixelsPerBlock;
						
						const FImageSize SrcSubSize  = FImageSize(BatchSizeInPixels.X, BatchSizeInPixels.Y);
						const FImageSize DestSubSize = FImageSize(
								BatchOutEndInPixels.X - BatchOutBeginInPixels.X, 
								BatchOutEndInPixels.Y - BatchOutBeginInPixels.Y);
		
						const FIntVector2& DestSubOffset = BatchOutBeginInPixels;
					
						const uint8* SrcSubData = JobStagingMemoryData;

						uint8* DestSubData = DestData + (DestSubOffset.Y*DestSize.X + DestSubOffset.X)*DestNumChannels;

						SubImageResizeFunc( 
								OneOverScalingFactor, SrcSubPixelOffset,
								DestSize, DestSubSize, DecSize, SrcSubSize, DestSubData, SrcSubData);
					}
				}
			});
		}
		else
		{
			checkf(false, TEXT("Format not supported"));
		}
	}
}

	void FImageOperator::ImageResizeLinear(FImage* Dest, int32 ImageCompressionQuality, const FImage* InBase)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageResizeLinear);

		check(!(InBase->Flags & FImage::IF_CANNOT_BE_SCALED));

		const FImageSize BaseSize = FImageSize(InBase->GetSizeX(), InBase->GetSizeY());
		const FImageSize DestSize = FImageSize(Dest->GetSizeX(), Dest->GetSizeY());

		// This case should be treated in the caller and skip the operation altogether.
		check(BaseSize != DestSize);

		if (!DestSize[0] || !DestSize[1] || !BaseSize[0] || !BaseSize[1])
		{
			return;
		}

		const bool bIsDownSizeByLessThanTwo =
			BaseSize.X >= DestSize.X && BaseSize.Y >= DestSize.Y && 
			BaseSize.X < DestSize.X*2 && BaseSize.Y < DestSize.Y*2; 

		const bool bIsUncompressedFormat = !UE::Mutable::Private::IsCompressedFormat(InBase->GetFormat());  
		const bool bIsBlockCompressedFormat = UE::Mutable::Private::IsBlockCompressedFormat(InBase->GetFormat());

		if (bIsDownSizeByLessThanTwo && (bIsUncompressedFormat || bIsBlockCompressedFormat) && bEnableImageResizeMemoryOptimizations)
		{
			// Special case for small down resizes where no more than 2 pixels need to be sampled. 
			// This could be extended to upsizes of any size.
			
			const EImageFormat BaseUncompressedFormat = GetUncompressedFormat(InBase->GetFormat());
			const EImageFormat DestFormat = Dest->GetFormat();
			
			TManagedPtr<FImage> TempImage;
			FImage* DecompressedDest = Dest;
			if (DestFormat != BaseUncompressedFormat)
			{
				TempImage = CreateImage(DestSize.X, DestSize.Y, 1, BaseUncompressedFormat, EInitializationType::NotInitialized);
				DecompressedDest = TempImage.Get();
			}
		
			if (bEnableVectorImplementationForSmallResizes)
			{
				constexpr bool bUseVectorImpl = true;
				ImageResizeLinearDownByLessThanTwo<bUseVectorImpl>(DecompressedDest, InBase);
			}
			else
			{
				constexpr bool bUseVectorImpl = false;
				ImageResizeLinearDownByLessThanTwo<bUseVectorImpl>(DecompressedDest, InBase);
			}

			if (DestFormat != BaseUncompressedFormat)
			{
				bool bSuccess = false;
				ImagePixelFormat(bSuccess, ImageCompressionQuality, Dest, DecompressedDest);
				check(bSuccess);
			}
		}
		else
		{
			const FImage* Base = InBase;
			TManagedPtr<FImage> BaseReformat;
			
			// Shouldn't happen! But if it does...
			const EImageFormat SourceFormat = Base->GetFormat();
			const EImageFormat UncompressedFormat = GetUncompressedFormat(SourceFormat);
			if (SourceFormat != UncompressedFormat)
			{
				BaseReformat = ImagePixelFormat(ImageCompressionQuality, InBase, UncompressedFormat);
				Base = BaseReformat.Get();
			}

			const EImageFormat BaseFormat = Base->GetFormat();

			// First resize X
			TManagedPtr<FImage> Temp;
			if (DestSize[0] > BaseSize[0])
			{
				Temp = CreateImage(DestSize[0], BaseSize[1], 1, BaseFormat, EInitializationType::NotInitialized);
				ImageMagnifyX(Temp.Get(), Base);
			}
			else if (DestSize[0] < BaseSize[0])
			{
				Temp = CreateImage(DestSize[0], BaseSize[1], 1, BaseFormat, EInitializationType::NotInitialized);
				ImageMinifyX(Temp.Get(), Base);
			}
			else
			{
				Temp = CloneImage(Base);
			}

			// No lonegr needed. Should be only set if a reformat has been done. 
			check(!(SourceFormat == UncompressedFormat) || !BaseReformat);
			if (BaseReformat)
			{
				check(Base == BaseReformat.Get());
				BaseReformat = nullptr;
				Base = nullptr;
			}

			// Now resize Y
			TManagedPtr<FImage> Temp2;
			if (DestSize[1] > BaseSize[1])
			{
				Temp2 = CreateImage(DestSize[0], DestSize[1], 1, BaseFormat, EInitializationType::NotInitialized);
				ImageMagnifyY(Temp2.Get(), Temp.Get());
			}
			else if (DestSize[1] < BaseSize[1])
			{
				Temp2 = CreateImage(DestSize[0], DestSize[1], 1, BaseFormat, EInitializationType::NotInitialized);
				ImageMinifyY(Temp2.Get(), Temp.Get());
			}
			else
			{
				Temp2 = Temp;
			}
			Temp = nullptr;


			// Reset format if it was changed to scale
			if (SourceFormat != UncompressedFormat)
			{
				TManagedPtr<FImage> OldTemp = Temp2;
				Temp2 = ImagePixelFormat(ImageCompressionQuality, Temp2.Get(), SourceFormat);
			}

			Dest->CopyMove(Temp2.Get());
			Temp2 = nullptr;
		}
		
		// Update the relevancy data of the image.
		if (InBase->Flags & FImage::EImageFlags::IF_HAS_RELEVANCY_MAP)
		{
			Dest->Flags |= FImage::EImageFlags::IF_HAS_RELEVANCY_MAP;

			float FactorY = float(DestSize[1]) / float(BaseSize[1]);

			Dest->RelevancyMinY = uint16(FMath::FloorToFloat(InBase->RelevancyMinY * FactorY));
			Dest->RelevancyMaxY = uint16(FMath::Min((int32)FMath::CeilToFloat(InBase->RelevancyMaxY * FactorY), Dest->GetSizeY() - 1));
		}
	}

}
