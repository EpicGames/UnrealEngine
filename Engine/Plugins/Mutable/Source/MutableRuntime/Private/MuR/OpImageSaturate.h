// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/Platform.h"
#include "MuR/ParallelExecutionUtils.h" 

#include "Math/VectorRegister.h"

namespace UE::Mutable::Private
{

	template<bool bUseVectorImpl = false>
	inline void ImageSaturate(FImage* AImage, float Factor)
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageSaturateInplace);

		if (!AImage)
		{
			return;
		}

		// Clamp the factor
		// TODO: See what happens if we don't
		Factor = FMath::Max(0.0f, Factor);
        uint16 FactorUNorm = (uint16)(Factor*255.0f);

		// Generic implementation

		const int32 BytesPerElem = GetImageFormatData(AImage->GetFormat()).BytesPerBlock;
		
		constexpr int32 NumBatchElems = 1 << 14;
		const int32 NumBatches = AImage->DataStorage.GetNumBatches(NumBatchElems, BytesPerElem); 

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[
			AImage, NumBatchElems, BytesPerElem, FactorUNorm
		](int32 BatchId)
		{
			TArrayView<uint8> AView = AImage->DataStorage.GetBatch(BatchId, NumBatchElems, BytesPerElem);
			uint8* ABuf = AView.GetData(); 
			const int32 NumElems = AView.Num() / BytesPerElem;

			switch (AImage->GetFormat())
			{
			case EImageFormat::RGB_UByte:
			{
				if constexpr (!bUseVectorImpl)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint16 PixelData[4] = {ABuf[3*I + 0], ABuf[3*I + 1], ABuf[3*I + 2], 0};

						const uint16 LumTimesFactor = ((76 * PixelData[0] + 150 * PixelData[1] + 29 * PixelData[2]) / 255) * (255 - FactorUNorm);

						ABuf[3*I + 0] = static_cast<uint8>((PixelData[0] * FactorUNorm + LumTimesFactor) / 255);
						ABuf[3*I + 1] = static_cast<uint8>((PixelData[1] * FactorUNorm + LumTimesFactor) / 255);
						ABuf[3*I + 2] = static_cast<uint8>((PixelData[2] * FactorUNorm + LumTimesFactor) / 255);
					}
				}
				else // if constexpr bUseVectorImpl
				{
					constexpr VectorRegister4Int StaurationFactorRGB = MakeVectorRegisterIntConstant(76, 150, 29, 0);
					constexpr VectorRegister4Int Value255 = MakeVectorRegisterIntConstant(255, 255, 255, 255);
					const VectorRegister4Int F = VectorIntSet1(FactorUNorm);
					const VectorRegister4Int OneMinusF = VectorIntSubtract(Value255, F);
					
					for (int32 I = 0; I < NumElems; ++I)
					{
						const VectorRegister4Int Value = MakeVectorRegisterInt(
							static_cast<int32>(ABuf[3*I + 0]), 
							static_cast<int32>(ABuf[3*I + 1]), 
							static_cast<int32>(ABuf[3*I + 2]), 
							0);

						const VectorRegister4Int ScaledValue = VectorIntMultiply(Value, StaurationFactorRGB);
						
						alignas(VectorRegister4Int) int32 IndexableScaledValue[4];
						VectorIntStoreAligned(ScaledValue, IndexableScaledValue);

						// Add scaled values and divide by 255
						const VectorRegister4Int L = VectorShiftRightImmLogical(
							VectorIntMultiply(
								VectorIntSet1(IndexableScaledValue[0] + IndexableScaledValue[1] + IndexableScaledValue[2]), 
								VectorIntSet1(32897)), 23);
						
						const VectorRegister4Int Lerp = VectorIntAdd(VectorIntMultiply(L, OneMinusF), VectorIntMultiply(Value, F));
					
						alignas(VectorRegister4Int) int32 IndexableResult[4];

						// divide by 255 and store. do we need to clamp?
						VectorIntStoreAligned(VectorShiftRightImmLogical(VectorIntMultiply(Lerp, VectorIntSet1(32897)), 23),  IndexableResult);

						ABuf[3*I + 0] = static_cast<uint8>(IndexableResult[0]);
						ABuf[3*I + 1] = static_cast<uint8>(IndexableResult[1]);
						ABuf[3*I + 2] = static_cast<uint8>(IndexableResult[2]);
					}
				}
				break;
			}

			case EImageFormat::RGBA_UByte:
			{
				if constexpr (!bUseVectorImpl)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint16 PixelData[4] = {ABuf[4*I + 0], ABuf[4*I + 1], ABuf[4*I + 2], ABuf[4*I + 3]};

						const uint16 LumTimesFactor = ((76 * PixelData[0] + 150 * PixelData[1] + 29 * PixelData[2]) / 255) * (255 - FactorUNorm);

						ABuf[4*I + 0] = static_cast<uint8>((PixelData[0] * FactorUNorm + LumTimesFactor) / 255);
						ABuf[4*I + 1] = static_cast<uint8>((PixelData[1] * FactorUNorm + LumTimesFactor) / 255);
						ABuf[4*I + 2] = static_cast<uint8>((PixelData[2] * FactorUNorm + LumTimesFactor) / 255);
					}
				}
				else // if constexpr bUseVectorImpl
				{
					constexpr VectorRegister4Int StaurationFactorRGB = MakeVectorRegisterIntConstant(76, 150, 29, 0);
					constexpr VectorRegister4Int Value255 = MakeVectorRegisterIntConstant(255, 255, 255, 255);
					const VectorRegister4Int F = VectorIntSet1(FactorUNorm);
					const VectorRegister4Int OneMinusF = VectorIntSubtract(Value255, F);
					
					for (int32 I = 0; I < NumElems; ++I)
					{
						const VectorRegister4Int Value = MakeVectorRegisterInt(
							static_cast<int32>(ABuf[4*I + 0]), 
							static_cast<int32>(ABuf[4*I + 1]), 
							static_cast<int32>(ABuf[4*I + 2]), 
							0);

						const VectorRegister4Int ScaledValue = VectorIntMultiply(Value, StaurationFactorRGB);
						
						alignas(VectorRegister4Int) int32 IndexableScaledValue[4];
						VectorIntStoreAligned(ScaledValue, IndexableScaledValue);

						// Add scaled values and divide by 255
						const VectorRegister4Int L = VectorShiftRightImmLogical(
							VectorIntMultiply(
								VectorIntSet1(IndexableScaledValue[0] + IndexableScaledValue[1] + IndexableScaledValue[2]), 
								VectorIntSet1(32897)), 23);
						
						const VectorRegister4Int Lerp = VectorIntAdd(VectorIntMultiply(L, OneMinusF), VectorIntMultiply(Value, F));
					
						alignas(VectorRegister4Int) int32 IndexableResult[4];

						// divide by 255 and store. do we need to clamp?
						VectorIntStoreAligned(VectorShiftRightImmLogical(VectorIntMultiply(Lerp, VectorIntSet1(32897)), 23),  IndexableResult);

						ABuf[4*I + 0] = static_cast<uint8>(IndexableResult[0]);
						ABuf[4*I + 1] = static_cast<uint8>(IndexableResult[1]);
						ABuf[4*I + 2] = static_cast<uint8>(IndexableResult[2]);
					}
				}	

				break;
			}

			case EImageFormat::BGRA_UByte:
			{
				if constexpr (!bUseVectorImpl)
				{
					for (int32 I = 0; I < NumElems; ++I)
					{
						uint16 PixelData[4] = {ABuf[4*I + 0], ABuf[4*I + 1], ABuf[4*I + 2], ABuf[4*I + 3]};

						const uint16 LumTimesFactor = ((29 * PixelData[0] + 150 * PixelData[1] + 76 * PixelData[2]) / 255) * (255 - FactorUNorm);

						ABuf[4*I + 0] = static_cast<uint8>((PixelData[0] * FactorUNorm + LumTimesFactor) / 255);
						ABuf[4*I + 1] = static_cast<uint8>((PixelData[1] * FactorUNorm + LumTimesFactor) / 255);
						ABuf[4*I + 2] = static_cast<uint8>((PixelData[2] * FactorUNorm + LumTimesFactor) / 255);
					}
				}
				else // if constexpr bUseVectorImpl
				{
					constexpr VectorRegister4Int StaurationFactorRGB = MakeVectorRegisterIntConstant(29, 150, 76, 0);
					constexpr VectorRegister4Int Value255 = MakeVectorRegisterIntConstant(255, 255, 255, 255);
					const VectorRegister4Int F = VectorIntSet1(FactorUNorm);
					const VectorRegister4Int OneMinusF = VectorIntSubtract(Value255, F);
					for (int32 I = 0; I < NumElems; ++I)
					{
						const VectorRegister4Int Value = MakeVectorRegisterInt(
							static_cast<int32>(ABuf[4*I + 0]), 
							static_cast<int32>(ABuf[4*I + 1]), 
							static_cast<int32>(ABuf[4*I + 2]), 
							0);

						const VectorRegister4Int ScaledValue = VectorIntMultiply(Value, StaurationFactorRGB);
						
						alignas(VectorRegister4Int) int32 IndexableScaledValue[4];
						VectorIntStoreAligned(ScaledValue, IndexableScaledValue);

						// Add scaled values and divide by 255
						const VectorRegister4Int L = VectorShiftRightImmLogical(
							VectorIntMultiply(
								VectorIntSet1(IndexableScaledValue[0] + IndexableScaledValue[1] + IndexableScaledValue[2]), 
								VectorIntSet1(32897)), 23);
						
						const VectorRegister4Int Lerp = VectorIntAdd(VectorIntMultiply(L, OneMinusF), VectorIntMultiply(Value, F));
					
						alignas(VectorRegister4Int) int32 IndexableResult[4];

						// divide by 255 and store. do we need to clamp?
						VectorIntStoreAligned(VectorShiftRightImmLogical(VectorIntMultiply(Lerp, VectorIntSet1(32897)), 23),  IndexableResult);

						ABuf[4*I + 0] = static_cast<uint8>(IndexableResult[0]);
						ABuf[4*I + 1] = static_cast<uint8>(IndexableResult[1]);
						ABuf[4*I + 2] = static_cast<uint8>(IndexableResult[2]);
					}
				}	
				
				break;
			}

			default:
				check(false);
			}
		});
	}
}
