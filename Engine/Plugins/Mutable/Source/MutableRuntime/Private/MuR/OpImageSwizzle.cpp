// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ImagePrivate.h"
#include "MuR/Operations.h"
#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{
	void ImageSwizzle(FImage* Result, const TManagedPtr<const FImage> Sources[], const uint8 Channels[])
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageSwizzle);

		EImageFormat Format = Result->GetFormat();

		// LODs may not match due to bugs, only precess the common avaliable lODs.
		int32 NumLODs = Result->GetLODCount();

        uint16 NumChannels = GetImageFormatData(Format).Channels;
		for (uint16 C = 0; C < NumChannels; ++C)
		{
			NumLODs = Sources[C] ? FMath::Min(NumLODs, Sources[C]->GetLODCount()) : NumLODs;
		}

		int32 NumDestChannels = 0;

		switch (Format)
		{
		case EImageFormat::L_UByte:
			NumDestChannels = 1;
			break;

		case EImageFormat::RGB_UByte:
			NumDestChannels = 3;
			break;

        case EImageFormat::RGBA_UByte:
        case EImageFormat::BGRA_UByte:
			NumDestChannels = 4;
			break;

        default:
			check(false);
		}

		for (int32 Channel = 0; Channel < NumDestChannels; ++Channel)
		{
			const FImage* Src = Sources[Channel].Get();
			
			const int32 DestChannel = Format == EImageFormat::BGRA_UByte && Channel < 3
								    ? FMath::Abs(Channel - 2)
									: Channel;

			bool bFilled = false;

			constexpr int32 NumBatchElems = 4096*2;

			if (Sources[Channel])
			{
				EImageFormat SrcFormat = Sources[Channel]->GetFormat();
				switch (SrcFormat)
				{
				case EImageFormat::L_UByte:
				{
					if (Channels[Channel] < 1)
					{
						const int32 NumBatches = Src->DataStorage.GetNumBatchesLODRange(NumBatchElems, 1, 0, NumLODs);
						check(NumBatches == Result->DataStorage.GetNumBatchesLODRange(NumBatchElems, NumDestChannels, 0, NumLODs));
		
						int32 SrcChannel = Channels[Channel];

						ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
						[
							Result, Src, Sources, NumDestChannels, DestChannel, SrcChannel, NumBatchElems, NumLODs
						](int32 BatchId)
						{
							TArrayView<const uint8> SrcView = Src->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, 1, 0, NumLODs);
							TArrayView<uint8> ResultView = Result->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, NumDestChannels, 0, NumLODs);

							const int32 NumElems = SrcView.Num() / 1;
							check(NumElems == ResultView.Num() / NumDestChannels);

							uint8* DestBuf = ResultView.GetData() + DestChannel;
							const uint8* SrcBuf = SrcView.GetData() + SrcChannel;

							for (int32 I = 0; I < NumElems; ++I)
							{
								DestBuf[I*NumDestChannels] = SrcBuf[I];
							}
						});

						bFilled = true;
					}
					break;
				}
				case EImageFormat::RGB_UByte:
				{
					if (Channels[Channel] < 3)
					{
						const int32 NumBatches = Src->DataStorage.GetNumBatchesLODRange(NumBatchElems, 3, 0, NumLODs);
						check(NumBatches == Result->DataStorage.GetNumBatchesLODRange(NumBatchElems, NumDestChannels, 0, NumLODs));
		
						const int32 SrcChannel = Channels[Channel];

						ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
						[
							Result, Src, Sources, NumDestChannels, DestChannel, SrcChannel, NumBatchElems, NumLODs
						](int32 BatchId)
						{
							TArrayView<const uint8> SrcView = Src->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, 3, 0, NumLODs);
							TArrayView<uint8> ResultView = Result->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, NumDestChannels, 0, NumLODs);

							const int32 NumElems = SrcView.Num() / 3;
							check(NumElems == ResultView.Num()/NumDestChannels);

							uint8* DestBuf = ResultView.GetData() + DestChannel;
							const uint8* SrcBuf = SrcView.GetData() + SrcChannel;
							
							for (int32 I = 0; I < NumElems; ++I)
							{
								DestBuf[I*NumDestChannels] = SrcBuf[I*3];
							}
						});

						bFilled = true;
					}
					break;
				}
				case EImageFormat::RGBA_UByte:
				case EImageFormat::BGRA_UByte:
				{
					if (Channels[Channel] < 4)
					{
						const int32 SrcChannel = SrcFormat == EImageFormat::BGRA_UByte && Channels[Channel] < 3
											   ? FMath::Abs(int32(Channels[Channel]) - 2)
											   : Channels[Channel];

						const int32 NumBatches = Src->DataStorage.GetNumBatchesLODRange(NumBatchElems, 4, 0, NumLODs);
						check(NumBatches == Result->DataStorage.GetNumBatchesLODRange(NumBatchElems, NumDestChannels, 0, NumLODs));
			
						ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
						[
							Result, Src, Sources, NumDestChannels, DestChannel, SrcChannel, NumBatchElems, NumLODs
						](int32 BatchId)
						{
							TArrayView<const uint8> SrcView = Src->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, 4, 0, NumLODs);  
							TArrayView<uint8> ResultView = Result->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, NumDestChannels, 0, NumLODs);  

							const int32 NumElems = SrcView.Num() / 4;
							check(NumElems == ResultView.Num()/NumDestChannels);

							uint8* DestBuf = ResultView.GetData() + DestChannel;
							const uint8* SrcBuf = SrcView.GetData() + SrcChannel;
							
							for (int32 I = 0; I < NumElems; ++I)
							{
								DestBuf[I*NumDestChannels] = SrcBuf[I*4];
							}
						});

						bFilled = true;
					}
					break;
				}
				default:
				{
					check(false);
				}
				}
			}

			if (!bFilled)
			{
				const int32 NumBatches = Result->DataStorage.GetNumBatchesLODRange(NumBatchElems, NumDestChannels, 0, NumLODs);

				// Alpha is expected to be filled with 1.
				const uint8 FillValue = DestChannel < 3 ? 0 : 255;

				ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
				[
					Result, NumDestChannels, NumBatchElems, NumLODs, DestChannel, FillValue
				](int32 BatchId)
				{
					TArrayView<uint8> ResultView = Result->DataStorage.GetBatchLODRange(BatchId, NumBatchElems, NumDestChannels, 0, NumLODs);
					int32 NumElems = ResultView.Num() / NumDestChannels;

					uint8* DestBuf = ResultView.GetData() + DestChannel;

					for (int32 I = 0; I < NumElems; ++I)
					{
						DestBuf[I * NumDestChannels] = FillValue;
					}
				});
			}
		}
	}

	TManagedPtr<FImage> FImageOperator::ImageSwizzle(EImageFormat Format, const TManagedPtr<const FImage> Sources[], const uint8 Channels[])
	{
		MUTABLE_CPUPROFILER_SCOPE(ImageSwizzle);

		int32 FirstValidSourceIndex = -1;
		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Sources[SourceIndex])
			{
				FirstValidSourceIndex = SourceIndex;
				break;
			}
		}

		if (FirstValidSourceIndex < 0)
		{
			return nullptr;
		}

		const FImageSize ResultSize = Sources[FirstValidSourceIndex]->GetSize();
		const int32 ResultNumLODs = Sources[FirstValidSourceIndex]->GetLODCount();

		TManagedPtr<FImage> Dest = CreateImage(ResultSize.X, ResultSize.Y, ResultNumLODs, Format, EInitializationType::Black);

		UE::Mutable::Private::ImageSwizzle(Dest.Get(), Sources, Channels);

		return Dest;
	}

}
