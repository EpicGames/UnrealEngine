// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"
#include "MuR/MutableTrace.h"

#include "MuR/OpImageTransformPrivate.h"

#include "Math/IntRect.h"
#include "Math/Box2D.h"

namespace UE::Mutable::Private
{
	inline void ImageTransform(
			FImage* pDestImage, const FImage* pImage,
			FTransform2f Transform, float MipFactor, EAddressMode AddressMode, bool bUseVectorImplCvar)
    {
		MUTABLE_CPUPROFILER_SCOPE(ImageTransform);

		if (!pImage->GetSizeX() || !pImage->GetSizeY())
		{
			return;
		}

		const FIntVector2 SrcSize  = FIntVector2(pImage->GetSizeX(), pImage->GetSizeY());
		const FIntVector2 DestSize = FIntVector2(pDestImage->GetSizeX(), pDestImage->GetSizeY());

		const FIntRect DestCropRect = Invoke([&]()
		{
			if (AddressMode != EAddressMode::ClampToBlack)
			{
				return FIntRect(0, 0, DestSize.X, DestSize.Y);
			}
	
			FBox2f NormalizedCropRect(ForceInit);
			NormalizedCropRect += Transform.TransformPoint(FVector2f(0.0f, 0.0f));
			NormalizedCropRect += Transform.TransformPoint(FVector2f(1.0f, 0.0f));
			NormalizedCropRect += Transform.TransformPoint(FVector2f(0.0f, 1.0f));
			NormalizedCropRect += Transform.TransformPoint(FVector2f(1.0f, 1.0f));

			const FIntRect CropRect(
				FInt32Point(FMath::FloorToInt(NormalizedCropRect.Min.X * (float)DestSize.X), 
							FMath::FloorToInt(NormalizedCropRect.Min.Y * (float)DestSize.Y)),
				FInt32Point(FMath::CeilToInt (NormalizedCropRect.Max.X * (float)DestSize.X), 
							FMath::CeilToInt (NormalizedCropRect.Max.Y * (float)DestSize.Y)));

			return FIntRect(
				FMath::Clamp(CropRect.Min.X, 0, DestSize.X), FMath::Clamp(CropRect.Min.Y, 0, DestSize.Y),
				FMath::Clamp(CropRect.Max.X, 0, DestSize.X), FMath::Clamp(CropRect.Max.Y, 0, DestSize.Y));
		});

		if (AddressMode == EAddressMode::ClampToBlack)
		{
			pDestImage->Flags |= FImage::IF_HAS_RELEVANCY_MAP;
			pDestImage->RelevancyMinY = DestCropRect.Min.Y;
			pDestImage->RelevancyMaxY = FMath::Max(DestCropRect.Max.Y - 1, DestCropRect.Min.Y);
		}

		uint8* DestData       = pDestImage->GetMipData(0);
		const uint8* Src0Data = pImage->GetMipData(0);
		const uint8* Src1Data = pImage->GetLODCount() > 1 ? pImage->GetMipData(1) : Src0Data;

		const FIntVector2 Src0Size = FIntVector2(pImage->GetSizeX(), pImage->GetSizeY());
		const FIntVector2 Src1Size = Src1Data != Src0Data ? pImage->CalculateMipSize(1) : Src0Size;

		switch (pDestImage->GetFormat())
		{
		
		case EImageFormat::L_UByte:
		{

			if (bUseVectorImplCvar)
			{
				constexpr bool bUseVectorImpl = true;
				switch (AddressMode)
				{
				case EAddressMode::ClampToBlack:
					ImageTransformImpl<1, EAddressMode::ClampToBlack, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::ClampToEdge:
					ImageTransformImpl<1, EAddressMode::ClampToEdge, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::Wrap:
					ImageTransformImpl<1, EAddressMode::Wrap, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				default:
					check(false);
					break;
				}
			}
			else
			{
				constexpr bool bUseVectorImpl = false;
				switch (AddressMode)
				{
				case EAddressMode::ClampToBlack:
					ImageTransformImpl<1, EAddressMode::ClampToBlack, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::ClampToEdge:
					ImageTransformImpl<1, EAddressMode::ClampToEdge, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::Wrap:
					ImageTransformImpl<1, EAddressMode::Wrap, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				default:
					check(false);
					break;
				}
			}
			break;
		}
		
		case EImageFormat::RGB_UByte:
		{
			if (bUseVectorImplCvar)
			{
				constexpr bool bUseVectorImpl = true;
				switch (AddressMode)
				{
				case EAddressMode::ClampToBlack:
					ImageTransformImpl<3, EAddressMode::ClampToBlack, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::ClampToEdge:
					ImageTransformImpl<3, EAddressMode::ClampToEdge, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::Wrap:
					ImageTransformImpl<3, EAddressMode::Wrap, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				default:
					check(false);
					break;
				}
			}
			else
			{
				constexpr bool bUseVectorImpl = false;
				switch (AddressMode)
				{
				case EAddressMode::ClampToBlack:
					ImageTransformImpl<3, EAddressMode::ClampToBlack, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::ClampToEdge:
					ImageTransformImpl<3, EAddressMode::ClampToEdge, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::Wrap:
					ImageTransformImpl<3, EAddressMode::Wrap, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				default:
					check(false);
					break;
				}
			}
		
			break;
		}

		case EImageFormat::BGRA_UByte:
		case EImageFormat::RGBA_UByte:
		{
			if (bUseVectorImplCvar)
			{
				constexpr bool bUseVectorImpl = true;
				switch (AddressMode)
				{
				case EAddressMode::ClampToBlack:
					ImageTransformImpl<4, EAddressMode::ClampToBlack, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::ClampToEdge:
					ImageTransformImpl<4, EAddressMode::ClampToEdge, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::Wrap:
					ImageTransformImpl<4, EAddressMode::Wrap, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				default:
					check(false);
					break;
				}
			}
			else
			{
				constexpr bool bUseVectorImpl = false;
				switch (AddressMode)
				{
				case EAddressMode::ClampToBlack:
					ImageTransformImpl<4, EAddressMode::ClampToBlack, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::ClampToEdge:
					ImageTransformImpl<4, EAddressMode::ClampToEdge, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				case EAddressMode::Wrap:
					ImageTransformImpl<4, EAddressMode::Wrap, bUseVectorImpl>(DestData, DestSize, DestCropRect, Src0Data, Src0Size, Src1Data, Src1Size, MipFactor, Transform);
					break;
				default:
					check(false);
					break;
				}
			}
			break;
		}

		default:
			// Case not implemented
            check( false );
		}
    }

}
