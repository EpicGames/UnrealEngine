// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpImageTransformPrivate.h"

#include "MuR/ParallelExecutionUtils.h"

#include "Math/Vector.h"
#include "Math/IntVector.h"
#include "Math/TransformCalculus.h"

namespace UE::Mutable::Private
{

    //---------------------------------------------------------------------------------------------
    //! Image transform, translate, scale, rotate 
    //---------------------------------------------------------------------------------------------

    template<int32 NumChannels, EAddressMode AddressMode>
	void ImageTransformNonVectorImpl(
		uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, 
		const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size,
		float MipFactor,
		const FTransform2f& Transform)
    {
		using FUint16Vector2 = UE::Math::TIntVector2<uint16>;
		struct FPixelData
		{
			alignas(8) uint16 Data[NumChannels];
		};

		const FVector2f DestSizeF = FVector2f(DestSize.X, DestSize.Y);

		const FVector2f DestNormFactor = FVector2f(1.0f) / DestSizeF;

		check(DestCropRect.Min.X >= 0 && DestCropRect.Max.X <= DestSize.X);
		check(DestCropRect.Min.Y >= 0 && DestCropRect.Max.Y <= DestSize.Y);

		const FTransform2f InverseTransform = Transform.Inverse();

		constexpr int32 BatchSize = 1 << 12;

		const int32 NumBatchRows = FMath::DivideAndRoundUp(BatchSize, DestCropRect.Width());
		const int32 NumBatches   = FMath::DivideAndRoundUp(DestCropRect.Height(), NumBatchRows);

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[=](int32 BatchId)
		{
			const int32 RowOffset = NumBatchRows * BatchId;
			const FIntRect BatchRect = FIntRect(
				DestCropRect.Min.X, DestCropRect.Min.Y + RowOffset,
				DestCropRect.Max.X, FMath::Min(DestCropRect.Min.Y + RowOffset + NumBatchRows, DestCropRect.Max.Y));

			const FVector2f Size0F(Src0Size.X, Src0Size.Y);
			const FVector2f Size1F(Src1Size.X, Src1Size.Y);

			for (int32 Y = BatchRect.Min.Y; Y < BatchRect.Max.Y; ++Y)
			{
				for (int32 X = BatchRect.Min.X; X < BatchRect.Max.X; ++X)
				{
					const FVector2f DestUv = InverseTransform.TransformPoint((FVector2f(X, Y) + 0.5f) * DestNormFactor);

					// TODO: Make the black edge antialiazed?
					if constexpr (AddressMode == EAddressMode::ClampToBlack)
					{
						if ((DestUv.X < 0.0f) | (DestUv.X > 1.0f) | (DestUv.Y < 0.0f) | (DestUv.Y > 1.0f))
						{
							// Black texture init is mandatory for ClampToBlack Address mode, no need to set black again. 
							//if constexpr (NumChannels == 4)
							//{
							//	FMemory::Memzero(&DestData[(Y * DestSize.X + X) * NumChannels], 4);
							//}
							//else
							//{
							//	for (uint32 Channel = 0; Channel < NumChannels; ++Channel)
							//	{
							//		DestData[(Y * DestSize.X + X) * NumChannels + Channel] = 0;
							//	}
							//}
							continue;
						}
					}

					const FVector2f Uv = Invoke([&]() -> FVector2f
					{
						if constexpr (AddressMode == EAddressMode::Wrap)
						{
							return FVector2f(FMath::Frac(DestUv.X), FMath::Frac(DestUv.Y));
						}
						else if (AddressMode == EAddressMode::ClampToBlack || AddressMode == EAddressMode::ClampToEdge)
						{
							return FVector2f(FMath::Clamp(DestUv.X, 0.0f, 1.0f), FMath::Clamp(DestUv.Y, 0.0f, 1.0f));
						}
						else
						{
							return DestUv;
						}
					});

					const FVector2f Coords0F = FVector2f(
						FMath::Clamp(Uv.X * Size0F.X - 0.5f, 0.0f, Size0F.X - 1.0f),
						FMath::Clamp(Uv.Y * Size0F.Y - 0.5f, 0.0f, Size0F.Y - 1.0f));

					const FUint16Vector2 Frac0 = FUint16Vector2(FMath::Frac(Coords0F.X) * 255.0f, FMath::Frac(Coords0F.Y) * 255.0f);

					const FIntVector2 Coords0 = FIntVector2(Coords0F.X, Coords0F.Y);
					const FIntVector2 Coords0PlusOne = Invoke([&]() -> FIntVector2
					{
						if constexpr (AddressMode == EAddressMode::Wrap)
						{
							return FIntVector2(Coords0.X + 1 < Src0Size.X ? Coords0.X + 1 : 0,
								Coords0.Y + 1 < Src0Size.Y ? Coords0.Y + 1 : 0);
						}
						else if (AddressMode == EAddressMode::ClampToBlack || AddressMode == EAddressMode::ClampToEdge)
						{
							return FIntVector2(FMath::Min(Src0Size.X - 1, Coords0.X + 1),
								FMath::Min(Src0Size.Y - 1, Coords0.Y + 1));
						}
					});

					const FVector2f Coords1F = FVector2f(
						FMath::Clamp(Uv.X * Size1F.X - 0.5f, 0.0f, Size1F.X - 1.0f),
						FMath::Clamp(Uv.Y * Size1F.Y - 0.5f, 0.0f, Size1F.Y - 1.0f));

					const FUint16Vector2 Frac1 = FUint16Vector2(FMath::Frac(Coords1F.X) * 255.0f, FMath::Frac(Coords1F.Y) * 255.0f);
					
					const FIntVector2 Coords1 = FIntVector2(Coords1F.X, Coords1F.Y);
					const FIntVector2 Coords1PlusOne = Invoke([&]() -> FIntVector2
					{
						if constexpr (AddressMode == EAddressMode::Wrap)
						{
							return FIntVector2(Coords1.X + 1 < Src1Size.X ? Coords1.X + 1 : 0,
								Coords1.Y + 1 < Src1Size.Y ? Coords1.Y + 1 : 0);
						}
						else if (AddressMode == EAddressMode::ClampToBlack || AddressMode == EAddressMode::ClampToEdge)
						{
							return FIntVector2(FMath::Min(Src1Size.X - 1, Coords1.X + 1),
								FMath::Min(Src1Size.Y - 1, Coords1.Y + 1));
						}
					});

					auto LoadPixel = [](const uint8* Ptr) -> FPixelData
					{
						FPixelData Result;
						if constexpr (NumChannels == 4)
						{
							uint32 PackedData;
							FMemory::Memcpy(&PackedData, Ptr, sizeof(uint32));

							Result.Data[0] = static_cast<uint16>((PackedData >> (8 * 0)) & 0xFF);
							Result.Data[1] = static_cast<uint16>((PackedData >> (8 * 1)) & 0xFF);
							Result.Data[2] = static_cast<uint16>((PackedData >> (8 * 2)) & 0xFF);
							Result.Data[3] = static_cast<uint16>((PackedData >> (8 * 3)) & 0xFF);
						}
						else
						{
							for (int32 C = 0; C < NumChannels; ++C)
							{
								Result.Data[C] = static_cast<uint16>(Ptr[C]);
							}
						}

						return Result;
					};

					FPixelData PixelData000 = LoadPixel(Src0Data + (Coords0.Y * Src0Size.X + Coords0.X) * NumChannels);
					FPixelData PixelData010 = LoadPixel(Src0Data + (Coords0.Y * Src0Size.X + Coords0PlusOne.X) * NumChannels);
					FPixelData PixelData001 = LoadPixel(Src0Data + (Coords0PlusOne.Y * Src0Size.X + Coords0.X) * NumChannels);
					FPixelData PixelData011 = LoadPixel(Src0Data + (Coords0PlusOne.Y * Src0Size.X + Coords0PlusOne.X) * NumChannels);
                                                                                                                                  
					FPixelData PixelData100 = LoadPixel(Src1Data + (Coords1.Y * Src1Size.X + Coords1.X) * NumChannels);
					FPixelData PixelData110 = LoadPixel(Src1Data + (Coords1.Y * Src1Size.X + Coords1PlusOne.X) * NumChannels);
					FPixelData PixelData101 = LoadPixel(Src1Data + (Coords1PlusOne.Y * Src1Size.X + Coords1.X) * NumChannels);
					FPixelData PixelData111 = LoadPixel(Src1Data + (Coords1PlusOne.Y * Src1Size.X + Coords1PlusOne.X) * NumChannels);

					FPixelData ResultPixelData;
					for (int32 C = 0; C < NumChannels; ++C)
					{
						const uint16 LerpY00 = ((PixelData010.Data[C] * Frac0.X) + PixelData000.Data[C] * (255 - Frac0.X)) / 255;
						const uint16 LerpY01 = ((PixelData011.Data[C] * Frac0.X) + PixelData001.Data[C] * (255 - Frac0.X)) / 255;

						const uint16 LerpY10 = ((PixelData110.Data[C] * Frac1.X) + PixelData100.Data[C] * (255 - Frac1.X)) / 255;
						const uint16 LerpY11 = ((PixelData111.Data[C] * Frac1.X) + PixelData101.Data[C] * (255 - Frac1.X)) / 255;
						
						const uint16 Mip0 = ((LerpY01 * Frac0.Y) + LerpY00 * (255 - Frac0.Y)) / 255;
						const uint16 Mip1 = ((LerpY11 * Frac1.Y) + LerpY10 * (255 - Frac1.Y)) / 255;

						ResultPixelData.Data[C] = ((Mip0 * MipFactor) + Mip1 * (255 - MipFactor)) / 255;
					}

					static_assert(NumChannels <= 4);

					if constexpr (NumChannels == 4)
					{
						const uint32 PackedPixel = 
							(static_cast<uint32>((ResultPixelData.Data[0]) & 0xFF) << (8 * 0)) | 
							(static_cast<uint32>((ResultPixelData.Data[1]) & 0xFF) << (8 * 1)) | 
							(static_cast<uint32>((ResultPixelData.Data[2]) & 0xFF) << (8 * 2)) | 
							(static_cast<uint32>((ResultPixelData.Data[3]) & 0xFF) << (8 * 3));
						*reinterpret_cast<uint32*>(&DestData[(Y * DestSize.X + X) * NumChannels]) = PackedPixel;
					}
					else
					{
						for (uint32 Channel = 0; Channel < NumChannels; ++Channel)
						{
							DestData[(Y * DestSize.X + X) * NumChannels + Channel] = 
								static_cast<uint8>(ResultPixelData.Data[Channel]);
						}
					}
				}
			}
		});
    }

    template<int32 NumChannels, EAddressMode AddressMode>
	void ImageTransformVectorImpl(
		uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, 
		const uint8* Src0Data, FIntVector2 Src0Size,
		const uint8* Src1Data, FIntVector2 Src1Size,
		float MipFactor,
		const FTransform2f& Transform)
    {
		if (DestCropRect.Area() == 0)
		{
			return;
		}

		const FVector2f DestSizeF = FVector2f(DestSize.X, DestSize.Y);
		const FVector2f DestNormFactor = FVector2f(1.0f) / DestSizeF;

		check(DestCropRect.Min.X >= 0 && DestCropRect.Max.X <= DestSize.X);
		check(DestCropRect.Min.Y >= 0 && DestCropRect.Max.Y <= DestSize.Y);

		const FTransform2f InverseTransform = Transform.Inverse();

		using namespace GlobalVectorConstants;

		const VectorRegister4Int SizeI = 
			MakeVectorRegisterInt(Src0Size.X, Src0Size.Y, Src1Size.X, Src1Size.Y);

		const VectorRegister4Float SizeF = VectorIntToFloat(SizeI);

		constexpr int32 BatchSize = 1 << 12;

		const int32 NumBatchRows = FMath::DivideAndRoundUp(BatchSize, DestCropRect.Width());
		const int32 NumBatches   = FMath::DivideAndRoundUp(DestCropRect.Height(), NumBatchRows);

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[=](int32 BatchId)
		{
			const int32 RowOffset = NumBatchRows * BatchId;
			const FIntRect BatchRect = FIntRect(
				DestCropRect.Min.X, DestCropRect.Min.Y + RowOffset,
				DestCropRect.Max.X, FMath::Min(DestCropRect.Min.Y + RowOffset + NumBatchRows, DestCropRect.Max.Y));
			
			const VectorRegister4Float VectorMipFactor = VectorSetFloat1(MipFactor); 

			for (int32 Y = BatchRect.Min.Y; Y < BatchRect.Max.Y; ++Y)
			{
				for (int32 X = BatchRect.Min.X; X < BatchRect.Max.X; ++X)
				{
					const FVector2f DestUv = InverseTransform.TransformPoint((FVector2f(X, Y) + 0.5f) * DestNormFactor);

					// TODO: Make the black edge antialiazed.
					if constexpr (AddressMode == EAddressMode::ClampToBlack)
					{
						if ((DestUv.X < 0.0f) | (DestUv.X > 1.0f) | (DestUv.Y < 0.0f) | (DestUv.Y > 1.0f))
						{
							// Black texture init is mandatory for ClampToBlack Address mode, no need to set black again. 
							//if constexpr (NumChannels == 4)
							//{
							//	FMemory::Memzero(&DestData[(Y * DestSize.X + X) * NumChannels], 4);
							//}
							//else
							//{
							//	for (uint32 Channel = 0; Channel < NumChannels; ++Channel)
							//	{
							//		DestData[(Y * DestSize.X + X) * NumChannels + Channel] = 0;
							//	}
							//}

							continue;
						}
					}

					const FVector2f Uv = Invoke([&]() -> FVector2f
					{
						if constexpr (AddressMode == EAddressMode::Wrap)
						{
							return FVector2f(FMath::Frac(DestUv.X), FMath::Frac(DestUv.Y));
						}
						else if (AddressMode == EAddressMode::ClampToBlack || AddressMode == EAddressMode::ClampToEdge)
						{
							return FVector2f(FMath::Clamp(DestUv.X, 0.0f, 1.0f), FMath::Clamp(DestUv.Y, 0.0f, 1.0f));
						}
						else
						{
							return DestUv;
						}
					});

					const VectorRegister4Float CoordsF = VectorMin(VectorMax(
						VectorMultiplyAdd(MakeVectorRegister(Uv.X, Uv.Y, Uv.X, Uv.Y), SizeF, FloatMinusOneHalf),
						FloatZero), 
						VectorSubtract(SizeF, FloatOne));

					const VectorRegister4Float Frac = VectorSubtract(CoordsF, VectorFloor(CoordsF));
					
					const VectorRegister4Int CoordsI = VectorFloatToInt(CoordsF);
					const VectorRegister4Int CoordsIPlusOne = VectorIntMin(
							VectorIntAdd(CoordsI, IntOne), VectorIntSubtract(SizeI, IntOne));

					alignas(VectorRegister4Int) int32 CoordsIData[4];
					VectorIntStoreAligned(CoordsI, CoordsIData);

					alignas(VectorRegister4Int) int32 CoordsIPlusOneData[4];
					VectorIntStoreAligned(CoordsIPlusOne, CoordsIPlusOneData);

					auto LoadPixel = [](const uint8* DataPtr) -> VectorRegister4Float
					{
						if constexpr (NumChannels == 4)
						{
							uint32 PackedData;
							FMemory::Memcpy(&PackedData, DataPtr, sizeof(uint32));

							return VectorIntToFloat(MakeVectorRegisterInt(
										(PackedData >> (8 * 0)) & 0xFF, 
										(PackedData >> (8 * 1)) & 0xFF,
										(PackedData >> (8 * 2)) & 0xFF,
										(PackedData >> (8 * 3)) & 0xFF));
						}
						else
						{
							alignas(VectorRegister4Int) int32 PixelData[4] = {0};
							for (int32 C = 0; C < NumChannels; ++C)
							{
								PixelData[C] = static_cast<int32>(DataPtr[C]);
							}

							return VectorIntToFloat(VectorIntLoadAligned(&PixelData[0]));
						}
					};

					const VectorRegister4Float Pixel000 = LoadPixel(Src0Data + (CoordsIData[1]        * Src0Size.X + CoordsIData[0])        * NumChannels);
					const VectorRegister4Float Pixel010 = LoadPixel(Src0Data + (CoordsIData[1]        * Src0Size.X + CoordsIPlusOneData[0]) * NumChannels);
					const VectorRegister4Float Pixel001 = LoadPixel(Src0Data + (CoordsIPlusOneData[1] * Src0Size.X + CoordsIData[0])        * NumChannels);
					const VectorRegister4Float Pixel011 = LoadPixel(Src0Data + (CoordsIPlusOneData[1] * Src0Size.X + CoordsIPlusOneData[0]) * NumChannels);
                                                                                                                                                         
					const VectorRegister4Float Pixel100 = LoadPixel(Src1Data + (CoordsIData[3]        * Src1Size.X + CoordsIData[2])        * NumChannels);
					const VectorRegister4Float Pixel110 = LoadPixel(Src1Data + (CoordsIData[3]        * Src1Size.X + CoordsIPlusOneData[2]) * NumChannels);
					const VectorRegister4Float Pixel101 = LoadPixel(Src1Data + (CoordsIPlusOneData[3] * Src1Size.X + CoordsIData[2])        * NumChannels);
					const VectorRegister4Float Pixel111 = LoadPixel(Src1Data + (CoordsIPlusOneData[3] * Src1Size.X + CoordsIPlusOneData[2]) * NumChannels);
					
					const VectorRegister4Float Frac0X = VectorReplicate(Frac, 0);
					const VectorRegister4Float Frac0Y = VectorReplicate(Frac, 1);

					// Bilerp image 0
					VectorRegister4Float Sample0 = VectorMultiplyAdd(Frac0X, VectorSubtract(Pixel010, Pixel000), Pixel000);
					Sample0 = VectorMultiplyAdd(
						Frac0Y, 
						VectorSubtract(
							VectorMultiplyAdd(Frac0X, VectorSubtract(Pixel011, Pixel001), Pixel001), 
							Sample0), 
						Sample0);

					const VectorRegister4Float Frac1X = VectorReplicate(Frac, 2);
					const VectorRegister4Float Frac1Y = VectorReplicate(Frac, 3);

					// Bilerp image 1
					VectorRegister4Float Sample1 = VectorMultiplyAdd(Frac1X, VectorSubtract(Pixel110, Pixel100), Pixel100);
					Sample1 = VectorMultiplyAdd(
						Frac1Y, 
						VectorSubtract(
							VectorMultiplyAdd(Frac1X, VectorSubtract(Pixel111, Pixel101), Pixel101), 
							Sample1), 
						Sample1);

					const VectorRegister4Float Result =
						VectorMultiplyAdd(VectorMipFactor, VectorSubtract(Sample1, Sample0), Sample0);

					if constexpr (NumChannels == 4)
					{
						VectorStoreByte4(Result, &DestData[(Y * DestSize.X + X) * NumChannels]);
						VectorResetFloatRegisters();
					}
					else
					{
						const AlignedFloat4 ResultData(VectorMin(Result, Float255));
						for (int32 C = 0; C < NumChannels; ++C)
						{
							DestData[(Y * DestSize.X + X) * NumChannels + C] = static_cast<uint8>(ResultData[C]);		
						}				
					}
				}
			}
		});
    }

    template<int32 NumChannels, EAddressMode AddressMode, bool bUseVectorImpl>
	void ImageTransformImpl(
		uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, 
		const uint8* Src0Data, FIntVector2 Src0Size,
		const uint8* Src1Data, FIntVector2 Src1Size,
		float MipFactor,
		const FTransform2f& Transform)
	{
		if constexpr (bUseVectorImpl)
		{
			ImageTransformVectorImpl<NumChannels, AddressMode>(
				DestData, DestSize, DestCropRect,
				Src0Data, Src0Size, Src1Data, Src1Size,
				MipFactor,
				Transform);
		}
		else
		{
			ImageTransformNonVectorImpl<NumChannels, AddressMode>(
				DestData, DestSize, DestCropRect,
				Src0Data, Src0Size, Src1Data, Src1Size,
				MipFactor,
				Transform);
		}
	}

	// Force instantiation of used templates. A link error will occur if any of them is missing.
	template void ImageTransformImpl<1, EAddressMode::ClampToBlack, true>(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<1, EAddressMode::ClampToEdge, true> (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<1, EAddressMode::Wrap, true>        (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<3, EAddressMode::ClampToBlack, true>(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<3, EAddressMode::ClampToEdge, true> (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<3, EAddressMode::Wrap, true>        (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<4, EAddressMode::ClampToBlack, true>(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<4, EAddressMode::ClampToEdge, true> (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<4, EAddressMode::Wrap, true>        (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);

	template void ImageTransformImpl<1, EAddressMode::ClampToBlack, false>(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<1, EAddressMode::ClampToEdge, false> (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<1, EAddressMode::Wrap, false>        (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<3, EAddressMode::ClampToBlack, false>(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<3, EAddressMode::ClampToEdge, false> (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<3, EAddressMode::Wrap, false>        (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<4, EAddressMode::ClampToBlack, false>(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<4, EAddressMode::ClampToEdge, false> (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	template void ImageTransformImpl<4, EAddressMode::Wrap, false>        (uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
}
