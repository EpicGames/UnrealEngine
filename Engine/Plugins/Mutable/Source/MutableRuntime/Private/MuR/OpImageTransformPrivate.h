// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ImagePrivate.h"

#include "Math/Vector.h"
#include "Math/IntVector.h"
#include "Math/TransformCalculus2D.h"

namespace UE::Mutable::Private
{

    //---------------------------------------------------------------------------------------------
    //! Image transform, translate, scale, rotate 
    //---------------------------------------------------------------------------------------------

	template<int32 NumChannels, EAddressMode, bool bUseVectorImpl> 
	void ImageTransformImpl(uint8* DestData, FIntVector2 DestSize, FIntRect DestCropRect, const uint8* Src0Data, FIntVector2 Src0Size, const uint8* Src1Data, FIntVector2 Src1Size, float MipFactor, const FTransform2f& Transform);
	
	// extern template could be used if for some reason we need to provide the full declaration here. 
	//extern template FORCENOINLINE void ImageTransformImpl<1, EAddressMode::ClampToBlack, false>(uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<1, EAddressMode::ClampToEdge, false> (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<1, EAddressMode::Wrap, false>        (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<3, EAddressMode::ClampToBlack, false>(uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<3, EAddressMode::ClampToEdge, false> (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<3, EAddressMode::Wrap, false>        (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<4, EAddressMode::ClampToBlack, false>(uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<4, EAddressMode::ClampToEdge, false> (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<4, EAddressMode::Wrap, false>        (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);

	//extern template FORCENOINLINE void ImageTransformImpl<1, EAddressMode::ClampToBlack, true>(uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<1, EAddressMode::ClampToEdge, true> (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<1, EAddressMode::Wrap, true>        (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<3, EAddressMode::ClampToBlack, true>(uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<3, EAddressMode::ClampToEdge, true> (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<3, EAddressMode::Wrap, true>        (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<4, EAddressMode::ClampToBlack, true>(uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<4, EAddressMode::ClampToEdge, true> (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
	//extern template FORCENOINLINE void ImageTransformImpl<4, EAddressMode::Wrap, true>        (uint8* DestData, FIntRect DestRect, FIntRect DestCropRect, const uint8* SrcData, FIntRect SrcRect, const FTransform2f& Transform);
}
