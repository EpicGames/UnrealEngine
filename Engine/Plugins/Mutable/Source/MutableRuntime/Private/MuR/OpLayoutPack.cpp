// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpLayoutPack.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/Layout.h"
#include "MuR/Platform.h"
#include "GenericPlatform/GenericPlatformMath.h"

namespace UE::Mutable::Private
{

	struct FPackLayoutBlock
	{
		FPackLayoutBlock() {}

		FPackLayoutBlock(int32 InIndex, FIntVector2 InSize, int32 InPriority = 0, bool bInReduceBothAxes = false, bool bInReduceByTwo = false)
		{
			Index = InIndex;
			size = InSize;
			priority = InPriority;
			bReduceBothAxes = bInReduceBothAxes;
			bReduceByTwo = bInReduceByTwo;
		}

		int32 Index = -1;
		FIntVector2 size = { 0, 0 };
		int32 priority = 0;
		bool bReduceBothAxes = false;
		bool bReduceByTwo = false;
	};

	inline bool CompareBlocks(const FPackLayoutBlock& a, const FPackLayoutBlock& b)
	{
		if (a.size[1] > b.size[1])
		{
			return true;
		}
		else if (a.size[1] < b.size[1])
		{
			return false;
		}
		else
		{
			int32 areaA = a.size[0] * a.size[1];
			int32 areaB = b.size[0] * b.size[1];
			if (areaA > areaB)
			{
				return true;
			}
			else if (areaA < areaB)
			{
				return false;
			}
			else
			{
				// This has to be deterministic, and indices are supposed to be unique
				return a.Index < b.Index;
			}
		}
	}

	inline bool CompareBlocksPriority(const FPackLayoutBlock& a, const FPackLayoutBlock& b)
	{
		if (a.priority == b.priority)
		{
			// TODO(Max): Check if this comparison modifies the block order while reducing them.
			return CompareBlocks(a, b);
		}

		return a.priority > b.priority;
	}


	struct FScratchLayoutPack
	{
		TArray< FIntVector2 > blocks;
		TArray< FPackLayoutBlock > sorted;
		TArray< FIntVector2 > positions;
		TArray< int32 > priorities;
		TArray< FIntVector2 > reductions;
		TArray< int32 > ReduceBothAxes;
		TArray< int32 > ReduceByTwo;
	};


    inline char DebugGetBlockAt( const FScratchLayoutPack& scratch,
                                   const TArray<uint8_t>& packedFlag,
                                   int32 x, int32 y )
    {
        for ( size_t b=0; b<packedFlag.Num(); ++b )
        {
            if (packedFlag[b])
            {
				int32 i = scratch.sorted[b].Index;
                if ( x>=scratch.positions[i][0] &&
                     x<scratch.positions[i][0]+scratch.sorted[b].size[0] &&
                     y>=scratch.positions[i][1] &&
                     y<scratch.positions[i][1]+scratch.sorted[b].size[1]
                     )
                {
					return char('a')+char(b);
                }
            }
        }
        return '.';
    }


	inline void ReductionOperation(int32& BlockSize, EReductionMethod ReductionMethod, int32 bReduceByTwo)
	{
		if (ReductionMethod == EReductionMethod::Unitary)
		{
			int32 Reduction = (bReduceByTwo && BlockSize > 2) ? 2 : 1;
			BlockSize -= Reduction;
			return;
		}
		
		// Reduce size by half
		BlockSize /= 2;
	}


    /** Updates the area, and the iterator. */
	inline void ReduceBlock(int32 BlockCount, int32& InOutArea, int32& InOutBlockIt, FScratchLayoutPack& scratch, EReductionMethod ReductionMethod)
	{
		int32 r_it = InOutBlockIt;
		bool pass = false;

		int32 oldBlockArea = scratch.sorted[r_it].size[0] * scratch.sorted[r_it].size[1];

		if (oldBlockArea>0
			&&
			(scratch.sorted[r_it].size[0] != 1 || scratch.sorted[r_it].size[1] != 1))
		{
			if (scratch.sorted[r_it].bReduceBothAxes)
			{
				// We reduce both sides of the block at the same time
				for (int32 Index = 0; Index <= 1; ++Index)
				{
					if (scratch.sorted[r_it].size[Index] > 1)
					{
						ReductionOperation(scratch.sorted[r_it].size[Index], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						ReductionOperation(scratch.blocks[scratch.sorted[r_it].Index][Index], ReductionMethod, scratch.sorted[r_it].bReduceByTwo );

						pass = true;
					}
				}
			}
			else
			{
				if (scratch.reductions[scratch.sorted[r_it].Index][0] > scratch.reductions[scratch.sorted[r_it].Index][1])
				{
					if (scratch.sorted[r_it].size[1] > 1)
					{
						ReductionOperation(scratch.sorted[r_it].size[1], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						ReductionOperation(scratch.blocks[scratch.sorted[r_it].Index][1], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						pass = true;
					}

					scratch.reductions[scratch.sorted[r_it].Index][1] += 1;
				}
				else if (scratch.reductions[scratch.sorted[r_it].Index][0] < scratch.reductions[scratch.sorted[r_it].Index][1])
				{
					if (scratch.sorted[r_it].size[0] > 1)
					{
						ReductionOperation(scratch.sorted[r_it].size[0], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						ReductionOperation(scratch.blocks[scratch.sorted[r_it].Index][0], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						pass = true;
					}

					scratch.reductions[scratch.sorted[r_it].Index][0] += 1;
				}
				else
				{
					// we select the first side to reduce "randomly"
					int32 Index = r_it % 2;

					// if we can't reduce a dimension then we try to reduce the other one
					if (scratch.sorted[r_it].size[Index] <= 1)
					{
						Index = r_it == 0 ? 1 : 0;
					}

					if (scratch.sorted[r_it].size[Index] > 1)
					{
						ReductionOperation(scratch.sorted[r_it].size[Index], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						ReductionOperation(scratch.blocks[scratch.sorted[r_it].Index][Index], ReductionMethod, scratch.sorted[r_it].bReduceByTwo);
						pass = true;
					}

					scratch.reductions[scratch.sorted[r_it].Index][Index] += 1;
				}
			}
		}
		else
		{
			pass = true;
		}
		
		int32 newBlockArea = scratch.sorted[r_it].size[0] * scratch.sorted[r_it].size[1];

		InOutArea = InOutArea - (oldBlockArea - newBlockArea);

		if (pass)
		{
			InOutBlockIt = InOutBlockIt + 1;

			if (InOutBlockIt >= BlockCount)
			{
				InOutBlockIt = 0;
			}
		}
	}


	inline bool SetPositions(int32 bestY,int32 layoutSizeY, int32* maxX, int32* maxY, FScratchLayoutPack& scratch, EPackStrategy packStrategy)
	{
		bool fits = true;

		// Number of blocks alrady packed
		size_t packed = 0;
		TArray<uint8_t> packedFlag;
		packedFlag.SetNumZeroed(scratch.sorted.Num());

		// Pack with fixed horizontal size
		check(*maxX < 256);
		if (*maxX > 256) return true;

		int16_t horizon[256];
		FMemory::Memzero(horizon, 256 * sizeof(int16_t));
		*maxY = 0;

		int32 iterations = 0;

		while (packed < scratch.sorted.Num() || iterations > 5000)
		{
			++iterations;

			int32 best = -1;
			int32 bestX = -1;
			int32 bestLevel = -1;
			int32 bestWithHole = -1;
			int32 bestWithHoleX = -1;
			int32 bestWithHoleLevel = -1;
			for (size_t candidate = 0; candidate < scratch.sorted.Num(); ++candidate)
			{
				// Skip it if we packed it already
				if (packedFlag[candidate]) continue;

				auto candidateSizeX = scratch.sorted[candidate].size[0];
				auto candidateSizeY = scratch.sorted[candidate].size[1];

				// Seek for the lowest span where the block fits
				int32 currentLevel = TNumericLimits<int32>::Max();
				int32 currentX = 0;
				int32 currentLevelWithoutHole = TNumericLimits<int32>::Max();
				int32 currentXWithoutHole = 0;
				for (int32 x = 0; x <= *maxX - candidateSizeX; ++x)
				{
					int32 level = 0;
					for (int32 xs = x; xs < x + candidateSizeX; ++xs)
					{
						level = FMath::Max(level, (int32)horizon[xs]);
					}

					if (level < currentLevel)
					{
						currentLevel = level;
						currentX = x;
					}

					// Does it make an unfillable hole with the top or side?
					int32 minX = TNumericLimits<int32>::Max();
					int32 minY = TNumericLimits<int32>::Max();
					for (size_t b = 0; b < scratch.sorted.Num(); ++b)
					{
						if (!packedFlag[b] && b != candidate)
						{
							minX = FMath::Min(minX, scratch.sorted[b].size[0]);
							minY = FMath::Min(minY, scratch.sorted[b].size[1]);
						}
					}

					bool hole =
						// Vertical unfillable gap
						(
						(minY != TNumericLimits<int32>::Max())
							&&
							(currentLevel + candidateSizeY) < bestY
							&&
							(currentLevel + minY + candidateSizeY) > bestY
							)
						||
						// Horizontal unfillable gap
						(
						(minX != TNumericLimits<int32>::Max())
							&&
							(currentX + candidateSizeX) < *maxX
							&&
							(currentX + minX + candidateSizeX) > *maxX
							);

					// Does it make a hole with the horizon?
					for (int32 xs = x; !hole && xs < x + scratch.sorted[candidate].size[0]; ++xs)
					{
						hole = (level > (int32)horizon[xs]);
					}

					if (!hole && level < currentLevelWithoutHole)
					{
						currentLevelWithoutHole = level;
						currentXWithoutHole = x;
					}
				}

				if (currentLevelWithoutHole <= currentLevel)
				{
					best = int32(candidate);
					bestX = currentXWithoutHole;
					bestLevel = currentLevelWithoutHole;
					break;
				}

				if (bestWithHole < 0)
				{
					bestWithHole = int32(candidate);
					bestWithHoleX = currentX;
					bestWithHoleLevel = currentLevel;
				}
			}

			check(best >= 0 || bestWithHole >= 0);

			// If there is no other option, accept leaving a hole.
			if (best < 0)
			{
				best = bestWithHole;
				bestX = bestWithHoleX;
				bestLevel = bestWithHoleLevel;
			}

			if (bestX >= 0)
			{
				// Update horizon
				for (int32 xs = bestX; xs < bestX + scratch.sorted[best].size[0]; ++xs)
				{
					horizon[xs] = (uint16)(bestLevel + scratch.sorted[best].size[1]);
				}
			}

			// Store
			scratch.positions[scratch.sorted[best].Index] = FIntVector2(bestX, bestLevel);
			*maxY = FMath::Max(*maxY, bestLevel + scratch.sorted[best].size[1] );

			if (packStrategy == EPackStrategy::Fixed && *maxY > layoutSizeY)
			{
				fits = false;
				break;
			}

			packedFlag[best] = 1;
			packed++;

			//                for (int32 y=0; y<*maxY; ++y)
			//                {
			//                    string line;
			//                    for (int32 x=0;x<*maxX;++x)
			//                    {
			//                        line += DebugGetBlockAt( *scratch, packedFlag, x, y );
			//                    }
			//                    AXE_LOG("layout",Warning,line.c_str());
			//                }
			//                AXE_LOG("layout",Warning,"--------------------------------------------------");
		}

		if (packed < scratch.sorted.Num())
		{
			fits = false;
		}

		*maxY = FGenericPlatformMath::RoundUpToPowerOfTwo(*maxY);

		return fits;
	}


    void LayoutPack3( FLayout* pResult, const FLayout* pSourceLayout )
    {
		MUTABLE_CPUPROFILER_SCOPE(MeshLayoutPack3);

        check( pResult->GetBlockCount() == pSourceLayout->GetBlockCount() );

		int32 BlockCount = pSourceLayout->GetBlockCount();

		FScratchLayoutPack scratch;
		scratch.blocks.SetNum(BlockCount);
		scratch.sorted.SetNum(BlockCount);
		scratch.positions.SetNum(BlockCount);
		scratch.priorities.SetNum(BlockCount);
		scratch.reductions.SetNum(BlockCount);
		scratch.ReduceBothAxes.SetNum(BlockCount);
		scratch.ReduceByTwo.SetNum(BlockCount);


        check( scratch.blocks.Num()==BlockCount );

		//Getting maximum layout grid size:
		int32 layoutSizeX, layoutSizeY;
		pSourceLayout->GetMaxGridSize(&layoutSizeX, &layoutSizeY);

		bool usePriority = false;

		EPackStrategy LayoutStrategy = pSourceLayout->GetLayoutPackingStrategy();
		EReductionMethod ReductionMethod = pSourceLayout->ReductionMethod;

        // Look for the maximum block sizes on the layout and the total area
		int32 maxX = 0;
		int32 maxY = 0;
        int32 area = 0;

        for ( int32 Index=0; Index<BlockCount; ++Index )
        {
            box< FIntVector2 > b;
			b.min = pSourceLayout->Blocks[Index].Min;
			b.size = pSourceLayout->Blocks[Index].Size;

			int32 p = pSourceLayout->Blocks[Index].Priority;
			bool bReduceBothAxes = pSourceLayout->Blocks[Index].bReduceBothAxes;
			bool bReduceByTwo = pSourceLayout->Blocks[Index].bReduceByTwo;

			FIntVector2 reductions;
			reductions[0] = 0;
			reductions[1] = 0;

			if (p > 0)
			{
				usePriority = true;
			}

            maxX = FMath::Max( maxX, b.size[0] );
            maxY = FMath::Max( maxY, b.size[1] );

            area += b.size[0] * b.size[1];

            scratch.blocks[Index] = b.size;
			scratch.priorities[Index] = p;
			scratch.reductions[Index] = reductions;
			scratch.ReduceBothAxes[Index] = (int32)bReduceBothAxes;
			scratch.ReduceByTwo[Index] = (int32)bReduceByTwo;
        }


        // Grow until the area is big enough to fit all blocks. We always grow X first, because
        // in case we cannot pack everything, we will grow Y with the current horizon algorithm.
		if (LayoutStrategy == EPackStrategy::Resizeable)
		{
			maxX = FGenericPlatformMath::RoundUpToPowerOfTwo(maxX);
			maxY = FGenericPlatformMath::RoundUpToPowerOfTwo(maxY);

			while (maxX*maxY < area)
			{
				if (maxX > maxY)
				{
					maxY *= 2;
				}
				else
				{
					maxX *= 2;
				}
			}
		}
		else
		{
			//Increase the maximum layout size if the grid area is smaller than the number of blocks
			while (BlockCount > layoutSizeX*layoutSizeY)
			{
				if (layoutSizeX > layoutSizeY)
				{
					layoutSizeY *= 2;

					if (layoutSizeY == 0)
					{
						layoutSizeY = 1;
					}
				}
				else
				{
					layoutSizeX *= 2;

					if (layoutSizeX == 0)
					{
						layoutSizeX = 1;
					}
				}
			}

			// Reducing blocks that do not fit in the layout grid
			while (maxX > layoutSizeX)
			{
				area = 0;

				for (int32 Index = 0; Index < BlockCount; ++Index)
				{
					if (scratch.blocks[Index][0] == maxX)
					{
						ReductionOperation(scratch.blocks[Index][0], ReductionMethod, scratch.ReduceByTwo[Index]);
						scratch.reductions[Index][0]++;

						if (scratch.ReduceBothAxes[Index] == 1)
						{
							ReductionOperation(scratch.blocks[Index][1], ReductionMethod, scratch.ReduceByTwo[Index]);
							scratch.reductions[Index][1]++;
						}
					}
				}

				maxX = maxY = 0;
				
				//recalculating area and maximum block sizes
				for (int32 Index = 0; Index < BlockCount; ++Index)
				{
					maxX = FMath::Max(maxX, scratch.blocks[Index][0]);

					// maxY could have been modified if a block had symmetry enabled in the previous reduction
					maxY = FMath::Max(maxY, scratch.blocks[Index][1]);

					area += scratch.blocks[Index][0] * scratch.blocks[Index][1];
				}
			}

			while (maxY > layoutSizeY)
			{
				area = 0;

				for (int32 Index = 0; Index < BlockCount; ++Index)
				{
					if (scratch.blocks[Index][1] == maxY)
					{
						ReductionOperation(scratch.blocks[Index][1], ReductionMethod, scratch.ReduceByTwo[Index]);
						scratch.reductions[Index][1]++;

						if (scratch.ReduceBothAxes[Index] == 1)
						{
							ReductionOperation(scratch.blocks[Index][0], ReductionMethod, scratch.ReduceByTwo[Index]);
							scratch.reductions[Index][0]++;
						}
					}
				}

				maxX = maxY = 0;

				//recalculating area and maximum block sizes
				for (int32 Index = 0; Index < BlockCount; ++Index)
				{
					maxY = FMath::Max(maxY, scratch.blocks[Index][1]);

					// maxX could have been modified if a block had symmetry enabled in the previous reduction
					maxX = FMath::Max(maxX, scratch.blocks[Index][0]);

					area += scratch.blocks[Index][0] * scratch.blocks[Index][1];
				}
			}

			maxX = FGenericPlatformMath::RoundUpToPowerOfTwo(maxX);
			maxY = FGenericPlatformMath::RoundUpToPowerOfTwo(maxY);

			// Grow until the area is big enough to fit all blocks or the size is equal to the max layout size.
			while (maxX*maxY < area && (maxX < layoutSizeX || maxY < layoutSizeY))
			{
				if (maxX > maxY)
				{
					maxY *= 2;
				}
				else
				{
					maxX *= 2;
				}
			}
		}
        
        int32 bestY = maxY;

		// This is used to iterate through blocks.
		int32 BlockIterator = 0;

		// Making a copy of the blocks array to sort them
        check( (int32)scratch.sorted.Num()==BlockCount );
        for ( int32 Index=0; Index<BlockCount; ++Index )
        {
            scratch.sorted[Index] = FPackLayoutBlock( Index, scratch.blocks[Index], scratch.priorities[Index], (bool)scratch.ReduceBothAxes[Index], (bool)scratch.ReduceByTwo[Index]);
        }

		// Sort blocks by height, area
		scratch.sorted.Sort(CompareBlocks);

		if(LayoutStrategy == EPackStrategy::Fixed)
		{
			if (usePriority)
			{
				//Sort blocks by priority
				scratch.sorted.Sort(CompareBlocksPriority);
			}

			// Shrink blocks in case we do not have enough space to pack everything
			while (maxX*maxY < area)
			{
				ReduceBlock(BlockCount, area, BlockIterator, scratch, ReductionMethod);
			}
		}

		bool fits = false;

		while (!fits)
		{
			// Sort by height&area before packing
			if (usePriority)
			{
				scratch.sorted.Sort(CompareBlocks);
			}

			// Try to pack everything
			fits = SetPositions(bestY, layoutSizeY, &maxX, &maxY, scratch, LayoutStrategy);

			if (!fits && LayoutStrategy == EPackStrategy::Fixed)
			{
				// Sort by priority before shrink
				if (usePriority)
				{
					scratch.sorted.Sort(CompareBlocksPriority);
				}

				ReduceBlock(BlockCount, area, BlockIterator, scratch, ReductionMethod);
			}
		}

        // Set data in the result
        pResult->SetGridSize( maxX, maxY );
        pResult->SetMaxGridSize(layoutSizeX, layoutSizeY);
		pResult->SetLayoutPackingStrategy(LayoutStrategy);
		pResult->ReductionMethod = ReductionMethod;

        for ( int32 Index=0; Index<BlockCount; ++Index )
        {
			pResult->Blocks[Index].Min = scratch.positions[Index];
			pResult->Blocks[Index].Size = scratch.blocks[Index];
			pResult->Blocks[Index].Priority = scratch.priorities[Index];
			pResult->Blocks[Index].bReduceBothAxes = scratch.ReduceBothAxes[Index];
			pResult->Blocks[Index].bReduceByTwo = scratch.ReduceByTwo[Index];
        }
    }
}
