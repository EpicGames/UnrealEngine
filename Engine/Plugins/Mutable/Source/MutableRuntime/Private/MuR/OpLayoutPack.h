// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


namespace UE::Mutable::Private
{
	class FLayout;

	extern void LayoutPack3(FLayout* Result, const FLayout* SourceLayout);

}
