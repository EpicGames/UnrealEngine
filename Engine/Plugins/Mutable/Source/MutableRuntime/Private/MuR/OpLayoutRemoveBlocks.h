// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Layout.h"
#include "MuR/MeshPrivate.h"
#include "MuR/Platform.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	inline TManagedPtr<FLayout> LayoutFromMesh_RemoveBlocks(const FMesh* InMesh, int32 InLayoutIndex)
	{
		if (!InMesh || InMesh->GetLayoutCount() <= InLayoutIndex)
		{
			return nullptr;
		}

		TManagedPtr<const FLayout> Source = InMesh->GetLayout(InLayoutIndex);
		TManagedPtr<FLayout> pResult;

		UntypedMeshBufferIteratorConst itBlocks(InMesh->GetVertexBuffers(), EMeshBufferSemantic::LayoutBlock, InLayoutIndex);
		if (itBlocks.GetFormat() == EMeshBufferFormat::UInt16)
		{
			// Relative blocks.

			const uint16* pBlocks = reinterpret_cast<const uint16*>(itBlocks.ptr());

			uint16 MaxId = 0;
			for (int32 i = 0; i < InMesh->GetVertexCount(); ++i)
			{
				uint16 RelativeId = pBlocks[i];
				check(RelativeId != 0xffff);
				MaxId = FMath::Max(MaxId, RelativeId);
			}


			// Create the list of blocks in the mesh.
			// Array that stores a flag for every ID, possibly wasting space.
			TBitArray BlocksFound;
			BlocksFound.Init(false,MaxId+1);
			for (int32 i = 0; i < InMesh->GetVertexCount(); ++i)
			{
				BlocksFound[pBlocks[i]] = true;
			}

			// Remove blocks that are not in the mesh
			pResult = Source->Clone();
			int32 DestBlockIndex = 0;
			for (int32 BlockIndex = 0; BlockIndex < pResult->Blocks.Num(); ++BlockIndex)
			{
				uint64 BlockId = pResult->Blocks[BlockIndex].Id;
				uint32 BlockIdPrefix = uint32(BlockId >> 32);
				uint32 RelativeBlockId = uint32(BlockId & 0xffffffff);

				bool bBlockMatchesPrefix = BlockIdPrefix == InMesh->MeshIDPrefix;

				if (bBlockMatchesPrefix && RelativeBlockId < uint32(BlocksFound.Num()) && BlocksFound[RelativeBlockId] )
				{
					// keep this block
					pResult->Blocks[DestBlockIndex] = pResult->Blocks[BlockIndex];
					++DestBlockIndex;
				}
			}
			pResult->SetBlockCount(DestBlockIndex);

		}
		else if (itBlocks.GetFormat() == EMeshBufferFormat::UInt64)
		{
			// Absolute blocks.

			// Create the list of blocks in the mesh.
			// TODO: SparseIndexSet? like in the 16-bit case, but per-prefix?
			TSet<uint64> BlocksFound;
			BlocksFound.Reserve(64);

			const uint64* BlockIdsPerVertex = reinterpret_cast<const uint64*>(itBlocks.ptr());
			for (int32 i = 0; i < InMesh->GetVertexCount(); ++i)
			{
				BlocksFound.FindOrAdd(BlockIdsPerVertex[i]);
			}

			// Remove blocks that are not in the mesh
			pResult = Source->Clone();
			int32 DestBlockIndex = 0;
			for (int32 BlockIndex = 0; BlockIndex < pResult->Blocks.Num(); ++BlockIndex)
			{
				uint64 BlockId = pResult->Blocks[BlockIndex].Id;
				bool bBlockFound = BlocksFound.Contains(BlockId);

				if (bBlockFound)
				{
					// keep this block
					pResult->Blocks[DestBlockIndex] = pResult->Blocks[BlockIndex];
					++DestBlockIndex;
				}
			}
			pResult->SetBlockCount(DestBlockIndex);

		}
		else if (itBlocks.GetFormat() == EMeshBufferFormat::None)
		{
			// This seems to happen.
			// May this happen when entire meshes are removed?
			return Source->Clone();
		}
		else
		{
			// Format not supported yet
			check(false);
		}


		return pResult;
	}


	inline TManagedPtr<FLayout> LayoutRemoveBlocks(const FLayout* Source, const FLayout* ReferenceLayout)
	{
		// Remove blocks that are not in the mesh
		TManagedPtr<FLayout> pResult = Source->Clone();
		int32 DestBlockIndex = 0;
		for (int32 BlockIndexInSource = 0; BlockIndexInSource < pResult->Blocks.Num(); ++BlockIndexInSource)
		{
			uint64 BlockId = pResult->Blocks[BlockIndexInSource].Id;
			int32 BlockIndexInReference = ReferenceLayout->FindBlock(BlockId);
			if (BlockIndexInReference >=0 )
			{
				// keep
				pResult->Blocks[DestBlockIndex] = pResult->Blocks[BlockIndexInSource];
				++DestBlockIndex;
			}
		}
		pResult->SetBlockCount(DestBlockIndex);

		return pResult;
	}


	inline TManagedPtr<FLayout> LayoutMerge(const FLayout* pA, const FLayout* pB )
	{
		TManagedPtr<FLayout> pResult = pA->Clone();

		for ( const FLayoutBlock& Block: pB->Blocks )
		{
			if ( pResult->FindBlock(Block.Id)<0 )
			{
				pResult->Blocks.Add(Block);
			}
		}

		return pResult;
	}


}
