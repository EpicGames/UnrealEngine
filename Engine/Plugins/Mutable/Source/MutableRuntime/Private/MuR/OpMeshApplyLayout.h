// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshPrivate.h"
#include "MuR/MutableRuntimeModule.h"
#include "MuR/MutableTrace.h"
#include "MuR/Platform.h"
#include "MuR/OpMeshRemove.h"

namespace UE::Mutable::Private
{


    inline void MeshApplyLayout( FMesh* Applied, const FLayout* InLayout, int32 TexCoordsIndex )
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshApplyLayout);

        int32 buffer = -1;
		int32 channel = -1;
		Applied->GetVertexBuffers().FindChannel( EMeshBufferSemantic::TexCoords, TexCoordsIndex, &buffer, &channel );

        int32 layoutBuffer = -1;
        int32 layoutChannel = -1;
        Applied->GetVertexBuffers().FindChannel( EMeshBufferSemantic::LayoutBlock, TexCoordsIndex, &layoutBuffer, &layoutChannel );

		if (buffer < 0 || layoutBuffer < 0)
		{
			return;
		}

		// Get the information about the texture coordinates channel
		EMeshBufferSemantic semantic;
		int32 semanticIndex;
		EMeshBufferFormat format;
		int32 components;
		int32 offset;
		Applied->GetVertexBuffers().GetChannel( buffer, channel, &semantic, &semanticIndex, &format, &components, &offset );
		check( semantic == EMeshBufferSemantic::TexCoords );

        uint8* pData = Applied->GetVertexBuffers().GetBufferData( buffer );
		int32 elemSize = Applied->GetVertexBuffers().GetElementSize( buffer );
		int32 channelOffset = Applied->GetVertexBuffers().GetChannelOffset( buffer, channel );
		pData += channelOffset;

		struct Box
		{
			FVector2f min, size;
		};
		TArray< Box > transforms;
		transforms.SetNum(InLayout->GetBlockCount());
		for ( int b=0; b<InLayout->GetBlockCount(); ++b )
		{
			FIntPoint grid = InLayout->GetGridSize();

			box< FIntVector2 > block;
			block.min = InLayout->Blocks[b].Min;
			block.size = InLayout->Blocks[b].Size;

			Box rect;
			rect.min[0] = ( (float)block.min[0] ) / (float) grid[0];
			rect.min[1] = ( (float)block.min[1] ) / (float) grid[1];
			rect.size[0] = ( (float)block.size[0] ) / (float) grid[0];
			rect.size[1] = ( (float)block.size[1] ) / (float) grid[1];
			transforms[b] = rect;
		}

        check( layoutBuffer>=0 && layoutChannel>=0 );
        check( Applied->GetVertexBuffers().Buffers[layoutBuffer].Channels.Num()==1 );
        check( Applied->GetVertexBuffers().Buffers[layoutBuffer].Channels[layoutChannel].ComponentCount==1 );

        const uint16* InLayoutData = reinterpret_cast<const uint16*>( Applied->GetVertexBuffers().GetBufferData( layoutBuffer ) );
		UntypedMeshBufferIterator ItLayoutData(Applied->GetVertexBuffers(), EMeshBufferSemantic::LayoutBlock, TexCoordsIndex);

		// In some corner case involving automatic LODs and remove meshes behaving differently among them we may need to remove vertices that don't 
		// have any block in the current layout. Track them here.
		TArray<int32> VerticesToRemove;

        uint8* pVertices = pData;
		for ( int32 v=0; v<Applied->GetVertexBuffers().GetElementCount(); ++v )
		{
			uint64 BlockId = 0;
			if (ItLayoutData.GetFormat() == EMeshBufferFormat::UInt16)
			{
				// Relative blocks.
				const uint16* SourceIds = reinterpret_cast<const uint16*>(ItLayoutData.ptr());
				BlockId = *SourceIds;
				BlockId = BlockId | (uint64(Applied->MeshIDPrefix) << 32);
			}
			else if (ItLayoutData.GetFormat() == EMeshBufferFormat::UInt64)
			{
				// Absolute blocks.
				const uint64* SourceIds = reinterpret_cast<const uint64*>(ItLayoutData.ptr());
				BlockId = *SourceIds;
			}
			else
			{
				// Format not supported
				check(false);
			}
			++ItLayoutData;

			// TODO: This could be optimised
			int32 relBlock = InLayout->FindBlock(BlockId);

			// This may still happen with lower LOD and "remove meshes" in a corner case:
			// Auto LODs with "Remove Meshes" that behave differently across LODs, and leave geometry in a block that has been removed in the higher LOD.
			if (relBlock < 0)
			{
				VerticesToRemove.Add(v); 
				pVertices += elemSize;
				continue;
			}

			switch( format )
			{
			case EMeshBufferFormat::Float32:
			{
				FVector2f uv;
				FMemory::Memcpy( &uv, pVertices, sizeof(float)*2 );

				uv = uv * transforms[ relBlock ].size + transforms[ relBlock ].min;
				FMemory::Memcpy( pVertices, &uv, sizeof(float)*2 );
				break;
			}

			case EMeshBufferFormat::Float16:
			{
				// TODO: Optimise
				FFloat16* pUV = reinterpret_cast<FFloat16*>( pVertices );

				FVector2f uv;
				uv[0] = float( pUV[0] );
				uv[1] = float( pUV[1] );
				uv = uv * transforms[ relBlock ].size + transforms[ relBlock ].min;

				pUV[0] = FFloat16( uv[0] );
				pUV[1] = FFloat16( uv[1] );
				break;
			}

			case EMeshBufferFormat::NUInt32:
			case EMeshBufferFormat::NInt32:
			case EMeshBufferFormat::UInt32:
			case EMeshBufferFormat::Int32:
			{
				// TODO: Optimise
                uint32* pUV = reinterpret_cast<uint32*>( pVertices );

				for ( int c=0; c<2; ++c )
				{
                    uint64 u_32 = pUV[c];
                    uint64 u_48 = u_32 * ((uint64)(((float)0xffff) * transforms[ relBlock ].size[c]));
                    u_48 += (uint64)(((float)0xffffffffffULL) * transforms[ relBlock ].min[c]);
                    pUV[c] = (uint16)( u_48 >> 16 );
				}
				break;
			}

			case EMeshBufferFormat::NUInt16:
			case EMeshBufferFormat::NInt16:
			case EMeshBufferFormat::UInt16:
			case EMeshBufferFormat::Int16:
			{
				// TODO: Optimise
                uint16* pUV = reinterpret_cast<uint16*>( pVertices );

				for ( int c=0; c<2; ++c )
				{
                    uint32 u_16 = pUV[c];
                    uint32 u_32 = u_16 * ((uint8)(((float)0xffff) * transforms[ relBlock ].size[c]));
                    u_32 += (uint32)(((float)0xffffffff) * transforms[ relBlock ].min[c]);
                    pUV[c] = (uint16)( u_32 >> 16 );
				}

				break;
			}

			case EMeshBufferFormat::NUInt8:
			case EMeshBufferFormat::NInt8:
			case EMeshBufferFormat::UInt8:
			case EMeshBufferFormat::Int8:
			{
				// TODO: Optimise
                uint8* pUV = reinterpret_cast<uint8*>( pVertices );

				for ( int c=0; c<2; ++c )
				{
                    uint32 u_8 = pUV[c];
                    uint32 u_24 = u_8 * ((uint32)(((float)0xffff) * transforms[ relBlock ].size[c]));
                    u_24 += (uint32)(((float)0xffffff) * transforms[ relBlock ].min[c]);
                    pUV[c] = (uint8)( u_24 >> 16 );
				}
				break;
			}

			default:
				UE_LOGF(LogMutableCore, Warning, "This case is not implemented..");
				check( false );
				break;

			}

			pVertices += elemSize;
		}

		if (VerticesToRemove.Num())
		{
			// Unpack vertices into a mask
			TBitArray<> VertexMask;
			VertexMask.SetNum(Applied->GetVertexCount(), false);
			for (int32 VertexIndex : VerticesToRemove)
			{
				VertexMask[VertexIndex] = true;
			}
	
			// Remove
			bool bRemoveIfAllVerticesCulled = true;
			MeshRemoveVerticesWithCullSet(Applied, VertexMask, bRemoveIfAllVerticesCulled);
		}

		//
		Applied->SetLayout( TexCoordsIndex, InLayout->Clone() );

	}

}
