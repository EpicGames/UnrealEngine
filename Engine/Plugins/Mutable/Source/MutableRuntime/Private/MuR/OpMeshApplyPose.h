// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshPrivate.h"
#include "MuR/ConvertData.h"
#include "MuR/MutableTrace.h"
#include "MuR/Platform.h"

#include "MuR/ParallelExecutionUtils.h"

namespace UE::Mutable::Private
{
	inline void SetPoseAsReference(FMesh* InOutResult, const FMesh* PoseMesh, const TArray<int32>& BoneToPoseIndex)
	{
		// Set the pose as the Result reference pose.
		// PoseMesh PoseXform can be decomposed to PoseXform = ModelPoseXform * ModelRefXform^-1, 
		// BaseMesh Poses have ModelRefXform, to get the new ModelRefXform to be ModelPoseXform we need 
		// to multiply PoseXform * ModelRefXform. 
		// (Note, transform multiplication application is from right to left for this comment, but TTransform::operator* is reversed).
		const int32 ResultNumBones = InOutResult->GetBonePoseCount();
		for (FMesh::FBonePose& BonePose : InOutResult->BonePoses)
		{
			const int32 FoundPoseIndex = BoneToPoseIndex[BonePose.BoneId.Index];
			if (FoundPoseIndex == INDEX_NONE)
			{
				continue;
			}

			FTransform3f PoseTransform;
			PoseMesh->GetBonePoseTransform(FoundPoseIndex, PoseTransform); 

			BonePose.BoneTransform = BonePose.BoneTransform * PoseTransform;

			// Adding the Reshaped flag will prioritize this bone over other bones without the flag
			// when merging poses with the same bone.
			// TODO: Add another flag to indicate this case or generalize the Reshaped flag. 
			EnumAddFlags(BonePose.BoneUsageFlags, EBoneUsageFlags::Reshaped);
		}
	}

	/**
    * Reference version
    */
    inline void MeshApplyPose(FMesh* Result, const FMesh* BaseMesh, const FMesh* PoseMesh, bool& bOutSuccess)
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshApplyPose);

		bOutSuccess = true;

		TManagedPtr<const FSkeleton> BaseSkeleton = BaseMesh->GetSkeleton();
		TManagedPtr<const FSkeleton> PoseSkeleton = PoseMesh->GetSkeleton();
		if (!BaseSkeleton || !PoseSkeleton)
		{
			bOutSuccess = false;
			return;
		}

        // We assume the matrices are transforms from the binding pose bone to the new pose

		// Find closest bone affected by the pose for each bone in the skeleton.
		const int32 NumBonesBase = BaseSkeleton->GetNumBones();
		TArray<int32> BoneToPoseIndex;
		BoneToPoseIndex.Init(INDEX_NONE, NumBonesBase);

		for (int32 BoneIndex = 0; BoneIndex < NumBonesBase; ++BoneIndex)
		{
			const int32 BoneNameIndex = PoseSkeleton->BoneNames.Find(BaseSkeleton->BoneNames[BoneIndex]);
			if (BoneNameIndex == INDEX_NONE)
			{
				continue; // BaseSkeleton bone not found in PoseSkeleton.
			}

			const int32 BonePoseIndex = PoseMesh->FindBonePoseByBoneIndex(BoneNameIndex);
			if (BonePoseIndex != INDEX_NONE)
			{
				BoneToPoseIndex[BoneIndex] = BonePoseIndex;
				continue;
			}

			// Parent bones are in a strictly increassing order. Set the pose index from the parent bone.
			const int32 ParentIndex = BaseSkeleton->BoneParents[BoneIndex];
			if (ParentIndex != INDEX_NONE)
			{
				BoneToPoseIndex[BoneIndex] = BoneToPoseIndex[ParentIndex];
			}
		}

		const TArray<FBoneIdOrIndex>& BoneMap = BaseMesh->GetBoneMap();
		const int32 NumBonesBoneMap = BoneMap.Num();

        // Prepare the skin matrices. They may be in different order, and we only need the ones
        // relevant for the base mesh deformation.
		TArray<FTransform3f> SkinTransforms;
		SkinTransforms.Reserve(NumBonesBoneMap);

		bool bBonesAffected = false;
		for (int32 Index = 0; Index < NumBonesBoneMap; ++Index)
		{
			const int32 PoseIndex = BoneToPoseIndex[BoneMap[Index].Index];
			if (PoseIndex != INDEX_NONE)
			{
				SkinTransforms.Add(PoseMesh->BonePoses[PoseIndex].BoneTransform);
				bBonesAffected = true;
			}
			else
			{
				// Bone not affected by the pose. Set identity.
				SkinTransforms.Add(FTransform3f::Identity);
			}
		}

		if (!bBonesAffected)
		{
			// The pose does not affect any vertex in the mesh. 
			bOutSuccess = false;
			return;
		}

        // Get pointers to vertex position data
		MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> SourcePositionIterBegin(BaseMesh->VertexBuffers, EMeshBufferSemantic::Position, 0);
        if (!SourcePositionIterBegin.ptr())
        {
            // Formats not implemented
            check(false);
			bOutSuccess = false;
            return;
        }
 
		// Get pointers to skinning data
		UntypedMeshBufferIteratorConst BoneIndicesIterBegin(BaseMesh->VertexBuffers, EMeshBufferSemantic::BoneIndices, 0);
		UntypedMeshBufferIteratorConst BoneWeightsIterBegin(BaseMesh->VertexBuffers, EMeshBufferSemantic::BoneWeights, 0);
        if (!BoneIndicesIterBegin.ptr() || !BoneWeightsIterBegin.ptr())
        {
            // No skinning data
            check(false);
			bOutSuccess = false;
            return;
        }

		Result->CopyFrom(*BaseMesh);

		SetPoseAsReference(Result, PoseMesh, BoneToPoseIndex);

		MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> TargetPositionIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Position, 0);
		
		// Tangent frame buffers are optional.
		UntypedMeshBufferIteratorConst SourceNormalIterBegin(BaseMesh->VertexBuffers, EMeshBufferSemantic::Normal, 0);
		UntypedMeshBufferIteratorConst SourceTangentIterBegin(BaseMesh->VertexBuffers, EMeshBufferSemantic::Tangent, 0);
		UntypedMeshBufferIteratorConst SourceBiNormalIterBegin(BaseMesh->VertexBuffers, EMeshBufferSemantic::Binormal, 0);

		UntypedMeshBufferIterator TargetNormalIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Normal, 0);
		UntypedMeshBufferIterator TargetTangentIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Tangent, 0);
		UntypedMeshBufferIterator TargetBiNormalIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Binormal, 0);
		
		check(TargetPositionIterBegin.ptr());

        const int32 VertexCount = BaseMesh->GetVertexCount();
        const int32 WeightCount = BoneIndicesIterBegin.GetComponents();
        check(WeightCount == BoneWeightsIterBegin.GetComponents());

        constexpr int32 MAX_BONES_PER_VERTEX = 16;
        check(WeightCount <= MAX_BONES_PER_VERTEX);

		constexpr int32 NumVertsPerBatch = 1 << 11;
		const int32 NumBatches = FMath::DivideAndRoundUp<int32>(VertexCount, NumVertsPerBatch);
		
		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
		[
			SkinTransforms = MakeArrayView<const FTransform3f>(SkinTransforms.GetData(), SkinTransforms.Num()),
			SourcePositionIterBegin,
			SourceNormalIterBegin,
			SourceTangentIterBegin,
			SourceBiNormalIterBegin,
			BoneIndicesIterBegin,
			BoneWeightsIterBegin,
			TargetPositionIterBegin,
			TargetNormalIterBegin,
			TargetTangentIterBegin,
			TargetBiNormalIterBegin,
			VertexCount,
			WeightCount,
			NumVertsPerBatch
		](int32 BatchId)
		{
			const int32 BatchBeginVertIndex = BatchId * NumVertsPerBatch;
			const int32 BatchEndVertIndex = FMath::Min(BatchBeginVertIndex + NumVertsPerBatch, VertexCount);
			
			const EMeshBufferFormat WeightsFormat = BoneWeightsIterBegin.GetFormat(); 
			const EMeshBufferFormat BoneIndexFormat = BoneIndicesIterBegin.GetFormat(); 
			const EMeshBufferFormat NormalFormat = TargetNormalIterBegin.ptr() 
					? TargetNormalIterBegin.GetFormat() : EMeshBufferFormat::None;	

			for (int32 VertexIndex = BatchBeginVertIndex; VertexIndex < BatchEndVertIndex; ++VertexIndex)
			{
				float TotalWeight = 0.0f;

				float Weights[MAX_BONES_PER_VERTEX];
				const uint8* VertexBoneWeightData = (BoneWeightsIterBegin + VertexIndex).ptr();
				for (int32 WeightIndex = 0; WeightIndex < WeightCount; ++WeightIndex)
				{
					ConvertData(WeightIndex, Weights, EMeshBufferFormat::Float32, VertexBoneWeightData, WeightsFormat);
					TotalWeight += Weights[WeightIndex];
				}

				uint32 BoneIndices[MAX_BONES_PER_VERTEX];
				const uint8* VertexBoneIndexData = (BoneIndicesIterBegin + VertexIndex).ptr();
				for (int32 WeightIndex = 0; WeightIndex < WeightCount; ++WeightIndex)
				{
					ConvertData(WeightIndex, BoneIndices, EMeshBufferFormat::UInt32, VertexBoneIndexData, BoneIndexFormat);
				}
		
				FVector3f SourcePosition = (SourcePositionIterBegin + VertexIndex).GetAsVec3f();
				FVector3f SourceNormal = SourceNormalIterBegin.ptr() 
						? (SourceNormalIterBegin + VertexIndex).GetAsVec3f() : FVector3f::ZeroVector;
				FVector3f SourceTangent  = SourceTangentIterBegin.ptr() 
						? (SourceTangentIterBegin + VertexIndex).GetAsVec3f() : FVector3f::ZeroVector;
				FVector3f SourceBiNormal = SourceBiNormalIterBegin.ptr() 
						? (SourceBiNormalIterBegin + VertexIndex).GetAsVec3f() : FVector3f::ZeroVector;

				FVector3f Position = FVector3f::ZeroVector;
				FVector3f Normal   = FVector3f::ZeroVector;
				FVector3f Tangent  = FVector3f::ZeroVector;
				FVector3f BiNormal = FVector3f::ZeroVector;
				
				for (int32 WeightIndex = 0; WeightIndex < WeightCount; ++WeightIndex)
				{
					Position += SkinTransforms[BoneIndices[WeightIndex]].TransformPosition(SourcePosition) * Weights[WeightIndex];
				
					if (SourceNormalIterBegin.ptr())
					{
						Normal += SkinTransforms[BoneIndices[WeightIndex]].TransformVector(SourceNormal) * Weights[WeightIndex];
					}

					if (SourceTangentIterBegin.ptr())
					{
						Tangent += SkinTransforms[BoneIndices[WeightIndex]].TransformVector(SourceTangent) * Weights[WeightIndex];
					}

					if (SourceBiNormalIterBegin.ptr())
					{
						BiNormal += SkinTransforms[BoneIndices[WeightIndex]].TransformVector(SourceBiNormal) * Weights[WeightIndex];
					}
				}

				const float TotalWeightRcp = TotalWeight > UE_SMALL_NUMBER ? 1.0f/TotalWeight : 1.0f; 
				Position *= TotalWeightRcp;

				float* TargetPositionData = *(TargetPositionIterBegin + VertexIndex); 
				TargetPositionData[0] = Position[0];
				TargetPositionData[1] = Position[1];
				TargetPositionData[2] = Position[2];

				if (TargetNormalIterBegin.ptr())
				{
					// This will maintain any packed format sign component if present.
					(TargetNormalIterBegin + VertexIndex).SetFromVec3f(Normal.GetSafeNormal());
				}

				if (TargetTangentIterBegin.ptr())
				{
					(TargetTangentIterBegin + VertexIndex).SetFromVec3f(Tangent.GetSafeNormal());
				}

				if (TargetBiNormalIterBegin.ptr())
				{
					(TargetBiNormalIterBegin + VertexIndex).SetFromVec3f(BiNormal.GetSafeNormal());
				}
			}
		});
    }
}
