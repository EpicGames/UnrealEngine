// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ConvertData.h"
#include "MuR/MeshPrivate.h"

#include "Math/Plane.h"
#include "Math/Ray.h"
#include "Math/NumericLimits.h"

#include "IndexTypes.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/GeometryUtils.h"

#include "PackedNormal.h"

#include "MuR/OpMeshSmoothing.h"

// TODO: Make the handling of rotations an option. It is more expensive on CPU and memory, and for some
// cases it is not required at all.

// TODO: Face stretch to scale the deformation per-vertex? 

// TODO: Support multiple binding influences per vertex, to have smoother deformations.

// TODO: Support multiple binding sets, to have different shapes deformations at once.

// TODO: Deformation mask, to select the intensity of the deformation per-vertex.

// TODO: This is a reference implementation with ample roof for optimization.

namespace UE::Mutable::Private
{
	using FShapeMeshTree = UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter>;

	struct FReshapeVertexBindingData
	{
		// Index of the shape triangle and the type of binding, Cluster or Vertex. 
		uint32 TriangleIndex_BindingType_NotValid; // the 30 low bits are the triangle index, 30 ith bit is the binding type, 31 ith bit is set if the biding is valid. 
		
		// Barycentric coordinates on the shape triangle
		FVector2f PositionBaryCoords;

		// Distance along the normals of the shape triangle.
		FVector3f D;

		union
		{
			struct
			{
				FVector3f NormalBaryCoords; // NormalBaryCoords.Z is not used.
				FVector3f NormalD;
			} Vertex;

			struct
			{
				FVector3f AttachmentPoint;
				FVector3f AttachmentNormal;
			} Cluster;
		};	

		// Weight of the effect for this binding. Ranged between 0 and 1 where 0 denotes no effect at all and 1
		// full effect.
		float Weight;

		FORCEINLINE uint32 GetTriangleIndex() const
		{
			return TriangleIndex_BindingType_NotValid & 0x3FFFFFFF;
		}

		FORCEINLINE bool IsClusterBinding() const
		{
			return (TriangleIndex_BindingType_NotValid & 0x40000000) != 0;
		}

		FORCEINLINE bool IsValidBinding() const
		{
			return (TriangleIndex_BindingType_NotValid & 0x80000000) == 0;
		}

		FORCEINLINE void SetTriangleIndex(uint32 Index)
		{	
			check(Index < 0x3FFFFFFF)
			TriangleIndex_BindingType_NotValid = 
					(TriangleIndex_BindingType_NotValid & ~0x3FFFFFFF) | (Index & 0x3FFFFFFF);
		}

		FORCEINLINE void SetClusterBinding(bool Value)
		{
			uint32 ValueBits = uint32(Value) << 30;
			TriangleIndex_BindingType_NotValid = 
					(TriangleIndex_BindingType_NotValid & ~0x40000000) | ValueBits;
		}

		FORCEINLINE void SetBindNotValid(bool Value)
		{
			uint32 ValueBits = uint32(Value) << 31;
			TriangleIndex_BindingType_NotValid = 
					(TriangleIndex_BindingType_NotValid & ~0x80000000) | ValueBits;
		}


	};
	static_assert(sizeof(FReshapeVertexBindingData) == 4*13);

	struct FReshapeVertexBindingDataBufferDescriptor
	{
		constexpr static int32 ElementSize = sizeof(FReshapeVertexBindingData);
		constexpr static int32 Channels = 4;
		constexpr static EMeshBufferSemantic Semantics[Channels] = { EMeshBufferSemantic::TriangleIndex, EMeshBufferSemantic::BarycentricCoords, EMeshBufferSemantic::Distance, EMeshBufferSemantic::Other };
		constexpr static EMeshBufferFormat Formats[Channels] = { EMeshBufferFormat::UInt32, EMeshBufferFormat::Float32, EMeshBufferFormat::Float32, EMeshBufferFormat::Float32 };
		constexpr static int32 Components[Channels] = { 1, 2, 3, 3+3+1 };
		constexpr static int32 Offsets[Channels] = { 0, 1*4, (1+2)*4, (1+2+3)*4 };

		const int32 SemanticIndices[Channels] = { 0, 0, 0, 0 };

		FReshapeVertexBindingDataBufferDescriptor(int32 DataSetIndex)
			: SemanticIndices{ DataSetIndex, DataSetIndex, DataSetIndex, DataSetIndex }
		{
		}
	};

	struct FReshapePointBindingData
	{
		float S, T;
		FVector3f D;
		int32 Triangle;
		float Weight;
	};

	static_assert(sizeof(FReshapePointBindingData) == 24+4);

	struct FReshapePointBindingDataBufferDescriptor
	{
		constexpr static int32 ElementSize = sizeof(FReshapeVertexBindingData);
		constexpr static int32 Channels = 4;
		constexpr static EMeshBufferSemantic Semantics[Channels] = { EMeshBufferSemantic::BarycentricCoords, EMeshBufferSemantic::Distance, EMeshBufferSemantic::TriangleIndex, EMeshBufferSemantic::Other };
		constexpr static EMeshBufferFormat Formats[Channels] = { EMeshBufferFormat::Float32, EMeshBufferFormat::Float32, EMeshBufferFormat::Int32, EMeshBufferFormat::Float32 };
		constexpr static int32 Components[Channels] = { 2, 3, 1, 1 };
		constexpr static int32 Offsets[Channels] = { 0, 8, 20, 24 };

		const int32 SemanticIndices[Channels] = { 0, 0, 0, 0 };

		FReshapePointBindingDataBufferDescriptor(int32 DataSetIndex)
			: SemanticIndices{ DataSetIndex, DataSetIndex, DataSetIndex, DataSetIndex }
		{
		}
	};
	
	struct FIntBufferDescriptor
	{
		constexpr static int32 ElementSize = sizeof(int32);
		constexpr static int32 Channels = 1;
		constexpr static EMeshBufferSemantic Semantics[Channels] = { EMeshBufferSemantic::Other };
		constexpr static EMeshBufferFormat Formats[Channels] = { EMeshBufferFormat::Int32 };
		constexpr static int32 Components[Channels] = { 1 };
		constexpr static int32 Offsets[Channels] = { 0 };
		constexpr static int32 SemanticIndices[Channels] = { 0 };

		FIntBufferDescriptor()
		{
		}
	};

	using ReshapePoseBindingType = FReshapePointBindingData;
	using ReshapePhysicsBindingType = FReshapePointBindingData;

	// Structure used for vertex bing data in vertexbuffers for Clip deform operations.
	struct FClipDeformVertexBindingData
	{
		// Barycentric coordinates on the shape triangle
		float S, T;

		// Index of the shape triangle.
		int32 Triangle;

		float Weight;
	};
	static_assert(sizeof(FClipDeformVertexBindingData) == 16);

	struct FClipDeformVertexBindingDataBufferDescriptor
	{
		constexpr static int32 Channels = 3;
		constexpr static EMeshBufferSemantic Semantics[Channels] = { EMeshBufferSemantic::BarycentricCoords, EMeshBufferSemantic::TriangleIndex, EMeshBufferSemantic::Other };
		constexpr static EMeshBufferFormat Formats[Channels] = { EMeshBufferFormat::Float32, EMeshBufferFormat::Int32, EMeshBufferFormat::Float32 };
		constexpr static int32 Components[Channels] = { 2, 1, 1 };
		constexpr static int32 Offsets[Channels] = { 0, 8, 12 };

		const int32 SemanticIndices[Channels] = { 0, 0, 0 };

		FClipDeformVertexBindingDataBufferDescriptor(int32 DataSetIndex)
			: SemanticIndices{ DataSetIndex, DataSetIndex, DataSetIndex }
		{
		}
	};

	struct FMeshBindColorChannelUsageMasks
	{
		uint32 MaskWeight = 0;
		uint32 ClusterId  = 0;
	};

	//---------------------------------------------------------------------------------------------
    //! Generate the mesh-shape binding data for Reshape operations
    //---------------------------------------------------------------------------------------------

	inline float GetVertexMaskWeight(const UntypedMeshBufferIteratorConst& ColorIter, const FMeshBindColorChannelUsageMasks& ChannelUsages)
	{
		check(ColorIter.GetFormat() == EMeshBufferFormat::UInt8 || ColorIter.GetFormat() == EMeshBufferFormat::NUInt8);
		check(ColorIter.GetComponents() == 4);
		check(ColorIter.ptr());

		const uint32 Value = *reinterpret_cast<const uint32*>(ColorIter.ptr());
		return static_cast<float>((Value >> FMath::CountTrailingZeros(ChannelUsages.MaskWeight) & 0xFF)) / 255.0f;
	}

	inline uint32 GetVertexClusterId(const UntypedMeshBufferIteratorConst& ColorIter, const FMeshBindColorChannelUsageMasks& ChannelUsages)
	{	
		check(ColorIter.GetFormat() == EMeshBufferFormat::UInt8 || ColorIter.GetFormat() == EMeshBufferFormat::NUInt8);
		check(ColorIter.GetComponents() == 4);
		check(ColorIter.ptr());

		const uint32 MaskedValue = *reinterpret_cast<const uint32*>(ColorIter.ptr()) & ChannelUsages.ClusterId;
		// Set all unused bits to 1 so we have a consistent value, in that case white, for the non clustered vertices.
		return MaskedValue | ~ChannelUsages.ClusterId;
	}


	FORCEINLINE FVector2f ComputeBarycentric(const FVector3f& Point, const FVector3f& A, const FVector3f& B, const FVector3f& C)
	{
		const FVector3f TriNorm = FVector3f::CrossProduct(B - A, C - A);

		const float TriNormSizeSquared = TriNorm.SizeSquared();
		// Return the center of the triangle if the area is very small.
		if (TriNormSizeSquared < UE_KINDA_SMALL_NUMBER)
		{
			return FVector2f(1.0f/3.0f);
		}

		const float InvNormSizeSquared = 1.0f / TriNormSizeSquared;


		const float AreaPBC = FVector3f::DotProduct(TriNorm, FVector3f::CrossProduct(B - Point, C - Point));
		const float AreaPCA = FVector3f::DotProduct(TriNorm, FVector3f::CrossProduct(C - Point, A - Point));
		
		FVector2f Result = FVector2f(AreaPBC, AreaPCA) * InvNormSizeSquared;

		return Result;
	}

	inline void BindReshapePoint(
			FShapeMeshTree& ShapeMeshTree,
			const FVector3f& Point, const FVector3f& Normal, float MaskWeight, FReshapeVertexBindingData& OutBindData, 
			const float ValidityTolerance = UE_KINDA_SMALL_NUMBER)
	{

		const GeometryUtils::FMeshGeometry& ShapeMesh = ShapeMeshTree.GetMesh()->Mesh;
	
		FMemory::Memzero(OutBindData);

		float Weight = MaskWeight;

		if (FMath::IsNearlyZero(MaskWeight))
		{
			OutBindData.SetBindNotValid(true);
			return;
		}

		double DistSqr = 0.0;

		int32 FoundIndex = ShapeMeshTree.FindNearestTriangle(FVector3d(Point), DistSqr);
			
		if (FoundIndex < 0)
		{
			OutBindData.SetBindNotValid(true);
			return;
		}

		// Calculate the binding data of the base mesh vertex to its bound shape triangle
		UE::Geometry::FIndex3i Triangle = ShapeMesh.Triangles[FoundIndex];

		// Project on the triangle, but using the vertex normals.
		// See reference implementation for details. 
		FVector3f TriangleA = (FVector3f)ShapeMesh.Positions[Triangle.A];
		FVector3f TriangleB = (FVector3f)ShapeMesh.Positions[Triangle.B];
		FVector3f TriangleC = (FVector3f)ShapeMesh.Positions[Triangle.C];

		FPlane4f TrianglePlane(TriangleA, TriangleB, TriangleC);
		FVector3f PlaneNormal = TrianglePlane.GetNormal();
		FPlane4f VertexPlane(Point, PlaneNormal);
		FPlane4f NormalPlane(Point + Normal, PlaneNormal);

		// T1 = Triangle projected on the vertex plane along the triangle vertex normals
		OutBindData.D = 
			FVector3f 
			{
				FMath::RayPlaneIntersectionParam(TriangleA, ShapeMesh.Normals[Triangle.A], VertexPlane),
				FMath::RayPlaneIntersectionParam(TriangleB, ShapeMesh.Normals[Triangle.B], VertexPlane),
				FMath::RayPlaneIntersectionParam(TriangleC, ShapeMesh.Normals[Triangle.C], VertexPlane)
			};

		OutBindData.Vertex.NormalD = 
			FVector3f
			{
				FMath::RayPlaneIntersectionParam(TriangleA, ShapeMesh.Normals[Triangle.A], NormalPlane),
				FMath::RayPlaneIntersectionParam(TriangleB, ShapeMesh.Normals[Triangle.B], NormalPlane),
				FMath::RayPlaneIntersectionParam(TriangleC, ShapeMesh.Normals[Triangle.C], NormalPlane)
			};

		FVector2f PositionBarycentric = ComputeBarycentric(Point,
					TriangleA + ShapeMesh.Normals[Triangle.A] * OutBindData.D.X, 
					TriangleB + ShapeMesh.Normals[Triangle.B] * OutBindData.D.Y, 
					TriangleC + ShapeMesh.Normals[Triangle.C] * OutBindData.D.Z); 

		OutBindData.PositionBaryCoords.X = PositionBarycentric.X;
		OutBindData.PositionBaryCoords.Y = PositionBarycentric.Y;
	
		FVector2f NormalBarycentric = ComputeBarycentric((Point + Normal), 
				TriangleA + ShapeMesh.Normals[Triangle.A] * OutBindData.Vertex.NormalD.X, 
				TriangleB + ShapeMesh.Normals[Triangle.B] * OutBindData.Vertex.NormalD.Y, 
				TriangleC + ShapeMesh.Normals[Triangle.C] * OutBindData.Vertex.NormalD.Z); 

		OutBindData.Vertex.NormalBaryCoords.X = NormalBarycentric.X;
		OutBindData.Vertex.NormalBaryCoords.Y = NormalBarycentric.Y;

		check(FoundIndex >= 0);
		OutBindData.SetTriangleIndex((uint32)FoundIndex);
		OutBindData.Weight = FMath::Clamp(MaskWeight, 0.0f, 1.0f);

		OutBindData.SetBindNotValid(FMath::IsNearlyZero(OutBindData.Weight));
	}
	
    //---------------------------------------------------------------------------------------------
    //! Find mesh clusters.
	//! Color {1,1,1,1} is reserved for the non rigid cluster
    //---------------------------------------------------------------------------------------------
	
	inline void FindRigidClusters(const FMesh* Mesh, const FMeshBindColorChannelUsageMasks& ColorUsageMasks, TArray<TArray<int32>>& OutClusters, int32& OutNonRigidClusterIdx)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshFindRigidClusters);
		
		constexpr uint32 NonRigidId = ~0;

		OutNonRigidClusterIdx = -1;
		
		const UntypedMeshBufferIteratorConst ItColorBase(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Color);
		if (ItColorBase.ptr())
		{
			const int32 VertexCount = Mesh->GetVertexCount();

			{
				int32 VertexIndex = 0;
				for (; VertexIndex < VertexCount; ++VertexIndex)
				{
					 if (NonRigidId != GetVertexClusterId(ItColorBase + VertexIndex, ColorUsageMasks))
					 {
						  break;
					 }
				}

				// If all equal to the non rigid we are done. 
				if (VertexIndex == VertexCount)
				{
					 return;
				}
			}
			
			OutClusters.Empty(16);
			TArray<TArray<int32>>& ClusterData = OutClusters;
			TMap<uint32, int32> ClusterSet;

			for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
			{
				const uint32 ClusterId = GetVertexClusterId(ItColorBase + VertexIndex, ColorUsageMasks);
				int32& ClusterIdx = ClusterSet.FindOrAdd(ClusterId, INDEX_NONE);
				TArray<int32>& Cluster = ClusterIdx < 0 ? ClusterData.Emplace_GetRef() : ClusterData[ClusterIdx];
				if (ClusterIdx < 0)
				{
					ClusterIdx = ClusterData.Num() - 1;
					Cluster.Reserve(32);
				}

				Cluster.Add(VertexIndex);	
			}

			// Cluster id 0xFFFFFFFF is reserved for the nonrigid cluster. The choice if this value is not arbitrary,
			// meshes without color will get white as default.
			const int32* NonRigidClusterIdxFound = ClusterSet.Find(NonRigidId);

			// If not found, add an empty cluster for the non rigid id.
			OutNonRigidClusterIdx = NonRigidClusterIdxFound ? *NonRigidClusterIdxFound : ClusterData.Emplace();
		}
	}
	
	inline void FindBindingForCluster( 
			const FMesh* BaseMesh, FShapeMeshTree& ShapeMeshTree,
			const TArray<int32>& Cluster, FReshapeVertexBindingData& OutBindingData, float BindTolerance)
	{
		const UntypedMeshBufferIteratorConst ItPositionBase(BaseMesh->GetVertexBuffers(), EMeshBufferSemantic::Position);

		FBox3f ClusterBoundingBox(ForceInit);
		for (const int32& V : Cluster)
		{
			ClusterBoundingBox += (ItPositionBase + V).GetAsVec3f();
		}

		FVector3f BoundPoint = ClusterBoundingBox.GetCenter();

		// Mask weight is set on a vertex by vertex basis, ignore weight for the shared data.
		// This will be filled in afterwards.
		constexpr float MaskWeight = 1.0f;
		BindReshapePoint(ShapeMeshTree, BoundPoint, FVector3f::ZAxisVector, MaskWeight, OutBindingData, BindTolerance);

		if (OutBindingData.IsValidBinding())
		{
			const GeometryUtils::FMeshGeometry& ShapeMesh = ShapeMeshTree.GetMesh()->Mesh;
			UE::Geometry::FIndex3i Triangle = ShapeMesh.Triangles[OutBindingData.GetTriangleIndex()];

			FVector3f TriangleA = (FVector3f)ShapeMesh.Positions[Triangle.A];
			FVector3f TriangleB = (FVector3f)ShapeMesh.Positions[Triangle.B];
			FVector3f TriangleC = (FVector3f)ShapeMesh.Positions[Triangle.C];

			//TODO: store triangle area for scale.
			//OutBindingData.Cluster.AttachementArea = FVector3f::CrossProduct((TriangleB - TriangleA), (TriangleC - TriangleA)).Size() / 2.0f;
			OutBindingData.Cluster.AttachmentPoint = BoundPoint;
			OutBindingData.Cluster.AttachmentNormal = FPlane4f(TriangleA, TriangleB, TriangleC).GetNormal(); 

			// Mark binding as cluster.
			OutBindingData.SetClusterBinding(true);
		}
	}

	inline TTuple<TArray<FReshapePointBindingData>, TArray<int32>, TArray<int32>> BindPhysicsBodies( 
			TArray<const FPhysicsBody*> PhysicsBodies, FShapeMeshTree& ShapeMeshTree, const FMesh& Mesh, const FSkeleton& Skeleton, 
			TConstArrayView<FName> PhysicsToDeform, bool bInvertSelection)
	{
		TTuple<TArray<FReshapePointBindingData>, TArray<int32>, TArray<int32>> ReturnValue;

		TArray<int32>& BodiesToDeformIndices = ReturnValue.Get<1>();
		TArray<int32>& BodiesToDeformOffsets = ReturnValue.Get<2>();

		const int32 NumPhysicsBodies = PhysicsBodies.Num();

		BodiesToDeformIndices.Reserve(NumPhysicsBodies * PhysicsToDeform.Num());
		BodiesToDeformOffsets.SetNum(NumPhysicsBodies + 1);
	
		BodiesToDeformOffsets[0] = 0;
		for (int32 PhysicsBodyIndex = 0; PhysicsBodyIndex < NumPhysicsBodies; ++PhysicsBodyIndex)
		{
			if (const FPhysicsBody* Body = PhysicsBodies[PhysicsBodyIndex])
			{
				const int32 NumBodies = PhysicsBodies[PhysicsBodyIndex]->GetBodyCount();
				for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
				{
					const bool bContains = PhysicsToDeform.Contains(Body->GetBodyBoneName(BodyIndex));
					if (bInvertSelection != bContains)
					{
						BodiesToDeformIndices.Add(BodyIndex);
					}
				}
			}

			BodiesToDeformOffsets[PhysicsBodyIndex + 1] = BodiesToDeformIndices.Num();
		}

		int32 TotalNumPoints = 0;
		for (int32 PhysicsBodyIndex = 0; PhysicsBodyIndex < NumPhysicsBodies; ++PhysicsBodyIndex)
		{
			const int32 IndicesBegin = BodiesToDeformOffsets[PhysicsBodyIndex];
			const int32 IndicesEnd = BodiesToDeformOffsets[PhysicsBodyIndex + 1];

			TArrayView<int32> BodyIndices = TArrayView<int32>(
				BodiesToDeformIndices.GetData() + IndicesBegin, IndicesEnd - IndicesBegin);

			const FPhysicsBody* Body = PhysicsBodies[PhysicsBodyIndex];
		
			int32 BodyNumPoints = 0;
			for (const int32 I : BodyIndices)
		{
				BodyNumPoints += Body->GetSphereCount(I) * 6;
				BodyNumPoints += Body->GetBoxCount(I) * 14;
				BodyNumPoints += Body->GetSphylCount(I) * 14;
				BodyNumPoints += Body->GetTaperedCapsuleCount(I) * 14;

				const int32 ConvexCount = Body->GetConvexCount(I);
				for (int C = 0; C < ConvexCount; ++C)
			{
				TArrayView<const FVector3f> Vertices;
				TArrayView<const int32> Indices;

				FTransform3f Transform;
					Body->GetConvex(I, C, Vertices, Indices, Transform);

					BodyNumPoints += Vertices.Num();
			}
		}
		
			TotalNumPoints += BodyNumPoints;
		}
		
		TArray<FVector3f> Points;
		Points.SetNumUninitialized(TotalNumPoints);
	
		// Bone transform needs to be applied to the body's sample points so they are in mesh space.

		// Create a point soup to be deformed based on the shapes in the aggregate.

		int32 AddedPoints = 0;

		for (int32 PhysicsBodyIndex = 0; PhysicsBodyIndex < NumPhysicsBodies; ++PhysicsBodyIndex)
		{
			const FPhysicsBody* Body = PhysicsBodies[PhysicsBodyIndex];

			const int32 IndicesBegin = BodiesToDeformOffsets[PhysicsBodyIndex];
			const int32 IndicesEnd = BodiesToDeformOffsets[PhysicsBodyIndex + 1];

			TArrayView<int32> BodyIndices = TArrayView<int32>(
				BodiesToDeformIndices.GetData() + IndicesBegin, IndicesEnd - IndicesBegin);

			for (const int32 B : BodyIndices)
			{
				const int32 BoneIndex = Skeleton.BoneNames.Find(Body->GetBodyBoneName(B));
				int32 PoseIndex = Mesh.FindBonePoseByBoneIndex(BoneIndex);

				FTransform3f T = FTransform3f::Identity;
				if (PoseIndex > 0)
				{
					Mesh.GetBonePoseTransform(PoseIndex, T);
				}

				const int32 SphereCount = Body->GetSphereCount(B);
				for (int32 I = 0; I < SphereCount; ++I, AddedPoints += 6)
				{
					FVector3f P;
					float R;

					Body->GetSphere(B, I, P, R);

					Points[AddedPoints + 0] = T.TransformPosition(P + FVector3f(R, 0.0f, 0.0f));
					Points[AddedPoints + 1] = T.TransformPosition(P - FVector3f(R, 0.0f, 0.0f));

					Points[AddedPoints + 2] = T.TransformPosition(P + FVector3f(0.0f, R, 0.0f));
					Points[AddedPoints + 3] = T.TransformPosition(P - FVector3f(0.0f, R, 0.0f));

					Points[AddedPoints + 4] = T.TransformPosition(P + FVector3f(0.0f, 0.0f, R));
					Points[AddedPoints + 5] = T.TransformPosition(P - FVector3f(0.0f, 0.0f, R));
				}

				const int32 BoxCount = Body->GetBoxCount(B);
				for (int32 I = 0; I < BoxCount; ++I, AddedPoints += 14)
				{
					FVector3f P;
					FQuat4f Q;
					FVector3f S;

					Body->GetBox(B, I, P, Q, S);

					const FVector3f BasisX = Q.RotateVector(FVector3f::UnitX());
					const FVector3f BasisY = Q.RotateVector(FVector3f::UnitY());
					const FVector3f BasisZ = Q.RotateVector(FVector3f::UnitZ());

					Points[AddedPoints + 0] = T.TransformPosition(P + BasisX * S.X + BasisY * S.Y + BasisZ * S.Z);
					Points[AddedPoints + 1] = T.TransformPosition(P + BasisX * S.X - BasisY * S.Y + BasisZ * S.Z);
					Points[AddedPoints + 2] = T.TransformPosition(P - BasisX * S.X + BasisY * S.Y + BasisZ * S.Z);
					Points[AddedPoints + 3] = T.TransformPosition(P - BasisX * S.X - BasisY * S.Y + BasisZ * S.Z);

					Points[AddedPoints + 4] = T.TransformPosition(P + BasisX * S.X + BasisY * S.Y - BasisZ * S.Z);
					Points[AddedPoints + 5] = T.TransformPosition(P + BasisX * S.X - BasisY * S.Y - BasisZ * S.Z);
					Points[AddedPoints + 6] = T.TransformPosition(P - BasisX * S.X + BasisY * S.Y - BasisZ * S.Z);
					Points[AddedPoints + 7] = T.TransformPosition(P - BasisX * S.X - BasisY * S.Y - BasisZ * S.Z);

					Points[AddedPoints + 8] = T.TransformPosition(P + BasisX * S.X);
					Points[AddedPoints + 9] = T.TransformPosition(P + BasisY * S.Y);
					Points[AddedPoints + 10] = T.TransformPosition(P + BasisZ * S.Z);

					Points[AddedPoints + 11] = T.TransformPosition(P - BasisX * S.X);
					Points[AddedPoints + 12] = T.TransformPosition(P - BasisY * S.Y);
					Points[AddedPoints + 13] = T.TransformPosition(P - BasisZ * S.Z);
				}

				const int32 SphylCount = Body->GetSphylCount(B);
				for (int32 I = 0; I < SphylCount; ++I, AddedPoints += 14)
				{
					FVector3f P;
					FQuat4f Q;
					float R;
					float L;

					Body->GetSphyl(B, I, P, Q, R, L);

					const float H = L * 0.5f;

					const FVector3f BasisX = Q.RotateVector(FVector3f::UnitX());
					const FVector3f BasisY = Q.RotateVector(FVector3f::UnitY());
					const FVector3f BasisZ = Q.RotateVector(FVector3f::UnitZ());

					// Top and Bottom
					Points[AddedPoints + 0] = T.TransformPosition(P + BasisZ * (H + R));
					Points[AddedPoints + 1] = T.TransformPosition(P - BasisZ * (H + R));

					// Top ring
					Points[AddedPoints + 2] = T.TransformPosition(P + BasisX * R + BasisZ * H);
					Points[AddedPoints + 3] = T.TransformPosition(P - BasisX * R + BasisZ * H);
					Points[AddedPoints + 4] = T.TransformPosition(P + BasisY * R + BasisZ * H);
					Points[AddedPoints + 5] = T.TransformPosition(P - BasisY * R + BasisZ * H);

					// Center ring
					Points[AddedPoints + 6] = T.TransformPosition(P + BasisX * R);
					Points[AddedPoints + 7] = T.TransformPosition(P - BasisX * R);
					Points[AddedPoints + 8] = T.TransformPosition(P + BasisY * R);
					Points[AddedPoints + 9] = T.TransformPosition(P - BasisY * R);

					// Bottom ring
					Points[AddedPoints + 10] = T.TransformPosition(P + BasisX * R - BasisZ * H);
					Points[AddedPoints + 11] = T.TransformPosition(P - BasisX * R - BasisZ * H);
					Points[AddedPoints + 12] = T.TransformPosition(P + BasisY * R - BasisZ * H);
					Points[AddedPoints + 13] = T.TransformPosition(P - BasisY * R - BasisZ * H);

				}

				const int32 TaperedCapsuleCount = Body->GetTaperedCapsuleCount(B);
				for (int32 I = 0; I < TaperedCapsuleCount; ++I, AddedPoints += 14)
				{
					FVector3f P;
					FQuat4f Q;
					float R0;
					float R1;
					float L;

					Body->GetTaperedCapsule(B, I, P, Q, R0, R1, L);

					const float H = L * 0.5f;
					const float RCenter = (R0 + R1) * 0.5f;

					const FVector3f BasisX = Q.RotateVector(FVector3f::UnitX());
					const FVector3f BasisY = Q.RotateVector(FVector3f::UnitY());
					const FVector3f BasisZ = Q.RotateVector(FVector3f::UnitZ());

					// Top and Bottom
					Points[AddedPoints + 0] = T.TransformPosition(P + BasisZ * (H + R0));
					Points[AddedPoints + 1] = T.TransformPosition(P - BasisZ * (H + R1));

					// Top ring
					Points[AddedPoints + 2] = T.TransformPosition(P + BasisX * R0 + BasisZ * H);
					Points[AddedPoints + 3] = T.TransformPosition(P - BasisX * R0 + BasisZ * H);
					Points[AddedPoints + 4] = T.TransformPosition(P + BasisY * R0 + BasisZ * H);
					Points[AddedPoints + 5] = T.TransformPosition(P - BasisY * R0 + BasisZ * H);

					// Center ring
					Points[AddedPoints + 6] = T.TransformPosition(P + BasisX * RCenter);
					Points[AddedPoints + 7] = T.TransformPosition(P - BasisX * RCenter);
					Points[AddedPoints + 8] = T.TransformPosition(P + BasisY * RCenter);
					Points[AddedPoints + 9] = T.TransformPosition(P - BasisY * RCenter);

					// Bottom ring
					Points[AddedPoints + 10] = T.TransformPosition(P + BasisX * R1 - BasisZ * H);
					Points[AddedPoints + 11] = T.TransformPosition(P - BasisX * R1 - BasisZ * H);
					Points[AddedPoints + 12] = T.TransformPosition(P + BasisY * R1 - BasisZ * H);
					Points[AddedPoints + 13] = T.TransformPosition(P - BasisY * R1 - BasisZ * H);
				}

				const int32 ConvexCount = Body->GetConvexCount(B);
				for (int32 I = 0; I < ConvexCount; ++I)
				{

					TArrayView<const FVector3f> VerticesView;
					TArrayView<const int32> IndicesView;
					FTransform3f ConvexT;

					Body->GetConvex(B, I, VerticesView, IndicesView, ConvexT);

					ConvexT = T * ConvexT;
					for (const FVector3f& P : VerticesView)
					{
						Points[AddedPoints++] = ConvexT.TransformPosition(P);
					}
				}
			}
		}
		
		TArray<FReshapePointBindingData>& PhysicsBodyBindData = ReturnValue.Get<0>();
		PhysicsBodyBindData.SetNumUninitialized(TotalNumPoints);

		FReshapeVertexBindingData VertexBindData;

		for (int32 PointIndex = 0; PointIndex < TotalNumPoints; ++PointIndex)
		{
			constexpr float MaskWeight = 1.0f;
			BindReshapePoint(ShapeMeshTree, Points[PointIndex], FVector3f::ZAxisVector, MaskWeight, VertexBindData, 0.1f);

			int32 TriangleIndex = VertexBindData.IsValidBinding() 
					? (int32)VertexBindData.GetTriangleIndex() 
					: -1;

			PhysicsBodyBindData[PointIndex] = FReshapePointBindingData
			{ 
				.S = VertexBindData.PositionBaryCoords.X, 
				.T = VertexBindData.PositionBaryCoords.Y, 
				.D = VertexBindData.D, 
				.Triangle = TriangleIndex,
				.Weight = VertexBindData.Weight,
			};
		}

		return ReturnValue;
	}

	inline TArray<FReshapeVertexBindingData> BindVerticesReshape(
			const FMesh* BaseMesh, FShapeMeshTree& ShapeMeshTree, const FMeshBindColorChannelUsageMasks& ColorUsageMasks)	
	{
	
		UE::Geometry::FAxisAlignedBox3d ShapeAABBox = ShapeMeshTree.GetBoundingBox();

		const float BindValidityTolerance = ShapeAABBox.MaxDim() < 1.0 
										  ? UE_KINDA_SMALL_NUMBER
										  : static_cast<float>(ShapeAABBox.MaxDim()) * 1e-3f;

		TArray<TArray<int32>> VertexClusters;
		int32 NonRigidClusterIdx = -1;

		if (ColorUsageMasks.ClusterId != 0)
		{
			FindRigidClusters(BaseMesh, ColorUsageMasks, VertexClusters, NonRigidClusterIdx);
		}
		
		// Find nearest shape triangle for each base mesh vertex		
		const int32 MeshVertexCount = BaseMesh->GetVertexCount();
		TArray<FReshapeVertexBindingData> BindData;
		{
			MUTABLE_CPUPROFILER_SCOPE(Project);

			BindData.SetNum(MeshVertexCount);

			MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> PositionIter(BaseMesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
			TArrayView<const FVector3f> Positions = TArrayView<const FVector3f>(reinterpret_cast<const FVector3f*>(PositionIter.ptr()), BaseMesh->GetVertexCount());

			const UntypedMeshBufferIteratorConst ItNormalBase(BaseMesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);

			// Disable vertex color reads if the color is not used for mask weights.
			const UntypedMeshBufferIteratorConst ItColorBase = ColorUsageMasks.MaskWeight == 0 
				? UntypedMeshBufferIteratorConst()
				: UntypedMeshBufferIteratorConst(BaseMesh->GetVertexBuffers(), EMeshBufferSemantic::Color);

			// Special case for non rigid parts

			// This indicates that we don't care about rigid parts,
			// only one clusters is found or there is no cluster data or rigid parts is disabled.
			if (NonRigidClusterIdx < 0)
			{
				const int32 VertexCount = BaseMesh->GetVertexCount();

				constexpr int32 NumBatchElems = 1 << 5; // Small batches perform better.
				const int32 NumBatches = FMath::DivideAndRoundUp<int32>(VertexCount, NumBatchElems);

				TArrayView<const FPackedNormal> PackedNormals;
				if (ItNormalBase.ptr())
				{
					PackedNormals = TArrayView<const FPackedNormal>(reinterpret_cast<const FPackedNormal*>(ItNormalBase.ptr()), VertexCount);
				}

				auto ProcessVertexBatch = [
					Positions, PackedNormals, ItColorBase, &ShapeMeshTree, &BindData, BindValidityTolerance, &ColorUsageMasks, VertexCount, NumBatchElems
				] (int32 BatchId)
					{
						const int32 BatchBeginVertexIndex = BatchId * NumBatchElems;
						const int32 BatchEndVertexIndex = FMath::Min(BatchBeginVertexIndex + NumBatchElems, VertexCount);

						for (int32 VertexIndex = BatchBeginVertexIndex; VertexIndex < BatchEndVertexIndex; ++VertexIndex)
						{
							const FVector3f VertexNormal = PackedNormals.Num() ? PackedNormals[VertexIndex].ToFVector3f() : FVector3f::ZAxisVector;
							const float MaskWeight = ItColorBase.ptr() ? GetVertexMaskWeight(ItColorBase + VertexIndex, ColorUsageMasks) : 1.0f;

							BindReshapePoint(ShapeMeshTree, Positions[VertexIndex], VertexNormal, MaskWeight, BindData[VertexIndex], BindValidityTolerance);
						}
					};

				ParallelFor(NumBatches, ProcessVertexBatch);
			}
			else
			{
				TArray<int32>& NonRigidCluster = VertexClusters[NonRigidClusterIdx];
				const int32 NonRigidVertexCount = NonRigidCluster.Num();

				for (int32 I = 0; I < NonRigidVertexCount; ++I)
				{
					const int32 VertexIndex = NonRigidCluster[I];
					const FVector3f VertexNormal = ItNormalBase.ptr() ? (ItNormalBase + VertexIndex).GetAsVec3f() : FVector3f::ZAxisVector;
					const float MaskWeight = ItColorBase.ptr() ? GetVertexMaskWeight(ItColorBase + VertexIndex, ColorUsageMasks) : 1.0f;
					
					BindReshapePoint(ShapeMeshTree, Positions[VertexIndex], VertexNormal, MaskWeight, BindData[VertexIndex], BindValidityTolerance);
				}

				// Remove data form the non rigid cluster so it ios not processed in the rigid parts binding step.
				NonRigidCluster.Reset();
			
				for (const TArray<int32>& RigidCluster : VertexClusters)
				{
					FReshapeVertexBindingData ClusterBinding;

					const int32 ClusterVertexCount = RigidCluster.Num();
					if (ClusterVertexCount)
					{

						FindBindingForCluster(BaseMesh, ShapeMeshTree, RigidCluster, ClusterBinding, BindValidityTolerance);
						// Copy cluster binding to every vertex of the cluster modifing weight if the vertex color is used.
						for (int32 I = 0; I < ClusterVertexCount; ++I)
						{
							const int32 VertexIndex = RigidCluster[I];

							const float MaskWeight = ItColorBase.ptr() ? GetVertexMaskWeight(ItColorBase + VertexIndex, ColorUsageMasks) : 1.0f;

							BindData[VertexIndex] = ClusterBinding;
							BindData[VertexIndex].Weight = FMath::Min(ClusterBinding.Weight, MaskWeight);

							// Invalidate binding for very small weights.
							BindData[VertexIndex].SetBindNotValid(FMath::IsNearlyZero(ClusterBinding.Weight));	
						}
					}
				}
			}
		}

		return BindData;
	}

	inline void GenerateAndAddLaplacianData(FMesh& InOutMesh)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateLaplacianData);


		// Storage for buffers in a format different than the supported one.
		// Not used if the buffer data is compatible. The data will always be accessed using a 
		// view regardless of compatibility.
		TArray<FVector3f> ConvertedVerticesStorage;
		TArray<uint32>    ConvertedIndicesStorage;

		TArrayView<const FVector3f> VerticesView;
		{
			const UntypedMeshBufferIteratorConst PositionBegin(InOutMesh.GetVertexBuffers(), EMeshBufferSemantic::Position);
			const int32 NumVertices = InOutMesh.GetVertexBuffers().GetElementCount();
			
			const bool bIsCompatibleBuffer =
				PositionBegin.GetElementSize() == sizeof(FVector3f) &&
				PositionBegin.GetFormat() == EMeshBufferFormat::Float32 &&
				PositionBegin.GetComponents() == 3;
			
			const bool bIsAlignmentGood = reinterpret_cast<UPTRINT>(PositionBegin.ptr()) % alignof(FVector3f) == 0;

			if (!bIsCompatibleBuffer || !bIsAlignmentGood)
			{
				ConvertedVerticesStorage.SetNumUninitialized(NumVertices);
				
				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					ConvertedVerticesStorage[VertexIndex] = (PositionBegin + VertexIndex).GetAsVec3f();
				}

				VerticesView = TArrayView<const FVector3f>(ConvertedVerticesStorage.GetData(), NumVertices);
			}
			else
			{
				VerticesView = TArrayView<const FVector3f>(reinterpret_cast<const FVector3f*>(PositionBegin.ptr()), NumVertices);
			}
		}

		TArrayView<const uint32> IndicesView;
		{
			const UntypedMeshBufferIteratorConst IndicesBegin(InOutMesh.GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
			const int32 NumIndices = InOutMesh.GetIndexBuffers().GetElementCount();
			
			const bool bIsCompatibleBuffer =
				IndicesBegin.GetElementSize() == sizeof(uint32) &&
				IndicesBegin.GetFormat() == EMeshBufferFormat::UInt32 &&
				IndicesBegin.GetComponents() == 1;

			const bool bIsAlignmentGood = reinterpret_cast<UPTRINT>(IndicesBegin.ptr()) % alignof(uint32) == 0;
			
			if (!bIsCompatibleBuffer || !bIsAlignmentGood)
			{
				ConvertedIndicesStorage.SetNumUninitialized(NumIndices);
				for (int32 I = 0; I < NumIndices; ++I)
				{
					ConvertedIndicesStorage[I] = (IndicesBegin + I).GetAsUINT32();
				}

				IndicesView = TArrayView<uint32>(ConvertedIndicesStorage.GetData(), ConvertedIndicesStorage.Num());
			}
			else
			{
				IndicesView = TArrayView<const uint32>(reinterpret_cast<const uint32*>(IndicesBegin.ptr()), NumIndices);
			}
		}

		TArray<int32> UniqueVertexMap = MakeUniqueVertexMap(VerticesView);
		TArray<TArray<int32, TInlineAllocator<8>>> VertexFaces = BuildVertexFaces(IndicesView, UniqueVertexMap);
		TMap<uint64, UE::Geometry::FIndex2i> EdgesFaces = BuildEdgesFaces(IndicesView, UniqueVertexMap);
		
		TArray<int32> VertexRingsOffsets;
		TArray<int32> VertexRingsData;
		Tie(VertexRingsOffsets, VertexRingsData) = BuildVertexRings(IndicesView, UniqueVertexMap, VertexFaces, EdgesFaces);

		FMeshBufferSet MeshLaplacianOffsetsBuffer;
		MeshLaplacianOffsetsBuffer.SetBufferCount(1);
		MeshLaplacianOffsetsBuffer.SetElementCount(VertexRingsOffsets.Num());

		FMeshBufferSet MeshLaplacianDataBuffer;
		MeshLaplacianDataBuffer.SetBufferCount(1);
		MeshLaplacianDataBuffer.SetElementCount(VertexRingsData.Num());

		// Don't add this to the vertex buffer set for now, it is currently only used for Laplacian smoothing a and it is removed
		// right away after use.
		FMeshBufferSet UniqueVertexMapBuffer;
		UniqueVertexMapBuffer.SetBufferCount(1);
		UniqueVertexMapBuffer.SetElementCount(UniqueVertexMap.Num());

		const FIntBufferDescriptor BufDesc;
		MeshLaplacianOffsetsBuffer.SetBuffer(0, BufDesc.ElementSize, BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components);
		MeshLaplacianDataBuffer.SetBuffer(0, BufDesc.ElementSize, BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components);
		UniqueVertexMapBuffer.SetBuffer(0, BufDesc.ElementSize, BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components);

		// TODO: Add a way for buffers to steal memory from temporaries, so we can avoid a copy here.
		FMemory::Memcpy(
				UniqueVertexMapBuffer.GetBufferData(0), 
				UniqueVertexMap.GetData(), 
				UniqueVertexMapBuffer.GetDataSize());

		FMemory::Memcpy(
				MeshLaplacianOffsetsBuffer.GetBufferData(0), 
				VertexRingsOffsets.GetData(), 
				MeshLaplacianOffsetsBuffer.GetDataSize());

		FMemory::Memcpy(
				MeshLaplacianDataBuffer.GetBufferData(0), 
				VertexRingsData.GetData(), 
				MeshLaplacianDataBuffer.GetDataSize());

		InOutMesh.AdditionalBuffers.Emplace(EMeshBufferType::UniqueVertexMap, MoveTemp(UniqueVertexMapBuffer));
		InOutMesh.AdditionalBuffers.Emplace(EMeshBufferType::MeshLaplacianOffsets, MoveTemp(MeshLaplacianOffsetsBuffer));
		InOutMesh.AdditionalBuffers.Emplace(EMeshBufferType::MeshLaplacianData, MoveTemp(MeshLaplacianDataBuffer));
	}


	inline TTuple<TArray<FReshapePointBindingData>, TArray<int32>> BindPose(
			const FMesh& Mesh, const FSkeleton& Skeleton, FShapeMeshTree& ShapeMeshTree, TConstArrayView<FName> BonesToDeform, bool bInvertSelection)
	{
		UE::Geometry::FAxisAlignedBox3d ShapeAABBox = ShapeMeshTree.GetBoundingBox();

		const float BindValidityTolerance = ShapeAABBox.MaxDim() < 1.0 
										  ? UE_KINDA_SMALL_NUMBER
										  : static_cast<float>(ShapeAABBox.MaxDim()) * 1e-3f;
		
		TTuple<TArray<FReshapePointBindingData>, TArray<int32>> ReturnValue;

		TArray<FReshapePointBindingData>& SkeletonBindDataArray = ReturnValue.Get<0>();
		TArray<int32>& BoneIndices = ReturnValue.Get<1>();

		const int32 BoneCount = Mesh.GetBonePoseCount();
		SkeletonBindDataArray.Reserve(BoneCount);
		BoneIndices.Reserve(BoneCount);

		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			const EBoneUsageFlags BoneUsageFlags = Mesh.BonePoses[BoneIndex].BoneUsageFlags;

			if (EnumHasAnyFlags(BoneUsageFlags, EBoneUsageFlags::Root))
			{
				continue;
			}

			const bool bContains = bInvertSelection != BonesToDeform.Contains(Skeleton.BoneNames[Mesh.BonePoses[BoneIndex].BoneId.Index]);
			if (!bContains)
			{
				continue;
			}

			FReshapeVertexBindingData BindData;
			constexpr float MaskWeight = 1.0f;
			BindReshapePoint(ShapeMeshTree, Mesh.BonePoses[BoneIndex].BoneTransform.GetLocation(), FVector3f::ZAxisVector, MaskWeight, BindData, BindValidityTolerance);

			// Only add binding if there is a chance of the bone moving.
			if (BindData.Weight > UE_SMALL_NUMBER && BindData.IsValidBinding())
			{
				SkeletonBindDataArray.Emplace(FReshapePointBindingData
				{ 
					.S = BindData.PositionBaryCoords.X, 
					.T = BindData.PositionBaryCoords.Y, 
					.D = BindData.D, 
					.Triangle = (int32)BindData.GetTriangleIndex(), 
					.Weight = BindData.Weight, 
				});
				BoneIndices.Add(BoneIndex);
			}	
		}

		return ReturnValue;
	}


	inline FMeshBindColorChannelUsageMasks MakeColorChannelUsageMasks(FMeshBindColorChannelUsages Usages)
	{
		FMeshBindColorChannelUsageMasks Masks;
	
		// We assume the color will have the FColor layout
		Masks.MaskWeight =
			(Usages.B == EMeshBindColorChannelUsage::MaskWeight ? 0x000000FF : 0) |
			(Usages.G == EMeshBindColorChannelUsage::MaskWeight ? 0x0000FF00 : 0) |
			(Usages.R == EMeshBindColorChannelUsage::MaskWeight ? 0x00FF0000 : 0) |
			(Usages.A == EMeshBindColorChannelUsage::MaskWeight ? 0xFF000000 : 0);

		Masks.ClusterId =
			(Usages.B == EMeshBindColorChannelUsage::ClusterId ? 0x000000FF : 0) |
			(Usages.G == EMeshBindColorChannelUsage::ClusterId ? 0x0000FF00 : 0) |
			(Usages.R == EMeshBindColorChannelUsage::ClusterId ? 0x00FF0000 : 0) |
			(Usages.A == EMeshBindColorChannelUsage::ClusterId ? 0xFF000000 : 0);

		// Maximum one weight channel.
		check(Masks.MaskWeight == 0 ||
			FMath::CountLeadingZeros(Masks.MaskWeight) + FMath::CountTrailingZeros(Masks.MaskWeight) == 32 - 8);
		
		// No overlapped channels.
		check((Masks.ClusterId & Masks.MaskWeight) == 0);

		return Masks;
	}

	//---------------------------------------------------------------------------------------------
    //! Generate the mesh-shape binding data
    //---------------------------------------------------------------------------------------------
    inline void MeshBindShapeReshape(
			FMesh* Result, UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter>& ShapeMeshTree,
			TConstArrayView<FName> BonesToDeform, TConstArrayView<FName> PhysicsToDeform,
			EMeshBindShapeFlags BindFlags, FMeshBindColorChannelUsages ColorChannelUsages,
			bool& bOutSuccess)
    {
		MUTABLE_CPUPROFILER_SCOPE(MeshBindShape);
		bOutSuccess = true;

		if (!Result)
		{
			bOutSuccess = false;
			return;
		}

		const bool bReshapeVertices   = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapeVertices);
		// TODO: For now bRecomputeNormals is not used when binding, we could skip normal morph data generation. 
		const bool bRecomputeNormals  = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::RecomputeNormals);
		const bool bApplyLaplacian    = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ApplyLaplacian);
		const bool bReshapeSkeleton   = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapeSkeleton);
		const bool bReshapePhysics    = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapePhysicsVolumes);
		const bool bReshapeSkeletonInvertSelection = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapeSkeletonInvertSelection);
		const bool bReshapePhysicsVolumesInvertSelection = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapePhysicsVolumesInvertSelection);

		const FMeshBindColorChannelUsageMasks ColorUsagesMasks = MakeColorChannelUsageMasks(ColorChannelUsages);

		// Early out if nothing will be modified and the vertices discarded. return null in this
		// case indicating nothing has modified so the Base Mesh can be reused.
		const bool bSkeletonModification = Result->GetSkeleton() && bReshapeSkeleton;
		const bool bPhysicsModification = (Result->GetPhysicsBody() || Result->AdditionalPhysicsBodies.Num()) && bReshapePhysics;

		if (!bReshapeVertices && !bSkeletonModification && !bPhysicsModification)
		{
			bOutSuccess = false;
			return;
		}

		int32 BindingDataIndex = 0;
		if (bReshapeVertices)
		{
			TArray<FReshapeVertexBindingData> VerticesBindData = BindVerticesReshape(Result, ShapeMeshTree, ColorUsagesMasks);
			
			// Add the binding information to the mesh
			// \TODO: Check that there is no other binding data.
			// \TODO: Support specifying the binding data channel for multiple binding support.
			FMeshBufferSet& VB = Result->GetVertexBuffers();
			int32 NewBufferIndex = VB.GetBufferCount();
			VB.SetBufferCount(NewBufferIndex + 1);

			FReshapeVertexBindingDataBufferDescriptor BufDesc(BindingDataIndex);
			VB.SetBuffer(NewBufferIndex, sizeof(FReshapeVertexBindingData), BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components);
			FMemory::Memcpy(VB.GetBufferData(NewBufferIndex), VerticesBindData.GetData(), VerticesBindData.Num() * sizeof(FReshapeVertexBindingData));
	
			if (bApplyLaplacian)
			{
				GenerateAndAddLaplacianData(*Result);
			}
		}

		// Bind the skeleton bones
		// \TODO: Build bind data only for actually modified bones?
		if (bReshapeSkeleton && Result->GetSkeleton())
		{
			MUTABLE_CPUPROFILER_SCOPE(BindSkeleton);

			TTuple<TArray<FReshapePointBindingData>, TArray<int32>> SkeletonBindingData = 
					BindPose(*Result, *Result->GetSkeleton(), ShapeMeshTree, BonesToDeform, bReshapeSkeletonInvertSelection);
			const TArray<FReshapePointBindingData>& SkeletonBindDataArray = SkeletonBindingData.Get<0>();
			const TArray<int32>& BoneIndices = SkeletonBindingData.Get<1>();

			check(BoneIndices.Num() == SkeletonBindDataArray.Num());

			const int32 NumBonesToDeform = SkeletonBindDataArray.Num();

			FMeshBufferSet SkeletonBuffer;
			SkeletonBuffer.SetBufferCount(2);
			SkeletonBuffer.SetElementCount(SkeletonBindDataArray.Num());

			FReshapePointBindingDataBufferDescriptor BufDesc(BindingDataIndex);
			SkeletonBuffer.SetBuffer(0, sizeof(FReshapePointBindingData), BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components);

			// Bone indices buffer
			EMeshBufferSemantic BoneSemantics[1] = { EMeshBufferSemantic::Other };
			EMeshBufferFormat BoneFormats[1] = { EMeshBufferFormat::Int32 };
			int32 BoneSemanticIndices[1] = { 0 };
			int32 BoneComponents[1] = { 1 };
			int32 BoneOffsets[1] = { 0 };

			SkeletonBuffer.SetBuffer(1, sizeof(int32), 1, BoneSemantics, BoneSemanticIndices, BoneFormats, BoneComponents, BoneOffsets);

			FMemory::Memcpy(SkeletonBuffer.GetBufferData(0), SkeletonBindDataArray.GetData(), NumBonesToDeform * sizeof(FReshapePointBindingData));
			FMemory::Memcpy(SkeletonBuffer.GetBufferData(1), BoneIndices.GetData(), NumBonesToDeform * sizeof(int32));

			Result->AdditionalBuffers.Emplace(EMeshBufferType::SkeletonDeformBinding, MoveTemp(SkeletonBuffer));
		}

		const FPhysicsBody* ResultPhysicsBody = Result->PhysicsBody.Get();
		if (bReshapePhysics && Result->GetSkeleton() && (ResultPhysicsBody || Result->AdditionalPhysicsBodies.Num()))
		{
			MUTABLE_CPUPROFILER_SCOPE(BindPhysicsBody);

			// Gather bodies respecting order, first main physics body then additional bodies.
			// Null entries are need to be able to maintain that order.  
			TArray<const FPhysicsBody*> PhysicsBodiesToBind;
			{
				PhysicsBodiesToBind.Reserve(Result->AdditionalPhysicsBodies.Num() + 1);

				PhysicsBodiesToBind.Add(ResultPhysicsBody);
				for (const TManagedPtr<const FPhysicsBody>& Body : Result->AdditionalPhysicsBodies)
				{
					PhysicsBodiesToBind.Add(Body.Get());
				}
			}

			TTuple<TArray<FReshapePointBindingData>, TArray<int32>, TArray<int32>> PhysicsBindingData = 
					BindPhysicsBodies(PhysicsBodiesToBind, ShapeMeshTree, *Result, *Result->GetSkeleton(), PhysicsToDeform, bReshapePhysicsVolumesInvertSelection);

			const TArray<FReshapePointBindingData>& PhysicsBindDataArray = PhysicsBindingData.Get<0>();
			const TArray<int32>& DeformedBodyIndices = PhysicsBindingData.Get<1>();
			const TArray<int32>& DeformedBodyIndicesOffsets = PhysicsBindingData.Get<2>();

			FMeshBufferSet PhysicsBodyBuffer;
			PhysicsBodyBuffer.SetBufferCount(1);
			PhysicsBodyBuffer.SetElementCount(PhysicsBindDataArray.Num());

			constexpr int32 BindDataIndex = 0; 
			FReshapePointBindingDataBufferDescriptor BufDesc(BindingDataIndex);
			PhysicsBodyBuffer.SetBuffer(0, sizeof(FReshapePointBindingData), BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components, BufDesc.Offsets);	
			FMemory::Memcpy(PhysicsBodyBuffer.GetBufferData(0), PhysicsBindDataArray.GetData(), PhysicsBindDataArray.Num() * sizeof(FReshapePointBindingData));

			FIntBufferDescriptor IntBufDesc;

			FMeshBufferSet PhysicsBodySelectionBuffer;
			PhysicsBodySelectionBuffer.SetBufferCount(1);
			PhysicsBodySelectionBuffer.SetElementCount(DeformedBodyIndices.Num());
			PhysicsBodySelectionBuffer.SetBuffer(0, sizeof(int32), IntBufDesc.Channels, IntBufDesc.Semantics, IntBufDesc.SemanticIndices, IntBufDesc.Formats, IntBufDesc.Components, IntBufDesc.Offsets);	
			FMemory::Memcpy(PhysicsBodySelectionBuffer.GetBufferData(0), DeformedBodyIndices.GetData(), DeformedBodyIndices.Num() * sizeof(int32));

			FMeshBufferSet PhysicsBodySelectionOffsetsBuffer;
			PhysicsBodySelectionOffsetsBuffer.SetBufferCount(1);
			PhysicsBodySelectionOffsetsBuffer.SetElementCount(DeformedBodyIndicesOffsets.Num());
			PhysicsBodySelectionOffsetsBuffer.SetBuffer(0, sizeof(int32), IntBufDesc.Channels, IntBufDesc.Semantics, IntBufDesc.SemanticIndices, IntBufDesc.Formats, IntBufDesc.Components, IntBufDesc.Offsets);	
			FMemory::Memcpy(PhysicsBodySelectionOffsetsBuffer.GetBufferData(0), DeformedBodyIndicesOffsets.GetData(), DeformedBodyIndicesOffsets.Num() * sizeof(int32));

			Result->AdditionalBuffers.Emplace(EMeshBufferType::PhysicsBodyDeformBinding, MoveTemp(PhysicsBodyBuffer));
			Result->AdditionalBuffers.Emplace(EMeshBufferType::PhysicsBodyDeformSelection, MoveTemp(PhysicsBodySelectionBuffer));
			Result->AdditionalBuffers.Emplace(EMeshBufferType::PhysicsBodyDeformOffsets, MoveTemp(PhysicsBodySelectionOffsetsBuffer));
		}	
    }

	//---------------------------------------------------------------------------------------------
    //! Generate the mesh-shape binding data for ClipDeform operations
    //---------------------------------------------------------------------------------------------
	inline void BindClipDeformPointClosestProject(
			FShapeMeshTree& ShapeMeshTree,
			const FVector3f& Point, FClipDeformVertexBindingData& OutBindData, float ValidityTolerance )
	{
		const GeometryUtils::FMeshGeometry& ShapeMesh = ShapeMeshTree.GetMesh()->Mesh;
		
		OutBindData.S = 0.0f;
		OutBindData.T = 0.0f;
		OutBindData.Weight = 0.0f;
		OutBindData.Triangle = -1;

		double DistSqr = 0.0;

		int32 FoundIndex = ShapeMeshTree.FindNearestTriangle( FVector3d(Point), DistSqr );
			
		if (FoundIndex < 0)
		{
			return;
		}

		// Calculate the binding data of the base mesh vertex to its bound shape triangle
		UE::Geometry::FIndex3i TriangleIndices = ShapeMesh.Triangles[FoundIndex];

		// Project on the triangle, but using the vertex normals.
		// See reference implementation for details.
		UE::Geometry::FTriangle3f ShapeTriangle {
				(FVector3f)ShapeMesh.Positions[TriangleIndices.A],
				(FVector3f)ShapeMesh.Positions[TriangleIndices.B],
				(FVector3f)ShapeMesh.Positions[TriangleIndices.C] };

		const FVector3f ShapeTriangleNormal = ShapeTriangle.Normal();
		const FPlane4f VertexPlane(Point, ShapeTriangleNormal);
	
		const UE::Geometry::FTriangle3f TriangleVertexPlane {
				FMath::RayPlaneIntersection(ShapeTriangle.V[0], ShapeMesh.Normals[TriangleIndices.A], VertexPlane),
				FMath::RayPlaneIntersection(ShapeTriangle.V[1], ShapeMesh.Normals[TriangleIndices.B], VertexPlane),
				FMath::RayPlaneIntersection(ShapeTriangle.V[2], ShapeMesh.Normals[TriangleIndices.C], VertexPlane) };

		FVector3f Barycentric = TriangleVertexPlane.GetBarycentricCoords(Point);

		FVector3f InterpolatedShapeNormal = 
				ShapeMesh.Normals[TriangleIndices.A] * Barycentric.X + 
				ShapeMesh.Normals[TriangleIndices.B] * Barycentric.Y + 
				ShapeMesh.Normals[TriangleIndices.C] * Barycentric.Z;

		FVector3f BindPoint = ShapeTriangle.BarycentricPoint(Barycentric);
		FVector3f ProjectedToVertex = (Point - BindPoint);

		// Compute reprojection value to see if the binding is valid.
		const float InterpolatedNormalSizeSquared = InterpolatedShapeNormal.SizeSquared();
		const float InvInterpolatedNormalSizeSquared = InterpolatedNormalSizeSquared > SMALL_NUMBER ? 1.0f / InterpolatedNormalSizeSquared : 0.0f;	
		float D = FVector3f::DotProduct(ProjectedToVertex, InterpolatedShapeNormal) * InvInterpolatedNormalSizeSquared;
	
		const FVector3f ReprojectedPoint = BindPoint + InterpolatedShapeNormal * D;
		
		const FVector3f ReprojectedVector = ReprojectedPoint - Point;
		const float ErrorEstimate = (ReprojectedPoint - Point).GetAbsMax();
		
		// If within the tolerance, 1.0, otherwise linear falloff based on the tolerance

		// Arbitrary factor, a binding will be considered valid (with its corresponding weight) to ErrorFalloffFactor times the validity tolerance.
		constexpr float ErrorFalloffFactor = 4.0f;
		OutBindData.Weight = 1.0f - FMath::Clamp( (ErrorEstimate - ValidityTolerance) / (ValidityTolerance * ErrorFalloffFactor), 0.0f, 1.0f);
	
		OutBindData.S = Barycentric.Y;
		OutBindData.T = Barycentric.Z;

		// Only move points that bind outside the shape.
		OutBindData.Triangle = FVector3f::DotProduct(ShapeTriangleNormal, Point - BindPoint) < 0 ? -1 : FoundIndex;		
	}

	inline void BindClipDeformPointClosestToSurface( 
       		const FShapeMeshTree& ShapeMeshTree, 
			const FVector3f& Point, FClipDeformVertexBindingData& OutBindData)
	{
		const GeometryUtils::FMeshGeometry& ShapeMesh = ShapeMeshTree.GetMesh()->Mesh;
		
		OutBindData.S = 0.0f;
		OutBindData.T = 0.0f;
		OutBindData.Weight = 1.0f;
		OutBindData.Triangle = -1;

		const FVector3d P = FVector3d(Point);

		double DistSqr = 0;
		int32 FoundTriIndex = ShapeMeshTree.FindNearestTriangle(P, DistSqr);
	
		if (FoundTriIndex < 0)
		{
			return;
		}

		check( FoundTriIndex >= 0); 

		UE::Geometry::FIndex3i Triangle = ShapeMesh.Triangles[FoundTriIndex];
		UE::Geometry::FTriangle3d NearestShapeTriangle { 
				(FVector3d)ShapeMesh.Positions[Triangle.A], 
				(FVector3d)ShapeMesh.Positions[Triangle.B], 
				(FVector3d)ShapeMesh.Positions[Triangle.C] };
	
		UE::Geometry::FDistPoint3Triangle3d Dist(P, NearestShapeTriangle);
		Dist.ComputeResult();

		const FVector3d BindPoint = NearestShapeTriangle.BarycentricPoint(Dist.TriangleBaryCoords);

		OutBindData.S = float(Dist.TriangleBaryCoords.Y);
		OutBindData.T = float(Dist.TriangleBaryCoords.Z); 

		// Only move points that bind outside the shape.
		OutBindData.Triangle = FVector3d::DotProduct(NearestShapeTriangle.Normal(), P - BindPoint) < 0 ? -1 : FoundTriIndex;
	}

	inline void BindClipDeformPointNormalProject( 
       		const FShapeMeshTree& ShapeMeshTree,
			const FVector3f& Point, const FVector3f& Normal, FClipDeformVertexBindingData& OutBindData)
	{
		const GeometryUtils::FMeshGeometry& ShapeMesh = ShapeMeshTree.GetMesh()->Mesh;

		OutBindData.S = 0.0f;
		OutBindData.T = 0.0f;
		OutBindData.Weight = 1.0f;
		OutBindData.Triangle = -1;
	
		const FRay3d NormalRay = FRay3d(FVector3d(Point), FVector3d(-Normal));
		double RayHitDist = 0.0;
		FVector3d RayHitBarycenticCoords = FVector3d::ZeroVector;
		int32 TriangleIndex = -1;

		const bool bHitFound = ShapeMeshTree.FindNearestHitTriangle(NormalRay, RayHitDist, TriangleIndex, RayHitBarycenticCoords);

		if (!bHitFound || TriangleIndex < 0)
		{
			return;
		}
		
		UE::Geometry::FIndex3i ShapeTriangleIndices = ShapeMesh.Triangles[TriangleIndex];
		UE::Geometry::FTriangle3d HitShapeTriangle {
				(FVector3d)ShapeMesh.Positions[ShapeTriangleIndices.A],
				(FVector3d)ShapeMesh.Positions[ShapeTriangleIndices.B],
				(FVector3d)ShapeMesh.Positions[ShapeTriangleIndices.C] };
	
		OutBindData.S = float(RayHitBarycenticCoords.Y);
		OutBindData.T = float(RayHitBarycenticCoords.Z);

		OutBindData.Triangle = FVector3d::DotProduct(HitShapeTriangle.Normal(), -NormalRay.Direction) < 0 ? -1 : TriangleIndex;
	}
	
	inline TArray<FClipDeformVertexBindingData> BindVerticesClipDeform(
			const FMesh* BaseMesh, 
			FShapeMeshTree& ShapeMeshTree, 
			EShapeBindingMethod BindingMethod )
	{	
		UE::Geometry::FAxisAlignedBox3d ShapeAABBox = ShapeMeshTree.GetBoundingBox();

		const float BindValidityTolerance = ShapeAABBox.MaxDim() < 1.0 
										  ? UE_KINDA_SMALL_NUMBER
										  : static_cast<float>(ShapeAABBox.MaxDim()) * 1e-3f;

		// Find nearest shape triangle for each base mesh vertex		
		const int32 MeshVertexCount = BaseMesh->GetVertexCount();
		TArray<FClipDeformVertexBindingData> BindData;
		{
			MUTABLE_CPUPROFILER_SCOPE(ClipDeformBind);

			BindData.SetNum( MeshVertexCount );

			UntypedMeshBufferIteratorConst ItPositionBase(BaseMesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
			UntypedMeshBufferIteratorConst ItNormalBase(BaseMesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);

			EShapeBindingMethod ActualBindingMethod = BindingMethod == EShapeBindingMethod::ClipDeformNormalProject && !ItNormalBase.ptr() 
					? EShapeBindingMethod::ClipDeformClosestToSurface 
					: BindingMethod;

			switch ( ActualBindingMethod )
			{
			case EShapeBindingMethod::ClipDeformNormalProject:
			{
				check(ItNormalBase.ptr());
				for (int32 VertexIndex = 0; VertexIndex < MeshVertexCount; ++VertexIndex)
				{
					FVector3f VertexPosition = (ItPositionBase + VertexIndex).GetAsVec3f();
					FVector3f VertexNormal = (ItNormalBase + VertexIndex).GetAsVec3f();
					BindClipDeformPointNormalProject(ShapeMeshTree, VertexPosition, VertexNormal, BindData[VertexIndex]);
				}	

				break;
			}
			case EShapeBindingMethod::ClipDeformClosestToSurface:
			{
				for (int32 VertexIndex = 0; VertexIndex < MeshVertexCount; ++VertexIndex)
				{
					FVector3f VertexPosition = (ItPositionBase + VertexIndex).GetAsVec3f();
					BindClipDeformPointClosestToSurface(ShapeMeshTree, VertexPosition, BindData[VertexIndex]);
				}

				break;
			}
			case EShapeBindingMethod::ClipDeformClosestProject:
			{
				for (int32 VertexIndex = 0; VertexIndex < MeshVertexCount; ++VertexIndex)
				{
					FVector3f VertexPosition = (ItPositionBase + VertexIndex).GetAsVec3f();
					BindClipDeformPointClosestProject(ShapeMeshTree, VertexPosition, BindData[VertexIndex], BindValidityTolerance);
				}

				break;
			}

			default:
			{
				check(false);
			}
			};
		}

		return BindData;
	}

    inline void MeshBindShapeClipDeform(FMesh* Result, UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter>& ShapeTree, EShapeBindingMethod BindingMethod, bool& bOutSuccess)
    {
		MUTABLE_CPUPROFILER_SCOPE(MeshBindShapeClipDeform);
	
		bOutSuccess = true;

		if (!Result)
		{
			bOutSuccess = false;
			return;
		}

		TArray<FClipDeformVertexBindingData> VerticesBindData = BindVerticesClipDeform(Result, ShapeTree, BindingMethod);
			
		// Add the binding information to the mesh
		// \TODO: Check that there is no other binding data.
		// \TODO: Support specifying the binding data channel for multiple binding support.
		FMeshBufferSet& VB = Result->GetVertexBuffers();
		int NewBufferIndex = VB.GetBufferCount();
		VB.SetBufferCount(NewBufferIndex + 1);

		// \TODO: Multiple binding dataset support
		int32 BindingDataIndex = 0;
			
		FClipDeformVertexBindingDataBufferDescriptor BufDesc(BindingDataIndex);
		VB.SetBuffer(NewBufferIndex, sizeof(FClipDeformVertexBindingData), BufDesc.Channels, BufDesc.Semantics, BufDesc.SemanticIndices, BufDesc.Formats, BufDesc.Components, BufDesc.Offsets);
		FMemory::Memcpy(VB.GetBufferData(NewBufferIndex), VerticesBindData.GetData(), VerticesBindData.Num()*sizeof(FClipDeformVertexBindingData));
		
    }
}
