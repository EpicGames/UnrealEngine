// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MuR/OpMeshBind.h"
#include "MuR/OpMeshRemove.h"
#include "MuR/MutableTrace.h"

#include "MuR/MeshPrivate.h"

#include "Algo/Copy.h"



// TODO: Make the handling of rotations an option. It is more expensive on CPU and memory, and for some
// cases it is not required at all.

// TODO: Face stretch to scale the deformation per-vertex? 

// TODO: Support multiple binding influences per vertex, to have smoother deformations.

// TODO: Support multiple binding sets, to have different shapes deformations at once.

// TODO: Deformation mask, to select the intensisty of the deformation per-vertex.

// TODO: This is a reference implementation with ample roof for optimization.

namespace UE::Mutable::Private
{

	struct FClipDeformShapeMeshDescriptorApply
	{
		TArrayView<const FVector3f> Positions;
		TArrayView<const FVector3f> Normals;
		TArrayView<const float> Weights;
		TArrayView<const UE::Geometry::FIndex3i> Triangles;
	};

	// Method to actually deform a point
	inline bool GetDeform(
			const FVector3f& Position,
			const FVector3f& Normal,
			const FClipDeformShapeMeshDescriptorApply& ShapeMesh,
			const FClipDeformVertexBindingData& Binding,
			FVector3f& OutPosition, 
			FQuat4f& OutRotation,
			float& OutWeight)
	{
		if (Binding.Triangle < 0 || Binding.Triangle >= ShapeMesh.Triangles.Num())
		{
			return false;
		}

		FVector2f BaryCoords = FVector2f(Binding.S, Binding.T);

		// Clamp Barycoords so we are always inside the bound triangle if the binding is not good.
		// This is only needed for Closest Project, which is not very robust and sometimes the binding
		// are not valid.
		if ( FMath::IsNearlyZero(Binding.Weight) )
		{
			BaryCoords.X = FMath::Max( 0.0f, BaryCoords.X );
			BaryCoords.Y = FMath::Max( 0.0f, BaryCoords.Y );
			if (BaryCoords.X + BaryCoords.Y > 1.0f)
			{
				BaryCoords.X = BaryCoords.X / (BaryCoords.X + BaryCoords.Y);
				BaryCoords.Y = 1.0f - BaryCoords.X;
			}
		}
		
		const UE::Geometry::FIndex3i& Triangle = ShapeMesh.Triangles[Binding.Triangle];
	
		if (!ShapeMesh.Positions.IsValidIndex(Triangle.A) || !ShapeMesh.Positions.IsValidIndex(Triangle.B) || !ShapeMesh.Positions.IsValidIndex(Triangle.C) )
		{
			// It seems to happen with some test objects.
			ensure(false);
			return false;
		}

		const FVector3f ProjectedVertexPosition = 
				ShapeMesh.Positions[Triangle.A] * (1.0f - BaryCoords.X - BaryCoords.Y) +
				ShapeMesh.Positions[Triangle.B] * BaryCoords.X +
				ShapeMesh.Positions[Triangle.C] * BaryCoords.Y;

		const float ProjectedVertexWeight = 
				ShapeMesh.Weights[Triangle.A] * (1.0f - BaryCoords.X - BaryCoords.Y) + 
				ShapeMesh.Weights[Triangle.B] * BaryCoords.X + 
				ShapeMesh.Weights[Triangle.C] * BaryCoords.Y;
		
		// Morph from the projected position in shape to the original position based on the weight defined in shape.
		OutWeight = FMath::Clamp(ProjectedVertexWeight, 0.0f, 1.0f);
		OutPosition = FMath::Lerp(Position, ProjectedVertexPosition, OutWeight);

		// This method approximates the shape face rotation
		FVector3f InterpolatedNormal = 
				ShapeMesh.Normals[Triangle.A] * (1.0f - BaryCoords.X - BaryCoords.Y) + 
				ShapeMesh.Normals[Triangle.B] * BaryCoords.X + 
				ShapeMesh.Normals[Triangle.C] * BaryCoords.Y;

		FVector3f NewNormal = FMath::Lerp( Normal, InterpolatedNormal, ProjectedVertexWeight ).GetSafeNormal();
		
		// Use shape normal to interpolate.
		//FVector3f NewNormal = FMath::Lerp( Normal, -Binding->ShapeNormal, ProjectedVertexWeight ).GetSafeNormal();
		
		OutRotation = FQuat4f::FindBetween(Normal, NewNormal);
		
		return true;
	}

	inline void MeshClipDeform(FMesh* Result, const FMesh* BaseMesh, const FMesh* ShapeMesh, const float ClipWeightThreshold, bool bRemoveIfAllVerticesCulled, bool& bOutSuccess )
	{
		MUTABLE_CPUPROFILER_SCOPE(ClipDeform);
		bOutSuccess = true;

		if (!BaseMesh)
		{
			bOutSuccess = false;
			return;
		}

		constexpr EMeshCopyFlags MeshCopyFlags = ~EMeshCopyFlags::WithVertexBuffers;
		Result->CopyFrom(*BaseMesh, MeshCopyFlags);
	
		int BarycentricDataBuffer = 0;
		int BarycentricDataChannel = 0;
		const FMeshBufferSet& VB = BaseMesh->GetVertexBuffers();
		VB.FindChannel(EMeshBufferSemantic::BarycentricCoords, 0, &BarycentricDataBuffer, &BarycentricDataChannel);
		
		// Copy buffers skipping binding data.
		FMeshBufferSet& ResultBuffers = Result->GetVertexBuffers();
		ResultBuffers.ElementCount = VB.ElementCount;
		// Remove one element to the number of buffers if BarycentricDataBuffer found. 
		ResultBuffers.Buffers.SetNum( FMath::Max( 0, VB.Buffers.Num() - int32(BarycentricDataBuffer >= 0) ) );
		
		for (int32 B = 0, R = 0; B < VB.Buffers.Num(); ++B)
		{
			if (B != BarycentricDataBuffer)
			{
				ResultBuffers.Buffers[R++] = VB.Buffers[B];
			}
		}

		if (BarycentricDataBuffer < 0)
		{
			return; 
		}

		if (!ShapeMesh)
		{
			return; 
		}

		// \TODO: More checks
		check(BarycentricDataChannel==0);
		check(VB.GetElementSize(BarycentricDataBuffer) == (int)sizeof(FClipDeformVertexBindingData));
		TArrayView<const FClipDeformVertexBindingData> BindingDataView = TArrayView<const FClipDeformVertexBindingData>(
				(const FClipDeformVertexBindingData*)VB.GetBufferData(BarycentricDataBuffer), VB.GetElementCount() );

		//
		int32 ShapeVertexCount = ShapeMesh->GetVertexCount();
		int ShapeTriangleCount = ShapeMesh->GetFaceCount();
		if (!ShapeVertexCount || !ShapeTriangleCount)
		{
			return; 
		}

		TArray<FVector3f> ShapePositions;
		TArray<FVector3f> ShapeNormals;
		TArray<float> ShapeWeights;

		FClipDeformShapeMeshDescriptorApply ShapeMeshDescriptor;
		{
			MUTABLE_CPUPROFILER_SCOPE(GenerateVertexQueryData);

			UntypedMeshBufferIteratorConst ItPositionBegin(ShapeMesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
			UntypedMeshBufferIteratorConst ItNormalBegin(ShapeMesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);
			UntypedMeshBufferIteratorConst ItUvsBegin(ShapeMesh->GetVertexBuffers(), EMeshBufferSemantic::TexCoords);

			const bool bIsPositionBufferCompatible = 
					ItPositionBegin.GetFormat() == EMeshBufferFormat::Float32 && 
					ItPositionBegin.GetElementSize() == sizeof(FVector3f);
			if (bIsPositionBufferCompatible)
			{
				ShapeMeshDescriptor.Positions = TArrayView<FVector3f>((FVector3f*)ItPositionBegin.ptr(), ShapeVertexCount);
			}
			else
			{
				ShapePositions.SetNum(ShapeVertexCount);
				UntypedMeshBufferIteratorConst ItPosition =  ItPositionBegin;
				for (int32 ShapeVertexIndex = 0; ShapeVertexIndex < ShapeVertexCount; ++ShapeVertexIndex)
				{
					FVector3f Position = ItPosition.GetAsVec3f();
					ShapePositions[ShapeVertexIndex] = Position;
					++ItPosition;
				}

				ShapeMeshDescriptor.Positions = TArrayView<FVector3f>(ShapePositions.GetData(), ShapePositions.Num());
			}

			const bool bIsNormalBufferCompatible = 
					ItNormalBegin.GetFormat() == EMeshBufferFormat::Float32 && 
					ItNormalBegin.GetElementSize() == sizeof(FVector3f);
			if (bIsNormalBufferCompatible)
			{
				ShapeMeshDescriptor.Normals = TArrayView<const FVector3f>((const FVector3f*)ItNormalBegin.ptr(), ShapeVertexCount);
			}
			else
			{
				ShapeNormals.SetNum(ShapeVertexCount);
				UntypedMeshBufferIteratorConst ItNormal = ItNormalBegin;
				for (int32 ShapeVertexIndex = 0; ShapeVertexIndex < ShapeVertexCount; ++ShapeVertexIndex)
				{
					FVector3f Normal = ItNormal.GetAsVec3f();
					ShapeNormals[ShapeVertexIndex] = Normal;
					++ItNormal;
				}

				ShapeMeshDescriptor.Normals = TArrayView<const FVector3f>(ShapeNormals.GetData(), ShapeNormals.Num());
			}
	
			// Don't try to use the buffer directly for weights since we only need a component.
			ShapeWeights.SetNum(ShapeVertexCount);
			UntypedMeshBufferIteratorConst ItUvs = ItUvsBegin;
			for (int32 ShapeVertexIndex = 0; ShapeVertexIndex < ShapeVertexCount; ++ShapeVertexIndex)
			{
				FVector2f Uvs = ItUvs.GetAsVec2f();
				ShapeWeights[ShapeVertexIndex] = 1.0f - Uvs.Y;
				++ItUvs;
			}

			ShapeMeshDescriptor.Weights = TArrayView<const float>(ShapeWeights.GetData(), ShapeWeights.Num());
		}
	
		TArray<UE::Geometry::FIndex3i> ShapeTriangles;
		{
			MUTABLE_CPUPROFILER_SCOPE(GenerateTriangleQueryData);

			const UntypedMeshBufferIteratorConst ItIndicesBegin(ShapeMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);

			check(ShapeMesh->GetIndexCount() % 3 == 0);
	
			const bool bIsIndexBufferCompatible = 
					(ItIndicesBegin.GetFormat() == EMeshBufferFormat::Int32 || ItIndicesBegin.GetFormat() == EMeshBufferFormat::UInt32) &&
					ItIndicesBegin.GetElementSize() == int32(sizeof(int32));

			if (bIsIndexBufferCompatible)
			{
				ShapeMeshDescriptor.Triangles = TArrayView<UE::Geometry::FIndex3i>( 
						(UE::Geometry::FIndex3i*)ItIndicesBegin.ptr(), ShapeMesh->GetIndexCount() / 3 );
			}
			else
			{
				ShapeTriangles.SetNum(ShapeTriangleCount);
				UntypedMeshBufferIteratorConst ItIndices = ItIndicesBegin;
				for (int32 TriangleIndex = 0; TriangleIndex < ShapeTriangleCount; ++TriangleIndex)
				{
					UE::Geometry::FIndex3i Triangle;
					Triangle.A = int(ItIndices.GetAsUINT32());
					++ItIndices;
					Triangle.B = int(ItIndices.GetAsUINT32());
					++ItIndices;
					Triangle.C = int(ItIndices.GetAsUINT32());
					++ItIndices;

					ShapeTriangles[TriangleIndex] = Triangle;
				}

				ShapeMeshDescriptor.Triangles = TArrayView<const UE::Geometry::FIndex3i>(ShapeTriangles.GetData(), ShapeTriangles.Num());
			}
		}
		
		// Update the result mesh positions
		int32 MeshVertexCount = BaseMesh->GetVertexCount();

		TBitArray<> VerticesToCull;
		VerticesToCull.SetNum(MeshVertexCount, false);
		{
			MUTABLE_CPUPROFILER_SCOPE(UpdateClipDeformVertices);

			UntypedMeshBufferIterator ItPosition(Result->GetVertexBuffers(), EMeshBufferSemantic::Position);
			UntypedMeshBufferIterator ItNormal(Result->GetVertexBuffers(), EMeshBufferSemantic::Normal);
			UntypedMeshBufferIterator ItTangent(Result->GetVertexBuffers(), EMeshBufferSemantic::Tangent);

			for (int32 MeshVertexIndex = 0; MeshVertexIndex < MeshVertexCount; ++MeshVertexIndex)
			{
				FVector3f NewPosition;
				FQuat4f TangentSpaceCorrection;
				float ClipWeight = 0.0f;
				const bool bModified = GetDeform( 
						ItPosition.GetAsVec3f(), ItNormal.GetAsVec3f(), 
						ShapeMeshDescriptor, 
						BindingDataView[MeshVertexIndex],
						NewPosition, TangentSpaceCorrection, ClipWeight);

				if (bModified)
				{
					ItPosition.SetFromVec3f(NewPosition);

					if (ItNormal.ptr())
					{
						FVector3f OldNormal = ItNormal.GetAsVec3f();
						FVector3f NewNormal = TangentSpaceCorrection.RotateVector(OldNormal);
						ItNormal.SetFromVec3f(NewNormal);
					}

					if (ItTangent.ptr())
					{
						FVector3f OldTangent = ItTangent.GetAsVec3f();
						FVector3f NewTangent = TangentSpaceCorrection.RotateVector(OldTangent);
						ItTangent.SetFromVec3f(NewTangent);
					}
					
					VerticesToCull[MeshVertexIndex] = ClipWeight >= (1.0f - SMALL_NUMBER);	
				}

				++ItPosition;

				if (ItNormal.ptr())
				{
					++ItNormal;
				}

				if (ItTangent.ptr())
				{
					++ItTangent;
				}
			}
		}

		MeshRemoveVerticesWithCullSet(Result, VerticesToCull, bRemoveIfAllVerticesCulled);
	}
}
