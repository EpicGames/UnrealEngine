// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshPrivate.h"
#include "MuR/ConvertData.h"
#include "MuR/Parameters.h"
#include "MuR/Platform.h"
#include "MuR/OpMeshRemove.h"

#include "Math/IntVector.h"

namespace UE::Mutable::Private
{
	inline bool PointInBoundingBox(const FVector3f& Point, const FShape& SelectionShape)
	{
		FVector3f V = Point - SelectionShape.position;

		for (int32 i = 0; i < 3; ++i)
		{
			if (FMath::Abs(V[i]) > SelectionShape.size[i])
			{
				return false;
			}
		}

		return true;
	}

	inline bool VertexIsInMaxRadius(const FVector3f& Position, const FVector3f& Origin, float VertexSelectionBoneMaxRadius)
	{
		if (VertexSelectionBoneMaxRadius < 0.f)
		{
			return true;
		}

		FVector3f RadiusVec = Position - Origin;
		float Radius2 = FVector3f::DotProduct(RadiusVec, RadiusVec);

		return Radius2 < VertexSelectionBoneMaxRadius*VertexSelectionBoneMaxRadius;
	}

	struct FVertexBoneInfo
	{
		TArray<int32, TFixedAllocator<16>> BoneIndices;
		TArray<int32, TFixedAllocator<16>> BoneWeights;
	};

	inline bool VertexIsAffectedByBone(int32 VertexIdx, const TBitArray<>& BoneIsAffected, const TArray<FVertexBoneInfo>& VertexInfo)
	{
		if (VertexIdx >= VertexInfo.Num())
		{
			return false;
		}

		check(VertexInfo[VertexIdx].BoneIndices.Num() == VertexInfo[VertexIdx].BoneWeights.Num());

        for (int32 i = 0; i < VertexInfo[VertexIdx].BoneIndices.Num(); ++i)
		{			
            check(VertexInfo[VertexIdx].BoneIndices[i] <= BoneIsAffected.Num());

			if (BoneIsAffected[VertexInfo[VertexIdx].BoneIndices[i]] && VertexInfo[VertexIdx].BoneWeights[i] > 0)
			{
				return true;
			}
		}

		return false;
	}

	//---------------------------------------------------------------------------------------------
	//! Reference version
	//---------------------------------------------------------------------------------------------
	inline void MeshClipMorphPlane(FMesh* Result, const FMesh* pBase, const FVector3f& Origin, const FVector3f& Normal, float Dist, float Factor, float Radius,
		float Radius2, float Angle, const FShape& SelectionShape, bool bRemoveIfAllVerticesCulled, bool& bOutSuccess, 
		FName BoneName = NAME_None, float VertexSelectionBoneMaxRadius = -1.f)
	{
		bOutSuccess = true;
		//float Radius = 8.f;
		//float Radius2 = 4.f;
		//float Factor = 1.f;

		// Generate vector perpendicular to normal for ellipse rotation reference base
		FVector3f AuxBase(0.f, 1.f, 0.f);

		if (FMath::Abs(FVector3f::DotProduct(Normal, AuxBase)) > 0.95f)
		{
			AuxBase = FVector3f(0.f, 0.f, 1.f);
		}
		
		FVector3f OriginRadiusVector = FVector3f::CrossProduct(Normal, AuxBase);		// Perpendicular vector to the plane normal and aux base
		check(FMath::Abs(FVector3f::DotProduct(Normal, OriginRadiusVector)) < 0.05f);

        uint32 VertexCount = pBase->GetVertexBuffers().GetElementCount();

		if (!VertexCount)
		{
			bOutSuccess = false;
			return;
		}

		TArray<FVertexBoneInfo> VertexInfo;

        TManagedPtr<const FSkeleton> BaseSkeleton = pBase->GetSkeleton();

		TBitArray<> AffectedBoneMapIndices;
		const int32 BaseBoneIndex = BaseSkeleton && BoneName != NAME_None ? BaseSkeleton->BoneNames.Find(BoneName) : INDEX_NONE;
        if (BaseBoneIndex != INDEX_NONE)
		{
			const int32 NumBonesBoneMap = pBase->BoneMap.Num();
			const TArray<FBoneIdOrIndex>& BoneMap = pBase->BoneMap;
			AffectedBoneMapIndices.SetNum(NumBonesBoneMap, false);

			const int32 NumBones = BaseSkeleton->GetNumBones();
			TBitArray<> AffectedSkeletonBones;
			AffectedSkeletonBones.SetNum(NumBones, false);

			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				const int32 ParentBoneIndex = BaseSkeleton->BoneParents[BoneIndex];
				check(ParentBoneIndex < BoneIndex);

				const bool bIsBoneAffected = (AffectedSkeletonBones.IsValidIndex(ParentBoneIndex) && AffectedSkeletonBones[ParentBoneIndex])
					|| BoneIndex == BaseBoneIndex;

				AffectedSkeletonBones[BoneIndex] = bIsBoneAffected;
			}

			for (int32 Index = 0; Index < NumBonesBoneMap; ++Index)
			{
				AffectedBoneMapIndices[Index] = AffectedSkeletonBones[BoneMap[Index].Index];
			}

			// Look for affected vertex indices
			VertexInfo.SetNum(VertexCount);
			//int32 FirstCount = pBase->GetVertexBuffers().GetElementCount();

			for (int32 BufferIndex = 0; BufferIndex < pBase->GetVertexBuffers().Buffers.Num(); ++BufferIndex)
			{
				const FMeshBuffer& Buffer = pBase->GetVertexBuffers().Buffers[BufferIndex];

				int32 ElemSize = pBase->GetVertexBuffers().GetElementSize(BufferIndex);
				//int32 FirstSize = FirstCount * ElemSize;

				for (int32 ChannelIndex = 0; ChannelIndex < pBase->GetVertexBuffers().GetBufferChannelCount(BufferIndex); ++ChannelIndex)
				{
					// Get info about the destination channel
					EMeshBufferSemantic Semantic = EMeshBufferSemantic::None;
					int32 SemanticIndex = 0;
					EMeshBufferFormat Format = EMeshBufferFormat::None;
					int32 Components = 0;
					int32 Offset = 0;
					pBase->GetVertexBuffers().GetChannel(BufferIndex, ChannelIndex, &Semantic, &SemanticIndex, &Format, &Components, &Offset);

					checkf(Components <= 16, TEXT("FVertexBoneInfo allocation is fixed to 16 elements."));

					//int32 ResultOffset = FirstSize + Offset;

					if (Semantic == EMeshBufferSemantic::BoneIndices)
					{
						int32 NumVertices = pBase->GetVertexCount();
						{
							switch (Format)
							{
							case EMeshBufferFormat::Int8:
							case EMeshBufferFormat::UInt8:
							{
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const uint8* pD = &Buffer.Data[i*ElemSize + Offset];
									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneIndices.Add(pD[j]);
									}
								}
								break;
							}

							case EMeshBufferFormat::Int16:
							case EMeshBufferFormat::UInt16:
							{
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const uint16* pD = reinterpret_cast<const uint16*>(&Buffer.Data[i*ElemSize + Offset]);

									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneIndices.Add(pD[j]);
									}
								}
								break;
							}

							case EMeshBufferFormat::Int32:
							case EMeshBufferFormat::UInt32:
							{
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const uint32* pD = reinterpret_cast<const uint32*>(&Buffer.Data[i*ElemSize + Offset]);

									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneIndices.Add(pD[j]);
									}
								}
								break;
							}

							default:
								checkf(false, TEXT("Bone index format not supported.") );
							}
						}
					}
					else if (Semantic == EMeshBufferSemantic::BoneWeights)
					{
						const int32 NumVertices = pBase->GetVertexCount();
						{
							switch (Format)
							{
							case EMeshBufferFormat::Int8:
							case EMeshBufferFormat::UInt8:
							case EMeshBufferFormat::NUInt8:
							{
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const uint8* pD = &Buffer.Data[i*ElemSize + Offset];

									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneWeights.Add(pD[j]);
									}
								}
								break;
							}

							case EMeshBufferFormat::Int16:
							case EMeshBufferFormat::UInt16:
							case EMeshBufferFormat::NUInt16:
							{
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const uint16* pD = reinterpret_cast<const uint16*>(&Buffer.Data[i*ElemSize + Offset]);

									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneWeights.Add(pD[j]);
									}
								}
								break;
							}

							case EMeshBufferFormat::Int32:
							case EMeshBufferFormat::UInt32:
							case EMeshBufferFormat::NUInt32:
							{
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const uint32* pD = reinterpret_cast<const uint32*>(&Buffer.Data[i*ElemSize + Offset]);

									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneWeights.Add(pD[j]);
									}
								}
								break;
							}

                            case EMeshBufferFormat::Float32:
                            {
								for (int32 i = 0; i < NumVertices; ++i)
								{
									const float* pD = reinterpret_cast<const float*>(&Buffer.Data[i*ElemSize + Offset]);

									for (int32 j = 0; j < Components; ++j)
									{
										VertexInfo[i].BoneWeights.Add(pD[j] > 0.0f);
									}
								}
                                break;
                            }

							default:
								checkf(false, TEXT("Bone weight format not supported.") );
							}
						}
					}
				}
			}
		}

		Result->CopyFrom(*pBase);

		const int32 NumVertices = Result->GetVertexCount();
        TBitArray<> VerticesToCull;
		VerticesToCull.SetNum(NumVertices, false);
		
		// Positions can  be assumed to be in a FVector3f struct but changing the iterator to a untyped one should 
		// work as well.
		const MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> PositionIterBegin(Result->GetVertexBuffers(), EMeshBufferSemantic::Position);
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{	
			FVector3f Position = (PositionIterBegin + VertexIndex).GetAsVec3f();

			const bool bIsVertexAffectedBone = 
					BaseBoneIndex != INDEX_NONE &&
					VertexIsInMaxRadius(Position, Origin, VertexSelectionBoneMaxRadius) &&
					VertexIsAffectedByBone(VertexIndex, AffectedBoneMapIndices, VertexInfo);
			
			const bool bIsVertexAffectedNoShape = 
					BaseBoneIndex == INDEX_NONE && 
					SelectionShape.type == (uint8)FShape::Type::None;

			const bool bIsVertexAffectedBoundingBox = 
					(SelectionShape.type == (uint8)FShape::Type::AABox && 
					PointInBoundingBox(Position, SelectionShape));

			bool bRemovedVertex = false;
			if (bIsVertexAffectedBone || bIsVertexAffectedNoShape || bIsVertexAffectedBoundingBox)
			{
				FVector3f MorphPlaneCenter = Origin; // Morph plane pos relative to root of the selected bone
				FVector3f ClipPlaneCenter = Origin + Normal*Dist; // Clipping plane pos
				FVector3f AuxMorph = Position - MorphPlaneCenter; // Morph plane --> current vertex
				FVector3f AuxClip = Position - ClipPlaneCenter;   // Clipping plane --> current vertex

				float DotMorph = FVector3f::DotProduct(AuxMorph, Normal); // Angle (morph plane to vertex and normal)
				float DotCut = FVector3f::DotProduct(AuxClip, Normal);    // Angle (clipping plane to vertex and normal)

				// Check if clipping or morph should be computed for v vertex
				if (DotMorph >= 0.f || DotCut >= 0.f)
				{
					FVector3f CurrentCenter = MorphPlaneCenter + Normal*DotMorph; // Projected point from the mroph plane (the closer the dot value of normal and vertex the further it goes)
					FVector3f RadiusVector = Position - CurrentCenter;	
					float RadiusVectorLen = RadiusVector.Length();
					FVector3f RadiusVectorUnit = RadiusVectorLen != 0.f ? RadiusVector / RadiusVectorLen : FVector3f(0.f, 0.f, 0.f); // Unitary vector that goes from the point to the vertex

					float AngleFromOrigin = FMath::Acos(FVector3f::DotProduct(RadiusVectorUnit, OriginRadiusVector));

					// Cross product between the perpendicular vector from radius vector and origin radius and the normal vector
					if (FVector3f::DotProduct(FVector3f::CrossProduct(RadiusVectorUnit, OriginRadiusVector), Normal) < 0.0f)
					{
						AngleFromOrigin = -AngleFromOrigin;
					}

					AngleFromOrigin += Angle * PI / 180.f;

					float Term1 = Radius2 * FMath::Cos(AngleFromOrigin);
					float Term2 = Radius * FMath::Sin(AngleFromOrigin);
					float EllipseRadiusAtAngle = Radius * Radius2 * FMath::InvSqrt(Term1*Term1 + Term2*Term2);

					FVector3f VertexProjEllipse = CurrentCenter + RadiusVectorUnit * EllipseRadiusAtAngle;
					// FVector3f VertexProjEllipse = CurrentCenter + RadiusVectorUnit * Radius;

					float MorphAlpha = Dist != 0.f && DotMorph <= Dist ? FMath::Clamp(FMath::Pow(DotMorph/Dist, Factor), 0.f, 1.f) : 1.f;

					Position = Position * (1.f - MorphAlpha) + VertexProjEllipse*MorphAlpha;

					// check if the vertex should be clipped
					if (DotCut >= 0.f)		
					{
						FVector3f VertDispl = Normal * -DotCut;
						Position = Position + VertDispl;
						VerticesToCull[VertexIndex] = true;
					}
				}
			}

			(PositionIterBegin + VertexIndex).SetFromVec3f(Position);
		}

        MeshRemoveVerticesWithCullSet(Result, VerticesToCull, bRemoveIfAllVerticesCulled);
	}
}
