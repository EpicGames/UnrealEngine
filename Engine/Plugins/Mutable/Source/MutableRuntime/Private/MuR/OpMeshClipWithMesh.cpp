// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshClipWithMesh.h"

#include "MuR/MeshBufferSet.h"
#include "MuR/MeshPrivate.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Layout.h"
#include "MuR/ConvertData.h"
#include "MuR/Platform.h"
#include "MuR/MutableTrace.h"
#include "MuR/MutableRuntimeModule.h"
#include "Math/Ray.h"
#include "TriangleTypes.h"
#include "BoxTypes.h"
#include "Intersection/IntrRay3Triangle3.h"
#include "MathUtil.h"
#include "IntVectorTypes.h"
#include "OpMeshRemove.h"
#include "Math/UnrealMathUtility.h"
#include "Spatial/PointHashGrid3.h"
#include "Spatial/MeshAABBTree3.h"
#include "MuR/ParallelExecutionUtils.h"
#include "MuR/GeometryUtils.h"

namespace UE::Mutable::Private { namespace 
{
    //---------------------------------------------------------------------------------------------
    //! Create a map from vertices into vertices, collapsing vertices that have the same position, 
	//! This version uses UE Containers to return.	
    //---------------------------------------------------------------------------------------------
	void MeshCreateCollapsedVertexMap(const FMesh* pMesh, TArray<int32>& OutCollapsedVertices, TArray<FVector3f>& OutVertices)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshCreateCollapsedVertexMap);

		const int32 NumVertices = pMesh->GetVertexCount();

		// Used to speed up vertex comparison
		UE::Geometry::TPointHashGrid3f<int32> VertHash(0.01f, INDEX_NONE);
		VertHash.Reserve(NumVertices);

		// Info to collect. Vertices and collapsed vertices
		OutVertices.SetNumUninitialized(NumVertices);
		OutCollapsedVertices.Init(INDEX_NONE, NumVertices);

		// Get Vertices
		UE::Mutable::Private::UntypedMeshBufferIteratorConst ItPosition = UE::Mutable::Private::UntypedMeshBufferIteratorConst(pMesh->GetVertexBuffers(), UE::Mutable::Private::EMeshBufferSemantic::Position);
		FVector3f* VertexData = OutVertices.GetData();

		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			*VertexData = ItPosition.GetAsVec3f();
			VertHash.InsertPointUnsafe(VertexIndex, *VertexData);

			++ItPosition;
			++VertexData;
		}

		// Find unique vertices
		TArray<int32> NearbyVertices;
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			if (OutCollapsedVertices[VertexIndex] != INDEX_NONE)
			{
				continue;
			}

			const FVector3f& Vertex = OutVertices[VertexIndex];

			NearbyVertices.Reset();
			VertHash.FindPointsInBall(Vertex, TMathUtilConstants<float>::ZeroTolerance,
				[&Vertex, &OutVertices](const int32& Other) -> float {return FVector3f::DistSquared(OutVertices[Other], Vertex); },
				NearbyVertices);

			for (int32 NearbyVertexIndex : NearbyVertices)
			{
				OutCollapsedVertices[NearbyVertexIndex] = VertexIndex;
			}
		}
	}


	/** Return true if the mesh is closed. Usually used to validate clipping meshes. */
	bool IsMeshClosed( const FMesh* Mesh, const TArray<int32>& CollapsedVertexMap)
    {
        MUTABLE_CPUPROFILER_SCOPE(IsMeshClosed);

		if (!Mesh)
		{
			return true;
		}

        int32 FaceCount = Mesh->GetFaceCount();

        // Acumulate edges
		using FEdge = TPair<int32, int32>;
        TMap< FEdge, int32 > FaceCountPerEdge;

        UntypedMeshBufferIteratorConst ItClipMesh(Mesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
        for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
        {
            int32 Face[3];
			Face[0] = CollapsedVertexMap[ItClipMesh.GetAsUINT32()]; ++ItClipMesh;
			Face[1] = CollapsedVertexMap[ItClipMesh.GetAsUINT32()]; ++ItClipMesh;
			Face[2] = CollapsedVertexMap[ItClipMesh.GetAsUINT32()]; ++ItClipMesh;

            for (int32 EdgeIndex=0; EdgeIndex <3; ++EdgeIndex)
            {
                int32 v0 = Face[EdgeIndex];
                int32 v1 = Face[(EdgeIndex+1)%3];

                if(v0==v1)
                {
                    // Degenerated mesh
                    return false;
                }

				FEdge Edge;
				Edge.Key = FMath::Min( v0, v1 );
				Edge.Value = FMath::Max( v0, v1 );

				int32& Count = FaceCountPerEdge.FindOrAdd(Edge);
				++Count;
            }
        }

        // See if every edge has 2 faces
        for( const TPair<FEdge, int32>& Entry: FaceCountPerEdge)
        {
            if (Entry.Value!=2)
            {
                return false;
            }
        }

        return true;
    }


    //---------------------------------------------------------------------------------------------
    //! Remove all unused vertices from a mesh, and fix its index buffers.
    //---------------------------------------------------------------------------------------------
    void MeshRemoveUnusedVertices( FMesh* pMesh )
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshRemoveUnusedVertices);

        // Mark used vertices
		TArray<uint8> used;
		used.SetNumZeroed(pMesh->GetVertexCount());
        UntypedMeshBufferIteratorConst iti(pMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
        int IndexCount = pMesh->GetIndexCount();
        for (int i = 0; i < IndexCount; ++i)
        {
            uint32 Index = iti.GetAsUINT32();
            ++iti;
            used[Index] = true;
        }

        // Build vertex map
		TArray<int32> oldToNewVertex;
		oldToNewVertex.SetNumUninitialized(pMesh->GetVertexCount());
        int32 totalNewVertices = 0;
        for (int32 v = 0; v<pMesh->GetVertexCount(); ++v)
        {
            if (used[v])
            {
                oldToNewVertex[v] = totalNewVertices;
                ++totalNewVertices;
            }
            else
            {
                oldToNewVertex[v] = -1;
            }
        }

        // Remove unused vertices and build index map
        for (int b = 0; b<pMesh->GetVertexBuffers().GetBufferCount(); ++b)
        {
            int elemSize = pMesh->GetVertexBuffers().GetElementSize(b);
            const uint8* pSourceData = pMesh->GetVertexBuffers().GetBufferData(b);
            uint8* pData = pMesh->GetVertexBuffers().GetBufferData(b);
            for (int v = 0; v<pMesh->GetVertexCount(); ++v)
            {
                if (oldToNewVertex[v]!=-1)
                {
                    uint8* pElemData = pData + elemSize*oldToNewVertex[v];
                    const uint8* pElemSourceData = pSourceData + elemSize*v;
                    // Avoid warning for overlapping memcpy in valgrind
                    if (pElemData != pElemSourceData)
                    {
                        memcpy(pElemData, pElemSourceData, elemSize);
                    }
                }
            }
        }
        pMesh->GetVertexBuffers().SetElementCount(totalNewVertices);

        // Update indices
        UntypedMeshBufferIteratorConst ito(pMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
        if (ito.GetFormat() == EMeshBufferFormat::UInt32)
        {
            for (int i = 0; i < IndexCount; ++i)
            {
                uint32 Index = *(uint32*)ito.ptr();
                int32 NewIndex = oldToNewVertex[Index];
                check(NewIndex >= 0);
                *(uint32*)ito.ptr() = (uint32)NewIndex;
                ++ito;
            }
        }
        else if (ito.GetFormat() == EMeshBufferFormat::UInt16)
        {
            for (int i = 0; i < IndexCount; ++i)
            {
                uint16 Index = *(uint16*)ito.ptr();
                int32 NewIndex = oldToNewVertex[Index];
                check(NewIndex >= 0);
                *(uint16*)ito.ptr() = (uint16)NewIndex;
                ++ito;
            }
        }
        else
        {
            checkf(false, TEXT("Format not implemented.") );
        }


        // \todo: face buffers?
    }


    inline int32 GetNumIntersections( const FRay3f& Ray,
                                    const TArray<FVector3f>& Vertices,
                                    const TArray<uint32>& Faces,
                                    const TArray<int32>& CollapsedVertexMap,
                                    TArray<uint8>& InOutVertexAlreadyIntersected,
                                    TSet<uint64>& InOutEdgeAlreadyIntersected,
									const float DynamicEpsilon )
    {
		using namespace UE::Geometry;    

        MUTABLE_CPUPROFILER_SCOPE(GetNumIntersections);
    	
        int32 NumIntersections = 0;

		FMemory::Memzero( InOutVertexAlreadyIntersected.GetData(), InOutVertexAlreadyIntersected.Num() );
		InOutEdgeAlreadyIntersected.Empty();

		auto GetCollapsedVertex = [&CollapsedVertexMap, &Vertices]( const uint32 V ) -> const FVector3f&
		{
			return Vertices[ CollapsedVertexMap[V] ];
		};

        int32 FaceCount = Faces.Num() / 3;

		UE::Geometry::FIntrRay3Triangle3f Intersector(Ray, UE::Geometry::FTriangle3f());

        // Check vertex against all ClipMorph faces
        for (int32 Face = 0; Face < FaceCount; ++Face)
        {
            const uint32 VertexIndexs[3] = { Faces[3 * Face], Faces[3 * Face + 1], Faces[3 * Face + 2] };

            const FVector3f& V0 = GetCollapsedVertex(VertexIndexs[0]);
            const FVector3f& V1 = GetCollapsedVertex(VertexIndexs[1]);
            const FVector3f& V2 = GetCollapsedVertex(VertexIndexs[2]);

			Intersector.Triangle = UE::Geometry::FTriangle3f( V0, V1, V2 );

			if ( Intersector.Find() )
            {
				// Find if close to edge using barycentric coordinates form intersector.
	
				// Is the Dynamic Epsilon needed?. Intersector bary coords are in the range [0.0, 1.0], 
				// furthermore, IntrRay3Triangle3f::TriangleBaryCoords are double pressison, not sure 
				// is intentional or is a bug ( is double even when the FReal type is float ) 
				const bool IntersectsEdge01 = FMath::IsNearlyZero( Intersector.TriangleBaryCoords.Z, DynamicEpsilon );
				const bool IntersectsEdge02 = FMath::IsNearlyZero( Intersector.TriangleBaryCoords.Y, DynamicEpsilon );
				const bool IntersectsEdge12 = FMath::IsNearlyZero( Intersector.TriangleBaryCoords.X, DynamicEpsilon );

				int32 IntersectedTriangleVertexId = -1;
				IntersectedTriangleVertexId =  (IntersectsEdge01 & IntersectsEdge02) ? 0 : IntersectedTriangleVertexId;
				IntersectedTriangleVertexId =  (IntersectsEdge01 & IntersectsEdge12) ? 1 : IntersectedTriangleVertexId;
				IntersectedTriangleVertexId =  (IntersectsEdge02 & IntersectsEdge12) ? 2 : IntersectedTriangleVertexId;

				bool bIsAlreadyIntersected = false;

                if ( IntersectedTriangleVertexId >= 0 )
                {
                    const int32 CollapsedVertIndex = CollapsedVertexMap[VertexIndexs[IntersectedTriangleVertexId]];
                    bIsAlreadyIntersected = InOutVertexAlreadyIntersected[CollapsedVertIndex] == 0;

                    InOutVertexAlreadyIntersected[CollapsedVertIndex] = true;
                }
				else if ( IntersectsEdge01 | IntersectsEdge02 | IntersectsEdge12 )
                {
					int32 EdgeV0 = (IntersectsEdge01 | IntersectsEdge02) ? 0 : 1;
					int32 EdgeV1 = IntersectsEdge01 ? 1 : 2;				

                    const int32 CollapsedEdgeVertIndex0 = CollapsedVertexMap[VertexIndexs[EdgeV0]];
                    const int32 CollapsedEdgeVertIndex1 = CollapsedVertexMap[VertexIndexs[EdgeV1]];

                    const uint64 EdgeKey = ( ( (uint64)FMath::Max( CollapsedEdgeVertIndex0, CollapsedEdgeVertIndex1 ) ) << 32 ) | 
							                   (uint64)FMath::Min( CollapsedEdgeVertIndex0, CollapsedEdgeVertIndex1 );
	
					InOutEdgeAlreadyIntersected.FindOrAdd( EdgeKey, &bIsAlreadyIntersected );
                }

                NumIntersections += (int32)(!bIsAlreadyIntersected);
            }
        }

        return NumIntersections;
    }

	void MeshUVMaskClassifyVertices(TBitArray<>& VertexClipped, const FMesh* Base, const FImage* Mask, const uint8 LayoutIndex)
	{
		using namespace UE::Geometry;

		MUTABLE_CPUPROFILER_SCOPE(MeshUVMaskClassifyVertices);

		uint32 VertexCount = Base->GetVertexCount();

		// Stores whether each vertex in the original mesh in the clip mesh volume
		VertexClipped.SetNum(VertexCount, false);

		// Now go through all vertices in the mesh and record whether they are inside or outside of the ClipMesh
		const FMeshBufferSet& MBSPriv = Base->GetVertexBuffers();
		for (int32 b = 0; b < MBSPriv.Buffers.Num(); ++b)
		{
			const TArray<UE::Mutable::Private::FMeshBufferChannel>& Channels = MBSPriv.Buffers[b].Channels;

			for (int32 c = 0; c < Channels.Num(); ++c)
			{
				EMeshBufferSemantic Sem = Channels[c].Semantic;
				if (Sem != EMeshBufferSemantic::TexCoords)
				{
					continue;
				}

				int32 SemIndex = Channels[c].SemanticIndex;
				if (SemIndex != LayoutIndex)
				{
					continue;
				}

				UntypedMeshBufferIteratorConst It(Base->GetVertexBuffers(), Sem, SemIndex);
				for (uint32 V = 0; V < VertexCount; ++V)
				{
					// \TODO: This could be optimized.
					FVector2f UV = It.GetAsVec2f();

					// \TODO: This could also be optimized
					FVector4f Color = Mask->Sample(UV);

					VertexClipped[V] = Color[0] >= 0.5f;

					++It;
				}

				break;
			}
		}
	}


	void MeshLayoutMaskClassifyVertices(TBitArray<>& VertexClipped, const FMesh* Base, const FLayout* Mask, const uint8 LayoutIndex)
	{
		using namespace UE::Geometry;

		MUTABLE_CPUPROFILER_SCOPE(MeshLayoutMaskClassifyVertices);

		uint32 VertexCount = Base->GetVertexCount();

		// Stores whether each vertex in the original mesh in the clip mesh volume
		VertexClipped.SetNum(VertexCount, false);

		// Now go through all vertices in the mesh and record whether they are inside or outside of the ClipMesh
		const FMeshBufferSet& MBSPriv = Base->GetVertexBuffers();
		for (int32 b = 0; b < MBSPriv.Buffers.Num(); ++b)
		{
			const TArray<UE::Mutable::Private::FMeshBufferChannel>& Channels = MBSPriv.Buffers[b].Channels;

			for (int32 c = 0; c < Channels.Num(); ++c)
			{
				EMeshBufferSemantic Sem = Channels[c].Semantic;
				if (Sem != EMeshBufferSemantic::TexCoords)
				{
					continue;
				}

				int32 SemIndex = Channels[c].SemanticIndex;
				if (SemIndex != LayoutIndex)
				{
					continue;
				}

				UntypedMeshBufferIteratorConst It(Base->GetVertexBuffers(), Sem, SemIndex);
				for (uint32 V = 0; V < VertexCount; ++V)
				{
					// \TODO: This could be optimized.
					FVector2f UV = It.GetAsVec2f();

					FVector2f Cell = UV * FVector2f(Mask->Size.X, Mask->Size.Y);

					// \TODO: This could also be optimized
					for (const FLayoutBlock& Block : Mask->Blocks)
					{
						if (
							float(Block.Min.X) <= Cell.X && float(Block.Min.Y) <= Cell.Y
							&&
							float(Block.Min.X + Block.Size.X) >= Cell.X && float(Block.Min.Y + Block.Size.Y) >= Cell.Y
							)
						{
							VertexClipped[V] = true; 
							break;
						}
					}

					++It;
				}

				break;
			}
		}
	}


	/** Make a mask with the indices of the vertices with 0 in the IncludedVertices array. */
	void CreateMask(FMesh* Result, const FMesh* Base, const TArray<uint8>& IncludedVertices)
	{
		int32 MaskVertexCount = 0;
		for (uint8 b : IncludedVertices)
		{
			if (!b)
			{
				++MaskVertexCount;
			}
		}

		Result->GetVertexBuffers().SetElementCount(MaskVertexCount);
		Result->GetVertexBuffers().SetBufferCount(1);
		Result->MeshIDPrefix = Base->MeshIDPrefix;

		EMeshBufferSemantic Semantic = EMeshBufferSemantic::VertexIndex;
		int32 SemanticIndex = 0;
		int32 Components = 1;
		int32 Offsets = 0;

		bool bMakeRelativeIds = !Base->AreVertexIdsExplicit();
		if (bMakeRelativeIds)
		{
			EMeshBufferFormat Format = EMeshBufferFormat::UInt32;
			Result->GetVertexBuffers().SetBuffer( 0, sizeof(uint32), 1, &Semantic, &SemanticIndex, &Format, &Components, &Offsets );

			// Fill the buffer
			uint32* Data = reinterpret_cast<uint32*>(Result->GetVertexBuffers().GetBufferData(0));
			MeshVertexIdIteratorConst ItBase(Base);
			for (int32 v = 0; v < IncludedVertices.Num(); ++v)
			{
				if (!IncludedVertices[v])
				{
					uint64 ID = ItBase.Get();
					*Data = uint32(ID & 0xffffffff);
					++Data;
				}
				++ItBase;
			}
		}
		else
		{
			EMeshBufferFormat Format = EMeshBufferFormat::UInt64;
			Result->GetVertexBuffers().SetBuffer( 0, sizeof(uint64), 1, &Semantic, &SemanticIndex, &Format, &Components, &Offsets );

			// Fill the buffer
			uint64* Data = reinterpret_cast<uint64*>( Result->GetVertexBuffers().GetBufferData(0) );
			MeshVertexIdIteratorConst ItBase(Base);
			for (int32 v = 0; v < IncludedVertices.Num(); ++v)
			{
				if (!IncludedVertices[v])
				{
					*Data = ItBase.Get();
					++Data;
				}
				++ItBase;
			}
		}
	}
}} // namespace UE::Mutable::Private::anonymous


namespace UE::Mutable::Private
{
	bool IsMeshClosed(const FMesh* Mesh)
	{
		if (!Mesh)
		{
			return true;
		}

		const int32 VCount = Mesh->GetVertexBuffers().GetElementCount();

		TArray<FVector3f> Vertices;
		Vertices.SetNumUninitialized(VCount);

		TArray<int32> CollapsedVertexMap;
		CollapsedVertexMap.AddUninitialized(VCount);

		MeshCreateCollapsedVertexMap(Mesh, CollapsedVertexMap, Vertices);

		return IsMeshClosed(Mesh, CollapsedVertexMap);
	}

	void MeshClipMeshClassifyVerticesWithAABBTree(TBitArray<>& VertexInsideClipVolume, TNotNull<const FMesh*> Mesh, UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter>& ClipVolumeTree)
	{	
		MUTABLE_CPUPROFILER_SCOPE(ClassifyVerticesWithAABBTree);

		// Careful updating BitArray in parallel, operations touch a whole word worth of bits. 
		VertexInsideClipVolume.Init(false, Mesh->GetVertexCount());

		constexpr int32 NumVerticesPerBatch = 1 << 6;
		static_assert((NumVerticesPerBatch & 31) == 0, "NumVerticesPerBatch must be multiple of 32 so words in VertexInsideClipVolume are always accessed by the same batch.");

		int32 NumBatches = FMath::DivideAndRoundUp(Mesh->GetVertexCount(), NumVerticesPerBatch);

		MeshBufferIteratorConst<EMeshBufferFormat::Float32, float, 3> BasePositionIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Position);

		ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches, 
		[
			NumVerticesPerBatch,
			PositionsView = TArrayView<const FVector3f>(reinterpret_cast<const FVector3f*>(BasePositionIter.ptr()), Mesh->GetVertexCount()),
			&ClipVolumeTree,
			&VertexInsideClipVolume
		](int32 BatchId)
		{
			int32 BatchVertexBegin = BatchId * NumVerticesPerBatch;
			int32 BatchVertexEnd   = FMath::Min(BatchVertexBegin + NumVerticesPerBatch, PositionsView.Num());

			TArray<MeshIntersection::FHitIntersectionResult> RayHits;
			RayHits.Reserve(32);

			for (int32 VertexIndex = BatchVertexBegin; VertexIndex < BatchVertexEnd; ++VertexIndex)
			{
				FVector3d VertexPosition = FVector3d(PositionsView[VertexIndex]);

				if (!ClipVolumeTree.GetBoundingBox().Contains(VertexPosition))
				{
					continue;
				}

				FVector3d Origin = FVector3d(VertexPosition);
				FVector3d Direction = FVector3d::UnitZ();

				UE::Geometry::FWatertightRay3d Ray(Origin, Direction);
				
				RayHits.Reset();
				ClipVolumeTree.FindAllHitTriangles(Ray, RayHits);

				const int32 NumRayHits = RayHits.Num();

				if (NumRayHits <= 1)
				{
					// If at most one hit then we are done.
					VertexInsideClipVolume[VertexIndex] = NumRayHits > 0;
				}
				else
				{
					// We need to filter very close hits to prevent edge and shared vertices hits to be counted multiple times.
					uint32 ValidHits = 1;
					int32 CompareHitIndex = 0;

					// Hit entries are sorted by distance from the ray origin.
					for (int32 HitIndex = 1; HitIndex < NumRayHits; ++HitIndex)
					{
						if (FMath::Abs(RayHits[CompareHitIndex].Distance - RayHits[HitIndex].Distance) > UE_DOUBLE_KINDA_SMALL_NUMBER)
						{
							++ValidHits;
							CompareHitIndex = HitIndex;
						}
					};

					// Even number of hits means we are outside the clip mesh volume.  
					VertexInsideClipVolume[VertexIndex] = (ValidHits & 1) > 0;
				}
			}
		});
	}

    /** Core Geometry version */
    void MeshClipMeshClassifyVertices(TBitArray<>& VertexInClipMesh, const FMesh* Base, const FMesh* ClipMesh)
    {
		using namespace UE::Geometry;

        MUTABLE_CPUPROFILER_SCOPE(MeshClipMeshClassifyVertices);
    	
        const int32 VCount = ClipMesh->GetVertexBuffers().GetElementCount();
        const int32 FCount = ClipMesh->GetFaceCount();

        int32 OrigVertCount = Base->GetVertexBuffers().GetElementCount();

        // Stores whether each vertex in the original mesh in the clip mesh volume
        VertexInClipMesh.SetNum(OrigVertCount, false);

        if (VCount == 0)
        {
            return;
        }

        TArray<FVector3f> Vertices; // ClipMesh vertex cache
		Vertices.SetNumUninitialized(VCount);

        TArray<uint32> Faces; // ClipMesh face cache
		Faces.SetNumUninitialized(FCount * 3); 

        // Map in ClipMesh from vertices to the one they are collapsed to because they are very 
        // similar, if they aren't collapsed then they are mapped to themselves
        TArray<int32> CollapsedVertexMap;
		CollapsedVertexMap.AddUninitialized(VCount);

        MeshCreateCollapsedVertexMap( ClipMesh, CollapsedVertexMap, Vertices );

#if !UE_BUILD_SHIPPING
		if (!IsMeshClosed(ClipMesh, CollapsedVertexMap))
		{
			UE_LOGF(LogMutableCore, Warning, "Mesh operation with a mesh that is not closed as required.");
		}
#endif

        // Create cache of the faces
        UntypedMeshBufferIteratorConst ItClipMesh(ClipMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);

        for ( int32 F = 0; F < FCount; ++F )
        {
            Faces[3 * F]     = ItClipMesh.GetAsUINT32(); ++ItClipMesh;
            Faces[3 * F + 1] = ItClipMesh.GetAsUINT32(); ++ItClipMesh;
            Faces[3 * F + 2] = ItClipMesh.GetAsUINT32(); ++ItClipMesh;
        }


        // Create a bounding box of the clip mesh
		UE::Geometry::FAxisAlignedBox3f ClipMeshBoundingBox = UE::Geometry::FAxisAlignedBox3f::Empty();

        for ( const FVector3f& Vert : Vertices )
        {
            ClipMeshBoundingBox.Contain( Vert );
        }

		// Dynamic distance epsilon to support different engines
		const float MaxDimensionBoundingBox = ClipMeshBoundingBox.DiagonalLength();
		// 0.000001 is the value that helps to achieve the dynamic epsilon, do not change it
		const float DynamicEpsilon = 0.000001f * MaxDimensionBoundingBox * (MaxDimensionBoundingBox < 1.0f ? MaxDimensionBoundingBox : 1.0f);

        // Create an acceleration grid to avoid testing all clip-mesh triangles.
        // This assumes that the testing ray direction is Z
        constexpr int32 GRID_SIZE = 8;
        TUniquePtr<TArray<uint32>[]> GridFaces( new TArray<uint32>[GRID_SIZE*GRID_SIZE] );
        const FVector2f GridCellSize = FVector2f( ClipMeshBoundingBox.Width(), ClipMeshBoundingBox.Height() ) / (float)GRID_SIZE;

        for ( int32 I = 0; I < GRID_SIZE; ++I )
        {
            for ( int32 J = 0; J < GRID_SIZE; ++J )
            {
				const FVector2f BBoxMin = FVector2f(ClipMeshBoundingBox.Min.X, ClipMeshBoundingBox.Min.Y) + GridCellSize * FVector2f(I, J);

                FAxisAlignedBox2f CellBox( BBoxMin, BBoxMin + GridCellSize );

                TArray<uint32>& CellFaces = GridFaces[I+J*GRID_SIZE];
                CellFaces.Empty(FCount/GRID_SIZE);
                for (int32 F = 0; F < FCount; ++F)
                {
                    // Imprecise, conservative classification of faces.
					const FVector3f& V0 = Vertices[ Faces[3*F + 0] ];
					const FVector3f& V1 = Vertices[ Faces[3*F + 1] ];
					const FVector3f& V2 = Vertices[ Faces[3*F + 2] ];

					FAxisAlignedBox2f FaceBox;
					FaceBox.Contain( FVector2f( V0.X, V0.Y ) );
                    FaceBox.Contain( FVector2f( V1.X, V1.Y ) );
                    FaceBox.Contain( FVector2f( V2.X, V2.Y ) );

                    if (CellBox.Intersects(FaceBox))
                    {
                        CellFaces.Add( Faces[3*F + 0] );
                        CellFaces.Add( Faces[3*F + 1] );
                        CellFaces.Add( Faces[3*F + 2] );
                    }
                }
            }
        }	

        // Now go through all vertices in the mesh and record whether they are inside or outside of the ClipMesh
        uint32 DestVertexCount = Base->GetVertexCount();

        const FMeshBufferSet& MBSPriv2 = Base->GetVertexBuffers();
        for (int32 b = 0; b < MBSPriv2.Buffers.Num(); ++b)
        {
            for (int32 c = 0; c < MBSPriv2.Buffers[b].Channels.Num(); ++c)
            {
                EMeshBufferSemantic Sem = MBSPriv2.Buffers[b].Channels[c].Semantic;
                int32 SemIndex = MBSPriv2.Buffers[b].Channels[c].SemanticIndex;

                UntypedMeshBufferIteratorConst It(Base->GetVertexBuffers(), Sem, SemIndex);

                TArray<uint8> VertexAlreadyIntersected;
				VertexAlreadyIntersected.AddZeroed(VCount);

                TSet<uint64> EdgeAlreadyIntersected;

                switch ( Sem )
                {
                case EMeshBufferSemantic::Position:
                    for (uint32 V = 0; V < DestVertexCount; ++V)
                    {
                        FVector3f Vertex(0.0f, 0.0f, 0.0f);
                        for (int32 Offset = 0; Offset < 3; ++Offset)
                        {
                            ConvertData(Offset, &Vertex[0], EMeshBufferFormat::Float32, It.ptr(), It.GetFormat());
                        }
						
						const FVector2f ClipBBoxMin = FVector2f( ClipMeshBoundingBox.Min.X, ClipMeshBoundingBox.Min.Y);
						const FVector2f ClipBBoxSize = FVector2f( ClipMeshBoundingBox.Width(), ClipMeshBoundingBox.Height() );

						const FVector2i HPos = FVector2i( ( ( FVector2f( Vertex.X, Vertex.Y ) - ClipBBoxMin ) / ClipBBoxSize ) * (float)GRID_SIZE );   
						const FVector2i CellCoord = FVector2i( FMath::Clamp( HPos.X, 0, GRID_SIZE - 1 ), 
														       FMath::Clamp( HPos.Y, 0, GRID_SIZE - 1 ) );

                        // Early discard test: if the vertex is not inside the bounding box of the clip mesh, it won't be clipped.
                        const bool bContainsVertex = ClipMeshBoundingBox.Contains( Vertex );

                        if ( bContainsVertex )
                        {
                            // Optimised test	
							
							// Z-direction. Don't change this without reviewing the acceleration structure.
							FVector3f RayDir = FVector3f(0.0f, 0.0f, 1.0f);	
                            const int32 CellIndex = CellCoord.X + CellCoord.Y*GRID_SIZE;

                            int32 NumIntersections = GetNumIntersections( 
									FRay3f( Vertex, RayDir ), 
									Vertices,
									GridFaces[CellIndex],
									CollapsedVertexMap, 
									VertexAlreadyIntersected, 
									EdgeAlreadyIntersected, 
								    DynamicEpsilon);

                            // Full test BLEH, debug
//                            int32 FullNumIntersections = GetNumIntersections(
//									FRay3f( Vertex, RayDir ), 
//									Vertices,
//									Faces,
//									CollapsedVertexMap, 
//									VertexAlreadyIntersected, 
//									EdgeAlreadyIntersected, 
//								    DynamicEpsilon);

							VertexInClipMesh[V] = NumIntersections % 2 == 1;

                            // This may be used to debug degenerated cases if the conditional above is also removed.
                            // \todo: make sure it works well
//                          if (!bContainsVertex && VertexInClipMesh[V])
//                          {
//                              assert(false);
//                          }
                        }

                        ++It;
                    }
                    break;

                default:
                    break;
                }
            }
        }
    }	

    void MeshClipWithMesh(FMesh* Result, const FMesh* pBase, const FMesh* ClipMesh, bool bRemoveAllVerticesCulled, bool& bOutSuccess)
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshClipWithMesh);
		bOutSuccess = true;

        uint32 VCount = ClipMesh->GetVertexBuffers().GetElementCount();
        if (!VCount)
        {
			bOutSuccess = false;
			return; // OutSuccess false indicates the pBase can be reused in this case. 
        }

		Result->CopyFrom(*pBase);

        TBitArray<> VertexInClipMesh;  // Stores whether each vertex in the original mesh is in the clip mesh volume
        MeshClipMeshClassifyVertices(VertexInClipMesh, pBase, ClipMesh);
		
		MeshRemoveVerticesWithCullSet(Result, VertexInClipMesh, bRemoveAllVerticesCulled);
    }


	void MeshMaskClipMesh(FMesh* Result, const FMesh* pBase, const FMesh* pClipMesh, bool& bOutSuccess)
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshMaskClipMesh);

		bOutSuccess = true;

        uint32 VCount = pClipMesh->GetVertexBuffers().GetElementCount();
        if (!VCount)
        {
			bOutSuccess = false;
			return;
        }


        TBitArray<> VertexInClipMesh;  // Stores whether each vertex in the original mesh in in the clip mesh volume
        MeshClipMeshClassifyVertices(VertexInClipMesh, pBase, pClipMesh);

        // We only remove vertices if all their faces are clipped
        TArray<uint8> VertexWithFaceNotClipped;
		VertexWithFaceNotClipped.AddZeroed( VertexInClipMesh.Num() );

        UntypedMeshBufferIteratorConst Ito(pBase->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
        int32 AFaceCount = pBase->GetFaceCount();
        for (int32 F = 0; F < AFaceCount; ++F)
        {
			uint32 OV[3] = { 0, 0, 0 };

            OV[0] = Ito.GetAsUINT32(); ++Ito;
            OV[1] = Ito.GetAsUINT32(); ++Ito;
            OV[2] = Ito.GetAsUINT32(); ++Ito;

            bool bFaceClipped =
                    VertexInClipMesh[OV[0]] &&
                    VertexInClipMesh[OV[1]] &&
                    VertexInClipMesh[OV[2]];

            if (!bFaceClipped)
            {
                VertexWithFaceNotClipped[OV[0]] = true;
                VertexWithFaceNotClipped[OV[1]] = true;
                VertexWithFaceNotClipped[OV[2]] = true;
            }
        }

        CreateMask(Result, pBase, VertexWithFaceNotClipped);
    }


	void MakeMeshMaskFromUVMask(FMesh* Result, const FMesh* Base, const FMesh* BaseForUVs, const FImage* Mask, uint8 LayoutIndex, bool& bOutSuccess)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshMaskUVMask);

		check(Result && Base && Mask && BaseForUVs);
		check(Base->VertexBuffers.GetElementCount() == BaseForUVs->VertexBuffers.GetElementCount());

		bOutSuccess = true;

		// Stores whether each vertex in the original mesh is in the clip mesh volume
		TBitArray<> VertexClipped; 

		MeshUVMaskClassifyVertices(VertexClipped, BaseForUVs, Mask, LayoutIndex);

		// We only remove vertices if all their faces are clipped
		TArray<uint8> VertexWithFaceNotClipped;
		VertexWithFaceNotClipped.AddZeroed(VertexClipped.Num());

		UntypedMeshBufferIteratorConst Ito(Base->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
		int32 FaceCount = Base->GetFaceCount();
		for (int32 F = 0; F < FaceCount; ++F)
		{
			uint32 OV[3] = { 0, 0, 0 };

			OV[0] = Ito.GetAsUINT32(); ++Ito;
			OV[1] = Ito.GetAsUINT32(); ++Ito;
			OV[2] = Ito.GetAsUINT32(); ++Ito;

			bool bFaceClipped =
				VertexClipped[OV[0]] &&
				VertexClipped[OV[1]] &&
				VertexClipped[OV[2]];

			if (!bFaceClipped)
			{
				VertexWithFaceNotClipped[OV[0]] = true;
				VertexWithFaceNotClipped[OV[1]] = true;
				VertexWithFaceNotClipped[OV[2]] = true;
			}
		}

		CreateMask(Result, Base, VertexWithFaceNotClipped);
	}


	void MakeMeshMaskFromLayout(FMesh* Result, const FMesh* Base, const FMesh* BaseForUVs, const FLayout* Mask, uint8 LayoutIndex, bool& bOutSuccess)
	{
		MUTABLE_CPUPROFILER_SCOPE(MakeMeshMaskFromLayout);

		check(Result && Base && Mask && BaseForUVs);
		//check(Base->VertexBuffers.GetElementCount()== BaseForUVs->VertexBuffers.GetElementCount());
		if (Base->VertexBuffers.GetElementCount() != BaseForUVs->VertexBuffers.GetElementCount())
		{
			ensure(false);
			return;
		}

		bOutSuccess = true;

		// Stores whether each vertex in the original mesh is in the clip mesh volume
		TBitArray<> VertexClipped;

		MeshLayoutMaskClassifyVertices(VertexClipped, BaseForUVs, Mask, LayoutIndex);

		// We only remove vertices if all their faces are clipped
		TArray<uint8> VertexWithFaceNotClipped;
		VertexWithFaceNotClipped.AddZeroed(VertexClipped.Num());

		UntypedMeshBufferIteratorConst Ito(Base->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
		int32 FaceCount = Base->GetFaceCount();
		for (int32 F = 0; F < FaceCount; ++F)
		{
			uint32 OV[3] = { 0, 0, 0 };

			OV[0] = Ito.GetAsUINT32(); ++Ito;
			OV[1] = Ito.GetAsUINT32(); ++Ito;
			OV[2] = Ito.GetAsUINT32(); ++Ito;

			bool bFaceClipped =
				VertexClipped[OV[0]] &&
				VertexClipped[OV[1]] &&
				VertexClipped[OV[2]];

			if (!bFaceClipped)
			{
				VertexWithFaceNotClipped[OV[0]] = true;
				VertexWithFaceNotClipped[OV[1]] = true;
				VertexWithFaceNotClipped[OV[2]] = true;
			}
		}

		CreateMask(Result, Base, VertexWithFaceNotClipped);
	}


	void MeshMaskDiff(FMesh* Result, const FMesh* pBase, const FMesh* pFragment, bool& bOutSuccess)
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshMaskDiff);
		bOutSuccess = true;

        uint32 vcount = pFragment->GetVertexBuffers().GetElementCount();
        if (!vcount)
        {
			bOutSuccess = false;
            return;
        }

        int sourceFaceCount = pBase->GetFaceCount();
        int sourceVertexCount = pBase->GetVertexCount();
        int fragmentFaceCount = pFragment->GetFaceCount();


        // Make a tolerance proportional to the mesh bounding box size
        // TODO: Use precomputed bounding box
        box< FVector3f > aabbox;
        if ( fragmentFaceCount > 0 )
        {
            MeshBufferIteratorConst<EMeshBufferFormat::Float32,float,3> itp( pFragment->GetVertexBuffers(), EMeshBufferSemantic::Position );

            aabbox.min = itp.GetAsVec3f();
            ++itp;

            for ( int v=1; v<pFragment->GetVertexBuffers().GetElementCount(); ++v )
            {
                aabbox.Bound(itp.GetAsVec3f());
                ++itp;
            }
        }
        float tolerance = 1e-5f * aabbox.size.Length();
        FMesh::FVertexMatchMap vertexMap;
        pFragment->GetVertexMap( *pBase, vertexMap, tolerance );


        // Classify the target faces in buckets along the Y axis
#define NUM_BUCKETS	128
#define AXIS		1
        TArray<int> buckets[ NUM_BUCKETS ];
        float bucketStart = aabbox.min[AXIS];
        float bucketSize = aabbox.size[AXIS] / NUM_BUCKETS;

        float bucketThreshold = ( 4 * tolerance ) / bucketSize;
        UntypedMeshBufferIteratorConst iti( pFragment->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex );
        MeshBufferIteratorConst<EMeshBufferFormat::Float32,float,3> itp( pFragment->GetVertexBuffers(), EMeshBufferSemantic::Position );
        for ( int tf=0; tf<fragmentFaceCount; tf++ )
        {
            uint32 index0 = iti.GetAsUINT32(); ++iti;
            uint32 index1 = iti.GetAsUINT32(); ++iti;
            uint32 index2 = iti.GetAsUINT32(); ++iti;
            float y = ( (*(itp+index0))[AXIS] + (*(itp+index1))[AXIS] + (*(itp+index2))[AXIS] ) / 3;
            float fbucket = (y-bucketStart) / bucketSize;
            int bucket = FMath::Min( NUM_BUCKETS-1, FMath::Max( 0, (int)fbucket ) );
            buckets[bucket].Add(tf);
            int hibucket = FMath::Min( NUM_BUCKETS-1, FMath::Max( 0, (int)(fbucket+bucketThreshold) ) );
            if (hibucket!=bucket)
            {
                buckets[hibucket].Add(tf);
            }
            int lobucket = FMath::Min( NUM_BUCKETS-1, FMath::Max( 0, (int)(fbucket-bucketThreshold) ) );
            if (lobucket!=bucket)
            {
                buckets[lobucket].Add(tf);
            }
        }

//		LogDebug("Box : min %.3f, %.3f, %.3f    size %.3f,%.3f,%.3f\n",
//				aabbox.min[0], aabbox.min[1], aabbox.min[2],
//				aabbox.size[0], aabbox.size[1], aabbox.size[2] );
//		for ( int b=0; b<NUM_BUCKETS; ++b )
//		{
//			LogDebug("bucket : %d\n", buckets[b].size() );
//		}

		TArray<uint8> faceClipped;
		faceClipped.SetNumZeroed(sourceFaceCount);

        UntypedMeshBufferIteratorConst ito( pBase->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex );
        MeshBufferIteratorConst<EMeshBufferFormat::Float32,float,3> itop( pBase->GetVertexBuffers(), EMeshBufferSemantic::Position );
        UntypedMeshBufferIteratorConst itti( pFragment->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex );
        for ( int f=0; f<sourceFaceCount; ++f )
        {
            bool hasFace = false;
            FUint32Vector3 ov;
            ov[0] = ito.GetAsUINT32(); ++ito;
            ov[1] = ito.GetAsUINT32(); ++ito;
            ov[2] = ito.GetAsUINT32(); ++ito;

            // find the bucket for this face
            float y = ( (*(itop+ov[0]))[AXIS] + (*(itop+ov[1]))[AXIS] + (*(itop+ov[2]))[AXIS] ) / 3;
            float fbucket = (y-bucketStart) / bucketSize;
            int bucket = FMath::Min( NUM_BUCKETS-1, FMath::Max( 0, (int)fbucket ) );

            for ( int32 btf=0; !hasFace && btf<buckets[bucket].Num(); btf++ )
            {
                int tf =  buckets[bucket][btf];

				FUint32Vector3 v;
                v[0] = (itti+3*tf+0).GetAsUINT32();
                v[1] = (itti+3*tf+1).GetAsUINT32();
                v[2] = (itti+3*tf+2).GetAsUINT32();

                hasFace = true;
                for ( int vi=0; hasFace && vi<3; ++vi )
                {
                    hasFace = vertexMap.DoMatch(v[vi],ov[0])
                         || vertexMap.DoMatch(v[vi],ov[1])
                         || vertexMap.DoMatch(v[vi],ov[2]);
                }
            }

            if ( hasFace )
            {
                faceClipped[f] = true;
            }
        }

        // We only remove vertices if all their faces are clipped
		TArray<uint8> vertex_with_face_not_clipped;
		vertex_with_face_not_clipped.SetNumZeroed(sourceVertexCount);

        UntypedMeshBufferIteratorConst itoi(pBase->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
        int aFaceCount = pBase->GetFaceCount();
        for (int f = 0; f < aFaceCount; ++f)
        {
			FUint32Vector3 ov;
            ov[0] = itoi.GetAsUINT32(); ++itoi;
            ov[1] = itoi.GetAsUINT32(); ++itoi;
            ov[2] = itoi.GetAsUINT32(); ++itoi;

            if (!faceClipped[f])
            {
                vertex_with_face_not_clipped[ov[0]] = true;
                vertex_with_face_not_clipped[ov[1]] = true;
                vertex_with_face_not_clipped[ov[2]] = true;
            }
        }

		CreateMask(Result, pBase, vertex_with_face_not_clipped);
    }

}
