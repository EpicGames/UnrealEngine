// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IndexTypes.h"
#include "MuR/Platform.h"



namespace UE::Geometry
{
	template<class T>
	class TMeshAABBTree3;
}


namespace UE::Mutable::Private
{
	namespace GeometryUtils
	{
		struct FMeshAdapter;
	}

	class FMesh;
	class FImage;
	class FLayout;

    /** Generate a classification list for which vertex of pBase is fully contained in pClipMesh */
	extern void MeshClipMeshClassifyVertices(TBitArray<>& VertexInClipMesh, const FMesh* Base, const FMesh* ClipMesh);

	void MeshClipMeshClassifyVerticesWithAABBTree(TBitArray<>& VertexInsideClipVolume, TNotNull<const FMesh*> Mesh, UE::Geometry::TMeshAABBTree3<GeometryUtils::FMeshAdapter>& ClipVolumeTree);

    /**  */
	extern void MeshClipWithMesh(FMesh* Result, const FMesh* Base, const FMesh* ClipMesh, bool bRemoveAllVerticesCulled, bool& bOutSuccess);

    /** Generate a mask mesh with the faces of the base mesh inside the clip mesh. */
	extern void MeshMaskClipMesh(FMesh* Result, const FMesh* Base, const FMesh* ClipMesh, bool& bOutSuccess);

	/** Generate a mask mesh with the faces of the base mesh that have all 3 vertices marked in the given mask. */
	extern void MakeMeshMaskFromUVMask(FMesh* Result, const FMesh* Base, const FMesh* BaseForUVs, const FImage* Mask, uint8 LayoutIndex, bool& bOutSuccess);

    /** Generate a mask mesh with the faces of the base mesh matching the fragment. */
	extern void MeshMaskDiff(FMesh* Result, const FMesh* Base, const FMesh* Fragment, bool& bOutSuccess);

	/** Generate a mask mesh with the faces of the base mesh that have all 3 vertices inside any block of the given layout. */
	extern void MakeMeshMaskFromLayout(FMesh* Result, const FMesh* Base, const FMesh* BaseForUVs, const FLayout* Mask, uint8 LayoutIndex, bool& bOutSuccess);

	/** Return true if the mesh is closed. Usually used to validate clipping meshes. */
	extern bool IsMeshClosed(const FMesh* Mesh);
}
