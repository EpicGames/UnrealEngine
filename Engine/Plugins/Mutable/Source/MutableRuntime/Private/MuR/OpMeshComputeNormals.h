// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/Vector.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "IndexTypes.h"

#include "MuR/Mesh.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"


namespace UE::Mutable::Private
{

	inline float ComputeTangentBasisDeterminantSign(const FVector3f& N, const FVector3f& T, const FVector3f& B)
	{
		const float BaseTangentBasisDeterminant =  
				B.X*T.Y*N.Z + B.Z*T.X*N.Y + B.Y*T.Z*N.Y -
				B.Z*T.Y*N.X - B.Y*T.X*N.Z - B.X*T.Z*N.Y;

		return BaseTangentBasisDeterminant < 0.0f ? -1.0f : 1.0f;
	}

	inline void OrthogonalizeTangentSpace(
			const FVector3f* NormalPtr, FVector3f* TangentPtr, FVector3f* BiNormalPtr, 
			float TangentBasisDeterminantSign)	
	{
		if (!!NormalPtr & !!TangentPtr)
		{
			// Orthogonalize Tangent based on new Normal. This assumes Normal and Tangent are normalized and different.
			*TangentPtr = (*TangentPtr - FVector3f::DotProduct(*NormalPtr, *TangentPtr) * *NormalPtr).GetSafeNormal();

			// BiNormal
			if (BiNormalPtr)
			{
				*BiNormalPtr = FVector3f::CrossProduct(*TangentPtr, *NormalPtr) * TangentBasisDeterminantSign;
			}
		}
	}

	inline void ComputeMeshNormals(FMesh& DestMesh)
	{
		MUTABLE_CPUPROFILER_SCOPE(ComputeMeshNormals);

		const UntypedMeshBufferIteratorConst BaseIndicesIter = UntypedMeshBufferIteratorConst(DestMesh.GetIndexBuffers(), EMeshBufferSemantic::VertexIndex, 0);
		const int32 NumIndices = DestMesh.GetIndexBuffers().GetElementCount();

		const UntypedMeshBufferIteratorConst BasePositionIter = UntypedMeshBufferIteratorConst(DestMesh.GetVertexBuffers(), EMeshBufferSemantic::Position, 0);
		const int32 NumVertices = DestMesh.GetVertexBuffers().GetElementCount();

		check(BaseIndicesIter.ptr());
		check(BasePositionIter.ptr());

		TArray<FVector3f> NormalsAccumulation;
		NormalsAccumulation.SetNumZeroed(NumVertices);	

		check(NumIndices % 3 == 0);
		for (int32 Face = 0; Face < NumIndices; Face += 3)
		{
			const UE::Geometry::FIndex3i FaceIndices = UE::Geometry::FIndex3i 
			{ 
				static_cast<int32>((BaseIndicesIter + Face + 0).GetAsUINT32()), 
				static_cast<int32>((BaseIndicesIter + Face + 1).GetAsUINT32()),
				static_cast<int32>((BaseIndicesIter + Face + 2).GetAsUINT32())
			};

			const FVector3f V0 = (BasePositionIter + FaceIndices[0]).GetAsVec3f();
			const FVector3f V1 = (BasePositionIter + FaceIndices[1]).GetAsVec3f();
			const FVector3f V2 = (BasePositionIter + FaceIndices[2]).GetAsVec3f();

			const FVector3f AreaWeightedTriangleNormal = FVector3f::CrossProduct(V2 - V0, V1 - V0);

			NormalsAccumulation[FaceIndices[0]] += AreaWeightedTriangleNormal;
			NormalsAccumulation[FaceIndices[1]] += AreaWeightedTriangleNormal;
			NormalsAccumulation[FaceIndices[2]] += AreaWeightedTriangleNormal;
		}

		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			NormalsAccumulation[VertexIndex] = NormalsAccumulation[VertexIndex].GetSafeNormal();
		}

		const UntypedMeshBufferIterator NormalIter = UntypedMeshBufferIterator(DestMesh.GetVertexBuffers(), EMeshBufferSemantic::Normal, 0);
		check(NormalIter.ptr());

		const UntypedMeshBufferIterator TangentIter  = UntypedMeshBufferIterator(DestMesh.GetVertexBuffers(), EMeshBufferSemantic::Tangent, 0);
		const UntypedMeshBufferIterator BiNormalIter = UntypedMeshBufferIterator(DestMesh.GetVertexBuffers(), EMeshBufferSemantic::Binormal, 0);

        const EMeshBufferFormat NormalFormat   = NormalIter.GetFormat();
        const EMeshBufferFormat TangentFormat  = TangentIter.GetFormat();
        const EMeshBufferFormat BiNormalFormat = BiNormalIter.GetFormat();
	
        const int32 NormalComps   = NormalIter.GetComponents();
        const int32 TangentComps  = TangentIter.GetComponents();
        const int32 BiNormalComps = BiNormalIter.GetComponents();

		// When normal is packed, binormal channel is not expected. It is not a big deal if it's there but we would be doing extra unused work in that case. 
		ensureAlways(!(NormalFormat == EMeshBufferFormat::PackedDir8_W_TangentSign || NormalFormat == EMeshBufferFormat::PackedDirS8_W_TangentSign) || !BiNormalIter.ptr());
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const FVector3f Normal = NormalsAccumulation[VertexIndex];
			const FVector3f OriginalNormal = NormalIter.ptr()
					? FVector3f::ZeroVector
					: (NormalIter + VertexIndex).GetAsVec3f();

			FVector3f Tangent = !TangentIter.ptr() 
					? FVector3f::ZeroVector
					: (TangentIter + VertexIndex).GetAsVec3f();

			FVector3f BiNormal = !BiNormalIter.ptr() 
					? FVector3f::ZeroVector
					: (BiNormalIter + VertexIndex).GetAsVec3f();
			
			OrthogonalizeTangentSpace(
					&Normal, 
					TangentIter.ptr() ? &Tangent : nullptr, 
					BiNormalIter.ptr() ? &BiNormal : nullptr,
					BiNormalIter.ptr() ? ComputeTangentBasisDeterminantSign(OriginalNormal, Tangent, BiNormal) : 0.0f);
			
			if (NormalIter.ptr())
			{
				// Leave the tangent basis sign untouched for packed normals formats.
				uint8 * const NormalElemPtr = (NormalIter + VertexIndex).ptr();
				for (int32 C = 0; C < NormalComps && C < 3; ++C)
				{
					ConvertData(C, NormalElemPtr, NormalFormat, &Normal, EMeshBufferFormat::Float32);
				}
			}

			if (TangentIter.ptr())
			{
				uint8 * const TangentElemPtr = (TangentIter + VertexIndex).ptr();
				for (int32 C = 0; C < TangentComps && C < 3; ++C)
				{
					ConvertData(C, TangentElemPtr, TangentFormat, &Tangent, EMeshBufferFormat::Float32);
				}
			}
			
			if (BiNormalIter.ptr())
			{
				uint8 * const BiNormalElemPtr = (BiNormalIter + VertexIndex).ptr();
				for (int32 C = 0; C < BiNormalComps && C < 3; ++C)
				{
					ConvertData(C, BiNormalElemPtr, BiNormalFormat, &BiNormal, EMeshBufferFormat::Float32);
				}
			}
		}
	}
}

