// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshPrivate.h"
#include "MuR/Platform.h"

namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	//! Create a diff from the mesh vertices. The meshes must have the same amount of vertices.
	//! If the channel list is empty all the channels will be compared.
	//---------------------------------------------------------------------------------------------
    inline void MeshDifference( FMesh* Result, const FMesh* pBase, const FMesh* pTarget,
                                   int32 numChannels,
                                   const EMeshBufferSemantic* semantics,
                                   const int32* semanticIndices,
                                   bool ignoreTexCoords, bool& bOutSuccess)

	{
		bOutSuccess = true;

        if (!pBase || !pTarget)
        {
			bOutSuccess = false;
            return;
        }

		bool bIsCorrect =
                pBase
                &&
				( pBase->GetVertexBuffers().GetElementCount() == pTarget->GetVertexBuffers().GetElementCount() )
				&&
                ( pBase->GetIndexCount() == pTarget->GetIndexCount() )
                &&
                ( pBase->GetVertexBuffers().GetElementCount()>0 )
                ;

		if (!bIsCorrect)
		{
			bOutSuccess = true; // Use the provided empty mesh as result.
			return ;
		}

		int32 VertexCount = pBase->GetVertexBuffers().GetElementCount();		

		// If no channels were specified, get them all
		TArray<EMeshBufferSemantic> allSemantics;		
		TArray<int32> allSemanticIndices;
		if ( !numChannels )
		{
			for ( int32 vb=0; vb<pBase->GetVertexBuffers().GetBufferCount(); ++vb )
			{
				for ( int32 c=0; c<pBase->GetVertexBuffers().GetBufferChannelCount(vb); ++c )
				{
					EMeshBufferSemantic sem = EMeshBufferSemantic::None;
					int32 semIndex = 0;
					pBase->GetVertexBuffers().GetChannel( vb, c, &sem, &semIndex, nullptr, nullptr, nullptr );			
					if ( sem != EMeshBufferSemantic::VertexIndex &&
					     sem != EMeshBufferSemantic::BoneIndices &&
					     sem != EMeshBufferSemantic::BoneWeights &&
					     sem != EMeshBufferSemantic::LayoutBlock &&
					     sem != EMeshBufferSemantic::Other && 
					     ( !ignoreTexCoords || sem!=EMeshBufferSemantic::TexCoords ) )
					{
						allSemantics.Add( sem );
						allSemanticIndices.Add( semIndex );
						++numChannels;
					}
				}
			}

			semantics = &allSemantics[0];
			semanticIndices = &allSemanticIndices[0];
		}


		// Make a delta of every vertex
		// TODO: Not always floats
		int32 differentVertexCount = 0;
		TArray<bool> isVertexDifferent;
		isVertexDifferent.SetNumZeroed(VertexCount);
		TArray< FVector3f > deltas;
		deltas.SetNumZeroed(VertexCount * numChannels);

		for ( int32 c=0; c<numChannels; ++c )
		{
			UntypedMeshBufferIteratorConst baseIt		
					( pBase->GetVertexBuffers(), semantics[c], semanticIndices[c] );

			UntypedMeshBufferIteratorConst targetIt		
					( pTarget->GetVertexBuffers(), semantics[c], semanticIndices[c] );

			for ( int32 v=0; v<VertexCount; ++v )
			{
				FVector3f base = baseIt.GetAsVec3f();
				FVector3f target = targetIt.GetAsVec3f();
				FVector3f delta = target-base;
				deltas[ v*numChannels+c ] = delta;		

				if (!delta.Equals(FVector3f(0,0,0),UE_SMALL_NUMBER))
				{
					if ( !isVertexDifferent[v] )
					{
						++differentVertexCount;
					}

					isVertexDifferent[v] = true;
				}

				++baseIt;		
				++targetIt;		
			}
		} 


		// Create the morph mesh
		{
			Result->GetVertexBuffers().SetElementCount( differentVertexCount );			
			Result->GetVertexBuffers().SetBufferCount( 2 );								

			TArray<EMeshBufferSemantic> semantic;
			TArray<int32> semanticIndex;
			TArray<EMeshBufferFormat> format;
			TArray<int32> components;
			TArray<int32> offsets;
			int32 ElementSize = 0;

			// The first buffer will contain the actual morph data.
			for (int32 c = 0; c < numChannels; ++c)
			{
				semantic.Add(semantics[c]);
				semanticIndex.Add(semanticIndices[c]);
				format.Add(EMeshBufferFormat::Float32);
				components.Add(3);
				offsets.Add(ElementSize);
				ElementSize += 3 * sizeof(float);
			}

			int32 BufferIndex = 0;
			Result->GetVertexBuffers().SetBuffer(BufferIndex, ElementSize, semantic.Num(), semantic.GetData(), semanticIndex.GetData(), format.GetData(), components.GetData(), offsets.GetData());

			// The second buffer will contain the ids of the vertices to morph.
			semantic = { EMeshBufferSemantic::VertexIndex };
			semanticIndex = { 0 };
			components = { 1 };
			offsets = { 0 };
			bool bGenerateRelativeIds = !pBase->AreVertexIdsExplicit();
			if (bGenerateRelativeIds)
			{
				format = { EMeshBufferFormat::UInt32 };
				ElementSize = sizeof(uint32);
			}
			else
			{
				format = { EMeshBufferFormat::UInt64 };
				ElementSize = sizeof(uint64);
			}

			BufferIndex = 1;
			Result->GetVertexBuffers().SetBuffer(BufferIndex, ElementSize, semantic.Num(), semantic.GetData(), semanticIndex.GetData(), format.GetData(), components.GetData(), offsets.GetData());

			Result->MeshIDPrefix = pBase->MeshIDPrefix;

			// Source vertex index channel
			MeshVertexIdIteratorConst IdIterator( pBase );
			check(IdIterator.IsValid());

			// Set the data
			uint8* ResultData = Result->GetVertexBuffers().GetBufferData(0);
			uint8* ResultIds = Result->GetVertexBuffers().GetBufferData(1);
			for (int32 v=0; v<VertexCount; ++v)
			{
				if ( isVertexDifferent[v] )
				{
					// Vertex Id
					uint64 FullId = IdIterator.Get();
					if (bGenerateRelativeIds)
					{
						uint32 RelativeId = uint32_t(FullId & 0xffffffff);
						*((uint32*)ResultIds) = RelativeId;
						ResultIds += sizeof(uint32);
					}
					else
					{
						*((uint64*)ResultIds) = FullId;
						ResultIds += sizeof(uint64);
					}

					// all channels
					for ( int32 c=0; c<numChannels; ++c )
					{
						constexpr int32 ChannelSizeBytes = 3*sizeof(float);
						FMemory::Memcpy(ResultData, &deltas[v * numChannels + c], ChannelSizeBytes);
						ResultData += ChannelSizeBytes;
					}
				}

				++IdIterator;
			}
		}


        // Morphs are always single surfaced
        Result->Surfaces.Empty();
        Result->ClothSections.Empty();
        Result->EnsureSurfaceData();
	}

}
