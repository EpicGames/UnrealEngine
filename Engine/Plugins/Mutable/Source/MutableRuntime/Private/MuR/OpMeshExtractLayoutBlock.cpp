// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshExtractLayoutBlock.h"

#include "MuR/MeshPrivate.h"
#include "MuR/OpMeshRemove.h"

namespace UE::Mutable::Private
{
	void MeshExtractLayoutBlock(TNotNull<FMesh*> Result, uint32 LayoutIndex, TArrayView<uint64> BlockIds)
	{
		UntypedMeshBufferIteratorConst ItBlocks(Result->GetVertexBuffers(), EMeshBufferSemantic::LayoutBlock, LayoutIndex);

		if (ItBlocks.GetFormat() != EMeshBufferFormat::None)
		{
			const int32 NumVertices = Result->GetVertexCount();

			TBitArray<> VerticesToCull;
			VerticesToCull.SetNumUninitialized(NumVertices);

			if (ItBlocks.GetFormat() == EMeshBufferFormat::UInt16)
			{
				TArrayView<const uint16> VertexBlockIdsView(reinterpret_cast<const uint16*>(ItBlocks.ptr()), NumVertices);

				const uint64 AbsoluteVertexBlockIdHighBits = uint64(Result->MeshIDPrefix) << 32;
				for (int32 VertexIndex = NumVertices - 1; VertexIndex >= 0; --VertexIndex)
				{
					const uint64 AbsoluteVertexBlockId = AbsoluteVertexBlockIdHighBits | VertexBlockIdsView[VertexIndex];
					VerticesToCull[VertexIndex] = BlockIds.Find(AbsoluteVertexBlockId) == INDEX_NONE;
				}
			}
			else if (ItBlocks.GetFormat() == EMeshBufferFormat::UInt64)
			{
				TArrayView<const uint64> VertexBlockIdsView(reinterpret_cast<const uint64*>(ItBlocks.ptr()), NumVertices);
				for (int32 VertexIndex = NumVertices - 1; VertexIndex >= 0; --VertexIndex)
				{
					VerticesToCull[VertexIndex] = BlockIds.Find(VertexBlockIdsView[VertexIndex]) == INDEX_NONE;
				}
			}
			else
			{
				check(false);
			}

			constexpr bool bRemoveIfAllVerticesCulled = true;
			MeshRemoveVerticesWithCullSet(Result, VerticesToCull, bRemoveIfAllVerticesCulled); 
		}
	}

	void MeshExtractLayoutBlock(TNotNull<FMesh*> Result, uint32 LayoutIndex)
	{
		UntypedMeshBufferIteratorConst ItBlocks(Result->GetVertexBuffers(), EMeshBufferSemantic::LayoutBlock, LayoutIndex);

		if (ItBlocks.GetFormat() != EMeshBufferFormat::None)
		{
			const int32 NumVertices = Result->GetVertexCount();
			TBitArray<> VerticesToCull;
			VerticesToCull.SetNumUninitialized(NumVertices);

			if (ItBlocks.GetFormat() == EMeshBufferFormat::UInt16)
			{
				TArrayView<const uint16> VertexBlockIdsView(reinterpret_cast<const uint16*>(ItBlocks.ptr()), NumVertices);

				for (int32 VertexIndex = NumVertices - 1; VertexIndex >= 0; --VertexIndex)
				{
					VerticesToCull[VertexIndex] = VertexBlockIdsView[VertexIndex] == std::numeric_limits<uint16>::max();
				}
			}
			else if (ItBlocks.GetFormat() == EMeshBufferFormat::UInt64)
			{
				TArrayView<const uint64> VertexBlockIdsView(reinterpret_cast<const uint64*>(ItBlocks.ptr()), NumVertices);
				for (int32 VertexIndex = NumVertices - 1; VertexIndex >= 0; --VertexIndex)
				{
					VerticesToCull[VertexIndex] = VertexBlockIdsView[VertexIndex] == std::numeric_limits<uint64>::max();
				}
			}
			else
			{
				check(false);
			}

			constexpr bool bRemoveIfAllVerticesCulled = true;
			MeshRemoveVerticesWithCullSet(Result, VerticesToCull, bRemoveIfAllVerticesCulled); 
		}
	}

}
