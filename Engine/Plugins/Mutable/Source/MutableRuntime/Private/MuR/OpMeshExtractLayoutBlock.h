// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/ArrayView.h"
#include "Misc/NotNull.h"

namespace UE::Mutable::Private
{
    class FMesh;

	/** Extract all vertices that have a layout block from the passed list on the given channel in place. */
    void MeshExtractLayoutBlock(TNotNull<FMesh*> Result, uint32 LayoutIndex, TArrayView<uint64> BlockIds);

	/** Extract all vertices that have a valid layout block on the given channel in place. */
	void MeshExtractLayoutBlock(TNotNull<FMesh*> Result, uint32 LayoutIndex);
}
