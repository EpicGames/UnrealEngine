// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshFormat.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ConvertData.h"
#include "MuR/Layout.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/MutableRuntimeModule.h"

#include "Engine/SkeletalMesh.h"
#include "UObject/StrongObjectPtr.h"

#include "GPUSkinPublicDefs.h"

namespace UE::Mutable::Private
{

	//-------------------------------------------------------------------------------------------------
	void MeshFormatBuffer( const FMeshBufferSet& Source, FMeshBuffer& ResultBuffer, int32 ResultOffsetElements, bool bHasSpecialSemantics, uint32 IDPrefix )
	{
		int32 SourceElementCount = Source.GetElementCount();
		if (!SourceElementCount)
		{
			return;
		}

		// This happens in the debugger
		if (ResultBuffer.Channels.IsEmpty() || ResultBuffer.ElementSize==0)
		{
			return;
		}

		check(ResultBuffer.ElementSize);
		int32 ResultElementCount = ResultBuffer.Data.Num() / ResultBuffer.ElementSize;
		check(SourceElementCount + ResultOffsetElements <= ResultElementCount);

		// For every channel in this buffer
		for (int32 ResultChannelIndex = 0; ResultChannelIndex < ResultBuffer.Channels.Num(); ++ResultChannelIndex)
		{
			FMeshBufferChannel& ResultChannel = ResultBuffer.Channels[ResultChannelIndex];

			// Find this channel in the source mesh
			int32 SourceBufferIndex;
			int32 SourceChannelIndex;
			Source.FindChannel(ResultChannel.Semantic, ResultChannel.SemanticIndex, &SourceBufferIndex, &SourceChannelIndex);

			int32 ResultElemSize = ResultBuffer.ElementSize;
			int32 ResultComponents = ResultChannel.ComponentCount;
			int32 ResultChannelSize = GetMeshFormatData(ResultChannel.Format).SizeInBytes * ResultComponents;
			uint8* ResultBuf = ResultBuffer.Data.GetData() + ResultOffsetElements*ResultBuffer.ElementSize;
			ResultBuf += ResultChannel.Offset;

			// Case 1: Special semantics that may be implicit or relative
			//---------------------------------------------------------------------------------
			if (bHasSpecialSemantics && ResultChannel.Semantic == EMeshBufferSemantic::VertexIndex)
			{
				bool bHasVertexIndices = (SourceBufferIndex >=0 );
				if (bHasVertexIndices)
				{
					check(SourceChannelIndex == 0 && Source.Buffers[SourceBufferIndex].Channels.Num() == 1);

					const FMeshBuffer& SourceBuffer = Source.Buffers[SourceBufferIndex];
					const FMeshBufferChannel& SourceChannel = SourceBuffer.Channels[SourceChannelIndex];

					bool bHasSameFormat = SourceChannel.Format == ResultChannel.Format;
					if (bHasSameFormat)
					{
						check(SourceChannel == ResultChannel);
						FMemory::Memcpy(ResultBuf,SourceBuffer.Data.GetData(),SourceBuffer.Data.Num());
					}
					else
					{
						// Relative vertex IDs
						check(IDPrefix);
						check(SourceChannel.Format == EMeshBufferFormat::UInt32);
						const uint32* SourceData = reinterpret_cast<const uint32*>(SourceBuffer.Data.GetData());

						check(ResultChannel.Format == EMeshBufferFormat::UInt64);
						uint64* ResultData = reinterpret_cast<uint64*>(ResultBuf);

						for (int32 i = 0; i < SourceElementCount; ++i)
						{
							uint32 SourceId = *SourceData++;
							uint64 Id = (uint64(IDPrefix) << 32) | uint64(SourceId);
							*ResultData = Id;
							++ResultData;
						}
					}
				}
				else
				{
					// Implicit IDs
					check(IDPrefix);

					check(ResultChannel.Format == EMeshBufferFormat::UInt64);
					uint64* ResultData = reinterpret_cast<uint64*>(ResultBuf);

					for (int32 VertexIndex = 0; VertexIndex < SourceElementCount; ++VertexIndex)
					{
						uint64 Id = (uint64(IDPrefix) << 32) | uint64(VertexIndex);
						*ResultData = Id;
						++ResultData;
					}
				}

				continue;
			}

			else if (bHasSpecialSemantics && ResultChannel.Semantic == EMeshBufferSemantic::LayoutBlock)
			{
				bool bHasBlockIds = (SourceBufferIndex >= 0);

				if (bHasBlockIds)
				{
					const FMeshBuffer& SourceBuffer = Source.Buffers[SourceBufferIndex];
					const FMeshBufferChannel& SourceChannel = SourceBuffer.Channels[SourceChannelIndex];

					bool bHasSameFormat = SourceChannel.Format == ResultChannel.Format;
					if (bHasSameFormat)
					{
						check(SourceChannel==ResultChannel);
						FMemory::Memcpy(ResultBuf, SourceBuffer.Data.GetData(), SourceBuffer.Data.Num());
					}
					else
					{
						// Relative vertex IDs
						check(SourceChannel.Format == EMeshBufferFormat::UInt16);
						const uint16* SourceData = reinterpret_cast<const uint16*>(SourceBuffer.Data.GetData());

						check(ResultChannel.Format == EMeshBufferFormat::UInt64);
						uint64* ResultData = reinterpret_cast<uint64*>(ResultBuf);

						for (int32 i = 0; i < SourceElementCount; ++i)
						{
							uint16 SourceId = *SourceData++;
							uint64 Id = (uint64(IDPrefix) << 32) | uint64(SourceId);
							*ResultData = Id;
							++ResultData;
						}
					}
					continue;
				}
				else
				{
					// This seems to happen with objects that mix meshes with layouts with meshes without layouts.
					// If we don't do anything here, it will be filled with zeros in the following code.
					//ensure(false);
				}

			}

			// Case 2: Not found in source: generate with default values, depending on semantic
			//---------------------------------------------------------------------------------
			if (SourceBufferIndex < 0)
			{
				// Not found: fill with zeros.

				// Special case for derived channel data
				bool generated = false;

				// If we have to add color channels, we will add them as white, to be neutral.
				// \todo: normal channels also should have special values.
				if (ResultChannel.Semantic == EMeshBufferSemantic::Color)
				{
					generated = true;

					switch (ResultChannel.Format)
					{
					case EMeshBufferFormat::Float32:
					{
						for (int32 v = 0; v < SourceElementCount; ++v)
						{
							float* pTypedResultBuf = (float*)ResultBuf;
							for (int32 i = 0; i < ResultComponents; ++i)
							{
								pTypedResultBuf[i] = 1.0f;
							}
							ResultBuf += ResultElemSize;
						}
						break;
					}

					case EMeshBufferFormat::NUInt8:
					{
						for (int32 v = 0; v < SourceElementCount; ++v)
						{
							uint8* pTypedResultBuf = (uint8*)ResultBuf;
							for (int32 i = 0; i < ResultComponents; ++i)
							{
								pTypedResultBuf[i] = 255;
							}
							ResultBuf += ResultElemSize;
						}
						break;
					}

					case EMeshBufferFormat::NUInt16:
					{
						for (int32 v = 0; v < SourceElementCount; ++v)
						{
							uint16* pTypedResultBuf = (uint16*)ResultBuf;
							for (int32 i = 0; i < ResultComponents; ++i)
							{
								pTypedResultBuf[i] = 65535;
							}
							ResultBuf += ResultElemSize;
						}
						break;
					}

					default:
						// Format not implemented
						check(false);
						break;
					}
				}

				if (!generated)
				{
					// TODO: and maybe raise a warning?
					for (int32 v = 0; v < SourceElementCount; ++v)
					{
						FMemory::Memzero(ResultBuf, ResultChannelSize);
						ResultBuf += ResultElemSize;
					}
				}

				continue;
			}


			// Case 3: Convert element by element
			//---------------------------------------------------------------------------------
			{
				// Get the data about the source format
				EMeshBufferSemantic SourceSemantic;
				int32 SourceSemanticIndex;
				EMeshBufferFormat SourceFormat;
				int32 SourceComponents;
				int32 SourceOffset;
				Source.GetChannel
				(
					SourceBufferIndex, SourceChannelIndex,
					&SourceSemantic, &SourceSemanticIndex,
					&SourceFormat, &SourceComponents,
					&SourceOffset
				);
				check(SourceSemantic == ResultChannel.Semantic
					&&
					SourceSemanticIndex == ResultChannel.SemanticIndex);

				int32 SourceElemSize = Source.GetElementSize(SourceBufferIndex);
				const uint8* SourceBuf = Source.GetBufferData(SourceBufferIndex);
				SourceBuf += SourceOffset;

				// Copy element by element
				if (ResultChannel.Format == SourceFormat && ResultComponents == SourceComponents)
				{
					for (int32 v = 0; v < SourceElementCount; ++v)
					{
						FMemory::Memcpy(ResultBuf, SourceBuf, ResultChannelSize);
						ResultBuf += ResultElemSize;
						SourceBuf += SourceElemSize;
					}
				}
				else if (ResultChannel.Format == SourceFormat && ResultComponents > SourceComponents)
				{
					const int32 ResultFormatSize = GetMeshFormatData(ResultChannel.Format).SizeInBytes;
					for (int32 v = 0; v < SourceElementCount; ++v)
					{
						FMemory::Memzero(ResultBuf, ResultFormatSize * ResultComponents);
						FMemory::Memcpy(ResultBuf, SourceBuf, ResultFormatSize * SourceComponents);
						ResultBuf += ResultElemSize;
						SourceBuf += SourceElemSize;
					}
				}
				else if (ResultChannel.Format == EMeshBufferFormat::PackedDir8_W_TangentSign
					||
					ResultChannel.Format == EMeshBufferFormat::PackedDirS8_W_TangentSign)
				{
					check(SourceComponents >= 3);
					check(ResultComponents == 4);

					// Look for the full tangent space if not done already
					int32 tanXBuf = -1, tanXChan = -1, tanYBuf = -1, tanYChan = -1, tanZBuf = -1, tanZChan = -1;
					Source.FindChannel(EMeshBufferSemantic::Tangent, ResultChannel.SemanticIndex, &tanXBuf, &tanXChan);
					Source.FindChannel(EMeshBufferSemantic::Binormal, ResultChannel.SemanticIndex, &tanYBuf, &tanYChan);
					Source.FindChannel(EMeshBufferSemantic::Normal, ResultChannel.SemanticIndex, &tanZBuf, &tanZChan);

					UntypedMeshBufferIteratorConst xIt(Source, EMeshBufferSemantic::Tangent, ResultChannel.SemanticIndex);
					UntypedMeshBufferIteratorConst yIt(Source, EMeshBufferSemantic::Binormal, ResultChannel.SemanticIndex);
					UntypedMeshBufferIteratorConst zIt(Source, EMeshBufferSemantic::Normal, ResultChannel.SemanticIndex);

					for (int32 v = 0; v < SourceElementCount; ++v)
					{
						// convert the 3 first components
						for (int32 i = 0; i < 3; ++i)
						{
							if (i < SourceComponents)
							{
								ConvertData
								(
									i,
									ResultBuf, ResultChannel.Format,
									SourceBuf, SourceFormat
								);
							}
						}

						// Add the tangent sign
						if (tanXBuf >= 0 && tanYBuf >= 0 && tanZBuf >= 0)
						{
							FMatrix44f Mat(xIt.GetAsVec3f(), yIt.GetAsVec3f(), zIt.GetAsVec3f(), FVector3f(0, 0, 0));

							if (ResultChannel.Format == EMeshBufferFormat::PackedDir8_W_TangentSign)
							{
								uint8* pData = reinterpret_cast<uint8*>(ResultBuf);
								uint8 Sign = Mat.RotDeterminant() < 0 ? 0 : 255;
								pData[3] = Sign;
							}
							else if (ResultChannel.Format == EMeshBufferFormat::PackedDirS8_W_TangentSign)
							{
								int8* pData = reinterpret_cast<int8*>(ResultBuf);
								int8 Sign = Mat.RotDeterminant() < 0 ? -128 : 127;
								pData[3] = Sign;
							}

							xIt++;
							yIt++;
							zIt++;
						}
						else
						{
							// At least initialize it to avoid garbage.
							int8* pData = reinterpret_cast<int8*>(ResultBuf);
							pData[3] = 0;
						}
						ResultBuf += ResultElemSize;
						SourceBuf += SourceElemSize;

					}
				}
				else
				{
					for (int32 v = 0; v < SourceElementCount; ++v)
					{
						// Convert formats
						for (int32 i = 0; i < ResultComponents; ++i)
						{
							if (i < SourceComponents)
							{
								ConvertData
								(
									i,
									ResultBuf, ResultChannel.Format,
									SourceBuf, SourceFormat
								);
							}
							else
							{
								// Add zeros. TODO: Warning?
								FMemory::Memzero
								(
									ResultBuf + GetMeshFormatData(ResultChannel.Format).SizeInBytes * i,
									GetMeshFormatData(ResultChannel.Format).SizeInBytes
								);
							}
						}


						// Extra step to normalise some semantics in some formats
						// TODO: Make it optional, and add different normalisation types n, n^2
						// TODO: Optimise
						if (SourceSemantic == EMeshBufferSemantic::BoneWeights)
						{
							if (ResultChannel.Format == EMeshBufferFormat::NUInt8)
							{
								uint8* Data = (uint8*)ResultBuf;
								uint8 Accum = 0;
								for (int32 i = 0; i < ResultComponents; ++i)
								{
									Accum += Data[i];
								}
								Data[0] += 255 - Accum;
							}

							else if (ResultChannel.Format == EMeshBufferFormat::NUInt16)
							{
								uint16* Data = (uint16*)ResultBuf;
								uint16 Accum = 0;
								for (int32 i = 0; i < ResultComponents; ++i)
								{
									Accum += Data[i];
								}
								Data[0] += 65535 - Accum;
							}
						}

						ResultBuf += ResultElemSize;
						SourceBuf += SourceElemSize;
					}
				}
			}
		}
	}


	//-------------------------------------------------------------------------------------------------
	void FormatBufferSet
	(
		const FMeshBufferSet& Source,
		FMeshBufferSet& Result,
		bool bKeepSystemBuffers,
		bool bIgnoreMissingChannels,
		bool bIsVertexBuffer, 
		uint32 IDPrefix = 0
	)
	{
		if (bIgnoreMissingChannels)
		{
			// Remove from the result the channels that are not present in the source, and re-pack the
			// offsets.
			for (int32 b = 0; b < Result.GetBufferCount(); ++b)
			{
				TArray<EMeshBufferSemantic> resultSemantics;
				TArray<int32> resultSemanticIndexs;
				TArray<EMeshBufferFormat> resultFormats;
				TArray<int32> resultComponentss;
				TArray<int32> resultOffsets;
				int32 offset = 0;

				// For every channel in this buffer
				for (int32 c = 0; c < Result.GetBufferChannelCount(b); ++c)
				{
					EMeshBufferSemantic resultSemantic;
					int32 resultSemanticIndex;
					EMeshBufferFormat resultFormat;
					int32 resultComponents;

					// Find this channel in the source mesh
					Result.GetChannel
					(
						b, c,
						&resultSemantic, &resultSemanticIndex,
						&resultFormat, &resultComponents,
						nullptr
					);

					int32 sourceBuffer;
					int32 sourceChannel;
					Source.FindChannel
					(resultSemantic, resultSemanticIndex, &sourceBuffer, &sourceChannel);

					if (sourceBuffer >= 0)
					{
						resultSemantics.Add(resultSemantic);
						resultSemanticIndexs.Add(resultSemanticIndex);
						resultFormats.Add(resultFormat);
						resultComponentss.Add(resultComponents);
						resultOffsets.Add(offset);

						offset += GetMeshFormatData(resultFormat).SizeInBytes * resultComponents;
					}
				}

				if (resultSemantics.IsEmpty())
				{
					Result.SetBuffer(b, 0, 0, nullptr, nullptr, nullptr, nullptr, nullptr);
				}
				else
				{
					Result.SetBuffer(b, offset, resultSemantics.Num(),
						&resultSemantics[0],
						&resultSemanticIndexs[0],
						&resultFormats[0],
						&resultComponentss[0],
						&resultOffsets[0]);
				}
			}
		}

		if (Source.IsDescriptor())
		{
			EnumAddFlags(Result.Flags, EMeshBufferSetFlags::IsDescriptor); 
		}

		int32 VertexCount = Source.GetElementCount();
		Result.SetElementCount(VertexCount);

		if (!Source.IsDescriptor())
		{
			// For every vertex buffer in result
			for (int32 BufferIndex = 0; BufferIndex < Result.GetBufferCount(); ++BufferIndex)
			{
				MeshFormatBuffer(Source, Result.Buffers[BufferIndex], 0, bIsVertexBuffer, IDPrefix);
			}
		}

		// Detect internal system buffers and clone them unmodified.
		if (bKeepSystemBuffers)
		{
			for (int32 b = 0; b < Source.GetBufferCount(); ++b)
			{
				// Detect system buffers and clone them unmodified.
				if (Source.GetBufferChannelCount(b) == 1)
				{
					EMeshBufferSemantic sourceSemantic;
					int32 sourceSemanticIndex;
					EMeshBufferFormat sourceFormat;
					int32 sourceComponents;
					int32 sourceOffset;
					Source.GetChannel
					(
						b, 0,
						&sourceSemantic, &sourceSemanticIndex,
						&sourceFormat, &sourceComponents,
						&sourceOffset
					);

					if (sourceSemantic == EMeshBufferSemantic::LayoutBlock
						||
						(bIsVertexBuffer && sourceSemantic == EMeshBufferSemantic::VertexIndex))
					{
						// Add it if it wasn't already there, which could happen if it was included in the format mesh.
						int32 AlreadyExistingBufIndex = -1;
						int32 AlreadyExistingChannelIndex = -1;
						Result.FindChannel(sourceSemantic, sourceSemanticIndex, &AlreadyExistingBufIndex, &AlreadyExistingChannelIndex);
						if (AlreadyExistingBufIndex==-1)
						{
							Result.AddBuffer(Source, b);
						}
						else
						{
							// Replace the buffer
							check(Result.Buffers[AlreadyExistingBufIndex].Channels.Num()==1);
							Result.Buffers[AlreadyExistingBufIndex] = Source.Buffers[b];
						}
					}
				}
			}
		}

	}

	int32 FindHighestBoneIndexInUse(const FMesh* InMesh, const FMeshBufferChannel& Channel)
	{
		const FMeshBufferSet& VertexBuffers = InMesh->GetVertexBuffers();

		// If InMesh buffers are descriptors, use the bone map to get an upper bound of the highest index
		// we will find in the mesh. Otherwise, find it from the vertex buffers data.
		if (VertexBuffers.IsDescriptor())
		{
			return FMath::Max(0, InMesh->BoneMap.Num() - 1);
		}

		const UntypedMeshBufferIteratorConst BoneIndicesBegin(
				VertexBuffers, EMeshBufferSemantic::BoneIndices, Channel.SemanticIndex);

		if (!BoneIndicesBegin.ptr())
		{
			return 0;
		}

		const int32 NumVertices = VertexBuffers.GetElementCount();
		const int32 NumInfluences = BoneIndicesBegin.GetComponents();
		
		int32 HighestBoneIndex = 0;
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			// If MAX_TOTAL_INFLUENCES ever changed, the next line would no longer work or compile and 
			// GetAsVec12i would need to be changed accordingly
			int32 VertexInfluences[MAX_TOTAL_INFLUENCES];
			(BoneIndicesBegin + VertexIndex).GetAsInt32Vec(VertexInfluences, MAX_TOTAL_INFLUENCES);
			
			for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
			{
				HighestBoneIndex = FMath::Max(HighestBoneIndex, VertexInfluences[InfluenceIndex]);
			}
		}

		return HighestBoneIndex;
	}

	/** Updates the InOutFormat bone index semantic format if it cannot represent the highest bone index in use. */
	void UpdateBoneIndexFormatIfNeeded(const FMesh* InMesh, FMeshBufferSet& InOutFormatBufferSet)
	{
		const FMeshBufferSet& VertexBuffers = InMesh->GetVertexBuffers();

		const int32 BufferCount = VertexBuffers.GetBufferCount();
		for (int32 BufferIndex = 0; BufferIndex < BufferCount; ++BufferIndex)
		{
			const int32 ChannelCount = VertexBuffers.GetBufferChannelCount(BufferIndex);
			for (int32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
			{
				const FMeshBufferChannel& Channel = VertexBuffers.Buffers[BufferIndex].Channels[ChannelIndex];

				if (Channel.Semantic == EMeshBufferSemantic::BoneIndices)
				{
					int32 ResultBuf = 0;
					int32 ResultChan = 0;
					InOutFormatBufferSet.FindChannel(EMeshBufferSemantic::BoneIndices, Channel.SemanticIndex, &ResultBuf, &ResultChan);
					if (ResultBuf >= 0)
					{
						int32 HighestBoneIndexInUse = FindHighestBoneIndexInUse(InMesh, Channel);
						check(HighestBoneIndexInUse >= 0);

						EMeshBufferFormat& BoneIndexFormat = InOutFormatBufferSet.Buffers[ResultBuf].Channels[ResultChan].Format;
						if (HighestBoneIndexInUse > 0xffff && (BoneIndexFormat == EMeshBufferFormat::UInt8 || BoneIndexFormat == EMeshBufferFormat::UInt16))
						{
							BoneIndexFormat = EMeshBufferFormat::UInt32;
							InOutFormatBufferSet.UpdateOffsets(ResultBuf);
						}
						else if (HighestBoneIndexInUse > 0x7fff && (BoneIndexFormat == EMeshBufferFormat::UInt8 || BoneIndexFormat == EMeshBufferFormat::UInt16))
						{
							BoneIndexFormat = EMeshBufferFormat::UInt32;
							InOutFormatBufferSet.UpdateOffsets(ResultBuf);
						}
						else if (HighestBoneIndexInUse > 0xff && BoneIndexFormat == EMeshBufferFormat::UInt8)
						{
							BoneIndexFormat = EMeshBufferFormat::UInt16;
							InOutFormatBufferSet.UpdateOffsets(ResultBuf);
						}
					}
				}
			}
		}
	}

	//-------------------------------------------------------------------------------------------------
	void MeshFormat
	(
		FMesh* Result, 
		const FMesh* Source,
		const FMesh* Format,
		bool bKeepSystemBuffers,
		bool bFormatVertices,
		bool bFormatIndices,
		bool bIgnoreMissingChannels,
		bool& bOutSuccess
	)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshFormat);
		bOutSuccess = true;

		if (!Source) 
		{
			check(false);
			bOutSuccess = false;	
			return;
		}

		if (!Format)
		{
			check(false);
			bOutSuccess = false;
			return;
		}

		EMeshCopyFlags CopyFlags = EMeshCopyFlags::AllFlags;
		EnumRemoveFlags(CopyFlags, EMeshCopyFlags::WithVertexBuffers | EMeshCopyFlags::WithIndexBuffers | EMeshCopyFlags::WithLayouts);

		Result->CopyFrom(*Source, CopyFlags);

		if (bFormatVertices)
		{
			Result->VertexBuffers = Format->VertexBuffers;

			UpdateBoneIndexFormatIfNeeded(Source, Result->GetVertexBuffers());

			FormatBufferSet(Source->GetVertexBuffers(), Result->GetVertexBuffers(),
				bKeepSystemBuffers, bIgnoreMissingChannels, true, Source->MeshIDPrefix);
		}
		else
		{
			Result->VertexBuffers = Source->GetVertexBuffers();
		}

		if (bFormatIndices)
		{
			Result->IndexBuffers = Format->IndexBuffers;

			// \todo Make sure that the vertex indices will fit in this format, or extend it.
			FormatBufferSet(Source->GetIndexBuffers(), Result->GetIndexBuffers(), bKeepSystemBuffers,
				bIgnoreMissingChannels, false);
		}
		else
		{
			Result->IndexBuffers = Source->GetIndexBuffers();
		}

		Result->Layouts.Reset(Source->Layouts.Num());
		for (const TManagedPtr<const FLayout>& Layout : Source->Layouts)
		{
			Result->Layouts.Add(Layout->Clone());
		}

		Result->ResetStaticFormatFlags();
		Result->EnsureSurfaceData();
	}


	void MeshOptimizeBuffers( FMesh* InMesh )
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshOptimizeBuffers);

		if (!InMesh)
		{
			return;
		}

		FMeshBufferSet& VertexBuffers = InMesh->VertexBuffers;

		// Ignore the operation if the MeshBuffersSet is a descriptor.
		if (VertexBuffers.IsDescriptor())
		{
			return;
		}

		// Reduce the number of influences if possible
		constexpr int32 SemanticIndex = 0;
		
		UntypedMeshBufferIteratorConst WeightIt(VertexBuffers, EMeshBufferSemantic::BoneWeights, SemanticIndex);
		if (WeightIt.ptr())
		{
			int32 BufferInfluences = WeightIt.GetComponents();
			int32 RealInfluences = 0;

			for (int32 VertexIndex = 0; VertexIndex < VertexBuffers.GetElementCount(); ++VertexIndex)
			{
				int32 ThisVertexInfluences = 0;

				switch (WeightIt.GetFormat())
				{
				case EMeshBufferFormat::NUInt8:
				{
					const uint8* Data = reinterpret_cast<const uint8*>(WeightIt.ptr());
					for (int32 InfluenceIndex = 0; InfluenceIndex < BufferInfluences; ++InfluenceIndex)
					{
						if (*Data>0)
						{
							++ThisVertexInfluences;
						}
						++Data;
					}
					break;
				}
				case EMeshBufferFormat::NUInt16:
				{
					const uint16* Data = reinterpret_cast<const uint16*>(WeightIt.ptr());
					for (int32 InfluenceIndex = 0; InfluenceIndex < BufferInfluences; ++InfluenceIndex)
					{
						if (*Data > 0)
						{
							++ThisVertexInfluences;
						}
						++Data;
					}
					break;
				}

				default:
					// Unsupported
					check(false);
					break;
				}

				++WeightIt;

				RealInfluences = FMath::Max(RealInfluences,ThisVertexInfluences);
			}

			if (RealInfluences<BufferInfluences)
			{
				// Remove the useless influences from the buffer.

				// \todo: This is a generic innefficient way
				FMeshBufferSet NewVertexBuffers;
				NewVertexBuffers.Buffers = VertexBuffers.Buffers;

				for (FMeshBuffer& Buffer: NewVertexBuffers.Buffers)
				{
					int32 OffsetDelta = 0;
					for (FMeshBufferChannel& Channel : Buffer.Channels)
					{
						Channel.Offset += OffsetDelta;

						if (Channel.SemanticIndex == SemanticIndex
							&&
							(Channel.Semantic == EMeshBufferSemantic::BoneWeights || Channel.Semantic == EMeshBufferSemantic::BoneIndices)
							)
						{
							Channel.ComponentCount = RealInfluences;
							OffsetDelta -= (BufferInfluences - RealInfluences) * GetMeshFormatData(Channel.Format).SizeInBytes;
						}
					}

					Buffer.ElementSize += OffsetDelta;
				}

				FormatBufferSet( VertexBuffers, NewVertexBuffers, true, false, true);

				InMesh->VertexBuffers = NewVertexBuffers;
			}
		}

	}
}
