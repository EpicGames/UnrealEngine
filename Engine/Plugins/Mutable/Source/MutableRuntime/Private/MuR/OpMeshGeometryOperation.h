// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshPrivate.h"
#include "MuR/ConvertData.h"
#include "MuR/MutableTrace.h"
#include "MuR/Platform.h"
#include "DynamicMesh/DynamicMesh3.h"

// This requires a separate plugin.
//#include "SpaceDeformerOps/TwistMeshOp.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	//! Conversion from Mutable mesh to DynamicMesh3.
	//! It only converts position and normal data.
	//! Reference version, that can be optimized with some assumptions on how Vertex IDs are generated.
	//---------------------------------------------------------------------------------------------
	inline TSharedPtr<UE::Geometry::FDynamicMesh3> MutableToDynamicMesh(const FMesh* MutableMesh, TArray<int>& VertexMutableToDyn)
	{
		if (!MutableMesh)
		{
			return nullptr;
		}

		// Create the FDynamicMesh
		TSharedPtr<UE::Geometry::FDynamicMesh3> DynMesh = MakeShared<UE::Geometry::FDynamicMesh3>();

		int32 VertexCount = MutableMesh->GetVertexCount();

		TArray<int> TriangleMutableToDyn;
		int TriangleCount = MutableMesh->GetFaceCount();
		{
			MUTABLE_CPUPROFILER_SCOPE(MutableToDynamicMesh);

			VertexMutableToDyn.SetNum(VertexCount);
			TriangleMutableToDyn.SetNum(TriangleCount);

			// \TODO: Simple but inefficient
			UntypedMeshBufferIteratorConst ItPosition(MutableMesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
			UntypedMeshBufferIteratorConst ItNormal(MutableMesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);
			for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
			{
				FVector3d Position = ItPosition.GetAsVec3d();
				++ItPosition;

				FVector3f Normal = ItNormal.GetAsVec3f();
				++ItNormal;

				int DynId = DynMesh->AppendVertex(UE::Geometry::FVertexInfo(Position, Normal));
				VertexMutableToDyn[VertexIndex] = DynId;
			}

			UntypedMeshBufferIteratorConst ItIndices(MutableMesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
			for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
			{
				UE::Geometry::FIndex3i Triangle;
				Triangle.A = int(ItIndices.GetAsUINT32());
				++ItIndices;
				Triangle.B = int(ItIndices.GetAsUINT32());
				++ItIndices;
				Triangle.C = int(ItIndices.GetAsUINT32());
				++ItIndices;

				int DynId = DynMesh->AppendTriangle(Triangle);
				TriangleMutableToDyn[TriangleIndex] = DynId;
			}
		}

		return DynMesh;
	}


	//---------------------------------------------------------------------------------------------
	//! Apply an example geometric operation (twist)
	//---------------------------------------------------------------------------------------------
	inline void UpdateMutableMesh(FMesh* MutableMesh, const UE::Geometry::FDynamicMesh3* DynMesh, const TArray<int>& VertexMutableToDyn)
	{
		MUTABLE_CPUPROFILER_SCOPE(DynamicMeshToMutable);

		// \TODO: Simple but inefficient
		UntypedMeshBufferIterator ItPosition(MutableMesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
		UntypedMeshBufferIterator ItNormal(MutableMesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);
		
		int32 VertexCount = MutableMesh->GetVertexCount();
		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			int DynId = VertexMutableToDyn[VertexIndex];

			FVector3d Position = DynMesh->GetVertex(DynId);
			ItPosition.SetFromVec3d(Position);
			++ItPosition;

			FVector3f Normal = DynMesh->GetVertexNormal(DynId);
			ItNormal.SetFromVec3f(Normal);
			++ItNormal;
		}
	}


    //---------------------------------------------------------------------------------------------
    //! Apply an example geometric operation (twist)
    //---------------------------------------------------------------------------------------------
    inline void MeshGeometryOperation(FMesh* Result, const FMesh* MeshA, const FMesh* MeshB, float ScalarA, float ScalarB, bool& bOutSuccess)
    {
		MUTABLE_CPUPROFILER_SCOPE(MeshGeometryOperation);
		bOutSuccess = true;

		if (!MeshA)
		{
			bOutSuccess = false;
			return;
		}

        // Generate the FDynamicMesh
		TArray<int> VertexMutableToDyn;
		TSharedPtr<UE::Geometry::FDynamicMesh3> DynMesh = MutableToDynamicMesh(MeshA, VertexMutableToDyn);

		// Apply the geometry operation
		// This is just an example, but it happens to depend on a separate plugin.
		// Do nothing for now.
		
		//TUniquePtr<UE::Geometry::FDynamicMesh3> DynMeshResult;
		//{
		//	MUTABLE_CPUPROFILER_SCOPE(ApplyOperation);

		//	UE::Geometry::FTwistMeshOp TwistOp;
		//	TwistOp.OriginalMesh = DynMesh;
		//	TwistOp.GizmoFrame = UE::Geometry::FFrame3d();
		//	TwistOp.LowerBoundsInterval = -ScalarA;
		//	TwistOp.UpperBoundsInterval = ScalarA;
		//	TwistOp.TwistDegrees = ScalarB;
		//	TwistOp.bLockBottom = true;
		//	TwistOp.CalculateResult(nullptr);

		//	DynMeshResult = TwistOp.ExtractResult();
		//}

		// Update the Mutable mesh from the FDynamicMesh
		Result->CopyFrom(*MeshA);
		//UpdateMutableMesh(Result, DynMeshResult.Get(), VertexMutableToDyn);
    }

}

