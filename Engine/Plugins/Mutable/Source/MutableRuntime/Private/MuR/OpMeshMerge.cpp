// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshMerge.h"

#include "MuR/MeshPrivate.h"
#include "MuR/MeshBufferUtils.h"
#include "MuR/Platform.h"
#include "MuR/MutableMath.h"
#include "MuR/OpMeshFormat.h"
#include "MuR/MutableTrace.h"
#include "MuR/MutableRuntimeModule.h"
#include "MuR/ParallelExecutionUtils.h"

#include "Engine/SkeletalMesh.h"
#include "Rendering/MorphTargetVertexCodec.h"

#include "UObject/StrongObjectPtr.h"

#include "Algo/Count.h"

namespace UE::Mutable::Private
{
namespace OpMeshMergeMorphsInternal
{
	uint32 ComputeMorphAllocationRequirementsInDwords(TConstArrayView<uint32> MorphHeaderData)
	{
		using namespace UE::MorphTargetVertexCodec;
		
		const int32 NumBatches = MorphHeaderData.Num() / NumBatchHeaderDwords;

		if (NumBatches <= 0)
		{
			return 0;
		}

		// First Dword in the header data correspond to the Offset in Bytes. See MorphTargetVertexCodec::WriteHeader()
		const uint32 DataOffsetFirstBatchInDwords = MorphHeaderData[0] / sizeof(uint32);
		const uint32 DataOffsetLastBatchInDwords  = MorphHeaderData[(NumBatches - 1) * NumBatchHeaderDwords] / sizeof(uint32);

		// We need the last header to compute its size, this could be approximated by an upper bound.
		TConstArrayView<uint32> LastHeaderData(
				&MorphHeaderData[(NumBatches - 1) * NumBatchHeaderDwords], 
				MorphTargetVertexCodec::NumBatchHeaderDwords);
			
		FDeltaBatchHeader LastBatchHeader;
		UE::MorphTargetVertexCodec::ReadHeader(LastBatchHeader, LastHeaderData);

		check(LastBatchHeader.DataOffset == DataOffsetLastBatchInDwords * sizeof(uint32));

		return CalculateBatchDwords(LastBatchHeader) + DataOffsetLastBatchInDwords - DataOffsetFirstBatchInDwords + MorphHeaderData.Num();
	}

	void GetMorphDataOffsetAndSize(TConstArrayView<uint32> MorphHeaderData, uint32& OutDataOffset, uint32& OutDataSize)
	{
		using namespace UE::MorphTargetVertexCodec;
		
		const int32 NumBatches = MorphHeaderData.Num() / NumBatchHeaderDwords;
		if (NumBatches <= 0)
		{
			OutDataOffset = 0;
			OutDataSize = 0;
			return;
		}

		// We need the last header to compute its size, this could be approximated by an upper bound.
		TConstArrayView<uint32> LastHeaderData(
			&MorphHeaderData[(NumBatches - 1) * NumBatchHeaderDwords],
			MorphTargetVertexCodec::NumBatchHeaderDwords);

		FDeltaBatchHeader LastBatchHeader;
		UE::MorphTargetVertexCodec::ReadHeader(LastBatchHeader, LastHeaderData);

		// First Dword in the header data correspond to the Offset in Bytes. See MorphTargetVertexCodec::WriteHeader()
		const uint32 DataOffsetFirstBatchInDwords = MorphHeaderData[0] / sizeof(uint32);
		const uint32 DataOffsetLastBatchInDwords = MorphHeaderData[(NumBatches - 1) * NumBatchHeaderDwords] / sizeof(uint32);
		check(LastBatchHeader.DataOffset == DataOffsetLastBatchInDwords * sizeof(uint32));

		OutDataOffset = DataOffsetFirstBatchInDwords;
		OutDataSize = CalculateBatchDwords(LastBatchHeader) + DataOffsetLastBatchInDwords - DataOffsetFirstBatchInDwords;
	}

	void FixMorphIndicesSurfaceMetadataAndApplyFilter(FMesh& Result, int32 VertexIndexOffset, int32 SurfacesOffset, EMorphUsageFlags UsageFilter)
	{
        MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_Merge_FixMorphIndices)

		using namespace UE::MorphTargetVertexCodec;

		if (SurfacesOffset != 0)
		{
			for (TArrayView<int32> SurfacesInUse : Result.Morph.SurfacesInUsePerMorph)
			{
				for (int32& SurfaceIndex : SurfacesInUse)
				{
					SurfaceIndex += SurfacesOffset;
				}
			}
		}

		int32 NumMorphsToRemove = Algo::CountIf(Result.Morph.UsageFlagsPerMorph, 
				[UsageFilter](EMorphUsageFlags Flags) { return !EnumHasAnyFlags(Flags, UsageFilter); });

		int32 NumMorphs = Result.Morph.Names.Num();

		if (NumMorphsToRemove != 0)
		{
			// Remove batches that don't pass the filter keeping original offsets for the next pass.
			Result.Morph.NumTotalBatches = 0;
			uint32 AccumulatedDataOffsetInDwords = 0;
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				if (EnumHasAnyFlags(Result.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
				{
					int32 NumBatches = Result.Morph.BatchesPerMorph[MorphIndex];
					Result.Morph.BatchStartOffsetPerMorph[MorphIndex] = Result.Morph.NumTotalBatches;
					Result.Morph.NumTotalBatches += NumBatches;
				
					if (Result.MorphDataBuffer.Num())
					{
						FMemory::Memmove(
								Result.MorphDataBuffer.GetData() + AccumulatedDataOffsetInDwords,
								Result.MorphDataBuffer.GetData() + Result.Morph.BatchStartOffsetPerMorph[MorphIndex] * NumBatchHeaderDwords,
								NumBatches * NumBatchHeaderDwords * sizeof(uint32));

						
						AccumulatedDataOffsetInDwords += NumBatches * NumBatchHeaderDwords;
					}
				}
				else
				{
					Result.Morph.BatchesPerMorph[MorphIndex] = 0;
					Result.Morph.BatchStartOffsetPerMorph[MorphIndex] = Result.Morph.NumTotalBatches;
				}
			}

			if (Result.MorphDataBuffer.Num())
			{
				// Move the data of the remaining batches and patch the offsets.
				for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
				{
					int32 NumBatches = Result.Morph.BatchesPerMorph[MorphIndex];

					if (NumBatches == 0)
					{
						continue;
					}

					int32 BatchStartOffset = Result.Morph.BatchStartOffsetPerMorph[MorphIndex];

					int32 MorphDeltasDataBeginInBytes = Result.MorphDataBuffer[BatchStartOffset * NumBatchHeaderDwords];
					int32 OldToNewDeltaOffsetInBytes = MorphDeltasDataBeginInBytes - (AccumulatedDataOffsetInDwords * sizeof(uint32));

					check(OldToNewDeltaOffsetInBytes >= 0);

					for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
					{
						Result.MorphDataBuffer[(BatchStartOffset + BatchIndex) * NumBatchHeaderDwords] -= OldToNewDeltaOffsetInBytes; 
					}
					
					int32 LastBatchDataOffsetInBytes = Result.MorphDataBuffer[(BatchStartOffset + NumBatches - 1) * NumBatchHeaderDwords];

					TArrayView<uint32> LastHeaderDataView = TArrayView<uint32>(
							&Result.MorphDataBuffer[(BatchStartOffset + NumBatches - 1) * NumBatchHeaderDwords],
							NumBatchHeaderDwords);

					FDeltaBatchHeader LastBatchHeader;
					UE::MorphTargetVertexCodec::ReadHeader(LastBatchHeader, LastHeaderDataView);

					int32 MorphDeltaDataSizeInBytes = 
						(LastBatchDataOffsetInBytes + CalculateBatchDwords(LastBatchHeader) * sizeof(uint32)) -
						(MorphDeltasDataBeginInBytes - OldToNewDeltaOffsetInBytes);

					FMemory::Memmove(
							Result.MorphDataBuffer.GetData() + AccumulatedDataOffsetInDwords,
							Result.MorphDataBuffer.GetData() + (MorphDeltasDataBeginInBytes / sizeof(uint32)),
							MorphDeltaDataSizeInBytes);	

					AccumulatedDataOffsetInDwords += MorphDeltaDataSizeInBytes / sizeof(uint32);
				}
				// Free unused data.
				Result.MorphDataBuffer.SetNum(AccumulatedDataOffsetInDwords, EAllowShrinking::Yes);
			}
		}	
		
		if (VertexIndexOffset != 0 && Result.MorphDataBuffer.Num())
		{
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				TArrayView<uint32> MorphHeadersDataView = TArrayView<uint32>(
						Result.MorphDataBuffer.GetData() + Result.Morph.BatchStartOffsetPerMorph[MorphIndex]*NumBatchHeaderDwords,
						Result.Morph.BatchesPerMorph[MorphIndex]*NumBatchHeaderDwords);

				int32 NumBatches = Result.Morph.BatchesPerMorph[MorphIndex];
				for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
				{
					TArrayView<uint32> HeaderDataView = TArrayView<uint32>(
							MorphHeadersDataView.GetData() + BatchIndex*NumBatchHeaderDwords,
							NumBatchHeaderDwords);

					FDeltaBatchHeader BatchHeader;
					UE::MorphTargetVertexCodec::ReadHeader(BatchHeader, HeaderDataView);

					BatchHeader.IndexMin += VertexIndexOffset;

					UE::MorphTargetVertexCodec::WriteHeader(BatchHeader, HeaderDataView);
				}
			}
		}
	}

	void MergeMorphs(FMesh& Result, const FMesh& MeshA, const FMesh& MeshB, bool bMergeSurfaces, EMorphUsageFlags UsageFilter)
	{
        MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_Merge_Merge)

		using namespace UE::MorphTargetVertexCodec;
		
		// Compute memory requirements.	
		Result.Morph.Names.Reserve(MeshA.Morph.Names.Num() + MeshB.Morph.Names.Num());
		Result.Morph.Names.Append(MeshA.Morph.Names);

		TArray<int32, TInlineAllocator<128>> ResultToBMorphNameMap; 
		ResultToBMorphNameMap.Init(INDEX_NONE, MeshB.Morph.Names.Num() + MeshA.Morph.Names.Num());

		// MeshA morphs map is the identity, compute only the MorphB morphs map.
		for (int32 Index = 0; Index < MeshB.Morph.Names.Num(); ++Index)
		{
			int32 MappedIndex = Result.Morph.Names.AddUnique(MeshB.Morph.Names[Index]);
			ResultToBMorphNameMap[MappedIndex] = Index;
		}

		const int32 MeshANumMorphs = MeshA.Morph.Names.Num();
		const int32 ResultNumMorphs = Result.Morph.Names.Num();

		Result.Morph.MinimumValuePerMorph.SetNumUninitialized(ResultNumMorphs);
		Result.Morph.MaximumValuePerMorph.SetNumUninitialized(ResultNumMorphs);
		Result.Morph.BatchesPerMorph.SetNumZeroed(ResultNumMorphs);
		Result.Morph.BatchStartOffsetPerMorph.SetNum(ResultNumMorphs);
		Result.Morph.SurfacesInUsePerMorph.SetNum(ResultNumMorphs);
		Result.Morph.UsageFlagsPerMorph.Init(EMorphUsageFlags::None, ResultNumMorphs);

		// Surface in use and usages.
		{
			int32 MeshBSurfaceOffset = bMergeSurfaces ? 0 : MeshA.Surfaces.Num();
			int32 MorphIndex = 0;
			for (; MorphIndex < MeshANumMorphs; ++MorphIndex)
			{
				if (EnumHasAnyFlags(MeshA.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
				{
					Result.Morph.SurfacesInUsePerMorph[MorphIndex] = MeshA.Morph.SurfacesInUsePerMorph[MorphIndex];
					EnumAddFlags(Result.Morph.UsageFlagsPerMorph[MorphIndex], MeshA.Morph.UsageFlagsPerMorph[MorphIndex]);
				}

				int32 MorphIndexInB = ResultToBMorphNameMap[MorphIndex];
				if (MorphIndexInB != INDEX_NONE && EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MorphIndexInB], UsageFilter))
				{
					EnumAddFlags(Result.Morph.UsageFlagsPerMorph[MorphIndex], MeshB.Morph.UsageFlagsPerMorph[MorphIndexInB]);

					for (int32 SurfaceInUse : MeshB.Morph.SurfacesInUsePerMorph[MorphIndexInB])
					{
						SurfaceInUse += MeshBSurfaceOffset;
						Result.Morph.SurfacesInUsePerMorph[MorphIndex].AddUnique(SurfaceInUse);
					}
				}
			}

			for (; MorphIndex < ResultNumMorphs; ++MorphIndex)
			{
				int32 MorphIndexInB = ResultToBMorphNameMap[MorphIndex];
			
				if (EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MorphIndexInB], UsageFilter))
				{
					check(MorphIndexInB != INDEX_NONE);

					Result.Morph.SurfacesInUsePerMorph[MorphIndex] = MeshB.Morph.SurfacesInUsePerMorph[MorphIndexInB];
					EnumAddFlags(Result.Morph.UsageFlagsPerMorph[MorphIndex], MeshB.Morph.UsageFlagsPerMorph[MorphIndexInB]);

					if (MeshBSurfaceOffset != 0)
					{
						for (int32& SurfaceInUse : Result.Morph.SurfacesInUsePerMorph[MorphIndex])
						{
							SurfaceInUse += MeshBSurfaceOffset;
						}
					}
				}
			}
		}

		// Compute memory allocation requirements and NumBatches. 
		{
			uint32 MergedDataBufferSizeInDwords = 0; 
			Result.Morph.NumTotalBatches =  0;
			for (int32 MorphIndex = MeshA.Morph.Names.Num() - 1; MorphIndex >= 0; --MorphIndex)
			{	
				if (EnumHasAnyFlags(MeshA.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
				{
					int32 NumBatches = MeshA.Morph.BatchesPerMorph[MorphIndex];
					Result.Morph.NumTotalBatches += NumBatches;

					if (MeshA.MorphDataBuffer.Num())
					{
						TConstArrayView<uint32> HeadersDataView(
								MeshA.MorphDataBuffer.GetData() + MeshA.Morph.BatchStartOffsetPerMorph[MorphIndex]*NumBatchHeaderDwords,
								NumBatchHeaderDwords*NumBatches);

						MergedDataBufferSizeInDwords += ComputeMorphAllocationRequirementsInDwords(HeadersDataView);
					}
				}
			}

			for (int32 MorphIndex = MeshB.Morph.Names.Num() - 1; MorphIndex >= 0; --MorphIndex)
			{
				if (EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
				{
					int32 NumBatches = MeshB.Morph.BatchesPerMorph[MorphIndex];
					Result.Morph.NumTotalBatches += NumBatches;

					if (MeshB.MorphDataBuffer.Num())
					{
						TConstArrayView<uint32> HeadersDataView(
								MeshB.MorphDataBuffer.GetData() + MeshB.Morph.BatchStartOffsetPerMorph[MorphIndex]*NumBatchHeaderDwords,
								NumBatchHeaderDwords*NumBatches);

						MergedDataBufferSizeInDwords += ComputeMorphAllocationRequirementsInDwords(HeadersDataView);
					}
				}
			}

			Result.MorphDataBuffer.SetNumUninitialized(MergedDataBufferSizeInDwords);
		}

		Result.Morph.PositionPrecision = MeshA.Morph.PositionPrecision;
		Result.Morph.TangentZPrecision = MeshA.Morph.TangentZPrecision;

		// Merge headers.
		int32 BatchAccumulatedOffsetInDwords = 0;
		int32 MorphIndex = 0;
		for (; MorphIndex < MeshANumMorphs; ++MorphIndex)
		{
			Result.Morph.MaximumValuePerMorph[MorphIndex] = MeshA.Morph.MaximumValuePerMorph[MorphIndex];
			Result.Morph.MinimumValuePerMorph[MorphIndex] = MeshA.Morph.MinimumValuePerMorph[MorphIndex];
			Result.Morph.BatchStartOffsetPerMorph[MorphIndex] = BatchAccumulatedOffsetInDwords / NumBatchHeaderDwords;	
			Result.Morph.BatchesPerMorph[MorphIndex] = 0;

			if (EnumHasAnyFlags(MeshA.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
			{
				Result.Morph.BatchesPerMorph[MorphIndex] = MeshA.Morph.BatchesPerMorph[MorphIndex];
				if (Result.MorphDataBuffer.Num())
				{
					FMemory::Memcpy(
							Result.MorphDataBuffer.GetData() + BatchAccumulatedOffsetInDwords, 
							MeshA.MorphDataBuffer.GetData() + MeshA.Morph.BatchStartOffsetPerMorph[MorphIndex]*NumBatchHeaderDwords,
							MeshA.Morph.BatchesPerMorph[MorphIndex] * NumBatchHeaderDwords*sizeof(uint32));
					
					BatchAccumulatedOffsetInDwords += MeshA.Morph.BatchesPerMorph[MorphIndex] * NumBatchHeaderDwords;
				}
			}

			int32 MeshBMorphIndex = ResultToBMorphNameMap[MorphIndex];
			if (MeshBMorphIndex >= 0)
			{
				if (EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MeshBMorphIndex], UsageFilter))
				{
					Result.Morph.BatchesPerMorph[MorphIndex] += MeshB.Morph.BatchesPerMorph[MeshBMorphIndex];

					const FVector4f& SourceMaxValue = MeshB.Morph.MaximumValuePerMorph[MeshBMorphIndex];
					const FVector4f& SourceMinValue = MeshB.Morph.MinimumValuePerMorph[MeshBMorphIndex];

					Result.Morph.MaximumValuePerMorph[MorphIndex] = 
							SourceMaxValue.ComponentMax(Result.Morph.MaximumValuePerMorph[MorphIndex]);
					Result.Morph.MinimumValuePerMorph[MorphIndex] = 
							SourceMinValue.ComponentMin(Result.Morph.MinimumValuePerMorph[MorphIndex]);

					if (Result.MorphDataBuffer.Num())
					{
						FMemory::Memcpy(
								Result.MorphDataBuffer.GetData() + BatchAccumulatedOffsetInDwords, 
								MeshB.MorphDataBuffer.GetData() + MeshB.Morph.BatchStartOffsetPerMorph[MeshBMorphIndex]*NumBatchHeaderDwords,
								MeshB.Morph.BatchesPerMorph[MeshBMorphIndex] * NumBatchHeaderDwords*sizeof(uint32));
					
						BatchAccumulatedOffsetInDwords += MeshB.Morph.BatchesPerMorph[MeshBMorphIndex] * NumBatchHeaderDwords;
					}
				}
			}
		}

		for (; MorphIndex < ResultNumMorphs; ++MorphIndex)
		{
			int32 MeshBMorphIndex = ResultToBMorphNameMap[MorphIndex];
			
			Result.Morph.BatchesPerMorph[MorphIndex] = 0;
			Result.Morph.BatchStartOffsetPerMorph[MorphIndex] = 0;

			if (EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MeshBMorphIndex], UsageFilter))
			{
				Result.Morph.BatchesPerMorph[MorphIndex] = MeshB.Morph.BatchesPerMorph[MeshBMorphIndex]; 
				Result.Morph.BatchStartOffsetPerMorph[MorphIndex] = BatchAccumulatedOffsetInDwords / NumBatchHeaderDwords;

				Result.Morph.MaximumValuePerMorph[MorphIndex] = MeshB.Morph.MaximumValuePerMorph[MeshBMorphIndex];
				Result.Morph.MinimumValuePerMorph[MorphIndex] = MeshB.Morph.MinimumValuePerMorph[MeshBMorphIndex];

				if (Result.MorphDataBuffer.Num())
				{
					FMemory::Memcpy(
							Result.MorphDataBuffer.GetData() + BatchAccumulatedOffsetInDwords, 
							MeshB.MorphDataBuffer.GetData() + MeshB.Morph.BatchStartOffsetPerMorph[MeshBMorphIndex]*NumBatchHeaderDwords,
							MeshB.Morph.BatchesPerMorph[MeshBMorphIndex] * NumBatchHeaderDwords*sizeof(uint32));
					
					BatchAccumulatedOffsetInDwords += MeshB.Morph.BatchesPerMorph[MeshBMorphIndex] * NumBatchHeaderDwords;
				}
			}
		}

		// Merge morph data and fix up headers.

		if (Result.MorphDataBuffer.Num())
		{
			const uint32 MeshANumVertices = MeshA.GetVertexCount();

			// Add Morphs in MeshA and common morphs with MeshB.
			MorphIndex = 0;
			for (; MorphIndex < MeshANumMorphs; ++MorphIndex)
			{
				TArrayView<uint32> MorphHeadersDataView = TArrayView<uint32>(
						Result.MorphDataBuffer.GetData() + Result.Morph.BatchStartOffsetPerMorph[MorphIndex]*NumBatchHeaderDwords,
						Result.Morph.BatchesPerMorph[MorphIndex] * NumBatchHeaderDwords);

				int32 BatchIndex = 0;
				
				if (EnumHasAnyFlags(MeshA.Morph.UsageFlagsPerMorph[MorphIndex], UsageFilter))
				{
					int32 NumBatches = MeshA.Morph.BatchesPerMorph[MorphIndex];
					for ( ; BatchIndex < NumBatches; ++BatchIndex)
					{
						TArrayView<uint32> HeaderDataView = TArrayView<uint32>(
								MorphHeadersDataView.GetData() + BatchIndex*NumBatchHeaderDwords,
								NumBatchHeaderDwords);

						FDeltaBatchHeader BatchHeader;
						ReadHeader(BatchHeader, HeaderDataView);

						const int32 SourceDataOffset = BatchHeader.DataOffset;
						BatchHeader.DataOffset = BatchAccumulatedOffsetInDwords * sizeof(uint32);

						WriteHeader(BatchHeader, HeaderDataView);

						const int32 NumBatchDwords = CalculateBatchDwords(BatchHeader);
						FMemory::Memcpy(
								Result.MorphDataBuffer.GetData() + (BatchHeader.DataOffset / sizeof(uint32)),
								MeshA.MorphDataBuffer.GetData() + (SourceDataOffset / sizeof(uint32)),
								NumBatchDwords * sizeof(uint32));

						BatchAccumulatedOffsetInDwords += NumBatchDwords;
					}
				}

				int32 MeshBMorphIndex = ResultToBMorphNameMap[MorphIndex];
				if (MeshBMorphIndex >= 0)
				{
					if (EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MeshBMorphIndex], UsageFilter))
					{
						int32 NumBatches = MeshB.Morph.BatchesPerMorph[MeshBMorphIndex];

						for (int32 MeshBBatchIndex = 0; MeshBBatchIndex < NumBatches; ++MeshBBatchIndex, ++BatchIndex)
						{
							TArrayView<uint32> HeaderDataView = TArrayView<uint32>(
									MorphHeadersDataView.GetData() + BatchIndex*NumBatchHeaderDwords,
									NumBatchHeaderDwords);

							FDeltaBatchHeader BatchHeader;
							ReadHeader(BatchHeader, HeaderDataView);

							const int32 SourceDataOffset = BatchHeader.DataOffset;
							BatchHeader.DataOffset = BatchAccumulatedOffsetInDwords * sizeof(uint32);
							BatchHeader.IndexMin += MeshANumVertices;

							WriteHeader(BatchHeader, HeaderDataView);

							const int32 NumBatchDwords = CalculateBatchDwords(BatchHeader);

							check(Result.MorphDataBuffer.Num() >= BatchHeader.DataOffset / sizeof(uint32) + NumBatchDwords);
							FMemory::Memcpy(
									Result.MorphDataBuffer.GetData() + (BatchHeader.DataOffset / sizeof(uint32)),
									MeshB.MorphDataBuffer.GetData() + (SourceDataOffset / sizeof(uint32)),
									NumBatchDwords * sizeof(uint32));
							
							BatchAccumulatedOffsetInDwords += NumBatchDwords;
						}
					}
				}
			}

			// Add Morphs in MeshB that are not present in MeshA.
			for (; MorphIndex < ResultNumMorphs; ++MorphIndex)
			{
				TArrayView<uint32> MorphHeadersDataView = TArrayView<uint32>(
						Result.MorphDataBuffer.GetData() + Result.Morph.BatchStartOffsetPerMorph[MorphIndex]*NumBatchHeaderDwords,
						Result.Morph.BatchesPerMorph[MorphIndex]*NumBatchHeaderDwords);

				int32 MeshBMorphIndex = ResultToBMorphNameMap[MorphIndex];
				int32 BatchIndex = 0;
				if (EnumHasAnyFlags(MeshB.Morph.UsageFlagsPerMorph[MeshBMorphIndex], UsageFilter))
				{
					int32 NumBatches = Result.Morph.BatchesPerMorph[MorphIndex];
					for (; BatchIndex < NumBatches; ++BatchIndex)
					{
						TArrayView<uint32> HeaderDataView = TArrayView<uint32>(
								MorphHeadersDataView.GetData() + BatchIndex*NumBatchHeaderDwords,
								NumBatchHeaderDwords);

						FDeltaBatchHeader BatchHeader;
						ReadHeader(BatchHeader, HeaderDataView);

						const int32 SourceDataOffset = BatchHeader.DataOffset;
						BatchHeader.DataOffset = BatchAccumulatedOffsetInDwords * sizeof(uint32);
						BatchHeader.IndexMin += MeshANumVertices;

						WriteHeader(BatchHeader, HeaderDataView);

						const int32 NumBatchDwords = CalculateBatchDwords(BatchHeader);
					
						check(Result.MorphDataBuffer.Num() >= BatchHeader.DataOffset / sizeof(uint32) + NumBatchDwords);
						FMemory::Memcpy(
								Result.MorphDataBuffer.GetData() + (BatchHeader.DataOffset / sizeof(uint32)),
								MeshB.MorphDataBuffer.GetData() + (SourceDataOffset / sizeof(uint32)),
								NumBatchDwords * sizeof(uint32));

						BatchAccumulatedOffsetInDwords += NumBatchDwords;
					}
				}
			}

			check(BatchAccumulatedOffsetInDwords <= Result.MorphDataBuffer.Num());
		}
	}
} // namespace OpMeshMergeMorphsInternal

namespace OpMeshMergeClothInternal
{
	/** Copy cloth from MeshA. MeshB only required to for data integrity. */
	void CopyClothFirst(FMesh& Result, const FMesh& MeshA, const FMesh& MeshB)
	{
		Result.ClothSections[0].ClothingAsset = MeshA.ClothSections[0].ClothingAsset;
		Result.ClothSections[0].AssetLODIndex = MeshA.ClothSections[0].AssetLODIndex;
			
		const uint32 NumClothVerticesA = MeshA.ClothSections[0].Data.Num();
		if (!NumClothVerticesA)
		{
			return;
		}

		const int32 NumInfluences = MeshA.ClothSections[0].Data.Num() / MeshA.VertexBuffers.GetElementCount();
		const uint32 NumClothVerticesB = NumInfluences * MeshB.VertexBuffers.GetElementCount();
		
		Result.ClothSections[0].Data.Reserve(NumClothVerticesA + NumClothVerticesB);

		Result.ClothSections[0].Data.Append(MeshA.ClothSections[0].Data);
		
		// Add MeshB vertices without simulation. See FMeshToMeshVertData::SourceMeshVertIndices.
		for (uint32 VertexIndex = 0; VertexIndex < NumClothVerticesB; ++VertexIndex)
		{
			FMeshToMeshVertData& Data = Result.ClothSections[0].Data.AddZeroed_GetRef();
			Data.SourceMeshVertIndices[3] = static_cast<uint16>(0xffff);
		}
	}
	
	
	/** Copy cloth from MeshB. MeshA only required to for data integrity. */
	void CopyClothSecond(FMesh& Result, const FMesh& MeshA, const FMesh& MeshB)
	{
		Result.ClothSections[0].ClothingAsset = MeshB.ClothSections[0].ClothingAsset;
		Result.ClothSections[0].AssetLODIndex = MeshB.ClothSections[0].AssetLODIndex;

		const uint32 NumClothVerticesB = MeshB.ClothSections[0].Data.Num();
		if (!NumClothVerticesB)
		{
			return;
		}
		
		const int32 NumInfluences = MeshB.ClothSections[0].Data.Num() / MeshB.VertexBuffers.GetElementCount();
		const uint32 NumClothVerticesA =  NumInfluences * MeshA.VertexBuffers.GetElementCount();

		Result.ClothSections[0].Data.Reserve(NumClothVerticesA + NumClothVerticesB);
		
		// Add MeshA vertices without simulation. See FMeshToMeshVertData::SourceMeshVertIndices.
		for (uint32 VertexIndex = 0; VertexIndex < NumClothVerticesA; ++VertexIndex)
		{
			FMeshToMeshVertData& Data = Result.ClothSections[0].Data.AddZeroed_GetRef();
			Data.SourceMeshVertIndices[3] = static_cast<uint16>(0xffff);					
		}
		
		Result.ClothSections[0].Data.Append(MeshB.ClothSections[0].Data);
	}
} // namespace OpMeshMergeClothInternal

namespace SkinWeightProfilesInternal
{
	void GatherMergedSkinWeightProfilesMetadata(TConstArrayView<TManagedPtr<const FMesh>> Meshes, 
		TArray<FSkinWeightProfile>& ProfilesWithMetadata, TArray<TArray<const FSkinWeightProfile*>>& SourceProfilesPerProfile)
	{
		ProfilesWithMetadata.Reset(8);
		SourceProfilesPerProfile.Reset(8);

		TArray<FName, TInlineAllocator<8>> ProfileNames;
		
		// Iterate all meshes to find the final list of merged SkinWeightProfiles.
		// Fill the metadata of the merged profiles and collect the source profiles contributing to each one. 
		const int32 NumMeshes = Meshes.Num();
		for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
		{
			const TManagedPtr<const FMesh>& Mesh = Meshes[MeshIndex];
			if (!Mesh || Mesh->SkinWeightProfiles.IsEmpty())
			{
				continue;
			}

			for (const FSkinWeightProfile& SourceProfile : Mesh->SkinWeightProfiles)
			{
				if (SourceProfile.VertexIndexToInfluenceOffset.IsEmpty())
				{
					continue;
				}

				int32 ProfileIndex = ProfileNames.Find(SourceProfile.Name);
				if (ProfileIndex == INDEX_NONE)
				{
					ProfileIndex = ProfileNames.Add(SourceProfile.Name);
					
					ProfilesWithMetadata.AddDefaulted();
					SourceProfilesPerProfile.AddDefaulted();

					// SourceProfilesPerProfile must match the number of meshes. Init to nullptr. 
					SourceProfilesPerProfile[ProfileIndex].Init(nullptr, Meshes.Num());
				}

				FSkinWeightProfile& Profile = ProfilesWithMetadata[ProfileIndex];
				Profile.Name = SourceProfile.Name;

				if (SourceProfile.bDefaultProfile)
				{
					if (Profile.bDefaultProfile)
					{
						Profile.DefaultProfileFromLODIndex = FMath::Min(Profile.DefaultProfileFromLODIndex, SourceProfile.DefaultProfileFromLODIndex);
					}
					else
					{
						Profile.bDefaultProfile = true;
						Profile.DefaultProfileFromLODIndex = SourceProfile.DefaultProfileFromLODIndex;
					}
				}

				Profile.NumBoneInfluences = FMath::Max(Profile.NumBoneInfluences, SourceProfile.NumBoneInfluences);
				Profile.bUse16BitBoneIndex = Profile.bUse16BitBoneIndex || SourceProfile.bUse16BitBoneIndex;
				Profile.bUse16BitBoneWeight = Profile.bUse16BitBoneWeight || SourceProfile.bUse16BitBoneWeight;

				SourceProfilesPerProfile[ProfileIndex][MeshIndex] = &SourceProfile;
			}
		}
	}

	struct FSkinWeightProfileView
	{
		FName Name;

		uint8 NumBoneInfluences = 0;
		bool bUse16BitBoneIndex = false;
		bool bUse16BitBoneWeight = false;

		TArray<uint8>* BoneIDs = nullptr;
		TArray<uint8>* BoneWeights = nullptr;

		// Mutable uses an array and the SkeletalMesh a map
		TMap<int32, int32>* VertexIndexToInfluenceOffsetMap = nullptr;
		TArray<FSkinWeightProfile::FVertexInfo>* VertexIndexToInfluenceOffset = nullptr;
	};

	void MergeSkinWeightProfiles(FSkinWeightProfileView& Result, TConstArrayView<const FSkinWeightProfile*> SourceProfiles, TConstArrayView<int32> NumVerticesPerMesh)
	{
		uint32 NumWeights = 0;
		uint32 NumVertexIndexToInfluenceOffsets = 0;

		check(Result.BoneIDs);
		check(Result.BoneWeights);
		check(Result.VertexIndexToInfluenceOffset || Result.VertexIndexToInfluenceOffsetMap);
		check(SourceProfiles.Num() == NumVerticesPerMesh.Num());

		// Accumulate the number of weights/influences and vertex infos to allocate.
		for (const FSkinWeightProfile* SourceProfile : SourceProfiles)
		{
			if (!SourceProfile)
			{
				continue;
			}

			if (SourceProfile->NumBoneInfluences == 0)
			{
				check(false);
				continue;
			}

			check(Result.Name == SourceProfile->Name);

			const int32 BoneIndexSize = SourceProfile->bUse16BitBoneIndex ? sizeof(uint16) : sizeof(uint8);
			NumWeights += SourceProfile->BoneIDs.Num() / (SourceProfile->NumBoneInfluences * BoneIndexSize);
			NumVertexIndexToInfluenceOffsets += SourceProfile->VertexIndexToInfluenceOffset.Num();
		}

		const uint8 NumInfluences = Result.NumBoneInfluences;
		const bool b16BitBoneIndices = Result.bUse16BitBoneIndex;
		const bool b16BitBoneWeights = Result.bUse16BitBoneWeight;

		const int32 BoneIndexByteSize = b16BitBoneIndices ? sizeof(uint16) : sizeof(uint8);
		const int32 BoneWeightByteSize = b16BitBoneWeights ? sizeof(uint16) : sizeof(uint8);

		const uint8 BoneIndicesStride = NumInfluences * BoneIndexByteSize;
		const uint8 BoneWeightsStride = NumInfluences * BoneWeightByteSize;

		Result.BoneIDs->SetNumUninitialized(NumWeights * BoneIndicesStride);
		Result.BoneWeights->SetNumUninitialized(NumWeights * BoneWeightsStride);

		int32 BoneWeightsIndex = 0;
		int32 BaseVertexOffset = 0;

		const int32 NumProfiles = SourceProfiles.Num();
		for (int32 ProfileIndex = 0; ProfileIndex < NumProfiles; ++ProfileIndex)
		{
			const FSkinWeightProfile* SourceProfile = SourceProfiles[ProfileIndex];
			if (!SourceProfile)
			{
				BaseVertexOffset += NumVerticesPerMesh[ProfileIndex];
				continue;
			}

			if (SourceProfile->NumBoneInfluences == 0)
			{
				BaseVertexOffset += NumVerticesPerMesh[ProfileIndex];
				check(false);
				continue;
			}

			// BoneIndices channel info
			const uint8 SourceBoneIndexByteSize = SourceProfile->bUse16BitBoneIndex ? 2 : 1;
			const uint8 SourceBoneIndicesStride = SourceBoneIndexByteSize * SourceProfile->NumBoneInfluences;

			// BoneWeights channel info
			const uint8 SourceBoneWeightByteSize = SourceProfile->bUse16BitBoneWeight ? 2 : 1;
			const uint8 SourceBoneWeightsStride = SourceBoneWeightByteSize * SourceProfile->NumBoneInfluences;

			check(SourceProfile->NumBoneInfluences <= NumInfluences);
			check(SourceProfile->BoneIDs.Num() + BoneWeightsIndex * BoneIndicesStride <= Result.BoneIDs->Num());
			check(SourceProfile->BoneWeights.Num() + BoneWeightsIndex * BoneWeightsStride <= Result.BoneWeights->Num());

			const int32 SourceNumBoneWeights = SourceProfile->BoneIDs.Num() / SourceBoneIndicesStride;

			// Copy Bone IDs. May add padding if source data has less influences.
			if (BoneIndexByteSize == SourceBoneIndexByteSize)
			{
				if (NumInfluences == SourceProfile->NumBoneInfluences)
				{
					FMemory::Memcpy(Result.BoneIDs->GetData() + BoneWeightsIndex * BoneIndicesStride, SourceProfile->BoneIDs.GetData(), SourceProfile->BoneIDs.Num());
				}
				else
				{
					uint8* BoneIDsData = Result.BoneIDs->GetData() + BoneWeightsIndex * BoneIndicesStride;
					const uint8* SourceBoneIDsData = SourceProfile->BoneIDs.GetData();

					for (int32 SourceWeightIndex = 0; SourceWeightIndex < SourceNumBoneWeights; ++SourceWeightIndex)
					{
						FMemory::Memzero(BoneIDsData, BoneIndicesStride);
						FMemory::Memcpy(BoneIDsData, SourceBoneIDsData, SourceBoneIndicesStride);

						BoneIDsData += BoneIndicesStride;
						SourceBoneIDsData += SourceBoneIndicesStride;
					}
				}
			}
			else
			{
				check(b16BitBoneIndices && !SourceProfile->bUse16BitBoneIndex); // Result must be the type with more bits

				uint16* TypedDestBoneIDs = reinterpret_cast<uint16*>(Result.BoneIDs->GetData() + BoneWeightsIndex * BoneIndicesStride);
				const uint8* SourceBoneIDsData = SourceProfile->BoneIDs.GetData();

				for (int32 SourceWeightIndex = 0; SourceWeightIndex < SourceNumBoneWeights; ++SourceWeightIndex)
				{
					FMemory::Memzero(TypedDestBoneIDs, BoneIndicesStride);

					for (int32 InfluenceIndex = 0; InfluenceIndex < SourceProfile->NumBoneInfluences; ++InfluenceIndex)
					{
						*(TypedDestBoneIDs + InfluenceIndex) = *(SourceBoneIDsData + InfluenceIndex);
					}

					TypedDestBoneIDs += NumInfluences;
					SourceBoneIDsData += SourceBoneIndicesStride;
				}
			}

			// Copy Bone Weights. May add padding if source data has less influences.
			if (BoneWeightByteSize == SourceBoneWeightByteSize)
			{
				if (NumInfluences == SourceProfile->NumBoneInfluences)
				{
					FMemory::Memcpy(Result.BoneWeights->GetData() + BoneWeightsIndex * BoneWeightsStride, SourceProfile->BoneWeights.GetData(), SourceProfile->BoneWeights.Num());
				}
				else
				{
					uint8* DestBoneWeights = Result.BoneWeights->GetData() + BoneWeightsIndex * BoneWeightsStride;
					const uint8* SourceBoneWeights = SourceProfile->BoneWeights.GetData();

					for (int32 SourceWeightIndex = 0; SourceWeightIndex < SourceNumBoneWeights; ++SourceWeightIndex)
					{
						FMemory::Memzero(DestBoneWeights, BoneWeightsStride);
						FMemory::Memcpy(DestBoneWeights, SourceBoneWeights, SourceBoneWeightsStride);

						DestBoneWeights += BoneWeightsStride;
						SourceBoneWeights += SourceBoneWeightsStride;
					}
				}
			}
			else
			{
				check(b16BitBoneWeights && !SourceProfile->bUse16BitBoneWeight);  // Result must be the type with more bits

				uint16* TypedDestBoneWeights = reinterpret_cast<uint16*>(Result.BoneWeights->GetData() + BoneWeightsIndex * BoneWeightsStride);
				const uint8* SourceBoneWeights = SourceProfile->BoneWeights.GetData();

				for (int32 SourceWeightIndex = 0; SourceWeightIndex < SourceNumBoneWeights; ++SourceWeightIndex)
				{
					FMemory::Memzero(TypedDestBoneWeights, BoneWeightsStride);

					for (int32 InfluenceIndex = 0; InfluenceIndex < SourceProfile->NumBoneInfluences; ++InfluenceIndex)
					{
						*(TypedDestBoneWeights +InfluenceIndex) = static_cast<uint16>(*(SourceBoneWeights + InfluenceIndex)) * 257;
					}

					TypedDestBoneWeights += NumInfluences;
					SourceBoneWeights += SourceBoneWeightsStride;
				}
			}

			// Add VertexIndex to Influence Offset 
			if (Result.VertexIndexToInfluenceOffsetMap)
			{
				Result.VertexIndexToInfluenceOffsetMap->Reserve(NumVertexIndexToInfluenceOffsets);

				for (const FSkinWeightProfile::FVertexInfo& VertexInfo : SourceProfile->VertexIndexToInfluenceOffset)
				{
					Result.VertexIndexToInfluenceOffsetMap->Add(VertexInfo.VertexIndex + BaseVertexOffset, VertexInfo.InfluenceOffset + BoneWeightsIndex);
				}
			}
			else if (Result.VertexIndexToInfluenceOffset)
			{
				Result.VertexIndexToInfluenceOffset->Reserve(NumVertexIndexToInfluenceOffsets);

				for (const FSkinWeightProfile::FVertexInfo& VertexInfo : SourceProfile->VertexIndexToInfluenceOffset)
				{
					Result.VertexIndexToInfluenceOffset->Emplace(VertexInfo.VertexIndex + BaseVertexOffset, VertexInfo.InfluenceOffset + BoneWeightsIndex);
				}
			}

			BoneWeightsIndex += SourceNumBoneWeights;
			BaseVertexOffset += NumVerticesPerMesh[ProfileIndex];
		}
	}
}

	void MergeSkeletons(FSkeleton& Base, const FSkeleton& Other, TArray<int32>& OutRemappedOtherBoneIndices)
	{
		const int32 NumBonesBaseBeforeMerge = Base.GetNumBones();
		const int32 NumBonesOther = Other.GetNumBones();

		OutRemappedOtherBoneIndices.SetNumUninitialized(NumBonesOther, EAllowShrinking::No);

		// Merge pSecond and build the remap array 
		for (int32 Index = 0; Index < NumBonesOther; ++Index)
		{
			OutRemappedOtherBoneIndices[Index] = Base.FindOrAddBone(Other.BoneNames[Index], Other.BoneParents[Index]);
		}

		// Remap Parent bone indices to new Skeleton
		for (int32 Index = NumBonesBaseBeforeMerge; Index < Base.BoneParents.Num(); ++Index)
		{
			int16 OtherBoneIndex = Base.BoneParents[Index];
			if (OtherBoneIndex != INDEX_NONE)
			{
				Base.BoneParents[Index] = OutRemappedOtherBoneIndices[OtherBoneIndex];
			}
		}
	}

	
	void MeshMerge(FMesh* Result, const TManagedPtr<const FMesh>& pFirst, const TManagedPtr<const FMesh>& pSecond, bool bMergeSurfaces)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshMerge);

		// Should never happen, but fixes static analysis warnings.
		if (!(pFirst && pSecond))
		{
			return;
		}

		// Indices
		//-----------------
		if (pFirst->GetIndexBuffers().GetBufferCount() > 0)
		{
			MUTABLE_CPUPROFILER_SCOPE(Indices);

			const int32 FirstCount = pFirst->GetIndexBuffers().GetElementCount();
			const int32 SecondCount = pSecond->GetIndexBuffers().GetElementCount();

			if (pFirst->IndexBuffers.IsDescriptor() || pSecond->IndexBuffers.IsDescriptor())
			{
				EnumAddFlags(Result->IndexBuffers.Flags, EMeshBufferSetFlags::IsDescriptor);
			}
			
			Result->GetIndexBuffers().SetElementCount(FirstCount + SecondCount);

			check(pFirst->GetIndexBuffers().GetBufferCount() <= 1);
			check(pSecond->GetIndexBuffers().GetBufferCount() <= 1);
			Result->GetIndexBuffers().SetBufferCount(1);

			FMeshBuffer& ResultIndexBuffer = Result->GetIndexBuffers().Buffers[0];

			const FMeshBuffer& FirstIndexBuffer = pFirst->GetIndexBuffers().Buffers[0];
			const FMeshBuffer& SecondIndexBuffer = pSecond->GetIndexBuffers().Buffers[0];

			// Avoid unused variable warnings
			(void)FirstIndexBuffer;
			(void)SecondIndexBuffer;

			// This will be changed below if need to change the format of the index buffers.
			EMeshBufferFormat IndexBufferFormat = EMeshBufferFormat::None;

			if (FirstCount && SecondCount)
			{
				check(!FirstIndexBuffer.Channels.IsEmpty());
				check(!SecondIndexBuffer.Channels.IsEmpty());

				// We need to know the total number of vertices in case we need to adjust the index buffer format.
				const uint64 TotalVertexCount = pFirst->GetVertexBuffers().GetElementCount() + pSecond->GetVertexBuffers().GetElementCount();
				const uint64 MaxValueBits = GetMeshFormatData(pFirst->GetIndexBuffers().Buffers[0].Channels[0].Format).MaxValueBits;
				const uint64 MaxSupportedVertices = uint64(1) << MaxValueBits;
				
				if (TotalVertexCount > MaxSupportedVertices)
				{
					IndexBufferFormat = TotalVertexCount > MAX_uint16 ? EMeshBufferFormat::UInt32 : EMeshBufferFormat::UInt16;
				}
				else if (GetMeshFormatData(pSecond->GetIndexBuffers().Buffers[0].Channels[0].Format).MaxValueBits <= MaxValueBits)
				{
					IndexBufferFormat = pFirst->GetIndexBuffers().Buffers[0].Channels[0].Format;
				}
				else
				{
					IndexBufferFormat = pSecond->GetIndexBuffers().Buffers[0].Channels[0].Format;
				}
			}
			
			if (IndexBufferFormat != EMeshBufferFormat::None)
			{
				// We only support vertex indices in case of having to change the format.
				check(FirstIndexBuffer.Channels.Num() == 1);

				ResultIndexBuffer.Channels.SetNum(1);
				ResultIndexBuffer.Channels[0].Semantic = EMeshBufferSemantic::VertexIndex;
				ResultIndexBuffer.Channels[0].Format = IndexBufferFormat;
				ResultIndexBuffer.Channels[0].ComponentCount = 1;
				ResultIndexBuffer.Channels[0].SemanticIndex = 0;
				ResultIndexBuffer.Channels[0].Offset = 0;
				ResultIndexBuffer.ElementSize = GetMeshFormatData(IndexBufferFormat).SizeInBytes;
			}
			else if (FirstCount)
			{
				ResultIndexBuffer.Channels = FirstIndexBuffer.Channels;
				ResultIndexBuffer.ElementSize = FirstIndexBuffer.ElementSize;
			}
			else if (SecondCount)
			{
				ResultIndexBuffer.Channels = SecondIndexBuffer.Channels;
				ResultIndexBuffer.ElementSize = SecondIndexBuffer.ElementSize;
			}


			check(ResultIndexBuffer.Channels.Num() == 1);
			check(ResultIndexBuffer.Channels[0].Semantic == EMeshBufferSemantic::VertexIndex);

			if (!Result->IndexBuffers.IsDescriptor())
			{

				ResultIndexBuffer.Data.SetNum(ResultIndexBuffer.ElementSize * (FirstCount + SecondCount));
			
				if (!ResultIndexBuffer.Data.IsEmpty())
				{
					if (FirstCount)
					{
						if (IndexBufferFormat == EMeshBufferFormat::None
							|| IndexBufferFormat == FirstIndexBuffer.Channels[0].Format)
						{
							FMemory::Memcpy(&ResultIndexBuffer.Data[0],
								&FirstIndexBuffer.Data[0],
								FirstIndexBuffer.ElementSize * FirstCount);
						}
						else
						{
							// Conversion required
							const uint8_t* pSource = &FirstIndexBuffer.Data[0];
							uint8_t* pDest = &ResultIndexBuffer.Data[0];
							switch (IndexBufferFormat)
							{
							case EMeshBufferFormat::UInt32:
							{
								switch (FirstIndexBuffer.Channels[0].Format)
								{
								case EMeshBufferFormat::UInt16:
								{
									for (int32 v = 0; v < FirstCount; ++v)
									{
										*(uint32_t*)pDest = *(const uint16*)pSource;
										pSource += FirstIndexBuffer.ElementSize;
										pDest += ResultIndexBuffer.ElementSize;
									}
									break;
								}

								case EMeshBufferFormat::UInt8:
								{
									for (int32 v = 0; v < FirstCount; ++v)
									{
										*(uint32_t*)pDest = *(const uint8_t*)pSource;
										pSource += FirstIndexBuffer.ElementSize;
										pDest += ResultIndexBuffer.ElementSize;
									}
									break;
								}

								default:
									checkf(false, TEXT("Format not supported."));
									break;
								}
								break;
							}

							case EMeshBufferFormat::UInt16:
							{
								switch (FirstIndexBuffer.Channels[0].Format)
								{

								case EMeshBufferFormat::UInt8:
								{
									for (int32 v = 0; v < FirstCount; ++v)
									{
										*(uint16*)pDest = *(const uint8_t*)pSource;
										pSource += FirstIndexBuffer.ElementSize;
										pDest += ResultIndexBuffer.ElementSize;
									}
									break;
								}

								default:
									checkf(false, TEXT("Format not supported."));
									break;
								}
								break;
							}

							default:
								checkf(false, TEXT("Format not supported."));
								break;
							}
						}
					}

					if (SecondCount)
					{
						const uint8_t* pSource = &SecondIndexBuffer.Data[0];
						uint8_t* pDest = &ResultIndexBuffer.Data[ResultIndexBuffer.ElementSize * FirstCount];

						uint32_t firstVertexCount = pFirst->GetVertexBuffers().GetElementCount();

						if (IndexBufferFormat == EMeshBufferFormat::None
							|| IndexBufferFormat == SecondIndexBuffer.Channels[0].Format)
						{
							switch (SecondIndexBuffer.Channels[0].Format)
							{
							case EMeshBufferFormat::Int32:
							case EMeshBufferFormat::UInt32:
							case EMeshBufferFormat::NInt32:
							case EMeshBufferFormat::NUInt32:
							{
								for (int32 v = 0; v < SecondCount; ++v)
								{
									*(uint32_t*)pDest = firstVertexCount + *(const uint32_t*)pSource;
									pSource += SecondIndexBuffer.ElementSize;
									pDest += ResultIndexBuffer.ElementSize;
								}
								break;
							}

							case EMeshBufferFormat::Int16:
							case EMeshBufferFormat::UInt16:
							case EMeshBufferFormat::NInt16:
							case EMeshBufferFormat::NUInt16:
							{
								for (int32 v = 0; v < SecondCount; ++v)
								{
									*(uint16*)pDest = uint16(firstVertexCount) + *(const uint16*)pSource;
									pSource += SecondIndexBuffer.ElementSize;
									pDest += ResultIndexBuffer.ElementSize;
								}
								break;
							}

							case EMeshBufferFormat::Int8:
							case EMeshBufferFormat::UInt8:
							case EMeshBufferFormat::NInt8:
							case EMeshBufferFormat::NUInt8:
							{
								for (int32 v = 0; v < SecondCount; ++v)
								{
									*(uint8_t*)pDest = uint8_t(firstVertexCount) + *(const uint8_t*)pSource;
									pSource += SecondIndexBuffer.ElementSize;
									pDest += ResultIndexBuffer.ElementSize;
								}
								break;
							}

							default:
								checkf(false, TEXT("Format not supported."));
								break;
							}
						}
						else
						{
							// Format conversion required
							switch (IndexBufferFormat)
							{

							case EMeshBufferFormat::UInt32:
							{
								switch (SecondIndexBuffer.Channels[0].Format)
								{
								case EMeshBufferFormat::Int16:
								case EMeshBufferFormat::UInt16:
								case EMeshBufferFormat::NInt16:
								case EMeshBufferFormat::NUInt16:
								{
									for (int32 v = 0; v < SecondCount; ++v)
									{
										*(uint32_t*)pDest = uint32_t(firstVertexCount) + *(const uint16*)pSource;
										pSource += SecondIndexBuffer.ElementSize;
										pDest += ResultIndexBuffer.ElementSize;
									}
									break;
								}

								case EMeshBufferFormat::Int8:
								case EMeshBufferFormat::UInt8:
								case EMeshBufferFormat::NInt8:
								case EMeshBufferFormat::NUInt8:
								{
									for (int32 v = 0; v < SecondCount; ++v)
									{
										*(uint32_t*)pDest = uint32_t(firstVertexCount) + *(const uint8_t*)pSource;
										pSource += SecondIndexBuffer.ElementSize;
										pDest += ResultIndexBuffer.ElementSize;
									}
									break;
								}

								default:
									checkf(false, TEXT("Format not supported."));
									break;
								}

								break;
							}

							case EMeshBufferFormat::UInt16:
							{
								switch (SecondIndexBuffer.Channels[0].Format)
								{
								case EMeshBufferFormat::Int8:
								case EMeshBufferFormat::UInt8:
								case EMeshBufferFormat::NInt8:
								case EMeshBufferFormat::NUInt8:
								{
									for (int32 v = 0; v < SecondCount; ++v)
									{
										*(uint16*)pDest = uint16(firstVertexCount) + *(const uint8_t*)pSource;
										pSource += SecondIndexBuffer.ElementSize;
										pDest += ResultIndexBuffer.ElementSize;
									}
									break;
								}

								default:
									checkf(false, TEXT("Format not supported."));
									break;
								}

								break;
							}

							default:
								checkf(false, TEXT("Format not supported."));
								break;

							}
						}
					}
				}
			}
		}


		// Layouts
		//-----------------
		{
			MUTABLE_CPUPROFILER_SCOPE(Layouts);

			int32 ResultLayoutCount = FMath::Max(pFirst->Layouts.Num(),pSecond->Layouts.Num());
			Result->Layouts.SetNum(ResultLayoutCount);
			for (int32 LayoutIndex = 0; LayoutIndex < ResultLayoutCount; ++LayoutIndex)
			{
				TManagedPtr<FLayout> pR;
				
				if (LayoutIndex < pFirst->Layouts.Num())
				{
					const FLayout* pF = pFirst->Layouts[LayoutIndex].Get();
					pR = pF->Clone();
				}

				if (LayoutIndex < pSecond->Layouts.Num())
				{
					const FLayout* pS = pSecond->Layouts[LayoutIndex].Get();
					if (!pR)
					{
						pR = pS->Clone();
					}
					else
					{
						pR->Blocks.Append(pS->Blocks);
					}
				}

				Result->Layouts[LayoutIndex] = pR;
			}
		}


		// Skeleton
		//---------------------------

		// Add SkeletonObjects
		Result->SkeletonObjects = pFirst->SkeletonObjects;

		for (const TPassthroughObjectPtr<USkeleton>& SkeletonObject : pSecond->SkeletonObjects)
		{
			Result->SkeletonObjects.AddUnique(SkeletonObject);
		}

		TManagedPtr<const FSkeleton> FirstSkeleton = pFirst->GetSkeleton();
		TManagedPtr<const FSkeleton> SecondSkeleton = pSecond->GetSkeleton();

		// Do they have the same skeleton?
		bool bMergeSkeletons = FirstSkeleton != SecondSkeleton;

		// Are they different skeletons but with the same data?
		if (bMergeSkeletons && FirstSkeleton && SecondSkeleton)
		{
			bMergeSkeletons = !(*FirstSkeleton == *SecondSkeleton);
		}

		TArray<int32> RemappedSecondBoneIndices;

		if (bMergeSkeletons)
		{
			MUTABLE_CPUPROFILER_SCOPE(MergeSkeleton);

			TManagedPtr<FSkeleton> ResultSkeleton = FirstSkeleton ? FirstSkeleton->Clone() : MakeManaged<FSkeleton>();
			Result->SetSkeleton(ResultSkeleton);

			MergeSkeletons(*ResultSkeleton.Get(), *SecondSkeleton.Get(), RemappedSecondBoneIndices);
		}
		else
		{
			Result->SetSkeleton(FirstSkeleton);

			const int32 NumBonesFirst = FirstSkeleton ? FirstSkeleton->GetNumBones() : 0;
			RemappedSecondBoneIndices.SetNumUninitialized(NumBonesFirst);
			for (int32 Index = 0; Index < NumBonesFirst; ++Index)
			{
				RemappedSecondBoneIndices[Index] = Index;
			}
		}


		// Surfaces
		//---------------------------
		
		// Remap bone indices if we merge surfaces since bonemaps will be merged too.
		bool bRemapBoneIndices = false;
		TArray<uint16> RemappedBoneMapIndices;

		const int32 NumBonesInSecondBoneMap = pSecond->BoneMap.Num();

		// Used to know the format of the bone index buffer
		uint32 MaxNumBonesInBoneMaps = 0;

		{
			MUTABLE_CPUPROFILER_SCOPE(Surfaces);
			
			const int32 NumFirstBonesInBoneMap = pFirst->BoneMap.Num();
			Result->BoneMap = pFirst->BoneMap;

			// Copy BoneMap to remap bones when merging skeletons
			TArray<FBoneIdOrIndex> SecondBoneMap = pSecond->BoneMap;

			if (bMergeSkeletons)
			{
				// Remap BoneMap to new skeleton
				for (int32 BoneIndex = 0; BoneIndex < NumBonesInSecondBoneMap; ++BoneIndex)
				{
					SecondBoneMap[BoneIndex].Index = RemappedSecondBoneIndices[SecondBoneMap[BoneIndex].Index];
				}
			}

			if (bMergeSurfaces)
			{
				// Merge BoneMaps
				RemappedBoneMapIndices.Reserve(NumBonesInSecondBoneMap);

				for (uint16 Index = 0; Index < NumBonesInSecondBoneMap; ++Index)
				{
					const int32 BoneMapIndex = Result->BoneMap.AddUnique(SecondBoneMap[Index]);
					RemappedBoneMapIndices.Add(BoneMapIndex);

					bRemapBoneIndices = bRemapBoneIndices || BoneMapIndex != Index;
				}

				FMeshSurface& NewSurface = Result->Surfaces.AddDefaulted_GetRef();
				Result->ClothSections.AddDefaulted();
				NewSurface.BoneMapCount = Result->BoneMap.Num();

				int32 NumFirstSubMeshes = 0;
				for (const FMeshSurface& Surf : pFirst->Surfaces)
				{
					NewSurface.SubMeshes.Append(Surf.SubMeshes);
					NumFirstSubMeshes += Surf.SubMeshes.Num();
					NewSurface.bCastShadow |= Surf.bCastShadow;
					NewSurface.bRecomputeTangent |= Surf.bRecomputeTangent;
				}

				for (const FMeshSurface& Surf : pSecond->Surfaces)
				{
					NewSurface.SubMeshes.Append(Surf.SubMeshes);
					NewSurface.bCastShadow |= Surf.bCastShadow;
					NewSurface.bRecomputeTangent |= Surf.bRecomputeTangent;
				}


				// Fix surface Submesh ranges.
				if (NumFirstSubMeshes > 0)
				{
					const int32 NumResultSubMeshes = NewSurface.SubMeshes.Num();
				
					const FSurfaceSubMesh LastFromFirstMesh = pFirst->Surfaces.Last().SubMeshes.Last();
	
					for (int32 SecondSubMeshIndex = NumFirstSubMeshes; SecondSubMeshIndex < NumResultSubMeshes; ++SecondSubMeshIndex)
					{
						NewSurface.SubMeshes[SecondSubMeshIndex].VertexBegin += LastFromFirstMesh.VertexEnd;	
						NewSurface.SubMeshes[SecondSubMeshIndex].VertexEnd += LastFromFirstMesh.VertexEnd;	
						NewSurface.SubMeshes[SecondSubMeshIndex].IndexBegin += LastFromFirstMesh.IndexEnd;	
						NewSurface.SubMeshes[SecondSubMeshIndex].IndexEnd += LastFromFirstMesh.IndexEnd;	
					}
				}
			}
			else
			{
				// Add the BoneMap of the second mesh
				Result->BoneMap.Append(SecondBoneMap);

				// Add pFirst surfaces
				Result->Surfaces.Reserve(pFirst->Surfaces.Num() + pSecond->Surfaces.Num());
				Result->Surfaces.Append(pFirst->Surfaces);
				
				const int32 FirstVertexEnd = pFirst->GetVertexCount();
				const int32 FirstIndexEnd = pFirst->GetIndexCount();

				for (int32 SurfaceIndex = 0; SurfaceIndex < pSecond->Surfaces.Num(); ++SurfaceIndex)
				{
					FMeshSurface& NewSurface = Result->Surfaces.Add_GetRef(pSecond->Surfaces[SurfaceIndex]);

					for (FSurfaceSubMesh& SubMesh : NewSurface.SubMeshes)
					{
						SubMesh.VertexBegin += FirstVertexEnd;
						SubMesh.VertexEnd += FirstVertexEnd;
						SubMesh.IndexBegin += FirstIndexEnd;
						SubMesh.IndexEnd += FirstIndexEnd;
					}

					NewSurface.BoneMapIndex += NumFirstBonesInBoneMap;
				}

				Result->ClothSections.SetNum(Result->Surfaces.Num());
			}

			for (const FMeshSurface& Surface : Result->Surfaces)
			{
				MaxNumBonesInBoneMaps = FMath::Max(MaxNumBonesInBoneMaps, Surface.BoneMapCount);
			}

			Result->BoneMap.Shrink();
		}


		// Pose
		//---------------------------
		if (TManagedPtr<const FSkeleton> ResultSkeleton = Result->GetSkeleton())
		{
			MUTABLE_CPUPROFILER_SCOPE(Pose);

			Result->BonePoses.Reserve(ResultSkeleton->BoneNames.Num());

			// Copy poses from the first mesh
			Result->BonePoses = pFirst->BonePoses;

			// Add or override bone poses
			for (const FMesh::FBonePose& SecondBonePose : pSecond->BonePoses)
			{
				const int32 RemappedBoneIndex = RemappedSecondBoneIndices[SecondBonePose.BoneId.Index];
				const int32 ResultBonePoseIndex = Result->FindBonePoseByBoneIndex(RemappedBoneIndex);

				if (ResultBonePoseIndex != INDEX_NONE)
				{
					FMesh::FBonePose& ResultBonePose = Result->BonePoses[ResultBonePoseIndex];

					// TODO: Not sure how to tune this priority, review it.
					// For now use a similar strategy as before. 
					auto ComputeBoneMergePriority = [](const FMesh::FBonePose& BonePose)
					{
						return (EnumHasAnyFlags(BonePose.BoneUsageFlags, EBoneUsageFlags::Skinning) ? 1 : 0) +
							(EnumHasAnyFlags(BonePose.BoneUsageFlags, EBoneUsageFlags::Reshaped) ? 1 : 0);
					};

					const int32 ResultBonePriority = ComputeBoneMergePriority(ResultBonePose);
					const int32 SecondBonePriority = ComputeBoneMergePriority(SecondBonePose);

					if (ResultBonePriority < SecondBonePriority ||
						(ResultBonePriority == SecondBonePriority && ResultBonePose.BonePosePriority < SecondBonePose.BonePosePriority))
					{
						//ResultBonePose.BoneName = SecondBonePose.BoneName;
						ResultBonePose.BoneTransform = SecondBonePose.BoneTransform;
						ResultBonePose.BonePosePriority = SecondBonePose.BonePosePriority;

						// Merge usage flags
						EnumAddFlags(ResultBonePose.BoneUsageFlags, SecondBonePose.BoneUsageFlags);
					}
				}
				else
				{
					FMesh::FBonePose NewPose = SecondBonePose;
					NewPose.BoneId.Index = RemappedBoneIndex;
					Result->BonePoses.Add(NewPose);
				}
			}

			Result->BonePoses.Shrink();
		}


		// Sockets
		//---------------------------
		{
			// Copy sockets from the first mesh
			Result->Sockets = pFirst->Sockets;

			for (const FMeshSocket& Socket : pSecond->Sockets)
			{
				FMeshSocket* Found = Result->Sockets.FindByPredicate(
					[SocketName = Socket.SocketName](const FMeshSocket& OtherSocket) { return OtherSocket.SocketName == SocketName; });

				if (!Found)
				{
					Result->Sockets.Add(Socket);
				}
				else if (Found->Priority < Socket.Priority)
				{
					*Found = Socket;
				}
			}
		}


		// PhysicsBodies
		//---------------------------
		{
			MUTABLE_CPUPROFILER_SCOPE(PhysicsBodies);
			
			// Append PhysicsAssets
			Result->PhysicsAssets = pFirst->PhysicsAssets;

			for (const TPassthroughObjectPtr<UPhysicsAsset>& PhysicsAsset : pSecond->PhysicsAssets)
			{
				Result->PhysicsAssets.AddUnique(PhysicsAsset);
			}

			// Appends InPhysicsBody to OutPhysicsBody removing Bodies that are equal and their properties are identical.
			auto AppendPhysicsBodiesUnique = [](FPhysicsBody& OutPhysicsBody, const FPhysicsBody& InPhysicsBody) -> bool
			{
				TArray<FName>& OutBones = OutPhysicsBody.BodiesBoneNames;
				TArray<FPhysicsBodyAggregate>& OutBodies = OutPhysicsBody.Bodies;

				const TArray<FName>& InBones = InPhysicsBody.BodiesBoneNames;
				const TArray<FPhysicsBodyAggregate>& InBodies = InPhysicsBody.Bodies;

				const int32 InBodyCount = InPhysicsBody.GetBodyCount();
				const int32 OutBodyCount = OutPhysicsBody.GetBodyCount();

				bool bModified = false;

				for (int32 InBodyIndex = 0; InBodyIndex < InBodyCount; ++InBodyIndex)
				{
					const int32 FoundIndex = OutBones.Find(InBones[InBodyIndex]);

					if (FoundIndex == INDEX_NONE)
					{
						OutBones.Add(InBones[InBodyIndex]);
						OutBodies.Add(InBodies[InBodyIndex]);

						bModified |= true;

						continue;
					}

					for (const FSphereBody& Body : InBodies[InBodyIndex].Spheres)
					{
						const int32 NumBeforeAddition = OutBodies[FoundIndex].Spheres.Num();
						bModified |= NumBeforeAddition == OutBodies[FoundIndex].Spheres.AddUnique(Body);
					}

					for (const FBoxBody& Body : InBodies[InBodyIndex].Boxes)
					{
						const int32 NumBeforeAddition = OutBodies[FoundIndex].Boxes.Num();
						bModified |= NumBeforeAddition == OutBodies[FoundIndex].Boxes.AddUnique(Body);
					}

					for (const FSphylBody& Body : InBodies[InBodyIndex].Sphyls)
					{
						const int32 NumBeforeAddition = OutBodies[FoundIndex].Sphyls.Num();
						bModified |= NumBeforeAddition == OutBodies[FoundIndex].Sphyls.AddUnique(Body);
					}

					for (const FTaperedCapsuleBody& Body : InBodies[InBodyIndex].TaperedCapsules)
					{
						const int32 NumBeforeAddition = OutBodies[FoundIndex].TaperedCapsules.Num();
						bModified |= NumBeforeAddition == OutBodies[FoundIndex].TaperedCapsules.AddUnique(Body);
					}

					for (const FConvexBody& Body : InBodies[InBodyIndex].Convex)
					{
						const int32 NumBeforeAddition = OutBodies[FoundIndex].Convex.Num();
						bModified |= NumBeforeAddition == OutBodies[FoundIndex].Convex.AddUnique(Body);
					}
				}

				return bModified;
			};

			TTuple<TManagedPtr<const FPhysicsBody>, bool> SharedResultPhysicsBody = Invoke([&]()
				-> TTuple<TManagedPtr<const FPhysicsBody>, bool>
			{
				if (pFirst->GetPhysicsBody() == pSecond->GetPhysicsBody())
				{
					return MakeTuple(pFirst->GetPhysicsBody(), true);
				}

				if (pFirst->GetPhysicsBody() && !pSecond->GetPhysicsBody())
				{
					return MakeTuple(pFirst->GetPhysicsBody(), true);
				}

				if (!pFirst->GetPhysicsBody() && pSecond->GetPhysicsBody())
				{
					return MakeTuple(pSecond->GetPhysicsBody(), true);
				}

				return MakeTuple(nullptr, false);
			});

			if (SharedResultPhysicsBody.Get<1>())
			{
				// Only one or non of the meshes has physics, share the result.
				Result->SetPhysicsBody(SharedResultPhysicsBody.Get<0>());
			}
			else
			{
				check(pFirst->GetPhysicsBody() && pSecond->GetPhysicsBody());

				TManagedPtr<FPhysicsBody> MergedResultPhysicsBody = pFirst->GetPhysicsBody()->Clone();

				MergedResultPhysicsBody->bBodiesModified =
					AppendPhysicsBodiesUnique(*MergedResultPhysicsBody, *pSecond->GetPhysicsBody()) ||
					pFirst->GetPhysicsBody()->bBodiesModified || pSecond->GetPhysicsBody()->bBodiesModified;

				Result->SetPhysicsBody(MergedResultPhysicsBody);
			}

			// Additional physics bodies.
			const int32 MaxAdditionalPhysicsResultNum = pFirst->AdditionalPhysicsBodies.Num() + pSecond->AdditionalPhysicsBodies.Num();

			Result->AdditionalPhysicsBodies.Reserve(MaxAdditionalPhysicsResultNum);
			Result->AdditionalPhysicsBodies.Append(pFirst->AdditionalPhysicsBodies);

			Result->AdditionalPhysicsAssets.Reserve(MaxAdditionalPhysicsResultNum);
			Result->AdditionalPhysicsAssets.Append(pFirst->AdditionalPhysicsAssets);
			
			// Not very many additional bodies expected, do a quadratic search to have unique bodies based on external id.
			const int32 NumSecondAdditionalBodies = pSecond->AdditionalPhysicsBodies.Num();
			check(pSecond->AdditionalPhysicsAssets.Num() == NumSecondAdditionalBodies);

			for (int32 Index = 0; Index < NumSecondAdditionalBodies; ++Index)
			{
				const int32 CustomIdToMerge = pSecond->AdditionalPhysicsBodies[Index]->CustomId;

				const TManagedPtr<const FPhysicsBody>* Found = pFirst->AdditionalPhysicsBodies.FindByPredicate(
					[CustomIdToMerge](const TManagedPtr<const FPhysicsBody>& Body) { return CustomIdToMerge == Body->CustomId; });

				// TODO: current usages do not expect collisions, but same Id collision with bodies modified in differnet ways
				// may need to be contemplated at some point.
				if (!Found)
				{
					Result->AdditionalPhysicsBodies.Add(pSecond->AdditionalPhysicsBodies[Index]);
					Result->AdditionalPhysicsAssets.Add(pSecond->AdditionalPhysicsAssets[Index]);
				}
			}
		}

		// This affects both vertex IDs and layout block ids.
		bool bNeedsExplicitIds = pFirst->MeshIDPrefix != pSecond->MeshIDPrefix;

		// These two extra checks are necessary for corner cases of meshes merging with fragments of themselves that
		// undergo different operations.
		if (!bNeedsExplicitIds)
		{
			UntypedMeshBufferIteratorConst FirstIDs(pFirst->VertexBuffers, EMeshBufferSemantic::VertexIndex, 0);
			UntypedMeshBufferIteratorConst SecondIDs(pSecond->VertexBuffers, EMeshBufferSemantic::VertexIndex, 0);
			bNeedsExplicitIds = FirstIDs.GetFormat() != SecondIDs.GetFormat();
		}
		if (!bNeedsExplicitIds)
		{
			UntypedMeshBufferIteratorConst FirstIDs(pFirst->VertexBuffers, EMeshBufferSemantic::LayoutBlock, 0);
			UntypedMeshBufferIteratorConst SecondIDs(pSecond->VertexBuffers, EMeshBufferSemantic::LayoutBlock, 0);
			bNeedsExplicitIds = FirstIDs.GetFormat() != SecondIDs.GetFormat();
		}

		if (!bNeedsExplicitIds)
		{
			// This is needed in case a mesh merges with itself.
			Result->MeshIDPrefix = pFirst->MeshIDPrefix;
		}

		// Vertices
		//-----------------
		{
            MUTABLE_CPUPROFILER_SCOPE(Vertices);

			const int32 FirstBufferCount = pFirst->VertexBuffers.Buffers.Num();
			const int32 SecondBufferCount = pSecond->VertexBuffers.Buffers.Num();
			const int32 FirstVertexCount = pFirst->GetVertexBuffers().GetElementCount();
			const int32 SecondVertexCount = pSecond->GetVertexBuffers().GetElementCount();

			// Check if the format of the BoneIndex buffer has to change
			bool bChangeBoneIndicesFormat = false;
			EMeshBufferFormat BoneIndexFormat = MaxNumBonesInBoneMaps > MAX_uint8 ? EMeshBufferFormat::UInt16 : EMeshBufferFormat::UInt8;
			bChangeBoneIndicesFormat = pFirst->GetVertexBuffers().HasAnySemanticWithDifferentFormat(EMeshBufferSemantic::BoneIndices, BoneIndexFormat);
			if (!bChangeBoneIndicesFormat)
			{
				bChangeBoneIndicesFormat = pSecond->GetVertexBuffers().HasAnySemanticWithDifferentFormat(EMeshBufferSemantic::BoneIndices, BoneIndexFormat);
			}

			// Step 1: Set up the initial vertex buffer structure of the result mesh.
			//-----------------------------------------------------------------------
			{
				MUTABLE_CPUPROFILER_SCOPE(ResultFormatSetup);

				// Start with the structure of the first mesh
				Result->GetVertexBuffers().SetBufferCount(FirstBufferCount);
				for (int32 BufferIndex = 0; BufferIndex < FirstBufferCount; ++BufferIndex)
				{
					Result->VertexBuffers.Buffers[BufferIndex].Channels = pFirst->VertexBuffers.Buffers[BufferIndex].Channels;
					Result->VertexBuffers.Buffers[BufferIndex].ElementSize = pFirst->VertexBuffers.Buffers[BufferIndex].ElementSize;
				}

				// See if we need to add additional buffers from the second mesh (like vertex colors or additional UV Channels)
				// This is a bit ad-hoc: we only add buffers containing all new channels
				for (const FMeshBuffer& SecondBuf : pSecond->GetVertexBuffers().Buffers)
				{
					bool bSomeChannel = false;
					bool bAllNewChannels = true;
					for (const FMeshBufferChannel& SecondChan : SecondBuf.Channels)
					{
						// Skip system buffers
						if (SecondChan.Semantic == EMeshBufferSemantic::VertexIndex
							||
							SecondChan.Semantic == EMeshBufferSemantic::LayoutBlock)
						{
							continue;
						}

						bSomeChannel = true;

						int32 FoundBuffer = -1;
						int32 FoundChannel = -1;
						pFirst->GetVertexBuffers().FindChannel(SecondChan.Semantic, SecondChan.SemanticIndex, &FoundBuffer, &FoundChannel);
						if (FoundBuffer >= 0)
						{
							// There's at least one channel that already exists in the first mesh. Don't add the buffer.
							bAllNewChannels = false;
							continue;
						}

						// If there are additional UV channels try to add them
						if (!bAllNewChannels && SecondChan.Semantic == EMeshBufferSemantic::TexCoords
							&&
							SecondChan.SemanticIndex > 0)
						{
							// Add additional UV channels if the previous one is found.
							FMeshBufferSet& ResultVertexBuffers = Result->GetVertexBuffers();
							ResultVertexBuffers.FindChannel(EMeshBufferSemantic::TexCoords, SecondChan.SemanticIndex - 1, &FoundBuffer, &FoundChannel);

							if (FoundBuffer >= 0)
							{
								FMeshBuffer& Buffer = ResultVertexBuffers.Buffers[FoundBuffer];
								Buffer.Channels.Insert(SecondChan, FoundChannel + 1);

								ResultVertexBuffers.UpdateOffsets(FoundBuffer);
							}
						}
					}

					if (bSomeChannel && bAllNewChannels)
					{
						int32 NewBufferIndex = Result->GetVertexBuffers().Buffers.Emplace();
						Result->VertexBuffers.Buffers[NewBufferIndex].Channels = SecondBuf.Channels;
						Result->VertexBuffers.Buffers[NewBufferIndex].ElementSize = SecondBuf.ElementSize;
					}
				}

				// See if we need to enlarge the components of any of the result channels
				int32 ResultBufferCount = Result->GetVertexBuffers().GetBufferCount();
				for (int32 BufferIndex = 0; BufferIndex < FMath::Min(ResultBufferCount, FirstBufferCount); ++BufferIndex)
				{
					// Expand component counts in vertex channels of the format mesh
					FMeshBuffer& result = Result->GetVertexBuffers().Buffers[BufferIndex];

					bool bResetOffsets = false;
					for (int32 c = 0; c < result.Channels.Num(); ++c)
					{
						int32 sb = -1;
						int32 sc = -1;
						pSecond->GetVertexBuffers().FindChannel
						(
							result.Channels[c].Semantic,
							result.Channels[c].SemanticIndex,
							&sb, &sc
						);

						if (sb >= 0)
						{
							const FMeshBuffer& second = pSecond->GetVertexBuffers().Buffers[sb];

							if (second.Channels[sc].ComponentCount>result.Channels[c].ComponentCount)
							{
								result.Channels[c].ComponentCount = second.Channels[sc].ComponentCount;
								bResetOffsets = true;
							}
						}
					}

					// Reset the channel offsets if necessary
					if (bResetOffsets)
					{
						Result->GetVertexBuffers().UpdateOffsets(BufferIndex);
					}
				}


				// Change the format of the bone indices buffer
				if (bChangeBoneIndicesFormat)
				{
					// Iterate all vertex buffers and update the format
					FMeshBufferSet& VertexBuffers = Result->GetVertexBuffers();
					for (int32 VertexBufferIndex = 0; VertexBufferIndex < VertexBuffers.Buffers.Num(); ++VertexBufferIndex)
					{
						FMeshBuffer& result = VertexBuffers.Buffers[VertexBufferIndex];

						const int32 ChannelsCount = VertexBuffers.GetBufferChannelCount(VertexBufferIndex);
						for (int32 ChannelIndex = 0; ChannelIndex < ChannelsCount; ++ChannelIndex)
						{
							if (result.Channels[ChannelIndex].Semantic == EMeshBufferSemantic::BoneIndices)
							{
								result.Channels[ChannelIndex].Format = BoneIndexFormat;

								// Reset offsets
								int32 offset = 0;
								for (int32 AuxChannelIndex = 0; AuxChannelIndex < ChannelsCount; ++AuxChannelIndex)
								{
									result.Channels[AuxChannelIndex].Offset = (uint8_t)offset;
									offset += result.Channels[AuxChannelIndex].ComponentCount
										*
										GetMeshFormatData(result.Channels[AuxChannelIndex].Format).SizeInBytes;
								}
								result.ElementSize = offset;
							}
						}
					}
				}

				// Manage vertex IDs
				if (bNeedsExplicitIds)
				{
					// Make sure the result format is suitable for the explicit IDs
					Result->MakeIdsExplicit();
				}
			}


			// Step 2: Fill the result buffers
			//-----------------------------------------------------------------------
			if (pFirst->VertexBuffers.IsDescriptor() || pSecond->VertexBuffers.IsDescriptor())
			{
				EnumAddFlags(Result->VertexBuffers.Flags, EMeshBufferSetFlags::IsDescriptor);
			}
			Result->VertexBuffers.SetElementCount(FirstVertexCount + SecondVertexCount);
			
			if (!Result->VertexBuffers.IsDescriptor())
			{
				MUTABLE_CPUPROFILER_SCOPE(ResultFill);
				
				// We have the final result vertex buffers structure: allocate the memory for them.
				int32 ResultBufferCount = Result->GetVertexBuffers().GetBufferCount();
				for (int32 ResultBufferIndex = 0; ResultBufferIndex < ResultBufferCount; ++ResultBufferIndex)
				{
					FMeshBuffer& ResultBuffer = Result->VertexBuffers.Buffers[ResultBufferIndex];

					// TODO: Don't assume the buffer order in First and Second matches Result, for more opportunities of fast-path
					bool bFirstFastPath = (ResultBufferIndex < FirstBufferCount)
						&&
						pFirst->VertexBuffers.HasSameFormat(ResultBufferIndex, Result->VertexBuffers, ResultBufferIndex);

					int32 FirstResultBufferSize = ResultBuffer.ElementSize * FirstVertexCount;
					uint8* Destination = ResultBuffer.Data.GetData();
					if (bFirstFastPath)
					{
						MUTABLE_CPUPROFILER_SCOPE(FirstFastPath);

						const FMeshBuffer& FirstBuffer = pFirst->VertexBuffers.Buffers[ResultBufferIndex];
						check(FirstResultBufferSize == FirstBuffer.Data.Num());
						FMemory::Memcpy(Destination, FirstBuffer.Data.GetData(), FirstResultBufferSize);
					}
					else
					{
						MUTABLE_CPUPROFILER_SCOPE(FirstSlowPath);

						int32 OffsetElements = 0;
						MeshFormatBuffer(pFirst->VertexBuffers, ResultBuffer, OffsetElements, true, pFirst->MeshIDPrefix);
					}


					bool bSecondFastPath = (ResultBufferIndex < SecondBufferCount)
						&&
						pSecond->VertexBuffers.HasSameFormat(ResultBufferIndex, Result->VertexBuffers, ResultBufferIndex);

					int32 SecondResultBufferSize = ResultBuffer.ElementSize * SecondVertexCount;
					Destination = ResultBuffer.Data.GetData() + FirstResultBufferSize;
					if (bSecondFastPath)
					{
						MUTABLE_CPUPROFILER_SCOPE(SecondFastPath);

						const FMeshBuffer& SecondBuffer = pSecond->VertexBuffers.Buffers[ResultBufferIndex];
						check(SecondResultBufferSize == SecondBuffer.Data.Num());
						FMemory::Memcpy(Destination, SecondBuffer.Data.GetData(), SecondResultBufferSize);
					}
					else
					{
						MUTABLE_CPUPROFILER_SCOPE(SecondSlowPath);

						int32 OffsetElements = FirstVertexCount;
						MeshFormatBuffer(pSecond->VertexBuffers, ResultBuffer, OffsetElements, true, pSecond->MeshIDPrefix);
					}
				}

				// TODO: This still needs to be optimized
				if (bRemapBoneIndices)
				{
					MUTABLE_CPUPROFILER_SCOPE(RemapBones);

					// We need to remap the bone indices of the second mesh vertices that we already copied to result
					check(!RemappedBoneMapIndices.IsEmpty());

					// Iterate all vertex buffers and update the format
					FMeshBufferSet& VertexBuffers = Result->GetVertexBuffers();
					for (int32 VertexBufferIndex = 0; VertexBufferIndex < VertexBuffers.Buffers.Num(); ++VertexBufferIndex)
					{
						FMeshBuffer& ResultBuffer = VertexBuffers.Buffers[VertexBufferIndex];

						const int32 ElemSize = VertexBuffers.GetElementSize(VertexBufferIndex);
						const int32 FirstSize = FirstVertexCount * ElemSize;

						const int32 ChannelsCount = VertexBuffers.GetBufferChannelCount(VertexBufferIndex);
						for (int32 ChannelIndex = 0; ChannelIndex < ChannelsCount; ++ChannelIndex)
						{
							if (ResultBuffer.Channels[ChannelIndex].Semantic != EMeshBufferSemantic::BoneIndices)
							{
								continue;
							}

							int32 ResultOffset = FirstSize + ResultBuffer.Channels[ChannelIndex].Offset;

							const int32 NumComponents = ResultBuffer.Channels[ChannelIndex].ComponentCount;

							// Bone indices may need remapping
							for (int32 VertexIndex = 0; VertexIndex < pSecond->GetVertexCount(); ++VertexIndex)
							{
								switch (BoneIndexFormat)
								{
								case EMeshBufferFormat::Int8:
								case EMeshBufferFormat::UInt8:
								{
									uint8* pD = reinterpret_cast<uint8*>(&ResultBuffer.Data[ResultOffset]);

									for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
									{
										uint8 BoneMapIndex = pD[ComponentIndex];

										// be defensive
										if (BoneMapIndex < NumBonesInSecondBoneMap)
										{
											pD[ComponentIndex] = (uint8)RemappedBoneMapIndices[BoneMapIndex];
										}
										else
										{
											pD[ComponentIndex] = 0;
										}
									}

									ResultOffset += ElemSize;
									break;
								}

								case EMeshBufferFormat::Int16:
								case EMeshBufferFormat::UInt16:
								{
									uint16* pD = reinterpret_cast<uint16*>(&ResultBuffer.Data[ResultOffset]);

									for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
									{
										uint16 BoneMapIndex = pD[ComponentIndex];

										// be defensive
										if (BoneMapIndex < NumBonesInSecondBoneMap)
										{
											pD[ComponentIndex] = (uint16)RemappedBoneMapIndices[BoneMapIndex];
										}
										else
										{
											pD[ComponentIndex] = 0;
										}
									}

									ResultOffset += ElemSize;
									break;
								}

								case EMeshBufferFormat::Int32:
								case EMeshBufferFormat::UInt32:
								{
									// Unreal does not support 32 bit bone indices
									checkf(false, TEXT("Format not supported."));
									break;
								}

								default:
									checkf(false, TEXT("Format not supported."));
								}
							}
						}
					}
				}
			}
		}


		// SkinWeightProfiles
		//-----------------
		{
			TArray<TManagedPtr<const FMesh>, TInlineAllocator<2>> Meshes({pFirst, pSecond});

			TArray<TArray<const FSkinWeightProfile*>> SourceProfilesPerProfile;
			SkinWeightProfilesInternal::GatherMergedSkinWeightProfilesMetadata(Meshes, Result->SkinWeightProfiles, SourceProfilesPerProfile);

			if (Result->SkinWeightProfiles.Num())
			{
				TArray<int32, TInlineAllocator<2>> NumVerticesPerMesh;
				NumVerticesPerMesh.SetNumZeroed(2);
				NumVerticesPerMesh[0] = pFirst ? pFirst->GetVertexCount() : 0;
				NumVerticesPerMesh[1] = pSecond ? pSecond->GetVertexCount() : 0;

				ParallelExecutionUtils::InvokeBatchParallelForUnbalanced(Result->SkinWeightProfiles.Num(),
					[Result,
					SourceProfilesPerProfile = TConstArrayView<TArray<const FSkinWeightProfile*>>(SourceProfilesPerProfile),
					NumVerticesPerMesh = TConstArrayView<int32>(NumVerticesPerMesh)](int32 Index)
					{
						FSkinWeightProfile& ResultProfile = Result->SkinWeightProfiles[Index];

						SkinWeightProfilesInternal::FSkinWeightProfileView SkinWeightProfileScratch;
						SkinWeightProfileScratch.Name = ResultProfile.Name;
						SkinWeightProfileScratch.NumBoneInfluences = ResultProfile.NumBoneInfluences;
						SkinWeightProfileScratch.bUse16BitBoneIndex = ResultProfile.bUse16BitBoneIndex;
						SkinWeightProfileScratch.bUse16BitBoneWeight = ResultProfile.bUse16BitBoneWeight;

						SkinWeightProfileScratch.BoneIDs = &ResultProfile.BoneIDs;
						SkinWeightProfileScratch.BoneWeights = &ResultProfile.BoneWeights;
						SkinWeightProfileScratch.VertexIndexToInfluenceOffset = &ResultProfile.VertexIndexToInfluenceOffset;
						TConstArrayView<const FSkinWeightProfile*> SourceProfiles(SourceProfilesPerProfile[Index]);

						SkinWeightProfilesInternal::MergeSkinWeightProfiles(SkinWeightProfileScratch, SourceProfilesPerProfile[Index], NumVerticesPerMesh);
					});
			}
		}


		// Tags
		Result->GameplayTags = pFirst->GameplayTags;

		for (const FName& SecondTag : pSecond->GameplayTags)
		{
			Result->GameplayTags.AddUnique(SecondTag);
		}

		// Streamed Resources
		Result->AssetUserData = pFirst->AssetUserData;

		const int32 NumAsserUserData = pSecond->AssetUserData.Num();
		for (int32 Index = 0; Index < NumAsserUserData; ++Index)
		{
			Result->AssetUserData.AddUnique(pSecond->AssetUserData[Index]);
		}
	
		// Animation slots
		Result->AnimationSlots = pFirst->AnimationSlots;
	
		for (const TPair<FName, TPassthroughObjectPtr<UClass>>& Pair : pSecond->AnimationSlots)
		{
			const TPair<FName, TPassthroughObjectPtr<UClass>>* FindResult = Result->AnimationSlots.FindByPredicate([&](const TPair<FName, TPassthroughObjectPtr<UClass>>& Element)
			{
				return Element.Key == Pair.Key;
			});
			
			if (!FindResult)
			{
				Result->AnimationSlots.Add(Pair);
			}
			else if (FindResult->Value != Pair.Value)
			{
				UE_LOGF(LogMutableCore, Warning, "Unable to merge AnimBP Slots. AnimBP Slot Name [%ls] already exists", *FindResult->Key.ToString());
			}
		}

		// Mesh morphs
		constexpr EMorphUsageFlags MorphUsageFilter = EMorphUsageFlags::Merged;
		if (pFirst->HasMorphs() && !pSecond->HasMorphs())
		{
			Result->Morph = pFirst->Morph;
			Result->MorphDataBuffer = pFirst->MorphDataBuffer;
			OpMeshMergeMorphsInternal::FixMorphIndicesSurfaceMetadataAndApplyFilter(*Result, 0, 0, MorphUsageFilter);
		}
		else if (!pFirst->HasMorphs() && pSecond->HasMorphs())
		{
			Result->Morph = pSecond->Morph;
			Result->MorphDataBuffer = pSecond->MorphDataBuffer;

			int32 SurfaceOffset = bMergeSurfaces ? 0 : pFirst->Surfaces.Num();
			OpMeshMergeMorphsInternal::FixMorphIndicesSurfaceMetadataAndApplyFilter(*Result, pFirst->GetVertexCount(), SurfaceOffset, MorphUsageFilter);
		}
		else if (pFirst->HasMorphs() && pSecond->HasMorphs())
		{
			OpMeshMergeMorphsInternal::MergeMorphs(*Result, *pFirst, *pSecond, bMergeSurfaces, MorphUsageFilter);
		}

		// Cloth
		if (bMergeSurfaces)
		{
			if (pFirst->ClothSections[0].IsValid() && !pSecond->ClothSections[0].IsValid())
			{
				OpMeshMergeClothInternal::CopyClothFirst(*Result, *pFirst, *pSecond);	
			}
			else if (!pFirst->ClothSections[0].IsValid() && pSecond->ClothSections[0].IsValid())
			{
				OpMeshMergeClothInternal::CopyClothSecond(*Result, *pFirst, *pSecond);
			}
			else if (pFirst->ClothSections[0].IsValid() && pSecond->ClothSections[0].IsValid())
			{
				UE_LOGF(LogMutableCore, Error, "Merging two mesh sections with cloth is not supported.");
				OpMeshMergeClothInternal::CopyClothFirst(*Result, *pFirst, *pSecond);
			}
		}
		else
		{
			for (int32 SectionIndexFirst = 0; SectionIndexFirst < pFirst->ClothSections.Num(); ++SectionIndexFirst)
			{
				Result->ClothSections[SectionIndexFirst] = pFirst->ClothSections[SectionIndexFirst];
			}

			for (int32 SectionIndexSecond = 0; SectionIndexSecond < pSecond->ClothSections.Num(); ++SectionIndexSecond)
			{
				Result->ClothSections[pFirst->Surfaces.Num() + SectionIndexSecond] = pSecond->ClothSections[SectionIndexSecond];
			}
		}
	}


	struct FCopyUnit
	{
		const uint8* Source = nullptr;
		uint8* Dest = nullptr;
		uint32 DataSize = 0;
	};

	struct FSourceMeshBuffers
	{
		TArray<const FMeshBuffer*> Buffers;
	};

	struct FMeshScratch
	{
		bool bIsDescriptor = false;

		int32 NumSurfaces = 0;

		int32 NumVertices = 0;

		int32 NumIndices = 0;
		uint32 IndexSize = 0;

		TArray<int32> NumVerticesPerMesh;
		TArray<int32> NumIndicesPerMesh;

		TArray<int32> VertexOffsetPerMesh;

		int32 NumTexCoords = 0;
		bool bUseFullPrecisionUVs = false;
		bool bUseHighPrecisionTangentBasis = false;

		uint16 NumBoneInfluences = 0;
		bool bUse16BitBoneWeight = false;
		bool bUse16BitBoneIndex = false;

		bool bHasVertexColors = false;

		// SkinWeightProfiles

		// Skeleton 
		int32 NumBonesBoneMap = 0;
		TArray<const FSkeleton*> SkeletonsToMerge;
		TArray<FName> BoneNames;

		TArray<FSourceMeshBuffers> SourceBuffersByType;

		TArray<FName> MorphNames;

		struct FSourceMorphInfo
		{
			uint8 MeshIndex = 0;
			uint8 SurfaceIndex = 0;
			uint16 MorphIndex = 0;
		};

		TArray<TArray<FSourceMorphInfo>, TInlineAllocator<128>> MorphNameToMorphData;
	};

	FMeshScratch GatherCommonBuffersInfo(const TArray<TManagedPtr<const FMesh>>& Meshes)
	{
		FMeshScratch Result;

		const int32 NumMeshes = Meshes.Num();
		Result.NumVerticesPerMesh.SetNum(NumMeshes);
		Result.NumIndicesPerMesh.SetNum(NumMeshes);

		Result.VertexOffsetPerMesh.SetNum(NumMeshes);

		Result.SourceBuffersByType.SetNum(EBaseMeshBufferType::NumTypes);
		
		for (FSourceMeshBuffers& SourceBuffer : Result.SourceBuffersByType)
		{
			SourceBuffer.Buffers.Init(nullptr, NumMeshes);
		}

		for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
		{
			const TManagedPtr<const FMesh>& Mesh = Meshes[MeshIndex];
			if (!Mesh)
			{
				continue;
			}

			if (!Mesh->GetVertexCount() || !Mesh->GetIndexCount())
			{
				continue;
			}

			check(Mesh->GetSurfaceCount() == 1);
			Result.NumSurfaces += Mesh->GetSurfaceCount();

			Result.VertexOffsetPerMesh[MeshIndex] = Result.NumVertices;
			Result.NumVertices += Mesh->GetVertexCount();

			Result.NumIndices += Mesh->GetIndexCount();
			Result.IndexSize = FMath::Max(Result.IndexSize, Mesh->GetIndexBuffers().Buffers[0].ElementSize);

			Result.NumVerticesPerMesh[MeshIndex] = Mesh->GetVertexCount();
			Result.NumIndicesPerMesh[MeshIndex] = Mesh->GetIndexCount();

			Result.NumBonesBoneMap += Mesh->BoneMap.Num();

			const FMeshBufferSet& VertexBuffers = Mesh->GetVertexBuffers();
			Result.bIsDescriptor = Result.bIsDescriptor || VertexBuffers.IsDescriptor();

			// Fill VertexBuffers info
			for (const FMeshBuffer& Buffer : VertexBuffers.Buffers)
			{
				EBaseMeshBufferType BufferType = EBaseMeshBufferType::None;

				const int32 NumChannels = Buffer.Channels.Num();
				for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
				{
					const FMeshBufferChannel& Channel = Buffer.Channels[ChannelIndex];
					switch (Channel.Semantic)
					{
					case EMeshBufferSemantic::Position:
					{
						BufferType = EBaseMeshBufferType::Position;
						break;
					}
					case EMeshBufferSemantic::TexCoords:
					{
						BufferType = EBaseMeshBufferType::TexCoords;
						Result.NumTexCoords = FMath::Max(Result.NumTexCoords, NumChannels);
						Result.bUseFullPrecisionUVs = Result.bUseFullPrecisionUVs || Channel.Format == EMeshBufferFormat::Float32;

						ChannelIndex = NumChannels; // No need to iterate the rest of the channels
						break;
					}
					case EMeshBufferSemantic::Tangent:
					{
						BufferType = EBaseMeshBufferType::Tangent;
						Result.bUseHighPrecisionTangentBasis = Result.bUseHighPrecisionTangentBasis || Channel.Format != EMeshBufferFormat::PackedDirS8;
						ChannelIndex = NumChannels; // No need to iterate the rest of the channels
						break;
					}
					case EMeshBufferSemantic::BoneIndices:
					{
						BufferType = EBaseMeshBufferType::Skinning;
						Result.NumBoneInfluences = FMath::Max(Result.NumBoneInfluences, Channel.ComponentCount);
						Result.bUse16BitBoneIndex = Result.bUse16BitBoneIndex || Channel.Format == EMeshBufferFormat::UInt16;
						break;
					}
					case EMeshBufferSemantic::BoneWeights:
					{
						BufferType = EBaseMeshBufferType::Skinning;
						Result.bUse16BitBoneWeight = Result.bUse16BitBoneWeight || Channel.Format == EMeshBufferFormat::NUInt16;
						break;
					}
					case EMeshBufferSemantic::Color:
					{
						BufferType = EBaseMeshBufferType::Color;
						Result.bHasVertexColors = true;
						break;
					}
					default: 
						break;
					}
				}

				if (BufferType != EBaseMeshBufferType::None)
				{
					check(Result.SourceBuffersByType.IsValidIndex(BufferType));
					check(Result.SourceBuffersByType[BufferType].Buffers.IsValidIndex(MeshIndex));
					Result.SourceBuffersByType[BufferType].Buffers[MeshIndex] = &Buffer;
				}
			}

			// Fill IndexBuffer Info
			check(Result.SourceBuffersByType.IsValidIndex(EBaseMeshBufferType::Indices));
			check(Result.SourceBuffersByType[EBaseMeshBufferType::Indices].Buffers.IsValidIndex(MeshIndex));
			Result.SourceBuffersByType[EBaseMeshBufferType::Indices].Buffers[MeshIndex] = &Mesh->GetIndexBuffers().Buffers[0];

			const int32 NumMorphs = Mesh->Morph.Names.Num();
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				const int32 MorphNameIndex = Result.MorphNames.AddUnique(Mesh->Morph.Names[MorphIndex]);
				if (!Result.MorphNameToMorphData.IsValidIndex(MorphNameIndex))
				{
					Result.MorphNameToMorphData.AddDefaulted();
				}

				FMeshScratch::FSourceMorphInfo& MorphInfo = Result.MorphNameToMorphData[MorphNameIndex].AddDefaulted_GetRef();
				MorphInfo.MeshIndex = MeshIndex;
				MorphInfo.SurfaceIndex = Result.NumSurfaces - 1;
				MorphInfo.MorphIndex = MorphIndex;
			}

			const int32 NumSkeletons = Result.SkeletonsToMerge.Num();
			int32 SkeletonIndex = Result.SkeletonsToMerge.AddUnique(Mesh->Skeleton.Get());
			if (SkeletonIndex == NumSkeletons)
			{
				const int32 NumBones = Mesh->Skeleton->GetNumBones();
				for (FName BoneName : Mesh->Skeleton->BoneNames)
				{
					Result.BoneNames.AddUnique(BoneName);
				}
			}

		}

		if (Result.NumVertices > MAX_uint16)
		{
			Result.IndexSize = 4;
		}

		return Result;
	}

	void MergeLODMeshesForConversion(FMesh* Result, TArray<TManagedPtr<const FMesh>>& Meshes, bool bIsInitialGeneration)
	{
		FMeshScratch MeshScratch = GatherCommonBuffersInfo(Meshes);

		const int32 NumMeshes = Meshes.Num();

		FMeshBufferSet& ResultVertexBuffers = Result->GetVertexBuffers();
		FMeshBufferSet& ResultIndexBuffers = Result->GetIndexBuffers();
		
		if (MeshScratch.bIsDescriptor)
		{
			EnumAddFlags(Result->VertexBuffers.Flags, EMeshBufferSetFlags::IsDescriptor);
			EnumAddFlags(Result->IndexBuffers.Flags, EMeshBufferSetFlags::IsDescriptor);
		}

		{
			MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_SetUpBuffers);

			ResultIndexBuffers.SetElementCount(MeshScratch.NumIndices);

			const int32 NumVertexBuffers = MeshScratch.bHasVertexColors ? 5 : 4;
			ResultVertexBuffers.SetBufferCount(NumVertexBuffers);
			ResultVertexBuffers.SetElementCount(MeshScratch.NumVertices);

			// Indices
			EMeshBufferFormat IndexBufferFormat = MeshScratch.IndexSize == 4 ? EMeshBufferFormat::UInt32 : EMeshBufferFormat::UInt16;
			MeshBufferUtils::SetupIndexBuffer(ResultIndexBuffers, IndexBufferFormat);

			// Position buffers
			MeshBufferUtils::SetupVertexPositionsBuffer(0, ResultVertexBuffers);

			// Tangent buffer
			MeshBufferUtils::SetupTangentBuffer(1, ResultVertexBuffers);
			
			// Texture coords buffer
			MeshBufferUtils::SetupTexCoordinatesBuffer(2, MeshScratch.NumTexCoords, MeshScratch.bUseFullPrecisionUVs, ResultVertexBuffers);

			// Skin buffer
			int32 BoneIndexSize = MeshScratch.bUse16BitBoneIndex ? sizeof(uint16) : sizeof(uint8);
			int32 BoneWeightSize = MeshScratch.bUse16BitBoneWeight ? sizeof(uint16) : sizeof(uint8);
			MeshBufferUtils::SetupSkinBuffer(3, BoneIndexSize, BoneWeightSize, MeshScratch.NumBoneInfluences, ResultVertexBuffers);

			// Color buffer
			if (MeshScratch.bHasVertexColors)
			{
				MeshBufferUtils::SetupVertexColorBuffer(4, ResultVertexBuffers);
			}
		}

		// Base buffers merge
		if (!MeshScratch.bIsDescriptor)
		{
			TArray<EBaseMeshBufferType> BufferTypesToConvert;
			BufferTypesToConvert.Reserve(EBaseMeshBufferType::NumTypes);

			// ParallelFor may execute the first buffer sync. Process Skin Weights since they are slower to convert to avoid over subscription
			BufferTypesToConvert.Add(EBaseMeshBufferType::Skinning);

			BufferTypesToConvert.Add(EBaseMeshBufferType::Position);
			BufferTypesToConvert.Add(EBaseMeshBufferType::Tangent);
			BufferTypesToConvert.Add(EBaseMeshBufferType::TexCoords);

			if (MeshScratch.bHasVertexColors)
			{
				BufferTypesToConvert.Add(EBaseMeshBufferType::Color);
			}

			BufferTypesToConvert.Add(EBaseMeshBufferType::Indices);

			ParallelExecutionUtils::InvokeBatchParallelForUnbalanced(BufferTypesToConvert.Num(), [ 
				BufferTypes = TArrayView<EBaseMeshBufferType>(BufferTypesToConvert),
				&MeshScratch,
				Result](int32 BufferTypeIndex)
				{
					EBaseMeshBufferType BufferType = BufferTypes[BufferTypeIndex];

					const TArray<const FMeshBuffer*>& SourceBuffers = MeshScratch.SourceBuffersByType[BufferType].Buffers;
					const int32 NumSourceBuffers = SourceBuffers.Num();

					check(MeshScratch.NumVerticesPerMesh.Num() == NumSourceBuffers);
					check(MeshScratch.NumIndicesPerMesh.Num() == NumSourceBuffers);

					if (BufferType == EBaseMeshBufferType::Indices)
					{
						MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Indices);

						FMeshBuffer& ResultIndexBuffer = Result->GetIndexBuffers().Buffers[0];
						EMeshBufferFormat TargetFormat = ResultIndexBuffer.Channels[0].Format;

						uint32 LastIndex = 0;
						for (int32 BufferIndex = 0; BufferIndex < NumSourceBuffers; ++BufferIndex)
						{
							const FMeshBuffer* SourceBuffer = SourceBuffers[BufferIndex];
							if (!SourceBuffer)
							{
								continue;
							}

							EMeshBufferFormat SourceFormat = SourceBuffer->Channels[0].Format;

							const int32 NumSourceIndices = MeshScratch.NumIndicesPerMesh[BufferIndex];

							const uint32 LastVertex = MeshScratch.VertexOffsetPerMesh[BufferIndex];

							switch (TargetFormat)
							{
							case UE::Mutable::Private::EMeshBufferFormat::UInt16:
							{
								check(SourceFormat == EMeshBufferFormat::UInt16);

								uint16* ResultData = reinterpret_cast<uint16*>(ResultIndexBuffer.Data.GetData()) + LastIndex;
								const uint16* SourceData = reinterpret_cast<const uint16*>(SourceBuffer->Data.GetData());

								for (int32 Index = 0; Index < NumSourceIndices; ++Index)
								{
									*ResultData = *SourceData + static_cast<uint16>(LastVertex);
									++ResultData;
									++SourceData;
								}

								break;
							}
							case UE::Mutable::Private::EMeshBufferFormat::UInt32:
							{
								uint32* ResultData = reinterpret_cast<uint32*>(ResultIndexBuffer.Data.GetData()) + LastIndex;

								if (SourceFormat == EMeshBufferFormat::UInt32)
								{
									const uint32* SourceData = reinterpret_cast<const uint32*>(SourceBuffer->Data.GetData());

									for (int32 Index = 0; Index < NumSourceIndices; ++Index)
									{
										*ResultData = LastVertex + *SourceData;
										++ResultData;
										++SourceData;
									}
								}
								else if (SourceFormat == EMeshBufferFormat::UInt16)
								{
									const uint16* SourceData = reinterpret_cast<const uint16*>(SourceBuffer->Data.GetData());

									for (int32 Index = 0; Index < NumSourceIndices; ++Index)
									{
										*ResultData = LastVertex + *SourceData;
										++ResultData;
										++SourceData;
									}
								}
								else
								{
									unimplemented();
								}
								break;
							}
							default: unimplemented();
								break;
							}

							LastIndex += NumSourceIndices;
						}

						return;
					}


					FMeshBufferSet& ResultVertexBuffers = Result->GetVertexBuffers();
					const uint32 ResultNumVertices = ResultVertexBuffers.GetElementCount();

					int32 VertexIndex = 0;
					for (int32 BufferIndex = 0; BufferIndex < NumSourceBuffers; ++BufferIndex)
					{
						const FMeshBuffer* SourceBuffer = SourceBuffers[BufferIndex];
						const int32 LocalNumVertices = MeshScratch.NumVerticesPerMesh[BufferIndex];

						switch (BufferType)
						{
						case EBaseMeshBufferType::Position:
						{
							MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Position);

							uint8* PositionData = reinterpret_cast<uint8*>(ResultVertexBuffers.GetBufferData(0));
							const uint32 ResultElementSize = ResultVertexBuffers.GetElementSize(0);

							PositionData += VertexIndex * ResultElementSize;

							if (SourceBuffer)
							{
								check((VertexIndex + LocalNumVertices) * SourceBuffer->ElementSize <= ResultNumVertices * ResultElementSize);
								check(SourceBuffer->ElementSize == ResultElementSize);

								FMemory::Memcpy(PositionData, SourceBuffer->Data.GetData(), LocalNumVertices * ResultElementSize);
							}
							else if (LocalNumVertices > 0) // Mesh without Positions?
							{
								FMemory::Memzero(PositionData, LocalNumVertices * ResultElementSize);
								check(false);
							}

							break;
						}
						case EBaseMeshBufferType::TexCoords:
						{
							MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_TexCoords);

							uint8* TexCoordData = reinterpret_cast<uint8*>(ResultVertexBuffers.GetBufferData(2));
							const uint32 ResultElementSize = ResultVertexBuffers.GetElementSize(2);

							TexCoordData += VertexIndex * ResultElementSize;

							if (SourceBuffer)
							{
								EMeshBufferFormat TexCoordsFormat = MeshScratch.bUseFullPrecisionUVs ? EMeshBufferFormat::Float32 : EMeshBufferFormat::Float16;
								const int32 NumChannels = SourceBuffer->Channels.Num();
								const int32 NumComponents = SourceBuffer->Channels[0].ComponentCount;
								const int32 NumTexCoords = MeshScratch.NumTexCoords;

								const uint8* SourceData = SourceBuffer->Data.GetData();
								const int32 SourceDataElementSize = SourceBuffer->ElementSize;

								if (TexCoordsFormat == SourceBuffer->Channels[0].Format)
								{
									if (NumChannels == NumTexCoords)
									{
										FMemory::Memcpy(TexCoordData, SourceData, SourceBuffer->Data.NumBytes());
									}
									else
									{
										for (int32 LocalVertexIndex = 0; LocalVertexIndex < LocalNumVertices; ++LocalVertexIndex)
										{
											FMemory::Memzero(TexCoordData, ResultElementSize);
											FMemory::Memcpy(TexCoordData, SourceData, SourceDataElementSize);
											TexCoordData += ResultElementSize;
											SourceData += SourceDataElementSize;
										}
									}
								}
								else if (TexCoordsFormat == EMeshBufferFormat::Float32 &&
									SourceBuffer->Channels[0].Format == EMeshBufferFormat::Float16)
								{
									const FFloat16* TypedSourceData = reinterpret_cast<const FFloat16*>(SourceData);

									TArray<float> AuxDestData;
									AuxDestData.SetNumZeroed(NumTexCoords * 2);

									for (int32 LocalVertexIndex = 0; LocalVertexIndex < LocalNumVertices; ++LocalVertexIndex)
									{
										for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
										{
											AuxDestData[ChannelIndex * 2] = TypedSourceData[ChannelIndex * 2];
											AuxDestData[ChannelIndex * 2 + 1] = TypedSourceData[ChannelIndex * 2 + 1];
										}

										FMemory::Memcpy(TexCoordData, AuxDestData.GetData(), ResultElementSize);

										TexCoordData += ResultElementSize;
										TypedSourceData += NumChannels * NumComponents;
									}
								}
								else
								{
									check(false); // Target format is smaller than source format? This should never happen.
								}
							}
							else if (LocalNumVertices)
							{
								FMemory::Memzero(TexCoordData, LocalNumVertices * ResultElementSize);
								check(false);
							}

							break;
						}
						case EBaseMeshBufferType::Tangent:
						{
							MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Tangent);

							uint8* TangentData = reinterpret_cast<uint8*>(ResultVertexBuffers.GetBufferData(1));
							const uint32 ResultElementSize = ResultVertexBuffers.GetElementSize(1);

							TangentData += VertexIndex * ResultElementSize;

							EMeshBufferFormat TargetTangentFormat = MeshScratch.bUseHighPrecisionTangentBasis ? EMeshBufferFormat::None : EMeshBufferFormat::PackedDirS8;

							check(!MeshScratch.bUseHighPrecisionTangentBasis); // Conversion to high precision Tangents is not supported

							if (SourceBuffer && SourceBuffer->Channels.Num() == 2)
							{
								if (TargetTangentFormat == SourceBuffer->Channels[0].Format)
								{
									check((VertexIndex + LocalNumVertices) * SourceBuffer->ElementSize <= MeshScratch.NumVertices * ResultElementSize);
									check(SourceBuffer->ElementSize == ResultElementSize);
									FMemory::Memcpy(TangentData, SourceBuffer->Data.GetData(), LocalNumVertices * ResultElementSize);
								}
								else
								{
									unimplemented();
								}
							}
							else if (LocalNumVertices > 0)
							{
								check(false)
							}

							break;
						}
						case EBaseMeshBufferType::Color:
						{
							MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Color);

							uint8* ColorData = reinterpret_cast<uint8*>(ResultVertexBuffers.GetBufferData(4));
							const uint32 ResultElementSize = ResultVertexBuffers.GetElementSize(4);

							ColorData += VertexIndex * ResultElementSize;

							check(MeshScratch.bHasVertexColors);

							if (SourceBuffer)
							{
								check((VertexIndex + LocalNumVertices) * SourceBuffer->ElementSize <= ResultNumVertices * ResultElementSize);
								check(SourceBuffer->ElementSize == ResultElementSize);

								FMemory::Memcpy(ColorData, SourceBuffer->Data.GetData(), LocalNumVertices * ResultElementSize);
							}
							else
							{
								FMemory::Memset(ColorData, 255, LocalNumVertices * ResultElementSize);
							}

							break;
						}
						case EBaseMeshBufferType::Skinning:
						{
							MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Skinning);

							uint8* SkinningData = reinterpret_cast<uint8*>(ResultVertexBuffers.GetBufferData(3));
							const uint32 ResultElementSize = ResultVertexBuffers.GetElementSize(3);

							SkinningData += VertexIndex * ResultElementSize;

							const int32 ResultNumComponents = MeshScratch.NumBoneInfluences;
							const int32 ResultBoneIndexSize = MeshScratch.bUse16BitBoneIndex ? sizeof(uint16) : sizeof(uint8);
							const int32 ResultBoneWeightSize = MeshScratch.bUse16BitBoneWeight ? sizeof(uint16) : sizeof(uint8);

							const int32 ResultIndicesSize = ResultBoneIndexSize * ResultNumComponents;
							const int32 ResultWeightsSize = ResultBoneWeightSize * ResultNumComponents;

							if (SourceBuffer && SourceBuffer->Channels.Num() == 2)
							{
								const int32 SourceNumComponents = SourceBuffer->Channels[0].ComponentCount;
								const int32 SourceBoneIndexSize = SourceBuffer->Channels[0].Format == EMeshBufferFormat::UInt16 ? sizeof(uint16) : sizeof(uint8);
								const int32 SourceBoneWeightSize = SourceBuffer->Channels[1].Format == EMeshBufferFormat::NUInt16 ? sizeof(uint16) : sizeof(uint8);

								const int32 SourceIndicesSize = SourceBoneIndexSize * SourceNumComponents;
								const int32 SourceWeightsSize = SourceBoneWeightSize * SourceNumComponents;

								if (SourceBuffer->ElementSize == ResultElementSize)
								{
									FMemory::Memcpy(SkinningData, SourceBuffer->Data.GetData(), LocalNumVertices* ResultElementSize);
								}
								else if(ResultBoneIndexSize == SourceBoneIndexSize &&
									ResultBoneWeightSize == SourceBoneWeightSize &&
									SourceNumComponents != ResultNumComponents)
								{
									const uint8* SourceSkinningData = SourceBuffer->Data.GetData();

									for (int32 LocalVertexIndex = 0; LocalVertexIndex < LocalNumVertices; ++LocalVertexIndex)
									{
										FMemory::Memzero(SkinningData, ResultElementSize);
										FMemory::Memcpy(SkinningData, SourceSkinningData, SourceIndicesSize);
										FMemory::Memcpy(SkinningData + ResultIndicesSize, SourceSkinningData + SourceIndicesSize, SourceWeightsSize);
										

										SkinningData += ResultElementSize;
										SourceSkinningData += SourceIndicesSize + SourceWeightsSize;
									}
								}
								else
								{
									const uint8* SourceSkinningData = SourceBuffer->Data.GetData();

									for (int32 LocalVertexIndex = 0; LocalVertexIndex < LocalNumVertices; ++LocalVertexIndex)
									{
										FMemory::Memzero(SkinningData, ResultElementSize);

										if (ResultBoneWeightSize == SourceBoneWeightSize)
										{
											// Convert Indices to new format
											for (int32 BoneIndex = 0; BoneIndex < SourceNumComponents; ++BoneIndex)
											{
												*(((uint16*)SkinningData) + BoneIndex) = *(SourceSkinningData + BoneIndex);
											}

											SkinningData += ResultIndicesSize;
											SourceSkinningData += SourceIndicesSize;

											// Copy Weights
											FMemory::Memcpy(SkinningData, SourceSkinningData, SourceWeightsSize);

											SkinningData += ResultWeightsSize;
											SourceSkinningData += SourceWeightsSize;
										}
										else
										{
											// Copy Indices
											FMemory::Memcpy(SkinningData, SourceSkinningData, SourceIndicesSize);
											SkinningData += ResultIndicesSize;
											SourceSkinningData += SourceIndicesSize;

											// Convert Weights to new format
											for (int32 BoneIndex = 0; BoneIndex < SourceNumComponents; ++BoneIndex)
											{
												*(((uint16*)SkinningData) + BoneIndex) = (*(SourceSkinningData + BoneIndex) * 257);
											}

											SkinningData += ResultWeightsSize;
											SourceSkinningData += SourceWeightsSize;
										}
									}
								}

							}
							else if(LocalNumVertices > 0)
							{
								FMemory::Memzero(SkinningData, LocalNumVertices* ResultElementSize);
								check(false);
							}

							break;
						}

						default: check(false);
						}

						VertexIndex += LocalNumVertices;
					}

					check(VertexIndex == MeshScratch.NumVertices);

				});

		}

		// Cloth
		{
			MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Cloth);

			Result->ClothSections.SetNum(MeshScratch.NumSurfaces);
			
			int32 SurfaceIndex = 0;
			for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
			{
				const TManagedPtr<const FMesh>& Mesh = Meshes[MeshIndex];
				if (!Mesh)
				{
					continue;
				}

				if (!Mesh->GetVertexCount() || !Mesh->GetIndexCount())
				{
					continue;
				}

				if (ensure(Result->ClothSections.IsValidIndex(SurfaceIndex)))
				{
					check(Mesh->ClothSections.Num() == 1);
					Result->ClothSections[SurfaceIndex] = Mesh->ClothSections[0];
				}

				++SurfaceIndex;
			}
		}

		// Mesh morphs
		{
			MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_Morphs); 

			using namespace UE::MorphTargetVertexCodec;

			constexpr EMorphUsageFlags UsageFilter = EMorphUsageFlags::Merged;

			const int32 NumMorphs = MeshScratch.MorphNames.Num();

			// Compute memory requirements.	
			Result->Morph.Names.Reserve(NumMorphs);
			Result->Morph.Names.Append(MeshScratch.MorphNames);

			Result->Morph.MinimumValuePerMorph.SetNumUninitialized(NumMorphs);
			Result->Morph.MaximumValuePerMorph.SetNumUninitialized(NumMorphs);
			Result->Morph.BatchesPerMorph.SetNumZeroed(NumMorphs);
			Result->Morph.BatchStartOffsetPerMorph.SetNum(NumMorphs);
			Result->Morph.SurfacesInUsePerMorph.SetNum(NumMorphs);
			Result->Morph.UsageFlagsPerMorph.Init(EMorphUsageFlags::None, NumMorphs);

			Result->Morph.NumTotalBatches = 0;

			uint32 MergedDataBufferSizeInDwords = 0;
			float PositionPrecision = 0.0f;
			float TangentZPrecision = 0.0f;

			TArray<uint16> MorphIndicesToConvert;
			MorphIndicesToConvert.Reserve(NumMorphs);

			int32 NumCopyUnits = 0;

			// Merge Metadata.
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				const TArray<FMeshScratch::FSourceMorphInfo>& MorphInfos = MeshScratch.MorphNameToMorphData[MorphIndex];

				Result->Morph.SurfacesInUsePerMorph[MorphIndex].Reserve(MorphInfos.Num());
				Result->Morph.BatchStartOffsetPerMorph[MorphIndex] = Result->Morph.NumTotalBatches;

				Result->Morph.MaximumValuePerMorph[MorphIndex] = FVector4f(FLT_MIN,FLT_MIN,FLT_MIN,FLT_MIN);
				Result->Morph.MinimumValuePerMorph[MorphIndex] = FVector4f(FLT_MAX,FLT_MAX,FLT_MAX,FLT_MAX);

				for (const FMeshScratch::FSourceMorphInfo& MorphInfo : MorphInfos)
				{
					const FMesh* Mesh = Meshes[MorphInfo.MeshIndex].Get();
					if (!Mesh || !Mesh->Morph.UsageFlagsPerMorph.IsValidIndex(MorphInfo.MorphIndex))
					{
						check(false);
						continue;
					}

					const int32 SourceMorphIndex = MorphInfo.MorphIndex;
					
					if (EnumHasAnyFlags(Mesh->Morph.UsageFlagsPerMorph[MorphInfo.MorphIndex], UsageFilter))
					{
						Result->Morph.SurfacesInUsePerMorph[MorphIndex].Add(MorphInfo.SurfaceIndex);
						EnumAddFlags(Result->Morph.UsageFlagsPerMorph[MorphIndex], Mesh->Morph.UsageFlagsPerMorph[SourceMorphIndex]);

						const int32 SourceNumBatches = Mesh->Morph.BatchesPerMorph[SourceMorphIndex];
						Result->Morph.NumTotalBatches += SourceNumBatches;
						Result->Morph.BatchesPerMorph[MorphIndex] += SourceNumBatches;

						const FVector4f& SourceMaxValue = Mesh->Morph.MaximumValuePerMorph[SourceMorphIndex];
						const FVector4f& SourceMinValue = Mesh->Morph.MinimumValuePerMorph[SourceMorphIndex];

						Result->Morph.MaximumValuePerMorph[MorphIndex] =
							SourceMaxValue.ComponentMax(Result->Morph.MaximumValuePerMorph[MorphIndex]);

						Result->Morph.MinimumValuePerMorph[MorphIndex] =
							SourceMinValue.ComponentMin(Result->Morph.MinimumValuePerMorph[MorphIndex]);


						if (Mesh->MorphDataBuffer.Num())
						{
							const uint32 SourceMorphHeadersSize = NumBatchHeaderDwords * SourceNumBatches;

							TConstArrayView<uint32> HeadersDataView(
								Mesh->MorphDataBuffer.GetData() + Mesh->Morph.BatchStartOffsetPerMorph[SourceMorphIndex] * NumBatchHeaderDwords,
								SourceMorphHeadersSize);

							MergedDataBufferSizeInDwords += OpMeshMergeMorphsInternal::ComputeMorphAllocationRequirementsInDwords(HeadersDataView);

							++NumCopyUnits;
						}
					}

					PositionPrecision = Mesh->Morph.PositionPrecision;
					TangentZPrecision = Mesh->Morph.TangentZPrecision;
				}

				if (Result->Morph.BatchesPerMorph[MorphIndex] > 0)
				{
					MorphIndicesToConvert.Add(MorphIndex);
				}
			}

			Result->Morph.PositionPrecision = PositionPrecision;
			Result->Morph.TangentZPrecision = TangentZPrecision;

			if (NumCopyUnits > 0)
			{
				TArray<FCopyUnit> CopyUnits;
				CopyUnits.Reserve(NumCopyUnits);

				Result->MorphDataBuffer.SetNumUninitialized(MergedDataBufferSizeInDwords);

				uint32 ResultDataOffset = NumBatchHeaderDwords * Result->Morph.NumTotalBatches * sizeof(uint32);

				// Copy headers and prepare the CopyUnits.
				for (int32 MorphIndex : MorphIndicesToConvert)
				{
					int32 BatchOffset = Result->Morph.BatchStartOffsetPerMorph[MorphIndex];

					const TArray<FMeshScratch::FSourceMorphInfo>& MorphInfos = MeshScratch.MorphNameToMorphData[MorphIndex];
					for (const FMeshScratch::FSourceMorphInfo& MorphInfo : MorphInfos)
					{
						const FMesh* Mesh = Meshes[MorphInfo.MeshIndex].Get();

						if (!Mesh || !EnumHasAnyFlags(Mesh->Morph.UsageFlagsPerMorph[MorphInfo.MorphIndex], UsageFilter) || !Mesh->MorphDataBuffer.Num())
						{
							continue;
						}

						const int32 VertexIndexOffset = MeshScratch.VertexOffsetPerMesh[MorphInfo.MeshIndex];

						const int32 SourceMorphIndex = MorphInfo.MorphIndex;
						const int32 SourceNumBatches = Mesh->Morph.BatchesPerMorph[SourceMorphIndex];
						const uint32 SourceMorphHeadersSize = NumBatchHeaderDwords * SourceNumBatches;

						// Prepare the data to copy
						{
							uint32 SourceMorphDataOffset = 0;
							uint32 SourceMorphDataSize = 0;

							TConstArrayView<uint32> SourceHeadersDataView(
								Mesh->MorphDataBuffer.GetData() + Mesh->Morph.BatchStartOffsetPerMorph[SourceMorphIndex] * NumBatchHeaderDwords,
								SourceMorphHeadersSize);

							OpMeshMergeMorphsInternal::GetMorphDataOffsetAndSize(SourceHeadersDataView, SourceMorphDataOffset, SourceMorphDataSize);

							FCopyUnit& CopyUnit = CopyUnits.AddDefaulted_GetRef();
							CopyUnit.Source = reinterpret_cast<const uint8*>(Mesh->MorphDataBuffer.GetData()) + SourceMorphDataOffset * sizeof(uint32);
							CopyUnit.Dest = reinterpret_cast<uint8*>(Result->MorphDataBuffer.GetData()) + ResultDataOffset;
							CopyUnit.DataSize = SourceMorphDataSize * sizeof(uint32);
						}

						// Fix and copy headers
						int32 SourceBatchOffset = Mesh->Morph.BatchStartOffsetPerMorph[SourceMorphIndex];
						for (int32 SourceBatchIndex = 0; SourceBatchIndex < SourceNumBatches; ++SourceBatchIndex)
						{
							TConstArrayView<uint32> SourceHeadersDataView(
								Mesh->MorphDataBuffer.GetData() + SourceBatchOffset * NumBatchHeaderDwords,
								NumBatchHeaderDwords);

							TArrayView<uint32> ResultHeadersDataView(
								Result->MorphDataBuffer.GetData() + BatchOffset * NumBatchHeaderDwords,
								NumBatchHeaderDwords);

							FDeltaBatchHeader BatchHeader;
							ReadHeader(BatchHeader, SourceHeadersDataView);

							BatchHeader.DataOffset = ResultDataOffset;
							BatchHeader.IndexMin += VertexIndexOffset;

							WriteHeader(BatchHeader, ResultHeadersDataView);

							const int32 NumBatchDwords = CalculateBatchDwords(BatchHeader);
							ResultDataOffset += NumBatchDwords * sizeof(uint32);

							++SourceBatchOffset;
							++BatchOffset;
						}
					}
				}

				check(ResultDataOffset == MergedDataBufferSizeInDwords * sizeof(uint32));

				constexpr uint32 NumBatchElems = 1 << 14;
				const uint32 NumElements = ResultDataOffset - NumBatchHeaderDwords * Result->Morph.NumTotalBatches * (uint32)sizeof(uint32);
				const uint32 NumBatches = FMath::DivideAndRoundUp(NumElements, NumBatchElems);

				ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
					[
						CopyUnits = TConstArrayView<FCopyUnit>(CopyUnits), NumElements, NumBatchElems
					](int32 BatchId)
					{
						uint32 TargetOffset = BatchId * NumBatchElems;

						const int32 NumCopyUnits = CopyUnits.Num();

						int32 BytesToCopy = FMath::Min(NumBatchElems, NumElements - TargetOffset);
						uint32 BatchOffset = 0;

						for (int32 Index = 0; Index < NumCopyUnits && BytesToCopy > 0; ++Index)
						{
							FCopyUnit CopyUnit = CopyUnits[Index];

							if (TargetOffset > BatchOffset + CopyUnit.DataSize)
							{
								BatchOffset += CopyUnit.DataSize;
								continue;
							}

							const int32 SourceOffset = TargetOffset - BatchOffset;
							const int32 DataToCopy = FMath::Min((int32)CopyUnit.DataSize - SourceOffset, BytesToCopy);

							FMemory::Memcpy(CopyUnit.Dest + SourceOffset, CopyUnit.Source + SourceOffset, DataToCopy);

							TargetOffset += DataToCopy;

							BytesToCopy -= DataToCopy;
							BatchOffset += CopyUnit.DataSize;
						}
					});
			}
		}

		// SkinWeightProfiles
		//-----------------
		{
			TArray<TArray<const FSkinWeightProfile*>> SourceProfilesPerProfile;
			SkinWeightProfilesInternal::GatherMergedSkinWeightProfilesMetadata(Meshes, Result->SkinWeightProfiles, SourceProfilesPerProfile);

			if (Result->SkinWeightProfiles.Num())
			{
				ParallelExecutionUtils::InvokeBatchParallelForUnbalanced(Result->SkinWeightProfiles.Num(),
					[Result,
					SourceProfilesPerProfile = TConstArrayView<TArray<const FSkinWeightProfile*>>(SourceProfilesPerProfile),
					NumVerticesPerMesh = TConstArrayView<int32>(MeshScratch.NumVerticesPerMesh)](int32 Index)
					{
						FSkinWeightProfile& ResultProfile = Result->SkinWeightProfiles[Index];

						SkinWeightProfilesInternal::FSkinWeightProfileView SkinWeightProfileScratch;
						SkinWeightProfileScratch.Name = ResultProfile.Name;
						SkinWeightProfileScratch.NumBoneInfluences = ResultProfile.NumBoneInfluences;
						SkinWeightProfileScratch.bUse16BitBoneIndex = ResultProfile.bUse16BitBoneIndex;
						SkinWeightProfileScratch.bUse16BitBoneWeight = ResultProfile.bUse16BitBoneWeight;

						SkinWeightProfileScratch.BoneIDs = &ResultProfile.BoneIDs;
						SkinWeightProfileScratch.BoneWeights = &ResultProfile.BoneWeights;
						SkinWeightProfileScratch.VertexIndexToInfluenceOffset = &ResultProfile.VertexIndexToInfluenceOffset;
						TConstArrayView<const FSkinWeightProfile*> SourceProfiles(SourceProfilesPerProfile[Index]);

						SkinWeightProfilesInternal::MergeSkinWeightProfiles(SkinWeightProfileScratch, SourceProfilesPerProfile[Index], NumVerticesPerMesh);
					});
			}
		}

		// Append Surfaces
		{
			MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_AppendSurfaces);

			int32 LastVertex = 0;
			int32 LastIndex = 0;
			int32 SurfaceIndex = 0;
			int32 LastBoneMapIndex = 0;

			Result->Surfaces.SetNum(MeshScratch.NumSurfaces);

			for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
			{
				const TManagedPtr<const FMesh>& Mesh = Meshes[MeshIndex];
				if (!Mesh)
				{
					continue;
				}

				if (!Mesh->GetVertexCount() || !Mesh->GetIndexCount())
				{
					continue;
				}

				if (!Mesh->Skeleton)
				{
					check(false);
					continue;
				}

				FMeshSurface& NewSurface = Result->Surfaces[SurfaceIndex++];
				NewSurface = Mesh->Surfaces[0];

				for (FSurfaceSubMesh& SubMesh : NewSurface.SubMeshes)
				{
					SubMesh.VertexBegin += LastVertex;
					SubMesh.VertexEnd += LastVertex;
					SubMesh.IndexBegin += LastIndex;
					SubMesh.IndexEnd += LastIndex;
				}

				LastVertex += MeshScratch.NumVerticesPerMesh[MeshIndex];
				LastIndex += MeshScratch.NumIndicesPerMesh[MeshIndex];

				// Update BoneMap
				NewSurface.BoneMapIndex += LastBoneMapIndex;
				LastBoneMapIndex += Mesh->BoneMap.Num();
			}
		}

		// Skeleton, Pose, BoneMaps, PhysicsBodies, etc.
		if(bIsInitialGeneration)
		{
			MUTABLE_CPUPROFILER_SCOPE(MergeLODMeshes_InitialGenerationOnly);

			Result->BoneMap.Reserve(MeshScratch.NumBonesBoneMap);
			Result->BonePoses.Reserve(MeshScratch.BoneNames.Num());

			TManagedPtr<FSkeleton> ResultSkeleton;
			TArray<int32> RemappedBoneIndices;
			RemappedBoneIndices.SetNumUninitialized(MeshScratch.BoneNames.Num());

			TManagedPtr<FPhysicsBody> ResultPhysicsBody;

			for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
			{
				const TManagedPtr<const FMesh>& Mesh = Meshes[MeshIndex];
				if (!Mesh)
				{
					continue;
				}

				if (!Mesh->GetVertexCount() || !Mesh->GetIndexCount())
				{
					continue;
				}

				if (!Mesh->Skeleton)
				{
					check(false);
					continue;
				}

				// Skeleton
				//---------------------------
				for (const TPassthroughObjectPtr<USkeleton>& SkeletonObject : Mesh->SkeletonObjects)
				{
					Result->SkeletonObjects.AddUnique(SkeletonObject);
				}

				if (!ResultSkeleton)
				{
					ResultSkeleton = MakeManaged<FSkeleton>();
					Result->Skeleton = ResultSkeleton;

					ResultSkeleton->BoneNames.Reserve(MeshScratch.BoneNames.Num());
					ResultSkeleton->BoneNames.Append(Mesh->Skeleton->BoneNames);

					ResultSkeleton->BoneParents.Reserve(MeshScratch.BoneNames.Num());
					ResultSkeleton->BoneParents.Append(Mesh->Skeleton->BoneParents);

					Result->BoneMap.Append(Mesh->BoneMap);
					Result->BonePoses.Append(Mesh->BonePoses);
				}
				else
				{
					// Merge Skeleton
					//---------------------------
					{
						MUTABLE_CPUPROFILER_SCOPE(MergeSkeleton);
						MergeSkeletons(*ResultSkeleton.Get(), *Mesh->Skeleton.Get(), RemappedBoneIndices);
					}

					// BoneMap Remap
					//---------------------------
					{
						MUTABLE_CPUPROFILER_SCOPE(UpdateBoneMap);
						const int32 NumBonesBoneMap = Mesh->BoneMap.Num();

						const TArray<FBoneIdOrIndex>& OtherBoneMap = Mesh->BoneMap;

						// Remap BoneMap to new skeleton
						for (int32 BoneIndex = 0; BoneIndex < NumBonesBoneMap; ++BoneIndex)
						{
							Result->BoneMap.Add({ .Index = RemappedBoneIndices[OtherBoneMap[BoneIndex].Index] });
						}
					}


					// Pose
					//---------------------------
					{
						MUTABLE_CPUPROFILER_SCOPE(Merge Pose);


						// Add or override bone poses
						for (const FMesh::FBonePose& SecondBonePose : Mesh->BonePoses)
						{
							const int32 RemappedBoneIndex = RemappedBoneIndices[SecondBonePose.BoneId.Index];
							const int32 ResultBonePoseIndex = Result->FindBonePoseByBoneIndex(RemappedBoneIndex);

							if (ResultBonePoseIndex != INDEX_NONE)
							{
								FMesh::FBonePose& ResultBonePose = Result->BonePoses[ResultBonePoseIndex];

								// TODO: Not sure how to tune this priority, review it.
								// For now use a similar strategy as before. 
								auto ComputeBoneMergePriority = [](const FMesh::FBonePose& BonePose)
									{
										return (EnumHasAnyFlags(BonePose.BoneUsageFlags, EBoneUsageFlags::Skinning) ? 1 : 0) +
											(EnumHasAnyFlags(BonePose.BoneUsageFlags, EBoneUsageFlags::Reshaped) ? 1 : 0);
									};

								const int32 ResultBonePriority = ComputeBoneMergePriority(ResultBonePose);
								const int32 SecondBonePriority = ComputeBoneMergePriority(SecondBonePose);

								if (ResultBonePriority < SecondBonePriority ||
									(ResultBonePriority == SecondBonePriority && ResultBonePose.BonePosePriority < SecondBonePose.BonePosePriority))
								{
									//ResultBonePose.BoneName = SecondBonePose.BoneName;
									ResultBonePose.BoneTransform = SecondBonePose.BoneTransform;
									ResultBonePose.BonePosePriority = SecondBonePose.BonePosePriority;

									// Merge usage flags
									EnumAddFlags(ResultBonePose.BoneUsageFlags, SecondBonePose.BoneUsageFlags);
								}
							}
							else
							{
								FMesh::FBonePose NewPose = SecondBonePose;
								NewPose.BoneId.Index = RemappedBoneIndex;
								Result->BonePoses.Add(NewPose);
							}
						}
					}
				}

				
				// Sockets
				//---------------------------
				{
					MUTABLE_CPUPROFILER_SCOPE(Sockets);

					for (const FMeshSocket& Socket : Mesh->Sockets)
					{

						FMeshSocket* Found = Result->Sockets.FindByPredicate(
							[SocketName = Socket.SocketName](const FMeshSocket& OtherSocket) { return OtherSocket.SocketName == SocketName; });

						if (!Found)
						{
							Result->Sockets.Add(Socket);
						}
						else if (Found->Priority < Socket.Priority)
						{
							*Found = Socket;
						}
					}
				}

				// PhysicsBodies
				//---------------------------
				{
					MUTABLE_CPUPROFILER_SCOPE(PhysicsBodies);

					// Append PhysicsAssets
					for (const TPassthroughObjectPtr<UPhysicsAsset>& PhysicsAsset : Mesh->PhysicsAssets)
					{
						Result->PhysicsAssets.AddUnique(PhysicsAsset);
					}

					// Appends InPhysicsBody to OutPhysicsBody removing Bodies that are equal and their properties are identical.
					auto AppendPhysicsBodiesUnique = [](FPhysicsBody& OutPhysicsBody, const FPhysicsBody& InPhysicsBody) -> bool
						{
							TArray<FName>& OutBones = OutPhysicsBody.BodiesBoneNames;
							TArray<FPhysicsBodyAggregate>& OutBodies = OutPhysicsBody.Bodies;

							const TArray<FName>& InBones = InPhysicsBody.BodiesBoneNames;
							const TArray<FPhysicsBodyAggregate>& InBodies = InPhysicsBody.Bodies;

							const int32 InBodyCount = InPhysicsBody.GetBodyCount();
							const int32 OutBodyCount = OutPhysicsBody.GetBodyCount();

							bool bModified = false;

							for (int32 InBodyIndex = 0; InBodyIndex < InBodyCount; ++InBodyIndex)
							{
								const int32 FoundIndex = OutBones.Find(InBones[InBodyIndex]);

								if (FoundIndex == INDEX_NONE)
								{
									OutBones.Add(InBones[InBodyIndex]);
									OutBodies.Add(InBodies[InBodyIndex]);

									bModified |= true;

									continue;
								}

								for (const FSphereBody& Body : InBodies[InBodyIndex].Spheres)
								{
									const int32 NumBeforeAddition = OutBodies[FoundIndex].Spheres.Num();
									bModified |= NumBeforeAddition == OutBodies[FoundIndex].Spheres.AddUnique(Body);
								}

								for (const FBoxBody& Body : InBodies[InBodyIndex].Boxes)
								{
									const int32 NumBeforeAddition = OutBodies[FoundIndex].Boxes.Num();
									bModified |= NumBeforeAddition == OutBodies[FoundIndex].Boxes.AddUnique(Body);
								}

								for (const FSphylBody& Body : InBodies[InBodyIndex].Sphyls)
								{
									const int32 NumBeforeAddition = OutBodies[FoundIndex].Sphyls.Num();
									bModified |= NumBeforeAddition == OutBodies[FoundIndex].Sphyls.AddUnique(Body);
								}

								for (const FTaperedCapsuleBody& Body : InBodies[InBodyIndex].TaperedCapsules)
								{
									const int32 NumBeforeAddition = OutBodies[FoundIndex].TaperedCapsules.Num();
									bModified |= NumBeforeAddition == OutBodies[FoundIndex].TaperedCapsules.AddUnique(Body);
								}

								for (const FConvexBody& Body : InBodies[InBodyIndex].Convex)
								{
									const int32 NumBeforeAddition = OutBodies[FoundIndex].Convex.Num();
									bModified |= NumBeforeAddition == OutBodies[FoundIndex].Convex.AddUnique(Body);
								}
							}

							return bModified;
						};

					if (Mesh->GetPhysicsBody())
					{
						if (ResultPhysicsBody)
						{
							ResultPhysicsBody->bBodiesModified =
								AppendPhysicsBodiesUnique(*ResultPhysicsBody, *Mesh->GetPhysicsBody()) ||
								ResultPhysicsBody->bBodiesModified || Mesh->GetPhysicsBody()->bBodiesModified;
						}
						else
						{
							ResultPhysicsBody = Mesh->GetPhysicsBody()->Clone();
							Result->SetPhysicsBody(ResultPhysicsBody);
						}
					}

					// Not very many additional bodies expected, do a quadratic search to have unique bodies based on external id.
					check(Mesh->AdditionalPhysicsAssets.Num() == Mesh->AdditionalPhysicsBodies.Num());

					const int32 NumOtherAdditionalBodies = Mesh->AdditionalPhysicsBodies.Num();
					for (int32 Index = 0; Index < NumOtherAdditionalBodies; ++Index)
					{
						const int32 CustomIdToMerge = Mesh->AdditionalPhysicsBodies[Index]->CustomId;

						const TManagedPtr<const FPhysicsBody>* Found = Result->AdditionalPhysicsBodies.FindByPredicate(
							[CustomIdToMerge](const TManagedPtr<const FPhysicsBody>& Body) { return CustomIdToMerge == Body->CustomId; });

						// TODO: current usages do not expect collisions, but same Id collision with bodies modified in different ways
						// may need to be contemplated at some point.
						if (!Found)
						{
							Result->AdditionalPhysicsBodies.Add(Mesh->AdditionalPhysicsBodies[Index]);
							Result->AdditionalPhysicsAssets.Add(Mesh->AdditionalPhysicsAssets[Index]);
						}
					}
				}

				// Tags
				for (const FName& SecondTag : Mesh->GameplayTags)
				{
					Result->GameplayTags.AddUnique(SecondTag);
				}

				// Streamed Resources
				const int32 NumAsserUserData = Mesh->AssetUserData.Num();
				for (int32 Index = 0; Index < NumAsserUserData; ++Index)
				{
					Result->AssetUserData.AddUnique(Mesh->AssetUserData[Index]);
				}

				// Animation slots
				for (const TPair<FName, TPassthroughObjectPtr<UClass>>& Pair : Mesh->AnimationSlots)
				{
					const TPair<FName, TPassthroughObjectPtr<UClass>>* FindResult = Result->AnimationSlots.FindByPredicate([&](const TPair<FName, TPassthroughObjectPtr<UClass>>& Element)
						{
							return Element.Key == Pair.Key;
						});

					if (!FindResult)
					{
						Result->AnimationSlots.Add(Pair);
					}
					else if (FindResult->Value != Pair.Value)
					{
						UE_LOGF(LogMutableCore, Warning, "Unable to merge AnimBP Slots. AnimBP Slot Name [%ls] already exists", *FindResult->Key.ToString());
					}
				}
			}
		}
	}

}
