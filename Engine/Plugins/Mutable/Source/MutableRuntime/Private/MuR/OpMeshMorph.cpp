// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpMeshMorph.h"

#include "MuR/MeshPrivate.h"
#include "MuR/ConvertData.h"
#include "MuR/MutableTrace.h"
#include "MuR/SparseIndexMap.h"
#include "PackedNormal.h"
#include "MuR/Mesh.h"
#include "MuR/MutableRuntimeModule.h"
#include "UObject/NameTypes.h"
#include "Animation/MorphTarget.h"
#include "Rendering/MorphTargetVertexCodec.h"

void UE::Mutable::Private::MeshMorph(FMesh* Mesh, const FName& MorphName, const float Factor)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshMorph_Parameter);

	using namespace UE::MorphTargetVertexCodec;

	if (!(Mesh && Mesh->MorphDataBuffer.Num()))
	{
		return;
	}

	const int32 MorphIndex = Mesh->Morph.Names.Find(MorphName);
	if (MorphIndex == INDEX_NONE)
	{
		return;
	}
	
	// Get pointers to vertex position data
	MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> MeshPositionIter(Mesh->VertexBuffers, EMeshBufferSemantic::Position, 0);	
	const bool bHasPositions = MeshPositionIter.ptr() != nullptr;
	
	// {BiNormal, Tangent, Normal}
	TStaticArray<UntypedMeshBufferIterator, 3> MeshTangentFrameChannelsIters;
	
	const FMeshBufferSet& VertexBufferSet = Mesh->GetVertexBuffers();

	const uint32 NumVertices = VertexBufferSet.GetElementCount();

	int32 NormalBufferIndex   = -1;
	int32 NormalBufferChannel = -1;
	
	VertexBufferSet.FindChannel(EMeshBufferSemantic::Normal, 0, &NormalBufferIndex, &NormalBufferChannel);
	int32 NormalNumChannels = NormalBufferIndex >= 0 ? VertexBufferSet.Buffers[NormalBufferIndex].Channels.Num() : 0;
	
	const bool bHasNormals = NormalBufferIndex >= 0;

	for (int32 ChannelIndex = 0; ChannelIndex < NormalNumChannels; ++ChannelIndex)
	{
		const FMeshBufferChannel& Channel = VertexBufferSet.Buffers[NormalBufferIndex].Channels[ChannelIndex];

		EMeshBufferSemantic Sem = Channel.Semantic;
		int32 SemIndex = Channel.SemanticIndex;

		if (Sem == EMeshBufferSemantic::Normal && bHasNormals)
		{
			MeshTangentFrameChannelsIters[2] = UntypedMeshBufferIterator(Mesh->GetVertexBuffers(), Sem, SemIndex);
		}
		else if (Sem == EMeshBufferSemantic::Tangent && bHasNormals)
		{
			MeshTangentFrameChannelsIters[1] = UntypedMeshBufferIterator(Mesh->GetVertexBuffers(), Sem, SemIndex);
		}
		else if (Sem == EMeshBufferSemantic::Binormal && bHasNormals)
		{
			MeshTangentFrameChannelsIters[0] = UntypedMeshBufferIterator(Mesh->GetVertexBuffers(), Sem, SemIndex);
		}
	}

	const UntypedMeshBufferIterator& MeshNormalIter = MeshTangentFrameChannelsIters[2];
	const UntypedMeshBufferIterator& MeshTangentIter = MeshTangentFrameChannelsIters[1];
	const UntypedMeshBufferIterator& MeshBiNormalIter = MeshTangentFrameChannelsIters[0];

	const EMeshBufferFormat NormalFormat = MeshNormalIter.GetFormat();
	const int32 NormalComps = MeshNormalIter.GetComponents();

	const EMeshBufferFormat TangentFormat = MeshTangentIter.GetFormat();
	const int32 TangentComps = MeshTangentIter.GetComponents();

	const EMeshBufferFormat BiNormalFormat = MeshBiNormalIter.GetFormat();
	const int32 BiNormalComps = MeshBiNormalIter.GetComponents();
	
	const bool bHasOptimizedNormals = NormalFormat == EMeshBufferFormat::PackedDirS8_W_TangentSign
		&& (!MeshTangentIter.ptr() || TangentFormat == EMeshBufferFormat::PackedDirS8) && !MeshBiNormalIter.ptr();
	
	const uint32 BatchStartOffset = Mesh->Morph.BatchStartOffsetPerMorph[MorphIndex];
	const uint32 MorphNumBatches  = Mesh->Morph.BatchesPerMorph[MorphIndex];

	TConstArrayView<uint32> MorphDataBufferView = Mesh->MorphDataBuffer;

	uint32 NumMalformedBatchesFound = 0;

	for (uint32 BatchHeaderIndex = 0; BatchHeaderIndex < MorphNumBatches; ++BatchHeaderIndex)
	{
		const uint32 BatchHeaderOffsetInDwords = (BatchStartOffset + BatchHeaderIndex) * NumBatchHeaderDwords;

		check(BatchHeaderOffsetInDwords + NumBatchHeaderDwords <= (uint32)MorphDataBufferView.Num());
		TConstArrayView<uint32> BatchHeaderData(MorphDataBufferView.GetData() + BatchHeaderOffsetInDwords, NumBatchHeaderDwords);

		FDeltaBatchHeader BatchHeader;
		ReadHeader(BatchHeader, BatchHeaderData);

		if (BatchHeader.NumElements == 0)
		{
			continue;
		}

		TArray<FQuantizedDelta, TInlineAllocator<UE::MorphTargetVertexCodec::BatchSize>> QuantizedDeltas;
		QuantizedDeltas.SetNumUninitialized(BatchHeader.NumElements);

		uint32 BatchSizeInDwords = CalculateBatchDwords(BatchHeader);

		TConstArrayView<uint32> Data;
		if (BatchSizeInDwords > 0)
		{
			uint32 BatchDataOffsetInDwords = BatchHeader.DataOffset / sizeof(uint32);

			// Don't assume data is good, prevent any carsh and record skipped batchs.
			if ((BatchDataOffsetInDwords + BatchSizeInDwords <= (uint32)MorphDataBufferView.Num()) & 
				((BatchHeader.DataOffset & (sizeof(uint32) - 1)) == 0))
			{
				Data = TConstArrayView<uint32>(MorphDataBufferView.GetData() + BatchDataOffsetInDwords, BatchSizeInDwords);
			}
			else
			{
				++NumMalformedBatchesFound;
				continue;
			}
		}

		ReadQuantizedDeltas(QuantizedDeltas, BatchHeader, Data);

		for (FQuantizedDelta& QuantizedDelta : QuantizedDeltas)
		{
			FMorphTargetDelta Delta;
			DequantizeDelta(Delta, BatchHeader.bTangents, QuantizedDelta, Mesh->Morph.PositionPrecision, Mesh->Morph.TangentZPrecision);

			// Positions
			if (bHasPositions)
			{
				check(Delta.SourceIdx < NumVertices);
				FVector3f& Position = *reinterpret_cast<FVector3f*>(*(MeshPositionIter + Delta.SourceIdx));
				Position += Delta.PositionDelta * Factor;
			}

			// Normals
			if (bHasOptimizedNormals)
			{
				// Normal
				check(Delta.SourceIdx < NumVertices);

				FPackedNormal* PackedNormal = reinterpret_cast<FPackedNormal*>((MeshNormalIter + Delta.SourceIdx).ptr());
				int8 W = PackedNormal->Vector.W;
				const FVector3f BaseNormal = PackedNormal->ToFVector3f();

				const FVector3f Normal = (BaseNormal + Delta.TangentZDelta * Factor).GetSafeNormal();

				*PackedNormal = Normal;
				PackedNormal->Vector.W = W;

				// Tangent
				if (MeshTangentIter.ptr())
				{
					FPackedNormal* PackedTangent = reinterpret_cast<FPackedNormal*>((MeshTangentIter + Delta.SourceIdx).ptr());
					const FVector3f BaseTangent = PackedTangent->ToFVector3f();

					// Orthogonalize Tangent based on new Normal. This assumes Normal and BaseTangent are normalized and different.
					const FVector3f Tangent = (BaseTangent - FVector3f::DotProduct(Normal, BaseTangent) * Normal).GetSafeNormal();

					*PackedTangent = Tangent;
				}
			}
			else if (MeshNormalIter.ptr())
			{
				// When normal is packed, binormal channel is not expected. It is not a big deal if it's there but we would be doing extra unused work in that case. 
				ensure(!(NormalFormat == EMeshBufferFormat::PackedDir8_W_TangentSign || NormalFormat == EMeshBufferFormat::PackedDirS8_W_TangentSign) || !MeshBiNormalIter.ptr());
				
				MUTABLE_CPUPROFILER_SCOPE(ApplyNormalMorph_SlowPath);
				
				UntypedMeshBufferIterator NormalIter = MeshNormalIter + Delta.SourceIdx;

				const FVector3f BaseNormal = NormalIter.GetAsVec3f();

				const FVector3f Normal = (BaseNormal + Delta.TangentZDelta * Factor).GetSafeNormal();

				// Leave the tangent basis sign untouched for packed normals formats.
				for (int32 C = 0; C < NormalComps && C < 3; ++C)
				{
					ConvertData(C, NormalIter.ptr(), NormalFormat, &Normal, EMeshBufferFormat::Float32);
				}

				// Tangent
				if (MeshTangentIter.ptr())
				{
					UntypedMeshBufferIterator TangentIter = MeshTangentIter + Delta.SourceIdx;

					const FVector3f BaseTangent = TangentIter.GetAsVec3f();

					// Orthogonalize Tangent based on new Normal. This assumes Normal and BaseTangent are normalized and different.
					const FVector3f Tangent = (BaseTangent - FVector3f::DotProduct(Normal, BaseTangent) * Normal).GetSafeNormal();

					for (int32 C = 0; C < TangentComps && C < 3; ++C)
					{
						ConvertData(C, TangentIter.ptr(), TangentFormat, &Tangent, EMeshBufferFormat::Float32);
					}

					// BiNormal
					if (MeshBiNormalIter.ptr())
					{
						UntypedMeshBufferIterator BiNormalIter = MeshBiNormalIter + Delta.SourceIdx;

						const FVector3f& N = BaseNormal;
						const FVector3f& T = BaseTangent;
						const FVector3f  B = BiNormalIter.GetAsVec3f();

						const float BaseTangentBasisDeterminant =
							B.X * T.Y * N.Z + B.Z * T.X * N.Y + B.Y * T.Z * N.Y -
							B.Z * T.Y * N.X - B.Y * T.X * N.Z - B.X * T.Z * N.Y;

						const float BaseTangentBasisDeterminantSign = BaseTangentBasisDeterminant >= 0 ? 1.0f : -1.0f;

						const FVector3f BiNormal = FVector3f::CrossProduct(Tangent, Normal) * BaseTangentBasisDeterminantSign;

						for (int32 C = 0; C < BiNormalComps && C < 3; ++C)
						{
							ConvertData(C, BiNormalIter.ptr(), BiNormalFormat, &BiNormal, EMeshBufferFormat::Float32);
						}
					}
				}
			}
		}
	}

	if (NumMalformedBatchesFound > 0)
	{
		UE_LOGF(LogMutableCore, Warning, "Morph operation found invalid data, some batches were skipped.");
	}
}
