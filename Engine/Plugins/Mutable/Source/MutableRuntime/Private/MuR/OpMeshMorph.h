// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

namespace UE::Mutable::Private
{
	class FMesh;
}


namespace UE::Mutable::Private
{
	/** Morph Parameter Meshes. Mesh contains the deltas in morph buffer. Deltas are compressed using the engine format. */
	void MeshMorph(FMesh* Mesh, const FName& MorphName, float Factor);
}
