// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshPrepareLayout.h"

#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"
#include "MuR/Platform.h"
#include "MuR/Mesh.h"
#include "MuR/Layout.h"
#include "MuR/Image.h"
#include "MuR/MutableRuntimeModule.h"


namespace UE::Mutable::Private
{

	template<typename T>
	struct TArray2D
	{
		int32 SizeX = 0;
		int32 SizeY = 0;
		TArray<T> Data;

		void Init(const T& Value, int32 InSizeX, int32 InSizeY)
		{
			SizeX = InSizeX;
			SizeY = InSizeY;
			Data.Init(Value, SizeX * SizeY);
		}

		inline const T& Get(int32 X, int32 Y) const
		{
			check(X >= 0 && X < SizeX);
			check(Y >= 0 && Y < SizeY);
			return Data[SizeX * Y + X];
		}

		inline void Set(int32 X, int32 Y, const T& Value)
		{
			check(X >= 0 && X < SizeX);
			check(Y >= 0 && Y < SizeY);
			Data[SizeX * Y + X] = Value;
		}

	};
	
	bool ClassifyMeshVerticesInLayout(
		const FMesh& Mesh,
		const FLayout& InLayout,
		int32 LayoutChannel,
		bool bNormalizeUVs,
		bool bClampUVIslands,
		FMutableLayoutClassifyResult& OutResult
	)
	{
		MUTABLE_CPUPROFILER_SCOPE(ClassifyMeshVerticesInLayout);

		if (Mesh.GetVertexCount() == 0)
		{
			return false;
		}

		const FLayout* Layout = &InLayout;

		// The layout must have block ids.
		check(Layout->Blocks.IsEmpty() || Layout->Blocks[0].Id != FLayoutBlock::InvalidBlockId);

		const int32 NumVertices = Mesh.GetVertexCount();
		const int32 NumBlocks = Layout->GetBlockCount();

		// Find block ids for each block in the grid. Calculate a grid size that contains all blocks
		FIntPoint LayoutGrid = Layout->GetGridSize();
		FIntPoint WorkingGrid = LayoutGrid;
		for (const FLayoutBlock& Block : Layout->Blocks)
		{
			WorkingGrid.X = FMath::Max(WorkingGrid.X, Block.Min.X + Block.Size.X);
			WorkingGrid.Y = FMath::Max(WorkingGrid.Y, Block.Min.Y + Block.Size.Y);
		}


		TArray2D<int32> GridBlockBlockId;
		GridBlockBlockId.Init(MAX_uint16, WorkingGrid.X, WorkingGrid.Y);

		// Create an array of block index per cell
		TArray<int32> OverlappingBlocks;
		for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			bool bBlockHasMask = Layout->Blocks[BlockIndex].MaskIndex != FLayoutBlock::InvalidMaskIndex;

			// Fill the block rect
			const FIntVector2& Min = Layout->Blocks[BlockIndex].Min;
			const FIntVector2& Size = Layout->Blocks[BlockIndex].Size;

			// Fill the block index per cell array
			// Ignore the block in this stage if it has a mask, because blocks with masks will very likely overlap other blocks
			if (!bBlockHasMask)
			{
				for (uint16 Y = Min.Y; Y < Min.Y + Size.Y; ++Y)
				{
					for (uint16 X = Min.X; X < Min.X + Size.X; ++X)
					{
						if (GridBlockBlockId.Get(X, Y) == MAX_uint16)
						{
							GridBlockBlockId.Set(X, Y, BlockIndex);
						}
						else
						{
							OverlappingBlocks.AddUnique(BlockIndex);
						}
					}
				}
			}
		}

		// Notify Overlapping layout blocks
		if (!OverlappingBlocks.IsEmpty())
		{
			UE_LOGF(LogMutableCore, Warning, "Mesh has %d layout block overlapping in channel %d.", OverlappingBlocks.Num() + 1, LayoutChannel);
		}

		// Get the information about the texture coordinates channel
		int32 TexCoordsBufferIndex = -1;
		int32 TexCoordsChannelIndex = -1;
		Mesh.GetVertexBuffers().FindChannel(EMeshBufferSemantic::TexCoords, LayoutChannel, &TexCoordsBufferIndex, &TexCoordsChannelIndex);


		if (TexCoordsBufferIndex < 0 || TexCoordsChannelIndex < 0)
		{
			// This is actually fine when using shared materials across LODs
			UE_LOGF(LogMutableCore, Log, "Trying to generate layout for missing UV channel %d. No layout blocks generated.", LayoutChannel);
			return false;
		}

		const FMeshBufferChannel& TexCoordsChannel = Mesh.VertexBuffers.Buffers[TexCoordsBufferIndex].Channels[TexCoordsChannelIndex];
		check(TexCoordsChannel.Semantic == EMeshBufferSemantic::TexCoords);

		const uint8* TexCoordData = Mesh.GetVertexBuffers().GetBufferData(TexCoordsBufferIndex);
		int32 TexCoordElemSize = Mesh.GetVertexBuffers().GetElementSize(TexCoordsBufferIndex);
		int32 TexCoordChannelOffset = TexCoordsChannel.Offset;
		TexCoordData += TexCoordChannelOffset;

		// Get a copy of the UVs as FVector2f to work with them. 
		TArray<FVector2f>& TexCoords = OutResult.TexCoords;
		{
			MUTABLE_CPUPROFILER_SCOPE(CopyUVs);

			TexCoords.SetNumUninitialized(NumVertices);

			const uint8* pVertices = TexCoordData;
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				FVector2f& UV = TexCoords[VertexIndex];
				if (TexCoordsChannel.Format == EMeshBufferFormat::Float32)
				{
					UV = *((FVector2f*)pVertices);
				}
				else if (TexCoordsChannel.Format == EMeshBufferFormat::Float16)
				{
					const FFloat16* pUV = reinterpret_cast<const FFloat16*>(pVertices);
					UV = FVector2f(float(pUV[0]), float(pUV[1]));
				}

				pVertices += TexCoordElemSize;
			}
		}

		const bool bIsOverlayLayout = Layout->GetLayoutPackingStrategy() == UE::Mutable::Private::EPackStrategy::Overlay;
		if (bNormalizeUVs && !bIsOverlayLayout)
		{
			MUTABLE_CPUPROFILER_SCOPE(NormalizeUVs);
			bool bNonNormalizedUVs = false;

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				FVector2f& UV = TexCoords[VertexIndex];

				// Check that UVs are normalized. If not, clamp the values and throw a warning.
				if (UV[0] < 0.f || UV[0] > 1.f || UV[1] < 0.f || UV[1] > 1.f)
				{
					UV[0] = FMath::Clamp(UV[0], 0.f, 1.f);
					UV[1] = FMath::Clamp(UV[1], 0.f, 1.f);
					bNonNormalizedUVs = true;
				}
			}

			// Mutable does not support non-normalized UVs
			if (bNonNormalizedUVs)
			{
				UE_LOGF(LogMutableCore, Warning, "Source mesh has non-normalized UVs index %d", LayoutChannel);
			}
		}


		const uint32 MaxGridX = bNormalizeUVs ? MAX_uint32 : WorkingGrid.X - 1;
		const uint32 MaxGridY = bNormalizeUVs ? MAX_uint32 : WorkingGrid.Y - 1;

		// Allocate the per-vertex layout block data
		OutResult.LayoutData.SetNumUninitialized(NumVertices);

		// Assign a block to each vertex
		{

			if (Layout->Masks.IsEmpty())
			{
				MUTABLE_CPUPROFILER_SCOPE(Assign);
				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					FVector2f& UV = TexCoords[VertexIndex];

					int32 VertexWorkingGridX = FMath::Clamp(LayoutGrid.X * UV[0], 0, LayoutGrid.X - 1);
					int32 VertexWorkingGridY = FMath::Clamp(LayoutGrid.Y * UV[1], 0, LayoutGrid.Y - 1);
					uint32 ClampedX = FMath::Min<uint32>(MaxGridX, VertexWorkingGridX);
					uint32 ClampedY = FMath::Min<uint32>(MaxGridY, VertexWorkingGridY);

					OutResult.LayoutData[VertexIndex] = GridBlockBlockId.Get(ClampedX, ClampedY);
				}
			}
			else
			{
				MUTABLE_CPUPROFILER_SCOPE(AssignWithMask);
				// TODO Optimize (6x slower)
				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					uint16 BlockIndex = FMutableLayoutClassifyResult::NullBlockId;

					FVector2f& UV = TexCoords[VertexIndex];

					int32 VertexWorkingGridX = FMath::Clamp(LayoutGrid.X * UV[0], 0, LayoutGrid.X - 1);
					int32 VertexWorkingGridY = FMath::Clamp(LayoutGrid.Y * UV[1], 0, LayoutGrid.Y - 1);

					// First: Assign the vertices to masked blocks in order
					for (int32 CandidateBlockIndex = 0; CandidateBlockIndex < NumBlocks; ++CandidateBlockIndex)
					{
						const UE::Mutable::Private::FImage* Mask = nullptr;
						int32 MaskIndex = Layout->Blocks[CandidateBlockIndex].MaskIndex;
						if (Layout->Masks.IsValidIndex(MaskIndex))
						{
							Mask = Layout->Masks[MaskIndex].Get();
						}

						if (Mask)
						{
							// First discard with block limits.
							FIntVector2 Min = Layout->Blocks[CandidateBlockIndex].Min;
							FIntVector2 Size = Layout->Blocks[CandidateBlockIndex].Size;

							bool bInBlock =
								(VertexWorkingGridX >= Min.X && VertexWorkingGridX < Min.X + Size.X)
								&&
								(VertexWorkingGridY >= Min.Y && VertexWorkingGridY < Min.Y + Size.Y);

							if (bInBlock)
							{
								FVector2f SampleUV;
								SampleUV.X = FMath::Fmod(UV.X, 1.0);
								SampleUV.Y = FMath::Fmod(UV.Y, 1.0);

								FVector4f MaskValue = Mask->Sample(SampleUV);
								if (MaskValue.X > 0.5f)
								{
									BlockIndex = CandidateBlockIndex;
									break;
								}
							}
						}
					}

					// Second: Assign to non-masked blocks if not assigned yet
					if (BlockIndex == FMutableLayoutClassifyResult::NullBlockId)
					{
						uint32 ClampedX = FMath::Min<uint32>(MaxGridX, FMath::Max<uint32>(0, VertexWorkingGridX));
						uint32 ClampedY = FMath::Min<uint32>(MaxGridY, FMath::Max<uint32>(0, VertexWorkingGridY));
						BlockIndex = GridBlockBlockId.Get(ClampedX, ClampedY);
					}

					OutResult.LayoutData[VertexIndex] = BlockIndex;
				}
			}
		}

		// Correct UVs and block assignment if necessary
		const int32 NumTriangles = Mesh.GetIndexCount() / 3;
		TArray<int32> ConflictiveTriangles;

		if (bClampUVIslands)
		{
			MUTABLE_CPUPROFILER_SCOPE(DetectConflictiveTriangles);

			UntypedMeshBufferIteratorConst IndicesBegin(Mesh.GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);

			const EMeshBufferFormat IndicesFormat = IndicesBegin.GetFormat();
			check(IndicesBegin.GetFormat() == EMeshBufferFormat::UInt32 || IndicesBegin.GetFormat() == EMeshBufferFormat::UInt16);

			if (IndicesBegin.GetFormat() == EMeshBufferFormat::UInt32)
			{
				const uint32* Indices = reinterpret_cast<const uint32*>(IndicesBegin.ptr());
				for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					const uint16 BlockIndexV0 = OutResult.LayoutData[Indices[0]];
					const uint16 BlockIndexV1 = OutResult.LayoutData[Indices[1]];
					const uint16 BlockIndexV2 = OutResult.LayoutData[Indices[2]];

					if (BlockIndexV0 != BlockIndexV1 || BlockIndexV0 != BlockIndexV2)
					{
						ConflictiveTriangles.Add(TriangleIndex);
					}

					Indices += 3;
				}
			}
			else if (IndicesBegin.GetFormat() == EMeshBufferFormat::UInt16)
			{
				const uint16* Indices = reinterpret_cast<const uint16*>(IndicesBegin.ptr());
				for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					const uint16 BlockIndexV0 = OutResult.LayoutData[Indices[0]];
					const uint16 BlockIndexV1 = OutResult.LayoutData[Indices[1]];
					const uint16 BlockIndexV2 = OutResult.LayoutData[Indices[2]];

					if (BlockIndexV0 != BlockIndexV1 || BlockIndexV0 != BlockIndexV2)
					{
						ConflictiveTriangles.Add(TriangleIndex);
					}

					Indices += 3;
				}
			}
		}

		if (bClampUVIslands && !ConflictiveTriangles.IsEmpty())
		{
			MUTABLE_CPUPROFILER_SCOPE(ResolveConflictiveTriangles);
			
			TArray<FTriangleInfo> Triangles;

			// Vertices mapped to unique vertex index
			TArray<int32> CollapsedVertices;

			// Vertex to face map used to speed up connectivity building
			TMultiMap<int32, uint32> VertexToFaceMap;

			// Find Unique Vertices
			VertexToFaceMap.Reserve(NumTriangles * 3);
			Triangles.SetNumUninitialized(NumTriangles);

			MeshCreateCollapsedVertexMap(&Mesh, CollapsedVertices);

			UntypedMeshBufferIteratorConst ItIndices(Mesh.GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
			{
				MUTABLE_CPUPROFILER_SCOPE(CreateTriangles);

				for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					uint32 Index0 = ItIndices.GetAsUINT32();
					++ItIndices;
					uint32 Index1 = ItIndices.GetAsUINT32();
					++ItIndices;
					uint32 Index2 = ItIndices.GetAsUINT32();
					++ItIndices;

					const uint16 BlockIndexV0 = OutResult.LayoutData[Index0];
					const uint16 BlockIndexV1 = OutResult.LayoutData[Index1];
					const uint16 BlockIndexV2 = OutResult.LayoutData[Index2];

					FTriangleInfo& Triangle = Triangles[TriangleIndex];

					Triangle.Indices[0] = Index0;
					Triangle.Indices[1] = Index1;
					Triangle.Indices[2] = Index2;
					Triangle.CollapsedIndices[0] = CollapsedVertices[Index0];
					Triangle.CollapsedIndices[1] = CollapsedVertices[Index1];
					Triangle.CollapsedIndices[2] = CollapsedVertices[Index2];

					Triangle.BlockIndices[0] = BlockIndexV0;
					Triangle.BlockIndices[1] = BlockIndexV1;
					Triangle.BlockIndices[2] = BlockIndexV2;
					Triangle.bUVsFixed = false;

					VertexToFaceMap.Add(Triangle.CollapsedIndices[0], TriangleIndex);
					VertexToFaceMap.Add(Triangle.CollapsedIndices[1], TriangleIndex);
					VertexToFaceMap.Add(Triangle.CollapsedIndices[2], TriangleIndex);
				}
			}

			// Clamp UV islands to the predominant block of each island.
			{
				MUTABLE_CPUPROFILER_SCOPE(ClampUVs);

				for (int32 ConflictiveTriangleIndex : ConflictiveTriangles)
				{
					FTriangleInfo& Triangle = Triangles[ConflictiveTriangleIndex];

					// Skip the ones that have been fixed already
					if (Triangle.bUVsFixed)
					{
						continue;
					}

					// Find triangles from the same UV Island
					TArray<uint32> TriangleIndices;
					GetUVIsland(Triangles, ConflictiveTriangleIndex, TriangleIndices, TexCoords, VertexToFaceMap);

					// Get predominant BlockId != MAX_uint16
					TArray<uint32> NumVerticesPerBlock;
					NumVerticesPerBlock.SetNumZeroed(NumBlocks);

					for (int32 TriangleIndex : TriangleIndices)
					{
						FTriangleInfo& OtherTriangle = Triangles[TriangleIndex];
						for (int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
						{
							const uint16& BlockIndex = OtherTriangle.BlockIndices[VertexIndex];
							if (BlockIndex != MAX_uint16)
							{
								NumVerticesPerBlock[BlockIndex]++;
							}
						}
					}

					uint16 BlockIndex = 0;
					uint32 CurrentMaxVertices = 0;
					for (int32 Index = 0; Index < NumBlocks; ++Index)
					{
						if (NumVerticesPerBlock[Index] > CurrentMaxVertices)
						{
							BlockIndex = Index;
							CurrentMaxVertices = NumVerticesPerBlock[Index];
						}
					}

					// Get the limits of the predominant block rect
					const FLayoutBlock& LayoutBlock = Layout->Blocks[BlockIndex];

					const float SmallNumber = 0.000001f;
					const float MinX = ((float)LayoutBlock.Min.X) / (float)LayoutGrid.X + SmallNumber;
					const float MinY = ((float)LayoutBlock.Min.Y) / (float)LayoutGrid.Y + SmallNumber;
					const float MaxX = (((float)LayoutBlock.Size.X + LayoutBlock.Min.X) / (float)LayoutGrid.X) - 2 * SmallNumber;
					const float MaxY = (((float)LayoutBlock.Size.Y + LayoutBlock.Min.Y) / (float)LayoutGrid.Y) - 2 * SmallNumber;

					// Iterate triangles and clamp the UVs
					for (int32 TriangleIndex : TriangleIndices)
					{
						FTriangleInfo& OtherTriangle = Triangles[TriangleIndex];

						for (int8 VertexIndex = 0; VertexIndex < 3; ++VertexIndex)
						{
							if (OtherTriangle.BlockIndices[VertexIndex] == BlockIndex)
							{
								continue;
							}

							OtherTriangle.BlockIndices[VertexIndex] = BlockIndex;

							// Clamp UVs to the block they are assigned to
							const int32 UVIndex = OtherTriangle.Indices[VertexIndex];
							FVector2f& UV = TexCoords[UVIndex];
							UV[0] = FMath::Clamp(UV[0], MinX, MaxX);
							UV[1] = FMath::Clamp(UV[1], MinY, MaxY);
							OutResult.LayoutData[UVIndex] = BlockIndex;
						}

						OtherTriangle.bUVsFixed = true;
					}
				}
			}
		}

		// Warn about vertices without a block id
		//int32 FirstLODToIgnoreWarnings = GeneratedLayout.Source->FirstLODToIgnoreWarnings;
		//if (FirstLODToIgnoreWarnings == -1 || StaticMeshOptions.LODIndex < FirstLODToIgnoreWarnings)
		//{
		//	TArray<float> UnassignedUVs;
		//	UnassignedUVs.Reserve(NumVertices / 100);

		//	const FVector2f* UVs = TexCoords.GetData();
		//	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		//	{
		//		if (LayoutData[VertexIndex] == MAX_uint16)
		//		{
		//			UnassignedUVs.Add((*(UVs + VertexIndex))[0]);
		//			UnassignedUVs.Add((*(UVs + VertexIndex))[1]);
		//		}
		//	}

		//	if (!UnassignedUVs.IsEmpty())
		//	{
		//		// TODO: How do we translate this to editor-time info?
		//		UE_LOGF(LogMutableCore, Warning, "Source mesh has %d vertices not assigned to any layout block in LOD %d in UVs index %d", UnassignedUVs.Num(), LayoutChannel);
		//	}
		//}

		return true;
	}


	void MeshPrepareLayout(
		FMesh& Mesh,
		const FLayout& InLayout,
		int32 LayoutChannel,
		bool bNormalizeUVs,
		bool bClampUVIslands,
		bool bEnsureAllVerticesHaveLayoutBlock,
		bool bUseAbsoluteBlockIds
	)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshPrepareLayout);

		if (Mesh.GetVertexCount() == 0)
		{
			return;
		}

		TManagedPtr<FLayout> Layout = InLayout.Clone();

		// The layout must have block ids.
		check(Layout->Blocks.IsEmpty() || Layout->Blocks[0].Id != FLayoutBlock::InvalidBlockId);

		Mesh.AddLayout(Layout);

		const int32 NumVertices = Mesh.GetVertexCount();
		const int32 NumBlocks = Layout->GetBlockCount();

		FMutableLayoutClassifyResult ClassifyResult;
		if (!ClassifyMeshVerticesInLayout(Mesh, *Layout, LayoutChannel,
			bNormalizeUVs,
			bClampUVIslands,
			ClassifyResult))
		{
			// Classify mesh vertices failed. Missing texture coords?
			return;
		}

		{
			MUTABLE_CPUPROFILER_SCOPE(CreateBuffers);

			// Create the layout block vertex buffer
			uint8* LayoutBufferPtr = nullptr;
			{
				const int32 LayoutBufferIndex = Mesh.GetVertexBuffers().GetBufferCount();
				Mesh.GetVertexBuffers().SetBufferCount(LayoutBufferIndex + 1);

				// TODO
				check(Layout->GetBlockCount() < MAX_uint16);
				const EMeshBufferSemantic LayoutSemantic = EMeshBufferSemantic::LayoutBlock;
				const int32 LayoutSemanticIndex = int32(LayoutChannel);
				const EMeshBufferFormat LayoutFormat = bUseAbsoluteBlockIds ? EMeshBufferFormat::UInt64 : EMeshBufferFormat::UInt16;
				const int32 LayoutComponents = 1;
				const int32 LayoutOffset = 0;
				int32 ElementSize = bUseAbsoluteBlockIds ? sizeof(uint64) : sizeof(uint16);
				Mesh.GetVertexBuffers().SetBuffer
				(
					LayoutBufferIndex,
					ElementSize,
					1,
					&LayoutSemantic, &LayoutSemanticIndex,
					&LayoutFormat, &LayoutComponents,
					&LayoutOffset
				);
				LayoutBufferPtr = Mesh.GetVertexBuffers().GetBufferData(LayoutBufferIndex);
			}

			{
				MUTABLE_CPUPROFILER_SCOPE(LayoutAndHomogenize);

				// 
				const  FIntPoint& LayoutGrid = Layout->GetGridSize();
				TArray<box<FVector2f>> BlockRects;
				BlockRects.SetNumUninitialized(NumBlocks);
				for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
				{
					bool bBlockHasMask = Layout->Blocks[BlockIndex].MaskIndex != FLayoutBlock::InvalidMaskIndex;

					// Fill the block rect
					const FIntVector2& Min = Layout->Blocks[BlockIndex].Min;
					const FIntVector2& Size = Layout->Blocks[BlockIndex].Size;

					box<FVector2f>& BlockRect = BlockRects[BlockIndex];
					BlockRect.min[0] = float(Min.X) / float(LayoutGrid.X);
					BlockRect.min[1] = float(Min.Y) / float(LayoutGrid.Y);
					BlockRect.size[0] = float(Size.X) / float(LayoutGrid.X);
					BlockRect.size[1] = float(Size.Y) / float(LayoutGrid.Y);
				}


				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					FVector2f& UV = ClassifyResult.TexCoords[VertexIndex];

					uint16 LayoutBlockIndex = ClassifyResult.LayoutData[VertexIndex];
					if (Layout->Blocks.IsValidIndex(LayoutBlockIndex))
					{
						uint64 LayoutBlockId = Layout->Blocks[LayoutBlockIndex].Id;

						UV = BlockRects[LayoutBlockIndex].Homogenize(UV);

						// Replace block index by the actual id of the block
						if (bUseAbsoluteBlockIds)
						{
							uint64* Ptr = reinterpret_cast<uint64*>(LayoutBufferPtr) + VertexIndex;
							*Ptr = LayoutBlockId;
						}
						else
						{
							uint16* Ptr = reinterpret_cast<uint16*>(LayoutBufferPtr) + VertexIndex;
							*Ptr = uint16(LayoutBlockId & 0xffff);
						}
					}
					else
					{
						// Map vertices without block
						if (bUseAbsoluteBlockIds)
						{
							uint64* Ptr = reinterpret_cast<uint64*>(LayoutBufferPtr) + VertexIndex;
							*Ptr = bEnsureAllVerticesHaveLayoutBlock ? 0 : std::numeric_limits<uint64>::max();
						}
						else
						{
							uint16* Ptr = reinterpret_cast<uint16*>(LayoutBufferPtr) + VertexIndex;
							*Ptr = bEnsureAllVerticesHaveLayoutBlock ? 0 : std::numeric_limits<uint16>::max();
						}
					}
				}
			}

			{
				MUTABLE_CPUPROFILER_SCOPE(CopyUVs);

				// Get the information about the texture coordinates channel
				int32 TexCoordsBufferIndex = -1;
				int32 TexCoordsChannelIndex = -1;
				Mesh.GetVertexBuffers().FindChannel(EMeshBufferSemantic::TexCoords, LayoutChannel, &TexCoordsBufferIndex, &TexCoordsChannelIndex);

				if (TexCoordsBufferIndex < 0 || TexCoordsChannelIndex < 0)
				{
					// This is actually fine when using shared materials across LODs
					UE_LOGF(LogMutableCore, Log, "Trying to generate layout for missing UV channel %d. No layout blocks generated.", LayoutChannel);
					return;
				}

				const FMeshBufferChannel& TexCoordsChannel = Mesh.VertexBuffers.Buffers[TexCoordsBufferIndex].Channels[TexCoordsChannelIndex];
				check(TexCoordsChannel.Semantic == EMeshBufferSemantic::TexCoords);

				uint8* TexCoordData = Mesh.GetVertexBuffers().GetBufferData(TexCoordsBufferIndex);
				int32 TexCoordElemSize = Mesh.GetVertexBuffers().GetElementSize(TexCoordsBufferIndex);
				int32 TexCoordChannelOffset = TexCoordsChannel.Offset;
				TexCoordData += TexCoordChannelOffset;


				// Copy UVs
				if (TexCoordsChannel.Format == EMeshBufferFormat::Float32 && TexCoordElemSize ==sizeof(float)*2)
				{
					FMemory::Memcpy(TexCoordData, ClassifyResult.TexCoords.GetData(), sizeof(float) * 2 * NumVertices);
				}
				else
				{
					MUTABLE_CPUPROFILER_SCOPE(SlowPath);

					for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
					{
						FVector2f& UV = ClassifyResult.TexCoords[VertexIndex];

						if (TexCoordsChannel.Format == EMeshBufferFormat::Float32)
						{
							FVector2f* pUV = reinterpret_cast<FVector2f*>(TexCoordData);
							*pUV = UV;
						}
						else if (TexCoordsChannel.Format == EMeshBufferFormat::Float16)
						{
							FFloat16* pUV = reinterpret_cast<FFloat16*>(TexCoordData);
							pUV[0] = FFloat16(UV[0]);
							pUV[1] = FFloat16(UV[1]);
						}

						TexCoordData += TexCoordElemSize;
					}
				}
			}
		}
	}

}
