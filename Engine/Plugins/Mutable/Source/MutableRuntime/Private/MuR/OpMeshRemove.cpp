// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshRemove.h"

#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"
#include "MuR/ParallelExecutionUtils.h"
#include "MuR/Platform.h"

#include "Templates/UnrealTemplate.h"
#include "Rendering/MorphTargetVertexCodec.h"

namespace UE::Mutable::Private
{
	int32 RemoveBufferUnusedElems(uint8* BufferData, uint32 ElemSize, int32 NumElems, const TBitArray<>& UsedElems)
	{
    	if (ElemSize == 0)
    	{
    		return 0;
    	}
    	
		int32 LastFreeElemIndex = 0;

		for (int32 ElemIndex = UsedElems.FindFrom(true, 0); ElemIndex >= 0;)
		{
			const int32 UsedSpanBegin = ElemIndex;
			ElemIndex = UsedElems.FindFrom(false, ElemIndex);
			ElemIndex = ElemIndex >= 0 ? ElemIndex : NumElems;
			const int32 UsedSpanEnd = ElemIndex;

			ElemIndex = UsedElems.FindFrom(true, ElemIndex);

			// Copy vertex buffer span.	
			const int32 UsedSpanSize = UsedSpanEnd - UsedSpanBegin;
            
			if (LastFreeElemIndex != UsedSpanBegin)
			{
				FMemory::Memmove(
						BufferData + LastFreeElemIndex*ElemSize,
						BufferData + UsedSpanBegin*ElemSize,
						UsedSpanSize*ElemSize);
			}

			LastFreeElemIndex += UsedSpanSize;
		}

		return LastFreeElemIndex;
	}


	void RemoveMorphBatchVertices(
		UE::MorphTargetVertexCodec::FDeltaBatchHeader& InOutBatchHeader, 
		TArrayView<uint32> DataView, 
		int32 BatchVertexRangeBegin, 
		int32 BatchVertexRangeEnd,
		const TBitArray<>& UsedVertices,
		TConstArrayView<int32> VertexRemap,
		int32 NumResultVertices
	)
	{
		check(VertexRemap.Num() == UsedVertices.Num());

		UE::MorphTargetVertexCodec::FQuantizedDelta QuantizedDeltasReadBuffer[UE::MorphTargetVertexCodec::BatchSize];

		int32 NumUsedVertices = UsedVertices.CountSetBits(BatchVertexRangeBegin, BatchVertexRangeEnd); 

		if (NumUsedVertices == 0)
		{
			// All vertices removed.
			InOutBatchHeader.IndexMin = 0;
			InOutBatchHeader.NumElements = 0;
			InOutBatchHeader.DataOffset = 0;
		}
		else if (NumUsedVertices == BatchVertexRangeEnd - BatchVertexRangeBegin)
		{
			// No vertices removed, remap Batch IndexMin.
			InOutBatchHeader.IndexMin = VertexRemap[InOutBatchHeader.IndexMin];
			check(InOutBatchHeader.IndexMin != INDEX_NONE);
		}
		else
		{
			TArrayView<uint32> BatchDataView = TArrayView<uint32>(
					DataView.GetData() + (InOutBatchHeader.DataOffset/sizeof(uint32)), 
					CalculateBatchDwords(InOutBatchHeader)); 

			check(InOutBatchHeader.NumElements <= UE::MorphTargetVertexCodec::BatchSize);
			TArrayView<UE::MorphTargetVertexCodec::FQuantizedDelta> BatchQuantizedDeltas(
					QuantizedDeltasReadBuffer, InOutBatchHeader.NumElements);

			ReadQuantizedDeltas(BatchQuantizedDeltas, InOutBatchHeader, BatchDataView); 
			
			const int32 NumDeltas = InOutBatchHeader.NumElements;
			check(NumDeltas <= UE::MorphTargetVertexCodec::BatchSize);

			InOutBatchHeader.IndexMin = 0;
		 
			int32 DeltaIndex = 0;
			for ( ; DeltaIndex < NumDeltas; ++DeltaIndex)
			{
				check((int32)BatchQuantizedDeltas[DeltaIndex].Index < UsedVertices.Num());
				
				if (UsedVertices[BatchQuantizedDeltas[DeltaIndex].Index])
				{
					int32 RemappedDeltaVertexIdx = VertexRemap[BatchQuantizedDeltas[DeltaIndex].Index];
					check(RemappedDeltaVertexIdx < NumResultVertices);
					InOutBatchHeader.IndexMin = (uint32)RemappedDeltaVertexIdx;

					break;
				}
			}

			UE::MorphTargetVertexCodec::FQuantizedDelta QuantizedDeltasKeepBuffer[UE::MorphTargetVertexCodec::BatchSize];

			uint32 DestDeltaIndex = 0;
			for ( ; DeltaIndex < NumDeltas; ++DeltaIndex)
			{
				if (UsedVertices[BatchQuantizedDeltas[DeltaIndex].Index])
				{
					check((int32)BatchQuantizedDeltas[DeltaIndex].Index < VertexRemap.Num());
					int32 RemappedDeltaVertexIdx = VertexRemap[BatchQuantizedDeltas[DeltaIndex].Index];

					check(RemappedDeltaVertexIdx < NumResultVertices);

					QuantizedDeltasKeepBuffer[DestDeltaIndex++] = UE::MorphTargetVertexCodec::FQuantizedDelta
					{
						.Position = BatchQuantizedDeltas[DeltaIndex].Position,
						.TangentZ = BatchQuantizedDeltas[DeltaIndex].TangentZ,
						.Index    = (uint32)RemappedDeltaVertexIdx,
					};	
				}
			}

			InOutBatchHeader.NumElements = DestDeltaIndex;

			if (InOutBatchHeader.NumElements == 0)
			{
				InOutBatchHeader.IndexMin = 0;
				InOutBatchHeader.DataOffset = 0;
			}
			else
			{
				check(DestDeltaIndex <= UE::MorphTargetVertexCodec::BatchSize);
				WriteQuantizedDeltas(TArrayView(QuantizedDeltasKeepBuffer, DestDeltaIndex), InOutBatchHeader, BatchDataView);
			}
		}
#if 0 // Check data integrity.
		{
			TArray<UE::MorphTargetVertexCodec::FQuantizedDelta, TInlineAllocator<UE::MorphTargetVertexCodec::BatchSize>> TestQuantizedDeltasStorage;
			TestQuantizedDeltasStorage.SetNum(UE::MorphTargetVertexCodec::BatchSize);

			check(InOutBatchHeader.NumElements <= UE::MorphTargetVertexCodec::BatchSize); 

			TArrayView<UE::MorphTargetVertexCodec::FQuantizedDelta> TestBatchQuantizedDeltas(
					TestQuantizedDeltasStorage.GetData(), InOutBatchHeader.NumElements);

			ReadQuantizedDeltas(TestBatchQuantizedDeltas, InOutBatchHeader, BatchDataView); 

			for (int32 TestDeltaIndex = 0; TestDeltaIndex < InOutBatchHeader.NumElements; ++TestDeltaIndex)
			{
				check(TestBatchQuantizedDeltas[TestDeltaIndex].Index < NumResultVertices);		
			}
		}
#endif

	}


	void MeshRemoveMeshMorphs(FMesh& Result, const TBitArray<>& UsedVertices, TConstArrayView<int32> VertexIndexRemap)
	{
		MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_RemoveMesh)

		using namespace UE::MorphTargetVertexCodec;

		if (!Result.MorphDataBuffer.Num())
		{
			return;
		}

		int32 NumMorphs = Result.Morph.Names.Num();
		TArrayView<uint32> DataView = Result.MorphDataBuffer; 
		uint32* MorphDataPtr = DataView.GetData();
		
		// Init to false so any empty batch coming from other operations are also removed.
		TBitArray<> BatchesToKeep;
		BatchesToKeep.Init(false, Result.Morph.NumTotalBatches);
		{
			MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_RemoveMesh_RemoveVertices)
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				const int32 NumBatches = Result.Morph.BatchesPerMorph[MorphIndex];
				
				if (NumBatches == 0)
				{
					continue;
				}

				int32 BatchStartOffset = Result.Morph.BatchStartOffsetPerMorph[MorphIndex];
	
				FDeltaBatchHeader BatchHeaderStorage[2];

				FDeltaBatchHeader* CurrBatchHeader = &BatchHeaderStorage[0];
				FDeltaBatchHeader* NextBatchHeader = &BatchHeaderStorage[1];
				
				int32 CurrBatchIndex = 0;
				//Find first non empty batch.
				for (; CurrBatchIndex < NumBatches; ++CurrBatchIndex)
				{
					ReadHeader(*CurrBatchHeader, TArrayView<uint32>(MorphDataPtr + ((BatchStartOffset + CurrBatchIndex) * NumBatchHeaderDwords), NumBatchHeaderDwords));
 
					if (CurrBatchHeader->NumElements)
					{
						break;
					}
				}

				const int32 NumResultVertices = Result.GetVertexCount(); 
				const int32 NumSourceVertices = VertexIndexRemap.Num();
				for (int32 NextBatchIndex = CurrBatchIndex + 1; NextBatchIndex <= NumBatches; ++NextBatchIndex)
				{
					// Find next non empty batch.
					for (; NextBatchIndex < NumBatches; ++NextBatchIndex)
					{
						ReadHeader(*NextBatchHeader, TArrayView<uint32>(MorphDataPtr + (BatchStartOffset + NextBatchIndex) * NumBatchHeaderDwords, NumBatchHeaderDwords));

						if (NextBatchHeader->NumElements)
						{
							break;
						}
					}
			
					int32 BatchRangeBegin = CurrBatchHeader->IndexMin;
					int32 BatchRangeEnd   = NextBatchIndex < NumBatches ? NextBatchHeader->IndexMin : NumSourceVertices; 	

					RemoveMorphBatchVertices(*CurrBatchHeader, DataView, BatchRangeBegin, BatchRangeEnd, UsedVertices, VertexIndexRemap, NumResultVertices);

					WriteHeader(*CurrBatchHeader, TArrayView<uint32>(MorphDataPtr + ((BatchStartOffset + CurrBatchIndex) * NumBatchHeaderDwords), NumBatchHeaderDwords));
					BatchesToKeep[BatchStartOffset + CurrBatchIndex] = CurrBatchHeader->NumElements != 0;

					::Swap(CurrBatchHeader, NextBatchHeader);
					CurrBatchIndex = NextBatchIndex;
				}
			}
		}

		int32 CumulativeDataOffsetInDwords = 0;
		int32 CumulativeBatchStartOffset = 0;
		{
			MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_RemoveMesh_MoveHeadersAndMetadata);

			CumulativeDataOffsetInDwords = RemoveBufferUnusedElems(
					(uint8*)MorphDataPtr, NumBatchHeaderDwords*sizeof(uint32), BatchesToKeep.Num(), BatchesToKeep) * NumBatchHeaderDwords; 

			// Fix Morph metadata.
			for (int32 MorphIndex = 0; MorphIndex < NumMorphs; ++MorphIndex)
			{
				int32 MorphBatchesBegin = Result.Morph.BatchStartOffsetPerMorph[MorphIndex];
				int32 MorphBatchesEnd   = MorphBatchesBegin + Result.Morph.BatchesPerMorph[MorphIndex];

				int32 NumMorphBatches = BatchesToKeep.CountSetBits(MorphBatchesBegin, MorphBatchesEnd);
				Result.Morph.BatchesPerMorph[MorphIndex] = NumMorphBatches; 

				Result.Morph.BatchStartOffsetPerMorph[MorphIndex] = CumulativeBatchStartOffset;
				CumulativeBatchStartOffset += NumMorphBatches;
			}

			Result.Morph.NumTotalBatches = CumulativeBatchStartOffset;
		}
		{
			MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_RemoveMesh_MoveData);

			check(CumulativeDataOffsetInDwords == CumulativeBatchStartOffset*NumBatchHeaderDwords);
			for (int32 BatchIndex = 0; BatchIndex < CumulativeBatchStartOffset; ++BatchIndex)
			{
				const int32 BatchDataOffsetInDwords = BatchIndex*NumBatchHeaderDwords;

				FDeltaBatchHeader Header;
				ReadHeader(Header, TArrayView<uint32>(MorphDataPtr + BatchDataOffsetInDwords, NumBatchHeaderDwords));
				int32 NumBatchDataDwords = CalculateBatchDwords(Header);
			
				int32 OriginalDataOffsetInDwords = Header.DataOffset / sizeof(uint32);
				if (CumulativeDataOffsetInDwords != OriginalDataOffsetInDwords)
				{
					check(CumulativeDataOffsetInDwords + NumBatchDataDwords <= Result.MorphDataBuffer.Num());
					FMemory::Memmove(
							MorphDataPtr + CumulativeDataOffsetInDwords,
							MorphDataPtr + OriginalDataOffsetInDwords,
							NumBatchDataDwords * sizeof(uint32));
				
					// First Batch dword corresponds to the data offset. See MorphTargetVertexCodec::WriteHeader()
					MorphDataPtr[BatchIndex*NumBatchHeaderDwords] = CumulativeDataOffsetInDwords * sizeof(uint32);
				}

				CumulativeDataOffsetInDwords += NumBatchDataDwords;
			}	
		}

		// Trim unused bytes. Don't free memory here, delay it until a copy is done.
		{
			MUTABLE_CPUPROFILER_SCOPE(CompressedMorphs_RemoveMesh_Trim);
			Result.MorphDataBuffer.SetNum(CumulativeDataOffsetInDwords, EAllowShrinking::No);
		}
	}

	void MeshRemoveSkinWeightProfiles(FMesh* Result, const TBitArray<>& UsedVertices, TConstArrayView<int32> VertexIndexRemap)
	{
		const int32 NumProfiles = Result->SkinWeightProfiles.Num();
		if (NumProfiles == 0)
		{
			return;
		}

		MUTABLE_CPUPROFILER_SCOPE(MeshRemove_SkinWeightProfiles);

		ParallelExecutionUtils::InvokeBatchParallelForUnbalanced(NumProfiles, [Result,
			&UsedVertices,
			VertexIndexRemap](int32 ProfileIndex)
			{
				FSkinWeightProfile& Profile = Result->SkinWeightProfiles[ProfileIndex];
				
				if (Profile.VertexIndexToInfluenceOffset.IsEmpty())
				{
					return;
				}

				if (Profile.NumBoneInfluences == 0)
				{
					check(false);
					return;
				}

				FSkinWeightProfile::FVertexInfo* Source = Profile.VertexIndexToInfluenceOffset.GetData();
				FSkinWeightProfile::FVertexInfo* Dest = Profile.VertexIndexToInfluenceOffset.GetData();

				int32 RemainingVertices = 0;

				// Iterate and remove unused vertices. VertexIndexToInfluenceOffset is not 1 : 1 with the number of vertices in the mesh.
				const int32 NumProfileVertices = Profile.VertexIndexToInfluenceOffset.Num();
				for (int32 VertexIndex = 0; VertexIndex < NumProfileVertices; )
				{
					int32 LastVertexIndex = VertexIndex;

					while (LastVertexIndex < NumProfileVertices &&
						UsedVertices[(Source + LastVertexIndex)->VertexIndex])
					{
						(Source + LastVertexIndex)->VertexIndex = VertexIndexRemap[(Source + LastVertexIndex)->VertexIndex];
						check((Source + LastVertexIndex)->VertexIndex >= 0);

						++LastVertexIndex;
					}

					if (LastVertexIndex != VertexIndex)
					{
						int32 SpanToCopy = (LastVertexIndex - VertexIndex);

						FMemory::Memmove(Dest + RemainingVertices, Source + VertexIndex, SpanToCopy * sizeof(FSkinWeightProfile::FVertexInfo));

						RemainingVertices += SpanToCopy;
						VertexIndex += SpanToCopy;
					}
					else
					{
						++VertexIndex;
					}
				}

				if (RemainingVertices > 0)
				{
					Profile.VertexIndexToInfluenceOffset.SetNum(RemainingVertices, EAllowShrinking::No);
				}
				else
				{
					Profile.BoneIDs.Empty();
					Profile.BoneWeights.Empty();
					Profile.VertexIndexToInfluenceOffset.Empty();
				}
			});
		
	}

    void MeshRemoveRecreateSurface(FMesh* Result, const TBitArray<>& UsedVertices, const TBitArray<>& UsedFaces)
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshRemoveRecreateSurface);
		TArray<FSurfaceSubMesh, TInlineAllocator<32>> OrigSubMeshes;
        for (FMeshSurface& ResultSurf : Result->Surfaces)
        {
			OrigSubMeshes = ResultSurf.SubMeshes;
			ResultSurf.SubMeshes.Reset();            

			int32 PrevMeshRangeVertexEnd = 0;
            int32 PrevMeshRangeIndexEnd = 0;
            for (const FSurfaceSubMesh& SubMesh : OrigSubMeshes)
            {
                int32 MeshRangeVertexEnd = PrevMeshRangeVertexEnd; 
                int32 MeshRangeIndexEnd = PrevMeshRangeIndexEnd;
               
                MeshRangeVertexEnd += UsedVertices.CountSetBits(SubMesh.VertexBegin, SubMesh.VertexEnd);
				
                // Only add the mesh if it has remaining vertices. 
                if (MeshRangeVertexEnd > PrevMeshRangeVertexEnd)
                {
					check(SubMesh.IndexBegin % 3 == 0);
					check((SubMesh.IndexEnd - SubMesh.IndexBegin) % 3 == 0);
                    MeshRangeIndexEnd += UsedFaces.CountSetBits(SubMesh.IndexBegin/3, SubMesh.IndexEnd/3)*3;

                    ResultSurf.SubMeshes.Emplace(FSurfaceSubMesh 
                            {
                                PrevMeshRangeVertexEnd, MeshRangeVertexEnd,
                                PrevMeshRangeIndexEnd, MeshRangeIndexEnd 
                            });
                }

                PrevMeshRangeVertexEnd = MeshRangeVertexEnd;
                PrevMeshRangeIndexEnd = MeshRangeIndexEnd;
            }
        }

		// Remove Empty surfaces but always keep the first one. 
		// The previous step has eliminated empty submeshes, so it is only needed to check if the surface has 
		// any submesh.
		
		TArray<int32, TInlineAllocator<16>> SurfaceIndexRemap;
		SurfaceIndexRemap.Init(-1, Result->Surfaces.Num());

		int32 NumRemovedSurfaces = 0;
		for (int32 I = 0; I < Result->Surfaces.Num(); ++I)
		{
			bool bSurfaceRemoved = !Result->Surfaces[I].SubMeshes.Num();
			NumRemovedSurfaces += static_cast<int32>(bSurfaceRemoved);

			if (!bSurfaceRemoved)
			{
				SurfaceIndexRemap[I] = I - NumRemovedSurfaces; 
			}
		}

		if (NumRemovedSurfaces > 0)
		{
			for (int32 I = Result->Surfaces.Num() - 1; I >= 1; --I)
			{
				if (SurfaceIndexRemap[I] < 0)
				{
					Result->Surfaces.RemoveAt(I, EAllowShrinking::No);
					Result->ClothSections.RemoveAt(I, EAllowShrinking::No);
				}
			}

			// Remove and remap surface indces in use from Morph.
			for (int32 MorphIndex = Result->Morph.SurfacesInUsePerMorph.Num() - 1; MorphIndex >= 0; --MorphIndex)
			{
				for (int32 SurfIndex = SurfaceIndexRemap.Num() - 1; SurfIndex >= 0; --SurfIndex)
				{
					if (SurfaceIndexRemap[SurfIndex] < 0)
					{
						Result->Morph.SurfacesInUsePerMorph[MorphIndex].Remove(SurfIndex);
					}
				}

				for (int32& SurfaceInUseIndex : Result->Morph.SurfacesInUsePerMorph[MorphIndex]) 
				{
					check(SurfaceIndexRemap[SurfaceInUseIndex] >= 0);
					SurfaceInUseIndex = SurfaceIndexRemap[SurfaceInUseIndex];
				}
			}
		}

		check(Result->Surfaces.Num() >= 1);
	
		// Add a defaulted empty submesh if the surface is empty. A surface always needs a submesh 
		// even if empty.
		if (!Result->Surfaces[0].SubMeshes.Num())
		{
			Result->Surfaces[0].SubMeshes.Emplace();
		}
    }

    //---------------------------------------------------------------------------------------------
    struct FIdInterval
    {
        uint64 idStart;
        int32 idPosition;
        int32 size;
    };


    void ExtractVertexIndexIntervals( TArray<FIdInterval>& intervals, const FMesh* Source )
    {
		MeshVertexIdIteratorConst itVI(Source);
		FIdInterval current;
		current.idStart = FMesh::InvalidVertexId;
		current.idPosition = 0;
		current.size = 0;
		for ( int32 sv=0; sv< Source->GetVertexBuffers().GetElementCount(); ++sv )
        {
            uint64 id = itVI.Get();
            ++itVI;

            if (current.idStart== FMesh::InvalidVertexId)
            {
                current.idStart = id;
                current.idPosition = sv;
                current.size = 1;
            }
            else
            {
                if (id==current.idStart+current.size)
                {
                    ++current.size;
                }
                else
                {
                    intervals.Add(current);
                    current.idStart = id;
                    current.idPosition = sv;
                    current.size = 1;
                }
            }
        }

        if (current.idStart!= FMesh::InvalidVertexId)
        {
            intervals.Add(current);
        }
    }


    int64 FindPositionInIntervals( const TArray<FIdInterval>& intervals, int64 id )
    {
        for( const FIdInterval& interval: intervals )
        {
            int64 deltaId = id - interval.idStart;
            if (deltaId>=0 && deltaId<interval.size)
            {
                return interval.idPosition+deltaId;
            }
        }
        return -1;
    }	
	
    void MeshRemoveVerticesWithCullSet(FMesh* Result, const TBitArray<>& VerticesToCull, bool bRemoveIfAllVerticesCulled)
    { 
        MUTABLE_CPUPROFILER_SCOPE(MeshRemoveVerticesWithCullSet);
		
        const UntypedMeshBufferIterator IndicesBegin(Result->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
		
        const int32 NumFaces = Result->GetFaceCount();
        const int32 NumVertices = Result->GetVertexCount();

		const uint32 IndexTypeSize = IndicesBegin.GetElementSize();
		check(IndexTypeSize == 4 || IndexTypeSize == 2);

		static constexpr int32 MaxNumBatches = 5;
		TStaticArray<TBitArray<>, MaxNumBatches> ParallelUsedVertices;
		TStaticArray<TBitArray<>, MaxNumBatches> ParallelUsedFaces;

		const TBitArray<>& UsedVertices = ParallelUsedVertices[0];
		const TBitArray<>& UsedFaces = ParallelUsedFaces[0];
		
		ParallelUsedFaces[0].Reserve(NumFaces);

		{
			const bool bUseBatches = NumFaces > (1 << 13);
			const int32 NumBatches = bUseBatches ? MaxNumBatches : 1;
			const int32 NumFacesPerBatch = FMath::DivideAndRoundUp<int32>(NumFaces, NumBatches);

			ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
				[
					IndicesBegin,
					IndexTypeSize,
					&VerticesToCull,
					&ParallelUsedVertices,
					&ParallelUsedFaces,
					NumFaces,
					NumFacesPerBatch,
					NumVertices,
					bRemoveIfAllVerticesCulled
				](int32 BatchId)
			{
				const int32 BatchBeginFaceIndex = BatchId * NumFacesPerBatch;
				const int32 BatchEndFaceIndex = FMath::Min(BatchBeginFaceIndex + NumFacesPerBatch, NumFaces);
				const int32 BatchNumFaces = BatchEndFaceIndex - BatchBeginFaceIndex;

				TBitArray<>& UsedFaces = ParallelUsedFaces[BatchId];
				UsedFaces.SetNum(BatchNumFaces, false);

				TBitArray<>& UsedVertices = ParallelUsedVertices[BatchId];
				UsedVertices.SetNum(NumVertices, false);

				if (IndexTypeSize == 4)
				{
					const uint32* FaceIndicesData = reinterpret_cast<uint32*>((IndicesBegin + BatchBeginFaceIndex * 3).ptr());
					for (int32 FaceIndex = 0; FaceIndex < BatchNumFaces; ++FaceIndex)
					{
						bool bRemoved = false;

						if (bRemoveIfAllVerticesCulled)
						{
							const bool bAllVertsRemoved =
								VerticesToCull[FaceIndicesData[0]] &
								VerticesToCull[FaceIndicesData[1]] &
								VerticesToCull[FaceIndicesData[2]];
							bRemoved = bAllVertsRemoved;
						}
						else
						{
							const bool bOneVertRemoved =
								VerticesToCull[FaceIndicesData[0]] |
								VerticesToCull[FaceIndicesData[1]] |
								VerticesToCull[FaceIndicesData[2]];
							bRemoved = bOneVertRemoved;
						}

						if (!bRemoved)
						{
							UsedFaces[FaceIndex] = true;

							UsedVertices[FaceIndicesData[0]] = true;
							UsedVertices[FaceIndicesData[1]] = true;
							UsedVertices[FaceIndicesData[2]] = true;
						}

						FaceIndicesData += 3;
					}
				}
				else if (IndexTypeSize == 2)
				{
					const uint16* FaceIndicesData = reinterpret_cast<uint16*>((IndicesBegin + BatchBeginFaceIndex * 3).ptr());

					for (int32 FaceIndex = 0; FaceIndex < BatchNumFaces; ++FaceIndex)
					{
						bool bRemoved = false;

						if (bRemoveIfAllVerticesCulled)
						{
							const bool bAllVertsRemoved =
								VerticesToCull[FaceIndicesData[0]] &
								VerticesToCull[FaceIndicesData[1]] &
								VerticesToCull[FaceIndicesData[2]];
							bRemoved = bAllVertsRemoved;
						}
						else
						{
							const bool bOneVertRemoved =
								VerticesToCull[FaceIndicesData[0]] |
								VerticesToCull[FaceIndicesData[1]] |
								VerticesToCull[FaceIndicesData[2]];
							bRemoved = bOneVertRemoved;
						}

						if (!bRemoved)
						{
							UsedFaces[FaceIndex] = true;

							UsedVertices[FaceIndicesData[0]] = true;
							UsedVertices[FaceIndicesData[1]] = true;
							UsedVertices[FaceIndicesData[2]] = true;
						}

						FaceIndicesData += 3;
					}
				}
			});

			for (int32 Index = 1; Index < NumBatches; ++Index)
			{
				ParallelUsedFaces[0].AddRange(ParallelUsedFaces[Index], ParallelUsedFaces[Index].Num(), 0);
				ParallelUsedVertices[0].CombineWithBitwiseOR(ParallelUsedVertices[Index], EBitwiseOperatorFlags::MaxSize);
			}
		}

		const bool bHasFaces = UsedFaces.Find(true) != INDEX_NONE;
		const bool bHasRemovedFaces = UsedFaces.Find(false) != INDEX_NONE;

		if (bHasFaces && bHasRemovedFaces && Result->AreVertexIdsImplicit())
		{
			Result->MakeVertexIdsRelative();
		}

		TArray<int32> UsedVerticesMap;
		UsedVerticesMap.Init(-1, NumVertices);
    	
		FMeshBufferSet& VertexBufferSet = Result->GetVertexBuffers();

		const int32 NumBuffers = VertexBufferSet.GetBufferCount();

		// Compute vertices indices remap
        int32 NumVerticesRemaining = 0;
		if (bHasFaces)
        {
		    int32 LastFreeVertexIndex = 0;
            for (int32 VertexIndex = UsedVertices.FindFrom(true, 0); VertexIndex >= 0;)
			{
                const int32 UsedSpanBegin = VertexIndex;
                VertexIndex = UsedVertices.FindFrom(false, VertexIndex);
				
				// At the end of the buffer we may not find a false element, in that case
				// FindForm returns INDEX_NONE, set the vertex at the range end. 
                VertexIndex = VertexIndex >= 0 ? VertexIndex : NumVertices;
                const int32 UsedSpanEnd = VertexIndex;

				// VertexIndex may be one past the end of the array, VertexIndex will become INDEX_NONE
				// and the loop will finish.
                VertexIndex = UsedVertices.FindFrom(true, VertexIndex);
				
                for (int32 I = UsedSpanBegin; I < UsedSpanEnd; ++I)
                {
                    UsedVerticesMap[I] = LastFreeVertexIndex + I - UsedSpanBegin;
                }

				LastFreeVertexIndex += UsedSpanEnd - UsedSpanBegin;
			}

            NumVerticesRemaining = LastFreeVertexIndex;
		}

		// Copy move buffers. We are recomputing the spans for each buffer, should be ok as
        // finding the span is fast compared to the data move.
        if (NumVerticesRemaining > 0)
        {
            for (int32 BufferIndex = 0; BufferIndex < NumBuffers; ++BufferIndex)
            {
            	uint8* BufferData = VertexBufferSet.GetBufferData(BufferIndex); 
            	const uint32 ElemSize = VertexBufferSet.GetElementSize(BufferIndex);

            	RemoveBufferUnusedElems(BufferData, ElemSize, NumVertices, UsedVertices);
            }
        }
		Result->GetVertexBuffers().SetElementCount(NumVerticesRemaining);
    	
    	check(Result->ClothSections.Num() == 1); // Check that we are only clipping meshes with one section.
    	
    	const uint32 NumClothVertices = Result->ClothSections[0].Data.Num();
    	if (Result->ClothSections[0].IsValid() && NumClothVertices)
    	{
    		uint8* BufferData = reinterpret_cast<uint8*>(Result->ClothSections[0].Data.GetData());
    		const int32 NumInfluences = NumVerticesRemaining ? NumClothVertices / NumVerticesRemaining : 0;
    		const uint32 ElemSize = sizeof(FMeshToMeshVertData) * NumInfluences;

    		RemoveBufferUnusedElems(BufferData, ElemSize, NumVertices, UsedVertices);
    		Result->ClothSections[0].Data.SetNum(NumVerticesRemaining * NumInfluences);
    	}

		int32 LastFreeFaceIndex = 0;
		if (bHasFaces)
		{
			// Move Indices
			for (int32 FaceIndex = UsedFaces.FindFrom(true, 0); FaceIndex >= 0;)
			{
				const int32 UsedSpanStart = FaceIndex;
				FaceIndex = UsedFaces.FindFrom(false, FaceIndex);
				FaceIndex = FaceIndex >= 0 ? FaceIndex : NumFaces;
				const int32 UsedSpanEnd = FaceIndex;

				FaceIndex = UsedFaces.FindFrom(true, FaceIndex);

				const int32 UsedSpanSize = UsedSpanEnd - UsedSpanStart;
				check(UsedSpanSize > 0);

				if (LastFreeFaceIndex != UsedSpanStart)
				{
					FMemory::Memmove(
						IndicesBegin.ptr() + LastFreeFaceIndex * IndexTypeSize * 3,
						IndicesBegin.ptr() + UsedSpanStart * IndexTypeSize * 3,
						UsedSpanSize * IndexTypeSize * 3);
				}

				LastFreeFaceIndex += UsedSpanSize;
			}

			constexpr int32 NumFacesPerBatch = (1 << 13);
			const int32 NumBatches = FMath::DivideAndRoundUp<int32>(LastFreeFaceIndex, NumFacesPerBatch);

			// Remap Vertices
			ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches,
				[
					IndicesBegin,
					IndexTypeSize,
					&UsedVerticesMap,
					LastFreeFaceIndex,
					NumFacesPerBatch
				](int32 BatchId)
				{
					const int32 BatchBeginFaceIndex = BatchId * NumFacesPerBatch;
					const int32 BatchEndFaceIndex = FMath::Min(BatchBeginFaceIndex + NumFacesPerBatch, LastFreeFaceIndex);

					if (IndexTypeSize == 4)
					{
						uint32* FaceIndicesData = reinterpret_cast<uint32*>((IndicesBegin + BatchBeginFaceIndex * 3).ptr());
						for (int32 FaceIndex = BatchBeginFaceIndex; FaceIndex < BatchEndFaceIndex; ++FaceIndex)
						{
							check(UsedVerticesMap[FaceIndicesData[0]] >= 0);
							check(UsedVerticesMap[FaceIndicesData[1]] >= 0);
							check(UsedVerticesMap[FaceIndicesData[2]] >= 0);

							FaceIndicesData[0] = UsedVerticesMap[FaceIndicesData[0]];
							FaceIndicesData[1] = UsedVerticesMap[FaceIndicesData[1]];
							FaceIndicesData[2] = UsedVerticesMap[FaceIndicesData[2]];

							FaceIndicesData += 3;
						}
					}
					else if (IndexTypeSize == 2)
					{
						uint16* FaceIndicesData = reinterpret_cast<uint16*>((IndicesBegin + BatchBeginFaceIndex * 3).ptr());
						for (int32 FaceIndex = BatchBeginFaceIndex; FaceIndex < BatchEndFaceIndex; ++FaceIndex)
						{
							check(UsedVerticesMap[FaceIndicesData[0]] >= 0);
							check(UsedVerticesMap[FaceIndicesData[1]] >= 0);
							check(UsedVerticesMap[FaceIndicesData[2]] >= 0);

							FaceIndicesData[0] = static_cast<uint16>(UsedVerticesMap[FaceIndicesData[0]]);
							FaceIndicesData[1] = static_cast<uint16>(UsedVerticesMap[FaceIndicesData[1]]);
							FaceIndicesData[2] = static_cast<uint16>(UsedVerticesMap[FaceIndicesData[2]]);

							FaceIndicesData += 3;
						}
					}
				});
		}

		check(LastFreeFaceIndex <= NumFaces);

		Result->GetIndexBuffers().SetElementCount(LastFreeFaceIndex*3);

		MeshRemoveRecreateSurface(Result, UsedVertices, UsedFaces);

		MeshRemoveMeshMorphs(*Result, UsedVertices, UsedVerticesMap);

		MeshRemoveSkinWeightProfiles(Result, UsedVertices, UsedVerticesMap);

    }

	void MeshRemoveMaskInline(FMesh* Mesh, const FMesh* Mask, bool bRemoveIfAllVerticesCulled)
    {
        MUTABLE_CPUPROFILER_SCOPE(MeshRemoveMask);

        if (!Mask->GetVertexCount() || !Mesh->GetVertexCount() || !Mesh->GetIndexCount())
        {
            return;
        }

		int32 MaskElementCount = Mask->GetVertexBuffers().GetElementCount();

        // For each source vertex, true if it is removed.
        int32 MeshVertexCount = Mesh->GetVertexCount();
		TBitArray<> RemovedVertices;
		RemovedVertices.SetNum(MeshVertexCount, false);
        {
			TArray<FIdInterval> Intervals;
            ExtractVertexIndexIntervals(Intervals, Mesh);

			MeshVertexIdIteratorConst itMaskVI(Mask);
			for ( int32 mv=0; mv<MaskElementCount; ++mv )
            {
                uint64 MaskVertexId = itMaskVI.Get();
                ++itMaskVI;

                int32 IndexInSource = FindPositionInIntervals(Intervals, MaskVertexId);
                if (IndexInSource >= 0)
                {
					RemovedVertices[IndexInSource] = true;
                }
            }
        }
		
        MeshRemoveVerticesWithCullSet(Mesh, RemovedVertices, bRemoveIfAllVerticesCulled);
	}
}
