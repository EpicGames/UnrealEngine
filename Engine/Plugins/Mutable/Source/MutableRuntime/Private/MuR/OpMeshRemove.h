// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "HAL/Platform.h"

#include "Containers/BitArray.h"

namespace UE::Mutable::Private
{
	class FMesh;

	/** Remove a list of vertices and related faces from a mesh. The list of vertices is stored in a specially formatted Mask mesh. 
	 * If bRemoveIfAllVerticesCulled is true, a face is remove if all its vertices have the bit set in VerticesToCull.
	 * If bRemoveIfAllVerticesCulled is false, remove a face if at least one vertex is removed.
	 */
	extern void MeshRemoveMaskInline(FMesh* Mesh, const FMesh* Mask, bool bRemoveIfAllVerticesCulled);


	/** 
	 * Remove a set of vertices and related faces from a mesh in-place. VertexToCull is a bitset where if bit i-th is set, 
	 * the vertex i-th will be removed if all faces referencing this vertex need to be removed. 
	 * If bRemoveIfAllVerticesCulled is true, a face is remove if all its vertices have the bit set in VerticesToCull. 
	 * If bRemoveIfAllVerticesCulled is false, remove a face if at least one vertex is removed.
	 */
	extern void MeshRemoveVerticesWithCullSet(FMesh* Result, const TBitArray<>& VerticesToCull, bool bRemoveIfAllVerticesCulled);

	/**
	 * Recreates the Surface and Surfaces Submeshes given a set of vertices and faces remaining after mesh removal.
	 */
	extern void MeshRemoveRecreateSurface(FMesh* Result, const TBitArray<>& UsedVertices, const TBitArray<>& UsedFaces);
}
