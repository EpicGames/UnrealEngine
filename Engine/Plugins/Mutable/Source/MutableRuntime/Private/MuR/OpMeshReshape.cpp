// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshReshape.h"

#include "MuR/ConvertData.h"
#include "MuR/MeshPrivate.h"
#include "MuR/Operations.h"
#include "MuR/OpMeshBind.h"
#include "MuR/System.h"

#include "Math/Plane.h"
#include "Math/Ray.h"
#include "Math/NumericLimits.h"

#include "IndexTypes.h"
#include "Spatial/MeshAABBTree3.h"
#include "Distance/DistPoint3Triangle3.h"
#include "LineTypes.h"

#include "MuR/OpMeshSmoothing.h"
#include "MuR/OpMeshComputeNormals.h"

#include "PackedNormal.h"

#include "Algo/AnyOf.h" 
#include "Algo/RemoveIf.h" 


// TODO: Make the handling of rotations an option. It is more expensive on CPU and memory, and for some
// cases it is not required at all.

// TODO: Face stretch to scale the deformation per-vertex? 

// TODO: Support multiple binding influences per vertex, to have smoother deformations.

// TODO: Support multiple binding sets, to have different shapes deformations at once.

// TODO: Deformation mask, to select the intensisty of the deformation per-vertex.

// TODO: This is a reference implementation with ample roof for optimization.

namespace UE::Mutable::Private
{

	// Methods to actually deform a point
	inline void GetDeform(const GeometryUtils::FMeshGeometry& Shape, const FReshapeVertexBindingData& Binding, FVector3f& NewPosition, FVector3f& NewNormalPosition)
	{
		const UE::Geometry::FIndex3i& Triangle = Shape.Triangles[Binding.GetTriangleIndex()];
	
		FVector3f VA = Shape.Positions[Triangle.A] + Shape.Normals[Triangle.A]*Binding.D.X;
		FVector3f VB = Shape.Positions[Triangle.B] + Shape.Normals[Triangle.B]*Binding.D.Y;
		FVector3f VC = Shape.Positions[Triangle.C] + Shape.Normals[Triangle.C]*Binding.D.Z;

		NewPosition = VC
				+ (VA - VC)*Binding.PositionBaryCoords.X 
				+ (VB - VC)*Binding.PositionBaryCoords.Y;

		FVector3f NVA = Shape.Positions[Triangle.A] + Shape.Normals[Triangle.A]*Binding.Vertex.NormalD.X;
		FVector3f NVB = Shape.Positions[Triangle.B] + Shape.Normals[Triangle.B]*Binding.Vertex.NormalD.Y;
		FVector3f NVC = Shape.Positions[Triangle.C] + Shape.Normals[Triangle.C]*Binding.Vertex.NormalD.Z;

		NewNormalPosition = NVC
				+ (NVA - NVC)*Binding.Vertex.NormalBaryCoords.X
				+ (NVB - NVC)*Binding.Vertex.NormalBaryCoords.Y;
	}

	inline void GetDeform(const GeometryUtils::FMeshGeometry& Shape, const FReshapePointBindingData& Binding, FVector3f& NewPosition)
	{
		const UE::Geometry::FIndex3i& Triangle = Shape.Triangles[Binding.Triangle];
	
		FVector3f VA = Shape.Positions[Triangle.A] + Shape.Normals[Triangle.A]*Binding.D.X;
		FVector3f VB = Shape.Positions[Triangle.B] + Shape.Normals[Triangle.B]*Binding.D.Y;
		FVector3f VC = Shape.Positions[Triangle.C] + Shape.Normals[Triangle.C]*Binding.D.Z;

		NewPosition = VC + (VA - VC)*Binding.S + (VB - VC)*Binding.T;
	}

    //---------------------------------------------------------------------------------------------
    //! Physics Bodies Reshape 
    //---------------------------------------------------------------------------------------------
	template<uint32 NumPoints>
	bool GetDeformedPoints(const GeometryUtils::FMeshGeometry& Shape, const FReshapePointBindingData* BindingData,  TStaticArray<FVector3f, NumPoints>& OutPoints)
	{
		const int32 ShapeNumTris = Shape.Triangles.Num();
		for (int32 I = 0; I < NumPoints; ++I)
		{
			const FReshapePointBindingData& BindingDataPoint = *(BindingData + I);
			if ((BindingDataPoint.Triangle < 0) | (BindingDataPoint.Triangle >= ShapeNumTris))
			{
				return false;
			}
		}

		for (int32 I = 0; I < NumPoints; ++I)
		{
			GetDeform(Shape, *(BindingData + I), OutPoints[I]);
		}

		return true;
	}

	void GetDeformedConvex(const GeometryUtils::FMeshGeometry& Shape, const FReshapePointBindingData* BindingData, TArrayView<FVector3f>& InOutDeformedVertices)
	{
		const int32 ShapeNumTris = Shape.Triangles.Num();
		const int32 ConvexVertCount = InOutDeformedVertices.Num();

		for (int32 I = 0; I < ConvexVertCount; ++I)
		{
			const FReshapePointBindingData& BindingDataPoint = *(BindingData + I);

			if ((BindingDataPoint.Triangle < 0) | (BindingDataPoint.Triangle >= ShapeNumTris))
			{
				continue;
			}

			GetDeform(Shape, BindingDataPoint, InOutDeformedVertices[I]);	
		}
	}

	void ComputeSphereFromDeformedPoints( 
			const TStaticArray<FVector3f, 6>& Points, 
			FVector3f& OutP, float& OutR, 
			const FTransform3f& InvBoneT )
	{	
		FVector3f Centroid(ForceInitToZero);
		for (const FVector3f& V : Points)
		{
			Centroid += V;	
		}

		Centroid *= (1.0f/6.0f);

		OutP = Centroid + InvBoneT.GetTranslation();
		
		float Radius = 0.0f;
		for (const FVector3f& V : Points)
		{
			Radius += (V - Centroid).Length();
		}

		Radius *= (1.0f/6.0f);
		
		OutR = Radius;
	}

	void ComputeBoxFromDeformedPoints( 
			const TStaticArray<FVector3f, 14>& Points, 
			FVector3f& OutP, FQuat4f& OutQ, FVector3f& OutS, 
			const FTransform3f& InvBoneT )
	{	
		const FVector3f TopC = (Points[0] + Points[1] + Points[2] + Points[3])*0.25f;
		const FVector3f BottomC = (Points[4+0] + Points[4+1] + Points[4+2] + Points[4+3])*0.25f;
    
		const FVector3f FrontC = (Points[0] + Points[1] + Points[4+0] + Points[4+1])*0.25f;
		const FVector3f BackC = (Points[2] + Points[3] + Points[4+2] + Points[4+3])*0.25f;
    
		const FVector3f RightC =  (Points[1] + Points[2] + Points[4+1] + Points[4+2])*0.25f;
		const FVector3f LeftC =  (Points[3] + Points[0] + Points[4+3] + Points[4+0])*0.25f;
    
		FVector3f ZB = (TopC - BottomC).GetSafeNormal();
		FVector3f XB = (RightC - LeftC).GetSafeNormal();
		FVector3f YB = (FrontC - BackC).GetSafeNormal();


		// Pick the 2 most offaxis vectors and construct a rotation form those.	
		// TODO: Find a better way of finding and orientation form ZB, XB, TN maybe by averaging somehow different bases created from the vectors, with quaternions?
		FVector3f OF = FVector3f( XB.X, YB.Y, ZB.Y ).GetAbs();

		float M0 = FMath::Max3( OF.X, OF.Y, OF.X);
		float M1 = M0 == OF.X ? FMath::Max(OF.Y, OF.Z) : (M0 == OF.Y ? FMath::Max(OF.X,OF.Z) : FMath::Max(OF.Y, OF.X));
		
		FMatrix44f RotationMatrix = FMatrix44f::Identity;
		if (M0 == OF.X)
		{
			RotationMatrix = M1 == OF.Y 
					? FRotationMatrix44f::MakeFromXY(XB, YB)
					: FRotationMatrix44f::MakeFromXZ(XB, ZB);
		}
		else if (M0 == OF.Y)
		{
			RotationMatrix = M1 == XB.X 
					? FRotationMatrix44f::MakeFromYX(YB, XB)
					: FRotationMatrix44f::MakeFromYZ(YB, ZB);
		}
		else
		{
			RotationMatrix = M1 == XB.X 
					? FRotationMatrix44f::MakeFromZX(ZB, XB)
					: FRotationMatrix44f::MakeFromZY(ZB, YB);	
		} 
	
		FTransform3f ShapeToBone = FTransform3f(RotationMatrix.ToQuat(), (TopC + BottomC) * 0.5f) * InvBoneT;

		OutQ = ShapeToBone.GetRotation();
		OutP = ShapeToBone.GetTranslation();
		OutS = FVector3f((RightC - LeftC).Size(), (FrontC - BackC).Size(), (TopC - BottomC).Size()) * 0.5f;	
	}

	void ComputeSphylFromDeformedPoints( 
			const TStaticArray<FVector3f, 14>& Points, 
			FVector3f& OutP, FQuat4f& OutQ, float& OutR, float& OutL, 
			const FTransform3f& InvBoneT)
	{
		constexpr int32 NumCentroids = 5;
		TStaticArray<FVector3f, NumCentroids> Centroids;
		Centroids[0] = Points[0];
		Centroids[1] = Points[1];

		for (int32 I = 0; I < NumCentroids - 2; ++I)
		{
			Centroids[2 + I] = 
				(Points[2 + I*4 + 0] + Points[2 + I*4 + 1] +
				 Points[2 + I*4 + 2] + Points[2 + I*4 + 3] ) * 0.25f;
		}

		// Geometric linear regression of top, bottom and rings centroids.
		FVector3f Centroid = FVector3f::Zero();
		for (const FVector3f& C : Centroids)
		{
			Centroid += C;
		}

		constexpr float OneOverNumCentroids = 1.0f / static_cast<float>(NumCentroids);
		Centroid *= OneOverNumCentroids;
		
		for (FVector3f& C : Centroids)
		{
			C -= Centroid;
		}

		constexpr int32 NumIters = 3;
		FVector3f Direction = (Centroids[0] - Centroids[1]).GetSafeNormal();
		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			FVector3f IterDirRefinement = Direction;
			for (const FVector3f& C : Centroids)
			{
				IterDirRefinement += C * FVector3f::DotProduct(Direction, C);
			}

			Direction = IterDirRefinement.GetSafeNormal();
		}

		// Project centroids to the line described by Direction and Centroid.
		for (FVector3f& C : Centroids)
		{
			C = Centroid + Direction * FVector3f::DotProduct(C, Direction);
		}

		// Quaternion form {0,0,1} to Direction.
		const FQuat4f Rotation = FQuat4f(-Direction.Y, Direction.X, 0.0f, 1.0f + FMath::Max(Direction.Z, -1.0f + UE_SMALL_NUMBER)).GetNormalized(); 

		FTransform3f ShapeToBone = FTransform3f(Rotation, Centroid) * InvBoneT;
		OutQ = ShapeToBone.GetRotation();
		OutP = ShapeToBone.GetTranslation();

		// Project ring points to plane formed by ring centroid and direction to extract ring radius.
		const auto ComputeRadiusContribution = [&](const FVector3f& P, const FVector3f& Origin, const FVector3f& Dir) -> float
		{
			return ((P + Dir * FVector3f::DotProduct(Dir, P - Origin)) - Origin).Length();
		};

		const float R0 = 
			ComputeRadiusContribution(Points[2], Centroids[2], Direction) +
			ComputeRadiusContribution(Points[3], Centroids[2], Direction) +
			ComputeRadiusContribution(Points[4], Centroids[2], Direction) +
			ComputeRadiusContribution(Points[5], Centroids[2], Direction);

		const float R1 = 
			ComputeRadiusContribution(Points[6], Centroids[3], Direction) +
			ComputeRadiusContribution(Points[7], Centroids[3], Direction) +
			ComputeRadiusContribution(Points[8], Centroids[3], Direction) +
			ComputeRadiusContribution(Points[9], Centroids[3], Direction);

		const float R2 = 
			ComputeRadiusContribution(Points[10], Centroids[4], Direction) +
			ComputeRadiusContribution(Points[11], Centroids[4], Direction) +
			ComputeRadiusContribution(Points[12], Centroids[4], Direction) +
			ComputeRadiusContribution(Points[13], Centroids[4], Direction);

		OutR =  (R0 + R1 + R2) * (0.25f/3.0f);
		OutL = FMath::Max(0.0f, (Centroids[0] - Centroids[1]).Length() - OutR*2.0f);
	}

	void ComputeTaperedCapsuleFromDeformedPoints( 
			const TStaticArray<FVector3f, 14>& Points, 
			FVector3f& OutP, FQuat4f& OutQ, float& OutR0, float& OutR1, float& OutL, 
			const FTransform3f& InvBoneT)
	{
		constexpr int32 NumCentroids = 5;
		TStaticArray<FVector3f, NumCentroids> Centroids;
		Centroids[0] = Points[0];
		Centroids[1] = Points[1];

		for (int32 I = 0; I < NumCentroids - 2; ++I)
		{
			Centroids[2 + I] = 
				(Points[2 + I*4 + 0] + Points[2 + I*4 + 1] +
				 Points[2 + I*4 + 2] + Points[2 + I*4 + 3] ) * 0.25f;
		}
	
		// Geometric linear regression of top, bottom and ring centroids.
		FVector3f Centroid = FVector3f::Zero();
		for (const FVector3f& C : Centroids)
		{
			Centroid += C;
		}

		constexpr float OneOverNumCentroids = 1.0f / static_cast<float>(NumCentroids);
		Centroid *= OneOverNumCentroids;
		
		for (FVector3f& C : Centroids)
		{
			C -= Centroid;
		}

		constexpr int32 NumIters = 3;
		FVector3f Direction = (Centroids[0] - Centroids[1]).GetSafeNormal();
		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			FVector3f IterDirRefinement = Direction;
			for (const FVector3f& C : Centroids)
			{
				IterDirRefinement += C * FVector3f::DotProduct(Direction, C);
			}

			Direction = IterDirRefinement.GetSafeNormal();
		}

		// Project centroids to the line described by Direction and Centroid.
		for (FVector3f& C : Centroids)
		{
			C = Centroid + Direction * FVector3f::DotProduct(C, Direction);
		}

		// Quaternion form {0,0,1} to Direction.
		const FQuat4f Rotation = FQuat4f(-Direction.Y, Direction.X, 0.0f, 1.0f + FMath::Max(Direction.Z, -1.0f + UE_SMALL_NUMBER)).GetNormalized(); 

		FTransform3f ShapeToBone = FTransform3f(Rotation, Centroid) * InvBoneT;
		OutQ = ShapeToBone.GetRotation();
		OutP = ShapeToBone.GetTranslation();

		// Project ring points to plane formed by ring centroid and direction to extract ring radius.
		const auto ComputeRadiusContribution = [&](const FVector3f& P, const FVector3f& Origin, const FVector3f& Dir) -> float
		{
			return ((P + Dir * FVector3f::DotProduct(Dir, P - Origin)) - Origin).Length();
		};

		const float R0 = 
			ComputeRadiusContribution(Points[2], Centroids[2], Direction) +
			ComputeRadiusContribution(Points[3], Centroids[2], Direction) +
			ComputeRadiusContribution(Points[4], Centroids[2], Direction) +
			ComputeRadiusContribution(Points[5], Centroids[2], Direction);

		const float R1 = 
			ComputeRadiusContribution(Points[6], Centroids[3], Direction) +
			ComputeRadiusContribution(Points[7], Centroids[3], Direction) +
			ComputeRadiusContribution(Points[8], Centroids[3], Direction) +
			ComputeRadiusContribution(Points[9], Centroids[3], Direction);

		const float R2 = 
			ComputeRadiusContribution(Points[10], Centroids[4], Direction) +
			ComputeRadiusContribution(Points[11], Centroids[4], Direction) +
			ComputeRadiusContribution(Points[12], Centroids[4], Direction) +
			ComputeRadiusContribution(Points[13], Centroids[4], Direction);

		// TODO: Ajust for R1, center ring radius.
		OutR0 = R0*0.25f;
		OutR1 = R2*0.25f;
		OutL = FMath::Max(0.0f, (Centroids[0] - Centroids[1]).Length() - (OutR0 + OutR1));
	}

	void ApplyToVertices(FMesh* Mesh, TArrayView<const FReshapeVertexBindingData> BindingData, const GeometryUtils::FMeshGeometry& Shape, bool bSkipNormalReshape, FLiveInstanceLogger& Logger)
	{
		check(Mesh);
		check(Mesh->GetVertexCount() == BindingData.Num());
		
		MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> PositionIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Position);
		TArrayView<FVector3f> Positions = TArrayView<FVector3f>(reinterpret_cast<FVector3f*>(PositionIter.ptr()), Mesh->GetVertexCount());

		const UntypedMeshBufferIterator NormalIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Normal);
		const UntypedMeshBufferIterator TangentIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Tangent);
		const UntypedMeshBufferIterator BiNormalIter(Mesh->GetVertexBuffers(), EMeshBufferSemantic::Binormal);

        const EMeshBufferFormat NormalFormat   = NormalIter.GetFormat();
        const EMeshBufferFormat TangentFormat  = TangentIter.GetFormat();
        const EMeshBufferFormat BiNormalFormat = BiNormalIter.GetFormat();
	
        const int32 NormalComps   = NormalIter.GetComponents();
        const int32 TangentComps  = TangentIter.GetComponents();
        const int32 BiNormalComps = BiNormalIter.GetComponents();

#if DO_CHECK
		// checking if the Base shape has more triangles than the target shape
		const bool bTriangleOutOfScopeFound = Algo::AnyOf(BindingData, 
		[NumShapeTriangles = (uint32)Shape.Triangles.Num()](const FReshapeVertexBindingData& B) 
		{ 
			return (B.GetTriangleIndex() >= NumShapeTriangles) && (B.IsValidBinding()); 
		});

		if (bTriangleOutOfScopeFound)
		{
			Logger.LogWarn(TEXT("Performing a Mesh Reshape where base shape and target shape do not have the same number of triangles."));
		}	
#endif

		const int32 ShapeTriangleCount = Shape.Triangles.Num();
		const int32 ShapePositionCount = Shape.Positions.Num();

		const int32 MeshVertexCount = BindingData.Num();

		const bool bComputeNormal = bSkipNormalReshape && NormalIter.ptr();
		for (int32 VertexIndex = 0; VertexIndex < MeshVertexCount; ++VertexIndex)
		{
			const FReshapeVertexBindingData& Binding = BindingData[VertexIndex];

			const bool bModified = Binding.IsValidBinding() & (Binding.GetTriangleIndex() < (uint32)ShapeTriangleCount);
			if (bModified)
			{
				FVector3f NewPosition;
				FVector3f NewNormalPosition;
				
				GetDeform(Shape, Binding, NewPosition, NewNormalPosition);

				FVector3f OldPosition = (PositionIter + VertexIndex).GetAsVec3f();
 
                FVector3f Position;
				FQuat4f ClusterQuat; // Only used for cluster binding.
                if (!Binding.IsClusterBinding())
                {
					Position = OldPosition + (NewPosition - OldPosition) * Binding.Weight;
				}
                else
                {
					const UE::Geometry::FIndex3i& Triangle = Shape.Triangles[Binding.GetTriangleIndex()];
	
					const FVector3f& VA = Shape.Positions[Triangle.A];
					const FVector3f& VB = Shape.Positions[Triangle.B];
					const FVector3f& VC = Shape.Positions[Triangle.C];

					FVector3f TriangleNormal = FPlane4f(VA, VB, VC).GetNormal();

					ClusterQuat = FQuat4f::FindBetweenNormals(Binding.Cluster.AttachmentNormal, TriangleNormal);
					ClusterQuat = FQuat4f::FastLerp(FQuat4f::Identity, ClusterQuat, Binding.Weight).GetNormalized();

					Position = (ClusterQuat.RotateVector(OldPosition - Binding.Cluster.AttachmentPoint)) + Binding.Cluster.AttachmentPoint;
					Position = Position + (NewPosition - Binding.Cluster.AttachmentPoint) * Binding.Weight;
                }
				
				(PositionIter + VertexIndex).SetFromVec3f(Position);

				if (bComputeNormal)
				{
					FVector3f Normal;

					const FVector3f OldNormal = (NormalIter + VertexIndex).GetAsVec3f();

					if (!Binding.IsClusterBinding())
					{
						const FVector3f OldNormalPosition = OldPosition + OldNormal;
						FVector3f NormalDisplacement = (NewNormalPosition - OldNormalPosition) * Binding.Weight;
						Normal = ((OldNormalPosition + NormalDisplacement) - NewPosition).GetSafeNormal();
					}
					else
					{
						Normal = ClusterQuat.RotateVector(OldNormal);	
					}

					FVector3f Tangent = TangentIter.ptr() ? (TangentIter + VertexIndex).GetAsVec3f() : FVector3f::XAxisVector;
					FVector3f BiNormal = BiNormalIter.ptr() ? (BiNormalIter + VertexIndex).GetAsVec3f() : FVector3f::YAxisVector;
				
					OrthogonalizeTangentSpace(
							&Normal, 
							TangentIter.ptr() ? &Tangent : nullptr, 
							BiNormalIter.ptr() ? &BiNormal : nullptr,
							BiNormalIter.ptr() ? ComputeTangentBasisDeterminantSign(OldNormal, Tangent, BiNormal) : 0.0f);
					
					// Leave the tangent basis sign untouched for packed normals formats.
					uint8 * const NormalElemPtr = (NormalIter + VertexIndex).ptr();
					for (int32 C = 0; C < NormalComps && C < 3; ++C)
					{
						ConvertData(C, NormalElemPtr, NormalFormat, &Normal, EMeshBufferFormat::Float32);
					}

					if (TangentIter.ptr())
					{
						uint8 * const TangentElemPtr = (TangentIter + VertexIndex).ptr();
						for (int32 C = 0; C < TangentComps && C < 3; ++C)
						{
							ConvertData(C, TangentElemPtr, TangentFormat, &Tangent, EMeshBufferFormat::Float32);
						}
					}
					
					if (BiNormalIter.ptr())
					{
						uint8 * const BiNormalElemPtr = (BiNormalIter + VertexIndex).ptr();
						for (int32 C = 0; C < BiNormalComps && C < 3; ++C)
						{
							ConvertData(C, BiNormalElemPtr, BiNormalFormat, &BiNormal, EMeshBufferFormat::Float32);
						}
					}
				}
			}
		}
	}

	void ApplyToPose(FMesh* Result, 
			TArrayView<const FReshapePointBindingData> BindingData, 
			TArrayView<const int32> BoneIndices,
			GeometryUtils::FMeshGeometry& TargetShape)
	{
		check(Result);

#if DO_CHECK
		// checking if the Base shape has more triangles than the target shape
		const bool bTriangleOutOfScopeFound = Algo::AnyOf(BindingData, 
			[NumShapeTriangles = TargetShape.Triangles.Num()](const FReshapePointBindingData& B) { return B.Triangle >= NumShapeTriangles; });

		if (bTriangleOutOfScopeFound)
		{
			UE_LOGF(LogMutableCore, Warning, "Performing a Mesh Reshape where base shape and target shape do not have the same number of triangles.");
		}	
#endif

		const int32 NumShapeTris = TargetShape.Triangles.Num();
		const int32 NumBoneIndices = BoneIndices.Num();
		for (int32 BoneSelectionIndex = 0; BoneSelectionIndex < NumBoneIndices; ++BoneSelectionIndex)
		{
			int32 BoneIndex = BoneIndices[BoneSelectionIndex];

			const FReshapePointBindingData& Binding = BindingData[BoneSelectionIndex];

			check(!EnumHasAnyFlags(Result->BonePoses[BoneIndex].BoneUsageFlags, EBoneUsageFlags::Root));

			FTransform3f& T = Result->BonePoses[BoneIndex].BoneTransform;

			FVector3f NewPosition(ForceInitToZero);

			const bool bModified = (Binding.Triangle >= 0) & (Binding.Triangle < NumShapeTris);
			
			if (bModified)
			{
				GetDeform(TargetShape, Binding, NewPosition);
			}

			const bool bHasChanged = bModified && FVector3f::DistSquared(NewPosition, T.GetLocation()) > UE_SMALL_NUMBER;
			// Only set it if has actually moved.
			if (bHasChanged)
			{	
				// Mark as reshaped. 
				EnumAddFlags(Result->BonePoses[BoneIndex].BoneUsageFlags, EBoneUsageFlags::Reshaped);

				// TODO: Review if the rotation also needs to be applied.
				T.SetLocation(FMath::Lerp(T.GetLocation(), NewPosition, Binding.Weight));
			}
		}
	}

	inline void ApplyToPhysicsBodies(
			FPhysicsBody& PBody, int32& InOutNumProcessedBindPoints, 
			const FMesh& BaseMesh, 
			const FSkeleton& BaseSkeleton, 
			TArrayView<const FReshapePointBindingData> BindingData, 
			TArrayView<const int32> UsedIndices, 
			const GeometryUtils::FMeshGeometry& Shape,
			FLiveInstanceLogger& MessageLogger)
	{
#if DO_CHECK
		// checking if the Base shape has more triangles than the target shape
		const bool bTriangleOutOfScopeFound = Algo::AnyOf(BindingData, 
			[NumShapeTriangles = Shape.Triangles.Num()](const FReshapePointBindingData& B) { return B.Triangle >= NumShapeTriangles; });

		if (bTriangleOutOfScopeFound)
		{
			MessageLogger.LogUpdateMessage("Performing a Mesh Reshape where base shape and target shape do not have the same number of triangles.", ELogVerbosity::Warning);
		}
#endif
		
		bool bAnyModified = false;

		// Retrieve them in the same order the boxes where put in, so they can be linked to the physics body volumes.
		for ( const int32 B : UsedIndices )
		{	
			const int32 BoneIndex = BaseSkeleton.BoneNames.Find(PBody.GetBodyBoneName(B));
			const int32 PoseIndex = BaseMesh.FindBonePoseByBoneIndex(BoneIndex);

			FTransform3f BoneTransform = FTransform3f::Identity;
			if (PoseIndex >= 0)
			{
				BaseMesh.GetBonePoseTransform(PoseIndex, BoneTransform);
			}
			
			FTransform3f InvBoneTransform = BoneTransform.Inverse();

			const int32 NumSpheres = PBody.GetSphereCount(B); 
			for ( int32 I = 0; I < NumSpheres; ++I )
			{
				FVector3f P;
				float R;

				TStaticArray<FVector3f, 6> Points; 
				const bool bDeformed = GetDeformedPoints(Shape, &BindingData[InOutNumProcessedBindPoints], Points);

				if (bDeformed)
				{
					ComputeSphereFromDeformedPoints(Points, P, R, InvBoneTransform);
					PBody.SetSphere(B, I, P, R);
					bAnyModified = true;
				}

				InOutNumProcessedBindPoints += Points.Num();
			}

			const int32 NumBoxes = PBody.GetBoxCount(B);
			for ( int32 I = 0; I < NumBoxes; ++I )
			{
				TStaticArray<FVector3f, 14> Points;
				const bool bDeformed = GetDeformedPoints(Shape, &BindingData[InOutNumProcessedBindPoints], Points);
				
				if (bDeformed)
				{
					FVector3f P;
					FQuat4f Q;
					FVector3f S;
					
					ComputeBoxFromDeformedPoints(Points, P, Q, S, InvBoneTransform);
					PBody.SetBox(B, I, P, Q, S);
					bAnyModified = true;
				}

				InOutNumProcessedBindPoints += Points.Num();
			}
			
			const int32 NumSphyls = PBody.GetSphylCount(B);
			for ( int32 I = 0; I < NumSphyls; ++I )
			{
				TStaticArray<FVector3f, 14> Points;
				const bool bDeformed = GetDeformedPoints(Shape, &BindingData[InOutNumProcessedBindPoints], Points);

				if ( bDeformed )
				{
					FVector3f P;
					FQuat4f Q;
					float R;
					float L;						

					ComputeSphylFromDeformedPoints(Points, P, Q, R, L, InvBoneTransform);
					PBody.SetSphyl(B, I, P, Q, R, L);
					bAnyModified = true;
				}

				InOutNumProcessedBindPoints += Points.Num();
			}

			const int32 NumTaperedCapsules = PBody.GetTaperedCapsuleCount(B);
			for ( int32 I = 0; I < NumTaperedCapsules; ++I )
			{
				TStaticArray<FVector3f, 14> Points;
				const bool bDeformed = GetDeformedPoints(Shape, &BindingData[InOutNumProcessedBindPoints], Points);

				if ( bDeformed )
				{
					FVector3f P;
					FQuat4f Q;
					float R0;
					float R1;
					float L;
				
					ComputeTaperedCapsuleFromDeformedPoints(Points, P, Q, R0, R1, L, InvBoneTransform);
					PBody.SetTaperedCapsule(B, I, P, Q, R0, R1, L);
					bAnyModified = true;
				}
				
				InOutNumProcessedBindPoints += Points.Num();
			}

			const int32 NumConvex = PBody.GetConvexCount(B);
			for ( int32 I = 0; I < NumConvex; ++I )
			{
				TArrayView<FVector3f> VerticesView;
				TArrayView<int32> IndicesView;
				PBody.GetConvexMeshView(B, I, VerticesView, IndicesView);

				FTransform3f ConvexTransform;
				PBody.GetConvexTransform(B, I, ConvexTransform);

				GetDeformedConvex(Shape, &BindingData[InOutNumProcessedBindPoints], VerticesView);

				FTransform3f InvConvexT = InvBoneTransform * ConvexTransform.Inverse();
				for ( FVector3f& V : VerticesView )
				{
					V = InvConvexT.TransformPosition( V );				
				}
				
				InOutNumProcessedBindPoints += VerticesView.Num();
				bAnyModified = true;
			}
		}

		PBody.bBodiesModified = bAnyModified;
	}

	void ApplyToAllPhysicsBodies(
		FMesh& Mesh, const FSkeleton& BaseSkeleton,
		TArrayView<const FReshapePointBindingData> BindingData,
		TArrayView<const int32> UsedIndices,
		TArrayView<const int32> BodyOffsets,
		const GeometryUtils::FMeshGeometry& Shape,
		FLiveInstanceLogger& MessageLogger)
	{
		if (!BodyOffsets.Num())
		{
			return; // Nothing to do.
		}

		int32 NumProcessedBindPoints = 0;

		auto ApplyPhysicsBody = [&](FPhysicsBody& OutBody, int32 IndicesBegin, int32 IndicesEnd) -> void
		{
			TArrayView<const int32> BodyUsedIndices(UsedIndices.GetData() + IndicesBegin, IndicesEnd - IndicesBegin);

			ApplyToPhysicsBodies(OutBody, NumProcessedBindPoints, Mesh, BaseSkeleton, BindingData, BodyUsedIndices, Shape, MessageLogger);
		};


		check(BodyOffsets.Num() > 1);
		check(Mesh.AdditionalPhysicsBodies.Num() + 1 == BodyOffsets.Num() - 1);
		const int32 PhysicsBodiesNum = BodyOffsets.Num() - 1;
		
		// Apply main physics body
		if (Mesh.PhysicsBody)
		{
			TManagedPtr<FPhysicsBody> NewBody = Mesh.PhysicsBody->Clone();
			ApplyPhysicsBody(*NewBody, 0, BodyOffsets[1]);
			Mesh.PhysicsBody = NewBody;
		}


		// Apply additional physics bodies
		for (int32 I = 1; I < PhysicsBodiesNum; ++I)
		{	
			TManagedPtr<FPhysicsBody> NewBody = Mesh.AdditionalPhysicsBodies[I - 1]->Clone();
			ApplyPhysicsBody(*NewBody, BodyOffsets[I], BodyOffsets[I + 1]);
			Mesh.AdditionalPhysicsBodies[I - 1] = NewBody;
		}

		check(NumProcessedBindPoints == BindingData.Num());	
	}


	void RemoveBindingData(FMesh* Mesh, int32 BindingDataIndex)
	{
		if (!Mesh)
		{
			return;
		}

		int32 NewNumAdditionalBuffers = Algo::RemoveIf(Mesh->AdditionalBuffers, 
		[](const TPair<EMeshBufferType, FMeshBufferSet>& A)
		{	
			const bool bIsBindOpBuffer = 
					A.Key == EMeshBufferType::SkeletonDeformBinding || 
					A.Key == EMeshBufferType::PhysicsBodyDeformBinding ||
					A.Key == EMeshBufferType::PhysicsBodyDeformSelection ||
					A.Key == EMeshBufferType::PhysicsBodyDeformOffsets;

			return bIsBindOpBuffer;
		});

		Mesh->AdditionalBuffers.SetNum(NewNumAdditionalBuffers, EAllowShrinking::No);

		int32 BarycentricDataBuffer = 0;
		int32 BarycentricDataChannel = 0;
		FMeshBufferSet& VertexBuffers = Mesh->GetVertexBuffers();
		VertexBuffers.FindChannel(EMeshBufferSemantic::BarycentricCoords, BindingDataIndex, &BarycentricDataBuffer, &BarycentricDataChannel);

		if (BarycentricDataBuffer >= 0)
		{
			VertexBuffers.RemoveBuffer(BarycentricDataBuffer);
		}
	}

	void MeshApplyShape(FMesh* Mesh, GeometryUtils::FMeshGeometry& TargetShape, EMeshBindShapeFlags BindFlags, bool& bOutSuccess, FLiveInstanceLogger& MessageLogger)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshApplyReshape);

		bOutSuccess = true;

		if (!Mesh)
		{
			bOutSuccess = false;
			return;
		}
		
		const bool bReshapeVertices = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapeVertices);
		const bool bRecomputeNormals = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::RecomputeNormals);
		const bool bReshapeSkeleton = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapeSkeleton);
		const bool bReshapePhysicsVolumes = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ReshapePhysicsVolumes);
		const bool bApplyLaplacian = EnumHasAnyFlags(BindFlags, EMeshBindShapeFlags::ApplyLaplacian);

		// Early out if nothing will be modified and the vertices discarted.
		const bool bSkeletonModification = Mesh->GetSkeleton() && bReshapeSkeleton;
		const bool bPhysicsModification = (Mesh->GetPhysicsBody() || Mesh->AdditionalPhysicsBodies.Num()) && bReshapePhysicsVolumes;

		if (!bReshapeVertices && !bSkeletonModification && !bPhysicsModification)
		{
			bOutSuccess = false;
			return;
		}

		// \TODO: Multiple binding data support
		int32 BindingDataIndex = 0;

		// If the base mesh has no binding data, just clone it.
		int32 BarycentricDataBuffer = 0;
		int32 BarycentricDataChannel = 0;
		const FMeshBufferSet& VB = Mesh->GetVertexBuffers();
		VB.FindChannel(EMeshBufferSemantic::BarycentricCoords, BindingDataIndex, &BarycentricDataBuffer, &BarycentricDataChannel);
		
		// Abort the operations as we don't have a shape.
		if (!(TargetShape.Triangles.Num() && TargetShape.Normals.Num() && TargetShape.Positions.Num()))
		{
			bOutSuccess = false;
			UE_LOGF(LogMutableCore, Error, "The shape mesh provided does not have proper data. Skipping reshape operation.");
			return;
		}

		if (bReshapeVertices && BarycentricDataBuffer >= 0)
		{
			{
				MUTABLE_CPUPROFILER_SCOPE(ReshapeVertices);
				// \TODO: More checks
				check(BarycentricDataChannel == 1);
				check(VB.GetElementSize(BarycentricDataBuffer) == (int)sizeof(FReshapeVertexBindingData));

				TArrayView<const FReshapeVertexBindingData> VerticesBindingData(
						(const FReshapeVertexBindingData*)VB.GetBufferData(BarycentricDataBuffer),
						VB.GetElementCount());

				ApplyToVertices(Mesh, VerticesBindingData, TargetShape, bRecomputeNormals, MessageLogger);
			}

			if (bApplyLaplacian)
			{
				constexpr bool bRemoveData = true;
				SmoothMeshLaplacian(*Mesh, bRemoveData);
			}

			if (bRecomputeNormals)
			{
				ComputeMeshNormals(*Mesh);
			}
		}
	
		if (bReshapeSkeleton)
		{
			MUTABLE_CPUPROFILER_SCOPE(ReshapeSkeleton);

			// If the base mesh has no binding data for the skeleton don't do anything.
			const FMeshBufferSet* SkeletonBindBuffer = nullptr;
			for (const TPair<EMeshBufferType, FMeshBufferSet>& A : Mesh->AdditionalBuffers)
			{
				if (A.Key == EMeshBufferType::SkeletonDeformBinding)
				{
					SkeletonBindBuffer = &A.Value;
					break;
				}
			}
			
			BarycentricDataBuffer = -1;
			if (SkeletonBindBuffer)
			{
				SkeletonBindBuffer->FindChannel(EMeshBufferSemantic::BarycentricCoords, BindingDataIndex, &BarycentricDataBuffer, &BarycentricDataChannel);
			}

			if (SkeletonBindBuffer && BarycentricDataBuffer >= 0)
			{
				// \TODO: More checks
				check(BarycentricDataChannel == 0);
				check(SkeletonBindBuffer && SkeletonBindBuffer->GetElementSize(BarycentricDataBuffer) == (int)sizeof(FReshapePointBindingData));

				TArrayView<const FReshapePointBindingData> SkeletonBindingData( 
						(const FReshapePointBindingData*)SkeletonBindBuffer->GetBufferData(BarycentricDataBuffer),
						SkeletonBindBuffer->GetElementCount());
			
				check(SkeletonBindBuffer->GetBufferCount() >= 2);
				TArrayView<const int32> BoneIndices( 
						(const int32*)SkeletonBindBuffer->GetBufferData(1), SkeletonBindBuffer->GetElementCount());

				ApplyToPose(Mesh, SkeletonBindingData, BoneIndices, TargetShape);
			}
		}

		// When transforming the physics volumes, the resulting pose of of the skeleton reshape operation will be used, so 
		// order of operation is important.

		// Transform physics volumes based on the deformed sampling points.
		const FPhysicsBody* OldPhysicsBody = Mesh->PhysicsBody.Get();
		const int32 AdditionalPhysicsBodiesNum = Mesh->AdditionalPhysicsBodies.Num();

		if (bReshapePhysicsVolumes && Mesh->GetSkeleton() && (OldPhysicsBody || AdditionalPhysicsBodiesNum))
		{	
			MUTABLE_CPUPROFILER_SCOPE(ReshapePhysicsBodies);
			
			using BufferEntryType = TPair<EMeshBufferType, FMeshBufferSet>;
			const BufferEntryType* FoundPhysicsBindBuffer = Mesh->AdditionalBuffers.FindByPredicate(
					[](BufferEntryType& E){ return E.Key == EMeshBufferType::PhysicsBodyDeformBinding; });

			const BufferEntryType* FoundPhysicsBindSelectionBuffer = Mesh->AdditionalBuffers.FindByPredicate(
					[](BufferEntryType& E){ return E.Key == EMeshBufferType::PhysicsBodyDeformSelection; });
	
			const BufferEntryType* FoundPhysicsBindOffsetsBuffer = Mesh->AdditionalBuffers.FindByPredicate(
					[](BufferEntryType& E){ return E.Key == EMeshBufferType::PhysicsBodyDeformOffsets; });

			BarycentricDataBuffer = -1;
			if (FoundPhysicsBindBuffer)
			{
				FoundPhysicsBindBuffer->Value.FindChannel(EMeshBufferSemantic::BarycentricCoords, BindingDataIndex, &BarycentricDataBuffer, &BarycentricDataChannel);
			}
			
			const bool bAllNeededBuffersFound =
				FoundPhysicsBindBuffer && FoundPhysicsBindSelectionBuffer && FoundPhysicsBindOffsetsBuffer && BarycentricDataBuffer >= 0;

			if (bAllNeededBuffersFound)
			{
				const FMeshBufferSet& PhysicsBindBuffer = FoundPhysicsBindBuffer->Value;
				const FMeshBufferSet& PhyiscsBindSelectionBuffer = FoundPhysicsBindSelectionBuffer->Value;
				const FMeshBufferSet& PhyiscsBindOffsetsBuffer = FoundPhysicsBindOffsetsBuffer->Value;

				// \TODO: More checks
				check(BarycentricDataChannel == 0);
				check(PhysicsBindBuffer.GetElementSize(BarycentricDataBuffer) == (int)sizeof(FReshapePointBindingData));
					
				TArrayView<const FReshapePointBindingData> BindingData(  
						(const FReshapePointBindingData*)PhysicsBindBuffer.GetBufferData(BarycentricDataBuffer),
						PhysicsBindBuffer.GetElementCount() );

				TArrayView<const int32> UsedIndices( 
						(const int32*)PhyiscsBindSelectionBuffer.GetBufferData(0), 
					    PhyiscsBindSelectionBuffer.GetElementCount());

				TArrayView<const int32> Offsets(
						(const int32*)PhyiscsBindOffsetsBuffer.GetBufferData(0), 
					    PhyiscsBindOffsetsBuffer.GetElementCount());

				ApplyToAllPhysicsBodies(*Mesh, *Mesh->GetSkeleton(), BindingData, UsedIndices, Offsets, TargetShape, MessageLogger);
			}
		}

		RemoveBindingData(Mesh, BindingDataIndex);
	}
}
