// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Operations.h"
#include "MuR/System.h"
#include "MuR/GeometryUtils.h"

namespace UE::Mutable::Private
{
	/** Rebuild the (previously bound) mesh data for a new shape. */
	void MeshApplyShape(FMesh* Mesh, GeometryUtils::FMeshGeometry& TargetShape, EMeshBindShapeFlags BindFlags, bool& bOutSuccess, FLiveInstanceLogger& MessageLogger);
}
