// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//#include "CoreMinimal.h"
#include "Algo/Reverse.h"
#include "Algo/RemoveIf.h"
#include "Math/Vector.h"
#include "Math/UnrealMathUtility.h"
#include "Spatial/PointHashGrid3.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"

#include "MuR/Mesh.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"
#include "MuR/MutableRuntimeModule.h"


namespace UE::Mutable::Private
{
	inline TArray<int32> MakeUniqueVertexMap(TArrayView<const FVector3f> Vertices)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshSmoothing_MakeUniqueVertexMap);

		// Build unique vertex map.
		const int32 NumVertices = Vertices.Num();

		const float CellSize = 0.01f; // TODO: this should be proportional to the mesh bounding box.
		UE::Geometry::TPointHashGrid3f<int32> VertHash(CellSize, INDEX_NONE);
		VertHash.Reserve(NumVertices);

		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			VertHash.InsertPointUnsafe(VertexIndex, Vertices[VertexIndex]);
		}

		TArray<int32> UniqueVertexMap;
		UniqueVertexMap.Init(INDEX_NONE, NumVertices);
	
		TArray<int32> CollapsedVertices;
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			if (UniqueVertexMap[VertexIndex] != INDEX_NONE)
			{
				continue;
			}
	
			FVector3f CurrentVertex = Vertices[VertexIndex];

			constexpr bool bAllowSrink = false;
			CollapsedVertices.Empty(32);

			VertHash.FindPointsInBall(CurrentVertex, TMathUtilConstants<float>::ZeroTolerance,
				[&CurrentVertex, &Vertices](const int32& Other) -> float 
				{ 
					return FVector3f::DistSquared(Vertices[Other], CurrentVertex); 
				},
				CollapsedVertices);
	
			for (int32 CollapsedVertexIndex : CollapsedVertices)
			{
				UniqueVertexMap[CollapsedVertexIndex] = VertexIndex;
			}
		}

		return UniqueVertexMap;
	}

	inline TArray<TArray<int32, TInlineAllocator<8>>> BuildVertexFaces(TArrayView<const uint32> Indices, const TArray<int32>& UniqueVertexMap)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshSmoothing_BuildVertexFaces);
		
		const int32 NumIndices  = Indices.Num();
		const int32 NumVertices = UniqueVertexMap.Num();
		
		TArray<TArray<int32, TInlineAllocator<8>>> VertexFaces;
		VertexFaces.SetNum(UniqueVertexMap.Num());

		check(NumIndices % 3 == 0);
		for (int32 Face = 0; Face < NumIndices; Face += 3)
		{
			VertexFaces[UniqueVertexMap[Indices[Face + 0]]].Add(Face);
			VertexFaces[UniqueVertexMap[Indices[Face + 1]]].Add(Face);
			VertexFaces[UniqueVertexMap[Indices[Face + 2]]].Add(Face);
		}

		return VertexFaces;
	}

	FORCEINLINE uint32 GetEdgeLowVertex(uint64 Edge)
	{
		return static_cast<uint32>(Edge);
	}

	FORCEINLINE uint32 GetEdgeHighVertex(uint64 Edge)
	{
		return static_cast<uint32>(Edge >> 32);
	}

	FORCEINLINE uint64 MakeEdge(uint32 E0, uint32 E1)
	{
		const uint64 TestMask = E0 < E1 ? 0 : ~0;
		return (static_cast<uint64>(E0) << (32 & TestMask)) | (static_cast<uint64>(E1) << (32 & (~TestMask)));
	}

	inline TMap<uint64, UE::Geometry::FIndex2i> BuildEdgesFaces(TArrayView<const uint32> Indices, const TArray<int32>& UniqueVertexMap)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshSmoothing_BuildEdgesFaces);

		const int32 NumIndices = Indices.Num();
		
		auto RemapVertexFaces = [&UniqueVertexMap](const UE::Geometry::FIndex3i& F) -> UE::Geometry::FIndex3i
		{
			return UE::Geometry::FIndex3i { UniqueVertexMap[F[0]], UniqueVertexMap[F[1]], UniqueVertexMap[F[2]] };
		};

		// 2-manifold meshes are assumed.	
		TMap<uint64, UE::Geometry::FIndex2i> EdgeFaces;
		EdgeFaces.Reserve(NumIndices);

		check(NumIndices % 3 == 0);
		for (int32 Face = 0; Face < NumIndices; Face += 3)
		{
			const UE::Geometry::FIndex3i FaceIndices = RemapVertexFaces(
			{
				static_cast<int32>(Indices[Face + 0]),
				static_cast<int32>(Indices[Face + 1]),
				static_cast<int32>(Indices[Face + 2]),
			});

			uint64 Edge0 = MakeEdge(FaceIndices[0], FaceIndices[1]);
			uint64 Edge1 = MakeEdge(FaceIndices[1], FaceIndices[2]);
			uint64 Edge2 = MakeEdge(FaceIndices[2], FaceIndices[0]);

			UE::Geometry::FIndex2i& Edge0Faces = EdgeFaces.FindOrAdd(Edge0, {-1, -1});
			Edge0Faces = Edge0Faces[0] < 0 ? UE::Geometry::FIndex2i{Face, -1} : UE::Geometry::FIndex2i{Edge0Faces[0], Face};

			UE::Geometry::FIndex2i& Edge1Faces = EdgeFaces.FindOrAdd(Edge1, {-1, -1});
			Edge1Faces = Edge1Faces[0] < 0 ? UE::Geometry::FIndex2i{Face, -1} : UE::Geometry::FIndex2i{Edge1Faces[0], Face};
			
			UE::Geometry::FIndex2i& Edge2Faces = EdgeFaces.FindOrAdd(Edge2, {-1, -1});
			Edge2Faces = Edge2Faces[0] < 0 ? UE::Geometry::FIndex2i{Face, -1} : UE::Geometry::FIndex2i{Edge2Faces[0], Face};
		}

		return EdgeFaces;
	}

	FORCEINLINE int32 FindEdgeOtherVertex(uint64 Edge, uint32 V)
	{
		check(GetEdgeLowVertex(Edge) == V || GetEdgeHighVertex(Edge) == V);
		return GetEdgeLowVertex(Edge) != V ? GetEdgeLowVertex(Edge) : GetEdgeHighVertex(Edge);
	}

	FORCEINLINE int32 FindOppositeFace(UE::Geometry::FIndex2i EdgeFaces, int32 Face)
	{
		if (Face == EdgeFaces[0])
		{
			return EdgeFaces[1];
		}

		if (Face == EdgeFaces[1])
		{
			return EdgeFaces[0];
		}	

		return INDEX_NONE;
	}

	FORCEINLINE uint32 FindFaceVertexNotInEdge(uint64 Edge, const UE::Geometry::FIndex3i& F)
	{
		if (Edge == MakeEdge(F[0], F[1])) return F[2];
		if (Edge == MakeEdge(F[0], F[2])) return F[1];
		if (Edge == MakeEdge(F[1], F[2])) return F[0];
	
		check(false);
		return -1;
	}

	FORCEINLINE int32 FindNextFaceVertex(int32 V, const UE::Geometry::FIndex3i& F)
	{
		if (V == F[0]) return F[1];
		if (V == F[1]) return F[2];
		if (V == F[2]) return F[0];

		check(false);
		return -1;
	}

	inline TTuple<TArray<int32>, TArray<int32>> BuildVertexRings(
		const TArrayView<const uint32> Indices,
		const TArray<int32>& UniqueVertexMap,
		const TArray<TArray<int32, TInlineAllocator<8>>>& VertexFaces,
		const TMap<uint64, UE::Geometry::FIndex2i>& EdgesFaces)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshSmoothing_BuildVertexRings);

#if DO_CHECK
		int32 BowtiedRingsFound = 0;
#endif
		auto RemapVertexFaces = [&UniqueVertexMap](const UE::Geometry::FIndex3i& F) -> UE::Geometry::FIndex3i
		{
			return UE::Geometry::FIndex3i { UniqueVertexMap[F[0]], UniqueVertexMap[F[1]], UniqueVertexMap[F[2]] };
		};

		const int32 NumVertices = UniqueVertexMap.Num();
	
		TArray<int32> VertexRingsOffsets;
		VertexRingsOffsets.Reserve(NumVertices*2);
		
		TArray<int32> VertexRings;
		VertexRings.Reserve(NumVertices*6);

		VertexRingsOffsets.Add(0);

		for (int32 V = 0; V < NumVertices; ++V)
		{
			if (UniqueVertexMap[V] != V)
			{
				continue;
			}

			const int32 NumVertexFaces = VertexFaces[V].Num();

			const int32 InitialFace = VertexFaces[V][0];
			UE::Geometry::FIndex3i FaceVertices = RemapVertexFaces(
			{
				static_cast<int32>(Indices[InitialFace + 0]), 
				static_cast<int32>(Indices[InitialFace + 1]), 
				static_cast<int32>(Indices[InitialFace + 2])
			});

			int32 V0 = FindNextFaceVertex(V,  FaceVertices);
			int32 V1 = FindNextFaceVertex(V0, FaceVertices);
			uint64 Edge0 = MakeEdge(V, V0);
			uint64 Edge1 = MakeEdge(V, V1);
			
			const int32 RingBeginOffset = VertexRings.Num();
		
			uint64 Edge = Edge1;
			int32 Face = FindOppositeFace(EdgesFaces[Edge1], InitialFace);

			VertexRings.Add(V0);
			VertexRings.Add(V1);

			if (Face >= 0)
			{
				for (;;)
				{
					FaceVertices = RemapVertexFaces(
					{
						static_cast<int32>(Indices[Face + 0]), 
						static_cast<int32>(Indices[Face + 1]), 
						static_cast<int32>(Indices[Face + 2])
					});

					const int32 NextVertex = FindFaceVertexNotInEdge(Edge, FaceVertices);
					
					Edge = MakeEdge(V, NextVertex);
					Face = FindOppositeFace(EdgesFaces[Edge], Face);

					if (Face == InitialFace)
					{
						break;
					}

					VertexRings.Add(NextVertex);
					
					if (Face < 0)
					{
						break;
					}
				}
			}

			// We have a closed ring.
			if (Face >= 0)
			{
				check(Edge == Edge0);
				check(Face == InitialFace);

				VertexRingsOffsets.Add(VertexRings.Num());
				continue;
			}

			const int32 RingHalfOffset = VertexRings.Num();

			Face = FindOppositeFace(EdgesFaces[Edge0], InitialFace);
			Edge = Edge0;

			if (Face >= 0)
			{
				for (;;)
				{
					FaceVertices = RemapVertexFaces(
					{
						static_cast<int32>(Indices[Face + 0]), 
						static_cast<int32>(Indices[Face + 1]), 
						static_cast<int32>(Indices[Face + 2])
					});

					const int32 NextVertex = FindFaceVertexNotInEdge(Edge, FaceVertices);	
					VertexRings.Add(NextVertex);

					Edge = MakeEdge(V, NextVertex);
					Face = FindOppositeFace(EdgesFaces[Edge], Face);
					
					if (Face < 0)
					{
						break;
					}	
				}
				
				const int32 RingEndOffset = VertexRings.Num();

				Algo::Reverse(VertexRings.GetData() + RingBeginOffset, RingEndOffset - RingBeginOffset);
				Algo::Reverse(VertexRings.GetData() + RingBeginOffset + RingEndOffset - RingHalfOffset, RingHalfOffset - RingBeginOffset);
			}

			// Mark ring as open.
			VertexRings.Add(-1);	

			const int32 OpenRingNumFaces = VertexRings.Num() - 1 - RingBeginOffset - 1;
			if (NumVertexFaces <= OpenRingNumFaces)
			{
				check(NumVertexFaces == OpenRingNumFaces);
					
				VertexRingsOffsets.Add(VertexRings.Num());
				continue;
			}

			// This case should happen rarely on well behaved meshes. 
			// TODO: For now the case is not supported. Review if actually needed.
#if DO_CHECK
			++BowtiedRingsFound;
#endif
			VertexRings.SetNum(RingBeginOffset, EAllowShrinking::No);
			VertexRingsOffsets.Add(VertexRings.Num());

			//continue;
		
			// The following code for computing bowtied rings is incomplete, but it gives an idea of how to proceed.
			//TBitArray VisitedVertexFaces(false, VertexFaces.Num());

			//for (int32 RingVertex = RingBeginOffset; RingVertex < RingEndOffset; ++RingVertex)
			//{
			//	UE::Geometry::FIndex2i EdgeFaces = EdgesFaces[MakeEdge(V, VetexRing[RingVetex])];

			//	if (EdgeFaces[0] > 0)
			//	{
			//		const int32 VertexFaceIndex = VertexFaces[V].IndexOf(EdgeFaces[0]); 
			//		check(VertexFaceIndex == INDEX_NONE);
			//		VisitedVertexFaces[VertexFaceIndex] = true;
			//	}

			//	if (EdgeFaces[1] > 0)
			//	{
			//		const int32 VertexFaceIndex = VertexFaces[V].IndexOf(EdgeFaces[1]); 
			//		check(VertexFaceIndex == INDEX_NONE);
			//		VisitedVertexFaces[VertexFaceIndex] = true;
			//	}
			//}


			//for (;;)
			//{

			//	int32 VertexFace = VisitedVeretexFaces.Find(false);
			//	if (VertexFace == INDEX_NONE)
			//	{
			//		break;
			//	}

			//	Face = VertexFaces[V][VertexFace];
			//	
			//	FaceVertices = RemapVertexFaces({ Indices[Face + 0], Indices[Face + 1], Indices[Face + 2] });
			//	
			//	V0 = FindNextVertexFace(V, FaceVertices);
			//	V1 = FindNextVertexFace(V0, FaceVertices);

			//	Edge0 = MakeEdge(V, V0); 
			//	Edge1 = MakeEdge(V, V1); 

			//	const int32 CurrentRingBegin = VetexRings.Num();

			//	VertexRings.Add(V0);
			//	VertexRings.Add(V1);
			//	
			//	const int32 
			//	Edge = Edge1;
			//	for (;;)
			//	{
			//		if (Face < 0)
			//		{
			//			break;
			//		}


			//		FaceVertices = RemapVertexFaces({ Indices[Face + 0], Indices[Face + 1], Indices[Face + 2] });

			//		const int32 NextVertex = FindFaceVertexNotInEdge(Edge, FaceVertices);
			//		VertexRings.Add(NextVertex);

			//		Edge = MakeEdge(V, NextVertex);
			//		Face = FindOtherFace(EdgesFaces[Edge], Face);

			//		const int32 VertexFaceIndex = VertexFaces[V].IndexOf(Face);
			//		check(VertexFaceIndex != INDEX_NONE);
			//		VisitedVertexFaces[VertexFaceIndex] = true;
			//	}

			//	const int32 CurrentRingHalf = VertexRings.Num();

			//	Edge = Edge0;
			//	for (;;)
			//	{
			//		if (Face < 0)
			//		{
			//			break;
			//		}

			//		FaceVertices = { Indices[Face + 0], Indices[Face + 1], Indices[Face + 2] };
			//		FaceVertices = { UniqueVertexMap[FaceVertices[0]], UniqueVertexMap[FaceVertices[1]], UniqueVertexMap[FaceVertices[2]] };

			//		const int32 NextVertex = FindFaceVertexNotInEdge(Edge, FaceVertices);
			//		VertexRings.Add(NextVertex);
			//		
			//		Edge = MakeEdge(V, FindFaceVertexNotInEdge(Edge, FaceVertices));
			//		Face = FindOtherFace(Face, EdgeFaces[Edge]);

			//		const int32 VertexFaceIndex = VertexFaces[V].IndexOf(Face);
			//		check(VertexFaceIndex != INDEX_NONE);
			//		VisitedVertexFaces[VertexFaceIndex] = true;

			//	}

			//	const int32 CurrentRingEnd = VertexRings.Num();


			//	Algo::Reverse(VertexRings.GetData() + CurrentRingBegin, CurrentRingEnd - CurrentRingBegin);
			//	Algo::Reverse(VertexRings.GetData() + CurrentRingEnd - CurrentRingHalf, CurrentRingHalf - CurrentRingBegin);
			//}			
		}
		
#if DO_CHECK
		if (BowtiedRingsFound > 0)
		{
			UE_LOGF(LogMutableCore, Warning, 
				"%d Bowtied vertex rings found in a mesh while computing smoothing operation data, this is currently not supported. " 
					 "Smoothing for the vertices involved may not work properly.", 
				BowtiedRingsFound);
		}
#endif

		return MakeTuple(MoveTemp(VertexRingsOffsets), MoveTemp(VertexRings));
	}
	
	inline void RemoveLaplacianData(FMesh& Mesh)
	{
		const int32 NewNumAdditionalBuffers = Algo::RemoveIf(Mesh.AdditionalBuffers,
		[](const TPair<EMeshBufferType, FMeshBufferSet>& A)
		{
			const bool bIsLaplacianBuffer =
				A.Key == EMeshBufferType::UniqueVertexMap   ||
				A.Key == EMeshBufferType::MeshLaplacianData ||
				A.Key == EMeshBufferType::MeshLaplacianOffsets;
			
			return bIsLaplacianBuffer;
		});

		Mesh.AdditionalBuffers.SetNum(NewNumAdditionalBuffers, EAllowShrinking::No);
	}

	inline void SmoothMeshLaplacian(FMesh& Mesh, bool bRemoveData, int32 Iterations = 1)
	{
		MUTABLE_CPUPROFILER_SCOPE(MeshSmoothing_SmoothMeshLaplacian);

		using BufferEntryType = TPair<EMeshBufferType, FMeshBufferSet>;

		const BufferEntryType* MeshLaplacianDataBuffer = Mesh.AdditionalBuffers.FindByPredicate(
				[](const BufferEntryType& E) { return E.Key == EMeshBufferType::MeshLaplacianData; });

		const BufferEntryType* MeshLaplacianOffsetsBuffer = Mesh.AdditionalBuffers.FindByPredicate(
				[](const BufferEntryType& E) { return E.Key == EMeshBufferType::MeshLaplacianOffsets; });

		const BufferEntryType* MeshUniqueVertexMapBuffer = Mesh.AdditionalBuffers.FindByPredicate(
				[](const BufferEntryType& E) { return E.Key == EMeshBufferType::UniqueVertexMap; });

		const bool bHasNecessaryBuffers = MeshLaplacianDataBuffer && MeshLaplacianOffsetsBuffer && MeshUniqueVertexMapBuffer;
		if (!bHasNecessaryBuffers)
		{
			UE_LOGF(LogMutableCore, Warning, "Smooth laplacian with source mesh missing some necessary buffers. Ignoring operation.");
			return;
		}

		TArrayView<const int32> LaplacianDataVertexOffsets = TArrayView<const int32>(
				reinterpret_cast<const int32*>(MeshLaplacianOffsetsBuffer->Value.GetBufferData(0)),
				MeshLaplacianOffsetsBuffer->Value.GetElementCount());

		TArrayView<const int32> LaplacianVertexRingData = TArrayView<const int32>(
				reinterpret_cast<const int32*>(MeshLaplacianDataBuffer->Value.GetBufferData(0)),
				MeshLaplacianDataBuffer->Value.GetElementCount());
		
		TArrayView<const int32> UniqueVertexMap = TArrayView<const int32>(
				reinterpret_cast<const int32*>(MeshUniqueVertexMapBuffer->Value.GetBufferData(0)),
				MeshUniqueVertexMapBuffer->Value.GetElementCount());

		const UntypedMeshBufferIterator SrcPositionBegin(Mesh.GetVertexBuffers(), EMeshBufferSemantic::Position);

		const int32 NumVertices = Mesh.GetVertexCount();

		TArray<FVector3f> AccumulatedPositions;
		AccumulatedPositions.SetNumUninitialized(NumVertices);

		FMemory::Memcpy(AccumulatedPositions.GetData(), SrcPositionBegin.ptr(), AccumulatedPositions.NumBytes());

		TArray<FVector3f, TInlineAllocator<8>> RingVerticesStorage;
		
		for (int32 VertexIndex = 0, UniqueVertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			if (VertexIndex != UniqueVertexMap[VertexIndex])
			{
				continue;
			}

			const int32 LaplacianVertexDataBegin = LaplacianDataVertexOffsets[UniqueVertexIndex];
			const int32 LaplacianVertexDataEnd   = LaplacianDataVertexOffsets[UniqueVertexIndex + 1];
			++UniqueVertexIndex;

			// Skip empty rings caused by bowtied topology. 
			if (LaplacianVertexDataEnd <= LaplacianVertexDataBegin)
			{
				continue;
			}

			const bool bIsOpenRing   = LaplacianVertexRingData[LaplacianVertexDataEnd - 1] < 0;
			const int32 RingNumElems = LaplacianVertexDataEnd - LaplacianVertexDataBegin - static_cast<int32>(bIsOpenRing);

			TArray<FVector3f, TInlineAllocator<8>>& Ring = RingVerticesStorage;
			Ring.SetNum(RingNumElems);
			for (int32 R = 0; R < RingNumElems; ++R)
			{
				Ring[R] = (SrcPositionBegin + LaplacianVertexRingData[LaplacianVertexDataBegin + R]).GetAsVec3f();
			}

			const FVector3f VertexPosition = (SrcPositionBegin + VertexIndex).GetAsVec3f();
			
			FVector3f WeightedPositionLaplacian = FVector3f::ZeroVector;
			float TotalWeight = 0.0f;
			
			int32 PrevIndex = bIsOpenRing ? 0 : RingNumElems - 1;
			int32 NextIndex = 1;

			for (int32 R = 0; R < RingNumElems; ++R)
			{
				const FVector3f CoAU = VertexPosition - Ring[PrevIndex]; 
				const FVector3f CoAV = Ring[R] - Ring[PrevIndex];

				const FVector3f CoBU = VertexPosition - Ring[NextIndex]; 
				const FVector3f CoBV = Ring[R] - Ring[NextIndex]; 

				const float CoA = FVector3f::DotProduct(CoAU, CoAV) * 
								  FMath::InvSqrt(
										  FMath::Max(UE_KINDA_SMALL_NUMBER, FVector3f::CrossProduct(CoAU, CoAV).SquaredLength())); 

				const float CoB = FVector3f::DotProduct(CoBU, CoBV) * 
								  FMath::InvSqrt(
										  FMath::Max(UE_KINDA_SMALL_NUMBER, FVector3f::CrossProduct(CoBU, CoBV).SquaredLength()));

				const float Weight = (CoA + CoB);// * 0.5f;	

				WeightedPositionLaplacian += Ring[R] * Weight;
				TotalWeight += Weight;

				PrevIndex = R;
				NextIndex = R + 2 < RingNumElems ? R + 2 : (R + 1)*static_cast<int32>(bIsOpenRing);
			}

			const FVector3f VertexPositionLaplacian = 
				(WeightedPositionLaplacian / FMath::Max(UE_KINDA_SMALL_NUMBER, TotalWeight));

			AccumulatedPositions[VertexIndex] = VertexPosition + (VertexPositionLaplacian-VertexPosition)*0.5f;
		}

		// Copy positions to the collapsed vertices.
		constexpr SIZE_T PositionElemSize = sizeof(FVector3f);

		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const int32 MappedIndex = UniqueVertexMap[VertexIndex];  

			FMemory::Memcpy((SrcPositionBegin + VertexIndex).ptr(), &AccumulatedPositions[MappedIndex], PositionElemSize);
		}

		if (bRemoveData)
		{
			RemoveLaplacianData(Mesh);
		}
	}
}

