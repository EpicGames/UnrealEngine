// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/MeshPrivate.h"
#include "MuR/ConvertData.h"
#include "MuR/Platform.h"


namespace UE::Mutable::Private
{

    //---------------------------------------------------------------------------------------------
    //! Reference version
    //---------------------------------------------------------------------------------------------
    inline void MeshTransform(FMesh* Result, const FMesh* pBase, const FMatrix44f& transform, bool& bOutSuccess)
	{
		bOutSuccess = true;

        uint32_t vcount = pBase->GetVertexBuffers().GetElementCount();

        if ( !vcount )
		{
			bOutSuccess = false;
			return;
		}

		if (pBase->GetVertexBuffers().IsDescriptor())
		{
			return; // Success.
		}

		Result->CopyFrom(*pBase);

		FMatrix44f transformIT = transform.Inverse().GetTransposed();

        const FMeshBufferSet& MBSPriv = Result->GetVertexBuffers();
        for ( int32 b=0; b<MBSPriv.Buffers.Num(); ++b )
        {

            for ( int32 c=0; c<MBSPriv.Buffers[b].Channels.Num(); ++c )
            {
                EMeshBufferSemantic sem = MBSPriv.Buffers[b].Channels[c].Semantic;
                int semIndex = MBSPriv.Buffers[b].Channels[c].SemanticIndex;

                UntypedMeshBufferIterator it( Result->GetVertexBuffers(), sem, semIndex );

                switch ( sem )
                {
                case EMeshBufferSemantic::Position:
                    for ( uint32_t v=0; v<vcount; ++v )
                    {
                        FVector4f value( 0.0f, 0.0f, 0.0f, 1.0f );
                        for( int i=0; i<it.GetComponents(); ++i )
                        {
                            ConvertData( i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat() );
                        }

                        value = transform.TransformFVector4( value );

                        for( int i=0; i<it.GetComponents(); ++i )
                        {
                            ConvertData( i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32 );
                        }

                        ++it;
                    }
                    break;

                case EMeshBufferSemantic::Normal:
                case EMeshBufferSemantic::Tangent:
                case EMeshBufferSemantic::Binormal:
	                {
                		const uint8 NumComponents = FMath::Min(it.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.

                		for ( uint32_t v=0; v<vcount; ++v )
                		{
                			FVector4f value( 0.0f, 0.0f, 0.0f, 0.0f );
                        
                			for (uint8 i = 0; i < NumComponents; ++i)
                			{
                				ConvertData( i, &value[0],  EMeshBufferFormat::Float32, it.ptr(), it.GetFormat() );
                			}

                			value = transformIT.TransformFVector4(value);

                			// Notice that 4th component is not modified.
                			for (uint8 i = 0; i < NumComponents; ++i)
                			{
                				ConvertData( i, it.ptr(), it.GetFormat(), &value[0],  EMeshBufferFormat::Float32 );
                			}

                			++it;
                		}
	                }
                	break;

                default:
                    break;
                }
            }
        }
	}

}

