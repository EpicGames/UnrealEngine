// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshTransformWithBone.h"

#include "MuR/Mesh.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"

#include "MuR/ParallelExecutionUtils.h"
#include "PackedNormal.h"

#include "UObject/NameTypes.h"

namespace UE::Mutable::Private
{


void MeshTransformWithBoneInline(FMesh* Mesh, const FMatrix44f& Transform, FName InBoneName, const float InThresholdFactor)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshTransformWithBone);

	if (!Mesh || Mesh->GetVertexCount() == 0 || Mesh->IsReference() || !Mesh->GetSkeleton())
	{
		return;
	}


	TManagedPtr<const FSkeleton>& BaseSkeleton = Mesh->Skeleton;

	const int32 BaseBoneIndex = BaseSkeleton->BoneNames.Find(InBoneName);
	if (BaseBoneIndex == INDEX_NONE)
	{
		return;
	}

	TBitArray<> AffectedBoneMapIndices;

	// Find affected bones
	{
		const int32 NumBonesBoneMap = Mesh->BoneMap.Num();
		const TArray<FBoneIdOrIndex>& BoneMap = Mesh->BoneMap;
		AffectedBoneMapIndices.SetNum(NumBonesBoneMap, false);

		const int32 NumBones = BaseSkeleton->GetNumBones();
		TBitArray<> AffectedSkeletonBones;
		AffectedSkeletonBones.SetNum(NumBones, false);

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const int32 ParentBoneIndex = BaseSkeleton->BoneParents[BoneIndex];
			check(ParentBoneIndex < BoneIndex);

			const bool bIsBoneAffected = (AffectedSkeletonBones.IsValidIndex(ParentBoneIndex) && AffectedSkeletonBones[ParentBoneIndex])
				|| BoneIndex == BaseBoneIndex;

			AffectedSkeletonBones[BoneIndex] = bIsBoneAffected;
		}

		for (int32 Index = 0; Index < NumBonesBoneMap; ++Index)
		{
			AffectedBoneMapIndices[Index] = AffectedSkeletonBones[BoneMap[Index].Index];
		}
	}

	// Get pointers to skinning data
	UntypedMeshBufferIteratorConst BoneIndicesIterBegin(Mesh->VertexBuffers, EMeshBufferSemantic::BoneIndices, 0);
	UntypedMeshBufferIteratorConst BoneWeightsIterBegin(Mesh->VertexBuffers, EMeshBufferSemantic::BoneWeights, 0);
	if (!BoneIndicesIterBegin.ptr() || !BoneWeightsIterBegin.ptr())
	{
		// No skinning data
		return;
	}

	const int32 NumVertices = Mesh->GetVertexCount();
	const int32 NumWeights = BoneWeightsIterBegin.GetComponents();

	const uint16 MinWeightThreshold = (BoneWeightsIterBegin.GetFormat() == EMeshBufferFormat::NUInt8 ? MAX_uint8 : MAX_uint16) * InThresholdFactor;

	// Classify which vertices in the SourceMesh are completely bounded by the BoundingMesh geometry.
	// If no BoundingMesh is provided, this defaults to act exactly like UE::Mutable::Private::MeshTransform
	TBitArray<> VertexInBoneHierarchy;
	VertexInBoneHierarchy.Init(false, NumVertices);
	{
		for (const FMeshSurface& Surface : Mesh->Surfaces)
		{
			const int32 AffectedBoneMapIndex = AffectedBoneMapIndices.FindFrom(true, int32(Surface.BoneMapIndex), int32(Surface.BoneMapIndex + Surface.BoneMapCount));
			if (AffectedBoneMapIndex == INDEX_NONE)
			{
				// Skip. No bones affected in this surface, vertices won't be affected either.
				continue;
			}

			const int32 FirstBoneIndexInBoneMap = Surface.BoneMapIndex;

			const int32 VertexStart = Surface.SubMeshes[0].VertexBegin;
			const int32 VertexEnd = Surface.SubMeshes.Last().VertexEnd;

			for (int32 VertexIndex = VertexStart; VertexIndex < VertexEnd; ++VertexIndex)
			{
				int8 NumValidWeights = 0;

				switch (BoneWeightsIterBegin.GetFormat())
				{
				case EMeshBufferFormat::NUInt8:
				{
					// Skin weights are ordered from highest to lowest. Break once the weight value falls below the threshold.
					const uint8* Weights = (BoneWeightsIterBegin + VertexIndex).ptr();
					for (; NumValidWeights < NumWeights && *Weights >= (uint8)MinWeightThreshold; ++NumValidWeights, ++Weights);
					break;
				}

				case EMeshBufferFormat::NUInt16:
				{
					// Skin weights are ordered from highest to lowest. Break once the weight value falls below the threshold.
					const uint16* Weights = reinterpret_cast<const uint16*>((BoneWeightsIterBegin + VertexIndex).ptr());
					for (; NumValidWeights < NumWeights && *Weights >= MinWeightThreshold; ++NumValidWeights, ++Weights);
					break;
				}
				default: unimplemented();
				}

				switch (BoneIndicesIterBegin.GetFormat())
				{
				case EMeshBufferFormat::UInt8:
				{
					const uint8* Indices = (BoneIndicesIterBegin + VertexIndex).ptr();
					for (int32 Index = 0; Index < NumValidWeights; ++Index)
					{
						if (AffectedBoneMapIndices[FirstBoneIndexInBoneMap + *Indices])
						{
							VertexInBoneHierarchy[VertexIndex] = true;
							break;
						}

						++Indices;
					}
					break;
				}

				case EMeshBufferFormat::UInt16:
				{
					const uint16* Indices = reinterpret_cast<const uint16*>((BoneIndicesIterBegin + VertexIndex).ptr());
					for (int32 Index = 0; Index < NumValidWeights; ++Index)
					{
						if (AffectedBoneMapIndices[FirstBoneIndexInBoneMap + *Indices])
						{
							VertexInBoneHierarchy[VertexIndex] = true;
							break;
						}

						++Indices;
					}
					break;
				}
				default: unimplemented();
				}
			}
		}
	}

	// Get pointers to vertex position data
	const MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> PositionIterBegin(Mesh->VertexBuffers, EMeshBufferSemantic::Position, 0);

	check(PositionIterBegin.ptr());
	check(PositionIterBegin.GetFormat() == EMeshBufferFormat::Float32 && PositionIterBegin.GetComponents() == 3);

	const FMatrix44f TransformInvT = Transform.Inverse().GetTransposed();

	constexpr int32 NumVerticesPerBatch = 1 << 13;

	// Tangent frame buffers are optional.
	const UntypedMeshBufferIterator NormalIterBegin(Mesh->VertexBuffers, EMeshBufferSemantic::Normal, 0);
	const UntypedMeshBufferIterator TangentIterBegin(Mesh->VertexBuffers, EMeshBufferSemantic::Tangent, 0);
	const UntypedMeshBufferIterator BiNormalIterBegin(Mesh->VertexBuffers, EMeshBufferSemantic::Binormal, 0);

	auto ProcessVertexBatch =
		[
			PositionIterBegin,
				NormalIterBegin,
				TangentIterBegin,
				BiNormalIterBegin,
				NumVertices,
				NumVerticesPerBatch,
				&VertexInBoneHierarchy,
				Transform,
				TransformInvT
		](int32 BatchId)
		{
			const uint8 NumNormalComponents = FMath::Min(NormalIterBegin.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.
			const uint8 NumTangentComponents = FMath::Min(TangentIterBegin.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.
			const uint8 NumBiNormalComponents = FMath::Min(BiNormalIterBegin.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.

			const EMeshBufferFormat NormalFormat = NormalIterBegin.GetFormat();
			const EMeshBufferFormat TangentFormat = TangentIterBegin.GetFormat();

			const int32 BatchBeginVertIndex = BatchId * NumVerticesPerBatch;
			const int32 BatchEndVertIndex = FMath::Min(BatchBeginVertIndex + NumVerticesPerBatch, NumVertices);

			const bool bHasOptimizedBuffers = NormalFormat == EMeshBufferFormat::PackedDirS8_W_TangentSign && TangentFormat == EMeshBufferFormat::PackedDirS8;

			for (int32 VertexIndex = VertexInBoneHierarchy.FindFrom(true, BatchBeginVertIndex, BatchEndVertIndex); VertexIndex >= 0;)
			{
				const int32 AffectedSpanBegin = VertexIndex;
				VertexIndex = VertexInBoneHierarchy.FindFrom(false, VertexIndex, BatchEndVertIndex);

				// At the end of the buffer we may not find a false element, in that case
				// FindForm returns INDEX_NONE, set the vertex at the range end. 
				VertexIndex = VertexIndex >= 0 && VertexIndex < BatchEndVertIndex ? VertexIndex : BatchEndVertIndex;
				const int32 AffectedSpanEnd = VertexIndex;

				// VertexIndex may be one past the end of the array, VertexIndex will become INDEX_NONE
				// and the loop will finish.
				VertexIndex = VertexInBoneHierarchy.FindFrom(true, VertexIndex, BatchEndVertIndex);
				VertexIndex = VertexIndex < BatchEndVertIndex ? VertexIndex : INDEX_NONE;

				MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> PositionIter = (PositionIterBegin + AffectedSpanBegin);
				for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
				{
					FVector3f* Position = reinterpret_cast<FVector3f*>(PositionIter.ptr());
					*Position = Transform.TransformFVector4(*Position);
					PositionIter++;
				}

				if (bHasOptimizedBuffers)
				{
					UntypedMeshBufferIterator TangentIter = TangentIterBegin + AffectedSpanBegin;
					UntypedMeshBufferIterator NormalIter = NormalIterBegin + AffectedSpanBegin;

					for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
					{
						// Tangents
						FPackedNormal* PackedTangent = reinterpret_cast<FPackedNormal*>(TangentIter.ptr());
						FVector4f Tangent = TransformInvT.TransformFVector4(PackedTangent->ToFVector3f());
						*PackedTangent = *reinterpret_cast<FVector3f*>(&Tangent);
						TangentIter++;

						// Normals
						FPackedNormal* PackedNormal = reinterpret_cast<FPackedNormal*>(NormalIter.ptr());
						int8 W = PackedNormal->Vector.W;
						FVector4f Normal = TransformInvT.TransformFVector4(PackedNormal->ToFVector3f());
						*PackedNormal = *reinterpret_cast<FVector3f*>(&Normal);
						PackedNormal->Vector.W = W;
						NormalIter++;
					}
				}
				else
				{
					MUTABLE_CPUPROFILER_SCOPE(MeshTransform_Vertices_SlowPath);
					if (NormalIterBegin.ptr())
					{
						UntypedMeshBufferIterator it = NormalIterBegin + AffectedSpanBegin;
						for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
						{
							FVector4f value(0.0f, 0.0f, 0.0f, 0.0f);

							for (uint8 i = 0; i < NumNormalComponents; ++i)
							{
								ConvertData(i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat());
							}

							value = TransformInvT.TransformFVector4(value);

							// Notice that 4th component is not modified.
							for (uint8 i = 0; i < NumNormalComponents; ++i)
							{
								ConvertData(i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32);
							}
							it++;
						}
					}

					if (TangentIterBegin.ptr())
					{
						UntypedMeshBufferIterator it = TangentIterBegin + AffectedSpanBegin;
						for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
						{
							FVector4f value(0.0f, 0.0f, 0.0f, 0.0f);

							for (uint8 i = 0; i < NumTangentComponents; ++i)
							{
								ConvertData(i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat());
							}

							value = TransformInvT.TransformFVector4(value);

							// Notice that 4th component is not modified.
							for (uint8 i = 0; i < NumTangentComponents; ++i)
							{
								ConvertData(i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32);
							}
							it++;
						}
					}

					if (BiNormalIterBegin.ptr())
					{
						UntypedMeshBufferIterator it = BiNormalIterBegin + AffectedSpanBegin;
						for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
						{
							FVector4f value(0.0f, 0.0f, 0.0f, 0.0f);

							for (uint8 i = 0; i < NumBiNormalComponents; ++i)
							{
								ConvertData(i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat());
							}

							value = TransformInvT.TransformFVector4(value);

							// Notice that 4th component is not modified.
							for (uint8 i = 0; i < NumBiNormalComponents; ++i)
							{
								ConvertData(i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32);
							}
							it++;
						}
					}
				}
			}
		};

	const int32 NumBatches = FMath::DivideAndRoundUp<int32>(NumVertices, NumVerticesPerBatch);
	ParallelExecutionUtils::InvokeBatchParallelFor(NumBatches, ProcessVertexBatch);
}

}
