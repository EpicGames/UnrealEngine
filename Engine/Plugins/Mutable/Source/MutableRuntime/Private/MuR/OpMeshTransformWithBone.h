// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class FName;

namespace UE::Mutable::Private
{
class FMesh;

/**  */
extern void MeshTransformWithBoneInline(FMesh* Mesh, const FMatrix44f& Transform, FName BoneName, const float Threshold);

}
