// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpMeshTransformWithMesh.h"

#include "OpMeshClipWithMesh.h"
#include "MuR/Mesh.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableTrace.h"

#include "Async/ParallelFor.h"
#include "PackedNormal.h"

namespace UE::Mutable::Private
{
void MeshTransformWithMesh(FMesh* Result, const FMesh* SourceMesh, const FMesh* BoundingMesh, const FMatrix44f& Transform, bool& bOutSuccess)
{
	MUTABLE_CPUPROFILER_SCOPE(MeshClipWithMesh);
	bOutSuccess = true;

	const uint32 VCount = SourceMesh->GetVertexBuffers().GetElementCount();
	if (!VCount)
	{
		bOutSuccess = false;
		return; // OutSuccess false indicates the SourceMesh can be reused in this case. 
	}

	Result->CopyFrom(*SourceMesh);

	// Classify which vertices in the SourceMesh are completely bounded by the BoundingMesh geometry.
	// If no BoundingMesh is provided, this defaults to act exactly like UE::Mutable::Private::MeshTransform
	TBitArray<> VertexInBoundaryMesh;  
	if (BoundingMesh)
	{
		MeshClipMeshClassifyVertices(VertexInBoundaryMesh, SourceMesh, BoundingMesh);
	}

	// Get pointers to vertex position data
	const MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> PositionIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Position, 0);
	if (!PositionIterBegin.ptr())
	{
		// Formats not implemented
		check(false);
		bOutSuccess = false;
		return;
	}
	
	check(PositionIterBegin.GetFormat() == EMeshBufferFormat::Float32 && PositionIterBegin.GetComponents() == 3);


	const FMatrix44f TransformInvT = Transform.Inverse().GetTransposed();

	const int32 NumVertices = Result->GetVertexCount();
	constexpr int32 NumVerticesPerBatch = 1 << 13;

	// Tangent frame buffers are optional.
	const UntypedMeshBufferIterator NormalIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Normal, 0);
	const UntypedMeshBufferIterator TangentIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Tangent, 0);
	const UntypedMeshBufferIterator BiNormalIterBegin(Result->VertexBuffers, EMeshBufferSemantic::Binormal, 0);

	auto ProcessVertexBatch =
		[
			PositionIterBegin,
				NormalIterBegin,
				TangentIterBegin,
				BiNormalIterBegin,
				NumVertices,
				NumVerticesPerBatch,
				&VertexInBoundaryMesh,
				Transform,
				TransformInvT
		](int32 BatchId)
		{
			//const int32 PositionBufferElementSize = PositionIterBegin.GetElementSize();
			//const int32 NormalBufferElementSize = NormalIterBegin.GetElementSize();
			//const int32 TangentBufferElementSize = TangentIterBegin.GetElementSize();

			const uint8 NumNormalComponents = FMath::Min(NormalIterBegin.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.
			const uint8 NumTangentComponents = FMath::Min(TangentIterBegin.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.
			const uint8 NumBiNormalComponents = FMath::Min(BiNormalIterBegin.GetComponents(), 3); // Due to quantization, the serialized component W may not be zero. Must be zero to avoid being affected by the transform position.

			const EMeshBufferFormat NormalFormat = NormalIterBegin.GetFormat();
			const EMeshBufferFormat TangentFormat = TangentIterBegin.GetFormat();

			const int32 BatchBeginVertIndex = BatchId * NumVerticesPerBatch;
			const int32 BatchEndVertIndex = FMath::Min(BatchBeginVertIndex + NumVerticesPerBatch, NumVertices);

			const bool bHasOptimizedBuffers = NormalFormat == EMeshBufferFormat::PackedDirS8_W_TangentSign && TangentFormat == EMeshBufferFormat::PackedDirS8;

			for (int32 VertexIndex = VertexInBoundaryMesh.FindFrom(true, BatchBeginVertIndex, BatchEndVertIndex); VertexIndex >= 0;)
			{
				const int32 AffectedSpanBegin = VertexIndex;
				VertexIndex = VertexInBoundaryMesh.FindFrom(false, VertexIndex, BatchEndVertIndex);

				// At the end of the buffer we may not find a false element, in that case
				// FindForm returns INDEX_NONE, set the vertex at the range end. 
				VertexIndex = VertexIndex >= 0 && VertexIndex < BatchEndVertIndex ? VertexIndex : BatchEndVertIndex;
				const int32 AffectedSpanEnd = VertexIndex;

				// VertexIndex may be one past the end of the array, VertexIndex will become INDEX_NONE
				// and the loop will finish.
				VertexIndex = VertexInBoundaryMesh.FindFrom(true, VertexIndex, BatchEndVertIndex);
				VertexIndex = VertexIndex < BatchEndVertIndex ? VertexIndex : INDEX_NONE;

				MeshBufferIterator<EMeshBufferFormat::Float32, float, 3> PositionIter = (PositionIterBegin + AffectedSpanBegin);
				for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
				{
					FVector3f* Position = reinterpret_cast<FVector3f*>(PositionIter.ptr());
					*Position = Transform.TransformFVector4(*Position);
					PositionIter++;
				}

				if (bHasOptimizedBuffers)
				{
					UntypedMeshBufferIterator TangentIter = TangentIterBegin + AffectedSpanBegin;
					UntypedMeshBufferIterator NormalIter = NormalIterBegin + AffectedSpanBegin;

					for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
					{
						// Tangents
						FPackedNormal* PackedTangent = reinterpret_cast<FPackedNormal*>(TangentIter.ptr());
						FVector4f Tangent = TransformInvT.TransformFVector4(PackedTangent->ToFVector3f());
						*PackedTangent = *reinterpret_cast<FVector3f*>(&Tangent);
						TangentIter++;

						// Normals
						FPackedNormal* PackedNormal = reinterpret_cast<FPackedNormal*>(NormalIter.ptr());
						int8 W = PackedNormal->Vector.W;
						FVector4f Normal = TransformInvT.TransformFVector4(PackedNormal->ToFVector3f());
						*PackedNormal = *reinterpret_cast<FVector3f*>(&Normal);
						PackedNormal->Vector.W = W;
						NormalIter++;
					}
				}
				else
				{
					MUTABLE_CPUPROFILER_SCOPE(MeshTransform_Vertices_SlowPath);
					if (NormalIterBegin.ptr())
					{
						UntypedMeshBufferIterator it = NormalIterBegin + AffectedSpanBegin;
						for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
						{
							FVector4f value(0.0f, 0.0f, 0.0f, 0.0f);

							for (uint8 i = 0; i < NumNormalComponents; ++i)
							{
								ConvertData(i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat());
							}

							value = TransformInvT.TransformFVector4(value);

							// Notice that 4th component is not modified.
							for (uint8 i = 0; i < NumNormalComponents; ++i)
							{
								ConvertData(i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32);
							}
							it++;
						}
					}

					if (TangentIterBegin.ptr())
					{
						UntypedMeshBufferIterator it = TangentIterBegin + AffectedSpanBegin;
						for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
						{
							FVector4f value(0.0f, 0.0f, 0.0f, 0.0f);

							for (uint8 i = 0; i < NumTangentComponents; ++i)
							{
								ConvertData(i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat());
							}

							value = TransformInvT.TransformFVector4(value);

							// Notice that 4th component is not modified.
							for (uint8 i = 0; i < NumTangentComponents; ++i)
							{
								ConvertData(i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32);
							}
							it++;
						}
					}

					if (BiNormalIterBegin.ptr())
					{
						UntypedMeshBufferIterator it = BiNormalIterBegin + AffectedSpanBegin;
						for (int32 Index = AffectedSpanBegin; Index < AffectedSpanEnd; ++Index)
						{
							FVector4f value(0.0f, 0.0f, 0.0f, 0.0f);

							for (uint8 i = 0; i < NumBiNormalComponents; ++i)
							{
								ConvertData(i, &value[0], EMeshBufferFormat::Float32, it.ptr(), it.GetFormat());
							}

							value = TransformInvT.TransformFVector4(value);

							// Notice that 4th component is not modified.
							for (uint8 i = 0; i < NumBiNormalComponents; ++i)
							{
								ConvertData(i, it.ptr(), it.GetFormat(), &value[0], EMeshBufferFormat::Float32);
							}
							it++;
						}
					}
				}
			}
		};

	const int32 NumBatches = FMath::DivideAndRoundUp<int32>(NumVertices, NumVerticesPerBatch);

	if (NumBatches == 1)
	{
		ProcessVertexBatch(0);
	}
	else
	{
		ParallelFor(NumBatches, ProcessVertexBatch);
	}

}

}
