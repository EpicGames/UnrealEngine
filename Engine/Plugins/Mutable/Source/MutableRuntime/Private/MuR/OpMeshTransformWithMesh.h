// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

namespace UE::Mutable::Private
{
class FMesh;

/**  */
extern void MeshTransformWithMesh(FMesh* Result, const FMesh* SourceMesh, const FMesh* BoundingMesh, const FMatrix44f& Transform, bool& bOutSuccess);

}
