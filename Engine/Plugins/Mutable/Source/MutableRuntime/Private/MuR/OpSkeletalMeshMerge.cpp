// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/OpSkeletalMeshMerge.h"

#include "MuR/CodeRunner.h"
#include "MuR/SkeletalMesh.h"
#include "MuR/LOD.h"
#include "MuR/Mesh.h"
#include "MuR/Material.h"
#include "MuR/OpMeshMerge.h"

// Temp 
#include "MuR/Operations.h"
#include "MuR/TVariant.h"
#include "MuR/ManagedPointer.h"
#include "MuR/Platform.h"

#include "UObject/StrongObjectPtr.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/MorphTargetVertexCodec.h"

#include "Algo/Count.h"

#include "MuR/MutableTrace.h"
#include "MuR/MutableRuntimeModule.h"

namespace UE::Mutable::Private
{
	void SkeletalMeshMerge(FSkeletalMesh& Result, FSkeletalMesh& AddedMesh)
	{
		MUTABLE_CPUPROFILER_SCOPE(SkeletalMeshMerge);

		// TODO SKMPIN
		// TODO: Append FMeshes instead of merging them and implement a new op 
		// to format the data when copying to the USkeletalMesh render data (async). 
		// 
		// TODO: Append Skeletons and Pose
		// TODO: Append PhysicsBodies

		TMap<uint32, uint32> AddedMeshIdMap; // Key is old Id. Value is new Id.
		
		// Merge Materials
		for (int32 Index = 0; Index < AddedMesh.MaterialSlotIds.Num(); ++Index)
		{
			const int32 OldId = AddedMesh.MaterialSlotIds[Index];

			uint32 NewId;
			if (Result.MaterialSlotIds.Contains(OldId))
			{
				do 
				{
					NewId = FMath::Rand32();
				} 
				while (Result.MaterialSlotIds.Contains(NewId) || AddedMesh.MaterialSlotIds.Contains(NewId));
				
				AddedMeshIdMap.Add(OldId, NewId);
			}
			else
			{
				NewId = OldId;
			}
			
			Result.MaterialSlotIds.Add(NewId);
			Result.MaterialSlotMaterials.Add(AddedMesh.MaterialSlotMaterials[Index]);
			Result.MaterialSlotNames.Add(AddedMesh.MaterialSlotNames[Index]);
		}

		// Merge LODs
		const int32 NumLODs = FMath::Max(Result.LODs.Num(), AddedMesh.LODs.Num());
		Result.LODs.SetNum(NumLODs);

		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			if (AddedMesh.LODs.IsValidIndex(LODIndex) && AddedMesh.LODs[LODIndex])
			{
				if (Result.LODs[LODIndex])
				{
					TManagedPtr<FLOD> MutableLOD = CloneOrTakeOver<FLOD>(MoveTemp(Result.LODs[LODIndex]));
					TManagedPtr<FLOD> AddedLOD = CloneOrTakeOver<FLOD>(MoveTemp(AddedMesh.LODs[LODIndex]));
					
					for (TManagedPtr<const FMesh>& Mesh : AddedLOD->Meshes)
					{
						if (Mesh && Mesh->GetVertexCount())
						{
							bool bChangeSurfaceId = false;
							for (int32 SurfaceIndex = 0; SurfaceIndex < Mesh->Surfaces.Num(); ++SurfaceIndex)
							{
								if (AddedMeshIdMap.Contains(Mesh->Surfaces[SurfaceIndex].Id))
								{
									bChangeSurfaceId = true;
									break;
								}
							}
						
							if (bChangeSurfaceId)
							{
								TManagedPtr<FMesh> ClonedMesh = CloneOrTakeOver<FMesh>(MoveTemp(Mesh));

								for (int32 SurfaceIndex = 0; SurfaceIndex < ClonedMesh->Surfaces.Num(); ++SurfaceIndex)
								{
									if (uint32* NewId = AddedMeshIdMap.Find(ClonedMesh->Surfaces[SurfaceIndex].Id))
									{
										ClonedMesh->Surfaces[SurfaceIndex].Id = *NewId;
										break;
									}
								}
								
								MutableLOD->Meshes.Add(ClonedMesh);
							}
							else
							{
								MutableLOD->Meshes.Add(Mesh);
							}
						}
					}

					Result.LODs[LODIndex] = MutableLOD;
				}
				else
				{
					Result.LODs[LODIndex] = AddedMesh.LODs[LODIndex];
				}
			}
		}
	}	
}
