// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::Mutable::Private
{
	class FSkeletalMesh;

	void SkeletalMeshMerge(FSkeletalMesh& Base, FSkeletalMesh& AddedMesh);
}
