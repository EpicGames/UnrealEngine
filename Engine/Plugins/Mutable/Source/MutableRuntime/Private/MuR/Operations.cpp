// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Operations.h"

#include "MuR/Image.h"
#include "MuR/Model.h"
#include "MuR/External/Operation.h"


namespace UE::Mutable::Private
{
    MUTABLE_IMPLEMENT_POD_SERIALISABLE(FOperation);
    MUTABLE_IMPLEMENT_POD_VECTOR_SERIALISABLE(FOperation);

    void SkeletalMeshOptionsUnpack(uint16 ExecutionOptions, bool& bInitialGeneration, uint8& LODIndex, uint8& FirstLODResident, uint8& FirstLODAvailable, uint8& NumLODs)
    {
    	bInitialGeneration = (ExecutionOptions & 0b0001000000000000) != 0;
    	if (bInitialGeneration)
    	{
    		FirstLODResident = (0b0000000000001111 & ExecutionOptions) >> 0;
    		FirstLODAvailable = (0b0000000011110000 & ExecutionOptions) >> 4;
    		NumLODs = (0b0000111100000000 & ExecutionOptions) >> 8;
    		LODIndex = MAX_uint8;
    	}
    	else
    	{
    		FirstLODResident = MAX_uint8;
    		FirstLODAvailable = MAX_uint8;
    		NumLODs = MAX_uint8;
    		LODIndex = 0b0000000000001111 & ExecutionOptions;
    	}
    }

	
    uint16 SkeletalMeshOptionsPack(bool bInitialGeneration, uint8 LODIndex, uint8 FirstLODResident, uint8 FirstLODAvailable, uint8 NumLODs)
    {
    	if (bInitialGeneration)
    	{
    		check(FirstLODResident < 16); // Not enough bits
    		check(FirstLODAvailable < 16); // Not enough bits
    		check(NumLODs < 16); // Not enough bits
    		
    		return 0b0001000000000000 | 
    			(0b0000000000001111 & static_cast<uint16>(FirstLODResident)) << 0 | 
    			(0b0000000000001111 & static_cast<uint16>(FirstLODAvailable)) << 4 | 
    			(0b0000000000001111 & static_cast<uint16>(NumLODs)) << 8;
    	}
    	else
    	{
    		check(LODIndex < 16); // Not enough bits
    		
    		return 0b0001000000001111 & LODIndex;
    	}
    }

	
    void SkeletalMeshObjectOptionsUnpack(uint16 ExecutionOptions, bool& bInitialGeneration, bool& bStreamMeshLODs, uint8& LODIndex)
    {
    	bInitialGeneration = (ExecutionOptions & 0b0010000000000000) != 0;
    	bStreamMeshLODs = (ExecutionOptions & 0b0001000000000000) != 0;
    	LODIndex = 0b0000000011111111 & ExecutionOptions;
    }

	
	uint16 SkeletalMeshObjectOptionsPack(bool bInitialGeneration, bool bStreamMeshLODs, uint8 LODIndex)
    {
    	return (bInitialGeneration ? 0x1 : 0x0) << 13 | (bStreamMeshLODs ? 0x1 : 0x0) << 12 | static_cast<uint16>(LODIndex);
    }


    void ForEachReference( const FProgram& program, FOperation::ADDRESS at, const TFunctionRef<void(FOperation::ADDRESS)> f )
    {
        EOpType type = program.GetOpType(at);
        switch ( type )
        {
        case EOpType::NONE:
        case EOpType::BO_CONSTANT:
        case EOpType::NU_CONSTANT:
        case EOpType::SC_CONSTANT:
        case EOpType::ST_CONSTANT:
        case EOpType::CO_CONSTANT:
        case EOpType::IM_CONSTANT:
        case EOpType::ME_CONSTANT:
        case EOpType::LA_CONSTANT:
        case EOpType::PR_CONSTANT:
		case EOpType::ED_CONSTANT:
		case EOpType::MA_CONSTANT:
		case EOpType::MI_CONSTANT:
        case EOpType::BO_PARAMETER:
        case EOpType::NU_PARAMETER:
        case EOpType::SC_PARAMETER:
        case EOpType::CO_PARAMETER:
        case EOpType::PR_PARAMETER:
		case EOpType::IM_PARAMETER:
		case EOpType::SK_PARAMETER:
		case EOpType::MA_PARAMETER:
		case EOpType::MI_PARAMETER:
		case EOpType::IS_PARAMETER:
		case EOpType::IM_REFERENCE:
		case EOpType::ME_REFERENCE:
		case EOpType::IM_PARAMETER_FROM_MATERIAL:
			break;

        case EOpType::SC_CURVE:
        {
			FOperation::ScalarCurveArgs args = program.GetOpArgs<FOperation::ScalarCurveArgs>(at);
            f(args.time );
            break;
        }

        case EOpType::NU_CONDITIONAL:
        case EOpType::SC_CONDITIONAL:
        case EOpType::CO_CONDITIONAL:
        case EOpType::IM_CONDITIONAL:
        case EOpType::SK_CONDITIONAL:
        case EOpType::ME_CONDITIONAL:
        case EOpType::LA_CONDITIONAL:
        case EOpType::IN_CONDITIONAL:
		case EOpType::ED_CONDITIONAL:
		case EOpType::MI_CONDITIONAL:
        {
			FOperation::ConditionalArgs args = program.GetOpArgs<FOperation::ConditionalArgs>(at);
            f(args.condition );
            f(args.yes );
            f(args.no );
            break;
        }

        case EOpType::NU_SWITCH:
        case EOpType::SC_SWITCH:
        case EOpType::CO_SWITCH:
        case EOpType::IM_SWITCH:
        case EOpType::ME_SWITCH:
        case EOpType::LA_SWITCH:
        case EOpType::IN_SWITCH:
		case EOpType::ED_SWITCH:
		case EOpType::MI_SWITCH:
		case EOpType::IS_SWITCH:
		case EOpType::SK_SWITCH:
		case EOpType::LD_SWITCH:
        {
			const uint8* Data = program.GetOpArgsPointer(at);
			
			FOperation::ADDRESS VarAddress;
			FMemory::Memcpy(&VarAddress, Data, sizeof(FOperation::ADDRESS));
			Data += sizeof(FOperation::ADDRESS);

			FOperation::ADDRESS DefAddress;
			FMemory::Memcpy(&DefAddress, Data, sizeof(FOperation::ADDRESS));
			Data += sizeof(FOperation::ADDRESS);

			FOperation::FSwitchCaseDescriptor CaseDesc;
			FMemory::Memcpy(&CaseDesc, Data, sizeof(FOperation::FSwitchCaseDescriptor));
			Data += sizeof(FOperation::FSwitchCaseDescriptor);

			f(VarAddress);
			f(DefAddress);

			if (!CaseDesc.bUseRanges)
			{
				for (uint32 C = 0; C < CaseDesc.Count; ++C)
				{
					//int32 Condition;
					//FMemory::Memcpy( &Condition, data, sizeof(int32));
					Data += sizeof(int32);

					FOperation::ADDRESS CaseAt;
					FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					f(CaseAt);
				}
			}
			else
			{
				for (uint32 C = 0; C < CaseDesc.Count; ++C)
				{
					//int32 ConditionStart;
					//FMemory::Memcpy( &ConditionStart, data, sizeof(int32));
					Data += sizeof(int32);

					//uint32 RangeSize;
					//FMemory::Memcpy( &RangeSize, data, sizeof(uint32));
					Data += sizeof(int32);

					FOperation::ADDRESS CaseAt;
					FMemory::Memcpy(&CaseAt, Data, sizeof(FOperation::ADDRESS));
					Data += sizeof(FOperation::ADDRESS);

					f(CaseAt);
				}
			}

            break;
        }

		case EOpType::SC_MATERIAL_BREAK:
		case EOpType::CO_MATERIAL_BREAK:
		case EOpType::IM_MATERIAL_BREAK:
		{
			FOperation::MaterialBreakArgs Args = program.GetOpArgs<FOperation::MaterialBreakArgs>(at);
			f(Args.Material);
			break;
		}
		
		case EOpType::MI_MODIFY:
		{
			FOperation::MaterialModifyArgs Args = program.GetOpArgs<FOperation::MaterialModifyArgs>(at);
			int32 ParameterCount = Args.NumParameters;

			//Dynamic data can't be stored into arg structures
			const uint8* Data = program.GetOpArgsPointer(at);
			Data += sizeof(FOperation::MaterialModifyArgs);

			// First, the material
			f(Args.Material);

			// Store parameter names and schedule the parameter operations
			for (int32 ParameterIndex = 0; ParameterIndex < ParameterCount; ++ParameterIndex)
			{
				FOperation::ADDRESS NewName;
				FMemory::Memcpy(&NewName, Data, sizeof(FOperation::ADDRESS));
				Data += sizeof(FOperation::ADDRESS);

				int8 LayerIndex;
				FMemory::Memcpy(&LayerIndex, Data, sizeof(int8));
				Data += sizeof(int8);

				FOperation::ADDRESS NewParameterOperation;
				FMemory::Memcpy(&NewParameterOperation, Data, sizeof(FOperation::ADDRESS));
				f(NewParameterOperation);
				Data += sizeof(FOperation::ADDRESS);

				// Evaluate non lazy branches here
				if (GetOpDataType(program.GetOpType(NewParameterOperation)) == EDataType::Image)
				{
					// Jump Image Parameter Index
					Data += sizeof(int32);
				}
			}

			break;
		}

        //-------------------------------------------------------------------------------------
        case EOpType::BO_EQUAL_INT_CONST:
        {
			FOperation::BoolEqualScalarConstArgs args = program.GetOpArgs<FOperation::BoolEqualScalarConstArgs>(at);
            f(args.Value );
            break;
        }

        case EOpType::BO_AND:
        case EOpType::BO_OR:
        {
			FOperation::BoolBinaryArgs args = program.GetOpArgs<FOperation::BoolBinaryArgs>(at);
            f(args.A );
            f(args.B );
            break;
        }

        case EOpType::BO_NOT:
        {
			FOperation::BoolNotArgs args = program.GetOpArgs<FOperation::BoolNotArgs>(at);
            f(args.A );
            break;
        }

        case EOpType::SC_ARITHMETIC:
        {
			FOperation::ArithmeticArgs args = program.GetOpArgs<FOperation::ArithmeticArgs>(at);
            f(args.A );
            f(args.B );
            break;
        }

        case EOpType::CO_SAMPLEIMAGE:
        {
			FOperation::ColorSampleImageArgs args = program.GetOpArgs<FOperation::ColorSampleImageArgs>(at);
            f(args.Image );
            f(args.X );
            f(args.Y );
            break;
        }

        case EOpType::CO_SWIZZLE:
        {
			FOperation::ColorSwizzleArgs args = program.GetOpArgs<FOperation::ColorSwizzleArgs>(at);
            for (int t=0;t<MUTABLE_OP_MAX_SWIZZLE_CHANNELS;++t)
            {
                f(args.sources[t] );
            }
            break;
        }

        case EOpType::CO_FROMSCALARS:
        {
			FOperation::ColorFromScalarsArgs args = program.GetOpArgs<FOperation::ColorFromScalarsArgs>(at);
			for (int32 t = 0; t < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++t)
			{
				f(args.V[t]);
			}
            break;
        }

        case EOpType::CO_ARITHMETIC:
        {
			FOperation::ArithmeticArgs args = program.GetOpArgs<FOperation::ArithmeticArgs>(at);
            f(args.A);
            f(args.B);
            break;
        }
		
		case EOpType::CO_LINEARTOSRGB:
        {
			FOperation::ColorArgs Args = program.GetOpArgs<FOperation::ColorArgs>(at);
            f(Args.Color);
            break;
        }

        //-------------------------------------------------------------------------------------
        case EOpType::IM_LAYER:
        {
			FOperation::ImageLayerArgs args = program.GetOpArgs<FOperation::ImageLayerArgs>(at);
            f(args.base );
        	f(args.mask );
            f(args.blended );
            break;
        }

        case EOpType::IM_LAYERCOLOR:
        {
			FOperation::ImageLayerColorArgs args = program.GetOpArgs<FOperation::ImageLayerColorArgs>(at);
            f(args.base );
        	f(args.mask );
            f(args.color );
            break;
        }

        case EOpType::IM_MULTILAYER:
        {
			FOperation::ImageMultiLayerArgs args = program.GetOpArgs<FOperation::ImageMultiLayerArgs>(at);
            f(args.rangeSize );
            f(args.base );
            f(args.mask );
            f(args.blended );
            break;
        }

		case EOpType::IM_NORMALCOMPOSITE:
		{
			FOperation::ImageNormalCompositeArgs args = program.GetOpArgs<FOperation::ImageNormalCompositeArgs>(at);
			f(args.base);
			f(args.normal);

			break;
		}

        case EOpType::IM_PIXELFORMAT:
        {
			FOperation::ImagePixelFormatArgs args = program.GetOpArgs<FOperation::ImagePixelFormatArgs>(at);
            f(args.source );
            break;
        }

        case EOpType::IM_CONFORM:
        {
			FOperation::ImageConformArgs Args = program.GetOpArgs<FOperation::ImageConformArgs>(at);
            f(Args.Source);
            break;
        }

        case EOpType::IM_MIPMAP:
        {
			FOperation::ImageMipmapArgs args = program.GetOpArgs<FOperation::ImageMipmapArgs>(at);
            f(args.source );
            break;
        }

        case EOpType::IM_RESIZE:
        {
			FOperation::ImageResizeArgs Args = program.GetOpArgs<FOperation::ImageResizeArgs>(at);
            f(Args.Source );
            break;
        }

        case EOpType::IM_RESIZELIKE:
        {
			FOperation::ImageResizeLikeArgs Args = program.GetOpArgs<FOperation::ImageResizeLikeArgs>(at);
            f(Args.Source );
            f(Args.SizeSource );
            break;
        }

        case EOpType::IM_RESIZEREL:
        {
			FOperation::ImageResizeRelArgs Args = program.GetOpArgs<FOperation::ImageResizeRelArgs>(at);
            f(Args.Source );
            break;
        }

        case EOpType::IM_BLANKLAYOUT:
        {
			FOperation::ImageBlankLayoutArgs Args = program.GetOpArgs<FOperation::ImageBlankLayoutArgs>(at);
            f(Args.Layout );
            break;
        }

        case EOpType::IM_COMPOSE:
        {
			FOperation::ImageComposeArgs args = program.GetOpArgs<FOperation::ImageComposeArgs>(at);
            f(args.layout );
            f(args.base );
            f(args.blockImage );
            f(args.mask );
            break;
        }

		case EOpType::IM_MULTICOMPOSE:
		{
			FOperation::ImageMultiComposeArgs args = program.GetOpArgs<FOperation::ImageMultiComposeArgs>(at);
			f(args.Layout);
			f(args.Base);
			f(args.SourceLayout);
			f(args.SourceImage);
			break;
		}

        case EOpType::IM_INTERPOLATE:
        {
			FOperation::ImageInterpolateArgs Args = program.GetOpArgs<FOperation::ImageInterpolateArgs>(at);
            f(Args.Factor );

            for (int32 TargetIndex=0; TargetIndex <MUTABLE_OP_MAX_INTERPOLATE_COUNT;++TargetIndex)
            {
                f(Args.Targets[TargetIndex]);
            }
            break;
        }

        case EOpType::IM_SWIZZLE:
        {
			FOperation::ImageSwizzleArgs args = program.GetOpArgs<FOperation::ImageSwizzleArgs>(at);
            for (int t=0;t<MUTABLE_OP_MAX_SWIZZLE_CHANNELS;++t)
            {
                f(args.sources[t] );
            }
            break;
        }

        case EOpType::IM_SATURATE:
        {
			FOperation::ImageSaturateArgs Args = program.GetOpArgs<FOperation::ImageSaturateArgs>(at);
            f(Args.Base );
            f(Args.Factor );
            break;
        }

        case EOpType::IM_LUMINANCE:
        {
			FOperation::ImageLuminanceArgs Args = program.GetOpArgs<FOperation::ImageLuminanceArgs>(at);
            f(Args.Base );
            break;
        }

        case EOpType::IM_COLORMAP:
        {
			FOperation::ImageColorMapArgs Args = program.GetOpArgs<FOperation::ImageColorMapArgs>(at);
            f(Args.Base );
            f(Args.Mask );
            f(Args.Map );
            break;
        }

        case EOpType::IM_BINARISE:
        {
			FOperation::ImageBinariseArgs Args = program.GetOpArgs<FOperation::ImageBinariseArgs>(at);
            f(Args.Base );
            f(Args.Threshold );
            break;
        }

        case EOpType::IM_PLAINCOLOR:
        {
			FOperation::ImagePlainColorArgs args = program.GetOpArgs<FOperation::ImagePlainColorArgs>(at);
            f(args.Color );
            break;
        }

        case EOpType::IM_CROP:
        {
			FOperation::ImageCropArgs args = program.GetOpArgs<FOperation::ImageCropArgs>(at);
            f(args.source );
            break;
        }

        case EOpType::IM_PATCH:
        {
			FOperation::ImagePatchArgs args = program.GetOpArgs<FOperation::ImagePatchArgs>(at);
            f(args.base );
            f(args.patch );
            break;
        }

        case EOpType::IM_RASTERMESH:
        {
			FOperation::ImageRasterMeshArgs args = program.GetOpArgs<FOperation::ImageRasterMeshArgs>(at);
            f(args.mesh );
            f(args.image );
            f(args.mask );
            f(args.angleFadeProperties );
            f(args.projector );
            break;
        }

        case EOpType::IM_MAKEGROWMAP:
        {
			FOperation::ImageMakeGrowMapArgs args = program.GetOpArgs<FOperation::ImageMakeGrowMapArgs>(at);
            f(args.mask );
            break;
        }

        case EOpType::IM_DISPLACE:
        {
			FOperation::ImageDisplaceArgs Args = program.GetOpArgs<FOperation::ImageDisplaceArgs>(at);
            f(Args.Source );
            f(Args.DisplacementMap );
            break;
        }

		case EOpType::IM_INVERT:
		{
			FOperation::ImageInvertArgs args = program.GetOpArgs<FOperation::ImageInvertArgs>(at);
			f(args.Base);
			break;
		}

		case EOpType::IM_TRANSFORM:
		{
			FOperation::ImageTransformArgs Args = program.GetOpArgs<FOperation::ImageTransformArgs>(at);
			f(Args.Base);
			f(Args.OffsetX);
			f(Args.OffsetY);
			f(Args.ScaleX);
			f(Args.ScaleY);
			f(Args.Rotation);
			break;
		}

        case EOpType::IM_PARAMETER_CONVERT:
        {
        	FOperation::ImageParameterConvertArgs Args = program.GetOpArgs<FOperation::ImageParameterConvertArgs>(at);
        	f(Args.ImageParameter);
        	break;
        }

        //-------------------------------------------------------------------------------------
        case EOpType::ME_APPLYLAYOUT:
        {
			FOperation::MeshApplyLayoutArgs args = program.GetOpArgs<FOperation::MeshApplyLayoutArgs>(at);
            f(args.Layout );
            f(args.Mesh );
            break;
        }

		case EOpType::ME_PREPARELAYOUT:
		{
			FOperation::MeshPrepareLayoutArgs Args = program.GetOpArgs<FOperation::MeshPrepareLayoutArgs>(at);
			f(Args.Layout);
			f(Args.Mesh);
			break;
		}

        case EOpType::ME_DIFFERENCE:
        {
			const uint8_t* data = program.GetOpArgsPointer(at);

			FOperation::ADDRESS BaseAt = 0;
			FMemory::Memcpy(&BaseAt, data, sizeof(FOperation::ADDRESS)); data += sizeof(FOperation::ADDRESS);
			f(BaseAt);

			FOperation::ADDRESS TargetAt = 0;
			FMemory::Memcpy(&TargetAt, data, sizeof(FOperation::ADDRESS)); data += sizeof(FOperation::ADDRESS);
			f(TargetAt);
			break;
        }

        case EOpType::ME_MORPH:
        {
			FOperation::MeshMorphArgs Args = program.GetOpArgs<FOperation::MeshMorphArgs>(at);

			f(Args.Factor);
			f(Args.Base);

			break;
        }

        case EOpType::ME_MERGE:
        {
			FOperation::MeshMergeArgs args = program.GetOpArgs<FOperation::MeshMergeArgs>(at);
            f(args.Base );
            f(args.Added );
            break;
        }

		case EOpType::ME_MASKCLIPMESH:
		{
			FOperation::MeshMaskClipMeshArgs args = program.GetOpArgs<FOperation::MeshMaskClipMeshArgs>(at);
			f(args.source);
			f(args.clip);
			break;
		}

		case EOpType::ME_MASKCLIPUVMASK:
		{
			FOperation::MeshMaskClipUVMaskArgs Args = program.GetOpArgs<FOperation::MeshMaskClipUVMaskArgs>(at);
			f(Args.Source);
			f(Args.UVSource);
			f(Args.MaskImage);
			f(Args.MaskLayout);
			break;
		}

        case EOpType::ME_MASKDIFF:
        {
			FOperation::MeshMaskDiffArgs args = program.GetOpArgs<FOperation::MeshMaskDiffArgs>(at);
            f(args.Source );
            f(args.Fragment );
            break;
        }

        case EOpType::ME_REMOVEMASK:
        {
            const uint8* data = program.GetOpArgsPointer(at);
            UE::Mutable::Private::FOperation::ADDRESS source;
            FMemory::Memcpy( &source, data, sizeof(FOperation::ADDRESS) ); data+=sizeof(FOperation::ADDRESS);
            f(source);

			EFaceCullStrategy FaceCullStrategy;
			FMemory::Memcpy(&FaceCullStrategy, data, sizeof(EFaceCullStrategy));
			data += sizeof(EFaceCullStrategy);

            uint16 removes = 0;
			FMemory::Memcpy( &removes, data, sizeof(uint16) ); data+=sizeof(uint16);
            for (uint16 r=0; r<removes; ++r)
            {
                UE::Mutable::Private::FOperation::ADDRESS condition;
				FMemory::Memcpy( &condition, data, sizeof(FOperation::ADDRESS) ); data+=sizeof(FOperation::ADDRESS);
                f(condition);

                UE::Mutable::Private::FOperation::ADDRESS mask;
				FMemory::Memcpy( &mask, data, sizeof(FOperation::ADDRESS) ); data+=sizeof(FOperation::ADDRESS);
                f(mask);
            }
            break;
        }

		case EOpType::ME_ADDMETADATA:
		{
			FOperation::MeshAddMetadataArgs Args = program.GetOpArgs<FOperation::MeshAddMetadataArgs>(at);
			f(Args.Source);
			break;
		}

		case EOpType::ME_SETMATERIALSLOTID:
		{
			FOperation::MeshSetMaterialSlotIdArgs Args = program.GetOpArgs<FOperation::MeshSetMaterialSlotIdArgs>(at);
			f(Args.Mesh);
			break;
		}

        case EOpType::ME_FORMAT:
        {
			FOperation::MeshFormatArgs args = program.GetOpArgs<FOperation::MeshFormatArgs>(at);
            f(args.source );
            f(args.format );
            break;
        }

        case EOpType::ME_TRANSFORM:
        {
			FOperation::MeshTransformArgs args = program.GetOpArgs<FOperation::MeshTransformArgs>(at);
            f(args.source );
            break;
        }

        case EOpType::ME_EXTRACTLAYOUTBLOCK:
        {
            const uint8_t* data = program.GetOpArgsPointer(at);
            UE::Mutable::Private::FOperation::ADDRESS source;
			FMemory::Memcpy( &source, data, sizeof(FOperation::ADDRESS) );
            f(source);
            break;
        }

        case EOpType::ME_CLIPMORPHPLANE:
        {
			FOperation::MeshClipMorphPlaneArgs args = program.GetOpArgs<FOperation::MeshClipMorphPlaneArgs>(at);
            f(args.Source);
            break;
        }

        case EOpType::ME_CLIPWITHMESH :
        {
			FOperation::MeshClipWithMeshArgs Args = program.GetOpArgs<FOperation::MeshClipWithMeshArgs>(at);
            f(Args.Source);
            f(Args.ClipMesh);
            break;
        }

        case EOpType::ME_CLIPDEFORM:
        {
			FOperation::MeshClipDeformArgs args = program.GetOpArgs<FOperation::MeshClipDeformArgs>(at);
            f(args.mesh);
            f(args.clipShape);
            break;
        }
       
        case EOpType::ME_PROJECT :
        {
			FOperation::MeshProjectArgs Args = program.GetOpArgs<FOperation::MeshProjectArgs>(at);
            f(Args.Mesh);
            f(Args.Projector);
            break;
        }

		case EOpType::ME_APPLYPOSE:
		{
			FOperation::MeshApplyPoseArgs Args = program.GetOpArgs<FOperation::MeshApplyPoseArgs>(at);
			f(Args.base);
			f(Args.pose);
			break;
		}

		case EOpType::ME_BINDSHAPE:
		{
			FOperation::MeshBindShapeArgs args = program.GetOpArgs<FOperation::MeshBindShapeArgs>(at);
			f(args.mesh);
			f(args.shape);
			break;
		}

		case EOpType::ME_APPLYSHAPE:
		{
			FOperation::MeshApplyShapeArgs args = program.GetOpArgs<FOperation::MeshApplyShapeArgs>(at);
			f(args.mesh);
			f(args.shape);
			break;
		}

        case EOpType::ME_TRANSFORMWITHMESH :
        {
        	FOperation::MeshTransformWithinMeshArgs args = program.GetOpArgs<FOperation::MeshTransformWithinMeshArgs>(at);
        	f(args.sourceMesh);
        	f(args.boundingMesh);
        	f(args.matrix);
        	break;
        }

		case EOpType::ME_TRANSFORMWITHBONE:
		{
			FOperation::MeshTransformWithBoneArgs args = program.GetOpArgs<FOperation::MeshTransformWithBoneArgs>(at);
			f(args.SourceMesh);
			f(args.Matrix);
			break;
		}

        case EOpType::ME_SKELETALMESH_BREAK:
        {
        	FOperation::MeshSkeletalMeshBreakArgs args = program.GetOpArgs<FOperation::MeshSkeletalMeshBreakArgs>(at);
        	f(args.SkeletalMeshParameter);
        	break;
        }
        
        //-------------------------------------------------------------------------------------
        case EOpType::LD_NEW:
        {
			FOperation::FLODNewArgs Args = program.GetOpArgs<FOperation::FLODNewArgs>(at);

        	const TArray<FOperation::ADDRESS>& MeshAddresses = program.ConstantUInt32Lists[Args.Meshes];
        		
        	for (FOperation::ADDRESS MeshAddress : MeshAddresses)
        	{
        		f(MeshAddress);
            }
        		
            break;
        }

        case EOpType::IN_ADDCOMPONENT:
        {
        	FOperation::InstanceAddComponentArgs args = program.GetOpArgs<FOperation::InstanceAddComponentArgs>(at);
        	f(args.Instance );
        	f(args.Value );
        	break;
        }
        	
        case EOpType::SK_NEW:
        {
        	FOperation::FSkeletalMeshNewArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshNewArgs>(at);
			
        	const TArray<FOperation::ADDRESS>& MaterialAddresses = program.ConstantUInt32Lists[Args.MaterialSlotMaterials];
        	const TArray<FOperation::ADDRESS>& LODsAddresses = program.ConstantUInt32Lists[Args.LODs];
				
        	for (int32 MaterialIndex = 0; MaterialIndex < MaterialAddresses.Num(); ++MaterialIndex)
        	{
        		const FOperation::ADDRESS MaterialAddress = MaterialAddresses[MaterialIndex];
        	
        		f(MaterialAddress); 
        	}
					
        	for (uint8 LODIndex = 0; LODIndex < LODsAddresses.Num(); ++LODIndex)
        	{
        		const FOperation::ADDRESS LODAddress = LODsAddresses[LODIndex];
							
        		f(LODAddress); 
        	}

        	break;
        }
		
		case EOpType::SK_CONVERT:
		{
			FOperation::FSkeletalMeshConvertArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshConvertArgs>(at);
			f(Args.SkeletalMeshObject);
			break;
		}

		case EOpType::SK_MORPH:
		{
			FOperation::FSkeletalMeshMorphArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshMorphArgs>(at);
			
			f(Args.Base);
			f(Args.Factor);

			break;
		}

		case EOpType::SK_RESHAPE:
		{
			FOperation::FSkeletalMeshReshapeArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshReshapeArgs>(at);
			
			f(Args.Base);
			f(Args.BaseShape);
			f(Args.TargetShape);

			break;
		}

		case EOpType::SK_MATERIALMODIFY:
		{
			FOperation::FSkeletalMeshMaterialModifyArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshMaterialModifyArgs>(at);
			
			f(Args.SkeletalMesh);
			f(Args.NewMaterial);

			break;
		}

		case EOpType::SK_CLIPMESHWITHMESH:
		{
			FOperation::FSkeletalMeshClipMeshWithMeshArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshClipMeshWithMeshArgs>(at);
			
			f(Args.Source);
			f(Args.Clip);

			break;
		}

		case EOpType::SK_MERGE:
		{
			FOperation::FSkeletalMeshMergeArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshMergeArgs>(at);
			f(Args.BaseMesh);
			f(Args.AddedMesh);

			break;
		}
        	
        case EOpType::SK_TRANSFORMWITHBONE:
        {
        	FOperation::FSkeletalMeshTransformWithBoneArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshTransformWithBoneArgs>(at);
        	f(Args.SourceSkeletalMesh);
        	f(Args.Matrix);

        	break;
        }

        case EOpType::SKO_CONVERT:
        {
        	FOperation::FSkeletalMeshObjectConvertArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshObjectConvertArgs>(at);

        	f(Args.SkeletalMesh); 
        		
        	break;
        }

		case EOpType::IN_ADDEXTENSIONDATA:
		{
			const FOperation::InstanceAddExtensionDataArgs Args = program.GetOpArgs<FOperation::InstanceAddExtensionDataArgs>(at);

			f(Args.Instance);
			f(Args.ExtensionData);

			break;
		}

		case EOpType::IN_ADDOVERRIDEMATERIAL:
		{
        		const FOperation::InstanceAddOverrideMaterialArgs Args = program.GetOpArgs<FOperation::InstanceAddOverrideMaterialArgs>(at);

        		f(Args.Instance);
        		f(Args.Material);

        		break;
		}

		case EOpType::IN_ADDOVERLAYMATERIAL:
        {
        	const FOperation::InstanceAddOverlayMaterialArgs Args = program.GetOpArgs<FOperation::InstanceAddOverlayMaterialArgs>(at);

        	f(Args.Instance);
        	f(Args.Material);
        		
        	break;
        }
        
        case EOpType::IN_ADDSKELETALMESH:
        {
        	const FOperation::InstanceAddArgs Args = program.GetOpArgs<FOperation::InstanceAddArgs>(at);

        	f(Args.instance);
        	f(Args.value);

        	break;
        }

        //-------------------------------------------------------------------------------------
        case EOpType::LA_PACK:
        {
			FOperation::LayoutPackArgs args = program.GetOpArgs<FOperation::LayoutPackArgs>(at);
            f(args.Source );
            break;
        }

        case EOpType::LA_MERGE:
        {
			FOperation::LayoutMergeArgs args = program.GetOpArgs<FOperation::LayoutMergeArgs>(at);
            f(args.Base );
            f(args.Added );
            break;
        }

		case EOpType::LA_REMOVEBLOCKS:
		{
			FOperation::LayoutRemoveBlocksArgs args = program.GetOpArgs<FOperation::LayoutRemoveBlocksArgs>(at);
			f(args.Source);
			f(args.ReferenceLayout);
			break;
		}

		case EOpType::LA_FROMMESH:
		{
			FOperation::LayoutFromMeshArgs args = program.GetOpArgs<FOperation::LayoutFromMeshArgs>(at);
			f(args.Mesh);
			break;
		}
        
        case EOpType::SC_EXTERNAL:
        case EOpType::CO_EXTERNAL:
        case EOpType::IM_EXTERNAL:
        case EOpType::ME_EXTERNAL:
        case EOpType::MI_EXTERNAL:
        case EOpType::IS_EXTERNAL:
        {
        	const uint8* Data = program.GetOpArgsPointer(at);

        	FOperation::ExternalArgs Args = program.GetOpArgs<FOperation::ExternalArgs>(at);
        	Data += sizeof(FOperation::ExternalArgs);
        		
        	for (int32 Index = 0; Index < Args.NumOperants; ++Index)
        	{
        		FOperation::ADDRESS OperantAt;
        		FMemory::Memcpy(&OperantAt, Data, sizeof(FOperation::ADDRESS));
        		Data += sizeof(FOperation::ADDRESS);

        		f(OperantAt);
        	}

       		break;
        }

        case EOpType::MI_SKELETALMESHOBJECT_BREAK:
        {
        	const FOperation::MaterialSkeletalMeshObjectBreakArgs Args = program.GetOpArgs<FOperation::MaterialSkeletalMeshObjectBreakArgs>(at);
        	f(Args.SkeletalMeshObject);
        	break;
        }
        	
        case EOpType::MI_SKELETALMESH_BREAK:
        {
        	const FOperation::MaterialSkeletalMeshBreakArgs Args = program.GetOpArgs<FOperation::MaterialSkeletalMeshBreakArgs>(at);
        	f(Args.SkeletalMesh);
        	break;
        }
        	
        case EOpType::SK_TRANSFORM:
        {
        	const FOperation::FSkeletalMeshTransformArgs Args = program.GetOpArgs<FOperation::FSkeletalMeshTransformArgs>(at);
        	f(Args.Source);
        	break;
        }
        	
        default:
			check( false );
            break;
        }

    }


}

