// Copyright Epic Games, Inc. All Rights Reserved.

#include <MuR/ParallelExecutionUtils.h>
#include <Async/ParallelFor.h>

namespace UE::Mutable::Private::ParallelExecutionUtils
{
	void InvokeBatchParallelFor(int32 Num, TFunctionRef<void(int32)> Body)
	{
		if (Num == 1)
		{
			Body(0);
		}
		else if (Num > 1)
		{
			ParallelFor(Num, Body);
		}
	}

	void InvokeBatchParallelForUnbalanced(int32 Num, TFunctionRef<void(int32)> Body)
	{
		if (Num == 1)
		{
			Body(0);
		}
		else if (Num > 1)
		{
			ParallelFor(Num, Body, EParallelForFlags::Unbalanced);
		}
	}

	void InvokeBatchParallelForSingleThread(int32 Num, TFunctionRef<void(int32)> Body)
	{
		for (int32 BatchId = 0; BatchId < Num; ++BatchId)
		{
			Body(BatchId);
		}
	}
}
