// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Parameters.h"

#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/Material.h"
#include "MuR/Parameters.h"
#include "MuR/Serialisation.h"
#include "MuR/Serialisation.h"
#include "MuR/System.h"


namespace UE::Mutable::Private
{
    MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EParameterType)              
    MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(EProjectorType)
	

	void FParameters::Serialise( const FParameters* p, FOutputArchive& arch )
	{
		arch << *p;
    }


	TSharedPtr<FParameters> FParameters::StaticUnserialise( FInputArchive& arch )
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		TSharedPtr<FParameters> pResult = MakeShared<FParameters>();
		arch >> *pResult;
		return pResult;
	}

	
	void FParameters::Serialise(FOutputArchive& Arch) const
	{
    	Arch << Values;
    	Arch << MultiValues;
	}

	
	void FParameters::Unserialise(FInputArchive& Arch)
	{
		Arch >> Values;
		Arch >> MultiValues;
	}


	TSharedPtr<FParameters> FParameters::Clone() const
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		TSharedPtr<FParameters> pRes = MakeShared<FParameters>();

		pRes->Model = Model;
		pRes->Values = Values;
		pRes->MultiValues = MultiValues;

		return pRes;
	}


	int32 FParameters::GetCount() const
	{
		return (int32)Values.Num();
	}


	const FString& FParameters::GetName( int32 Index ) const
	{
		const FProgram& Program = Model->GetProgram();
		check( Index>=0 && Index<(int32)Program.Parameters.Num() );

		return Program.Parameters[Index].Name;
	}


	const FGuid& FParameters::GetUid( int32 Index ) const
	{
		const FProgram& Program = Model->GetProgram();
		check( Index>=0 && Index<Program.Parameters.Num() );

		return Program.Parameters[Index].UID;
	}


	int32 FParameters::Find( const FString& Name ) const
	{
    	const FProgram& Program = Model->GetProgram();

    	int32 result = -1;

    	for( int32 i=0; result<0 && i<(int32)Program.Parameters.Num(); ++i )
    	{
    		const FParameterDesc& p = Program.Parameters[i];

    		if (p.Name == Name)
    		{
    			result = i;
    		}
    	}

    	return result;
	}


	EParameterType FParameters::GetType( int32 Index ) const
	{
		const FProgram& Program = Model->GetProgram();
		check( Index>=0 && Index<Program.Parameters.Num() );

		return Program.Parameters[Index].Type;
	}


    TSharedPtr<FRangeIndex> FParameters::NewRangeIndex( int32 ParamIndex ) const
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

        const FProgram& Program = Model->GetProgram();
        check( ParamIndex>=0 && ParamIndex<Program.Parameters.Num() );
		TSharedPtr<FRangeIndex> Range;

        if ( ParamIndex>=0 && ParamIndex<int32(Program.Parameters.Num()) )
        {
            int32 RangeCount = Program.Parameters[ParamIndex].Ranges.Num();
            if (RangeCount >0 )
            {
				Range = MakeShared<FRangeIndex>();
				Range->Parameters = this->AsShared();
				Range->Parameter = ParamIndex;
				Range->Values.SetNumZeroed(RangeCount);
            }
        }

        return Range;
    }


    int32 FParameters::GetValueCount( int32 ParamIndex ) const
    {
        if (ParamIndex<0 || ParamIndex >= int32(MultiValues.Num()) )
        {
            return 0;
        }

        return int32( MultiValues[ParamIndex].Num() );
    }


    TSharedPtr<FRangeIndex> FParameters::GetValueIndex( int32 ParamIndex, int32 ValueIndex ) const
    {
        if (ParamIndex<0 || ParamIndex >= MultiValues.Num() )
        {
            return nullptr;
        }

        if (ValueIndex <0 || ValueIndex >= int32(MultiValues[ParamIndex].Num()) )
        {
            return nullptr;
        }

		TMap<TArray<int32>, FParameterValue>::TConstIterator it = MultiValues[ParamIndex].CreateConstIterator();
        for ( int32 i=0; i< ValueIndex; ++i )
        {
            ++it;
        }

		TSharedPtr<FRangeIndex> result = NewRangeIndex( ParamIndex );
        result->Values = it->Key;

        return result;
    }


    void FParameters::ClearAllValues( int32 ParamIndex )
    {
        if (ParamIndex<0 || ParamIndex >= int32(MultiValues.Num()) )
        {
            return;
        }

        MultiValues[ParamIndex].Empty();
    }


    bool FParameters::GetBoolValue( int32 Index, const FRangeIndex* Pos ) const
	{
		check( Index>=0 && Index<(int32)Values.Num() );
        check( GetType(Index)==EParameterType::Bool );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Bool )
        {
            return false;
        }

        // Single value case
        if (!Pos)
        {
            // Return the single value
            return Values[Index].Get<FParamBoolType>();
        }

        // Multivalue case
        check( Pos->Parameter==Index );

        if ( Index<int32(MultiValues.Num()))
        {
            const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(Pos->Values);
            if (it)
            {
                return it->Get<FParamBoolType>();
            }
        }

        // Multivalue parameter, but no multivalue set. Return single value.
        return Values[Index].Get<FParamBoolType>();
	}


    void FParameters::SetBoolValue( int32 Index, bool value,
                                   const FRangeIndex* Pos )
	{
		check( Index>=0 && Index<(int32)Values.Num() );
		check( GetType(Index)== EParameterType::Bool );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Bool )
        {
            return;
        }

        // Single value case
        if (!Pos)
        {
            // Clear multivalue, if set.
            if (Index<int32(MultiValues.Num()))
            {
                MultiValues[Index].Empty();
            }

            Values[Index].Set<FParamBoolType>(value);
        }

        // Multivalue case
        else
        {
            check( Pos->Parameter==Index );

            if ( Index>=int32(MultiValues.Num()))
            {
                MultiValues.SetNum(Index+1);
            }

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(Pos->Values);
            it.Set<FParamBoolType>(value);
        }
	}


	int32 FParameters::GetIntPossibleValueCount( int32 ParamIndex ) const
	{
		const FProgram& Program = Model->GetProgram();
		check( ParamIndex>=0 && ParamIndex<(int32)Program.Parameters.Num() );
		return (int32)Program.Parameters[ParamIndex].PossibleValues.Num();
	}


	int32 FParameters::GetIntPossibleValue( int32 ParamIndex, int32 valueIndex ) const
	{
		const FProgram& Program = Model->GetProgram();
		check( ParamIndex>=0
				&& ParamIndex<(int32)Program.Parameters.Num() );
		check( valueIndex>=0
				&& valueIndex<(int32)Program.Parameters[ParamIndex].PossibleValues.Num() );

		return (int32)Program.Parameters[ParamIndex].PossibleValues[valueIndex].Value;
	}


	int32 FParameters::GetIntValueIndex(int32 ParamIndex, const FString& ValueName) const
	{
		const FProgram& Program = Model->GetProgram();
		check(ParamIndex >= 0
			&& ParamIndex < (int32)Program.Parameters.Num());

		int32 result = -1;
		for (size_t v = 0; v < Program.Parameters[ParamIndex].PossibleValues.Num(); ++v)
		{
			if (Program.Parameters[ParamIndex].PossibleValues[v].Name == ValueName)
			{
				result = (int32)v;
				break;
			}
		}
		return result;
	}


	int32 FParameters::GetIntValueIndex(int32 ParamIndex, int32 Value) const
	{
		const FProgram& Program = Model->GetProgram();
		check(ParamIndex >= 0
			&& ParamIndex < (int32)Program.Parameters.Num());

		for (size_t v = 0; v < Program.Parameters[ParamIndex].PossibleValues.Num(); ++v)
		{
			if (Program.Parameters[ParamIndex].PossibleValues[v].Value == Value)
			{
				return (int32)v;
			}
		}
		return -1;
	}


	const FString& FParameters::GetIntPossibleValueName( int32 ParamIndex, int32 valueIndex ) const
	{
		const FProgram& Program = Model->GetProgram();
		check( ParamIndex>=0
				&& ParamIndex<(int32)Program.Parameters.Num() );
		check( valueIndex>=0
				&& valueIndex<(int32)Program.Parameters[ParamIndex].PossibleValues.Num() );

		return Program.Parameters[ParamIndex].PossibleValues[valueIndex].Name;
	}


    int32 FParameters::GetIntValue( int32 Index, const FRangeIndex* Pos ) const
	{
		check( Index>=0 && Index<(int32)Values.Num() );
		check( GetType(Index)== EParameterType::Int );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Int )
        {
            return 0;
        }

        // Single value case
        if (!Pos)
        {
            // Return the single value
            return Values[Index].Get<FParamIntType>();
        }

        // Multivalue case
        check( Pos->Parameter==Index );

        if ( Index<int32(MultiValues.Num()))
        {
            const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(Pos->Values);
            if (it)
            {
                return it->Get<FParamIntType>();
            }
        }

        // Multivalue parameter, but no multivalue set. Return single value.
        return Values[Index].Get<FParamIntType>();
    }


    void FParameters::SetIntValue( int32 Index, int32 value,
                                  const FRangeIndex* Pos )
	{
		check( Index>=0 && Index<(int32)Values.Num() );
		check( GetType(Index)== EParameterType::Int );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Int )
        {
            return;
        }

        // Single value case
        if (!Pos)
        {
            // Clear multivalue, if set.
            if (Index<int32(MultiValues.Num()))
            {
                MultiValues[Index].Empty();
            }

            Values[Index].Set<FParamIntType>(value);
        }

        // Multivalue case
        else
        {
            check( Pos->Parameter==Index );

            if ( Index>=int32(MultiValues.Num()))
            {
                MultiValues.SetNum(Index+1);
            }

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(Pos->Values);
            it.Set<FParamIntType>(value);
        }
    }


    float FParameters::GetFloatValue( int32 Index,
                                     const FRangeIndex* Pos ) const
	{
		check( Index>=0 && Index<(int32)Values.Num() );
		check( GetType(Index)== EParameterType::Float );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Float )
        {
            return 0.0f;
        }

        // Single value case
        if (!Pos)
        {
            // Return the single value
            return Values[Index].Get<FParamFloatType>();
        }

        // Multivalue case
        check( Pos->Parameter==Index );

        if ( Index<int32(MultiValues.Num()))
        {
            const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(Pos->Values);
            if (it)
            {
                return it->Get<FParamFloatType>();
            }
        }

        // Multivalue parameter, but no multivalue set. Return single value.
        return Values[Index].Get<FParamFloatType>();
    }


    void FParameters::SetFloatValue( int32 Index, float value,
                                    const FRangeIndex* Pos )
	{
		check( Index>=0 && Index<(int32)Values.Num() );
		check( GetType(Index)== EParameterType::Float );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Float )
        {
            return;
        }

        // Single value case
        if (!Pos)
        {
            // Clear multivalue, if set.
            if (Index<int32(MultiValues.Num()))
            {
                MultiValues[Index].Empty();
            }

            Values[Index].Set<FParamFloatType>(value);
        }

        // Multivalue case
        else
        {
            check( Pos->Parameter==Index );

            if ( Index>=int32(MultiValues.Num()))
            {
                MultiValues.SetNum(Index+1);
            }

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(Pos->Values);
            it.Set<FParamFloatType>(value);
        }
    }


    void FParameters::GetColorValue( int32 Index, FVector4f& OutValue, const FRangeIndex* Pos ) const
    {
        check( Index>=0 && Index<(int32)Values.Num() );
        check( GetType(Index)== EParameterType::Color );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Color )
        {
            return;
        }

        // Single value case
        if (!Pos)
        {
			OutValue = Values[Index].Get<FParamColorType>();
            return;
        }

        // Multivalue case
        check( Pos->Parameter==Index );

        if ( Index<int32(MultiValues.Num()))
        {
            const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(Pos->Values);
            if (it)
            {
				OutValue = it->Get<FParamColorType>();
                return;
            }
        }

        // Multivalue parameter, but no multivalue set. Return single value.
		OutValue = Values[Index].Get<FParamColorType>();
        return;
    }


    void FParameters::SetColorValue( int32 Index, FVector4f InValue, const FRangeIndex* Pos )
    {
        check( Index>=0 && Index<(int32)Values.Num() );
        check( GetType(Index)== EParameterType::Color );

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Color )
        {
            return;
        }

        // Single value case
        if (!Pos)
        {
            // Clear multivalue, if set.
            if (Index<int32(MultiValues.Num()))
            {
                MultiValues[Index].Empty();
            }

        	Values[Index].Set<FParamColorType>(InValue);
        }

        // Multivalue case
        else
        {
            check( Pos->Parameter==Index );

            if ( Index>=int32(MultiValues.Num()))
            {
                MultiValues.SetNum(Index+1);
            }

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(Pos->Values);

        	it.Set<FParamColorType>(InValue);
        }
    }


	USkeletalMesh* FParameters::GetSkeletalMeshValue(int32 Index, const FRangeIndex* Pos) const
	{
		check(Index >= 0 && Index < (int32)Values.Num());
		check(GetType(Index) == EParameterType::SkeletalMesh);

		// Early out in case of invalid parameters
		if (Index < 0
			||
			Index >= (int32)Values.Num()
			||
			GetType(Index) != EParameterType::SkeletalMesh)
		{
			return {};
		}

		// Single value case
		if (!Pos)
		{
			return Values[Index].Get<FParamSkeletalMeshType>().Get();
		}

		// Multivalue case
		check(Pos->Parameter == Index);

		if (Index < int32(MultiValues.Num()))
		{
			const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(Pos->Values);
			if (it)
			{
				return it->Get<FParamSkeletalMeshType>().Get();
			}
		}

		// Multivalue parameter, but no multivalue set. Return single value.
		return Values[Index].Get<FParamSkeletalMeshType>().Get();
	}


	void FParameters::SetSkeletalMeshValue(int32 Index, USkeletalMesh* Value, const FRangeIndex* Pos)
	{
		check(Index >= 0 && Index < (int32)Values.Num());
		check(GetType(Index) == EParameterType::SkeletalMesh);

		// Single value case
		if (!Pos)
		{
			Values[Index].Set<FParamSkeletalMeshType>(TStrongObjectPtr(Value));
		}

		// Multivalue case
		else
		{
			check(Pos->Parameter == Index);

			if (Index >= int32(MultiValues.Num()))
			{
				MultiValues.SetNum(Index + 1);
			}

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(Pos->Values);
			it.Set<FParamSkeletalMeshType>(TStrongObjectPtr(Value));
		}
	}


	UTexture* FParameters::GetTextureValue( int32 Index, const FRangeIndex* Pos ) const
    {
        check( Index >= 0 && Index < (int32)Values.Num() );
        check( GetType( Index ) == EParameterType::Texture );

		// Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= (int32)Values.Num()
             ||
             GetType(Index) != EParameterType::Texture )
        {
            return {};
        }
		
		// Single value case
		if (!Pos)
		{
			return Values[Index].Get<FParamTextureType>().Get();
		}

		// Multivalue case
		check(Pos->Parameter == Index);

		if (Index < int32(MultiValues.Num()))
		{
			const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(Pos->Values);
			if (it)
			{
				return it->Get<FParamTextureType>().Get();
			}
		}

		// Multivalue parameter, but no multivalue set. Return single value.
		return Values[Index].Get<FParamTextureType>().Get();
    }


    void FParameters::SetTextureValue( int32 Index, UTexture* Value, const FRangeIndex* Pos )
    {
        check( Index >= 0 && Index < (int32)Values.Num() );
        check( GetType( Index ) == EParameterType::Texture );

		// Single value case
		if (!Pos)
		{
			Values[Index].Set<FParamTextureType>(TStrongObjectPtr(Value));
		}

		// Multivalue case
		else
		{
			check(Pos->Parameter == Index);

			if (Index >= int32(MultiValues.Num()))
			{
				MultiValues.SetNum(Index + 1);
			}

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(Pos->Values);
			it.Set<FParamTextureType>(TStrongObjectPtr(Value));
		}
    }


    void FParameters::GetStringValue( int32 Index, FString& OutValue, const FRangeIndex* Pos ) const
    {
        check( Index >= 0 && Index < (int32)Values.Num() );
        check( GetType( Index ) == EParameterType::String );

        // Early out in case of invalid parameters
        if ( Index < 0 || Index >= (int32)Values.Num() ||
             GetType( Index ) != EParameterType::String )
        {
            OutValue = TEXT("");
			return;
        }

        // Single value case
        if ( !Pos )
        {
            // Return the single value
			OutValue = Values[Index].Get<FParamStringType>();
            return;
        }

        // Multivalue case
        check( Pos->Parameter == Index );

        if ( Index < int32( MultiValues.Num() ) )
        {
            const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find( Pos->Values );
            if ( it )
            {
				OutValue = it->Get<FParamStringType>();
                return;
            }
        }

        // Multivalue parameter, but no multivalue set. Return single value.
		OutValue = Values[Index].Get<FParamStringType>();
        return;
    }


    void FParameters::SetStringValue( int32 Index, const FString& Value, const FRangeIndex* Pos )
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

        check( Index >= 0 && Index < (int32)Values.Num() );
        check( GetType( Index ) == EParameterType::Float );

        // Early out in case of invalid parameters
        if ( Index < 0 || Index >= (int32)Values.Num() ||
             GetType( Index ) != EParameterType::Float )
        {
            return;
        }

        // Single value case
        if ( !Pos )
        {
            // Clear multivalue, if set.
            if ( Index < int32( MultiValues.Num() ) )
            {
                MultiValues[Index].Empty();
            }

        	Values[Index].Set<FParamStringType>(Value);
        }

        // Multivalue case
        else
        {
            check( Pos->Parameter == Index );

            if ( Index >= int32( MultiValues.Num() ) )
            {
                MultiValues.SetNum( Index + 1 );
            }

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd( Pos->Values );
        	it.Set<FParamStringType>(Value);
        }
    }

    void FParameters::GetMatrixValue(int32 Index, FMatrix44f& OutValue, const FRangeIndex* Pos) const
    {
    	check( Index >= 0 && Index < (int32)Values.Num() );
    	check( GetType( Index ) == EParameterType::Matrix );

    	// Early out in case of invalid parameters
    	if ( Index < 0 || Index >= (int32)Values.Num() ||
			 GetType( Index ) != EParameterType::Matrix )
    	{
    		OutValue = FMatrix44f::Identity;
    		return;
    	}

    	// Single value case
    	if ( !Pos )
    	{
    		// Return the single value
    		OutValue = Values[Index].Get<FParamMatrixType>();
    		return;
    	}

    	// Multivalue case
    	check( Pos->Parameter == Index );

    	if ( Index < int32( MultiValues.Num() ) )
    	{
    		const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
    		const FParameterValue* it = m.Find( Pos->Values );
    		if ( it )
    		{
    			OutValue = it->Get<FParamMatrixType>();
    			return;
    		}
    	}

    	// Multivalue parameter, but no multivalue set. Return single value.
    	OutValue = Values[Index].Get<FParamMatrixType>();
    }

    void FParameters::SetMatrixValue(int32 Index, const FMatrix44f& Value, const FRangeIndex* Pos)
    {
    	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

    	check( Index >= 0 && Index < (int32)Values.Num() );
    	check( GetType( Index ) == EParameterType::Matrix );

    	// Early out in case of invalid parameters
    	if ( Index < 0 || Index >= (int32)Values.Num() ||
			 GetType( Index ) != EParameterType::Matrix )
    	{
    		return;
    	}

    	// Single value case
    	if ( !Pos )
    	{
    		// Clear multivalue, if set.
    		if ( Index < int32( MultiValues.Num() ) )
    		{
    			MultiValues[Index].Empty();
    		}

    		Values[Index].Set<FParamMatrixType>(Value);
    	}

    	// Multivalue case
    	else
    	{
    		check( Pos->Parameter == Index );

    		if ( Index >= int32( MultiValues.Num() ) )
    		{
    			MultiValues.SetNum( Index + 1 );
    		}

    		TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
    		FParameterValue& it = m.FindOrAdd( Pos->Values );
    		it.Set<FParamMatrixType>(Value);
    	}
    }


	UMaterialInterface* FParameters::GetMaterialValue(int32 Index, const FRangeIndex* RangeIndex) const
	{
		check(Index >= 0 && Index < (int32)Values.Num());
		check(GetType(Index) == EParameterType::Material);

		// Early out in case of invalid parameters
		if (Index < 0 || Index >= (int32)Values.Num() ||
			GetType(Index) != EParameterType::Material)
		{
			return nullptr;
		}

		// Single value case
		if (!RangeIndex)
		{
			// Return the single value
			return Values[Index].Get<FParamMaterialType>().Get();
		}

		// Multivalue case
		check(RangeIndex->Parameter == Index);

		if (Index < int32(MultiValues.Num()))
		{
			const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			const FParameterValue* it = m.Find(RangeIndex->Values);
			if (it)
			{
				return it->Get<FParamMaterialType>().Get();
			}
		}

		// Multivalue parameter, but no multivalue set. Return single value.
		return Values[Index].Get<FParamMaterialType>().Get();
	}


	void FParameters::SetMaterialValue(int32 Index, UMaterialInterface* Material, const FRangeIndex* RangeIndex)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

		check(Index >= 0 && Index < (int32)Values.Num());
		check(GetType(Index) == EParameterType::Material);

		// Early out in case of invalid parameters
		if (Index < 0 || Index >= (int32)Values.Num() ||
			GetType(Index) != EParameterType::Material)
		{
			return;
		}

		// Single value case
		if (!RangeIndex)
		{
			// Clear multivalue, if set.
			if (Index < int32(MultiValues.Num()))
			{
				MultiValues[Index].Empty();
			}

			Values[Index].Set<FParamMaterialType>(TStrongObjectPtr(Material));
		}

		// Multivalue case
		else
		{
			check(RangeIndex->Parameter == Index);

			if (Index >= int32(MultiValues.Num()))
			{
				MultiValues.SetNum(Index + 1);
			}

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
			FParameterValue& it = m.FindOrAdd(RangeIndex->Values);
			it.Set<FParamMaterialType>(TStrongObjectPtr(Material));
		}
	}


	TManagedPtr<const FInstancedStruct> FParameters::GetInstancedStructValue(int32 Index, const FRangeIndex* RangeIndex) const
	{
    	check(Index >= 0 && Index < (int32)Values.Num());
    	check(GetType(Index) == EParameterType::InstancedStruct);

    	// Early out in case of invalid parameters
    	if (Index < 0 || Index >= (int32)Values.Num() ||
			GetType(Index) != EParameterType::InstancedStruct)
    	{
    		return {};
    	}

    	// Single value case
    	if (!RangeIndex)
    	{
    		// Return the single value
    		return Values[Index].Get<FParamInstancedStructType>();
    	}

    	// Multivalue case
    	check(RangeIndex->Parameter == Index);

    	if (Index < int32(MultiValues.Num()))
    	{
    		const TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
    		const FParameterValue* it = m.Find(RangeIndex->Values);
    		if (it)
    		{
    			return it->Get<FParamInstancedStructType>();
    		}
    	}

    	// Multivalue parameter, but no multivalue set. Return single value.
    	return Values[Index].Get<FParamInstancedStructType>();
	}


	void FParameters::SetInstancedStructValue(int32 Index, const TManagedPtr<const FInstancedStruct>& Value, const FRangeIndex* RangeIndex)
	{
    	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

    	check(Index >= 0 && Index < (int32)Values.Num());
    	check(GetType(Index) == EParameterType::InstancedStruct);

    	// Early out in case of invalid parameters
    	if (Index < 0 || Index >= (int32)Values.Num() ||
			GetType(Index) != EParameterType::InstancedStruct)
    	{
    		return;
    	}

    	// Single value case
    	if (!RangeIndex)
    	{
    		// Clear multivalue, if set.
    		if (Index < int32(MultiValues.Num()))
    		{
    			MultiValues[Index].Empty();
    		}

    		Values[Index].Set<FParamInstancedStructType>(Value);
    	}

    	// Multivalue case
    	else
    	{
    		check(RangeIndex->Parameter == Index);

    		if (Index >= int32(MultiValues.Num()))
    		{
    			MultiValues.SetNum(Index + 1);
    		}

    		TMap< TArray<int32>, FParameterValue >& m = MultiValues[Index];
    		FParameterValue& it = m.FindOrAdd(RangeIndex->Values);
    		it.Set<FParamInstancedStructType>(Value);
    	}
	}


	FProjector FParameters::GetProjectorValue( int32 Index, const FRangeIndex* Pos ) const
    {
        const FProgram& Program = Model->GetProgram();

        // Early out in case of invalid parameters
        if ( Index < 0
             ||
             Index >= Values.Num()
			 || 
			 Index >= Program.Parameters.Num()
             ||
             Program.Parameters[Index].Type != EParameterType::Projector )
        {
			check(false);
            return FProjector();
        }

        // Single value case
        if (!Pos)
        {
            // Return the single value
			return Values[Index].Get<FParamProjectorType>();
        }

		const FProjector* Result = nullptr;

        // Multivalue case
        check( Pos->Parameter==Index );

        if ( Index<MultiValues.Num())
        {
            const FParameterValue* it = MultiValues[Index].Find(Pos->Values);
            if (it)
            {
				return it->Get<FParamProjectorType>();
            }
        }

        // Multivalue parameter, but no multivalue set. Return single value.
        return Values[Index].Get<FParamProjectorType>();
    }


	void FParameters::GetProjectorValue( int32 ParameterIndex,
		EProjectorType* OutType,
		FVector3f* OutPos,
		FVector3f* OutDir,
		FVector3f* OutUp,
		FVector3f* OutScale,
		float* OutProjectionAngle,
		const FRangeIndex* Pos ) const
	{
		check(ParameterIndex >=0 && ParameterIndex <Values.Num() );
		check( GetType(ParameterIndex)== EParameterType::Projector );

        // Early out in case of invalid parameters
        if (ParameterIndex < 0
             ||
			ParameterIndex >= Values.Num()
             ||
             GetType(ParameterIndex) != EParameterType::Projector )
        {
            return;
        }

        FProjector result = GetProjectorValue(ParameterIndex, Pos );

        // Copy results
        if (OutType) *OutType = result.type;

        if (OutPos) *OutPos = result.position;
        if (OutDir) *OutDir = result.direction;
        if (OutUp) *OutUp = result.up;
        if (OutScale) *OutScale = result.scale;

        if (OutProjectionAngle) *OutProjectionAngle = result.projectionAngle;
    }


	void FParameters::SetProjectorValue( int32 ParameterIndex,
		const FVector3f& Pos,
		const FVector3f& dir,
		const FVector3f& up,
		const FVector3f& scale,
        float projectionAngle,
        const FRangeIndex* RangePosition)
	{
		check(ParameterIndex >=0 && ParameterIndex <Values.Num() );
		check( GetType(ParameterIndex)== EParameterType::Projector );

        // Early out in case of invalid parameters
        if (ParameterIndex < 0
             ||
			ParameterIndex >= Values.Num()
             ||
             GetType(ParameterIndex) != EParameterType::Projector )
        {
            return;
        }

	    // FParameters cannot change the projector type anymore
		FProjector Value;
        Value.type = EProjectorType::Count;
        if (Model)
        {
            const FProgram& Program = Model->GetProgram();
            const FProjector& Projector = Program.Parameters[ParameterIndex].DefaultValue.Get<FParamProjectorType>();
            Value.type = Projector.type;
        }

        Value.position = Pos;
        Value.direction = dir;
        Value.up = up;        
        Value.scale = scale;

        Value.projectionAngle = projectionAngle;
		
        // Single value case
        if (!RangePosition)
        {
            // Clear multivalue, if set.
            if (ParameterIndex<MultiValues.Num())
            {
                MultiValues[ParameterIndex].Empty();
            }

            Values[ParameterIndex].Set<FParamProjectorType>(Value);
        }

        // Multivalue case
        else
        {
            check(RangePosition->Parameter== ParameterIndex);

            if (ParameterIndex >=MultiValues.Num())
            {
                MultiValues.SetNum(ParameterIndex+1);
            }

			TMap< TArray<int32>, FParameterValue >& m = MultiValues[ParameterIndex];
			FParameterValue& it = m.FindOrAdd(RangePosition->Values);
            it.Set<FParamProjectorType>(Value);
        }
    }


    bool FParameters::HasSameValue( int32 thisParamIndex,
                                   const TSharedPtr<const FParameters>& other,
                                   int32 otherParamIndex ) const
    {
        if ( GetType(thisParamIndex) != other->GetType(otherParamIndex) )
        {
            return false;
        }

        if ( !(Values[thisParamIndex]==other->Values[otherParamIndex] ) )
        {
            return false;
        }

        size_t thisNumMultiValues = 0;
        bool thisHasMultiValues = int32(MultiValues.Num()) > thisParamIndex;
        if (thisHasMultiValues)
        {
            thisNumMultiValues = MultiValues[thisParamIndex].Num();
        }

        size_t otherNumMultiValues = 0;
        bool otherHasMultiValues = int32(other->MultiValues.Num()) > otherParamIndex;
        if (otherHasMultiValues)
        {
            otherNumMultiValues = other->MultiValues[otherParamIndex].Num();
        }

        if ( thisNumMultiValues != otherNumMultiValues )
        {
            return false;
        }

        if ( thisHasMultiValues
             &&
             otherHasMultiValues
             &&
             !(MultiValues[thisParamIndex]==other->MultiValues[otherParamIndex] ))
        {
            return false;
        }

        return true;
    }
	
	
	bool FParameters::HasMultipleValues(int32 ParamIndex) const
    {
    	if (ParamIndex >= MultiValues.Num())
    	{
    		return false;
    	}

    	return MultiValues[ParamIndex].Num()>0;
    }
	

    int32 FRangeIndex::GetRangeCount() const
    {
        return int32( Values.Num() );
    }


    const FString& FRangeIndex::GetRangeName( int32 Index ) const
    {
        check( Index >= 0 && Index < GetRangeCount() );
        check( Index < int32(Parameters->Values.Num()) );
        TSharedPtr<const FModel> Model = Parameters->Model;
        check(Model);
        check( Index < int32(Model->GetProgram().Ranges.Num()) );

        return Model->GetProgram().Ranges[Index].Name;
    }


    const FString& FRangeIndex::GetRangeUid( int32 Index ) const
    {
        check( Index >= 0 && Index < GetRangeCount() );
        check( Index < int32(Parameters->Values.Num()) );
		TSharedPtr<const FModel> Model = Parameters->Model;
		check(Model);
        check( Index < int32(Model->GetProgram().Ranges.Num()) );

        return Model->GetProgram().Ranges[Index].UID;
    }


    void FRangeIndex::SetPosition( int32 Index, int32 Position )
    {
        check( Index >= 0 && Index < GetRangeCount() );
        if ( Index >= 0 && Index < GetRangeCount() )
        {
            Values[Index] = Position;
        }
    }


    int32 FRangeIndex::GetPosition( int32 Index ) const
    {
        check( Index >= 0 && Index < GetRangeCount() );
        if ( Index >= 0 && Index < GetRangeCount() )
        {
            return Values[Index];
        }
        return 0;
    }
}

