// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/PassthroughObject.h"

#include "MuR/MutableRuntimeModule.h"


namespace UE::Mutable::Private
{
	PASSTHROUGH_ID GetPassthroughId(const TSoftObjectPtr<UObject>& SoftObjectPtr)
	{
		PASSTHROUGH_ID Id = GetTypeHash(SoftObjectPtr.ToSoftObjectPath().ToString().ToLower());

		if (Id == PASSTHROUGH_ID_INVALID)
		{
			++Id;
		}
		
		return Id;
	}


	void FPassthroughObjectLoader::Load(const TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>>& ResolutionMap, TArray<FSoftObjectPath>& AssetsToStream) const
	{
		for (const TPair<PASSTHROUGH_ID, TPassthroughObjectPtr<UObject>>& Pair : PassthroughObjects)
		{
			const TPassthroughObjectPtr<UObject>& ExternalObject = Pair.Value;
				
			if (!ExternalObject.Resource->IsType<PASSTHROUGH_ID>()) 
			{
				// Already resolved.
				continue;
			}

			const PASSTHROUGH_ID ID = ExternalObject.Resource->Get<PASSTHROUGH_ID>();
			check(ID != PASSTHROUGH_ID_INVALID);

			const TSoftObjectPtr<UObject>* Result = ResolutionMap.Find(ID);
			if (!Result)
			{
				UE_LOGF(LogMutableCore, Error, "Unable to find Passthrough Object with id: %u", Pair.Key);
				continue;
			}
			
			AssetsToStream.Add(Result->ToSoftObjectPath());
		}
	}


	void FPassthroughObjectLoader::Resolve(TMap<PASSTHROUGH_ID, TSoftObjectPtr<UObject>>& ResolutionMap)
	{
		for (const TPair<PASSTHROUGH_ID, TPassthroughObjectPtr<UObject>>& Pair:  PassthroughObjects)
		{
			const TPassthroughObjectPtr<UObject>& ExternalObject = Pair.Value;

			if (!ExternalObject.Resource->IsType<PASSTHROUGH_ID>())
			{
				// Already resolved.
				continue;
			}

			const PASSTHROUGH_ID ID = ExternalObject.Resource->Get<PASSTHROUGH_ID>();
			check(ID != PASSTHROUGH_ID_INVALID);
			
			const TSoftObjectPtr<UObject>* Result = ResolutionMap.Find(ID);
			if (!Result)
			{
				// Error already thrown in Load.
				ExternalObject.Resource->Set<TStrongObjectPtr<UObject>>(nullptr);
				continue;
			}

			UObject* Object = Result->Get();
			if (!Object)
			{
				UE_LOGF(LogMutableCore, Error, "Unable to load Passthrough object: %ls", *Result->ToSoftObjectPath().ToString());
				ExternalObject.Resource->Set<TStrongObjectPtr<UObject>>(nullptr);
				continue;
			}

			ExternalObject.Resource->Set<TStrongObjectPtr<UObject>>(TStrongObjectPtr(Object));
		}
	}
}
