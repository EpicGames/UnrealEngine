// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/PhysicsBody.h"

#include "MuR/Serialisation.h"

namespace UE::Mutable::Private
{


	void FPhysicsBody::Serialise(const FPhysicsBody* p, FOutputArchive& Arch)
	{
		Arch << *p;
	}


	TManagedPtr<FPhysicsBody> FPhysicsBody::StaticUnserialise(FInputArchive& Arch)
	{
		TManagedPtr<FPhysicsBody> Result = MakeManaged<FPhysicsBody>();
		Arch >> *Result;
		return Result;
	}


	void FBodyShape::Serialise(FOutputArchive& Arch) const
	{
		Arch << Name;
		Arch << Flags;
	}


	void FBodyShape::Unserialise(FInputArchive& Arch)
	{
		Arch >> Name;
		Arch >> Flags;
	}


	void FSphereBody::Serialise(FOutputArchive& Arch) const
	{
		FBodyShape::Serialise(Arch);

		Arch << Position;
		Arch << Radius;
	}


	void FSphereBody::Unserialise(FInputArchive& Arch)
	{
		FBodyShape::Unserialise(Arch);

		Arch >> Position;
		Arch >> Radius;
	}


	void FBoxBody::Serialise(FOutputArchive& Arch) const
	{
		FBodyShape::Serialise( Arch );

		Arch << Position;
		Arch << Orientation;
		Arch << Size;
	}


	void FBoxBody::Unserialise(FInputArchive& Arch)
	{
		FBodyShape::Unserialise( Arch );

		Arch >> Position;
		Arch >> Orientation;
		Arch >> Size;
	}

	
	void FSphylBody::Serialise(FOutputArchive& Arch) const
	{
		FBodyShape::Serialise( Arch );

		Arch << Position;
		Arch << Orientation;
		Arch << Radius;
		Arch << Length;
	}


	void FSphylBody::Unserialise(FInputArchive& Arch)
	{
		FBodyShape::Unserialise( Arch );

		Arch >> Position;
		Arch >> Orientation;
		Arch >> Radius;
		Arch >> Length;
	}


	void FTaperedCapsuleBody::Serialise(FOutputArchive& Arch) const
	{
		FBodyShape::Serialise( Arch );

		Arch << Position;
		Arch << Orientation;
		Arch << Radius0;
		Arch << Radius1;
		Arch << Length;
	}


	void FTaperedCapsuleBody::Unserialise(FInputArchive& Arch)
	{
		FBodyShape::Unserialise( Arch );

		Arch >> Position;
		Arch >> Orientation;
		Arch >> Radius0;
		Arch >> Radius1;
		Arch >> Length;
	}


	void FConvexBody::Serialise(FOutputArchive& Arch) const
	{
		FBodyShape::Serialise( Arch );

		Arch << Vertices;
		Arch << Indices;
		Arch << Transform;
	}


	void FConvexBody::Unserialise(FInputArchive& Arch)
	{
		FBodyShape::Unserialise( Arch );

		Arch >> Vertices;
		Arch >> Indices;
		Arch >> Transform;
	}


	void FPhysicsBodyAggregate::Serialise(FOutputArchive& Arch) const
	{
		Arch << Spheres;
		Arch << Boxes;
		Arch << Convex;
		Arch << Sphyls;
		Arch << TaperedCapsules;
	}


	void FPhysicsBodyAggregate::Unserialise(FInputArchive& Arch)
	{
		Arch >> Spheres;
		Arch >> Boxes;
		Arch >> Convex;
		Arch >> Sphyls;
		Arch >> TaperedCapsules;
	}


	TManagedPtr<FPhysicsBody> FPhysicsBody::Clone() const
	{
		TManagedPtr<FPhysicsBody> Result = MakeManaged<FPhysicsBody>();

		Result->CustomId = CustomId;

		Result->Bodies = Bodies;
		Result->BodiesBoneNames = BodiesBoneNames;
		Result->BoneIds = BoneIds;

		Result->bBodiesModified = bBodiesModified;


		return Result;
	}

	void FPhysicsBody::SetBodyCount(int32 Count)
	{
		Bodies.SetNum(Count);
		BodiesBoneNames.SetNum(Count);
	}

	int32 FPhysicsBody::GetBodyCount() const
	{
		return Bodies.Num();
	}

	void FPhysicsBody::SetBody(int32 BodyIndex, FName InBoneName)
	{
		BodiesBoneNames[BodyIndex] = InBoneName;
	}

	FName FPhysicsBody::GetBodyBoneName(int32 BodyIndex) const
	{
		check(BodiesBoneNames.IsValidIndex(BodyIndex));
		return BodiesBoneNames[BodyIndex];
	}

	int32 FPhysicsBody::GetSphereCount(int32 B) const
	{
		return Bodies[B].Spheres.Num();
	}

	int32 FPhysicsBody::GetBoxCount(int32 B) const
	{
		return Bodies[B].Boxes.Num();
	}

	int32 FPhysicsBody::GetConvexCount(int32 B) const
	{
		return Bodies[B].Convex.Num();
	}

	int32 FPhysicsBody::GetSphylCount(int32 B) const
	{
		return Bodies[B].Sphyls.Num();
	}

	int32 FPhysicsBody::GetTaperedCapsuleCount(int32 B) const
	{
		return Bodies[B].TaperedCapsules.Num();
	}

	void FPhysicsBody::SetSphereCount(int32 B, int32 Count)
	{
		Bodies[B].Spheres.SetNum(Count);
	}

	void FPhysicsBody::SetBoxCount(int32 B, int32 Count)
	{
		Bodies[B].Boxes.SetNum(Count);
	}

	void FPhysicsBody::SetConvexCount(int32 B, int32 Count)
	{
		Bodies[B].Convex.SetNum(Count);
	}

	void FPhysicsBody::SetSphylCount(int32 B, int32 Count)
	{
		Bodies[B].Sphyls.SetNum(Count);
	}

	void FPhysicsBody::SetTaperedCapsuleCount(int32 B, int32 Count)
	{
		Bodies[B].TaperedCapsules.SetNum(Count);
	}

	void FPhysicsBody::SetSphere(
		int32 B, int32 I,
		FVector3f Position, float Radius)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Spheres.Num());

		Bodies[B].Spheres[I].Position = Position;
		Bodies[B].Spheres[I].Radius = Radius;
	}

	void FPhysicsBody::SetBox(
		int32 B, int32 I,
		FVector3f Position, FQuat4f Orientation, FVector3f Size)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Boxes.Num());

		Bodies[B].Boxes[I].Position = Position;
		Bodies[B].Boxes[I].Orientation = Orientation;
		Bodies[B].Boxes[I].Size = Size;
	}

	void FPhysicsBody::SetConvexMesh(
		int32 B, int32 I,
		TArrayView<const FVector3f> Vertices, TArrayView<const int32> Indices)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		Bodies[B].Convex[I].Vertices = TArray<FVector3f>(Vertices);
		Bodies[B].Convex[I].Indices = TArray<int32>(Indices);
	}

	void FPhysicsBody::SetConvexTransform(
		int32 B, int32 I,
		const FTransform3f& Transform)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		Bodies[B].Convex[I].Transform = Transform;
	}


	void FPhysicsBody::SetSphyl(
		int32 B, int32 I,
		FVector3f Position, FQuat4f Orientation, float Radius, float Length)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Sphyls.Num());

		Bodies[B].Sphyls[I].Position = Position;
		Bodies[B].Sphyls[I].Orientation = Orientation;
		Bodies[B].Sphyls[I].Radius = Radius;
		Bodies[B].Sphyls[I].Length = Length;
	}

	void FPhysicsBody::SetSphereFlags(int32 B, int32 I, uint32 Flags)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Spheres.Num());

		Bodies[B].Spheres[I].Flags = Flags;
	}

	void FPhysicsBody::SetBoxFlags(int32 B, int32 I, uint32 Flags)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Boxes.Num());

		Bodies[B].Boxes[I].Flags = Flags;
	}

	void FPhysicsBody::SetConvexFlags(int32 B, int32 I, uint32 Flags)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		Bodies[B].Convex[I].Flags = Flags;
	}

	void FPhysicsBody::SetSphylFlags(int32 B, int32 I, uint32 Flags)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Sphyls.Num());

		Bodies[B].Sphyls[I].Flags = Flags;
	}

	void FPhysicsBody::SetTaperedCapsuleFlags(int32 B, int32 I, uint32 Flags)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].TaperedCapsules.Num());

		Bodies[B].TaperedCapsules[I].Flags = Flags;
	}

	void FPhysicsBody::SetSphereName(int32 B, int32 I, const char* Name)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Spheres.Num());

		Bodies[B].Spheres[I].Name = Name;
	}

	void FPhysicsBody::SetBoxName(int32 B, int32 I, const char* Name)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Boxes.Num());

		Bodies[B].Boxes[I].Name = Name;
	}

	void FPhysicsBody::SetConvexName(int32 B, int32 I, const char* Name)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		Bodies[B].Convex[I].Name = Name;
	}

	void FPhysicsBody::SetSphylName(int32 B, int32 I, const char* Name)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Sphyls.Num());

		Bodies[B].Sphyls[I].Name = Name;
	}

	void FPhysicsBody::SetTaperedCapsuleName(int32 B, int32 I, const char* Name)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].TaperedCapsules.Num());

		Bodies[B].TaperedCapsules[I].Name = Name;
	}

	void FPhysicsBody::SetTaperedCapsule(
		int32 B, int32 I,
		FVector3f Position, FQuat4f Orientation,
		float Radius0, float Radius1, float Length)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].TaperedCapsules.Num());

		Bodies[B].TaperedCapsules[I].Position = Position;
		Bodies[B].TaperedCapsules[I].Orientation = Orientation;
		Bodies[B].TaperedCapsules[I].Radius0 = Radius0;
		Bodies[B].TaperedCapsules[I].Radius1 = Radius1;
		Bodies[B].TaperedCapsules[I].Length = Length;
	}

	void FPhysicsBody::GetSphere(
		int32 B, int32 I,
		FVector3f& OutPosition, float& OutRadius) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Spheres.Num());

		OutPosition = Bodies[B].Spheres[I].Position;
		OutRadius = Bodies[B].Spheres[I].Radius;
	}

	void FPhysicsBody::GetBox(
		int32 B, int32 I,
		FVector3f& OutPosition, FQuat4f& OutOrientation, FVector3f& OutSize) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Boxes.Num());

		OutPosition = Bodies[B].Boxes[I].Position;
		OutOrientation = Bodies[B].Boxes[I].Orientation;
		OutSize = Bodies[B].Boxes[I].Size;
	}

	void FPhysicsBody::GetConvex(
		int32 B, int32 I,
		TArrayView<const FVector3f>& OutVertices, TArrayView<const int32>& OutIndices,
		FTransform3f& OutTransform) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		OutVertices = TArrayView<const FVector3f>(Bodies[B].Convex[I].Vertices.GetData(), Bodies[B].Convex[I].Vertices.Num());
		OutIndices = TArrayView<const int32>(Bodies[B].Convex[I].Indices.GetData(), Bodies[B].Convex[I].Indices.Num());
		OutTransform = Bodies[B].Convex[I].Transform;
	}

	void FPhysicsBody::GetConvexMeshView(int32 B, int32 I,
		TArrayView<FVector3f>& OutVerticesView, TArrayView<int32>& OutIndicesView)
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		OutVerticesView = TArrayView<FVector3f>(Bodies[B].Convex[I].Vertices.GetData(), Bodies[B].Convex[I].Vertices.Num());
		OutIndicesView = TArrayView<int32>(Bodies[B].Convex[I].Indices.GetData(), Bodies[B].Convex[I].Indices.Num());
	}

	void FPhysicsBody::GetConvexTransform(int32 B, int32 I, FTransform3f& OutTransform) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		OutTransform = Bodies[B].Convex[I].Transform;
	}

	void FPhysicsBody::GetSphyl(
		int32 B, int32 I,
		FVector3f& OutPosition, FQuat4f& OutOrientation, float& OutRadius, float& OutLength) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Sphyls.Num());

		OutPosition = Bodies[B].Sphyls[I].Position;
		OutOrientation = Bodies[B].Sphyls[I].Orientation;
		OutRadius = Bodies[B].Sphyls[I].Radius;
		OutLength = Bodies[B].Sphyls[I].Length;
	}

	void FPhysicsBody::GetTaperedCapsule(
		int32 B, int32 I,
		FVector3f& OutPosition, FQuat4f& OutOrientation,
		float& OutRadius0, float& OutRadius1, float& OutLength) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].TaperedCapsules.Num());

		OutPosition = Bodies[B].TaperedCapsules[I].Position;
		OutOrientation = Bodies[B].TaperedCapsules[I].Orientation;
		OutRadius0 = Bodies[B].TaperedCapsules[I].Radius0;
		OutRadius1 = Bodies[B].TaperedCapsules[I].Radius1;
		OutLength = Bodies[B].TaperedCapsules[I].Length;
	}


	uint32 FPhysicsBody::GetSphereFlags(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Spheres.Num());

		return Bodies[B].Spheres[I].Flags;
	}

	uint32 FPhysicsBody::GetBoxFlags(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Boxes.Num());

		return Bodies[B].Boxes[I].Flags;
	}

	uint32 FPhysicsBody::GetConvexFlags(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		return Bodies[B].Convex[I].Flags;
	}

	uint32 FPhysicsBody::GetSphylFlags(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Sphyls.Num());

		return Bodies[B].Sphyls[I].Flags;
	}

	uint32 FPhysicsBody::GetTaperedCapsuleFlags(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].TaperedCapsules.Num());

		return Bodies[B].TaperedCapsules[I].Flags;
	}

	const FString& FPhysicsBody::GetSphereName(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Spheres.Num());

		return Bodies[B].Spheres[I].Name;
	}

	const FString& FPhysicsBody::GetBoxName(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Boxes.Num());

		return Bodies[B].Boxes[I].Name;
	}

	const FString& FPhysicsBody::GetConvexName(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Convex.Num());

		return Bodies[B].Convex[I].Name;
	}

	const FString& FPhysicsBody::GetSphylName(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].Sphyls.Num());

		return Bodies[B].Sphyls[I].Name;
	}

	const FString& FPhysicsBody::GetTaperedCapsuleName(int32 B, int32 I) const
	{
		check(B >= 0 && B < Bodies.Num());
		check(I >= 0 && I < Bodies[B].TaperedCapsules.Num());

		return Bodies[B].TaperedCapsules[I].Name;
	}

	void FPhysicsBody::ConvertBoneIdsToNames(const TMap<uint32, FName>& ConstantNames)
	{
		if (BodiesBoneNames.IsEmpty())
		{
			BodiesBoneNames.Reserve(BoneIds.Num());

			for (uint32 BoneId : BoneIds)
			{
				BodiesBoneNames.Add(ConstantNames.FindChecked(BoneId));
			}

			BoneIds.Empty();
		}
	}

	void FPhysicsBody::Serialise(FOutputArchive& Arch) const
	{
		Arch << CustomId;
		Arch << Bodies;
		Arch << BoneIds;
		Arch << bBodiesModified;
	}

	void FPhysicsBody::Unserialise(FInputArchive& Arch)
	{
		Arch >> CustomId;        	
		Arch >> Bodies;
		Arch >> BoneIds;
		Arch >> bBodiesModified;
	}
}
