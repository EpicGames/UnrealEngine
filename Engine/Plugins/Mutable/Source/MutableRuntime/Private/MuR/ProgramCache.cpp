// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ProgramCache.h"
#include "MuR/MemoryCounters.h"
#include "MuR/Mesh.h"
#include "MuR/Image.h"

#include "Concepts/ConvertibleTo.h"

namespace UE::Mutable::Private
{
namespace ProgramCacheInternals
{

	enum class EOpExecDataFlags : uint8
	{
		None     = 0,
		IsSet    = 1 << 0,
		IsLocked = 1 << 1,
		IsMarkedToKeep = 1 << 2,

		Aborted  = 1 << 3,

		AllocCounted  = 1 << 4,
	};

	ENUM_CLASS_FLAGS(EOpExecDataFlags);

	struct alignas(2) FAddressStateUpdate
	{
		EOpExecDataFlags AddedFlags = EOpExecDataFlags::None;
		int8 OpHitDifferential = 0; 
	};

	struct FOpCacheVTable
	{
		using GetFlagsFuncType = EOpExecDataFlags(void*, FCacheAddress);
		using UpdateAddressStateFuncType = void(void*, FCacheAddress, FAddressStateUpdate);
		using AcquireCachedResultForSchedulingFuncType = bool(void*, FCacheAddress);
		using AcquireCachedResultFuncType = bool(void*, FCacheAddress);
		using StageAcquireResultFuncType = FProgramCache::EAcquireSetResult(void*, FCacheAddress);
		using CommitAcquireResultFuncType = void(void*, FCacheAddress);
		using RollbackAcquireResultFuncType = void(void*, FCacheAddress);
		using LockUnlockFuncType = void(void*);

		constexpr FOpCacheVTable(
				GetFlagsFuncType* InGetFlags, 
				UpdateAddressStateFuncType* InUpdateAddressState, 
				AcquireCachedResultForSchedulingFuncType* InAcquireCachedResultForScheduling,
				AcquireCachedResultFuncType* InAcquireCachedResult,
				StageAcquireResultFuncType* InStageAcquireResult,
				CommitAcquireResultFuncType* InCommitAcquireResult,
				RollbackAcquireResultFuncType* InRollbackAcquireResult,
				LockUnlockFuncType* InLock,
				LockUnlockFuncType* InUnlock
		)
			: GetFlags(InGetFlags)
			, UpdateAddressState(InUpdateAddressState)
			, AcquireCachedResultForScheduling(InAcquireCachedResultForScheduling)
			, AcquireCachedResult(InAcquireCachedResult)
			, StageAcquireResult(InStageAcquireResult)
			, CommitAcquireResult(InCommitAcquireResult)
			, RollbackAcquireResult(InRollbackAcquireResult)
			, Lock(InLock)
			, Unlock(InUnlock)
		{
		}

		GetFlagsFuncType* GetFlags;
		UpdateAddressStateFuncType* UpdateAddressState;
		AcquireCachedResultForSchedulingFuncType* AcquireCachedResultForScheduling;
		AcquireCachedResultFuncType* AcquireCachedResult;
		StageAcquireResultFuncType* StageAcquireResult;
		CommitAcquireResultFuncType* CommitAcquireResult;
		RollbackAcquireResultFuncType* RollbackAcquireResult;

		LockUnlockFuncType* Lock;
		LockUnlockFuncType* Unlock;
	};

	template<class CacheType>
	struct TOpCacheDataVTableImpl
	{
		static EOpExecDataFlags GetFlags(void* Cache, FCacheAddress Address)
		{
			return static_cast<CacheType*>(Cache)->GetFlags(Address);
		}

		static void UpdateAddressState(void* Cache, FCacheAddress Address, FAddressStateUpdate Update)
		{
			return static_cast<CacheType*>(Cache)->UpdateAddressState(Address, Update);
		}

		static bool AcquireCachedResultForScheduling(void* Cache, FCacheAddress Address)
		{
			return static_cast<CacheType*>(Cache)->AcquireCachedResultForScheduling(Address);
		}

		static bool AcquireCachedResult(void* Cache, FCacheAddress Address)
		{
			return static_cast<CacheType*>(Cache)->AcquireCachedResult(Address);
		}

		static FProgramCache::EAcquireSetResult StageAcquireResult(void* Cache, FCacheAddress Address)
		{
			return static_cast<CacheType*>(Cache)->StageAcquireResult(Address);
		}

		static void CommitAcquireResult(void* Cache, FCacheAddress Address)
		{
			return static_cast<CacheType*>(Cache)->CommitAcquireResult(Address);
		}

		static void RollbackAcquireResult(void* Cache, FCacheAddress Address)
		{
			return static_cast<CacheType*>(Cache)->RollbackAcquireResult(Address);
		}

		static void Lock(void* Cache)
		{
			return static_cast<CacheType*>(Cache)->Lock();
		}

		static void Unlock(void* Cache)
		{
			return static_cast<CacheType*>(Cache)->Unlock();
		}
	};

	template <typename Cache>
	constexpr FOpCacheVTable GOpCacheVTableImpl(
		&TOpCacheDataVTableImpl<Cache>::GetFlags,
		&TOpCacheDataVTableImpl<Cache>::UpdateAddressState,
		&TOpCacheDataVTableImpl<Cache>::AcquireCachedResultForScheduling,
		&TOpCacheDataVTableImpl<Cache>::AcquireCachedResult,
		&TOpCacheDataVTableImpl<Cache>::StageAcquireResult,
		&TOpCacheDataVTableImpl<Cache>::CommitAcquireResult,
		&TOpCacheDataVTableImpl<Cache>::RollbackAcquireResult,
		&TOpCacheDataVTableImpl<Cache>::Lock,
		&TOpCacheDataVTableImpl<Cache>::Unlock
	);

	struct FTypeErasedOpCacheDataRef
	{
		template<typename CacheType>
		UE_NODEBUG [[nodiscard]] FTypeErasedOpCacheDataRef(CacheType& InCache UE_LIFETIMEBOUND)
			: VPtr(&GOpCacheVTableImpl<CacheType>)
			, CachePtr(&InCache)
		{
		}

		UE_NODEBUG [[nodiscard]] FTypeErasedOpCacheDataRef(const FOpCacheVTable* InVPtr, void* InCachePtr)
			: VPtr(InVPtr)
			, CachePtr(InCachePtr)
		{
		}

		UE_NODEBUG EOpExecDataFlags GetFlags(FCacheAddress Address) const
		{
			return this->VPtr->GetFlags(this->CachePtr, Address);
		}

		UE_NODEBUG void UpdateAddressState(FCacheAddress Address, FAddressStateUpdate Update) const
		{
			return this->VPtr->UpdateAddressState(this->CachePtr, Address, Update);
		}

		UE_NODEBUG bool AcquireCachedResultForScheduling(FCacheAddress Address) const
		{
			return this->VPtr->AcquireCachedResultForScheduling(this->CachePtr, Address);
		}

		UE_NODEBUG bool AcquireCachedResult(FCacheAddress Address) const
		{
			return this->VPtr->AcquireCachedResult(this->CachePtr, Address);
		}

		UE_NODEBUG FProgramCache::EAcquireSetResult StageAcquireResult(FCacheAddress Address) const
		{
			return this->VPtr->StageAcquireResult(this->CachePtr, Address);
		}

		UE_NODEBUG void CommitAcquireResult(FCacheAddress Address) const
		{
			return this->VPtr->CommitAcquireResult(this->CachePtr, Address);
		}

		UE_NODEBUG void RollbackAcquireResult(FCacheAddress Address) const
		{
			return this->VPtr->RollbackAcquireResult(this->CachePtr, Address);
		}

		UE_NODEBUG void Lock() const
		{
			return this->VPtr->Lock(this->CachePtr);
		}

		UE_NODEBUG void Unlock() const
		{
			return this->VPtr->Lock(this->CachePtr);
		}

		const FOpCacheVTable* VPtr;
		void* CachePtr;
	};

	struct FVoidType {};

	struct FOpExecDataBase
	{
		FCacheAddress Address;
		uint16 OpHitCount = 0;
		EOpExecDataFlags Flags = EOpExecDataFlags::None;
		uint8 StagedWeakCount = 0;
	};

	template<typename T>
	struct TOpExecData : public FOpExecDataBase
	{
		using PayloadType = T; 
		T Payload {};
	};

	template<>
	struct TOpExecData<FVoidType> : public FOpExecDataBase
	{
		using PayloadType = FVoidType;
	};

	template<typename T>
	struct TOpExecData<TManagedPtr<T>> : public FOpExecDataBase
	{
		using PayloadType = TManagedPtr<T>;

		TManagedPtr<T> Payload {};
		TManagedWeakPtr<T> Weak {};
	};

	template<typename T>
	concept CIsPointerWithExternalDataSizeQuery = requires(T Val) 
	{ 
		{ Val->GetDataSize() } -> CConvertibleTo<SSIZE_T>; 
	};

	template<typename T>
	concept CHasWeakField = requires { T::Weak; };

	template<typename T>
	concept CIsWeakable = CHasWeakField<TOpExecData<T>> && CIsPointerWithExternalDataSizeQuery<T>;

	template<typename T>
	concept CIsVoid = std::is_same_v<T, FVoidType>;

	template<class T>
	struct TOpCacheData
	{
		using PayloadDataType = T;
		
		using FAddressArray = TArray<uint16, TMemoryTrackingAllocatorWrapper<TAlignedHeapAllocator<PLATFORM_CACHE_LINE_SIZE>, MemoryCounters::FInternalMemoryCounter>>;
		using FDataArray    = TArray<TOpExecData<T>, TMemoryTrackingAllocatorWrapper<TAlignedHeapAllocator<PLATFORM_CACHE_LINE_SIZE>, MemoryCounters::FInternalMemoryCounter>>;

		FAddressArray AddressEntries;
		FDataArray Data;

		int32 LastIndex = -1;

		mutable FMutex Mutex;
 
		TOpCacheData& operator=(const TOpCacheData& Other) = delete;
		TOpCacheData& operator=(TOpCacheData&& Other) = delete;

		FORCEINLINE uint16 GetMangledAddress(FOperation::ADDRESS Address)
		{
			return static_cast<uint16>(Address >> 2);
		}

		UE_REWRITE int32 FindImpl(FCacheAddress Address)
		{
			uint16 MangledAddress = GetMangledAddress(Address.At); 

			int32 FoundIndex = LastIndex;

			for (; FoundIndex >= 0; --FoundIndex)
			{
				if (AddressEntries[FoundIndex] != MangledAddress)
				{
					continue;
				}

				if (Data[FoundIndex].Address == Address)
				{
					break;
				}
			}

			return FoundIndex;
		}

		int32 AllocateNew(FCacheAddress Address)
		{	
			if (AddressEntries.Num() - 1 <= LastIndex)
			{
				int32 NewNumElems = AddressEntries.Num() + PLATFORM_CACHE_LINE_SIZE;

				AddressEntries.SetNumUninitialized(NewNumElems);
				Data.SetNum(NewNumElems);
			}

			++LastIndex;

			AddressEntries[LastIndex] = GetMangledAddress(Address.At);

			Data[LastIndex] = TOpExecData<T>{{.Address = Address}};

			return LastIndex;
		}

		void Free(int32 Index)
		{
			if constexpr (!std::is_trivial_v<T>)
			{
				if constexpr (CIsWeakable<T>)
				{
					Data[Index].Payload = {};
					Data[Index].Weak = {};
				}
				else
				{
					Data[Index].Payload = T{};
				}
			}
		
			AddressEntries[Index] = AddressEntries[LastIndex];
			Data[Index] = MoveTemp(Data[LastIndex]);

			--LastIndex;
		}

		void ClearImpl(FProgramCache::EClearFlags ClearFlags)
		{
			for (int32 Index = LastIndex; Index >= 0; --Index)
			{
				if (Data[Index].OpHitCount > 0)
				{
					continue;
				}

				EOpExecDataFlags AddressFlags = Data[Index].Flags;

				if (EnumHasAnyFlags(AddressFlags, EOpExecDataFlags::IsMarkedToKeep) && 
				    !EnumHasAnyFlags(ClearFlags, FProgramCache::EClearFlags::ForceClearMarkedToKeep))
				{
					continue;
				}	

				bool bIsLockedAddress = EnumHasAnyFlags(AddressFlags, EOpExecDataFlags::IsLocked);
				if constexpr (CIsWeakable<T>)
				{
					const bool bMakeWeak = 
						(bIsLockedAddress) & 
						(!EnumHasAnyFlags(ClearFlags, FProgramCache::EClearFlags::Locked)); 

					if (bMakeWeak)
					{
						Data[Index].Weak = Data[Index].Payload;
						Data[Index].Payload = {};
					}
				}

				const bool bFreeAddress = 
					(  bIsLockedAddress  && EnumHasAnyFlags(ClearFlags, FProgramCache::EClearFlags::Locked)) || 
					((!bIsLockedAddress) && EnumHasAnyFlags(ClearFlags, FProgramCache::EClearFlags::Unlocked));

				if (bFreeAddress)
				{
					Free(Index);
				}
			}

			if (EnumHasAnyFlags(ClearFlags, FProgramCache::EClearFlags::FreeMemory))
			{
				Data.SetNum(LastIndex + 1, EAllowShrinking::Yes);
				AddressEntries.SetNum(LastIndex + 1, EAllowShrinking::Yes);
			}
		}

		EOpExecDataFlags GetFlagsImpl(FCacheAddress Address)
		{			
			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex < 0)
			{
				return EOpExecDataFlags::None;
			}

			if constexpr (CIsWeakable<T>)
			{
				// For weakable types the IsSet flag is weak. If false, then its not valid,
				// but if true it may not be.

				// To check a value we must have a strong reference, otherwise we can not guarantee the 
				// result will be valid when used. To check for validity the acquire functions must be used.  
				
				EOpExecDataFlags Flags = Data[FoundIndex].Flags;
				EnumRemoveFlags(Flags, EOpExecDataFlags::IsSet);

				EOpExecDataFlags StrongSetFlags = Data[FoundIndex].Payload.IsSet() 
						? EOpExecDataFlags::IsSet : EOpExecDataFlags::None;

				EnumAddFlags(Flags, StrongSetFlags);
				return Flags;		
			}
			else
			{
				return Data[FoundIndex].Flags;
			}
		}

		/** Sets the value at address with Elem if the address exists in the cache, return true if the operation is succesful. */
		bool SetValue(FCacheAddress Address, const PayloadDataType& Payload)
		{
			if (Address.At == 0)
			{
				return true;
			}

			TScopeLock<FMutex> LockGuard(Mutex);

			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex < 0)
			{
				FoundIndex = AllocateNew(Address);
			}

			if constexpr (!CIsVoid<T>)
			{
				if constexpr (CIsWeakable<T>)
				{
					// We expect to be a set value enven if null.
					check(Payload.IsSet());
				}

				Data[FoundIndex].Payload = Payload;
			}

			EnumAddFlags(Data[FoundIndex].Flags, EOpExecDataFlags::IsSet);
			return true;
		}

		/** Gets the value at address with Elem, returns true if address had a valid value, false otherwise */
		bool GetValue(FCacheAddress Address, PayloadDataType& OutPayload, bool bDecreaseOpHitCount = true)
		{
			if (Address.At == 0)
			{
				return true;
			}

			TScopeLock<FMutex> LockGuard(Mutex);

			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex >= 0)
			{
				if constexpr (!CIsVoid<T>)
				{
					OutPayload = Data[FoundIndex].Payload;
				}

				if constexpr (CIsWeakable<T>)
				{
					// Get value is only valid to call for strong references that need to be acquired before
					// getting the result. A strong reference will never have the Weak set.
					check(!Data[FoundIndex].Weak.IsValid());
				}

				check(!bDecreaseOpHitCount || Data[FoundIndex].OpHitCount > 0);

				Data[FoundIndex].OpHitCount -= static_cast<uint16>(bDecreaseOpHitCount);
				return EnumHasAnyFlags(Data[FoundIndex].Flags, EOpExecDataFlags::IsSet);
			}

			return false;
		}

		/** 
		 * Move the value at Address to OutValue if possible. returns true if the value has been moved, false otherwise.
		 * In the later case, the reference in the cache becomes weak.
		 */
		bool MoveValueIfPossible(FCacheAddress Address, PayloadDataType& OutValue)
		{
			if (Address.At == 0)
			{
				return false;
			}

			TScopeLock<FMutex> LockGuard(Mutex);
			
			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex < 0)
			{
				return false;
			}
				
			check(Data[FoundIndex].OpHitCount > 0);

			EOpExecDataFlags AddressFlags = Data[FoundIndex].Flags;

			const bool bIsLastHit = --Data[FoundIndex].OpHitCount == 0;
			const bool bCanFree = bIsLastHit & (!EnumHasAnyFlags(AddressFlags, EOpExecDataFlags::IsLocked));

			if constexpr (!CIsWeakable<T>)
			{
				check(EnumHasAnyFlags(AddressFlags, EOpExecDataFlags::IsSet));

				if (bCanFree)
				{
					OutValue = MoveTemp(Data[FoundIndex].Payload);	
					Free(FoundIndex);
				}
				else
				{
					OutValue = Data[FoundIndex].Payload;
				}
			}
			else
			{
				check(Data[FoundIndex].Payload.IsSet());

				if (bCanFree)
				{
					OutValue = MoveTemp(Data[FoundIndex].Payload);	
					Free(FoundIndex);
				}
				else
				{
					if (bIsLastHit)
					{
						OutValue = MoveTemp(Data[FoundIndex].Payload);
						Data[FoundIndex].Payload = {};
						Data[FoundIndex].Weak = OutValue;
					}
					else
					{
						OutValue = Data[FoundIndex].Payload;
					}
				}
			}

			return bCanFree;
		}
	
		EOpExecDataFlags GetFlags(FCacheAddress Address)
		{
			if (Address.At == 0)
			{
				return EOpExecDataFlags::IsSet;
			}

			TScopeLock<FMutex> LockGuard(Mutex);

			EOpExecDataFlags Flags = GetFlagsImpl(Address);

			return Flags;
		}

		bool IsSet(FCacheAddress Address) 
		{
			if (Address.At == 0)
			{
				return true;
			}

			TScopeLock<FMutex> LockGuard(Mutex);

			EOpExecDataFlags Flags = GetFlagsImpl(Address);

			return EnumHasAnyFlags(Flags, EOpExecDataFlags::IsSet);
		}

		void UpdateAddressState(FCacheAddress Address, FAddressStateUpdate Update)
		{
			if (Address.At == 0)
			{
				return;
			}

			TScopeLock<FMutex> LockGuard(Mutex);
			
			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex < 0)
			{
				FoundIndex = AllocateNew(Address);
			}

			EnumAddFlags(Data[FoundIndex].Flags, Update.AddedFlags);
			Data[FoundIndex].OpHitCount += Update.OpHitDifferential;
			check(Data[FoundIndex].OpHitCount >= 0);
		}

		bool AcquireCachedResultForScheduling(FCacheAddress Address)
		{
			if (Address.At == 0)
			{
				return false;
			}

			TScopeLock<FMutex> LockGuard(Mutex);
	
			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex < 0)
			{
				FoundIndex = AllocateNew(Address);
			}

			// If the op hit count is 0 and the result is not set then we need scheduling.
			bool bNeedsScheduling = Data[FoundIndex].OpHitCount == 0;
			
			if constexpr (!CIsWeakable<T>)
			{
				bNeedsScheduling &= !EnumHasAnyFlags(Data[FoundIndex].Flags, EOpExecDataFlags::IsSet);
			}
			else
			{
				if (bNeedsScheduling)
				{
					if (Data[FoundIndex].Payload.IsSet())
					{
						bNeedsScheduling = false;
					}
					else
					{
						Data[FoundIndex].Payload = Data[FoundIndex].Weak.Pin();
						Data[FoundIndex].Weak = {};
						Data[FoundIndex].StagedWeakCount = 0;
					
						bNeedsScheduling = !Data[FoundIndex].Payload.IsSet();
					}
				}
			}

			++Data[FoundIndex].OpHitCount;
			return bNeedsScheduling;
		}


		void Clear(FProgramCache::EClearFlags ClearFlags)
		{
			TScopeLock<FMutex> LockGuard(Mutex);

			ClearImpl(ClearFlags);
		}

		bool AcquireCachedResult(FCacheAddress Address)
		{
			if (Address.At == 0)
			{
				return true;
			}

			TScopeLock<FMutex> LockGuard(Mutex);
	
			int32 FoundIndex = FindImpl(Address);

			if (FoundIndex < 0)
			{
				return false;
			}

			if constexpr (!CIsWeakable<T>)
			{
				return EnumHasAnyFlags(Data[FoundIndex].Flags, EOpExecDataFlags::IsSet);
			}
			else
			{
				if (Data[FoundIndex].Payload.IsSet())
				{
					return true;
				}
				
				Data[FoundIndex].Payload = Data[FoundIndex].Weak.Pin();
				Data[FoundIndex].Weak = {};
				Data[FoundIndex].StagedWeakCount = 0;

				return Data[FoundIndex].Payload.IsSet();
			}
		}


		FProgramCache::EAcquireSetResult StageAcquireResult(FCacheAddress Address)
		{
			if (Address.At == 0)
			{
				return FProgramCache::EAcquireSetResult::Success;
			}

			TScopeLock<FMutex> LockGuard(Mutex);
			
			if constexpr (!CIsWeakable<T>)
			{
				EOpExecDataFlags Flags = GetFlagsImpl(Address);
				
				FProgramCache::EAcquireSetResult Result = FProgramCache::EAcquireSetResult::Success;	
				if (!EnumHasAnyFlags(Flags, EOpExecDataFlags::IsSet))
				{
					Result = FProgramCache::EAcquireSetResult::Failure;
				}

				if (EnumHasAnyFlags(Flags, EOpExecDataFlags::Aborted))
				{
					Result = FProgramCache::EAcquireSetResult::Abort;
				}

				return Result;
			}
			else
			{
				int32 FoundIndex = FindImpl(Address);
				
				if (FoundIndex < 0)
				{
					return FProgramCache::EAcquireSetResult::Failure;
				}

				if (Data[FoundIndex].Payload.IsSet())
				{
					return FProgramCache::EAcquireSetResult::Success;
				}
	
				Data[FoundIndex].Payload = Data[FoundIndex].Weak.Pin();

				bool bStaged = Data[FoundIndex].Payload.IsSet();	

				check(Data[FoundIndex].StagedWeakCount < std::numeric_limits<uint8>::max());
				Data[FoundIndex].StagedWeakCount += static_cast<uint8>(bStaged);
				Data[FoundIndex].Weak = {};
			
				FProgramCache::EAcquireSetResult Result = FProgramCache::EAcquireSetResult::Failure;
				if (bStaged)
				{
					Result = FProgramCache::EAcquireSetResult::Success;
				}

				if (EnumHasAnyFlags(Data[FoundIndex].Flags, EOpExecDataFlags::Aborted))
				{
					Result = FProgramCache::EAcquireSetResult::Abort;
				}

				return Result;
			}
		}

		void CommitAcquireResult(FCacheAddress Address)
		{
			if constexpr (!CIsWeakable<T>)
			{
				return;
			}
			else
			{
				TScopeLock<FMutex> LockGuard(Mutex);
					
				int32 FoundIndex = FindImpl(Address);
				
				if (FoundIndex < 0)
				{
					return;
				}

				Data[FoundIndex].StagedWeakCount = 0;
			}
		}

		void RollbackAcquireResult(FCacheAddress Address)
		{
			if constexpr (!CIsWeakable<T>)
			{
				return;
			}
			else
			{
				TScopeLock<FMutex> LockGuard(Mutex);

				int32 FoundIndex = FindImpl(Address);
				
				if (FoundIndex < 0)
				{
					return;
				}

				// The result has been acquired by someone else, bailout.
				if (Data[FoundIndex].StagedWeakCount == 0)
				{
					return;
				}

				if (--Data[FoundIndex].StagedWeakCount == 0)
				{
					Data[FoundIndex].Weak = Data[FoundIndex].Payload;
					Data[FoundIndex].Payload = {};
				}
			}
		}

		SSIZE_T TryFreeWeakMemory(SSIZE_T TargetBytesToFree)
		{
			if constexpr (!CIsWeakable<T>)
			{
				return 0;
			}
			else
			{
				SSIZE_T RemovedBytes = 0;
				
				int32 LocalLastIndex = -1;
				{
					TScopeLock<FMutex> LockGuard(Mutex);
					LocalLastIndex = LastIndex;
				}

				for (int32 I = LocalLastIndex; I >= 0; --I)
				{
					bool bCacheEntryWasRemoved = false;
					typename TOpExecData<T>::PayloadType LocalPayloadValue;
					{
						TScopeLock<FMutex> LockGuard(Mutex);

						// Now that the cache is locked, revisit the index to remove.
						// NOTE: Any entry added to the cache while the lock is not taken will be skipped.
						I = FMath::Min(I, LastIndex);
						if (I < 0)
						{
							break;
						}

						if ((!EnumHasAnyFlags(Data[I].Flags, EOpExecDataFlags::IsSet)) | (Data[I].OpHitCount > 0))
						{
							continue;
						}

						if (Data[I].Weak.IsValid())
						{
							check(!Data[I].Payload.IsSet());
							
							LocalPayloadValue = Data[I].Weak.Pin();

							// Remove weak reference so IsUniqueReference() give the correct result.
							Data[I].Weak = {};

							if (LocalPayloadValue.IsUniqueReference())
							{	
								Free(I);
								bCacheEntryWasRemoved = true;
							}
							else
							{
								// Restore the weak reference if the entry cannot be freed.
								Data[I].Weak = LocalPayloadValue;
							}
						} // LockGuard::Unlock
					}

					// The deallocation is delayed outside the lock to minimize contention.
					// bCacheEntryWasRemoved gaurantees no other thread can increase the OpHitCount
					// after the lock is released since the entry has been removed.
					//
					// NOTE: It can happen that the cache entry is removed but no data is freed, this
					// should be a rare case that does not affect the correctness of the operation. This
					// can only happen if multiple different addresses point to the same resource.
					if (bCacheEntryWasRemoved && LocalPayloadValue.IsValid())
					{
						SSIZE_T FreedBytes = LocalPayloadValue->GetDataSize();
						bool bObjectSuccesfullyDeleted = LocalPayloadValue.TryDeleteObject();

						RemovedBytes += FreedBytes * static_cast<SSIZE_T>(bObjectSuccesfullyDeleted);
					}
				}

				return RemovedBytes;
			}
		}


		SSIZE_T CountBytes() const 
		{
			TScopeLock<FMutex> LockGuard(Mutex);
			
			SSIZE_T Result = Data.GetAllocatedSize() + AddressEntries.GetAllocatedSize();

			if constexpr (CIsPointerWithExternalDataSizeQuery<T>)
			{
				//for (int32 I = Data.Num() - 1; I >= 0; --I)
				//{
				//	const TOpExecData<T>& ExecData = Data[I];

			
				//	ExecData.Payload = ExecData.Weak.Pin();
			

				//	if (ExecData.Payload && !ExecData.bAllocCounted)
				//	{
				//		Result += ExecData.Payload->GetDataSize();
				//	}

				//	if (!Data.Payload.IsUniqueReference())
				//	{
				//		for (int32 J = I - 1; J >= 0; --J)
				//		{
				//			if (Data[J].Payload == ExecData.Payload || )
				//			{
				//				Data[J].bAllocCounted = true;
				//			}
				//		}
				//	}
				//}

				//for (int32 I = Data.Num() - 1; I >= 0; --I)
				//{
				//	Data[I].bAllocCounted = false;
				//}
			}

			return Result;
		}

		bool CheckHitCountsCleared() const 
		{
			TScopeLock<FMutex> LockGuard(Mutex);
			
			for (const TOpExecData<T>& ExecData : Data)
			{
				if (ExecData.OpHitCount != 0)
				{
					return false;
				}	
			}

			return true;
		}

		void Lock()
		{
			Mutex.Lock();
		}

		void Unlock()
		{
			Mutex.Unlock();
		}
	};

	struct FRangesCache
	{	
		mutable FMutex Mutex;

		using FRangeEntry = TPair<int32, int32>;

		using FExecutionIndex = TArray<FRangeEntry, TInlineAllocator<2>>;
		
		// The zero-th element represents the null entry. Add and empty entry.
		TArray<FExecutionIndex, TInlineAllocator<1>> Storage = {{}};

		FRangesCache& operator=(const FRangesCache& Other) = delete;
		FRangesCache& operator=(FRangesCache&& Other) = delete;

		/** Set or add a value to the index */
		int32 GetOrAddModifiedExecutionIndex(int32 TemplateExecutionIndex, uint16 RangeIndex, int32 RangeValue)
		{
			TScopeLock<FMutex> LockGuard(Mutex);
			
			FExecutionIndex NewExecutionIndex;
			
			if (TemplateExecutionIndex > 0)
			{
				NewExecutionIndex = Storage[TemplateExecutionIndex];
			}

			FRangeEntry* FoundEntry = NewExecutionIndex.FindByPredicate([RangeIndex](const FRangeEntry& Entry)
			{
				return Entry.Key == RangeIndex;
			});

			if (!FoundEntry)
			{
				FoundEntry = &NewExecutionIndex.AddDefaulted_GetRef();
			}

			*FoundEntry = FRangeEntry { RangeIndex, RangeValue };

			int32 FoundExecutionIndexIndex = Storage.Find(NewExecutionIndex);

			if (FoundExecutionIndexIndex == INDEX_NONE) 
			{
				FoundExecutionIndexIndex = Storage.Add(MoveTemp(NewExecutionIndex));
			}

			return FoundExecutionIndexIndex;
		}

		/** Get the value of the index from the range index in the model. */
		int32 GetExecutionIndexValue(int32 ExecutionIndex, int32 RangeIndex) const
		{
			TScopeLock<FMutex> LockGuard(Mutex);

			check(Storage.IsValidIndex(ExecutionIndex));

			const FRangeEntry* FoundEntry = Storage[ExecutionIndex].FindByPredicate([RangeIndex](const FRangeEntry& Entry)
			{
				return Entry.Key == RangeIndex;
			});

			if (FoundEntry)
			{
				return FoundEntry->Value;
			}

			return 0;
		}
	};

	struct FWaitEvents
	{
		FMutex Mutex;
		TArray<Tasks::FTaskEvent> Events;

		Tasks::FTaskEvent& RegisterEvent(const TCHAR* DebugName)
		{
			TScopeLock<FMutex> LockGuard(Mutex);
			return Events.Emplace_GetRef(DebugName);
		}
		
		void TriggerEvents()
		{
			TScopeLock<FMutex> LockGuard(Mutex);
			for (Tasks::FTaskEvent& Event : Events)
			{
				Event.Trigger();
			}

			Events.Reset();
		}
	};

	struct FProgramCacheInternals
	{
		static inline const TManagedPtr<const FInstance> NullInstance = MakeManagedNull<const FInstance>();
		static inline const TManagedPtr<const String> NullString = MakeManagedNull<const String>();
		static inline const TManagedPtr<const FExtensionData> NullExtensionData = MakeManagedNull<const FExtensionData>();
		static inline const TManagedPtr<const FInstancedStruct> NullInstancedStruct = MakeManagedNull<const FInstancedStruct>();
		static inline const TManagedPtr<const FMaterial> NullMaterial = MakeManagedNull<const FMaterial>();
		static inline const TManagedPtr<const FLOD> NullSkeletalMeshLOD = MakeManagedNull<const FLOD>();
		static inline const TManagedPtr<const FSkeletalMesh> NullSkeletalMesh = MakeManagedNull<const FSkeletalMesh>();

		static inline const TManagedPtr<const FImage> NullImage = MakeManagedNull<const FImage>();
		static inline const TManagedPtr<const FMesh> NullMesh = MakeManagedNull<const FMesh>();
		static inline const TManagedPtr<const FLayout> NullLayout = MakeManagedNull<const FLayout>();

		TOpCacheData<bool> BoolCache;
		TOpCacheData<int32> IntegerCache;
		TOpCacheData<float> ScalarCache;

		TOpCacheData<TManagedPtr<const FInstance>> InstanceCache;
		TOpCacheData<TManagedPtr<const String>> StringCache;
		TOpCacheData<TManagedPtr<const FExtensionData>> ExtensionDataCache;
		TOpCacheData<TManagedPtr<const FInstancedStruct>> InstancedStructCache;
		TOpCacheData<TManagedPtr<const FMaterial>> MaterialCache;
		TOpCacheData<TManagedPtr<const FLOD>> LODCache;
		TOpCacheData<TManagedPtr<const FSkeletalMesh>> SkeletalMeshCache;

		TOpCacheData<TManagedPtr<const FLayout>> LayoutCache;
		TOpCacheData<TManagedPtr<const FImage>> ImageCache;
		TOpCacheData<TManagedPtr<const FMesh>> MeshCache;
		
		TOpCacheData<FVector4f> ColorCache;
		TOpCacheData<FProjector> ProjectorCache;
		TOpCacheData<FMatrix44f> MatrixCache;

		TOpCacheData<FExtendedImageDesc> ImageDescCache;
		
		TOpCacheData<FVoidType> VoidCache;

		FRangesCache RangesCache;
		
		FWaitEvents WaitEvents;
	};

	FTypeErasedOpCacheDataRef GetTypeErasedCacheForAddress(FProgramCacheInternals* Internals, FCacheAddress Address)
	{
		EDataType DataType = GetAddressDataType(Address.At);
		
		if (Address.Type == static_cast<uint16>(FScheduledOp::EType::Full))
		{
			switch (DataType)
			{
			case EDataType::Bool:
			{
				return FTypeErasedOpCacheDataRef(Internals->BoolCache);
			}
			case EDataType::Scalar:
			{
				return FTypeErasedOpCacheDataRef(Internals->ScalarCache);
			}
			case EDataType::Int:
			{
				return FTypeErasedOpCacheDataRef(Internals->IntegerCache);
			}
			case EDataType::Color:
			{
				return FTypeErasedOpCacheDataRef(Internals->ColorCache);
			}
			case EDataType::Matrix:
			{
				return FTypeErasedOpCacheDataRef(Internals->MatrixCache);
			}
			case EDataType::Projector:
			{
				return FTypeErasedOpCacheDataRef(Internals->ProjectorCache);
			}
			case EDataType::InstancedStruct:
			{
				return FTypeErasedOpCacheDataRef(Internals->InstancedStructCache);
			}
			case EDataType::Image:
			{
				return FTypeErasedOpCacheDataRef(Internals->ImageCache);
			}
			case EDataType::Material:
			{
				return FTypeErasedOpCacheDataRef(Internals->MaterialCache);
			}
			case EDataType::Mesh:
			{
				return FTypeErasedOpCacheDataRef(Internals->MeshCache);
			}
			case EDataType::SkeletalMesh:
			{
				return FTypeErasedOpCacheDataRef(Internals->SkeletalMeshCache);
			}
			case EDataType::Layout:
			{
				return FTypeErasedOpCacheDataRef(Internals->LayoutCache);
			}
			case EDataType::String:
			{
				return FTypeErasedOpCacheDataRef(Internals->StringCache);
			}
			case EDataType::Instance:
			{
				return FTypeErasedOpCacheDataRef(Internals->InstanceCache);
			}
			case EDataType::ExtensionData:
			{
				return FTypeErasedOpCacheDataRef(Internals->ExtensionDataCache);
			}
			case EDataType::LOD:
			{
				return FTypeErasedOpCacheDataRef(Internals->LODCache);
			}
			default:
			{
				// The void cache will be used for the null address. All caches operations 
				// check for the null address and skip any state manipulation as the result
				// is already known.
				check(Address.At == 0);
				return FTypeErasedOpCacheDataRef(Internals->VoidCache);
			}
			}
		}
		else if (Address.Type == static_cast<uint16>(FScheduledOp::EType::ImageDesc))
		{
			return FTypeErasedOpCacheDataRef(Internals->ImageDescCache);
		}
		else
		{
			return FTypeErasedOpCacheDataRef(Internals->VoidCache);
		}
	}

} // namespace ProgramCacheInternals

	FProgramCache::FProgramCache()
	{
		using namespace ProgramCacheInternals;

		// Make the storage requirements comparison less_than so if different platforms result in 
		// different storage requirements we can always use the largest. 
		static_assert(FMath::IsPowerOfTwo(FProgramCache::InternalsAlign));
		static_assert(alignof(FProgramCacheInternals) <= FProgramCache::InternalsAlign);
		static_assert(sizeof(FProgramCacheInternals) <= FProgramCache::InternalsSize);

		FProgramCacheInternals* Internals = new(InternalsStorage.Data) FProgramCacheInternals;
		check(GetInternals() == Internals);
	}

	FProgramCache::~FProgramCache()
	{
		// Destroy the internal object, no need to free the memory as it belongs to this object.
		GetInternals()->~FProgramCacheInternals();
	}

	ProgramCacheInternals::FProgramCacheInternals* FProgramCache::GetInternals() const
	{
		using namespace ProgramCacheInternals;
		return (FProgramCacheInternals*)InternalsStorage.Data;
	}

	void FProgramCache::LockAddress(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		constexpr FAddressStateUpdate Update
		{
			.AddedFlags        = EOpExecDataFlags::IsLocked,
			.OpHitDifferential = 0,
		};

		GetTypeErasedCacheForAddress(GetInternals(), Address).UpdateAddressState(Address, Update);
	}

	void FProgramCache::MarkAddressToKeep(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		constexpr FAddressStateUpdate Update
		{
			.AddedFlags        = EOpExecDataFlags::IsMarkedToKeep,
			.OpHitDifferential = 0,
		};

		GetTypeErasedCacheForAddress(GetInternals(), Address).UpdateAddressState(Address, Update);
	}

	void FProgramCache::SetAborted(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		constexpr FAddressStateUpdate Update
		{
			.AddedFlags        = EOpExecDataFlags::Aborted | EOpExecDataFlags::IsSet,
			.OpHitDifferential = 0,
		};

		GetTypeErasedCacheForAddress(GetInternals(), Address).UpdateAddressState(Address, Update);
	}

	void FProgramCache::UpdateHitCount(FCacheAddress Address, int8 Differential)
	{
		using namespace ProgramCacheInternals;
		FAddressStateUpdate Update
		{
			.AddedFlags        = EOpExecDataFlags::None,
			.OpHitDifferential = Differential,
		};

		GetTypeErasedCacheForAddress(GetInternals(), Address).UpdateAddressState(Address, Update);
	}

	bool FProgramCache::IsSet(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		EOpExecDataFlags AddressFlags = GetTypeErasedCacheForAddress(GetInternals(), Address).GetFlags(Address);

		return EnumHasAnyFlags(AddressFlags, EOpExecDataFlags::IsSet);
	}

	bool FProgramCache::IsAborted(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		EOpExecDataFlags AddressFlags = GetTypeErasedCacheForAddress(GetInternals(), Address).GetFlags(Address);

		return EnumHasAnyFlags(AddressFlags, EOpExecDataFlags::Aborted);
	}

	void FProgramCache::StoreBool(FCacheAddress Address, bool Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Bool);
		
		bool bSuccess = GetInternals()->BoolCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreScalar(FCacheAddress Address, float Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Scalar);
		
		bool bSuccess = GetInternals()->ScalarCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreInt(FCacheAddress Address, int32 Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Int);
		
		bool bSuccess = GetInternals()->IntegerCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreColor(FCacheAddress Address, const FVector4f& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Color);
		
		bool bSuccess = GetInternals()->ColorCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreMatrix(FCacheAddress Address, const FMatrix44f& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Matrix);

		bool bSuccess = GetInternals()->MatrixCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreProjector(FCacheAddress Address, const FProjector& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Projector);
		
		bool bSuccess = GetInternals()->ProjectorCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreInstancedStruct(FCacheAddress Address, const TManagedPtr<const FInstancedStruct>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::InstancedStruct);

		bool bSuccess = GetInternals()->InstancedStructCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullInstancedStruct);
		check(bSuccess);
	}

	void FProgramCache::StoreMaterial(FCacheAddress Address, const TManagedPtr<const FMaterial>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Material);
		
		bool bSuccess = GetInternals()->MaterialCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullMaterial);
		check(bSuccess);
	}

	void FProgramCache::StoreImage(FCacheAddress Address, const TManagedPtr<const FImage>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Image);

		bool bSuccess = GetInternals()->ImageCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullImage);
		check(bSuccess);
	}

	void FProgramCache::StoreMesh(FCacheAddress Address, const TManagedPtr<const FMesh>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Mesh);

		bool bSuccess = GetInternals()->MeshCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullMesh);
		check(bSuccess);
	}
	
	void FProgramCache::StoreLOD(FCacheAddress Address, const TManagedPtr<const FLOD>& Value)
    {
    	check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::LOD);

    	bool bSuccess = GetInternals()->LODCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullSkeletalMeshLOD);
    	check(bSuccess);
    }

	void FProgramCache::StoreSkeletalMesh(FCacheAddress Address, const TManagedPtr<const FSkeletalMesh>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::SkeletalMesh);

		bool bSuccess = GetInternals()->SkeletalMeshCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullSkeletalMesh);
		check(bSuccess);
	}

	void FProgramCache::StoreLayout(FCacheAddress Address, const TManagedPtr<const FLayout>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Layout);
		
		bool bSuccess = GetInternals()->LayoutCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullLayout);
		check(bSuccess);
	}

	void FProgramCache::StoreString(FCacheAddress Address, const TManagedPtr<const String>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::String);

		bool bSuccess = GetInternals()->StringCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullString);
		check(bSuccess);
	}

	void FProgramCache::StoreInstance(FCacheAddress Address, const TManagedPtr<const FInstance>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Instance);

		bool bSuccess = GetInternals()->InstanceCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullInstance);
		check(bSuccess);
	}

	void FProgramCache::StoreExtensionData(FCacheAddress Address, const TManagedPtr<const FExtensionData>& Value)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::ExtensionData);

		bool bSuccess = GetInternals()->ExtensionDataCache.SetValue(Address, Value.IsSet() ? Value : ProgramCacheInternals::FProgramCacheInternals::NullExtensionData);
		check(bSuccess);
	}

	void FProgramCache::StoreImageDesc(FCacheAddress Address, const FExtendedImageDesc& Value)
	{
		//check(GetAddressDataType(Address.At) == EDataType::Image);
		check(Address.Type == static_cast<uint16>(FScheduledOp::EType::ImageDesc));

		bool bSuccess = GetInternals()->ImageDescCache.SetValue(Address, Value);
		check(bSuccess);
	}

	void FProgramCache::StoreVoid(FCacheAddress Address)
	{
		check(Address.Type != static_cast<uint16>(FScheduledOp::EType::Full));

		bool bSuccess = GetInternals()->VoidCache.SetValue(Address, ProgramCacheInternals::FVoidType{});
		check(bSuccess);
	}

	bool FProgramCache::GetBoolIfSet(FCacheAddress Address, bool& OutResult)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Bool);

		OutResult = false;
		return GetInternals()->BoolCache.GetValue(Address, OutResult, false);
	}

	bool FProgramCache::GetMaterialIfSet(FCacheAddress Address, TManagedPtr<const FMaterial>& OutValue)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Material);

		OutValue = nullptr;
		return GetInternals()->MaterialCache.GetValue(Address, OutValue, false);
	}

	bool FProgramCache::LoadBool(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Bool);

		bool Value = false;
		GetInternals()->BoolCache.GetValue(Address, Value);

		return Value;
	}

	float FProgramCache::LoadScalar(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Scalar);

		float Value = 0.0f;
		GetInternals()->ScalarCache.GetValue(Address, Value);

		return Value;
	}

	int32 FProgramCache::LoadInt(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Int);
		
		int32 Value = 0;
		GetInternals()->IntegerCache.GetValue(Address, Value);

		return Value;
	}

	FVector4f FProgramCache::LoadColor(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Color);

		FVector4f Value = FVector4f{};
		GetInternals()->ColorCache.GetValue(Address, Value);

		return Value;
	}

	FMatrix44f FProgramCache::LoadMatrix(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Matrix);

		FMatrix44f Value = FMatrix44f::Identity;
		GetInternals()->MatrixCache.GetValue(Address, Value);
		
		return Value;
	}

	FProjector FProgramCache::LoadProjector(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Projector);
		
		FProjector Value = FProjector{};
		GetInternals()->ProjectorCache.GetValue(Address, Value);

		return Value;
	}

	TManagedPtr<const FMaterial> FProgramCache::LoadMaterial(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Material);

		TManagedPtr<const FMaterial> Value;
		GetInternals()->MaterialCache.GetValue(Address, Value);

		return Value;
	}

	TManagedPtr<const FInstancedStruct> FProgramCache::LoadInstancedStruct(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::InstancedStruct);

		TManagedPtr<const FInstancedStruct> Value;
		GetInternals()->InstancedStructCache.GetValue(Address, Value);

		return Value;
	}

	TManagedPtr<const FImage> FProgramCache::LoadImage(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Image);

		static_assert(ProgramCacheInternals::CIsWeakable<TManagedPtr<const FImage>>);

		TManagedPtr<const FImage> Value;
		GetInternals()->ImageCache.MoveValueIfPossible(Address, Value);

		return Value;
	}

	TManagedPtr<const FMesh> FProgramCache::LoadMesh(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Mesh);

		static_assert(ProgramCacheInternals::CIsWeakable<TManagedPtr<const FMesh>>);

		TManagedPtr<const FMesh> Value;
		GetInternals()->MeshCache.MoveValueIfPossible(Address, Value);

		return Value;
	}

	TManagedPtr<const FLOD> FProgramCache::LoadLOD(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::LOD);

		TManagedPtr<const FLOD> Value;
		GetInternals()->LODCache.GetValue(Address, Value);

		return Value;
	}
	
	TManagedPtr<const FSkeletalMesh> FProgramCache::LoadSkeletalMesh(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::SkeletalMesh);

		TManagedPtr<const FSkeletalMesh> Value;
		GetInternals()->SkeletalMeshCache.MoveValueIfPossible(Address, Value);

		return Value;
	}

	TManagedPtr<const FLayout> FProgramCache::LoadLayout(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Layout);

		TManagedPtr<const FLayout> Value;
		GetInternals()->LayoutCache.GetValue(Address, Value);

		return Value;
	}

	TManagedPtr<const String> FProgramCache::LoadString(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::String);

		TManagedPtr<const String> Value;
		GetInternals()->StringCache.GetValue(Address, Value);

		return Value;
	}

	TManagedPtr<const FInstance> FProgramCache::LoadInstance(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::Instance);

		TManagedPtr<const FInstance> Value;
		GetInternals()->InstanceCache.GetValue(Address, Value);

		return Value;
	}

	TManagedPtr<const FExtensionData> FProgramCache::LoadExtensionData(FCacheAddress Address)
	{
		check(Address.At == 0 || GetAddressDataType(Address.At) == EDataType::ExtensionData);

		TManagedPtr<const FExtensionData> Value;
		GetInternals()->ExtensionDataCache.GetValue(Address, Value);

		return Value;
	}

	FExtendedImageDesc FProgramCache::LoadImageDesc(FCacheAddress Address)
	{
		//checkSlow(GetAddressDataType(Address.At) == EDataType::Image);
		check(Address.Type == static_cast<uint16>(FScheduledOp::EType::ImageDesc));
		
		FExtendedImageDesc Value;
		GetInternals()->ImageDescCache.GetValue(Address, Value);

		return Value;
	}

	void FProgramCache::LoadVoid(FCacheAddress Address)
	{
		check(Address.Type !=  static_cast<uint16>(FScheduledOp::EType::Full));

		ProgramCacheInternals::FVoidType Result;
		GetInternals()->VoidCache.GetValue(Address, Result);
	}

	void FProgramCache::Clear(EClearFlags ClearFlags)
	{
		GetInternals()->BoolCache.Clear(ClearFlags);
		GetInternals()->IntegerCache.Clear(ClearFlags);
		GetInternals()->ScalarCache.Clear(ClearFlags);

		GetInternals()->LayoutCache.Clear(ClearFlags);
		GetInternals()->InstanceCache.Clear(ClearFlags);
		GetInternals()->StringCache.Clear(ClearFlags);
		GetInternals()->ExtensionDataCache.Clear(ClearFlags);
		GetInternals()->InstancedStructCache.Clear(ClearFlags);
		GetInternals()->MaterialCache.Clear(ClearFlags);
		GetInternals()->LODCache.Clear(ClearFlags);
		GetInternals()->SkeletalMeshCache.Clear(ClearFlags);

		GetInternals()->ImageCache.Clear(ClearFlags);
		GetInternals()->MeshCache.Clear(ClearFlags);
				
		GetInternals()->ColorCache.Clear(ClearFlags);
		GetInternals()->ProjectorCache.Clear(ClearFlags);
		GetInternals()->MatrixCache.Clear(ClearFlags);

		EClearFlags FullClearFlags = 
				EClearFlags::Locked | EClearFlags::Unlocked | (ClearFlags & EClearFlags::FreeMemory);

		if (EnumHasAnyFlags(ClearFlags, EClearFlags::ImageDesc))
		{
			GetInternals()->ImageDescCache.Clear(FullClearFlags);
		}

		if (EnumHasAnyFlags(ClearFlags, EClearFlags::Void))
		{
			GetInternals()->VoidCache.Clear(FullClearFlags);
		}
	}

	FProgramCache::EAcquireSetResult FProgramCache::TryAcquireCachedResultSet(TConstArrayView<FCacheAddress> AddressSet)
	{
		if (AddressSet.IsEmpty())
		{
			return EAcquireSetResult::Success;
		}

		EAcquireSetResult Result = EAcquireSetResult::Success;

		int32 CurrentIndex = 0;
		for (; CurrentIndex < AddressSet.Num(); ++CurrentIndex)
		{
			EAcquireSetResult AddressResult = GetTypeErasedCacheForAddress(GetInternals(), AddressSet[CurrentIndex]).StageAcquireResult(AddressSet[CurrentIndex]);
			if (AddressResult != EAcquireSetResult::Success)
			{
				Result = AddressResult;
				break;
			}
		}

		if (Result == EAcquireSetResult::Success)
		{
			--CurrentIndex;
			for (; CurrentIndex >= 0; --CurrentIndex)
			{
				GetTypeErasedCacheForAddress(GetInternals(), AddressSet[CurrentIndex]).CommitAcquireResult(AddressSet[CurrentIndex]);
			}
		}
		else
		{
			for (; CurrentIndex >= 0; --CurrentIndex)
			{
				GetTypeErasedCacheForAddress(GetInternals(), AddressSet[CurrentIndex]).RollbackAcquireResult(AddressSet[CurrentIndex]);
			}
		}

		return Result;
	}

	bool FProgramCache::TryAcquireCachedResult(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		return GetTypeErasedCacheForAddress(GetInternals(), Address).AcquireCachedResult(Address);
	}

	bool FProgramCache::AcquireCachedResultForScheduling(FCacheAddress Address)
	{
		using namespace ProgramCacheInternals;
		return GetTypeErasedCacheForAddress(GetInternals(), Address).AcquireCachedResultForScheduling(Address);
	}

	SSIZE_T FProgramCache::TryFreeWeakMemory(SSIZE_T TargetBytesToFree)
	{
		SSIZE_T OriginalTarget = TargetBytesToFree;

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->ImageCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->MeshCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->LayoutCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->InstanceCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->StringCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->ExtensionDataCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->InstancedStructCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->MaterialCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		if (TargetBytesToFree > 0)
		{
			TargetBytesToFree -= GetInternals()->SkeletalMeshCache.TryFreeWeakMemory(TargetBytesToFree);
		}

		return TargetBytesToFree - OriginalTarget; 
	}

	SSIZE_T FProgramCache::CountBytes() const
	{
		SSIZE_T Result = 0;

		Result += GetInternals()->BoolCache.CountBytes();
		Result += GetInternals()->IntegerCache.CountBytes();
		Result += GetInternals()->ScalarCache.CountBytes();
		Result += GetInternals()->ColorCache.CountBytes();
		Result += GetInternals()->ProjectorCache.CountBytes();
		Result += GetInternals()->MatrixCache.CountBytes();

		Result += GetInternals()->LayoutCache.CountBytes();
		Result += GetInternals()->InstanceCache.CountBytes();
		Result += GetInternals()->StringCache.CountBytes();
		Result += GetInternals()->ExtensionDataCache.CountBytes();
		Result += GetInternals()->InstancedStructCache.CountBytes();
		Result += GetInternals()->MaterialCache.CountBytes();
		Result += GetInternals()->SkeletalMeshCache.CountBytes();

		Result += GetInternals()->ImageCache.CountBytes();
		Result += GetInternals()->MeshCache.CountBytes();

		Result += GetInternals()->ImageDescCache.CountBytes();
		Result += GetInternals()->VoidCache.CountBytes();

		return Result;
	}

	bool FProgramCache::CheckHitCountsCleared() const
	{
		bool Result = true;

		Result = Result && GetInternals()->BoolCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->IntegerCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->ScalarCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->ColorCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->ProjectorCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->MatrixCache.CheckHitCountsCleared();

		Result = Result && GetInternals()->LayoutCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->InstanceCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->StringCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->ExtensionDataCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->InstancedStructCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->MaterialCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->SkeletalMeshCache.CheckHitCountsCleared();

		Result = Result && GetInternals()->ImageCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->MeshCache.CheckHitCountsCleared();

		Result = Result && GetInternals()->ImageDescCache.CheckHitCountsCleared();
		Result = Result && GetInternals()->VoidCache.CheckHitCountsCleared();

		return Result;
	}

	int32 FProgramCache::GetOrAddModifiedExecutionIndex(int32 TemplateExecutionIndex, uint16 RangeIndex, int32 RangeValue)
	{
		return GetInternals()->RangesCache.GetOrAddModifiedExecutionIndex(TemplateExecutionIndex, RangeIndex, RangeValue);
	}

	int32 FProgramCache::GetExecutionIndexValue(int32 ExecutionIndex, uint16 RangeIndex) const
	{
		return GetInternals()->RangesCache.GetExecutionIndexValue(ExecutionIndex, RangeIndex);
	}

	Tasks::FTaskEvent& FProgramCache::RegisterWaitEvent(const TCHAR* DebugName)
	{
		return GetInternals()->WaitEvents.RegisterEvent(DebugName);
	}

	void FProgramCache::TriggerWaitEvents()
	{
		GetInternals()->WaitEvents.TriggerEvents();
	}
}


