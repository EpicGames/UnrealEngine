// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/ResourceID.h"

#include "Hash/xxhash.h"


uint32 UE::Mutable::Private::GetTypeHash(const FGeneratedImageKey& Key)
{
	return HashCombineFast(Key.Address, static_cast<uint32>(FXxHash64::HashBuffer(Key.ParameterValuesBlob.GetData(), Key.ParameterValuesBlob.Num()).Hash));
}


uint32 UE::Mutable::Private::GetTypeHashPersistent(const FImageId& Id)
{
	if (const FGeneratedImageKey* Key = Id.GetKey())
	{
		return HashCombine(Key->Address, static_cast<uint32>(FXxHash64::HashBuffer(Key->ParameterValuesBlob.GetData(), Key->ParameterValuesBlob.Num()).Hash));
	}
	else
	{
		return 0;
	}
}


uint32 UE::Mutable::Private::GetTypeHash(const FGeneratedMaterialKey& Key)
{
	return HashCombineFast(Key.Address, static_cast<uint32>(FXxHash64::HashBuffer(Key.ParameterValuesBlob.GetData(), Key.ParameterValuesBlob.Num()).Hash));
}


uint32 UE::Mutable::Private::GetTypeHashPersistent(const FMaterialId& Id)
{
	if (const FGeneratedMaterialKey* Key = Id.GetKey())
	{
		return HashCombine(Key->Address, static_cast<uint32>(FXxHash64::HashBuffer(Key->ParameterValuesBlob.GetData(), Key->ParameterValuesBlob.Num()).Hash));
	}
	else
	{
		return 0;
	}
}


uint32 UE::Mutable::Private::GetTypeHash(const FGeneratedSkeletalMeshKey& Key)
{
	return HashCombineFast(Key.Address, static_cast<uint32>(FXxHash64::HashBuffer(Key.ParameterValuesBlob.GetData(), Key.ParameterValuesBlob.Num()).Hash));
}

uint32 UE::Mutable::Private::GetTypeHashPersistent(const FSkeletalMeshId& Id)
{
	if (const FGeneratedSkeletalMeshKey* Key = Id.GetKey())
	{
		return HashCombine(Key->Address, static_cast<uint32>(FXxHash64::HashBuffer(Key->ParameterValuesBlob.GetData(), Key->ParameterValuesBlob.Num()).Hash));
	}
	else
	{
		return 0;
	}
}
