// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Serialisation.h"

#include "SkeletalMeshTypes.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Hash/CityHash.h"
#include "MuR/Image.h"
#include "MuR/Serialisation.h"


namespace UE::Mutable::Private
{
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(float);    
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(double);   
                                                  
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( int8 );   
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( int16 );  
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( int32 );  
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( int64 );  
                                                  
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( uint8 );  
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( uint16 ); 
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( uint32 ); 
    MUTABLE_IMPLEMENT_POD_SERIALISABLE( uint64 )
	
	// Unreal POD Serializables                                                       
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FGuid);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FUintVector2);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FIntVector2);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(UE::Math::TIntVector2<uint16>);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(UE::Math::TIntVector2<int16>);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FVector2f);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FVector4f);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FMatrix44f);
	MUTABLE_IMPLEMENT_POD_SERIALISABLE(FRichCurveKey);

	
    void operator<<(FOutputArchive& arch, const FString& t)
    {
	    const TArray<TCHAR>& Data = t.GetCharArray();
    	arch << Data;
    }


    void operator>>(FInputArchive& arch, FString& t)
    {
		TArray<TCHAR> Data;
    	arch >> Data;

    	t = FString(Data.GetData()); // Construct from raw pointer to avoid double zero terminating character
    }


	void operator<<(FOutputArchive& arch, const FRichCurve& t)
	{
		arch << t.Keys;
	}


	void operator>>(FInputArchive& arch, FRichCurve& t)
	{
		arch >> t.Keys;
	}


    void operator<<(FOutputArchive& arch, const FName& v)
    {
	    arch << v.ToString();
    }


    void operator>>(FInputArchive& arch, FName& v)
    {
	    FString Temp;
	    arch >> Temp;
	    v = FName(Temp);
    }


    void operator<<(FOutputArchive& Ar, const FMeshToMeshVertData& Value)
    {
    	Ar << Value.PositionBaryCoordsAndDist; 
    	Ar << Value.NormalBaryCoordsAndDist;
    	Ar << Value.TangentBaryCoordsAndDist;
    	Ar << Value.SourceMeshVertIndices[0];
    	Ar << Value.SourceMeshVertIndices[1];
    	Ar << Value.SourceMeshVertIndices[2];
    	Ar << Value.SourceMeshVertIndices[3];
    	Ar << Value.Weight;
    }


    void operator>>(FInputArchive& Ar, FMeshToMeshVertData& Value)
    {
    	Ar >> Value.PositionBaryCoordsAndDist; 
    	Ar >> Value.NormalBaryCoordsAndDist;
    	Ar >> Value.TangentBaryCoordsAndDist;
    	Ar >> Value.SourceMeshVertIndices[0];
    	Ar >> Value.SourceMeshVertIndices[1];
    	Ar >> Value.SourceMeshVertIndices[2];
    	Ar >> Value.SourceMeshVertIndices[3];
    	Ar >> Value.Weight;
    }


    void operator<<(FOutputArchive& Arch, const bool& T)
    {
    	uint8 S = T ? 1 : 0;
    	Arch.Stream->Write(&S, sizeof(uint8));
    }


	void operator>>(FInputArchive& Arch, bool& T)
    {
    	uint8 S;
    	Arch.Stream->Read(&S, sizeof(uint8));
    	T = S != 0;
    }


	void operator<<(FOutputArchive& Arch, const TManagedPtr<const FInstancedStruct>& V)
	{
		// Do Nothing. UStructs values are saved in the CO.	
	}


	void operator>>(FInputArchive& Arch, TManagedPtr<const FInstancedStruct>& V)
	{
		// Do Nothing. UStructs values are saved in the CO.	
	}


	FInputMemoryStream::FInputMemoryStream( const void* InBuffer, uint64 InSize )
    {
        Buffer = InBuffer;
        Size = InSize;
    }


    void FInputMemoryStream::Read( void* Data, uint64 InSize)
    {
        if (InSize)
        {
            check( Pos + InSize <= Size );

            const uint8* Source = reinterpret_cast<const uint8*>(Buffer)+Pos;
            FMemory::Memcpy( Data, Source, (SIZE_T)InSize);
            Pos += InSize;
        }
    }


    FOutputMemoryStream::FOutputMemoryStream(uint64 Reserve )
    {
        if (Reserve)
        {
             Buffer.Reserve( Reserve );
        }
    }


    void FOutputMemoryStream::Write( const void* Data, uint64 Size)
    {
        if (Size)
        {
            uint64 Pos = Buffer.Num();
			Buffer.SetNum( Pos + Size, EAllowShrinking::No );
			FMemory::Memcpy( Buffer.GetData()+Pos, Data, Size);
        }
    }


    const uint8* FOutputMemoryStream::GetBuffer() const
    {
        const uint8* Result = nullptr;

        if (Buffer.Num() )
        {
            Result = Buffer.GetData();
        }

        return Result;
    }


	int32 FOutputMemoryStream::GetBufferSize() const
    {
        return Buffer.Num();
    }


	void FOutputMemoryStream::Reset()
	{
		Buffer.SetNum(0,EAllowShrinking::No);
	}


	void FOutputSizeStream::Write( const void*, uint64 size )
    {
        WrittenBytes += size;
    }


	uint64 FOutputSizeStream::GetBufferSize() const
    {
        return WrittenBytes;
    }


	void FOutputHashStream::Write(const void* Data, uint64 size)
	{
		Hash = CityHash64WithSeed(reinterpret_cast<const char*>(Data), size, Hash);
	}


	uint64 FOutputHashStream::GetHash() const
	{
		return Hash;
	}


 	FInputArchive::FInputArchive( FInputStream* InStream )
	{
		Stream = InStream;
	}


	FOutputArchive::FOutputArchive( FOutputStream* InStream )
	{
		Stream = InStream;
	}

}

