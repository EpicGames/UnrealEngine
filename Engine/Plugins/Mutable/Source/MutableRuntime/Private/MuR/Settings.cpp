// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/Settings.h"

namespace UE::Mutable::Private
{

    void FSettings::SetProfile( bool bEnabled )
    {
        bProfile = bEnabled;
    }


    void FSettings::SetWorkingMemoryBytes( uint64 Bytes )
    {
        WorkingMemoryBytes = Bytes;
    }


    void FSettings::SetImageCompressionQuality( int32 Quality )
    {
        ImageCompressionQuality = Quality;
    }

}
