// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuR/Skeleton.h"

#include "MuR/Serialisation.h"


namespace UE::Mutable::Private
{

	void FSkeleton::Serialise(const FSkeleton* In, FOutputArchive& Arch)
	{
		Arch << *In;
	}


	TManagedPtr<FSkeleton> FSkeleton::StaticUnserialise(FInputArchive& Arch)
	{
		TManagedPtr<FSkeleton> Result = MakeManaged<FSkeleton>();
		Arch >> *Result;
		return Result;
	}


	TManagedPtr<FSkeleton> FSkeleton::Clone() const
	{
		TManagedPtr<FSkeleton> Result = MakeManaged<FSkeleton>();

		Result->BoneNames = BoneNames;
		//Result->BoneIds = BoneIds; // Runtime operations work with FNames
		Result->BoneParents = BoneParents;

		return Result;
	}


	int32 FSkeleton::AddBone(const FName InBoneName, const int16 InParentIndex)
	{
		BoneNames.Add(InBoneName);
		return BoneParents.Add(InParentIndex);
	}

	int32 FSkeleton::FindOrAddBone(const FName InBoneName, const int16 InParentIndex)
	{
		int32 BoneIndex = BoneNames.Find(InBoneName);
		if (BoneIndex != INDEX_NONE)
		{
			return BoneIndex;
		}

		return AddBone(InBoneName, InParentIndex);
	}

	int32 FSkeleton::GetNumBones() const
	{
		return !BoneNames.IsEmpty() ? BoneNames.Num() : BoneIds.Num();
	}


	void FSkeleton::Serialise(FOutputArchive& Arch) const
	{
		//Arch << BoneNames; Serialized as ConstantNames
		Arch << BoneIds;
		Arch << BoneParents;
	}


	void FSkeleton::Unserialise(FInputArchive& Arch)
	{
		//Arch >> BoneNames; Serialized as ConstantNames
		Arch >> BoneIds;
		Arch >> BoneParents;
	}
}
