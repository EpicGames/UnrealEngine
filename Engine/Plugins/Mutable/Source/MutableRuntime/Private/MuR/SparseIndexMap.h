// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StaticArray.h"

#include "Templates/UniquePtr.h"

#include "Math/NumericLimits.h"

namespace UE::Mutable::Private
{

    /** 
	 * Very basic sparse index map where the i-th element contains the mapped index.
     * Only indices in the range [ MinIndex, MaxIndex ] can be accessed.
     *
     **/
    class SparseIndexMap
    {
    public: 
        static constexpr uint32 NotFoundValue = TNumericLimits<uint32>::Max();

    private:
        static constexpr uint32 Log2BlockSizeBytes = 12; 
        static constexpr uint32 Log2IndexSizeBytes = 2; 
        
		static_assert( (1 << Log2IndexSizeBytes) == sizeof(uint32) );
        static_assert( Log2BlockSizeBytes > Log2IndexSizeBytes );

        static constexpr uint32 Log2BlockElems = Log2BlockSizeBytes - Log2IndexSizeBytes;  
     
        const uint32 MinIndex;
        const uint32 MaxIndex;

        using BlockType = TStaticArray<uint32, 1 << Log2BlockElems>;
        
        TArray< TUniquePtr<BlockType> > Blocks;
        
    public:

        SparseIndexMap( const uint32 InMinIndex, const uint32 InMaxIndex )
            : MinIndex( InMinIndex )
            , MaxIndex( InMaxIndex )
        {
            check( InMinIndex <= InMaxIndex );

            const uint32 BlockCount = InMinIndex > InMaxIndex 
                                    ? 0 
                                    : ( (InMaxIndex - InMinIndex + 1) >> Log2BlockElems ) + 1;

            Blocks.SetNum( BlockCount ); 
        }


        bool Insert( const uint32 KeyIndex, const uint32 ValueIndex )
        {
            if ( !(KeyIndex >= MinIndex && KeyIndex <= MaxIndex) )
            {
                return false;
            }

            const uint32 MappedIndex = KeyIndex - MinIndex;

            TUniquePtr< BlockType >& BlockPtr = Blocks[MappedIndex >> Log2BlockElems];
            
            if ( !BlockPtr )
            {
                BlockPtr = MakeUnique<BlockType>(  
                            MakeUniformStaticArray<uint32, 1 << Log2BlockElems>( NotFoundValue ) );
            }

            (*BlockPtr)[ MappedIndex & ( ( 1 << Log2BlockElems ) - 1 ) ] = ValueIndex;

            return true;
        }

        uint32 Find( const uint32 KeyIndex ) const
        {
            if ( !(KeyIndex >= MinIndex && KeyIndex <= MaxIndex) )
            {
                return NotFoundValue;
            }

            const uint32 MappedIndex = KeyIndex - MinIndex;
            const uint32 BlockIndex = MappedIndex >> Log2BlockElems;

            check( BlockIndex < uint32(Blocks.Num()) );

            const TUniquePtr<BlockType>& BlockPtr = Blocks[ BlockIndex ];

            if ( !BlockPtr )
            {
                return NotFoundValue;
            }

            return (*BlockPtr)[ MappedIndex & ( ( 1 << Log2BlockElems ) - 1 ) ];
        }

    };


	class SparseIndexMapSet
	{
	private:

		struct FRange
		{
			uint32 Prefix;
			SparseIndexMap Map;
		};

		TArray<FRange> Ranges;

	public:

		struct FRangeDesc
		{
			uint32 Prefix;
			uint32 MinIndex;
			uint32 MaxIndex;
		};

		SparseIndexMapSet(const TArray<FRangeDesc>& InRanges)
		{
			Ranges.Reserve(InRanges.Num());
			for ( const FRangeDesc& Desc: InRanges )
			{
				Ranges.Add({ Desc.Prefix, SparseIndexMap(Desc.MinIndex,Desc.MaxIndex)});
			}
		}

		bool Insert(const uint64 KeyIndex, const uint32 ValueIndex)
		{
			uint32 Prefix = KeyIndex >> 32;
			uint32 KeyValue = KeyIndex & 0xffffffff;
			for (FRange& Range : Ranges)
			{
				if (Range.Prefix == Prefix)
				{
					return Range.Map.Insert(KeyValue, ValueIndex);
				}
			}

			ensure(false);
			return false;
		}

		uint32 Find(const uint64 KeyIndex) const
		{
			uint32 Prefix = KeyIndex >> 32;
			uint32 KeyValue = KeyIndex & 0xffffffff;
			for (const FRange& Range : Ranges)
			{
				if (Range.Prefix == Prefix)
				{
					return Range.Map.Find(KeyValue);
				}
			}

			check(false);
			return SparseIndexMap::NotFoundValue;
		}

	};
}
