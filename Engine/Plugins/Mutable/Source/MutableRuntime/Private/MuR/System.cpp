// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuR/System.h"

#include "CodeRunnerBegin.h"
#include "BitWriterResourceKey.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Serialization/BitWriter.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "MuR/CodeRunner.h"
#include "MuR/CodeVisitor.h"
#include "MuR/Mesh.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableString.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Platform.h"
#include "MuR/Serialisation.h"
#include "MuR/MutableRuntimeModule.h"
#include "MuR/RomManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "PackedNormal.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(System)


DECLARE_STATS_GROUP(TEXT("MutableCore"), STATGROUP_MutableCore, STATCAT_Advanced);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Working Memory"), STAT_MutableWorkingMemory, STATGROUP_MutableCore);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Working Memory Excess"), STAT_MutableWorkingMemoryExcess, STATGROUP_MutableCore);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Current Memory"), STAT_MutableCurrentMemory, STATGROUP_MutableCore);

namespace 
{

bool bEnableDetailedMemoryBudgetExceededLogging = false;
static FAutoConsoleVariableRef CVarEnableDetailedMemoryBudgetExceededLogging (
	TEXT("mutable.EnableDetailedMemoryBudgetExceededLogging"),
	bEnableDetailedMemoryBudgetExceededLogging,
	TEXT("If set to true, enables a more detailed logging when memory budget is exceeded. Only for Debug and Development builds."),
	ECVF_Default);
}

namespace UE::Mutable::Private::MemoryCounters
{
	std::atomic<SSIZE_T>& FInternalMemoryCounter::Get()
	{
		static std::atomic<SSIZE_T> Counter{0};
		return Counter;
	}
}

namespace UE::Mutable::Private
{
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(ETextureCompressionStrategy);

	TRACE_DECLARE_INT_COUNTER(MutableRuntime_LiveInstances,		TEXT("MutableRuntime/LiveInstances"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_Updates,			TEXT("MutableRuntime/Updates"));

	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemInternal,	TEXT("MutableRuntime/MemInternal"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemTemp,		TEXT("MutableRuntime/MemTemp"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemPool,		TEXT("MutableRuntime/MemPool"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemCache,		TEXT("MutableRuntime/MemCache"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemRom,		TEXT("MutableRuntime/MemRom"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemImage,		TEXT("MutableRuntime/MemImage"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemMesh,		TEXT("MutableRuntime/MemMesh"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemStream,		TEXT("MutableRuntime/MemStream"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemTotal,		TEXT("MutableRuntime/MemTotal"));
	TRACE_DECLARE_INT_COUNTER(MutableRuntime_MemBudget,		TEXT("MutableRuntime/MemBudget"));


	std::atomic<uint32> FSystem::RomTick = 0;

	
    FSystem::FSystem(const FSettings& InSettings)
    {
    	LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));

    	Settings = InSettings;
	
    	WorkingMemoryManager.BudgetBytes = Settings.WorkingMemoryBytes;

    	UpdateStats();
    }

   
    void FSystem::SetStreamingInterface(const TSharedPtr<FModelReader>& InInterface)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
       
        StreamInterface = InInterface;
    }


	void FSystem::SetWorkingMemoryBytes(uint64 InBytes)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SetWorkingMemoryBytes);

		WorkingMemoryManager.BudgetBytes = InBytes;
		WorkingMemoryManager.EnsureBudgetBelow(0, nullptr);

		UpdateStats();
	}
	
   
    TSharedRef<FLiveInstance> FSystem::NewInstance( const TSharedPtr<FModel>& InModel,
		const TSharedPtr<UE::Mutable::Private::FImageIdRegistry>& InImageIdRegistry,
		const TSharedPtr<UE::Mutable::Private::FMaterialIdRegistry>& InMaterialIdRegistry,
		const TSharedPtr<UE::Mutable::Private::FSkeletalMeshIdRegistry>& InSkeletalMeshIdRegistry,
    	const TSharedPtr<FExternalResourceProvider>& ExternalResourceProvider,
    	const TSharedPtr<FPassthroughObjectLoader>& PassthroughObjectLoader,
    	const FImageOperator::FImagePixelFormatFunc& PixelFormatOverride,
		const TSharedPtr<FLiveInstanceLogger>& LiveInstanceLogger)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(NewInstance);
    	check(InModel)

		TSharedRef<FLiveInstance> InstanceData = MakeShared<FLiveInstance>();
		InstanceData->Instance = nullptr;
		InstanceData->Model = InModel;
		InstanceData->State = -1;
    	InstanceData->ExternalResourceProvider = ExternalResourceProvider;
    	InstanceData->PassthroughObjectLoader = PassthroughObjectLoader;
    	InstanceData->ImageIdRegistry = InImageIdRegistry;
    	InstanceData->MaterialIdRegistry = InMaterialIdRegistry;
    	InstanceData->SkeletalMeshIdRegistry = InSkeletalMeshIdRegistry;
    	InstanceData->PixelFormatOverride = PixelFormatOverride;
    	InstanceData->UpdateLogger = LiveInstanceLogger;
    	
		check(InstanceData->ImageIdRegistry.IsValid())
		check(InstanceData->MaterialIdRegistry.IsValid())
		check(InstanceData->SkeletalMeshIdRegistry.IsValid())

		InstanceData->UpdateLogger->LogUpdateMessage(TEXT("Creating Mutable instance"), ELogVerbosity::Verbose);
    	
		{
			TScopeLock<FMutex> LockGuard(WorkingMemoryManager.LiveInstancesMutex);
			WorkingMemoryManager.LiveInstances.Add(InstanceData.ToWeakPtr());
			WorkingMemoryManager.LiveInstances.RemoveAllSwap([](const TWeakPtr<FLiveInstance>& Instance)
			{
				return !Instance.IsValid();
			});
		}
		return InstanceData;
	}

	
    TSharedRef<FLiveInstance> FSystem::NewBuildInstance(const TSharedPtr<FModel>& InModel,
		const TSharedPtr<const FParameters>& InParameters,
	    const TSharedPtr<FExternalResourceProvider>& ExternalResourceProvider)
    {
    	MUTABLE_CPUPROFILER_SCOPE(NewBuildInstance);

    	UE_LOGF(LogMutableCore, Verbose, "Creating Mutable instance");
    	check(InModel)
    	
    	TSharedRef<FLiveInstance> InstanceData = MakeShared<FLiveInstance>();
    	InstanceData->Instance = nullptr;
    	InstanceData->Parameters = InParameters;
    	InstanceData->Model = InModel;
    	InstanceData->State = -1;
    	InstanceData->ExternalResourceProvider = ExternalResourceProvider;
    	InstanceData->PassthroughObjectLoader = nullptr;
    	InstanceData->ImageIdRegistry = MakeShared<FImageIdRegistry>();
    	InstanceData->MaterialIdRegistry = MakeShared<FMaterialIdRegistry>();
    	InstanceData->SkeletalMeshIdRegistry = MakeShared<FSkeletalMeshIdRegistry>();
    	InstanceData->UpdateLogger = MakeShared<FLiveInstanceLogger>();

    	return InstanceData;
    }


    //---------------------------------------------------------------------------------------------
    void FSystem::BeginUpdate_GameThread(const TSharedRef<FLiveInstance>& LiveInstance,uint32 InLodMask)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(BeginUpdate_GameThread);

		check(LiveInstance->Parameters);

    	CodeRunnerBegin Runner(LiveInstance, InLodMask);

    	check(LiveInstance->State >= 0);
    	FScheduledOpInline Root(LiveInstance->Model->GetProgram().States[LiveInstance->State].Root, 0, 0, true);
    	Runner.RunCode(FScheduledOpInline(Root, 0));
	}

	
    TManagedPtr<const FInstance> FSystem::BeginUpdate_MutableThread(const TSharedRef<FLiveInstance>& LiveInstance, bool bClearCacheLayer1)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(BeginUpdate_MutableThread);
		TRACE_COUNTER_INCREMENT(MutableRuntime_Updates);

		if (!LiveInstance->Parameters)
		{
			LiveInstance->UpdateLogger->LogUpdateMessage(TEXT("Invalid parameters in mutable update."));
			return nullptr;
		}

		const FProgram& Program = LiveInstance->Model->GetProgram();

    	const int32 LiveInstanceState = LiveInstance->State;
    	
		const bool bValidState = LiveInstanceState >= 0 && LiveInstanceState < (int32)Program.States.Num();
		if (!bValidState)
		{
			LiveInstance->UpdateLogger->LogUpdateMessage(TEXT("Invalid state in mutable update."));
			return nullptr;
		}

		// This may free resources that allow us to use less memory.
		LiveInstance->Instance = nullptr;

    	// Program cache setup
    	{
			FProgramCache::EClearFlags ClearFlags = FProgramCache::EClearFlags::Unlocked;
			if (bClearCacheLayer1)
			{
				EnumAddFlags(ClearFlags, FProgramCache::EClearFlags::Locked);
			}

			LiveInstance->Cache->Clear(ClearFlags);

			// Prepare instance cache
			if (Program.States.IsValidIndex(LiveInstanceState))
			{
				const FProgram::FState& State = Program.States[LiveInstanceState];
				for (uint32 Address : State.m_updateCache)
				{
					LiveInstance->Cache->LockAddress(FCacheAddress(Address, 0, 0));
				}
			}
    	}

    	
    	TManagedPtr<const FInstance> Result;
		{
			const FOperation::ADDRESS RootAt = Program.States[LiveInstanceState].Root;
			
			if (RunCodeInline(LiveInstance, RootAt, 0))
			{
				Result = LiveInstance->Cache->LoadInstance(FCacheAddress(RootAt, 0, 0));
			}

			// Debug check to see if we managed the op-hit-counts correctly
			LiveInstance->Instance = Result;

			if (!Result)
			{
				// In case of failure return an empty instance, to prevent following code to have to check it every time
				Result = MakeManaged<FInstance>();
			}
		}
		return Result;
	}


    TManagedRef<FImage> BlackImage()
    {
    	return MakeManaged<FImage>(16, 16, 1, EImageFormat::RGBA_UByte, EInitializationType::Black);
    }
	

	TManagedPtr<const FImage> FSystem::GetImageInline(const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId, int32 MipsToSkip)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SystemGetImage);

    	if (!ImageId)
    	{
			return BlackImage();
    	}

		FOperation::ADDRESS RootAddress = ImageId.GetKey()->Address;
		TManagedPtr<const FImage> Result = BuildImage(LiveInstance, RootAddress, MipsToSkip);

		if (!Result)
		{
			Result = BlackImage();
		}
    	
		return Result;
	}

	UE::Tasks::TTask<FGetImageResult> FSystem::GetImage(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId, int32 MipsToSkip)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SystemGetImage);

    	if (!ImageId)
    	{
    		return UE::Tasks::Launch(TEXT("FSystem::GetImageInvalid"),
			[]() -> FGetImageResult
			{
				FGetImageResult Output;
				Output.bWasOperationSuccessful = false;
				Output.MutableImage = MakeManaged<FImage>();
				return Output;
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
    	}

    	FOperation::ADDRESS RootAddress = ImageId.GetKey()->Address;
    	
    	EOpType OpType = LiveInstance->Model->GetProgram().GetOpType(RootAddress);
    	if (GetOpDataType(OpType) != EDataType::Image)
    	{
    		return UE::Tasks::Launch(TEXT("FSystem::GetImageInvalid"),
			[]() -> FGetImageResult
			{
				FGetImageResult Output;
				Output.bWasOperationSuccessful = false;
				Output.MutableImage = MakeManaged<FImage>();
				return Output;
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
    	}

    	struct FContext
    	{
    		TSharedPtr<CodeRunner> Runner;
    	};
    	TSharedRef<FContext> Context = MakeShared<FContext>();

    	UE::Tasks::FTask RunnerTask = UE::Tasks::Launch(TEXT("FSystem::GetImage"),
		[Context, System = this, RootAddress, LiveInstance, MipsToSkip]()
		{
			Context->Runner = CodeRunner::Create(
				LiveInstance,
				System->Settings,
				*System,
				EExecutionStrategy::MinimizeMemory,
				RootAddress,
				MipsToSkip,
				FScheduledOp::EType::Full);
				
			constexpr bool bForceInlineExecution = false;
			Context->Runner->StartRun(bForceInlineExecution);
		},
		UE::Tasks::Prerequisites(Prerequisite),
		UE::Tasks::ETaskPriority::Inherit);
    	
		return UE::Tasks::Launch(TEXT("FSystem::GetImageResultTask"),
		[Context, System = this, RootAddress, MipsToSkip, LiveInstance]() -> FGetImageResult
		{
			TManagedPtr<const FImage> Result;

			const FCacheAddress RootCacheAddress = FCacheAddress(RootAddress, 0, MipsToSkip);
			const bool bAborted = LiveInstance->Cache->IsAborted(RootCacheAddress);
			
			if (!bAborted)
			{
				Result = LiveInstance->Cache->LoadImage(RootCacheAddress);
			}

			if (!Result)
			{
				Result = MakeManaged<FImage>(16, 16, 1, EImageFormat::RGBA_UByte, EInitializationType::Black);
			}
			
			FGetImageResult Output;
			Output.MutableImage = Result;
			Output.bWasOperationSuccessful = !Context->Runner->bUnrecoverableError && !bAborted;
				
			return Output;
		},
		UE::Tasks::Prerequisites(RunnerTask),
		UE::Tasks::ETaskPriority::Inherit,
		UE::Tasks::EExtendedTaskPriority::Inline);
	}
	
	
	FExtendedImageDesc FSystem::GetImageDescInline(const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SystemGetImageDesc);

		if (!ImageId)
		{
			return {};
		}
		
		FExtendedImageDesc Result;

		FOperation::ADDRESS RootAddress = ImageId.GetKey()->Address;

		const UE::Mutable::Private::FModel* Model = LiveInstance->Model.Get();
		const UE::Mutable::Private::FProgram& Program = Model->GetProgram();

		UE::Mutable::Private::EOpType OpType = Program.GetOpType(RootAddress);
		if (GetOpDataType(OpType) == EDataType::Image)
		{
			// GetImageDesc may call normal execution paths where meshes are computed.
					
			uint16 ExecutionOptions = 0;
			TSharedRef<CodeRunner> Runner = CodeRunner::Create(
					LiveInstance,
					Settings,
					*this,
					EExecutionStrategy::MinimizeMemory,
					RootAddress,
					ExecutionOptions,
					FScheduledOp::EType::ImageDesc);
			
			constexpr bool bForceInlineExecution = true;
			Runner->StartRun(bForceInlineExecution);

			Result = Runner->GetImageDescResult(RootAddress);
			
			LiveInstance->Cache->Clear(FProgramCache::EClearFlags::ImageDesc);
		}
		
		return Result;
	}


	UE::Tasks::TTask<FExtendedImageDesc> FSystem::GetImageDesc(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FImageId& ImageId)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SystemGetImageDesc);

		if (!ImageId)
		{
			return UE::Tasks::Launch(TEXT("FSystem::GetImageDescInvalid"),
			[]()
			{
				return FExtendedImageDesc();
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
		}
		
		FOperation::ADDRESS RootAddress = ImageId.GetKey()->Address;
		
		EOpType OpType = LiveInstance->Model->GetProgram().GetOpType(RootAddress);
		if (GetOpDataType(OpType) != EDataType::Image)
		{
			return UE::Tasks::Launch(TEXT("FSystem::GetImageDescInvalid"),
			[]()
			{
				return FExtendedImageDesc();
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
		}

		struct FContext
		{
			TSharedPtr<CodeRunner> Runner;
		};
		TSharedRef<FContext> Context = MakeShared<FContext>();
		
		uint16 ExecutionOptions = 0;
		UE::Tasks::FTask RunnerTask = UE::Tasks::Launch(TEXT("FSystem::GetImageDesc"),
		[Context, System = this, LiveInstance, RootAddress, ExecutionOptions]()
		{
			// GetImageDesc may call normal execution paths where meshes are computed.
			Context->Runner = CodeRunner::Create(
			LiveInstance,
				System->Settings,
				*System,
				EExecutionStrategy::MinimizeMemory,
				RootAddress,
				ExecutionOptions,
				FScheduledOp::EType::ImageDesc);

			constexpr bool bForceInlineExecution = false;
			Context->Runner->StartRun(bForceInlineExecution);
		}, 
		UE::Tasks::Prerequisites(Prerequisite),
		UE::Tasks::ETaskPriority::Inherit);
		
		return UE::Tasks::Launch(TEXT("FSystem::GetImageDescResultTask"),
		[Context, System = this, RootAddress, LiveInstance, ExecutionOptions]() -> FExtendedImageDesc
		{
			FExtendedImageDesc Result = Context->Runner->LoadImageDesc(FCacheAddress(RootAddress, 0, ExecutionOptions, FScheduledOp::EType::ImageDesc));

			LiveInstance->Cache->Clear(FProgramCache::EClearFlags::ImageDesc);
				
			return Result;
		},
		UE::Tasks::Prerequisites(RunnerTask),
		UE::Tasks::ETaskPriority::Inherit,
		UE::Tasks::EExtendedTaskPriority::Inline);
	}	
	
	
	UE::Tasks::TTask<FGetSkeletalMeshResult> FSystem::GetSkeletalMesh(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FSkeletalMeshId& SkeletalMeshId, const uint16 ExecutionOptions)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SystemGetSkeletalMesh);
		
		if (!SkeletalMeshId)
		{
			return UE::Tasks::Launch(TEXT("FSystem::GetSkeletalMeshInvalid"),
			[]() -> FGetSkeletalMeshResult
			{
				FGetSkeletalMeshResult Output;
				Output.MutableSkeletalMesh = MakeManaged<FSkeletalMesh>();
				Output.bWasOperationSuccessful = false;
				return Output;
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
		}

		FOperation::ADDRESS RootAddress = SkeletalMeshId.GetKey()->Address;
		
		UE::Mutable::Private::EOpType OpType = LiveInstance->Model->GetProgram().GetOpType(RootAddress);
		if (GetOpDataType(OpType) != EDataType::SkeletalMesh)
		{
			return UE::Tasks::Launch(TEXT("FSystem::GetSkeletalMeshInvalid"),
				[]() -> FGetSkeletalMeshResult
				{
					FGetSkeletalMeshResult Output;
					Output.MutableSkeletalMesh = MakeManaged<FSkeletalMesh>();
					Output.bWasOperationSuccessful = false;
					return Output;
				},
				UE::Tasks::Prerequisites(Prerequisite),
				UE::Tasks::ETaskPriority::Inherit,
				UE::Tasks::EExtendedTaskPriority::Inline);
		}

		struct FContext
		{
			TSharedPtr<CodeRunner> Runner;
		};
		TSharedRef<FContext> Context = MakeShared<FContext>();
		
		UE::Tasks::FTask RunnerTask = UE::Tasks::Launch(TEXT("FSystem::GetSkeletalMesh"),
		[Context, System = this, RootAddress, LiveInstance, ExecutionOptions]()
		{
			Context->Runner = CodeRunner::Create(
			LiveInstance,
					System->Settings,
					*System, 
					EExecutionStrategy::MinimizeMemory, 
					RootAddress, 
					ExecutionOptions, 
					FScheduledOp::EType::Full);

			constexpr bool bForceInlineExecution = false;
			Context->Runner->StartRun(bForceInlineExecution);
		},
		UE::Tasks::Prerequisites(Prerequisite),
		UE::Tasks::ETaskPriority::Inherit);
		
		return UE::Tasks::Launch(TEXT("FSystem::GetSkeletalMeshResultTask"),
		[Context, System = this, RootAddress, ExecutionOptions, LiveInstance]() -> FGetSkeletalMeshResult
		{
			TManagedPtr<const FSkeletalMesh> Result;

			FCacheAddress RootCacheAddress = FCacheAddress(RootAddress, 0, ExecutionOptions);

			bool bAborted = LiveInstance->Cache->IsAborted(RootCacheAddress);
			if (!bAborted)
			{
				check(LiveInstance->Cache->IsSet(RootCacheAddress));
				Result = LiveInstance->Cache->LoadSkeletalMesh(RootCacheAddress);
			}
				
			if (!Result)
			{
				Result = MakeManaged<FSkeletalMesh>();
			}
				
			FGetSkeletalMeshResult Output;
			Output.MutableSkeletalMesh = Result;
			Output.bWasOperationSuccessful = !Context->Runner->bUnrecoverableError && !bAborted;
				
			return Output;
		},
		UE::Tasks::Prerequisites(RunnerTask),
		UE::Tasks::ETaskPriority::Inherit,
		UE::Tasks::EExtendedTaskPriority::Inline);
	}

	
	UE::Tasks::TTask<FGetMaterialResult> FSystem::GetMaterial(UE::Tasks::FTask Prerequisite, const TSharedRef<FLiveInstance>& LiveInstance, const FMaterialId& MaterialId)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(SystemGetMaterial);
		
		if (!MaterialId)
		{
			return UE::Tasks::Launch(TEXT("FSystem::GetMaterialInvalid"),
			[]() -> FGetMaterialResult
			{
				FGetMaterialResult Output;
				Output.MutableMaterial = MakeManaged<FMaterial>();
				Output.bWasOperationSuccessful = false;
				return Output;
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
		}
		
		FOperation::ADDRESS RootAddress = MaterialId.GetKey()->Address;
	
		EOpType OpType = LiveInstance->Model->GetProgram().GetOpType(RootAddress);
		if (GetOpDataType(OpType) != EDataType::Material)
		{
			return UE::Tasks::Launch(TEXT("FSystem::GetMaterialInvalid"),
			[]() -> FGetMaterialResult
			{
				FGetMaterialResult Output;
				Output.MutableMaterial = MakeManaged<FMaterial>();
				Output.bWasOperationSuccessful = false;
				return Output;
			},
			UE::Tasks::Prerequisites(Prerequisite),
			UE::Tasks::ETaskPriority::Inherit,
			UE::Tasks::EExtendedTaskPriority::Inline);
		}

		struct FContext
		{
			TSharedPtr<CodeRunner> Runner;
		};
		TSharedRef<FContext> Context = MakeShared<FContext>();
		
		UE::Tasks::FTask RunnerTask = UE::Tasks::Launch(TEXT("FSystem::GetMaterial"),
		[Context, System = this, RootAddress, LiveInstance]()
		{
			Context->Runner = CodeRunner::Create(
			LiveInstance,
					System->Settings,
					*System, 
					EExecutionStrategy::MinimizeMemory, 
					RootAddress, 
					0,
					FScheduledOp::EType::Full);

			constexpr bool bForceInlineExecution = false;
			Context->Runner->StartRun(bForceInlineExecution);
		},
		UE::Tasks::Prerequisites(Prerequisite),
		UE::Tasks::ETaskPriority::Inherit);
		
		return UE::Tasks::Launch(TEXT("FSystem::GetMaterialResultTask"),
		[Context, System = this, RootAddress, LiveInstance]() -> FGetMaterialResult
		{
			TManagedPtr<const FMaterial> Result;

			FCacheAddress RootCacheAddress = FCacheAddress(RootAddress, 0, 0);

			const bool bAborted = LiveInstance->Cache->IsAborted(RootCacheAddress);
			if (!bAborted)
			{
				Result = Context->Runner->LoadMaterial(RootCacheAddress);
			}
			
			if (!Result)
			{
				Result = MakeManaged<FMaterial>();
			}
				
			FGetMaterialResult Output;
			Output.MutableMaterial = Result;
			Output.bWasOperationSuccessful = !Context->Runner->bUnrecoverableError && !bAborted;
						
			return Output;
		},
		UE::Tasks::Prerequisites(RunnerTask),
		UE::Tasks::ETaskPriority::Inherit,
		UE::Tasks::EExtendedTaskPriority::Inline);
	}
	

	void FSystem::EndUpdate(const TSharedRef<FLiveInstance>& LiveInstance, bool bClearRoms, bool bFreeCache)
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(EndUpdate);

		LiveInstance->Instance = nullptr;

		// We don't want to clear the cache layer 1 because it contains data that can be useful for a 
		// future update (same states, just runtime parameters changed).
		//LiveInstance->Cache->ClearLayer1();

		// We need to clear the layer 0 cache, because it contains data that is only valid for the current 
		// parameter values (unless it is data marked as state cache)
		
		FProgramCache::EClearFlags ClearFlags = FProgramCache::EClearFlags::Unlocked;
		if (bFreeCache)
		{
			EnumAddFlags(ClearFlags, FProgramCache::EClearFlags::Locked | FProgramCache::EClearFlags::FreeMemory); 
		}
		LiveInstance->Cache->Clear(ClearFlags);
	

		// Reduce the cache until it fits the limit.
		// Not providing a cache at this point as there is no need of keeping the cache active anymore
		WorkingMemoryManager.EnsureBudgetBelow(0, nullptr);

		if (bClearRoms)
		{
			LiveInstance->Model->RomManager.UnloadRoms();
		}
		
		UpdateStats();
	}
	

    //---------------------------------------------------------------------------------------------
    class RelevantParameterVisitor : public UniqueDiscreteCoveredCodeVisitor<>
    {
    public:

        RelevantParameterVisitor(
                FSystem* InSystem,
				const TSharedRef<FLiveInstance>& InLiveInstance,
                TArray<uint32>& OutRelevantParameters) :
				UniqueDiscreteCoveredCodeVisitor<>(InSystem, InLiveInstance),
		    	RelevantParameters(OutRelevantParameters)
        {
			const FOperation::ADDRESS At = InLiveInstance->Model->GetProgram().States[0].Root;

            Run(At);
        }

        bool Visit(FOperation::ADDRESS At, const FProgram& Program) override
        {
            switch (Program.GetOpType(At))
            {
            case EOpType::BO_PARAMETER:
            case EOpType::NU_PARAMETER:
            case EOpType::SC_PARAMETER:
            case EOpType::CO_PARAMETER:
            case EOpType::PR_PARAMETER:
			case EOpType::IM_PARAMETER:
			case EOpType::SK_PARAMETER:
			case EOpType::MA_PARAMETER:
			case EOpType::MI_PARAMETER:
			case EOpType::IS_PARAMETER:
            {
				FOperation::ParameterArgs args = Program.GetOpArgs<FOperation::ParameterArgs>(At);
				FOperation::ADDRESS ParamIndex = args.variable;
                RelevantParameters.Add(ParamIndex);
                break;
            }
            default:
                break;
            }

            return UniqueDiscreteCoveredCodeVisitor<>::Visit( At, Program );
        }

    private:
        TArray<uint32>& RelevantParameters;
    };

	
    void FSystem::GetParameterRelevancy(const TSharedRef<FLiveInstance>& LiveInstance, TArray<uint32>& OutRelevantParameters)
    {
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
    	
		RelevantParameterVisitor visitor(this, LiveInstance, OutRelevantParameters);
    }

	
	bool FSystem::CheckForUpdatedParameters(const TSharedRef<FLiveInstance>& LiveInstance, const TSharedPtr<const FParameters>& InOldParameters)
    {
        bool bFullBuild = false;

		if (!LiveInstance->Parameters || !InOldParameters)
		{
			return true;
		}

    	const TSharedRef<const FParameters> NewParameters = LiveInstance->Parameters.ToSharedRef();

        // check what parameters have changed
        const FProgram& Program = LiveInstance->Model->GetProgram();
        const TArray<int>& RuntimeParameters = Program.States[ LiveInstance->State ].m_runtimeParameters;

        check( NewParameters->GetCount() == (int32)Program.Parameters.Num() );
        check( NewParameters->GetCount() == NewParameters->GetCount() );

        for ( int32 ParameterIndex=0; ParameterIndex<Program.Parameters.Num() ; ++ParameterIndex )
        {
            const bool bIsRuntimeParameter = RuntimeParameters.Contains( ParameterIndex );
            const bool bHasChanged = !NewParameters->HasSameValue( ParameterIndex, InOldParameters, ParameterIndex );

        	if (bHasChanged && !bIsRuntimeParameter)
        	{
        		bFullBuild = true;
        		break;
        	}
        }

        return bFullBuild;
    }


	bool FSystem::RunCodeInline(
		const TSharedRef<FLiveInstance>& InLiveInstance,
		FOperation::ADDRESS InCodeRoot,
		uint16 ExecutionOptions)
	{
		TSharedRef<CodeRunner> Runner = CodeRunner::Create(
			InLiveInstance,
			Settings,
			*this,
			EExecutionStrategy::MinimizeMemory,
			InCodeRoot,
			ExecutionOptions,
			FScheduledOp::EType::Full);
    	
		constexpr bool bForceInlineExecution = true;
		Runner->StartRun(bForceInlineExecution);

    	// The runner will display an unrecoverable error if one was found during the run operation or if the cache reports that the operation was marked as aborted
    	const bool bWasRunSuccessful = !Runner->bUnrecoverableError;
    	return bWasRunSuccessful;
	}
	
	
	bool FSystem::BuildBool(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
    {
    	bool bResult = false;
    	if (RunCodeInline(InLiveInstance, InAddress))
    	{
    		bResult = InLiveInstance->Cache->LoadBool(FCacheAddress(InAddress, 0, 0));
    	}
	
    	return bResult;    
    }
	
	
	int32 FSystem::BuildInt(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
		int32 Result = 0;
		if (RunCodeInline(InLiveInstance, InAddress))
		{
			Result = InLiveInstance->Cache->LoadInt(FCacheAddress(InAddress, 0, 0));
		}

		return Result;
	}

	
	float FSystem::BuildScalar(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
    	float Result = 0.0f;		
    	if (RunCodeInline(InLiveInstance, InAddress))
    	{
    		Result = InLiveInstance->Cache->LoadScalar(FCacheAddress(InAddress, 0, 0));
    	}

    	return Result;
	}


	FVector4f FSystem::BuildColor(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
		FVector4f Result(0,0,0,1);
    	
		UE::Mutable::Private::EOpType OpType = InLiveInstance->Model->GetProgram().GetOpType(InAddress);
		if (GetOpDataType(OpType) == EDataType::Color)
		{
			if (RunCodeInline(InLiveInstance, InAddress))
			{
				Result = InLiveInstance->Cache->LoadColor(FCacheAddress(InAddress, 0, 0));
			}
		}

		return Result;
	}

	
	FProjector FSystem::BuildProjector(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
		FProjector Result;
		if (RunCodeInline(InLiveInstance, InAddress))
		{
			Result = InLiveInstance->Cache->LoadProjector(FCacheAddress(InAddress, 0, 0));
		}

		return Result;
	}


	TManagedPtr<const FImage> FSystem::BuildImage(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress, int32 MipsToSkip)
	{
		TManagedPtr<const FImage> Result;
    	
		UE::Mutable::Private::EOpType OpType = InLiveInstance->Model->GetProgram().GetOpType(InAddress);
		if (GetOpDataType(OpType) == EDataType::Image)
		{
			if (RunCodeInline(InLiveInstance, InAddress, uint8(MipsToSkip)))
			{
				Result = InLiveInstance->Cache->LoadImage(FCacheAddress(InAddress, 0, MipsToSkip));
			}			
		}

		return Result;
	}


	TManagedPtr<const FMesh> FSystem::BuildMesh(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InRootAddress, EMeshContentFlags MeshContentFilter)
	{
		TManagedPtr<const FMesh> Result;
    	
		UE::Mutable::Private::EOpType OpType = InLiveInstance->Model->GetProgram().GetOpType(InRootAddress);
		if (GetOpDataType(OpType) == EDataType::Mesh)
		{
			const uint16 ExecutionOptions = static_cast<uint16>(MeshContentFilter);
			if (RunCodeInline(InLiveInstance, InRootAddress, ExecutionOptions))
			{
				Result = InLiveInstance->Cache->LoadMesh(FCacheAddress(InRootAddress, 0, ExecutionOptions));
			}
		}

		return Result;
	}


	TManagedPtr<const FInstance> FSystem::BuildInstance(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
		TManagedPtr<const FInstance> Result;
    	
		UE::Mutable::Private::EOpType OpType = InLiveInstance->Model->GetProgram().GetOpType(InAddress);
		if (GetOpDataType(OpType) == EDataType::Instance)
		{
			if (RunCodeInline(InLiveInstance, InAddress))
			{
				Result = InLiveInstance->Cache->LoadInstance(FCacheAddress(InAddress, 0, 0));
			}
		}

		return Result;
	}


	TManagedPtr<const FLayout> FSystem::BuildLayout(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
		TManagedPtr<const FLayout> Result;
    	
		if (InLiveInstance->Model->GetProgram().States[0].Root)
		{
			UE::Mutable::Private::EOpType OpType = InLiveInstance->Model->GetProgram().GetOpType(InAddress);
			if (GetOpDataType(OpType) == EDataType::Layout)
			{
				if (RunCodeInline(InLiveInstance, InAddress))
				{
					Result = InLiveInstance->Cache->LoadLayout(FCacheAddress(InAddress, 0, 0));
				}
			}
		}

		return Result;
	}


	TManagedPtr<const String> FSystem::BuildString(const TSharedRef<FLiveInstance>& InLiveInstance, FOperation::ADDRESS InAddress)
	{
		TManagedPtr<const String> Result;
    	
		if (InLiveInstance->Model->GetProgram().States[0].Root)
		{
			UE::Mutable::Private::EOpType OpType = InLiveInstance->Model->GetProgram().GetOpType(InAddress);
			if (GetOpDataType(OpType) == EDataType::String)
			{
				if (RunCodeInline(InLiveInstance, InAddress))
				{
					Result = InLiveInstance->Cache->LoadString(FCacheAddress(InAddress, 0, 0));
				}
			}
		}

		return Result;
	}
	

	void FSystem::UpdateStats()
	{
		// Updater stats
		int32 WorkingMemoryKb = WorkingMemoryManager.BudgetBytes / 1024;
		SET_DWORD_STAT(STAT_MutableWorkingMemory, WorkingMemoryKb);

		int32 WorkingMemoryExcessKb = WorkingMemoryManager.BudgetExcessBytes / 1024;
		SET_DWORD_STAT(STAT_MutableWorkingMemoryExcess, WorkingMemoryExcessKb);

		int32 CurrentMemoryKb = WorkingMemoryManager.GetCurrentMemoryBytes() / 1024;
		SET_DWORD_STAT(STAT_MutableCurrentMemory, CurrentMemoryKb);
	}


	int32 FWorkingMemoryManager::GetRomBytes() const
    {
    	int32 Result = 0;
		TScopeLock<FMutex> LockGuard(LiveInstancesMutex);
    	for (const TWeakPtr<FLiveInstance>& WeakInstance : LiveInstances)
    	{
    		TSharedPtr<FLiveInstance> Instance = WeakInstance.Pin();
    		if (!Instance)
    		{
    			continue;
    		}

    		Result +=  Instance->Model->RomManager.GetRomBytes();
    	}

    	return Result;
    }
	

	void FWorkingMemoryManager::LogWorkingMemory(const CodeRunner* CurrentRunner) const
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		MUTABLE_CPUPROFILER_SCOPE(LogWorkingMemory);

		// For now, we calculate these for every log. We will later track on resource creation, destruction or state change.
		// All resource memory is tracked by the memory allocator, but that does not give information about where the memory is
		// located. Keep the localized memory computation for now.   
		const uint32 RomBytes = GetRomBytes();
		const uint32 CacheBytes = GetCacheBytes();
		uint32 HeapBytes = 0;
	
    	if (CurrentRunner)
    	{
    		for (const CodeRunner::FScheduledOpData& Data : CurrentRunner->HeapData)
    		{
    			if (Data.Resource)
    			{
    				HeapBytes += Data.Resource->GetDataSize();
    			}
    		}
    	}

		// Get allocator counters.
		const SSIZE_T ImageAllocBytes	 = MemoryCounters::FImageMemoryCounter::Get().load(std::memory_order_relaxed);
		const SSIZE_T MeshAllocBytes     = MemoryCounters::FMeshMemoryCounter::Get().load(std::memory_order_relaxed);
		const SSIZE_T StreamAllocBytes   = MemoryCounters::FStreamingMemoryCounter::Get().load(std::memory_order_relaxed);
		const SSIZE_T InternalAllocBytes = MemoryCounters::FInternalMemoryCounter::Get().load(std::memory_order_relaxed);

		SSIZE_T TotalBytes = ImageAllocBytes + MeshAllocBytes + StreamAllocBytes + InternalAllocBytes;

		UE_LOGF(LogMutableCore, Log, 
			"Mem KB: ImageAlloc %7lld, MeshAlloc %7lld, StreamAlloc %7lld, InternalAlloc %7lld, AllocTotal %7lld / %7lld" "Resources MemLoc: Cache0+1 %7d, Rom %7d, Heap %7d",
			ImageAllocBytes/1024, MeshAllocBytes/1024, StreamAllocBytes/1024, InternalAllocBytes/1024, TotalBytes/1024, BudgetBytes/1024, CacheBytes/1024, RomBytes/1024, HeapBytes/1024);

		TRACE_COUNTER_SET(MutableRuntime_MemRom,      RomBytes);
		TRACE_COUNTER_SET(MutableRuntime_MemInternal, InternalAllocBytes);
		TRACE_COUNTER_SET(MutableRuntime_MemMesh,     MeshAllocBytes);
		TRACE_COUNTER_SET(MutableRuntime_MemImage,    ImageAllocBytes);
		TRACE_COUNTER_SET(MutableRuntime_MemStream,   StreamAllocBytes);
		TRACE_COUNTER_SET(MutableRuntime_MemTotal,    TotalBytes);
		TRACE_COUNTER_SET(MutableRuntime_MemBudget,   BudgetBytes);
#endif
	}	

	int64 FWorkingMemoryManager::GetCurrentMemoryBytes() const
	{
		MUTABLE_CPUPROFILER_SCOPE(GetCurrentMemoryBytes);

		SSIZE_T TotalBytes = MemoryCounters::FImageMemoryCounter::Get().load(std::memory_order_relaxed) + 
						     MemoryCounters::FMeshMemoryCounter::Get().load(std::memory_order_relaxed) +
							 MemoryCounters::FStreamingMemoryCounter::Get().load(std::memory_order_relaxed) +
							 MemoryCounters::FInternalMemoryCounter::Get().load(std::memory_order_relaxed);

		return TotalBytes;
	}

	bool FWorkingMemoryManager::IsMemoryBudgetFull() const
	{
		// If we have 0 budget it means we have unlimited budget
		if (BudgetBytes == 0)
		{
			return false;
		}

		uint64 CurrentBytes = GetCurrentMemoryBytes();
		uint64 BudgetThresholdBytes = (BudgetBytes * 9) / 10;
		
		return CurrentBytes > BudgetThresholdBytes;
	}

	bool FWorkingMemoryManager::EnsureBudgetBelow(int64 AdditionalMemory, const TSharedPtr<FLiveInstance>& InCurrentLiveInstance)
    {
		MUTABLE_CPUPROFILER_SCOPE(EnsureBudgetBelow);

		// If we have 0 budget it means we have unlimited budget
		if (BudgetBytes == 0)
		{
			return true;
		}		

		int64 TotalBytes = GetCurrentMemoryBytes();

		// Add the extra memory that we are trying to allocate when we return.
		TotalBytes += AdditionalMemory;

        bool bFinished = TotalBytes <= BudgetBytes;
	
		// Try to free loaded roms
		if (!bFinished)
		{
			TScopeLock<FMutex> LockGuard(LiveInstancesMutex);
			for (TWeakPtr<FLiveInstance>& WeakLiveInstance : LiveInstances)
			{
				TSharedPtr<FLiveInstance> LiveInstance = WeakLiveInstance.Pin();
				if (!LiveInstance)
				{
					continue;
				}
				
				if (LiveInstance == InCurrentLiveInstance)
				{
					// Ignore the current live instance.
					continue;
				}
				
				const int64 RomRemovedBytes = LiveInstance->Model->RomManager.EnsureBudgetBelow(TotalBytes - BudgetBytes);
				TotalBytes -= RomRemovedBytes;

				bFinished = TotalBytes <= BudgetBytes;
				if (bFinished)
				{
					break;
				}
			}
		}
    	
    	if (!bFinished && InCurrentLiveInstance)
    	{	
    		const int64 RomRemovedBytes = InCurrentLiveInstance->Model->RomManager.EnsureBudgetBelow(TotalBytes - BudgetBytes);
    		TotalBytes -= RomRemovedBytes;
			
    		bFinished = TotalBytes <= BudgetBytes;
    	}

		auto TryRemoveFromCache = [&bFinished, &TotalBytes, Budget = BudgetBytes](FProgramCache& Cache)
		{
			TotalBytes -= Cache.TryFreeWeakMemory(TotalBytes - Budget);
			bFinished = TotalBytes <= Budget;
		};

		// Try to free cache 1 memory
		if (!bFinished)
		{
			MUTABLE_CPUPROFILER_SCOPE(EnsureBudgetBelow_FreeCached);	
			TScopeLock<FMutex> LockGuard(LiveInstancesMutex);
			for (const TWeakPtr<FLiveInstance>& WeakInstance : LiveInstances)
			{
				TSharedPtr<FLiveInstance> Instance = WeakInstance.Pin();
				if (!Instance)
				{
					continue;
				}
				
				if (Instance == InCurrentLiveInstance)
				{
					// Ignore the current live instance.
					continue;
				}

				MUTABLE_CPUPROFILER_SCOPE(EnsureBudgetBelow_FreeCached_GatherAndRemove_Other);
				// Gather all data in the cache for this instance

				TryRemoveFromCache(*Instance->Cache);

				if (bFinished)
				{
					break;
				}
			}
		}

		// From the current live instances. It is more involved: we have to make sure any data we want to free is not also
		// in any cache (0 or 1) position with hit-count > 0.
		if (!bFinished && InCurrentLiveInstance)
		{
			MUTABLE_CPUPROFILER_SCOPE(EnsureBudgetBelow_FreeCached_Current);

			TryRemoveFromCache(*InCurrentLiveInstance->Cache);
		}


		if (!bFinished)
		{
			int64 ExcessBytes = TotalBytes - BudgetBytes;

			if (ExcessBytes > BudgetExcessBytes)
			{
				BudgetExcessBytes = ExcessBytes;

				// We failed to free enough memory. Log this, but try to continue anyway.
				// This is a good place to insert a breakpoint to detect callstacks with memory peaks
				UE_LOG(LogMutableCore, Log, TEXT("Failed to keep memory budget. Budget: %" INT64_FMT ", Current: %" INT64_FMT ", New: %" UINT64_FMT),
					BudgetBytes / 1024, (TotalBytes - AdditionalMemory) / 1024, AdditionalMemory / 1024);
				
				if (bEnableDetailedMemoryBudgetExceededLogging)
				{
					// We won't show correct internal or streaming buffer memory.
					LogWorkingMemory(nullptr);
				}
			}
		}

        return bFinished;
    }
	

	static void AddMultiValueKeys(FBitWriter& Blob, const TMap< TArray<int32>, FParameterValue >& Multi)
	{
		uint16 Num = uint16(Multi.Num());
		Blob.Serialize(&Num, 2);

		for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
		{
			uint16 RangeNum = uint16(v.Key.Num());
			Blob.Serialize(&RangeNum, 2);
			Blob.Serialize((void*)v.Key.GetData(), RangeNum * sizeof(int32));
		}
	}


	void GetResourceKey(const FModel& Model, const FParameters& Params, uint32 ParamListIndex, TMemoryTrackedArray<uint8>& ParameterValuesBlob)
	{
		MUTABLE_CPUPROFILER_SCOPE(GetResourceKey);
    	
		const FProgram& Program = Model.GetProgram();

		// Find the list of relevant parameters
		const TArray<uint16>* RelevantParams = nullptr;
		if (ParamListIndex < (uint32)Program.ParameterLists.Num())
		{
			RelevantParams = &Program.ParameterLists[ParamListIndex];
		}
    	
		if (!RelevantParams)
		{
			return;
		}

		// Generate the relevant parameters blob
		FBitWriterResourceKey Blob(2048*8, true);

		const TArray<FParameterDesc>& ParamDescs = Program.Parameters;

		// First make a mask with a bit for each relevant parameter. It will be on for parameters included in the blob.
		// A parameter will be excluded from the blob if it has the default value, and no multivalues.
		TBitArray IncludedParameters(0, RelevantParams->Num());
		if (RelevantParams->Num())
		{
			for (int32 IndexIndex = 0; IndexIndex < RelevantParams->Num(); ++IndexIndex)
			{
				int32 ParamIndex = (*RelevantParams)[IndexIndex];
				bool bInclude = Params.HasMultipleValues(ParamIndex);
				if (!bInclude)
				{
					bInclude = !(
						Params.Values[ParamIndex]
						==
						ParamDescs[ParamIndex].DefaultValue
						);
				}

				IncludedParameters[IndexIndex] = bInclude;
			}
			Blob.SerializeBits(IncludedParameters.GetData(), IncludedParameters.Num());
		}

		// Second: serialize the value of the selected parameters.
		for (int32 IndexIndex = 0; IndexIndex < RelevantParams->Num(); ++IndexIndex)
		{
			int32 ParamIndex = (*RelevantParams)[IndexIndex];
			if (!IncludedParameters[IndexIndex])
			{
				continue;
			}

			int32 DataSize = 0;

			switch (Program.Parameters[ParamIndex].Type)
			{
			case EParameterType::Bool:
				Blob.WriteBit(Params.Values[ParamIndex].Get<FParamBoolType>() ? 1 : 0);

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Blob.WriteBit(v.Value.Get<FParamBoolType>() ? 1 : 0);
					}
				}
				break;

			case EParameterType::Int:
			{
				int32 MaxValue = ParamDescs[ParamIndex].PossibleValues.Num();
				int32 Value = Params.Values[ParamIndex].Get<FParamIntType>();
				if (MaxValue)
				{
					// It is an enum
					uint32 LimitedValue = FMath::Clamp( Params.GetIntValueIndex(ParamIndex,Value), 0, MaxValue-1 );
					Blob.SerializeInt(LimitedValue, uint32(MaxValue));
				}
				else
				{
					// It may have any value
					DataSize = sizeof(FParamIntType);
					Blob.Serialize(&Value, DataSize);
				}

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Value = v.Value.Get<FParamIntType>();
						if (MaxValue)
						{
							// It is an enum
							uint32 LimitedValue = Value;
							Blob.SerializeInt(LimitedValue, uint32(MaxValue));
						}
						else
						{
							// It may have any value
							DataSize = sizeof(FParamIntType);
							Blob.Serialize(&Value, DataSize);
						}
					}
				}
				break;
			}

			case EParameterType::Float:
				DataSize = sizeof(FParamFloatType);
				Blob.Serialize(&const_cast<FParameters*>(&Params)->Values[ParamIndex].Get<FParamFloatType>(), DataSize);

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Blob.Serialize((void*)&v.Value.Get<FParamFloatType>(), DataSize);
					}
				}
				break;

			case EParameterType::Color:
				DataSize = sizeof(FParamColorType);
				Blob.Serialize(&const_cast<FParameters*>(&Params)->Values[ParamIndex].Get<FParamColorType>(), DataSize);

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Blob.Serialize((void*)&v.Value.Get<FParamColorType>(), DataSize);
					}
				}
				break;

			case EParameterType::Projector:
			{
				FPackedNormal TempVec;
				const FProjector& Value = Params.Values[ParamIndex].Get<FParamProjectorType>();
				Blob.Serialize((void*)&Value.position, sizeof(FVector3f));
				TempVec = FPackedNormal(Value.direction);
				Blob.Serialize(&TempVec, sizeof(FPackedNormal));
				TempVec = FPackedNormal(Value.up);
				Blob.Serialize(&TempVec, sizeof(FPackedNormal));
				Blob.Serialize((void*)&Value.scale, sizeof(FVector3f));
				Blob.Serialize((void*)&Value.projectionAngle, sizeof(float));

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						const FProjector& MultiValue = v.Value.Get<FParamProjectorType>();
						Blob.Serialize((void*)&MultiValue.position, sizeof(FVector3f));
						TempVec = FPackedNormal(MultiValue.direction);
						Blob.Serialize(&TempVec, sizeof(FPackedNormal));
						TempVec = FPackedNormal(MultiValue.up);
						Blob.Serialize(&TempVec, sizeof(FPackedNormal));
						Blob.Serialize((void*)&MultiValue.scale, sizeof(FVector3f));
						Blob.Serialize((void*)&MultiValue.projectionAngle, sizeof(float));
					}
				}
				break;
			}

			case EParameterType::Texture:
			{
				TStrongObjectPtr<UTexture> Object = Params.Values[ParamIndex].Get<FParamTextureType>();
				FString Path = Object ? Object->GetPathName() : FString();
				Blob.Serialize(GetData(Path), Path.Len() * sizeof(FString::ElementType));

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Object = v.Value.Get<FParamTextureType>();
						Path = Object ? Object->GetPathName() : FString();
						Blob.Serialize(GetData(Path), Path.Len() * sizeof(FString::ElementType));
					}
				}
				break;
			}

			case EParameterType::SkeletalMesh:
			{
				TStrongObjectPtr<USkeletalMesh> Object = Params.Values[ParamIndex].Get<FParamSkeletalMeshType>();
				FString Path = Object ? Object->GetPathName() : FString();
				Blob.Serialize(GetData(Path), Path.Len() * sizeof(FString::ElementType));

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Object = v.Value.Get<FParamSkeletalMeshType>();
						Path = Object ? Object->GetPathName() : FString();
						Blob.Serialize(GetData(Path), Path.Len() * sizeof(FString::ElementType));
					}
				}
				break;
			}

			case EParameterType::Material:
			{
				TStrongObjectPtr<UMaterialInterface> Object = Params.Values[ParamIndex].Get<FParamMaterialType>();
				FString Path = Object ? Object->GetPathName() : FString();
				Blob.Serialize(GetData(Path), Path.Len() * sizeof(FString::ElementType));

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap< TArray<int32>, FParameterValue >& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair< TArray<int32>, FParameterValue >& v : Multi)
					{
						Object = v.Value.Get<FParamMaterialType>();
						Path = Object ? Object->GetPathName() : FString();
						Blob.Serialize(GetData(Path), Path.Len() * sizeof(FString::ElementType));
					}
				}
				break;
			}

			case EParameterType::Matrix:
			{
				DataSize = sizeof(FMatrix44f);

				const FMatrix44f& Value = Params.Values[ParamIndex].Get<FParamMatrixType>();
				Blob.Serialize((void*)&Value, DataSize);

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap<TArray<int32>, FParameterValue>& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair<TArray<int32>, FParameterValue>& MultiValuePair : Multi)
					{
						const FMatrix44f& MultiValue = MultiValuePair.Value.Get<FParamMatrixType>();
						Blob.Serialize((void*)&MultiValue, DataSize);
					}
				}
				break;
			}

			case EParameterType::InstancedStruct:
			{
				const FParamInstancedStructType& Value = Params.Values[ParamIndex].Get<FParamInstancedStructType>();

				FInstancedStruct* ValueInstancedStruct = const_cast<FInstancedStruct*>(Value.Get());
				ValueInstancedStruct->Serialize(Blob);

				// Multi-values
				if (Params.HasMultipleValues(ParamIndex))
				{
					const TMap<TArray<int32>, FParameterValue>& Multi = Params.MultiValues[ParamIndex];
					AddMultiValueKeys(Blob, Multi);
					for (const TPair<TArray<int32>, FParameterValue>& MultiValuePair : Multi)
					{
						const FParamInstancedStructType& MultiValue = MultiValuePair.Value.Get<FParamInstancedStructType>();

						FInstancedStruct* MultiValueInstancedStruct = const_cast<FInstancedStruct*>(MultiValue.Get());
						MultiValueInstancedStruct->Serialize(Blob);
					}
				}
				break;
			}
				
			default:
				// unsupported parameter type
				check(false);
			}
		}
    	
		int32 BlobBytes = Blob.GetNumBytes();
		ParameterValuesBlob.SetNum(BlobBytes);
		FMemory::Memcpy(ParameterValuesBlob.GetData(), Blob.GetData(), BlobBytes);
	}
	
    	
	FImageId FSystem::GetImageId(const TSharedPtr<FImageIdRegistry>& Registry, const FModel& Model, const FParameters& Params, FOperation::ADDRESS RootAt)
    {
    	check(Registry)
    
    	FGeneratedImageKey NewKey;
    	NewKey.Address = RootAt;

    	const int32* Result = Model.GetProgram().RelevantParameterList.Find(RootAt);
    	const int32 ParamListIndex = Result ? *Result : INDEX_NONE; 
    	GetResourceKey(Model, Params, ParamListIndex, NewKey.ParameterValuesBlob);

    	return Registry->Add(NewKey, {});
    }

	
	FMaterialId FSystem::GetMaterialId(const TSharedPtr<FMaterialIdRegistry>& Registry, const FModel& Model, const FParameters& Params, FOperation::ADDRESS RootAt)
    {
    	check(Registry)
    
    	FGeneratedMaterialKey NewKey;
    	NewKey.Address = RootAt;

    	const int32* Result = Model.GetProgram().RelevantParameterList.Find(RootAt);
    	const int32 ParamListIndex = Result ? *Result : INDEX_NONE; 
    	GetResourceKey(Model, Params, ParamListIndex, NewKey.ParameterValuesBlob);

    	return Registry->Add(NewKey, {});
    }

	
	FSkeletalMeshId FSystem::GetSkeletalMeshId(const TSharedPtr<FSkeletalMeshIdRegistry>& Registry, const FModel& Model, const FParameters& Params, FOperation::ADDRESS RootAt)
	{
		check(Registry)
	
    	FGeneratedSkeletalMeshKey NewKey;
    	NewKey.Address = RootAt;

    	const int32* Result = Model.GetProgram().RelevantParameterList.Find(RootAt);
    	const int32 ParamListIndex = Result ? *Result : INDEX_NONE; 
    	GetResourceKey(Model, Params, ParamListIndex, NewKey.ParameterValuesBlob);

    	return Registry->Add(NewKey, {});
	}


	// FLiveInstanceLogger ---------------------------------------

	void FLiveInstanceLogger::SetLogger(TSharedPtr<IMutableEditorLogger> InInstanceLogger)
	{
		InstanceLogger = InInstanceLogger;
	}


	void FLiveInstanceLogger::LogUpdateMessage(const FString& Message, ELogVerbosity::Type Verbosity)
	{
		switch (Verbosity)
		{
		case ELogVerbosity::Error:
			UE_LOGF(LogMutableCore, Error, "%ls", *Message);
			break;
		case ELogVerbosity::Warning:
			UE_LOGF(LogMutableCore, Warning, "%ls", *Message);
			break;
		case ELogVerbosity::Verbose:
			UE_LOGF(LogMutableCore, Verbose, "%ls", *Message);
			break;
		case ELogVerbosity::Display:
			UE_LOGF(LogMutableCore, Display, "%ls", *Message);
			break;
		case ELogVerbosity::Log:
		default:
			UE_LOGF(LogMutableCore, Log, "%ls", *Message);
			break;
		}

		if (InstanceLogger)
		{
			EMessageSeverity::Type MessageSeverity;

			switch (Verbosity)
			{
			case ELogVerbosity::Error:
				MessageSeverity = EMessageSeverity::Error;
				break;
			case ELogVerbosity::Warning:
				MessageSeverity = EMessageSeverity::Warning;
				break;
			case ELogVerbosity::Verbose:
			case ELogVerbosity::Log:
			case ELogVerbosity::Display:
			default:
				MessageSeverity = EMessageSeverity::Info;
				break;
			}

			TSharedRef<FTokenizedMessage> LogMessage = FTokenizedMessage::Create(MessageSeverity, FText::FromString(Message));
			LogMessage->SetIndentationLevel(1);
			TSharedPtr<IMutableEditorLogger> CopyMessageLogListing = InstanceLogger;

			// Required since slate widgets need to run in the gamethread
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [CopyMessageLogListing, LogMessage]() mutable
			{
				CopyMessageLogListing->LogMessage(LogMessage);
			});
		}
	}

}
