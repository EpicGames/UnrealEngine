// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"

#include "FloatAdapter.generated.h"

namespace UE::Mutable
{
	struct FMaterialAdapter;

	namespace Private
	{
		class FMesh;
		class CodeRunner;
	}


	USTRUCT()
	struct FFloatAdapter
	{
		GENERATED_BODY()
		
		friend Private::CodeRunner;
		friend FMaterialAdapter;

		FFloatAdapter() = default;
		
		FFloatAdapter(const FFloatAdapter& Other);
		
		FFloatAdapter(FFloatAdapter&& Other) = delete;

		FFloatAdapter& operator=(const FFloatAdapter& Other);
		
		FFloatAdapter& operator=(FFloatAdapter&& Other) = delete;

		MUTABLERUNTIME_API float GetValue() const;

		MUTABLERUNTIME_API void SetValue(float Value);
	
	private:
		float Value = false;
	};
}
