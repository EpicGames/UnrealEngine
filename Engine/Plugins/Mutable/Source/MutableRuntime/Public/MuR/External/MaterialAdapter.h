// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Value.h"
#include "MuR/Image.h"

#include "MaterialAdapter.generated.h"

namespace UE::Mutable
{
	struct FFloatAdapter;
	struct FVectorAdapter;
	struct FTextureAdapter;

	namespace Private
	{
		class FMaterial;
		class CodeRunner;

		struct FParameterKey;
	}

	
	USTRUCT()
	struct FMaterialAdapter
	{
		GENERATED_BODY()
		
		friend Private::CodeRunner;

		FMaterialAdapter();
		
		FMaterialAdapter(const FMaterialAdapter& Other);
		
		FMaterialAdapter(FMaterialAdapter&& Other) = delete;

		FMaterialAdapter& operator=(const FMaterialAdapter& Other);
		
		FMaterialAdapter& operator=(FMaterialAdapter&& Other) = delete;

		MUTABLERUNTIME_API TOptional<TValueConst<FFloatAdapter>> GetFloat(const Private::FParameterKey& ParameterKey) const;

		MUTABLERUNTIME_API bool SetFloat(const Private::FParameterKey& ParameterKey, const TValueConst<FFloatAdapter>* Value);
		
		MUTABLERUNTIME_API TOptional<TValueConst<FVectorAdapter>> GetVector(const Private::FParameterKey& ParameterKey) const;
		
		MUTABLERUNTIME_API bool SetVector(const Private::FParameterKey& ParameterKey, const TValueConst<FVectorAdapter>* Value);
		
		MUTABLERUNTIME_API TOptional<TValueConst<FTextureAdapter>> GetTexture(const Private::FParameterKey& ParameterKey) const;

		MUTABLERUNTIME_API bool SetTexture(const Private::FParameterKey& ParameterKey, const TValueConst<FTextureAdapter>* Value);
	
	private:
		Private::TManagedPtr<Private::FMaterial> Material;
	};
}
