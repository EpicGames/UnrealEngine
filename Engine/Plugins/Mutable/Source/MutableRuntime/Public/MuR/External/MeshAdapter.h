// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ManagedPointer.h"

#include "MeshAdapter.generated.h"

namespace UE::Mutable
{
	namespace Private
	{
		class FMesh;
		class CodeRunner;
	}


	USTRUCT()
	struct FMeshAdapter
	{
		GENERATED_BODY()
		
		friend Private::CodeRunner;

		FMeshAdapter();
		
		FMeshAdapter(const FMeshAdapter& Other);
		
		FMeshAdapter(FMeshAdapter&& Other) = delete;

		FMeshAdapter& operator=(const FMeshAdapter& Other);
		
		FMeshAdapter& operator=(FMeshAdapter&& Other) = delete;

		// Mesh interface
		MUTABLERUNTIME_API int32 GetNumVertices() const;

		MUTABLERUNTIME_API FVector3f GetVertex(int32 Index) const;

		MUTABLERUNTIME_API void SetVertex(int32 Index, const FVector3f& Vertex);
		
		MUTABLERUNTIME_API void RemoveVertices(const TBitArray<>& Vertices);
		
		// Internal use only. Use at your own risk. Public API not maintained.
		MUTABLERUNTIME_API Private::TManagedPtr<Private::FMesh> GetPrivate();
		MUTABLERUNTIME_API Private::TManagedPtr<const Private::FMesh> GetPrivate() const;
		
	private:
		Private::TManagedPtr<Private::FMesh> Mesh;
	};
}
