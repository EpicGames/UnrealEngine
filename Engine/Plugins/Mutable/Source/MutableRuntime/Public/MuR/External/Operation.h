// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "Value.h"
#include "MuR/ManagedPointer.h"
#include "StructUtils/InstancedStruct.h"

#include "Operation.generated.h"


namespace UE::Mutable
{
	class FContext;
	
	namespace Private
	{
		class CodeRunner;
	}
	
	/** Context of the External Operation. */
	class FContext
	{
		friend Private::CodeRunner;
		
	public:
		/** Returns and removes the input from the Context.
		 *
		 * Once called the owner is the requester. This allows CopyOrMove to move (see CopyOrMove for details). */
		MUTABLERUNTIME_API FValueConst GetInput(const FText& Name);

		/** Return the pointer to store the ouput. */
		MUTABLERUNTIME_API void SetOutput(FValueConst&& Value);
		
	private:
		TMap<const FString, FValueConst> Inputs;

		FValueConst Output;
	};
	

	/** External Operation Base class.
	 *
	 * All External Operations must inherit from this class.
	 *
	 * Due to Mutable being a Pure Functional VM, all operations must not store information (no members, globals, statics...).
	 * Furthermore, due to cache, given the same inputs an operation must always produce the same output (no randomness).
	 *
	 * An operation can have UProperty members but they are considered constants (they are copied when compiling and must not be modified at runtime).
	 * UProperties marked as Visible/EditAnywhere are automatically shown in the Node Details panel. */
	USTRUCT()
	struct FExternalOperation
	{
		GENERATED_BODY()
		
		virtual ~FExternalOperation() = default;

		/** Version of the operation. If it differs from the compiled one, it will automatically trigger a compilation. */
		virtual uint32 GetVersion() const { return 0; }
		
		/** Static. It must always return the same inputs. */
		virtual TArray<TPair<FText, const UScriptStruct*>> GetInputs() const { return {}; }

		/** Static. It must always return the same output. */
		virtual TPair<FText, const UScriptStruct*> GetOutput() const { return {}; }

		/** This function must be pure functional (same inputs, same output). */
		virtual void Evaluate(FContext& Context) const {}
	};
}
