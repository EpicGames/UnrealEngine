// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MuR/ManagedPointer.h"
#include "TextureAdapter.generated.h"


namespace UE::Mutable
{
	struct FMaterialAdapter;

	namespace Private
	{
		class FImage;
		class CodeRunner;
	}


	USTRUCT()
	struct FTextureAdapter
	{
		GENERATED_BODY()
		
		friend Private::CodeRunner;
		friend FMaterialAdapter;

		FTextureAdapter();
		
		FTextureAdapter(const FTextureAdapter& Other);
		
		FTextureAdapter(FTextureAdapter&& Other) = delete;

		FTextureAdapter& operator=(const FTextureAdapter& Other);
		
		FTextureAdapter& operator=(FTextureAdapter&& Other) = delete;
		
	private:
		Private::TManagedPtr<Private::FImage> Image;
	};
}
