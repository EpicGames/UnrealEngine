// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "MuR/ManagedPointer.h"

struct FInstancedStruct;

namespace UE::Mutable
{
	namespace Private
	{
		class CodeRunner;
	}
	class FValueConst;
	class FContext;
	struct FMaterialAdapter;
	template<typename T> class TValue;
	template<typename T> class TValueConst;

	
	class FValue
	{
		friend Private::CodeRunner;
		friend FValueConst;
		friend MUTABLERUNTIME_API FValue CopyOrMove(FValueConst&&);
		
	public:
		template<typename T>
		static FValue New()
		{
			FValue Value;
			Value.Ptr = Private::MakeManaged<FInstancedStruct>();
			Value.Ptr->InitializeAs(T::StaticStruct());

			return Value;
		}
		
		FValue(const FValue& Other) = delete;
		
		FValue& operator=(const FValue& Other) = delete;

		FValue(FValue&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
		}

		FValue& operator=(FValue&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
			return *this;
		}

		FValue(const UScriptStruct* ScriptStruct)
		{
			Ptr = Private::MakeManaged<FInstancedStruct>();
			Ptr->InitializeAs(ScriptStruct);
		}
		
		template <typename T>
		T& Get()
		{
			return Ptr.Get()->GetMutable<T>();
		}

		template <typename T>
		T* GetPtr()
		{
			return Ptr.Get()->GetMutablePtr<T>();
		}
		
	private:
		FValue() = default;
		
		Private::TManagedPtr<FInstancedStruct> Ptr;
	};


	class FValueConst
	{
		friend Private::CodeRunner;
		friend FContext;
		friend MUTABLERUNTIME_API FValue CopyOrMove(FValueConst&&);

	public:
		template<typename T>
		static FValueConst New()
		{
			FValueConst Value;
			Value.Ptr = Private::MakeManaged<const FInstancedStruct>();
			Value.Ptr->InitializeAs(T::StaticStruct());

			return Value;
		}
		
		FValueConst(const FValueConst& Other) = delete;
		
		FValueConst& operator=(const FValueConst& Other) = delete;

		FValueConst(FValueConst&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
		}

		FValueConst& operator=(FValueConst&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
			return *this;
		}
		
		MUTABLERUNTIME_API FValueConst(FValue&& Value);

		template <typename T>
		const T& Get() const
		{
			return Ptr.Get()->Get<const T>();
		}

		template <typename T>
		const T* GetPtr() const
		{
			return Ptr.Get()->GetPtr<T>();
		}
		
	private:
		FValueConst() = default;
		
		Private::TManagedPtr<const FInstancedStruct> Ptr;
	};

	
	template<typename T>
	class TValue
	{
		friend Private::CodeRunner;
		friend FMaterialAdapter;
		friend FValueConst;
		template<typename U> friend class TValueConst;
		template<typename U> friend MUTABLERUNTIME_API TValue CopyOrMove(TValueConst<U>&&);
		
	public:
		static TValue New()
		{
			TValue Value;
			Value.Ptr = Private::MakeManaged<FInstancedStruct>();
			Value.Ptr->InitializeAs(T::StaticStruct());

			return Value;
		}
		
		TValue(const TValue& Other) = delete;
		
		TValue& operator=(const TValue& Other) = delete;

		TValue(TValue&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
		}

		TValue& operator=(TValue&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
			return *this;
		}

		TValue(const UScriptStruct* ScriptStruct)
		{
			Ptr = Private::MakeManaged<FInstancedStruct>();
			Ptr->InitializeAs(ScriptStruct);
		}
		
		T& Get()
		{
			return Ptr.Get()->template GetMutable<T>();
		}

		T* GetPtr()
		{
			return Ptr.Get()->template GetMutablePtr<T>();
		}
		
	private:
		TValue() = default;
		
		Private::TManagedPtr<FInstancedStruct> Ptr; // TODO GMT Extension TVariant Lazy	
	};
	

	template<typename T>
	class TValueConst
	{
		friend Private::CodeRunner;
		friend FContext;
		template<typename U> friend class TValue;
		template<typename U> friend MUTABLERUNTIME_API TValue<U> CopyOrMove(TValueConst&&);
		
	public:
		static TValueConst New()
		{
			TValueConst Value;
			Value.Ptr = Private::MakeManaged<const FInstancedStruct>();
			Value.Ptr->InitializeAs(T::StaticStruct());

			return Value;
		}
		
		TValueConst(const FValueConst& Other) = delete;
		
		TValueConst& operator=(const FValueConst& Other) = delete;

		TValueConst(TValueConst&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
		}

		TValueConst& operator=(TValueConst&& Other)
		{
			Ptr = MoveTemp(Other.Ptr);
			return *this;
		}

		TValueConst(TValue<T>&& Value)
		{
			Ptr = MoveTemp(Value.Ptr);
		}
		
		const T& Get() const
		{
			return Ptr.Get()->template Get<const T>();
		}

		const T* GetPtr() const
		{
			return Ptr.Get()->template GetPtr<T>();
		}
		
	private:
		TValueConst() = default;
		
		Private::TManagedPtr<const FInstancedStruct> Ptr;
	};
	
	
	/** Copy (or move) a value to remove it's const.
	 *
	 * Instead of copy, a value will be moved if it is uniquely referenced. A value may not be unique due to:
	 * - Being being used in more than one subsequent input.
	 * - User holding additional references to it. Use MoveTemp whenever possible. */
	MUTABLERUNTIME_API FValue CopyOrMove(FValueConst&& ValueConst);

	template<typename T>
	TValue<T> CopyOrMove(TValueConst<T>&& ValueConst)
	{
		if (ValueConst.Ptr.IsUnique())
		{
			TValue<T> Value;
			Value.Ptr = Private::ConstCastManagedPtr<FInstancedStruct>(ValueConst.Ptr); 

			return MoveTemp(Value);
		}
		else
		{
			TValue<T> Value;
			Value.Ptr = Private::MakeManaged<FInstancedStruct>(*ValueConst.Ptr); 

			return MoveTemp(Value);
		}
	}
}
