// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"

#include "VectorAdapter.generated.h"

namespace UE::Mutable
{
	struct FMaterialAdapter;

	namespace Private
	{
		class FMesh;
		class CodeRunner;
	}


	USTRUCT()
	struct FVectorAdapter
	{
		GENERATED_BODY()
		
		friend Private::CodeRunner;
		friend FMaterialAdapter;

		FVectorAdapter() = default;
		
		FVectorAdapter(const FVectorAdapter& Other);
		
		FVectorAdapter(FVectorAdapter&& Other) = delete;

		FVectorAdapter& operator=(const FVectorAdapter& Other);
		
		FVectorAdapter& operator=(FVectorAdapter&& Other) = delete;

		MUTABLERUNTIME_API FVector4f GetValue() const;

		MUTABLERUNTIME_API void SetValue(const FVector4f& Value);
	
	private:
		FVector4f Value = FVector4f::Zero();
	};
}
