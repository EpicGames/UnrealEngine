// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Compiler.h"
#include "MuT/NodeImage.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"
#include "Templates/Function.h"
#include "Hash/CityHash.h"

#include <array>
#include <atomic>
#include <unordered_map>

namespace std
{

  template<typename T>
  struct hash<UE::Mutable::Private::Ptr<T>>
  {
    uint64 operator()(const UE::Mutable::Private::Ptr<T>& k) const
    {
      return hash<const void*>()(k.get());
    }
  };

}


namespace UE::Mutable::Private
{
	class ASTOp;
	class ASTOpFixed;
	class ASTOpImageCrop;
	class ASTOpImagePixelFormat;
	class ASTOpMeshFormat;
	class ASTOpImageSwizzle;
	class ASTOpImageMipmap;
	class ASTOpMeshExtractLayoutBlocks;

	template<typename T>
	inline uint32 GetTypeHash(const Ptr<const T>& p)
	{
		return ::GetTypeHash(p.get());
	}


	template<typename T>
	inline uint32 GetTypeHash(const Ptr<T>& p)
	{
		return ::GetTypeHash(p.get());
	}


    //---------------------------------------------------------------------------------------------
    //! This class stores the expression that defines the size of an image.
    //--------------------------------------------------------------------------------------------
    class ImageSizeExpression : public RefCounted
    {
    public:

        enum
        {
            ISET_UNKNOWN,
            ISET_CONSTANT,
            ISET_LAYOUTFACTOR,
            ISET_CONDITIONAL
        } type;

        // For constant sizes
        FImageSize size = FImageSize(0, 0);

        // For layout factor sizes
        Ptr<class ASTOp> layout;
        uint16 factor[2];

        // For conditionals
        Ptr<class ASTOp> condition;
        Ptr<ImageSizeExpression> yes;
        Ptr<ImageSizeExpression> no;

        //!
        ImageSizeExpression()
        {
            type = ISET_UNKNOWN;
            size[0] = 0;
            size[1] = 0;
            layout = 0;
            factor[0] = 0;
            factor[1] = 0;
            condition = 0;
        }

        //!
        void CopyFrom(const ImageSizeExpression& o)
        {
            type = o.type;
            size = o.size;
            layout = o.layout;
            factor[0] = o.factor[0];
            factor[1] = o.factor[1];
            condition = o.condition;
            yes = o.yes;
            no = o.no;
        }

        //!
        bool operator==( const ImageSizeExpression& other ) const
        {
            if ( type == other.type )
            {
                switch (type)
                {
                case ISET_CONSTANT:
                    return size[0]==other.size[0]
                            && size[1] == other.size[1];

                case ISET_LAYOUTFACTOR:
                    return layout==other.layout
                            && factor[0]==other.factor[0]
                            && factor[1]==other.factor[1];

                case ISET_CONDITIONAL:
                    return condition==other.condition
                            && *yes==*(other.yes)
                            && *no==*(other.no);

                default:
                    return false;
                }
            }

            return false;
        }

        void Optimise()
        {
            switch (type)
            {
            case ISET_CONSTANT:
                break;

            case ISET_LAYOUTFACTOR:
                // TODO: See if the layout is constant and so is this expression.
                break;

            case ISET_CONDITIONAL:
                yes->Optimise();
                no->Optimise();

                // TODO: See if the condition is constant
                if ( *yes==*no )
                {
                    CopyFrom( *yes );
                }

            default:
                break;
            }
        }

    };


    typedef TArray<Ptr<ASTOp>> ASTOpList;
    typedef TSet<Ptr<ASTOp>> ASTOpSet;


    //! Detailed optimization flags
    struct FModelOptimizationOptions
    {
        bool bEnabled = true;
        bool bOptimiseOverlappedMasks = false;
        bool bConstReduction = true;

        //! Preprocess all mesh fragments so that they use the same skeleton, even if not all bones
        //! are relevant for all fragments.
        bool bUniformizeSkeleton = true;

        //! Maximum number of iterations when optimising models. If 0 as many as necessary will be performed.
        int32 MaxOptimisationLoopCount = 8;

		/** Compile optimizing for the generation of smaller mipmaps of every image. */
		bool bEnableProgressiveImages = false;

        // Additional advanced fine-tuning parameters
        //---------------------------------------------------------------------

        //! Ratio used to decide if it is worth to generate a crop operation
        float AcceptableCropRatio = 0.5f;

        //! Ratio used to decide if it is worth to generate a crop operation
        float MinRLECompressionGain = 1.2f;

		// External resource provision functions
		//---------------------------------------------------------------------

		/** Function used to request an engine resource from the compiler. */
		FReferencedImageResourceFunc ReferencedImageResourceProvider;
		FReferencedMeshResourceFunc ReferencedMeshResourceProvider;

        bool bDisableMaterialGeneration = false;
	};


	struct FLinkerOptions
	{
		FLinkerOptions(FImageOperator& InImOp)
			: ImageOperator(InImOp)
		{
		}

		// TODO: Unused?
		int32 MinTextureResidentMipCount = 0;

		/** This flag controls the splitting of image data into mips to store separately. It is usually necessary to
		* be able to generate progressive textures (for texture streaming).
		*/
		bool bSeparateImageMips = true;

		/** Structure used to speedup mesh constant comparison. */
		struct FDeduplicationMeshFuncs : TDefaultMapHashableKeyFuncs<TManagedPtr<const FMesh>, int32, false>
		{
			static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
			{
				return *A == *B;
			}

			static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
			{
				const UE::Mutable::Private::FMesh* Data = Key.Get();
				return HashCombineFast(
					::GetTypeHash(Data->VertexBuffers.GetElementCount()),
					::GetTypeHash(Data->IndexBuffers.GetElementCount())
				);
			}
		};

		TMap<TManagedPtr<const FMesh>, int32, FDefaultSetAllocator, FDeduplicationMeshFuncs> MeshConstantMap;

		/** Structure used to speedup image mip comparison. */
		struct FDeduplicationImageFuncs : TDefaultMapHashableKeyFuncs<TManagedPtr<const FImage>, int32, false>
		{
			static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
			{
				return *A == *B;
			}

			static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
			{
				const UE::Mutable::Private::FImage* Data = Key.Get();
				uint32 Hash = HashCombineFast(::GetTypeHash(Data->GetFormat()), GetTypeHash(Data->GetSize()));

				TArrayView<const uint8> DataView = Data->DataStorage.GetLOD(0);
				uint64 DataHash = CityHash64(reinterpret_cast<const char*>(DataView.GetData()), DataView.Num());
				Hash = HashCombineFast(Hash, ::GetTypeHash(DataHash));

				return Hash;
			}
		};

		TMap<TManagedPtr<const FImage>, int32, FDefaultSetAllocator, FDeduplicationImageFuncs> ImageConstantMipMap;

		/** Image operation functions, so that they can be overriden. */
		FImageOperator& ImageOperator;

		/** Store for additional data generated during compilation, but not necessary for the runtime. */
		struct FAdditionalData
		{
			/** Source data descriptor for every image constant that has been generated.
			* It must have the same size than the Program::ConstantImages array.
			*/
			TArray<FSourceDataDescriptor> SourceImagePerConstant;

			/** Source data descriptor for every mesh constant that has been generated.
			* It must have the same size than the Program::ConstantMeshes array.
			*/
			TArray<FSourceDataDescriptor> SourceMeshPerConstant;
		};

		FAdditionalData AdditionalData;

		TMap<FOperation::ADDRESS, FInstancedStruct>* ExternalOperations = nullptr;
	};


	//! For each operation we sink, the map from old instructions to new instructions.
	struct FSinkerOldToNewKey
	{
		Ptr<const ASTOp> Op;
		Ptr<const ASTOp> SinkingOp;

		friend inline uint32 GetTypeHash(const FSinkerOldToNewKey& Key)
		{
			return HashCombineFast(::GetTypeHash(Key.Op.get()), ::GetTypeHash(Key.SinkingOp.get()));
		}

		friend inline bool operator==(const FSinkerOldToNewKey& A, const FSinkerOldToNewKey& B)
		{
			return A.Op==B.Op && A.SinkingOp==B.SinkingOp;
		}
	};


	/** */
	class Sink_ImageCropAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpImageCrop* root);

	protected:

		const ASTOpImageCrop* Root = nullptr;
		Ptr<ASTOp> InitialSource;
		//! For each operation we sink, the map from old instructions to new instructions.
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(Ptr<ASTOp> at, const ASTOpImageCrop* currentCropOp);
	};


	/** */
	class Sink_ImagePixelFormatAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpImagePixelFormat* root);

	protected:

		const ASTOpImagePixelFormat* Root = nullptr;
		Ptr<ASTOp> InitialSource;
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(Ptr<ASTOp> at, const ASTOpImagePixelFormat* currentFormatOp);
	};


	/** */
	class Sink_ImageSwizzleAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpImageSwizzle* Root);

	protected:

		const ASTOpImageSwizzle* Root = nullptr;
		Ptr<ASTOp> InitialSource;
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(Ptr<ASTOp> at, const ASTOpImageSwizzle* CurrentSwizzleOp);
	};


	/** */
	class Sink_MeshFormatAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpMeshFormat* Root);

	protected:

		const ASTOpMeshFormat* Root = nullptr;
		Ptr<ASTOp> InitialSource;
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(const Ptr<ASTOp>& Op, const ASTOpMeshFormat* CurrentSinkOp);
	};


	/** */
	class Sink_MeshExtractLayoutBlocksAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpMeshExtractLayoutBlocks* Root);

	protected:

		const ASTOpMeshExtractLayoutBlocks* Root = nullptr;
		Ptr<ASTOp> InitialSource;
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(const Ptr<ASTOp>& at, const ASTOpMeshExtractLayoutBlocks* currentFormatOp);
	};


	/** */
	class Sink_ImageMipmapAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpImageMipmap* Root);

	protected:

		const ASTOpImageMipmap* Root = nullptr;
		Ptr<ASTOp> InitialSource;

		//! For each operation we sink, the map from old instructions to new instructions.
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(Ptr<ASTOp> at, const ASTOpImageMipmap* currentMipmapOp);

	};

	struct FOptimizeSinkContext
	{
		Sink_ImageCropAST ImageCropSinker;
		Sink_ImagePixelFormatAST ImagePixelFormatSinker;
		Sink_ImageSwizzleAST ImageSwizzleSinker;
		Sink_ImageMipmapAST ImageMipmapSinker;
		Sink_MeshFormatAST MeshFormatSinker;
		Sink_MeshExtractLayoutBlocksAST MeshExtractLayoutBlocksSinker;
	};


    //!
    class ASTChild
    {
    public:
    	struct FInvalidTag {};
    	
        explicit ASTChild(ASTOp* parent, const Ptr<ASTOp>& child=Ptr<ASTOp>());
        ASTChild(const Ptr<ASTOp>& parent, const Ptr<ASTOp>& child);
        ~ASTChild();

        ASTChild(const ASTChild&) = delete;
        ASTChild& operator=(const ASTChild&) = delete;

    	explicit ASTChild(FInvalidTag) {}

        // move constructor    	
        ASTChild(ASTChild&& rhs)
             : Parent(rhs.Parent)
             , Child(rhs.Child)
             , ParentIndexInChild(rhs.ParentIndexInChild)
        {
            rhs.Parent=nullptr;
            rhs.Child.reset();
        }

        // Move assignment
        ASTChild& operator=( ASTChild&& );

        ASTChild& operator=( const Ptr<ASTOp>& );

        inline explicit operator bool() const
        {
            return Child.get()!=nullptr;
        }

        inline Ptr<class ASTOp>& child()
        {
            return Child;
        }

        inline const Ptr<class ASTOp>& child() const
        {
            return Child;
        }

        inline const Ptr<class ASTOp>& operator->() const
        {
            return Child;
        }

        inline bool operator==(const ASTChild& o) const
        {
            return Child==o.Child;
        }

        class ASTOp* Parent;
        Ptr<class ASTOp> Child;
        int32 ParentIndexInChild = 0;

    private:

        inline void AddParent();

        inline void ClearParent();
    };

	
	inline uint32 GetTypeHash(const ASTChild& Child)
	{
		return PointerHash(Child.child().get());
	}


    //---------------------------------------------------------------------------------------------
    //! Abstract Syntax Tree of operations in the mutable virtual machine.
    //! Avoid any kind of recursivity here, since the hierarchy can be very deep, and it will
    //! easily cause stack overflows with production models.
    //---------------------------------------------------------------------------------------------
    class ASTOp : public RefCounted
    {
    private:

        //! Operations referring to this one. They may be null: elements are never removed
        //! from this TArray.
		TArray<ASTOp*, TInlineAllocator<4> > Parents;

		UE::FMutex ParentsMutex;

    public:

        inline ASTOp()
        {
			bIsConstantSubgraph = false;
			bHasSpecialOpInSubgraph = false;
        }

        virtual ~ASTOp() = default;

        //! Get the operation type
        virtual EOpType GetOpType() const = 0;

        //! Validate that everything is fine with this tree
        virtual void Assert();

        //! Run something for each child operation, with a chance to modify it.
        virtual void ForEachChild( const TFunctionRef<void(ASTChild&)> ) = 0;

        //! Run something for each parent operation+.
		void ForEachParent(const TFunctionRef<void(ASTOp*)>) const;

        //! Run something for each child operation, with a chance to modify it.
        virtual bool operator==( const ASTOp& other ) const;

        //! Hint hash method for op sorting and containers. It's a hash of the actual operation.
        virtual uint32 Hash() const = 0;

        //! Shallow clone. New node will have no parents but reference to the same children.
		using MapChildFunc = TFunction<Ptr<ASTOp>(const Ptr<ASTOp>&)>;
		using MapChildFuncRef = TFunctionRef<Ptr<ASTOp>(const Ptr<ASTOp>&)>;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const = 0;

		//
		virtual bool IsConditional() const { return false; }
		virtual bool IsSwitch() const { return false; }

    protected:

        void RemoveChildren();

        virtual bool IsEqual(const ASTOp& other) const = 0;

    public:

        //---------------------------------------------------------------------------------------------
        static void FullAssert( const TArray<Ptr<ASTOp>>& roots );

        static int32 CountNodes( const TArray<Ptr<ASTOp>>& roots );

		inline bool IsConstantOp() const
		{
			EOpType Type = GetOpType();
			return Type == EOpType::BO_CONSTANT
				|| Type == EOpType::NU_CONSTANT
				|| Type == EOpType::SC_CONSTANT
				|| Type == EOpType::CO_CONSTANT
				|| Type == EOpType::IM_CONSTANT
				|| Type == EOpType::ME_CONSTANT
				|| Type == EOpType::LA_CONSTANT
				|| Type == EOpType::PR_CONSTANT
				|| Type == EOpType::ST_CONSTANT
				|| Type == EOpType::ED_CONSTANT
				;
		}

        //! Deep clone. New node will have no parents and reference new children
        static Ptr<ASTOp> DeepClone( const Ptr<ASTOp>& );

		//!
		static void LogHistogram(ASTOpList& roots);

        // Code optimisation methods
        //---------------------------------------------------------------------------------------------

        //!
        virtual Ptr<ASTOp> OptimiseSize() const { return nullptr; }
        virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const { return nullptr; }
        virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext& ) const { return nullptr; }
        virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const
        {
            check( false );
            return nullptr;
        }


        // Code linking
        //---------------------------------------------------------------------------------------------

		/** Convert the operation graph at Root into code in the given program.
		* Potentially destroys the data in this operation, so it shouldn't be used after calling Link.
		*/
		static FOperation::ADDRESS FullLink( Ptr<ASTOp>& Root, FProgram&, FLinkerOptions* );

    private:

        /** Convert this operation into code in the given program. 
		* It assumes children have been linked already
		* Potentially destroys the data in this operation, so it shouldn't be used after calling Link. 
		*/
        virtual void Link( FProgram&, FLinkerOptions* ) = 0;

    protected:

        //---------------------------------------------------------------------------------------------
        //!
        //---------------------------------------------------------------------------------------------
        struct FRangeData
        {
            //!
            ASTChild rangeSize;

            //!
            FString rangeName;

            //!
			FString rangeUID;

            //!
            FRangeData( ASTOp* parentOp, Ptr<ASTOp> childOp, const FString& name, const FString& uid )
                : rangeSize( parentOp, childOp )
                , rangeName(name)
                , rangeUID(uid)
            {
            }

            FRangeData(const FRangeData&) = delete;
            FRangeData& operator=(const FRangeData&) = delete;
            FRangeData& operator=(FRangeData&&) = delete;


            // move constructor
            FRangeData(FRangeData&& rhs)
                 : rangeSize(MoveTemp(rhs.rangeSize))
                 , rangeName(MoveTemp(rhs.rangeName))
                 , rangeUID(MoveTemp(rhs.rangeUID))
            {
            }

            //!

            //!
            bool operator==(const FRangeData& o) const
            {
                return rangeSize==o.rangeSize
                        &&
                        rangeName==o.rangeName
                        &&
                        rangeUID==o.rangeUID;
            }
        };

        //!
		static void LinkRange(FProgram& program,
			const FRangeData& range,
			FOperation::ADDRESS& rangeSize,
			uint16& rangeId);

    public:

        // Generic traversals
        //---------------------------------------------------------------------------------------------

        //!
        static void Traverse_TopDown_Unique( const TArray<Ptr<ASTOp>>& roots, TFunctionRef<bool(Ptr<ASTOp>&)> f );

        //! \todo: it is not strictly top down.
        static void Traverse_TopDown_Unique_Imprecise( const TArray<Ptr<ASTOp>>& roots, TFunctionRef<bool(Ptr<ASTOp>&)> f );

        //! Kind of top-down, but really not.
        //! This version is slighlty faster, but doesn't support recursive traversals so
        //! use it only in controlled cases.
        static void Traverse_TopRandom_Unique_NonReentrant
            (
                const TArray<Ptr<ASTOp>>& roots,
				TFunctionRef<bool(Ptr<ASTOp>&)> f
            );


		template<typename STATE>
		struct FKeyFuncs : BaseKeyFuncs<TPair<Ptr<ASTOp>, STATE>, Ptr<ASTOp>, false>
		{
			static Ptr<ASTOp> GetSetKey(TPair<Ptr<ASTOp>, STATE> Element) { return Element.Key; }
			static bool Matches(const Ptr<ASTOp>& lhs, const Ptr<ASTOp>& rhs) { return lhs == rhs || *lhs == *rhs; }
			static uint32 GetKeyHash(const Ptr<ASTOp>& Key) { return Key->Hash(); }
		};


        //! Kind of top-down, but really not.
        //! This version is slightly faster, but doesn't support recursive traversals so
        //! use it only in controlled cases.
        template<typename STATE>
        static inline void Traverse_TopDown_Unique_Imprecise_WithState
            (
                Ptr<ASTOp>& root, const STATE& initialState,
				TFunctionRef<bool(Ptr<ASTOp>&, STATE&, TArray<TPair<Ptr<ASTOp>,STATE>>&)> f
            )
        {
            if (!root) { return; }

            TArray<TPair<Ptr<ASTOp>,STATE>> Pending;
			Pending.Emplace( root, initialState );


            struct custom_partial_hash
            {
				uint64 operator()(const std::pair<Ptr<ASTOp>,STATE>& k) const
                {
                    return std::hash<const void*>()(k.first.get());
                }
            };
            TSet<TPair<Ptr<ASTOp>,STATE>, FKeyFuncs<STATE>> Traversed;

            while (!Pending.IsEmpty())
            {
				TPair<Ptr<ASTOp>,STATE> current = Pending.Pop();

                // It could have been completed in another branch
				bool bAlreadyAdded = false;
				Traversed.Add({ current.Key,current.Value }, &bAlreadyAdded);
                if (!bAlreadyAdded)
                {
                    // Process. State in current.Value may change
                    bool bRecurse = f(current.Key,current.Value,Pending);

                    // Recurse children
                    if (bRecurse)
                    {
                        current.Key->ForEachChild([&]( ASTChild& c )
                        {
							if (c.Child && !Traversed.Contains(c.Child))
                            {
								Pending.Emplace( c.Child, current.Value );
                            }
                        });
                    }
                }
            }
        }


        //! This version is slighlty faster, but doesn't support recursive traversals so
        //! use it only in controlled cases.
        static void Traverse_BottomUp_Unique_NonReentrant( ASTOpList& roots, TFunctionRef<void(Ptr<ASTOp>&)> f );

        //! This version is slighlty faster, but doesn't support recursive traversals so
        //! use it only in controlled cases.
        static void Traverse_BottomUp_Unique_NonReentrant
        (
            ASTOpList& roots,
			TFunctionRef<void(Ptr<ASTOp>&)> f,
			TFunctionRef<bool(const ASTOp*)> accept
        );

        //!
        static void Traverse_BottomUp_Unique
        (
            ASTOpList& roots,
			TFunctionRef<void(Ptr<ASTOp>&)> f,
			TFunctionRef<bool(const ASTOp*)> accept = [](const ASTOp*){return true;}
        );

        //!
        static void Traverse_BottomUp_Unique
        (
            Ptr<ASTOp>& root,
			TFunctionRef<void(Ptr<ASTOp>&)> f,
			TFunctionRef<bool(const ASTOp*)> accept = [](const ASTOp*){return true;}
        );

        //!
        FOperation::ADDRESS LinkedAddress = 0;

        //! Generic traverse control counter. It should always be left to 0 after any process for all
        //! nodes in the hierarchy.
        uint32 m_traverseIndex = 0;
        static std::atomic<uint32> s_lastTraverseIndex;

        //!
        int8 linkedRange = -1;

        /** Embedded node data for the constant subtree detection.This flag is only valid if the
         * constant detection process has been executed and no relevant AST transformations have
         * happened. */
        uint8 bIsConstantSubgraph : 1;
        uint8 bHasSpecialOpInSubgraph : 1;

    private:
        friend class ASTChild;

        //! Remove a node from the parent list if it is there.
        //void RemoveParent(const ASTOp* parent);

    public:

		/** Get the number of ASTOps that have this one as child, in any existing AST graph. */
        int32 GetParentCount() const;

        //! Make all parents of this node point at the other node instead.
        static void Replace( const Ptr<ASTOp>& node, const Ptr<ASTOp>& other );

        // Other code generation utilities
        //---------------------------------------------------------------------------------------------

        //! This class contains the support data to accelerate the GetImageDesc recursive function.
        //! If none is provided in the call, one will be created at that level and used from there on.
        class FGetImageDescContext
        {
        public:
            TMap<const ASTOp*, FImageDesc> m_results;
			bool bIsFormatFromImageParameter = false;
			FParameterKey ParameterKey;
        };

        //!
        virtual FImageDesc GetImageDesc( bool returnBestOption=false, FGetImageDescContext* context=nullptr ) const;

		/** */
		class FGetSourceDataDescriptorContext
		{
		public:
			TMap<const ASTOp*, FSourceDataDescriptor> Cache;
		};

		/** */
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext* = nullptr) const;

        //! Optional cache struct to use int he method below.
        using FBlockLayoutSizeCache=TMap< const TPair<ASTOp*,uint64>, TPair<int32,int32>>;

        //! Return the size in layout blocks of a particular block given by absolute index
        virtual void GetBlockLayoutSize( uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache );
        void GetBlockLayoutSizeCached(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache );


        //! Return the size in pixels of the layout grid block for the image operation
        virtual void GetLayoutBlockSize( int32* pBlockX, int32* pBlockY );

        virtual bool IsImagePlainConstant(FVector4f& color ) const;
        virtual bool IsColorConstant(FVector4f& color ) const;

        //!
        virtual bool GetNonBlackRect( FImageRect& maskUsage ) const;

		enum class EClosedMeshTest : uint8
		{
			No,
			Yes,
			Unknown
		};

		/** This can be overriden to help detect subgraph mesh properties.  */
		virtual EClosedMeshTest IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache=nullptr ) const;

        // Logic expression evaluation
        //---------------------------------------------------------------------------------------------
        typedef enum
        {
            BET_UNKNOWN,
            BET_TRUE,
            BET_FALSE,
        } FBoolEvalResult;


        using FEvaluateBoolCache = std::unordered_map<const ASTOp*,FBoolEvalResult>;

        virtual FBoolEvalResult EvaluateBool( ASTOpList& /*facts*/, FEvaluateBoolCache* = nullptr ) const
        {
            check(false);
            return BET_UNKNOWN;
        }

        virtual int EvaluateInt( ASTOpList& /*facts*/, bool &unknown ) const
        {
            check(false);
            unknown = true;
            return 0;
        }

    };

    template<class DERIVED>
    inline Ptr<DERIVED> Clone( const ASTOp* s )
    {
		ASTOp::MapChildFunc Identity = [](const Ptr<ASTOp>& o) {return o; };
		Ptr<ASTOp> c = s->Clone(Identity);
		Ptr<DERIVED> t = static_cast<DERIVED*>(c.get());
        return t;
    }

    template<class DERIVED>
    inline Ptr<DERIVED> Clone( const Ptr<const ASTOp>& s )
    {
		ASTOp::MapChildFunc Identity = [](const Ptr<ASTOp>& o) {return o; };
		Ptr<ASTOp> c = s->Clone(Identity);
        Ptr<DERIVED> t = static_cast<DERIVED*>(c.get());
		return t;
    }
	

    //---------------------------------------------------------------------------------------------
    //!
    //---------------------------------------------------------------------------------------------
    template<typename STATE>
    class Visitor_TopDown_Unique_Const
    {
    private:

        /** @return if true, visit children. */
        virtual bool Visit( const Ptr<ASTOp>& node ) = 0;

    private:

        //! States found so far
        TArray<STATE> States;

        //! Index of the current state, from the States TArray.
        int32 CurrentState;

        struct FPending
        {
			FPending()
            {
                StateIndex = 0;
            }
			FPending(Ptr<ASTOp> InAt, int32 InStateIndex)
            {
                At = InAt;
                StateIndex = InStateIndex;
            }

            Ptr<ASTOp> At;
            int32 StateIndex;
        };

        TArray<FPending> Pending;

        //! List of Traversed nodes with the state in which they were Traversed.
        TMultiMap<Ptr<ASTOp>,int32> Traversed;

    protected:

        //!
        const STATE& GetCurrentState() const
        {
            return States[CurrentState];
        }

        //!
        STATE GetDefaultState() const
        {
            return States[0];
        }

        //! For manual recursion that changes the state for a specific path.
        void RecurseWithState(const Ptr<ASTOp>& InAt, const STATE& NewState)
        {
            if(InAt)
            {
				int32 Index = States.Find(NewState);
                if (Index==INDEX_NONE)
                {
                    States.Add(NewState);
                }
                int32 StateIndex = States.Find(NewState);

                Pending.Add( FPending(InAt,StateIndex) );
            }
        }

        //! For manual recursion that doesn't change the state for a specific path.
        void RecurseWithCurrentState(const Ptr<ASTOp>& InAt)
        {
            if(InAt)
            {
                Pending.Emplace(InAt, CurrentState );
            }
        }

        //! Can be called from visit to set the state to visit all children ops
        void SetCurrentState(const STATE& NewState)
        {
			int32 Index = States.Find(NewState);
            if (Index == INDEX_NONE)
            {
                States.Add(NewState);
            }
            CurrentState = States.Find(NewState);
        }

    public:

        //! Ensure virtual destruction
        virtual ~Visitor_TopDown_Unique_Const() = default;

        //!
        void Traverse(const ASTOpList& Roots, const STATE& InitialState)
        {
            Pending.Empty();
            States.Empty();
            States.Add(InitialState);
            CurrentState = 0;

            for( const Ptr<ASTOp>& Root: Roots)
            {
                if (Root)
                {
                    Pending.Add(FPending(Root,CurrentState) );
                }
            }

            while (Pending.Num())
            {
                FPending Item = Pending.Pop();

				TArray<int32> Found;
                Traversed.MultiFind(Item.At, Found);
                bool bVisitedInThisState = false;
                for (int32 Value: Found)
                {
                    if (Value==Item.StateIndex)
                    {
                        bVisitedInThisState = true;
                        break;
                    }
                }

                // It could have been completed in another branch
                if (!bVisitedInThisState)
                {
					Traversed.Add( Item.At, Item.StateIndex);

                    // Process
                    CurrentState = Item.StateIndex;
                    bool bRecurse = Visit(Item.At);

                    // Recurse children
                    if (bRecurse)
                    {
                        Item.At->ForEachChild([&]( ASTChild& c )
                        {
                            if (c)
                            {
                                Pending.Add( FPending(c.child(), CurrentState) );
                            }
                        });
                    }
                }
            }
        }
    };


    //---------------------------------------------------------------------------------------------
    //! Stateless top-down code visitor that can change the instructions. Iterative version.
    //! Once an instruction has changed, all the chain of instructions up to the root will be
    //! cloned, referencing the new instruction.
    //---------------------------------------------------------------------------------------------
    class Visitor_TopDown_Unique_Cloning
    {
    public:

        //! Ensure virtual destruction
        virtual ~Visitor_TopDown_Unique_Cloning() = default;

    protected:

        //! Do the actual work by overriding this in the derived classes.
        virtual Ptr<ASTOp> Visit( Ptr<ASTOp> at, bool& processChildren ) = 0;


        void Traverse( Ptr<ASTOp>& root );

    private:

        //! Operations to be processed
        TArray< TPair<bool,Ptr<ASTOp>> > Pending;

        //! Map for visited operations
        TMap<Ptr<ASTOp>,Ptr<ASTOp>> OldToNew;

        //!
        inline Ptr<ASTOp> GetOldToNew( const Ptr<ASTOp>& Old ) const
        {
            Ptr<ASTOp> n;
			const Ptr<ASTOp>* it = OldToNew.Find(Old);
            if (it)
            {
                n = *it;
            }

            it = OldToNew.Find(n);
            while (it && it->get()!=nullptr && *it!=n)
            {
                n = *it;
                it = OldToNew.Find(n);
            }

            return n;
        }

        //! Process all the Pending operations and visit all children if necessary
        void Process();

    };


    inline void ASTChild::AddParent()
    {
		UE::TScopeLock Lock(Child->ParentsMutex);
        ParentIndexInChild = Child->Parents.Num();
        Child->Parents.Add(Parent);
    }


	inline void ASTChild::ClearParent()
    {
		UE::TScopeLock Lock(Child->ParentsMutex);
		
		check( ParentIndexInChild<Child->Parents.Num() );
		// Can't do this, because the indices are stored in children.
		//Child->m_parents.RemoveAtSwap(ParentIndexInChild);
		Child->Parents[ParentIndexInChild] = nullptr;
	}


	/** */
    class FUniqueOpPool
    {
    private:

		struct FKeyFuncs : BaseKeyFuncs<Ptr<ASTOp>, Ptr<ASTOp>, false>
		{
			static KeyInitType GetSetKey(ElementInitType Element) { return Element; }
			static bool Matches(const Ptr<ASTOp>& lhs, const Ptr<ASTOp>& rhs) { return lhs==rhs || *lhs == *rhs; }
			static uint32 GetKeyHash(const Ptr<ASTOp>& Key) { return Key->Hash(); }
		};

        // Existing ops, per type
        TSet<Ptr<ASTOp>, FKeyFuncs> Visited[(int32)EOpType::COUNT];

    public:

        bool bDisabled = false;

        Ptr<ASTOp> Add( const Ptr<ASTOp>& Op )
        {
			if (bDisabled)
			{
				return Op;
			}

			if (!Op)
			{
				return nullptr;
			}

			TSet<Ptr<ASTOp>, FKeyFuncs>& Container = Visited[(int32)Op->GetOpType()];
            return Container.FindOrAdd(Op);
        }
    };

}

