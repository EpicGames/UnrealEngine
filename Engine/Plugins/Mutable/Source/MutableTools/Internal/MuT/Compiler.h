// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/ErrorLog.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Image.h"
#include "MuR/System.h"
#include "Templates/SharedPointer.h"
#include "HAL/PlatformMath.h"
#include "StructUtils/InstancedStruct.h"
#include "Tasks/Task.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
    // Forward declarations
    class Compiler;
    class FModel;

    class Transform;
    using TransformPtr=Ptr<Transform>;
    using TransformPtrConst=Ptr<const Transform>;

    class Node;
    class NodeTransformedObject;

	/** Arguments are Texture ID, output generated texture and a "run immediately" flag. */
	typedef TFunction<UE::Tasks::FTask(PASSTHROUGH_ID, TSharedPtr<TManagedPtr<FImage>>, bool)> FReferencedImageResourceFunc;

	/** Arguments are 
	* - Mesh ID, 
	* - optional morph name, 
	* - output generated mesh 
	* - "run immediately" flag. 
	*/
	typedef TFunction<UE::Tasks::FTask(PASSTHROUGH_ID, const FString&, TSharedPtr<TManagedPtr<FMesh>>, bool)> FReferencedMeshResourceFunc;

    /** Options used to compile the models with a compiler. */
    class CompilerOptions : public RefCounted
    {
    public:

        //! Create new settings with the default values as explained in each method below.
        UE_API CompilerOptions();

        //!
        UE_API void SetLogEnabled( bool bEnabled );

        //!
        UE_API void SetOptimisationEnabled( bool bEnabled );

        //!
        UE_API void SetConstReductionEnabled( bool bEnabled );

        //!
        UE_API void SetOptimisationMaxIteration( int32 MaxIterations );

        //!
        UE_API void SetIgnoreStates( bool bIgnore );

		//! Set the quality for the image compression algorithms. The level value is used internally
		//! with FSystem::SetImagecompressionQuality
		UE_API void SetImageCompressionQuality(int32 Quality);

        //! 
        UE_API void SetDataPackingStrategy( int32 MinTextureResidentMipCount, uint64 EmbeddedDataBytesLimit);

		/** If enabled it will make sure that the object is compile to generate smaller mips of the images. */
		UE_API void SetEnableProgressiveImages(bool bEnabled);

		/** Set an optional pixel conversion function that will be called before any pixel format conversion. */
		UE_API void SetImagePixelFormatOverride(const FImageOperator::FImagePixelFormatFunc&);

		/** */
		UE_API void SetReferencedResourceCallback(const FReferencedImageResourceFunc&, const FReferencedMeshResourceFunc&);

        UE_API void SetDisableMaterialGeneration(bool bDisabled);

        //! Different data packing strategies
        enum class TextureLayoutStrategy : uint8
        {
            //! Pack texture layouts without changing any scale
            Pack,

            //! Do not touch mesh or image texture layouts
            None,

            //! Helper value, not really a strategy.
            Count
        };

        //! 
        static UE_API const char* GetTextureLayoutStrategyName( TextureLayoutStrategy s );

        //-----------------------------------------------------------------------------------------
        // Interface pattern
        //-----------------------------------------------------------------------------------------
        class Private;
        UE_API Private* GetPrivate() const;

        CompilerOptions( const CompilerOptions& ) = delete;
        CompilerOptions( CompilerOptions&& ) = delete;
        CompilerOptions& operator=(const CompilerOptions&) = delete;
        CompilerOptions& operator=(CompilerOptions&&) = delete;

    protected:

        /** Forbidden. Manage with the Ptr<> template. */
        UE_API ~CompilerOptions() override;

    private:

        Private* m_pD;

    };


    //!
    struct FStateOptimizationOptions
    {
		uint8 NumExtraLODsToBuildAfterFirstLOD = 0;
		bool bOnlyFirstLOD = false;
		ETextureCompressionStrategy TextureCompressionStrategy = ETextureCompressionStrategy::None;
    };


	//! Information about an object state in the source data
	struct FObjectState
	{
		//! Name used to identify the state from the code and user interface.
		FString Name;

		//! GPU Optimisation options
		FStateOptimizationOptions Optimisation;

		//! List of names of the runtime parameters in this state
		TArray<FString> RuntimeParams;
	};

	
    /** */
    class Compiler : public RefCounted
    {
    public:

        UE_API Compiler( Ptr<CompilerOptions> Options, TFunction<void()>& InWaitCallback );

        //! Compile the expression into a run-time model.
		UE_API TSharedPtr<FModel> Compile( const Ptr<Node>& pNode, TMap<FOperation::ADDRESS, FInstancedStruct>& ExternalOperations);

        //! Return the log of messages of all the compile operations executed so far.
        UE_API TSharedPtr<FErrorLog> GetLog() const;

    protected:

        /** Forbidden. Manage with the Ptr<> template. */
        UE_API ~Compiler();

    private:

        class Private;
        Private* m_pD;

    };

}

#undef UE_API
