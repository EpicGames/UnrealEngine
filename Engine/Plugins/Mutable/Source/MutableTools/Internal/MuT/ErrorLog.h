// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformCrt.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "Templates/SharedPointer.h"
#include "Async/Mutex.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** Types of message stored in the log. */
	typedef enum
	{
        ELMT_NONE=0,
		ELMT_ERROR,
		ELMT_WARNING,
		ELMT_INFO
	} ErrorLogMessageType;

	/** Categories of message stored in the log for the purpose of limiiting duplication of non - identical messages. */
	typedef enum
	{
		ELMSB_ALL = 0,
		ELMSB_UNKNOWN_TAG
	} ErrorLogMessageSpamBin;

	struct ErrorLogMessageAttachedDataView 
	{
		const float* UnassignedUVs = nullptr;
		int32 UnassignedUVsSize = 0;
	};

	/** Storage for mutable error, warning and information messages from several processes performed by the tools library, like compilation. 
	* It supports concurrent addition of messages, but not retrieval.
	*/	
	class FErrorLog
	{
	public:

		struct FErrorData
		{
			TArray< float > UnassignedUVs;
		};

		struct FMessage
		{
			ErrorLogMessageType Type = ELMT_NONE;
			ErrorLogMessageSpamBin Spam = ELMSB_ALL;
			FString Text;
			TSharedPtr<FErrorData> Data;
			
			TArray<const void*, TInlineAllocator<4>> Contexts;
		};

	private:

		/** This mutex is used to control the access to the messages array when adding them. */
		UE::FMutex MessageMutex;

		TArray<FMessage> Messages;

	public:

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		//! Get the number of messages.
		//! If a message type is provided, return the number of that type of message only.
		UE_API int32 GetMessageCount() const;

		//! Get the text of a message.
		//! \param index index of the message from 0 to GetMessageCount()-1
		UE_API const FString& GetMessageText( int32 index ) const;

		//! Get the opaque contexts of a message.
		//! \param Index index of the message from 0 to GetMessageCount()-1
		UE_API const TArray<const void*, TInlineAllocator<4>>* GetMessageContexts(int32 Index) const;

		//!
		UE_API ErrorLogMessageType GetMessageType( int32 index ) const;

		//!
		UE_API ErrorLogMessageSpamBin GetMessageSpamBin(int32 index) const;

		//! Get the attached data of a message.
		//! \param index index of message data from 0 to GetMessageCount(ELMT_NONE)-1
		UE_API ErrorLogMessageAttachedDataView GetMessageAttachedData( int32 index ) const;

		//!
		UE_API void Log() const;

		//!
		UE_API void Merge( const FErrorLog* Other );

		//!
		UE_API void Add(const FString& Message, ErrorLogMessageType Type, const void* Context, ErrorLogMessageSpamBin SpamBin = ELMSB_ALL);
		UE_API void Add(const FString& Message, ErrorLogMessageType Type, const void* Context, const void* Context2, ErrorLogMessageSpamBin SpamBin = ELMSB_ALL);
		UE_API void Add(const FString& Message, ErrorLogMessageType Type, TArray<const void*, TInlineAllocator<4>>& Contexts, ErrorLogMessageSpamBin SpamBin = ELMSB_ALL);
		
		//!
		UE_API void Add(const FString& Message, const ErrorLogMessageAttachedDataView& Data, ErrorLogMessageType Type, const void* Context, ErrorLogMessageSpamBin SpamBin = ELMSB_ALL);

	};
	
	MUTABLETOOLS_API extern const TCHAR* s_opNames[];
}

#undef UE_API
