// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


namespace UE::Mutable::Private
{
	struct FLODInfo
	{
		float ScreenSize = 0.0f;

		float LODHysteresis = 0.0f;
	
		bool bSupportUniformlyDistributedSampling = false;
	
		bool bAllowCPUAccess = false;
		
		bool operator==(const FLODInfo& Other) const = default;
	};
	
	
	uint32 GetTypeHash(const FLODInfo& Key);
}
