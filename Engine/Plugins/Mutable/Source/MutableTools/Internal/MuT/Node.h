// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/RefCounted.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	// Forward declarations
	class Node;
	typedef Ptr<Node> NodePtr;
	typedef Ptr<const Node> NodePtrConst;

	class NodeMap;
	typedef Ptr<NodeMap> NodeMapPtr;
	typedef Ptr<const NodeMap> NodeMapPtrConst;



    //! %Base class for all graphs used in the source data to define models and transforms.
	class Node : public RefCounted
	{
	public:

		/** Non-stable enumeration of all node types. */
		enum class EType : uint8
		{
			None,

			Node,

			Mesh,
			MeshConstant,
			MeshTable,
			MeshFormat,
			MeshTangents,
			MeshMorph,
			MeshMakeMorph,
			MeshSwitch,
			MeshFragment,
			MeshTransform,
			MeshClipMorphPlane,
			MeshClipWithMesh,
			MeshRemoveMesh,
			MeshTransformWithBone,
			MeshTransformInMesh,
			MeshApplyPose,
			MeshVariation,
			MeshReshape,
			MeshClipDeform,
			MeshSkeletalMeshObjectBreak,
			MeshExternal,

			Image,
			ImageConstant,
			ImageInterpolate,
			ImageSaturate,
			ImageTable,
			ImageSwizzle,
			ImageColorMap,
			ImageGradient,
			ImageBinarise,
			ImageLuminance,
			ImageLayer,
			ImageLayerColor,
			ImageResize,
			ImagePlainColor,
			ImageProject,
			ImageMipmap,
			ImageSwitch,
			ImageConditional,
			ImageFormat,
			ImageConform,
			ImageMultiLayer,
			ImageInvert,
			ImageVariation,
			ImageNormalComposite,
			ImageTransform,
			ImageFromMaterialParameter,
			ImageExternal,
			ImageConvert,

			ImageObject,
			ImageObjectParameter, 
			
			Bool,
			BoolConstant,
			BoolParameter,
			BoolNot,
			BoolAnd,

			Color,
			ColorConstant,
			ColorParameter,
			ColorSampleImage,
			ColorTable,
			ColorImageSize,
			ColorFromScalars,
			ColorArithmeticOperation,
			ColorSwitch,
			ColorVariation,
			ColorLinearToSRGB,
			ColorExternal,

			Scalar,
			ScalarConstant,
			ScalarParameter,
			ScalarEnumParameter,
			ScalarCurve,
			ScalarSwitch,
			ScalarArithmeticOperation,
			ScalarVariation,
			ScalarTable,
			ScalarExternal,

			String,
			StringConstant,
			StringParameter,

			Projector,
			ProjectorConstant,
			ProjectorParameter,

			Range,
			RangeFromScalar,

			Layout,

			PatchImage,
			PatchMesh,

			Surface,
			SurfaceNew,
			SurfaceSwitch,
			SurfaceVariation,

			LOD,

			SkeletalMesh,
			SkeletalMeshNew,
			SkeletalMeshMerge,
			SkeletalMeshConvert,
			SkeletalMeshModify,
			SkeletalMeshMorph,
			SkeletalMeshReshape,
			SkeletalMeshSwitch,
			SkeletalMeshVariation,
			SkeletalMeshTransform,
			SkeletalMeshTransformWithBone,
			SkeletalMeshClipWithSkeletalMesh,

			SkeletalMeshObject,
			SkeletalMeshObjectConvert,
			SkeletalMeshObjectParameter,
			SkeletalMeshObjectSwitch,
			
			Component,
			ComponentNew,
			ComponentSwitch,
			ComponentVariation,

			Object,
			ObjectNew,
			ObjectGroup,

			Modifier,
			ModifierSkeletalMeshMerge,
			
			SurfaceModifier,
			SurfaceModifierMeshClipMorphPlane,
			SurfaceModifierMeshClipWithMesh,
			SurfaceModifierMeshClipDeform,
			SurfaceModifierMeshClipWithUVMask,
			SurfaceModifierSurfaceEdit,
			SurfaceModifierTransformInMesh,
			SurfaceModifierTransformWithBone,

			External,
			ExternalOperation,
			ExternalParameter,
			ExternalSwitch,
			
			ExtensionData,
			ExtensionDataConstant,
			ExtensionDataSwitch,
			ExtensionDataVariation,

			Matrix,
			MatrixConstant,
			MatrixParameter,

			Material,
			MaterialConstant,
			MaterialTable,
			MaterialSwitch,
			MaterialVariation,
			MaterialParameter,
			MaterialExternal,
			MaterialSkeletalMeshObjectBreak,
			MaterialSkeletalMeshBreak,
			MaterialModify,
			
			ImageMaterialBreak,
			ScalarMaterialBreak,
			ColorMaterialBreak,
			
			Count
		};


		/** Information about the type of a node, to provide some means to the tools to deal generically with nodes. */
		struct FNodeType
		{
			FNodeType();
			FNodeType(Node::EType, const FNodeType* Parent);

			Node::EType Type;
			const FNodeType* Parent;

			inline bool IsA(const FNodeType* CandidateType) const
			{
				if (CandidateType == this)
				{
					return true;
				}

				if (Parent)
				{
					return Parent->IsA(CandidateType);
				}

				return false;
			}
		};


		//-----------------------------------------------------------------------------------------
		// Own Interface
		//-----------------------------------------------------------------------------------------

		/** Node type hierarchy data. */
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		/** Set the opaque context returned in messages in the compiler log. */
		UE_API void SetMessageContext(const void* context);
		UE_API const void* GetMessageContext() const;

		//-----------------------------------------------------------------------------------------
        // Interface pattern
		//-----------------------------------------------------------------------------------------
		class Private;

	protected:

		inline ~Node() = default;

		/** This is an opaque context used to attach to reported error messages. */
		const void* MessageContext = nullptr;

	private:

		static UE_API FNodeType StaticType;

	};


	using FNodeType = Node::FNodeType;
}

#undef UE_API
