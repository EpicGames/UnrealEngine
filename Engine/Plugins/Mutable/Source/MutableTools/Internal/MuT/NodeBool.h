// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** Base class of any node that outputs a Bool value. */
	class NodeBool : public Node
	{
	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeBool() = default;

	private:

		static UE_API FNodeType StaticType;

	};


	/** Node returning a Bool constant value. */
	class NodeBoolConstant : public NodeBool
	{
	public:

		bool Value = false;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeBoolConstant() = default;

	private:

		static UE_API FNodeType StaticType;

	};


	/** Node that defines a Bool model parameter. */
	class NodeBoolParameter : public NodeBool
	{
	public:

		bool DefaultValue;
		FString Name;
		FString UID;

		TArray<Ptr<NodeRange>> Ranges;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeBoolParameter() = default;

	private:

		static UE_API FNodeType StaticType;

	};


	/** Node that returns the oposite of the input value. */
	class NodeBoolNot : public NodeBool
	{
	public:

		Ptr<NodeBool> Source;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeBoolNot() = default;

	private:

		static UE_API FNodeType StaticType;

	};


	/** */
	class NodeBoolAnd : public NodeBool
	{
	public:

		Ptr<NodeBool> A;
		Ptr<NodeBool> B;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeBoolAnd() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
