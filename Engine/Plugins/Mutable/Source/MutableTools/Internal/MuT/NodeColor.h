// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    /** Base class of any node that outputs a color.
	*/
    class NodeColor : public Node
    {
    public:

        // Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

    protected:

        /** Forbidden. Manage with the Ptr<> template. */
        inline ~NodeColor() = default;

	private:

		static UE_API FNodeType StaticType;

    };

}

#undef UE_API
