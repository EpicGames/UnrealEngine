// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/NodeColor.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    /** Perform a per - component arithmetic operation between two colors. 
	*/
	class NodeColorArithmeticOperation : public NodeColor
	{
	public:

		/** Possible arithmetic operations. */
		enum class EOperation
		{
			Add,
			Subtract,
			Multiply,
			Divide
		};

		EOperation Operation;
		Ptr<NodeColor> A;
		Ptr<NodeColor> B;

	public:

		/** Node type hierarchy data. */
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeColorArithmeticOperation() = default;

	private:

		static UE_API FNodeType StaticType;

	};

	
}

#undef UE_API
