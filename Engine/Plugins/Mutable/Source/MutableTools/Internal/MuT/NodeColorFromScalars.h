// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** Obtain a color by sampling an image at specific homogeneous coordinates.
	*/
	class NodeColorFromScalars : public NodeColor
	{
	public:

		Ptr<NodeScalar> X;
		Ptr<NodeScalar> Y;
		Ptr<NodeScalar> Z;
		Ptr<NodeScalar> W;

	public:

		// Node interface
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeColorFromScalars() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
