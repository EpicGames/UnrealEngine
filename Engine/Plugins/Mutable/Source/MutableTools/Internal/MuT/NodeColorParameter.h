// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeRange.h"
#include "Math/Vector4.h"
#include "Math/MathFwd.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that defines a color model parameter.
	*/
	class NodeColorParameter : public NodeColor
	{
	public:

		FVector4f DefaultValue;
		FString Name;
		FString Uid;

		TArray<Ptr<NodeRange>> Ranges;

	public:

		// Node interface
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeColorParameter() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
