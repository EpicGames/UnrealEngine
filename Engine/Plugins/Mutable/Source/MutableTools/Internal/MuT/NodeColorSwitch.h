// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    /** This node selects an output Color from a set of input Colors based on a parameter. 
	*/
    class NodeColorSwitch : public NodeColor
	{
	public:

		Ptr<NodeScalar> Parameter;
		TArray<Ptr<NodeColor>> Options;

	public:

		/** Node type hierarchy data. */
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeColorSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
