// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeColor.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** Convert a linear color to sRGB
	*/
	class NodeColorToSRGB : public NodeColor
	{
	public:

		Ptr<NodeColor> Color;

	public:

		// Node interface
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeColorToSRGB() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
