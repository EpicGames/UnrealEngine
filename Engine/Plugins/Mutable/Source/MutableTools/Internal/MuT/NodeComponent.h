// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeLOD.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshNew;

	class NodeComponent : public Node
	{
	public:
        // Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

        //-----------------------------------------------------------------------------------------
        // Own interface
        //-----------------------------------------------------------------------------------------

		virtual const class NodeComponentNew* GetParentComponentNew() const = 0;

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeComponent() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
