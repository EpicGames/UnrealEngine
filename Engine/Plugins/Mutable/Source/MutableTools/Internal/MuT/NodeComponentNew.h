// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeComponent.h"

#include "MuR/ComponentId.h"
#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMeshObject.h"
#include "MuT/Node.h"
#include "MuT/NodeMaterial.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	class NodeComponentNew : public NodeComponent
	{
	public:

		/** Externally managed id assigned to this component. */
		FComponentId Id = INDEX_NONE;

		Ptr<NodeMaterial> OverlayMaterial;
		
		Ptr<NodeSkeletalMeshObject> SkeletalMeshObject;

		/** Map where the key is the target Material slot and the value the replacement material to be used in the target material slot*/
		TMap<const FName, Ptr<NodeMaterial>> OverrideMaterials;
		
		/** Map where the key is the target Material slot and the value the material to be rendered on top in the target material slot*/
		TMap<const FName, Ptr<NodeMaterial>> OverlayMaterials;
		
		//-----------------------------------------------------------------------------------------
        // Node interface
		//-----------------------------------------------------------------------------------------		

		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

		//-----------------------------------------------------------------------------------------
		// NodeComponent interface
		//-----------------------------------------------------------------------------------------		
		virtual const class NodeComponentNew* GetParentComponentNew() const override { return this;  }

		//-----------------------------------------------------------------------------------------
        // Own interface
		//-----------------------------------------------------------------------------------------

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeComponentNew() = default;

	private:

		static UE_API FNodeType StaticType;
		
	};

}

#undef UE_API
