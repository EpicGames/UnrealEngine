// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    /** This node selects an output component from a set of input components based on a parameter. 
	*/
    class NodeComponentSwitch : public NodeComponent
	{
	public:

		Ptr<NodeScalar> Parameter;
		TArray<Ptr<NodeComponent>> Options;

	public:

		/** Node type hierarchy data. */
		virtual const FNodeType* GetType() const { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// NodeComponent interface
		virtual const class NodeComponentNew* GetParentComponentNew() const override { check(false); return nullptr; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeComponentSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
