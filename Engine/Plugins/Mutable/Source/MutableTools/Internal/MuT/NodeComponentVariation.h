// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeComponent.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** Select different component subgraphs based on active tags. */
    class NodeComponentVariation : public NodeComponent
    {
	public:

		Ptr<NodeComponent> DefaultComponent;

		struct FVariation
		{
			Ptr<NodeComponent> Component;
			FString Tag;
		};

		TArray<FVariation> Variations;

    public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// NodeComponent interface
		virtual const class NodeComponentNew* GetParentComponentNew() const override { check(false); return nullptr; }

    protected:

        /** Forbidden. Manage with the Ptr<> template. */
		~NodeComponentVariation() = default;

	private:

		static UE_API FNodeType StaticType;
	
	};


} // namespace UE::Mutable::Private

#undef UE_API
