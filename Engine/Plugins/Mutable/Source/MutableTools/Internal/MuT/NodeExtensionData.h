// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeObject.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{


	//! Node that evaluates to an ExtensionData
	class NodeExtensionData : public Node
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Node Interface
		//-----------------------------------------------------------------------------------------
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeExtensionData() = default;

	private:

		static UE_API FNodeType StaticType;

	};



}

#undef UE_API
