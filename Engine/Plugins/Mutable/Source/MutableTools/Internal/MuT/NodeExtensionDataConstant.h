// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeExtensionData.h"

#include "MuR/PassthroughObject.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	//! Node that outputs a constant ExtensionData
	//! \ingroup model
	class NodeExtensionDataConstant : public NodeExtensionData
	{
	public:
		PASSTHROUGH_ID ExtensionDataId = PASSTHROUGH_ID_INVALID;

		//-----------------------------------------------------------------------------------------
		// Node Interface
		//-----------------------------------------------------------------------------------------
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeExtensionDataConstant() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
