// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ExtensionData.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/NodeExtensionData.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	class NodeExtensionDataSwitch : public NodeExtensionData
	{
	public:

		Ptr<NodeScalar> Parameter;
		TArray<Ptr<NodeExtensionData>> Options;

	public:

		//-----------------------------------------------------------------------------------------
		// Node Interface
		//-----------------------------------------------------------------------------------------
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

		//-----------------------------------------------------------------------------------------
		// Own Interface
		//-----------------------------------------------------------------------------------------
		UE_API Ptr<NodeScalar> GetParameter() const;
		UE_API void SetParameter(Ptr<NodeScalar> InParameter);

		UE_API void SetOptionCount(int);

		UE_API Ptr<NodeExtensionData> GetOption(int32 t) const;
		UE_API void SetOption(int32 t, Ptr<NodeExtensionData>);


	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeExtensionDataSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};
}

#undef UE_API
