// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/ExtensionData.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/NodeExtensionData.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	class NodeExtensionDataVariation : public NodeExtensionData
	{
	public:

		Ptr<NodeExtensionData> DefaultValue;

		struct FVariation
		{
			Ptr<NodeExtensionData> Value;
			FString Tag;
		};

		TArray<FVariation> Variations;

	public:

		//-----------------------------------------------------------------------------------------
		// Node Interface
		//-----------------------------------------------------------------------------------------
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

		//-----------------------------------------------------------------------------------------
		// Own Interface
		//-----------------------------------------------------------------------------------------
		UE_API void SetDefaultValue(Ptr<NodeExtensionData> InValue);

		UE_API void SetVariationCount(int32 InCount);
		UE_API int GetVariationCount() const;

		UE_API void SetVariationTag(int32 InIndex, const FString& InTag);

		UE_API void SetVariationValue(int32 InIndex, Ptr<NodeExtensionData> InValue);


	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeExtensionDataVariation() = default;

	private:

		static UE_API FNodeType StaticType;

	};
}

#undef UE_API
