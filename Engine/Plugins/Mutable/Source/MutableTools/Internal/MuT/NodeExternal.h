// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeExternal : public Node
	{
	public:
		// Node interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeExternal() override = default;

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
