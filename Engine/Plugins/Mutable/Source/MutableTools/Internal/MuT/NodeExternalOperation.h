// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeExternal.h"

#include "MuR/Ptr.h"
#include "StructUtils/InstancedStruct.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeExternalOperation : public NodeExternal
	{
	public:
		// Node interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeExternalOperation() override = default;

	private:
		static UE_API FNodeType StaticType;

	public:
		FInstancedStruct OperationInstancedStruct;

		TArray<Ptr<Node>> Inputs;
	};
}


#undef UE_API
