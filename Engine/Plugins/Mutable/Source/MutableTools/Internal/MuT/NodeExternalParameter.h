// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeExternal.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeExternalParameter : public NodeExternal
	{
	public:
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		virtual ~NodeExternalParameter() override = default;

	private:
		static UE_API FNodeType StaticType;

	public:
		TManagedPtr<const FInstancedStruct> DefaultValue;
		
		FString Name;
		FString UID;

		TArray<Ptr<NodeRange>> Ranges;
	};
}


#undef UE_API
