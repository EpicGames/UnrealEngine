// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeExternal.h"

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
    class NodeExternalSwitch : public NodeExternal
	{
	public:
		Ptr<NodeScalar> Parameter;
		TArray<Ptr<NodeExternal>> Options;

		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeExternalSwitch() override = default;

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
