// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	// Forward definitions
	class NodeImage;
	typedef Ptr<NodeImage> NodeImagePtr;
	typedef Ptr<const NodeImage> NodeImagePtrConst;

	/** Data related to the a source image that is necessary to classify the final image and mesh fragments
	 * that are derived from this source.
	 */
	struct FSourceDataDescriptor
	{
		enum ESpecialValues
		{
			EInvalid = -2,
			ENeutral = -1,
		};

		/** 
		 * The size of the first mip to be considered optional. Mips with equal or larger size will 
		 * be considered optional.
		 * If -1 it means the entire descriptor is neutral.
		 * If -2 it means the descriptor is invalid (results of an operation that shouldn't happen).
		 * If less than equal 0 it means all mips are non optional.
		 */
		int32 OptionalMaxLODSize = ENeutral;

		/** 
		 * Bias to the final number of optional mips. Any mip will have at least this number of optional 
		 * lods if not in the NumNonOptionalLODs range. 
		 */
		int32 OptionalLODBias = 0;
		
		/** Number of lods from the tail that will never be considered optional */
		int32 NumNonOptionalLODs = 0;

		/** Whether or not the source texture had mip generation settings set to NoMipmaps. */
		bool bNoMipmaps = false;

		/** Source tags that mark this data and prevent it from mixing with other data at compile time. */
		TArray<FString> Tags;

		/** Source Id */
		uint32 SourceId = MAX_uint32;

		inline bool operator==(const FSourceDataDescriptor& Other) const
		{
			return 
				Tags == Other.Tags &&
				OptionalMaxLODSize == Other.OptionalMaxLODSize &&
				OptionalLODBias == Other.OptionalLODBias &&
				NumNonOptionalLODs == Other.NumNonOptionalLODs &&
				bNoMipmaps == Other.bNoMipmaps;
		}

		inline bool IsInvalid() const
		{
			return OptionalMaxLODSize == EInvalid;
		}

		inline bool IsNeutral() const
		{
			return OptionalMaxLODSize == ENeutral;
		}

		void CombineWith(const FSourceDataDescriptor& Other)
		{
			if (IsInvalid() || Other.IsInvalid())
			{
				SourceId = MAX_uint32;
				Tags.Empty();
				OptionalMaxLODSize = EInvalid;
				OptionalLODBias = 0;
				NumNonOptionalLODs = 0;
				bNoMipmaps = false;
				return;
			}

			if (Other.IsNeutral())
			{
				return;
			}

			if (IsNeutral())
			{
				SourceId = Other.SourceId;
				Tags = Other.Tags;
		 		OptionalMaxLODSize = Other.OptionalMaxLODSize;
		 		OptionalLODBias = Other.OptionalLODBias;
		 		NumNonOptionalLODs = Other.NumNonOptionalLODs;
				bNoMipmaps = Other.bNoMipmaps;
				return;
			}

			if (!(*this == Other))
			{
				SourceId = MAX_uint32;
				Tags.Empty();
				OptionalMaxLODSize = EInvalid;
				OptionalLODBias = 0;
				NumNonOptionalLODs = 0;
				bNoMipmaps = false;
				return;
			}

			return;
		}
	};


    /** Base class of any node that outputs an image. */
	class NodeImage : public Node
	{
	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeImage() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
