// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** This node changes the colors of a selectd part of the image, applying a color map from
	* conteined in another image.
	*/
	class NodeImageColorMap : public NodeImage
	{
	public:

		Ptr<NodeImage> Base;
		Ptr<NodeImage> Mask;
		Ptr<NodeImage> Map;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageColorMap() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
