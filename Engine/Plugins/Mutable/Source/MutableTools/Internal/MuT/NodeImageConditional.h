// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeBool.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** This node selects an output image from a set of input images based on a parameter. */
    class NodeImageConditional : public NodeImage
	{
	public:

		Ptr<NodeBool> Parameter;
		Ptr<NodeImage> True;
		Ptr<NodeImage> False;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageConditional() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
