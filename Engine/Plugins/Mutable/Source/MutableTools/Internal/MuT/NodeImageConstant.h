// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that outputs a constant image.	*/
	class NodeImageConstant : public NodeImage
	{
	public:

		/** */
		TManagedPtr<FImage> Image;
		
		/** */
		FSourceDataDescriptor SourceDataDescriptor;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageConstant() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
