// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/ImageTypes.h"
#include "MuT/Node.h"
#include "MuT/NodeImageObjectParameter.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeImageConvert : public NodeImage
	{
	public:
		Ptr<NodeImageObject> ImageParameter;
		FImageDesc ImageDescriptor;
		
		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		virtual ~NodeImageConvert() override = default;

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
