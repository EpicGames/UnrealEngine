// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuR/Material.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeImageFromMaterialParameter : public NodeImage
	{
	public:

		FParameterKey ImageParameterKey;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageFromMaterialParameter() = default;

	private:

		static UE_API FNodeType StaticType;
	};
}
