// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that interpolates linearly between several images based on a weight.
	* The different input images are uniformly distributed in the 0 to 1 range:
	* - if two images are set, the first one is 0.0 and the second one is 1.0
	* - if three images are set, the first one is 0.0 the second one is 0.5 and the third one 1.0
	* regardless of the input slots used: If B and C are set, but not A, B will be at weight 0
	* and C will be at weight 1
	*/
	class NodeImageInterpolate : public NodeImage
	{
	public:

		Ptr<NodeScalar> Factor;
		TArray<Ptr<NodeImage>> Targets;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageInterpolate() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
