// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** This node applies a layer blending effect on a base image using a mask and a blended image. */
	class NodeImageLayer : public NodeImage
	{
	public:

		Ptr<NodeImage> Base;
		Ptr<NodeImage> Mask;
		Ptr<NodeImage> Blended;
		EBlendType Type;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageLayer() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
