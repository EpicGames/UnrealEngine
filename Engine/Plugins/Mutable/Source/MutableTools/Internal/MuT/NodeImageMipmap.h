// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Generate mimaps for an image. */
    class NodeImageMipmap : public NodeImage
	{
	public:

		Ptr<NodeImage> Source;
		Ptr<NodeScalar> Factor;

		FMipmapGenerationSettings Settings;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageMipmap() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
