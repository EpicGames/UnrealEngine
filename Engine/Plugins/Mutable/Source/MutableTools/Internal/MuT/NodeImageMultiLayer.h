// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** This node applies any numbre of layer blending effect on a base image using a mask and a
    * blended image. The number of layers depends on a scalar input of this node.
	*/
    class NodeImageMultiLayer : public NodeImage
	{
	public:

		Ptr<NodeImage> Base;
		Ptr<NodeImage> Mask;
		Ptr<NodeImage> Blended;
		Ptr<NodeRange> Range;
		EBlendType Type;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageMultiLayer() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
