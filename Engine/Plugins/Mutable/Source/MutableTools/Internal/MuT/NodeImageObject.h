// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeImageObject : public NodeImage // TODO GMT It should inherit from Node
	{
	public:
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeImageObject() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API

