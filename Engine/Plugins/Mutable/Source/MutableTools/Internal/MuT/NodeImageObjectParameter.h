// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeImageObject.h"

#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** */
    class NodeImageObjectParameter : public NodeImageObject
	{
	public:
		/** Name of the parameter */
		FString Name;

		/** User provided ID to identify the parameter. */
		FString UID;

		/** Ranges for the parameter in case it is multidimensional. */
		TArray<Ptr<NodeRange>> Ranges;

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeImageObjectParameter() override = default;

	private:
		static UE_API FNodeType StaticType;
	};
}

#undef UE_API
