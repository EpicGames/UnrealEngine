// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeColor.h"
#include "MuR/Image.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that multiplies the colors of an image, channel by channel. */
	class NodeImagePlainColor : public NodeImage
	{
	public:

		Ptr<NodeColor> Color;
		int32 SizeX = 4;
		int32 SizeY = 4;
		EImageFormat Format = EImageFormat::RGBA_UByte;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImagePlainColor() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
