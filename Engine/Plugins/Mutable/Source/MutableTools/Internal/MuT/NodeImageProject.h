// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeProjector.h"
#include "MuT/NodeScalar.h"
#include "MuR/Image.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** */
	class NodeImageProject : public NodeImage
	{
	public:

		Ptr<NodeProjector> Projector;
		Ptr<NodeMesh> Mesh;
		Ptr<NodeScalar> AngleFadeStart;
		Ptr<NodeScalar> AngleFadeEnd;
		Ptr<NodeImage> Image;
		Ptr<NodeImage> Mask;
		FUintVector2 ImageSize;
		uint8 Layout = 0;
		bool bIsRGBFadingEnabled = true;
		bool bIsAlphaFadingEnabled = true;
		bool bEnableTextureSeamCorrection = true;
		ESamplingMethod SamplingMethod = ESamplingMethod::Point;
		EMinFilterMethod MinFilterMethod = EMinFilterMethod::None;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageProject() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
