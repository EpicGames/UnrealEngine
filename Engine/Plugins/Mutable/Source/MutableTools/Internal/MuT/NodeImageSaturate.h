// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Change the saturation of an image.This node can be used to increase the saturation with a
	* factor bigger than 1, or to decrease it or desaturate it completely with a factor smaller
	* than 1 or 0.
	*/
	class NodeImageSaturate : public NodeImage
	{
	public:

		Ptr<NodeImage> Source;

		Ptr<NodeScalar> Factor;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageSaturate() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
