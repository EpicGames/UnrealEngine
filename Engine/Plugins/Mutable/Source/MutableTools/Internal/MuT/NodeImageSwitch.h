// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	//! This node selects an output image from a set of input images based on a parameter.
	class NodeImageSwitch : public NodeImage
	{
	public:

		Ptr<NodeScalar> Parameter;

		TArray<Ptr<NodeImage>> Options;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
