// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that composes a new image by gathering pixel data from channels in other images. */
	class NodeImageSwizzle : public NodeImage
	{
	public:

		EImageFormat NewFormat;
		TArray<Ptr<NodeImage>> Sources;
		TArray<int32> SourceChannels;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// Own interface 
		inline void SetFormat(EImageFormat InFormat)
		{
			NewFormat = InFormat;

			int32 ChannelCount = GetImageFormatData(InFormat).Channels;
			Sources.SetNum(ChannelCount);
			SourceChannels.SetNum(ChannelCount);
		}

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageSwizzle() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
