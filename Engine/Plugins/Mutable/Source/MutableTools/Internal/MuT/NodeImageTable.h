// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/ImageTypes.h"
#include "MuT/Table.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{


	/** This node provides the meshes stored in the column of a table. */
	class NodeImageTable : public NodeImage
	{
	public:

		FString ParameterName;

		Ptr<FTable> Table;

		FString ColumnName;

		uint16 MaxTextureSize = 0;

		FImageDesc ReferenceImageDesc;

		bool bNoneOption = false;

		FString DefaultRowName;

		/** */
		FSourceDataDescriptor SourceDataDescriptor;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageTable() = default;

	private:

		static UE_API FNodeType StaticType;


	};


}

#undef UE_API
