// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"
#include "MuR/ImageTypes.h"
#include "MuR/Ptr.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/**  */
	class NodeImageTransform : public NodeImage
	{
	public:

		Ptr<NodeImage> Base;
		Ptr<NodeScalar> OffsetX;
		Ptr<NodeScalar> OffsetY;
		Ptr<NodeScalar> ScaleX;
		Ptr<NodeScalar> ScaleY;
		Ptr<NodeScalar> Rotation;

		EAddressMode AddressMode = EAddressMode::Wrap;
		uint32 SizeX = 0;
		uint32 SizeY = 0;

		bool bKeepAspectRatio = false;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeImageTransform() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
