// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** */
    class NodeImageVariation : public NodeImage
    {
	public:

		Ptr<NodeImage> DefaultImage;

		struct FVariation
		{
			Ptr<NodeImage> Image;
			FString Tag;
		};

		TArray<FVariation> Variations;

    public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

    protected:

        /** Forbidden. Manage with the Ptr<> template. */
		~NodeImageVariation() = default;

    private:

		static UE_API FNodeType StaticType;
	
	};

}

#undef UE_API
