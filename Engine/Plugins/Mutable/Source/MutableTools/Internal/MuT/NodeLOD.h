// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeSurface.h"
#include "MuT/NodeSurfaceModifier.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeLOD : public Node
	{
	public:

		//-----------------------------------------------------------------------------------------
		// Node Interface
		//-----------------------------------------------------------------------------------------

		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

		//-----------------------------------------------------------------------------------------
		// Own Interface
		//-----------------------------------------------------------------------------------------

		TArray<Ptr<NodeSurface>> Surfaces;

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeLOD() = default;

	private:

		static UE_API FNodeType StaticType;

	};



}

#undef UE_API
