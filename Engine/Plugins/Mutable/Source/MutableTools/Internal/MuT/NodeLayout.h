// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Node.h"
#include "MuR/Layout.h"
#include "MuR/Image.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "HAL/Platform.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class FMesh;

	/** Data for a layout block before it is compiled. */
	struct FSourceLayoutBlock
	{
		/** Optional mask image that selects the vertices to include in the block. */
		TManagedPtr<FImage> Mask;

		FIntVector2 Min = { 0, 0 };
		FIntVector2 Size = { 0, 0 };

		//! Priority value to control the shrink texture layout strategy
		int32 Priority = 0;

		//! Value to control the method to reduce the block
		uint32 bReduceBothAxes : 1 = 0;

		//! Value to control if a block has to be reduced by two in an unitary reduction strategy
		uint32 bReduceByTwo : 1 = 0;
	};


	/** This node is used to define the texture layout for a texture coordinates channel of a mesh. */
	class NodeLayout : public Node
	{
	public:

		//!
		FIntVector2 Size = FIntVector2(0, 0);

		/** Maximum size in layout blocks that this layout can grow to. From there on, blocks will shrink to fit.
		* If 0,0 then no maximum size applies.
		*/
		FIntVector2 MaxSize = FIntVector2(0, 0);

		//!
		EAutoBlocksStrategy AutoBlockStrategy = EAutoBlocksStrategy::Rectangles;
		TArray<FSourceLayoutBlock> Blocks;

		//! Packing strategy
		EPackStrategy Strategy = EPackStrategy::Resizeable;
		EReductionMethod ReductionMethod = EReductionMethod::Halve;

		/** When compiling, ignore generated warnings from this LOD on.
		* -1 means all warnings are generated.
		*/
		int32 FirstLODToIgnoreWarnings = 0;

		int32 TexCoordsIndex = INDEX_NONE;

		bool bMergeChildBlocks = false;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// Own interface

		/** */
		TManagedPtr<FLayout> BuildRuntimeLayout(uint32 BlockIDPrefix) const;

		/** Generate the blocks of a layout using the UV of the meshes. 
		* A list of existing blocks may be provided to specify starting blocks that shouldn't be modified unless it is mandatory because they contain partial face islands. 
		*/
		UE_API void GenerateLayoutBlocks(const TManagedPtr<FMesh>&, int32 LayoutIndex );
		UE_API void GenerateLayoutBlocksFromUVIslands(const TManagedPtr<FMesh>&, int32 LayoutIndex);

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeLayout() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
