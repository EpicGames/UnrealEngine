// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Material.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/PassthroughObject.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeImage.h"
#include "MuR/Types.h"
#include "Math/Vector4.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	/** NodeMaterialConstant class. */
	class NodeMaterialConstant : public NodeMaterial
	{
	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		PASSTHROUGH_ID MaterialId = PASSTHROUGH_ID_INVALID;

		TMap<FParameterKey, FVector4f> ColorValues;
		TMap<FParameterKey, float> ScalarValues;
		TMap<FParameterKey, Ptr<NodeImage>> ImageValues;

	protected:

		// Forbidden. Manage with the Ptr<> template.
		inline ~NodeMaterialConstant() = default;

	private:

		static UE_API FNodeType StaticType;
	};
}

#undef UE_API
