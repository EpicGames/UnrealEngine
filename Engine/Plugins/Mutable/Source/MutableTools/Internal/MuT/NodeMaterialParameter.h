// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "MuR/Material.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeImage.h"


#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** NodeMaterialParameter class. */
	class NodeMaterialParameter : public NodeMaterial
	{
	public:

		/** Name of the parameter */
		FString Name;

		/** User provided ID to identify the parameter. */
		FString UID;

		/** Ranges for the parameter in case it is multidimensional. */
		TArray<Ptr<NodeRange>> Ranges;

		TMap<FParameterKey, Ptr<NodeImage>> ImageParametersToCompile;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		// Forbidden. Manage with the Ptr<> template.
		inline ~NodeMaterialParameter() = default;

	private:

		static UE_API FNodeType StaticType;

	};
}

#undef UE_API
