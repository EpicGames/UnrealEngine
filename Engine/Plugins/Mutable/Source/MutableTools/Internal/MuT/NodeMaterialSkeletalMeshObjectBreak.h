// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeSkeletalMeshObject.h"


#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	/** Provided a mesh get the material whose material slot name does match the "SlotName" set. */
	class NodeMaterialSkeletalMeshObjectBreak : public NodeMaterial
	{
	public:
		/** The name of the material slot to get from the given mesh. */
		FName SlotName;

		/** The mesh to use as source for the material set in the slot with "SlotName" name. */
		Ptr<NodeSkeletalMeshObject> SkeletalMeshObject;
		
		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		// Forbidden. Manage with the Ptr<> template.
		virtual ~NodeMaterialSkeletalMeshObjectBreak() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
