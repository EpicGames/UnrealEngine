// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeMaterial.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** NodeMaterialSwitch class. */
	class NodeMaterialSwitch : public NodeMaterial
	{
	public:

		Ptr<NodeScalar> Parameter;

		TArray<Ptr<NodeMaterial>> Options;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		// Forbidden. Manage with the Ptr<> template.
		inline ~NodeMaterialSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
