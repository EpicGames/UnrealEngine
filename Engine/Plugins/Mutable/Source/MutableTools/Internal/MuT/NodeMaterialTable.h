// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Table.h"
#include "MuT/NodeMaterial.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** NodeMaterialTable class. */
	class NodeMaterialTable : public NodeMaterial
	{
	public:
		FString ParameterName;
		FString ColumnName;
		FString DefaultRowName;
		
		bool bNoneOption = false;
		
		Ptr<FTable> Table;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		// Forbidden. Manage with the Ptr<> template.
		inline ~NodeMaterialTable() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
