// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/NodeMaterial.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** NodeMaterialVariation class. */
	class NodeMaterialVariation : public NodeMaterial
	{
	public:
		Ptr<NodeMaterial> DefaultMaterial;

		struct FVariation
		{
			Ptr<NodeMaterial> Material;
			FString Tag;
		};

		TArray<FVariation> Variations;
	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		// Forbidden. Manage with the Ptr<> template.
		inline ~NodeMaterialVariation() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
