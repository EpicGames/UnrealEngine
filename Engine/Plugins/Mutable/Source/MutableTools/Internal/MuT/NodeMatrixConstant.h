// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeMatrix.h"
#include "Math/Matrix.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	class NodeMatrixConstant : public NodeMatrix
	{
	public:
		FMatrix44f Value;

		// Node interface
		const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeMatrixConstant() override = default;

	private:

		static UE_API FNodeType StaticType;
	};

}

#undef UE_API
