// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** Node that applies a pose to a mesh, baking it into the vertex data. */
    class NodeMeshApplyPose : public NodeMesh
	{
	public:

		Ptr<NodeMesh> Base;
		Ptr<NodeMesh> Pose;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden.Manage with the Ptr<> template. */
		~NodeMeshApplyPose() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
