// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeMesh.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	//! 
	class NodeMeshClipDeform : public NodeMesh
	{
	public:

		Ptr<NodeMesh> BaseMesh;
		Ptr<NodeMesh> ClipShape;
		Ptr<NodeImage> ShapeWeights;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeMeshClipDeform() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
