// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    //! This node applies a geometric transform represented by a 4x4 matrix to a mesh
    class NodeMeshClipMorphPlane : public NodeMesh
	{
	public:

		Ptr<NodeMesh> Source;

		FClipMorphPlaneParameters Parameters;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		//-----------------------------------------------------------------------------------------
		// Own Interface
		//-----------------------------------------------------------------------------------------

		//! Source mesh to be clipped and morphed
        UE_API NodeMeshPtr GetSource() const;
		UE_API void SetSource( NodeMesh* );

		UE_API void SetPlane(FVector3f Center, FVector3f Normal);
		UE_API void SetParams(float dist, float factor);
		UE_API void SetMorphEllipse(float radius1, float radius2, float rotation);

		//! Define an axis-aligned box that will select the vertices to be morphed.
		//! If this is not set, or the size of the box is 0, all vertices will be affected.
		//! Only one of Box or Bone Hierarchy can be used (the last one set)
		UE_API void SetVertexSelectionBox(FVector3f Center, FVector3f Radius);

		//! Define the root bone of the subhierarchy of the mesh that will be affected.
		//! Only one of Box or Bone Hierarchy can be used (the last one set)
		UE_API void SetVertexSelectionBone(FName BoneId, float maxEffectRadius);


	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshClipMorphPlane() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
