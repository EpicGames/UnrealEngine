// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/Types.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    //! This node applies a geometric transform represented by a 4x4 matrix to a mesh
	//! \ingroup model
    class NodeMeshClipWithMesh : public NodeMesh
	{
	public:

		Ptr<NodeMesh> Source;
    	
		Ptr<NodeMesh> ClipMesh;

    	EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
    	
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeMeshClipWithMesh() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
