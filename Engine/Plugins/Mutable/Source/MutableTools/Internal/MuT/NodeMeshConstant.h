// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeLayout.h"
#include "MuR/Mesh.h"
#include "Templates/SharedPointer.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that outputs a constant mesh.
	* It allows to define the layouts for the texture channels of the constant mesh
	*/
	class NodeMeshConstant : public NodeMesh
	{
	public:

		FSourceDataDescriptor SourceDataDescriptor;

		UE::Mutable::Private::TManagedPtr<FMesh> Value;

		TArray<Ptr<NodeLayout>> Layouts;
		
		TArray<FName> RealTimeMorphNames;

		int8 BonePosePriority = 0;
		int8 SocketPriority = 0;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshConstant() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
