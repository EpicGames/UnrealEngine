// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeMesh.h"

#include "MuR/Ptr.h"
#include "MuT/NodeExternal.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeMeshExternal : public NodeMesh
	{
	public:
		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		// Forbidden. Manage with the Ptr<> template.
		virtual ~NodeMeshExternal() override = default;

	private:
		static UE_API FNodeType StaticType;

	public:
		Ptr<NodeExternal> Node;
	};
}

#undef UE_API
