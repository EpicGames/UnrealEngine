// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuR/Mesh.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	// Forward definitions
	class FMeshBufferSet;

	/** This node can change the buffer formats of a mesh vertices, indices and faces. */
	class NodeMeshFormat : public NodeMesh
	{
	public:

		/** Source mesh to transform. */
		Ptr<NodeMesh> Source;

		/** New mesh format.The buffers in the sets have no elements, but they define the formats. */
		FMeshBufferSet VertexBuffers;
		FMeshBufferSet IndexBuffers;

		/** */
		bool bOptimizeBuffers = false;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshFormat() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
