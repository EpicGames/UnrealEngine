// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeLayout.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** This node extracts a fragment of a mesh by some selection on a texture layout space. */
	class NodeMeshFragment : public NodeMesh
	{
	public:

		/** Mesh from which a fragment will be extracted. */
		Ptr<NodeMesh> SourceMesh;

		/** Layout defining the blocks of mesh to extract by texture space. 
		* If no layout is specified, all the blocks in the layout in the SourceMesh subgraph will be extracted.
		*/
		Ptr<NodeLayout> Layout;

		/** Index of the UV channel in the source mesh to apply the Layout to decide what to extract. */
		int32 LayoutIndex = 0;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshFragment() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
