// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that morphs a base mesh with one weighted target. */
	class NodeMeshMorph : public NodeMesh
	{
	public:

		FName Name;
		Ptr<NodeScalar> Factor;
		Ptr<NodeMesh> Base;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshMorph() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
