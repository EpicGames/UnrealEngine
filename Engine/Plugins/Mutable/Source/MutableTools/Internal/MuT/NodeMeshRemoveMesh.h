// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/Types.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	
	class NodeMeshRemoveMesh : public NodeMesh
	{
	public:

		Ptr<NodeMesh> Source;
    	
		Ptr<NodeMesh> RemoveMesh;

		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
    	
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeMeshRemoveMesh() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
