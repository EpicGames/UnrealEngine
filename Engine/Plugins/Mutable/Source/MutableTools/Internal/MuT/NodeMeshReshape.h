// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuR/Mesh.h"
#include "MuR/Skeleton.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that morphs a base mesh with one or two weighted targets from a sequence. */
	class NodeMeshReshape : public NodeMesh
	{
	public:

		Ptr<NodeMesh> BaseMesh;
		Ptr<NodeMesh> BaseShape;
		Ptr<NodeMesh> TargetShape;
		bool bReshapeVertices = true;
		bool bRecomputeNormals = false;
		bool bApplyLaplacian = false;
		bool bReshapeSkeleton = false;
		bool bReshapePhysicsVolumes = false;
		bool bReshapeSkeletonInvertSelection = false;
		bool bReshapePhysicsVolumesInvertSelection = false;

		EVertexColorUsage ColorRChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage ColorGChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage ColorBChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage ColorAChannelUsage = EVertexColorUsage::None;

		TArray<FName> BonesToDeform;
		TArray<FName> PhysicsToDeform;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshReshape() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
