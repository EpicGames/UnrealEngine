// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeSkeletalMeshObject.h"

#include "NodeSkeletalMeshObjectParameter.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeLayout.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeMeshSkeletalMeshObjectBreak : public NodeMesh
	{
	public:
		TManagedPtr<FMesh> ReferenceMesh;

		TArray<Ptr<NodeLayout>> Layouts;

		TArray<FName> RealTimeMorphNames;

		uint8 BaseLODIndex = 0;

		uint8 BaseSectionIndex = 0;

		uint8 LODBias = 0;

		uint8 ConversionFlags = 0;

		int8 BonePosePriority = 0;
		int8 SocketPriority = 0;

		Ptr<NodeSkeletalMeshObject> SkeletalMeshObject;
		
		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeMeshSkeletalMeshObjectBreak() override {}

	private:
		static UE_API FNodeType StaticType;
	};

}

#undef UE_API
