// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeLayout.h"
#include "MuT/NodeImage.h"
#include "MuT/Table.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	//! This node provides the meshes stored in the column of a table.
	class NodeMeshTable : public NodeMesh
	{
	public:

		FString ParameterName;
		Ptr<FTable> Table;
		FString ColumnName;
		bool bNoneOption = false;
		FString DefaultRowName;

		TArray<Ptr<NodeLayout>> Layouts;
		TArray<FName> RealTimeMorphNames;

		/** */
		FSourceDataDescriptor SourceDataDescriptor;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshTable() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
