// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/Types.h"
#include "MuT/Node.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeMatrix.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** */
	class NodeMeshTransformInMesh : public NodeMesh
	{
	public:

		/** */
		Ptr<NodeMesh> SourceMesh;
		
		/** */
		Ptr<NodeMesh> BoundingMesh;

		/** */
		Ptr<NodeMatrix> MatrixNode;
		
	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeMeshTransformInMesh() override = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API