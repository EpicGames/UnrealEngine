// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeMesh.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** */
	class NodeMeshVariation : public NodeMesh
    {
	public:

		Ptr<NodeMesh> DefaultMesh;

		struct FVariation
		{
			Ptr<NodeMesh> Mesh;
			FString Tag;
		};

		TArray<FVariation> Variations;

    public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

    protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMeshVariation() = default;

    private:

		static UE_API FNodeType StaticType;

    };

}

#undef UE_API
