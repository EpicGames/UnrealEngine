// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeModifier.h"

#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMeshNew.h"
#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeModifierSkeletalMeshMerge : public NodeModifier
	{
	public:
		FName ParentSkeletalMeshName;

		Ptr<NodeSkeletalMesh> ToAddSkeletalMesh;

		// Node interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeModifierSkeletalMeshMerge() override = default;

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
