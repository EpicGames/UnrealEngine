// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Base class for any node that outputs an object. */
	class NodeObject : public Node
	{
	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// Own interface

		//! Get the name of the object.
		virtual const FString& GetName() const = 0;

		//! Set the name of the object.
		virtual void SetName( const FString& ) = 0;

		//! Get the uid of the object.
		virtual const FString& GetUid() const = 0;

		//! Set the uid of the object.
		virtual void SetUid( const FString& ) = 0;

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeObject() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
