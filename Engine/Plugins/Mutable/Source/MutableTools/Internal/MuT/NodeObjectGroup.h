// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeObject.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that creates a group of objects and describes how they are selected. */
	class NodeObjectGroup : public NodeObject
	{
	public:

		FString Name;
		FString Uid;

		//! Typed of child selection
		typedef enum
		{
			//! All objects in the group will always be enabled, and no parameter will be generated.
			CS_ALWAYS_ALL,

			//! Only one children may be selected, but it is allowed to have none.
			//! An enumeration parameter will be generated but it may have a null value
			CS_ONE_OR_NONE,

			//! One and only one children has to be selected at all times
			//! An enumeraation parameter will be generated and it cannot be null
			CS_ALWAYS_ONE,

			//! Each child in the group can be enabled or disabled individually.
			//! A boolean parameter will be generated for every child.
			CS_TOGGLE_EACH

		} EChildSelection;

		EChildSelection Type;

		TArray<Ptr<NodeObject>> Children;

		//! Default value for CS_ONE_OR_NONE or CS_ALWAYS_ONE groups
		//! the value is the index of the child option in the group
		//! -1 is the value for the None option.
		//! 0 is the first child whether or not the NONE option is present.
		int32 DefaultValue;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// NodeObject Interface
        UE_API virtual const FString& GetName() const override;
		UE_API virtual void SetName( const FString&) override;
		UE_API virtual const FString& GetUid() const override;
		UE_API virtual void SetUid( const FString&) override;

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeObjectGroup() = default;

	private:

		static UE_API FNodeType StaticType;

	};



}

#undef UE_API
