// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuT/Node.h"
#include "MuT/NodeObject.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeExtensionData.h"
#include "MuT/Compiler.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeModifier;

	//! Node that creates a new object by setting its levels-of-detail and its children.
	class NodeObjectNew : public NodeObject
	{
	public:

		/** Name of the object. */
		FString Name;

		/** Externally provided id for the object. */
		FString Uid;

		/** Components defined in the object. */
		TArray<Ptr<NodeComponent>> Components;

		/** Modifiers defined in the object. */
		TArray<Ptr<NodeModifier>> Modifiers;

		/** Children objects. */
		TArray<Ptr<NodeObject>> Children;

		/** Extension data attached to this object. */
		TArray<Ptr<NodeExtensionData>> ExtensionDataNodes;

		/** States defined in this object. */
		TArray<FObjectState> States;


	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// NodeObject Interface
        UE_API virtual const FString& GetName() const override;
		UE_API virtual void SetName( const FString& ) override;
		UE_API virtual const FString& GetUid() const override;
		UE_API virtual void SetUid( const FString& ) override;

		// Own Interface

		//! Set the number of states that the model can be in.
		UE_API int32 GetStateCount() const;
		UE_API void SetStateCount( int32 c );

		//! Set the name of a state
		UE_API void SetStateName( int32 s, const FString& n );

		//! See if a state has a parameter as runtime.
		UE_API bool HasStateParam( int32 s, const FString& param ) const;

		//! Add a runtime parameter to the state.
		UE_API void AddStateParam( int32 s, const FString& param );

        //! Set the optimisation properties of a state
		UE_API void SetStateProperties(int32 StateIndex,
			ETextureCompressionStrategy TextureCompressionStrategy,
			bool bOnlyFirstLOD,
			uint8 NumExtraLODsToBuildAfterFirstLOD);

		//! Connect a node that produces ExtensionData to be added to the final Instance, and provide a name to associate with the data
		UE_API void AddExtensionDataNode(Ptr<NodeExtensionData> Node);

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeObjectNew() = default;

	private:

		static UE_API FNodeType StaticType;

	};



}

#undef UE_API
