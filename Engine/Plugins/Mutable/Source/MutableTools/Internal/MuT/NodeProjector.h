// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    //! Base class of any node that outputs a Projector.
	class NodeProjector : public Node
	{
	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeProjector() = default;

	private:

		static UE_API FNodeType StaticType;

	};


	//! This node outputs a predefined Projector value.
	class NodeProjectorConstant : public NodeProjector
	{
	public:

		EProjectorType Type = EProjectorType::Planar;
		FVector3f Position = FVector3f::ZeroVector;
		FVector3f Direction = FVector3f::ZeroVector;
		FVector3f Up = FVector3f::ZeroVector;
		FVector3f Scale = FVector3f::ZeroVector;
		float ProjectionAngle = 0.0f;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// Own Interface

		//! Get the value that this node returns
        UE_API void GetValue( EProjectorType* OutType,
			FVector3f* OutPos,
			FVector3f* OutDir,
			FVector3f* OutUp,
			FVector3f* OutScaleU,
			float* OutProjectionAngle ) const;

		//! Set the value to be returned by this node
        UE_API void SetValue( EProjectorType type, FVector3f pos, FVector3f dir, FVector3f up, FVector3f scale, float projectionAngle);

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeProjectorConstant() = default;

	private:

		static UE_API FNodeType StaticType;

	};


	//! Node that defines a Projector model parameter.
	class NodeProjectorParameter : public NodeProjector
	{
	public:

		EProjectorType Type = EProjectorType::Planar;
		FVector3f Position = FVector3f::ZeroVector;
		FVector3f Direction = FVector3f::ZeroVector;
		FVector3f Up = FVector3f::ZeroVector;
		FVector3f Scale = FVector3f::ZeroVector;
		float ProjectionAngle = 0.0f;

		FString Name;
		FString UID;

		TArray<Ptr<NodeRange>> Ranges;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		// Own Interface

		//! Set the name of the parameter.
		UE_API void SetName( const FString& );

		//! Get the uid of the parameter. It will be exposed in the final compiled data.
		UE_API const FString& GetUid() const;
		UE_API void SetUid( const FString& );

		//! Set the default value of the parameter.
        UE_API void SetDefaultValue( EProjectorType type,
			FVector3f pos,
			FVector3f dir,
			FVector3f up,
			FVector3f scale,
			float projectionAngle );

        //! Set the number of ranges (dimensions) for this parameter.
        //! By default a parameter has 0 ranges, meaning it only has one value.
        UE_API void SetRangeCount( int32 i );
        UE_API void SetRange( int32 i, Ptr<NodeRange> Range );

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeProjectorParameter() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
