// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeScalar.h"
#include "Containers/UnrealString.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	// Forward definitions
    class NodeScalar;

    class NodeRangeFromScalar : public NodeRange
	{
	public:

		Ptr<NodeScalar> Size;

		FString Name;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeRangeFromScalar() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
