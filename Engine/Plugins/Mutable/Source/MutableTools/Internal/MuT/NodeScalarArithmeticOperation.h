// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** Perform an arithmetic operation between two scalars. */
    class NodeScalarArithmeticOperation : public NodeScalar
	{
	public:

		/** Possible arithmetic operations. */
		typedef enum
		{
			AO_ADD = 0,
			AO_SUBTRACT,
			AO_MULTIPLY,
			AO_DIVIDE,
			_AO_COUNT
		} EOperation;

		EOperation Operation;
		Ptr<NodeScalar> A;
		Ptr<NodeScalar> B;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarArithmeticOperation() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
