// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"
#include "Curves/RichCurve.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

    /** This node makes a new scalar value transforming another scalar value with a curve. */
    class NodeScalarCurve : public NodeScalar
	{
	public:

		FRichCurve Curve;
		Ptr<NodeScalar> CurveSampleValue;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarCurve() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
