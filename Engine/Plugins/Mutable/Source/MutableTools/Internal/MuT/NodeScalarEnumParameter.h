// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** Node that defines a scalar model parameter to be selected from a set of named values. */
	class NodeScalarEnumParameter : public NodeScalar
	{
	public:

		FString Name;
		FString UID;
		int32 DefaultValue = 0;

		struct FOption
		{
			FString Name;
			float Value = 0.0f;
		};

		TArray<FOption> Options;

		TArray<Ptr<NodeRange>> Ranges;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarEnumParameter() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
