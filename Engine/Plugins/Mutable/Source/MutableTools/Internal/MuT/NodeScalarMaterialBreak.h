// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Node.h"
#include "MuT/NodeScalar.h"
#include "MuR/Ptr.h"
#include "MuT/NodeMaterial.h"
#include "MuR/Material.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	/** NodeMaterialTable class. */
	class NodeScalarMaterialBreak : public NodeScalar
	{
	public:

		Ptr<NodeMaterial> MaterialSource;
		FParameterKey ParameterKey;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarMaterialBreak() = default;

	private:

		static UE_API FNodeType StaticType;
	};
}

#undef UE_API
