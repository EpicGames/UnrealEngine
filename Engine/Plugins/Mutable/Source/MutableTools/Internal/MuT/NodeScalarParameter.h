// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	//! Node that defines a scalar model parameter.
	class NodeScalarParameter : public NodeScalar
	{
	public:

		float DefaultValue = 0.0f;
		FString Name;
		FString UID;

		TArray<Ptr<NodeRange>> Ranges;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarParameter() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
