// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

    /** This node selects an output Scalar from a set of input Scalars based on a parameter. */
    class NodeScalarSwitch : public NodeScalar
	{
	public:

		Ptr<NodeScalar> Parameter;
		TArray<Ptr<NodeScalar>> Options;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
