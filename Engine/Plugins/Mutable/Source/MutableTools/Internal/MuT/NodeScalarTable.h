// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"
#include "MuT/Table.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** This node provides the meshes stored in the column of a table. */
	class NodeScalarTable : public NodeScalar
	{
	public:

		FString ParameterName;
		Ptr<FTable> Table;
		FString ColumnName;
		bool bNoneOption = false;
		FString DefaultRowName;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return GetStaticType(); }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeScalarTable() = default;

	private:

		static UE_API FNodeType StaticType;

	};

}

#undef UE_API
