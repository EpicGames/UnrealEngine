// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeSkeletalMesh.h"
#include "MuR/Ptr.h"
#include "MuR/Types.h"
#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeSkeletalMeshClipWithSkeletalMesh : public NodeSkeletalMesh
	{
	public:

		Ptr<NodeSkeletalMesh> SourceSkeletalMesh;
    		
		Ptr<NodeSkeletalMesh> ClipSkeletalMesh;

		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
    		
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeSkeletalMeshClipWithSkeletalMesh() = default;

	private:

		static UE_API FNodeType StaticType;

	};
}

#undef UE_API
	