// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSkeletalMesh.h"

#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMeshObject.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshConvert : public NodeSkeletalMesh
	{
	public:
		Ptr<NodeSkeletalMeshObject> SkeletalMesh;
		uint8 ConversionFlags = 0;

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeSkeletalMeshConvert() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API

