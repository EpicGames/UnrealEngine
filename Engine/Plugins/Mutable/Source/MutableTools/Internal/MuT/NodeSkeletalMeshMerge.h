// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeSkeletalMesh.h"
// #include "HAL/Platform.h"
#include "MuR/Ptr.h"
//#include "MuT/Node.h"
// #include "MuT/NodeLOD.h"


#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	class NodeSkeletalMeshMerge : public NodeSkeletalMesh
	{
	public:
		
		/** Mesh where we want to merge into the sections of the ToAddSkeletalMesh*/
		Ptr<NodeSkeletalMesh> BaseSkeletalMesh;
		
		/** Skeletal Mesh whose section will be merged with the ones on the BaseSkeletalMesh*/
		Ptr<NodeSkeletalMesh> ToAddSkeletalMesh;
		
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeSkeletalMeshMerge() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}

#undef UE_API
