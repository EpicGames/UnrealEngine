// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeMaterial.h"
#include "NodeSkeletalMesh.h"
#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeLOD.h"


#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshModify : public NodeSkeletalMesh
	{
	public:
		
		// Skeletal mesh whose slots we want to override
		Ptr<NodeSkeletalMesh> SkeletalMesh;
		
		// Map of Slots (FName) where each one of them will have a Material assigned to them
		TMap<FName, Ptr<NodeMaterial>> SlotMaterials;

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeSkeletalMeshModify() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API