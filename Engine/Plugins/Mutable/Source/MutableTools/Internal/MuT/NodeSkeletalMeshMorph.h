// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeSkeletalMesh.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	/** Node that morphs a base skeletal mesh with one weighted target. */
	class NodeSkeletalMeshMorph : public NodeSkeletalMesh
	{
	public:

		FName Name;
		Ptr<NodeScalar> Factor;
		Ptr<NodeSkeletalMesh> Base;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeSkeletalMeshMorph() = default;

	private:

		static UE_API FNodeType StaticType;

	};
}

#undef UE_API