// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeSkeletalMesh.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeLOD.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshNew : public NodeSkeletalMesh
	{
	public:
		TArray<Ptr<NodeLOD>> LODs;

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeSkeletalMeshNew() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API

