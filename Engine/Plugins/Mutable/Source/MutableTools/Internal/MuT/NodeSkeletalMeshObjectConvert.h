// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeSkeletalMeshObject.h"

#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMesh.h"
#include "MuT/LODInfo.h"
#include "UObject/PerPlatformProperties.h"
#include "PerQualityLevelProperties.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshObjectConvert : public NodeSkeletalMeshObject
	{
	public:
		Ptr<NodeSkeletalMesh> SkeletalMesh;
		
		FName Name;
		int32 NumLODs = 0;
		int32 FirstLODAvailable = 0;
		int32 FirstLODResident = 0;
		
		FPerPlatformInt MinLODs;
		FPerQualityLevelInt MinQualityLevelLODs;
		
		TArray<FLODInfo> LODInfos;
		
		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeSkeletalMeshObjectConvert() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API

