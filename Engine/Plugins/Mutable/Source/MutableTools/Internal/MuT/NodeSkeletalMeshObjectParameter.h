// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSkeletalMeshObject.h"

#include "MuR/Ptr.h"
#include "MuT/NodeRange.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshObjectParameter : public NodeSkeletalMeshObject
	{
	public:
		FString Name;

		FString UID;

		TArray<Ptr<NodeRange>> Ranges;
		
		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		virtual ~NodeSkeletalMeshObjectParameter() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
