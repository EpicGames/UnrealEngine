// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSkeletalMeshObject.h"

#include "MuR/Ptr.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshObjectSwitch : public NodeSkeletalMeshObject
	{
	public:

		Ptr<NodeScalar> Parameter;

		TArray<Ptr<NodeSkeletalMeshObject>> Options;

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		virtual ~NodeSkeletalMeshObjectSwitch() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
