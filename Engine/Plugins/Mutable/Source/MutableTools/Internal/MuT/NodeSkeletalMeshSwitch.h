// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuT/NodeSkeletalMesh.h"
#include "MuT/NodeScalar.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	/** This node selects an output Skeletal Mesh from a set of input Skeletal Meshes based on a parameter. */
	class NodeSkeletalMeshSwitch : public NodeSkeletalMesh
	{
	public:

		Ptr<NodeScalar> Parameter;

		TArray<Ptr<NodeSkeletalMesh>> Options;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeSkeletalMeshSwitch() = default;

	private:

		static UE_API FNodeType StaticType;

	};
	
	
}

#undef UE_API
