// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSkeletalMesh.h"

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "Math/Matrix.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
    /** This node applies a geometric transform represented by a 4x4 matrix to a mesh. */
    class NodeSkeletalMeshTransform : public NodeSkeletalMesh
	{
	public:

		Ptr<NodeSkeletalMesh> Source;
		FMatrix44f Transform;

	public:

		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:
		/** Forbidden. Manage with the Ptr<> template. */
		virtual ~NodeSkeletalMeshTransform() override = default;

	private:

		static UE_API FNodeType StaticType;

	};
}


#undef UE_API
