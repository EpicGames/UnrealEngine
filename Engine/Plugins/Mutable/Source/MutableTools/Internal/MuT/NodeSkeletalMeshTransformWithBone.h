// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSkeletalMesh.h"

#include "HAL/Platform.h"
#include "MuT/NodeMatrix.h"
#include "MuR/Ptr.h"
#include "MuT/Node.h"
#include "MuT/NodeLOD.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	class NodeSkeletalMeshTransformWithBone : public NodeSkeletalMesh
	{
	public:
		FName BoneName = NAME_None;
		
		float ThresholdFactor = 0.0f;
		
		Ptr<NodeSkeletalMesh> Source;

		Ptr<NodeMatrix> MatrixNode;

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }
	
	protected:
		virtual ~NodeSkeletalMeshTransformWithBone() override {}

	private:
		static UE_API FNodeType StaticType;
	};
}


#undef UE_API