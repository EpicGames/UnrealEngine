// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	// Forward definitions
    class NodeSurface;
    typedef Ptr<NodeSurface> NodeSurfacePtr;
    typedef Ptr<const NodeSurface> NodeSurfacePtrConst;


    /** This class is the parent of all nodes that output a Surface. */
    class NodeSurface : public Node
	{
	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
        inline ~NodeSurface() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
