// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeModifier.h"

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "Containers/UnrealString.h"

#include "NodeSurfaceModifier.generated.h"

#define UE_API MUTABLETOOLS_API


/** Despite being an UEnum, this is not always version-serialized (in MutableTools).
* Beware of changing the enum options or order.
*/
UENUM()
enum class EMutableMultipleTagPolicy : uint8
{
	/** All the Mesh Sections that do have at least one of the Target tags will be modified. */
	OnlyOneRequired,

	/** Only the Mesh Sections that do have all the Target tags will be modified. */
	AllRequired
};


namespace UE::Mutable::Private
{

	// Forward definitions
	class NodeSurfaceModifier;
	typedef Ptr<NodeSurfaceModifier> NodeModifierPtr;
	typedef Ptr<const NodeSurfaceModifier> NodeModifierConst;

	/** Parent of all node classes that apply modifiers to surfaces. */
	class NodeSurfaceModifier : public NodeModifier
	{
	public:

		/** If not negative, this modifier will only be applied to the nodes of the component with matching id. Otherwise it will be applied to all components. */
		int32 RequiredComponentId = -1;

		/** Tags that target surface need to have enabled to receive this modifier. */
		TArray<FString> RequiredTags;

		/** In case of multiple tags in RequiredTags: are they all required, or one is enough? */
		EMutableMultipleTagPolicy MultipleTagsPolicy = EMutableMultipleTagPolicy::OnlyOneRequired;

		/** Wether the modifier has to be applied before the normal node operations or after. */
		bool bApplyBeforeNormalOperations = true;

		/** Tags enabled by this modifier. Other modifiers activated by these tags will be applied to this modifier's "child data" like meshes added by this modifier.
		* Not to be confused with the RequiredTags.
		*/
		TArray<FString> EnableTags;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeSurfaceModifier() = default;

	private:

		static UE_API FNodeType StaticType;
	};
}


#undef UE_API
