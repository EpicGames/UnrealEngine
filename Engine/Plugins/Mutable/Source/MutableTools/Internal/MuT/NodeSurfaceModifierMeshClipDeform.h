// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Mesh.h"
#include "MuT/Node.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeMesh.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	
	/** */
	class NodeSurfaceModifierMeshClipDeform : public NodeSurfaceModifier
	{
	public:

		Ptr<NodeMesh> ClipMesh;

		EShapeBindingMethod BindingMethod = EShapeBindingMethod::ClipDeformClosestProject;

		/** */
		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeSurfaceModifierMeshClipDeform() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
