// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/Skeleton.h"
#include "MuT/Node.h"
#include "MuT/NodeSurfaceModifier.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	struct FClipMorphPlaneParameters
	{
		// Morph field parameters

		// Distance to the plane of last affected vertex
		float DistanceToPlane = 0;

		// "Linearity" factor of the influence.
		float LinearityFactor = 0;

		// Ellipse location
		FVector3f Origin = FVector3f(0, 0, 0);
		FVector3f Normal = FVector3f(0, 0, 0);
		float Radius1 = 0, Radius2 = 0, Rotation = 0;

		// Vertex selection box
		EClipVertexSelectionType VertexSelectionType = EClipVertexSelectionType::None;
		FVector3f SelectionBoxOrigin = FVector3f(0, 0, 0);
		FVector3f SelectionBoxRadius = FVector3f(0, 0, 0);
		FName VertexSelectionBone = NAME_None;

		// Max distance a vertex can have to the bone in order to be affected. A negative value
		// means no limit.
		float MaxEffectRadius = -1.0f;

		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
	};


	/** */
	class NodeSurfaceModifierMeshClipMorphPlane : public NodeSurfaceModifier
	{
	public:

		FClipMorphPlaneParameters Parameters;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

		//-----------------------------------------------------------------------------------------
        // Own interface
		//-----------------------------------------------------------------------------------------

		UE_API void SetPlane(FVector3f Center, FVector3f Normal);
		UE_API void SetParams(float dist, float factor);
		UE_API void SetMorphEllipse(float radius1, float radius2, float rotation);

		//! Define an axis-aligned box that will select the vertices to be morphed.
		//! Only one of Box or Bone Hierarchy can be used (the last one set)
		UE_API void SetVertexSelectionBox(float centerX, float centerY, float centerZ, float radiusX, float radiusY, float radiusZ);

		//! Define the root bone of the subhierarchy of the mesh that will be affected.
		//! Only one of Box or Bone Hierarchy can be used (the last one set)
		UE_API void SetVertexSelectionBone(FName BoneId, float maxEffectRadius);

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeSurfaceModifierMeshClipMorphPlane() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
