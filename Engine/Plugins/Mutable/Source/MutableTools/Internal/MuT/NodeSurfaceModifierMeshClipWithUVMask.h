// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeLayout.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** This node can clip part of a mesh using UV space-data (an image or a list of layout blocks). */
	class NodeSurfaceModifierMeshClipWithUVMask : public NodeSurfaceModifier
	{
	public:

		/** Image with the regions to remove. It will be interpreted as a bitmap. */
		Ptr<NodeImage> ClipMask;

		/** If the image above is null, clipping may hapopen with layout blocks. */
		Ptr<NodeLayout> ClipLayout;

		/** Layout index of the UVs to use inthe source mesh to ben clipped with the mask. */
		uint8 LayoutIndex = 0;

		/** */
		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		inline ~NodeSurfaceModifierMeshClipWithUVMask() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
