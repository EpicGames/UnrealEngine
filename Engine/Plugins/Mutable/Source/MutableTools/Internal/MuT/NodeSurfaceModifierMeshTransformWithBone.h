// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/Types.h"
#include "MuR/Skeleton.h"
#include "MuT/Node.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeMatrix.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

/** */
class NodeSurfaceModifierMeshTransformWithBone : public NodeSurfaceModifier
{
public:

	/* */
	FName BoneName = NAME_None;
	float ThresholdFactor = 0.0f;

	/** */
	Ptr<NodeMatrix> MatrixNode;


public:

	// Node interface
	virtual const FNodeType* GetType() const override { return &StaticType; }
	static const FNodeType* GetStaticType() { return &StaticType; }

protected:

	/** Forbidden. Manage with the Ptr<> template. */
	virtual ~NodeSurfaceModifierMeshTransformWithBone() override = default;

private:

	static UE_API FNodeType StaticType;

};

}

#undef UE_API
