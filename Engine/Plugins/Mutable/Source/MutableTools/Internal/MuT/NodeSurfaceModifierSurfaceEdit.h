// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"
#include "MuR/ImageTypes.h"
#include "MuR/Material.h"
#include "MuR/RefCounted.h"
#include "MuR/Ptr.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{

	/** This node modifies a surface node.
    * It allows to extend, cut and morph the parent Surface's meshes.
    * It also allows to patch the parent Surface's textures.
	*/
    class NodeSurfaceModifierSurfaceEdit : public NodeSurfaceModifier
	{
	public:

		/** Data for every modified texture. */
		struct FTexture
		{
			/** Key used to match the image with the original one being modified. 
			* This should match the MaterialParameterKey in a NodeSurfaceNew::FImageData.
			*/
			FParameterKey MaterialParameterKey;

			/** Image to add if extgending. */
			Ptr<NodeImage> Extend;

			/** Image to blend if patching. */
			Ptr<NodeImage> PatchImage;

			/** Optional mask controlling the blending area. */
			Ptr<NodeImage> PatchMask;

			/** Rects in the parent layout homogeneous UV space to patch. */
			TArray<FBox2f> PatchBlocks;

			/** Type of patching operation. */
			EBlendType PatchBlendType = EBlendType::BT_BLEND;

			/** Patch alpha channel as well? */
			bool bPatchApplyToAlpha = false;
		};

		struct FLOD
		{
			/** Mesh to remove from the modified surface. */
			Ptr<NodeMesh> MeshRemove;

			/** Mesh to add to the modified surface. */
			Ptr<NodeMesh> MeshAdd;

			/** Textures to modify. */
			TArray<FTexture> Textures;
		};

		TArray<FLOD> LODs;

		/** For remove operations, use this strategy to cull faces. */
		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

		/** Name of the morph to apply to the modified surface if it has it. */
		FString MeshMorph;

		/** Factor of the morph to apply. */
		Ptr<NodeScalar> MorphFactor;

		/** Source modifier guid, used to share resources between LODs and generate a deterministic surface guid. */
		FGuid ModifierGuid;

	public:

		NodeSurfaceModifierSurfaceEdit() 
		{
			// This modifier needs to be applied at the end of the operations.
			bApplyBeforeNormalOperations = false;
		}

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeSurfaceModifierSurfaceEdit() = default;

	private:

		static UE_API FNodeType StaticType;

	};



}

#undef UE_API
