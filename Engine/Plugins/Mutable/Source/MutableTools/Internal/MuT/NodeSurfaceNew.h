// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeSurface.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeString.h"
#include "MuT/NodeColor.h"

#include "Misc/Guid.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	/** This node makes a new Surface (Mesh Material Section) from a mesh and several material parameters like images, vectors and scalars. */
	class NodeSurfaceNew : public NodeSurface
	{
	public:
		FName Name;

		/** An optional, opaque id that will be returned in the surfaces of the created
		* instances. Can be useful to identify surfaces on the application side.
		*/
		uint32 ExternalId = 0;

		/** Source surface Guid, used to share layouts between LODs and generate a deterministic SharedSurfaceId. */
		FGuid SurfaceGuid;

		Ptr<NodeMesh> Mesh;

		Ptr<NodeMaterial> Material;

		struct FStringData
		{
			FString Name;
			Ptr<NodeString> String;
		};

		TArray<FStringData> Strings;

		/** Tags added to the surface:
		* - the surface will be affected by modifier nodes with the same tag
		* - the tag will be enabled when the surface is added to an object, and it can activate
		* variations for any surface.
		*/
		TArray<FString> Tags;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeSurfaceNew() = default;

	private:

		static UE_API FNodeType StaticType;

	};


}

#undef UE_API
