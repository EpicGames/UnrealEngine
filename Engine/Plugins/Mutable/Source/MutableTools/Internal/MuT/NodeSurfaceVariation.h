// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeSurface.h"
#include "MuT/NodeSurfaceModifier.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{

	//! This node modifies a node of the parent object of the object that this node belongs to.
    //! It allows to extend, cut and morph the parent Surface's meshes.
    //! It also allows to patch the parent Surface's textures.
    class NodeSurfaceVariation : public NodeSurface
	{
	public:

		TArray<Ptr<NodeSurface>> DefaultSurfaces;
		TArray<Ptr<NodeSurfaceModifier>> DefaultModifiers;

		//!
		enum class VariationType : uint8
		{
			//! The variation selection is controlled by tags defined in other surfaces.
			//! Default value.
			Tag = 0,

			//! The variation selection is controlled by the state the object is in.
			State
		};

		NodeSurfaceVariation::VariationType Type = NodeSurfaceVariation::VariationType::Tag;

		struct FVariation
		{
			TArray<Ptr<NodeSurface>> Surfaces;
			TArray<Ptr<NodeSurfaceModifier>> Modifiers;
			FString Tag;
		};

		TArray<FVariation> Variations;

	public:

		// Node interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeSurfaceVariation() = default;

	private:

		static UE_API FNodeType StaticType;

	};



}

#undef UE_API
