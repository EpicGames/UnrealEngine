// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Serialisation.h"

#define UE_API MUTABLETOOLS_API


namespace UE::Mutable::Private
{
	// Forward declarations
	class FTable;
	class FMesh;
	class FImage;

	/** Types of the values for the table cells. */
	enum class ETableColumnType : uint32
	{
		None,
		Scalar,
		Color,
		Image,
		Mesh,
		String,
		Material
	};


	//! A table that contains many rows and defines attributes like meshes, images,
	//! colors, etc. for every column. It is useful to define a big number of similarly structured
	//! objects, by using the NodeDatabase in a model expression.
	//! \ingroup model
	class FTable : public RefCounted
	{
	public:

		UE_API FTable();

		//-----------------------------------------------------------------------------------------
		// Own interface
		//-----------------------------------------------------------------------------------------

		//!
		UE_API void SetName(const FString&);
		UE_API const FString& GetName() const;

		//!
		UE_API int32 AddColumn(const FString&, ETableColumnType );

		//! Return the column index with the given name. -1 if not found.
		UE_API int32 FindColumn(const FString&) const;

		//!
        UE_API void AddRow( uint32 id );

		//!
        UE_API void SetCell( int32 Column, uint32 RowId, int32 IntValue, const void* ErrorContext = nullptr);
        UE_API void SetCell( int32 Column, uint32 RowId, float Value, const void* ErrorContext = nullptr);
        UE_API void SetCell( int32 Column, uint32 RowId, const FVector4f& Value, const void* ErrorContext = nullptr);
		UE_API void SetCell( int32 Column, uint32 RowId, const UE::Mutable::Private::TManagedPtr<FImage>& Value, const void* ErrorContext = nullptr);
		UE_API void SetCell( int32 Column, uint32 RowId, const UE::Mutable::Private::TManagedPtr<FMesh>& Value, const void* ErrorContext = nullptr);
        UE_API void SetCell( int32 Column, uint32 RowId, const FString& Value, const void* ErrorContext = nullptr);


		//-----------------------------------------------------------------------------------------
		// Interface pattern
		//-----------------------------------------------------------------------------------------
		class Private;
		UE_API Private* GetPrivate() const;

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		UE_API ~FTable();

	private:

		Private* m_pD;

	};

}

#undef UE_API
