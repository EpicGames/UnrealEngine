// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Table.h"
#include "MuR/MutableMath.h"


namespace UE::Mutable::Private
{
	MUTABLE_DEFINE_ENUM_SERIALISABLE(ETableColumnType)


	struct FTableColumn
	{
		FString Name;

		ETableColumnType Type;
	};


	struct FTableValue
	{
		// TODO: Union
		int32 Int;
		float Scalar;
		FVector4f Color;
		TManagedPtr<FImage> Image;
		TManagedPtr<FMesh> Mesh;
		FString String;

		const void* ErrorContext;
	};


	struct FTableRow
	{
        uint32 Id;
		TArray<FTableValue> Values;
	};


	//!
	class FTable::Private
	{
	public:

		FString Name;
		TArray<FTableColumn> Columns;
		TArray<FTableRow> Rows;

		// Transient value for serialization compatibility
		bool bNoneOption_DEPRECATED = false;

		//! Find a row in the table by id. Return -1 if not found.
        int32 FindRow( uint32 id ) const
		{
			int32 res = -1;

			for ( int32 r=0; res<0 && r<Rows.Num(); ++r )
			{
				if ( Rows[r].Id==id )
				{
					res = (int32)r;
				}
			}

			return res;
		}

	};

}
