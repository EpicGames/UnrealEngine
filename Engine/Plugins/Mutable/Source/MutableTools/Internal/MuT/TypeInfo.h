// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Parameters.h"

#define UE_API MUTABLETOOLS_API

// This header file provides additional information about data types defined in the runtime, that
// can be useful only for the tools. For instance, it provides strings for some enumeration values.

namespace UE::Mutable::Private
{

	class TypeInfo
	{
	public:

		static UE_API const char* s_imageFormatName[size_t(EImageFormat::Count)];

		static UE_API const char* s_meshBufferSemanticName[uint32(EMeshBufferSemantic::Count)];

		static UE_API const char* s_meshBufferFormatName[uint32(EMeshBufferFormat::Count) ];

		static UE_API const char* s_blendModeName[uint32(EBlendType::_BT_COUNT) ];

		static UE_API const char* s_projectorTypeName [ static_cast<uint32>(UE::Mutable::Private::EProjectorType::Count) ];

	};

}

#undef UE_API
