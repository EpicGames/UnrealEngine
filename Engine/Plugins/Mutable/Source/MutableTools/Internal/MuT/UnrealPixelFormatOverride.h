// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "Interfaces/ITextureFormatModule.h"
#include "Engine/TextureDefines.h"
#include "TextureCompressorModule.h"
#include "TextureBuildUtilities.h"
#include "ImageCore.h"

/** This function needs to be called from the game thread before any call to UnrealPixelFormatFunc can be done. */
MUTABLETOOLS_API extern void PrepareUnrealCompression();

/** */
void FillBuildSettingsFromMutableFormat(FTextureBuildSettings& Settings, bool& bOutHasAlpha, UE::Mutable::Private::EImageFormat Format);

/** */
void MutableToImageCore(const UE::Mutable::Private::FImage* InMutable, FImage& CoreImage, int32 LOD, bool bSwizzleRGBHack);

/** */
bool ImageCoreToMutable(const FCompressedImage2D& Compressed, UE::Mutable::Private::FImage* Mutable, int32 LOD);

/** Convert an Unreal platform pixel format to Mutable. */
MUTABLETOOLS_API extern UE::Mutable::Private::EImageFormat UnrealToMutablePixelFormat(EPixelFormat PlatformFormat, bool bHasAlpha);

/** Function that remaps some formats to the ones that provide more acceptable quality in Mutable. */
MUTABLETOOLS_API extern UE::Mutable::Private::EImageFormat QualityAndPerformanceFix(UE::Mutable::Private::EImageFormat Format);

/** Try to convert an image using unreal's compression. If the format is not supported, it will return false in bOutSuccess.
* Can be called from any thread.
* PrepareUnrealCompression needs to be called from the game thread once prior to using this function. 
*/
MUTABLETOOLS_API extern void UnrealPixelFormatFunc(bool& bOutSuccess, int32 Quality, UE::Mutable::Private::FImage* Target, const UE::Mutable::Private::FImage* Source, int32 OnlyLOD);

/** Debug export the given image to a temp file in the saved folder. */
MUTABLETOOLS_API void DebugImageDump(const UE::Mutable::Private::FImage*, const FString& FilePath);
