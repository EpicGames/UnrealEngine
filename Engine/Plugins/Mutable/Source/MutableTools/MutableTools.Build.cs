// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class MutableTools : ModuleRules
    {
        public MutableTools(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "MuT";

			DefaultBuildSettings = BuildSettingsVersion.Latest;
			IWYUSupport = IWYUSupport.KeepAsIsForNow;

			PublicDependencyModuleNames.AddRange(
                new string[] {
					"MutableRuntime",
					"Core",
					"CoreUObject",
					"GeometryCore",
					"ImageCore",
					"TextureCompressor",
					"TextureBuildUtilities",
					"Engine",
				}
			);
		}
	}
}
