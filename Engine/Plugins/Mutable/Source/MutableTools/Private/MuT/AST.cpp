// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/AST.h"

#include "Containers/Set.h"
#include "Containers/Queue.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "MuR/Mesh.h"
#include "MuR/MutableTrace.h"
#include "MuR/Platform.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpParameter.h"
#include "Trace/Detail/Channel.h"

#include <atomic>


std::atomic<uint32> UE::Mutable::Private::ASTOp::s_lastTraverseIndex(1);


namespace UE::Mutable::Private
{

//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
ASTChild::ASTChild(ASTOp* p, const Ptr<ASTOp>& c)
    : Parent(p)
    , Child(c)
{
    if (Parent && Child.get())
    {
        AddParent();
    }
}

ASTChild::ASTChild( const Ptr<ASTOp>& p, const Ptr<ASTOp>& c)
    : ASTChild(p.get(),c)
{
}

ASTChild::~ASTChild()
{
    if (Child && Parent)
    {
        ClearParent();
    }
}

ASTChild& ASTChild::operator=( const Ptr<ASTOp>& c )
{
    if (c!=Child)
    {
        if (Child && Parent)
        {
            ClearParent();
        }

        Child = c;

        if (Child && Parent)
        {
            AddParent();
        }
    }

    return *this;
}

ASTChild& ASTChild::operator=( ASTChild&& rhs )
{
    Parent = rhs.Parent;
    ParentIndexInChild = rhs.ParentIndexInChild;
    Child = rhs.Child;
    rhs.Parent=nullptr;
    rhs.Child.reset();

    return *this;
}



//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void ASTOp::ForEachParent(const TFunctionRef<void(ASTOp*)> f) const
{
	for (ASTOp* Parent : Parents)
	{
		if (Parent)
		{
			f(Parent);
		}
	}
}


//-------------------------------------------------------------------------------------------------
void ASTOp::RemoveChildren()
{
    // Actually destroyed when running out of scope
    TArray<Ptr<ASTOp>> toDestroy;

    // Try to make children destruction iterative
    TArray<ASTOp*> pending;
	pending.Reserve(1024);
    pending.Add(this);

    while (pending.Num())
    {
        ASTOp* n = pending.Pop(EAllowShrinking::No);

        n->ForEachChild( [&](ASTChild& c)
        {
            if (c)
            {
                // Are we clearing the last reference?
                if (c.child()->IsUnique())
                {
                    toDestroy.Add(c.child());
                    pending.Add(c.child().get());
                }

                c = nullptr;
            }
        });
    }
}


//-------------------------------------------------------------------------------------------------
void ASTOp::Assert()
{
    // Check that every valid parent has us a child
    // TODO: Should count the numbers, since a node may be child of another in multiple connections.
    ForEachParent( [&](ASTOp*parent)
    {
        if(parent)
        {
            bool foundInParent = false;
            parent->ForEachChild([&](ASTChild&c)
            {
                if (c && c.Child.get()==this)
                {
                    foundInParent = true;
                }
            });

            // If we hit this, we have a parent that doesn't know us.
            check(foundInParent);
        }
    });

    // Validate the children
	ForEachChild( [this](ASTChild&c)
    {
        if(c)
        {
            // The child must have us as the parent.
//            bool found = false;
//            c.child()->ForEachParent( [&](ASTOp* childParent)
//            {
//                if (childParent==this)
//                {
//                    found = true;
//                }
//            });
//            check(found);
            check( c.ParentIndexInChild < c.child()->Parents.Num() );
            check( c.child()->Parents[c.ParentIndexInChild]==this );
        }
     });
}


//-------------------------------------------------------------------------------------------------
bool ASTOp::operator==( const ASTOp& other ) const
{
//    if (typeid(*this) != typeid(other))
//        return false;

    return IsEqual(other);
}


//---------------------------------------------------------------------------------------------
void ASTOp::FullAssert( const TArray<Ptr<ASTOp>>& Roots)
{
    MUTABLE_CPUPROFILER_SCOPE(AST_FullAssert);
    Traverse_TopDown_Unique_Imprecise(Roots, [](const Ptr<ASTOp>& n)
    {
        n->Assert();
        return true;
    });
}

//-------------------------------------------------------------------------------------------------
int32 ASTOp::CountNodes( const TArray<Ptr<ASTOp>>& Roots )
{
    MUTABLE_CPUPROFILER_SCOPE(AST_CountNodes);
	int32 Count=0;
    Traverse_TopRandom_Unique_NonReentrant(Roots, [&](const Ptr<ASTOp>&)
    {
        ++Count;
        return true;
    });
    return Count;
}


//-------------------------------------------------------------------------------------------------
UE::Mutable::Private::Ptr<ASTOp> ASTOp::DeepClone( const Ptr<ASTOp>& root )
{
    MUTABLE_CPUPROFILER_SCOPE(AST_DeepClone);

    TMap<Ptr<const ASTOp>,Ptr<ASTOp>> Visited;

    MapChildFunc m = [&](const Ptr<ASTOp>&n)
    {
		if (!n)
		{
			return Ptr<ASTOp>();
		}
		const Ptr<ASTOp>* it = Visited.Find(n);
        check(it);
        return *it;
    };

    Ptr<ASTOp> r = root;
    Traverse_BottomUp_Unique( r, [&](Ptr<ASTOp> Op)
    {
        Ptr<ASTOp> Cloned = Op->Clone( m );
        Visited.Add(Op, Cloned);
    });

	const Ptr<ASTOp>* it = Visited.Find(r);
    check(it);
    return *it;
}


//-------------------------------------------------------------------------------------------------
FOperation::ADDRESS ASTOp::FullLink( Ptr<ASTOp>& Root, FProgram& Program, FLinkerOptions* Options )
{
    MUTABLE_CPUPROFILER_SCOPE(AST_FullLink);

    auto Link = [&Program, Options](const Ptr<ASTOp>& Node)
    {
        Node->Link(Program, Options);
		check(!(GetAddressDataType(Node->LinkedAddress) == EDataType::None) || Node->LinkedAddress == 0);
    };

    auto Accept = [](const Ptr<const ASTOp>& Node)
    {
        return Node->LinkedAddress == 0;
    };

    Traverse_BottomUp_Unique(Root, Link, Accept); 

	FOperation::ADDRESS Result = Root->LinkedAddress;

	// This signals the caller that the Root pointer shouldn't be used anymore.
	Root = nullptr;

	return Result;
}


//---------------------------------------------------------------------------------------------
void ASTOp::LogHistogram( ASTOpList& roots )
{
    (void)roots;

 //   uint64 CountPerType[(int)EOpType::COUNT];
 //   FMemory::Memzero(CountPerType,sizeof(CountPerType));

 //   size_t count=0;

 //   Traverse_TopRandom_Unique_NonReentrant( roots,
 //                            [&](const Ptr<ASTOp>& n)
 //   {
 //       ++count;
	//	CountPerType[(int)n->GetOpType()]++;
 //       return true;
 //   });

	//TArray< TPair<uint64, EOpType> > Sorted;
	//Sorted.SetNum((int)EOpType::COUNT);
 //   for (int i=0; i<(int)EOpType::COUNT; ++i)
 //   {
 //       Sorted[i].Value = (EOpType)i;
 //       Sorted[i].Key = CountPerType[i];
 //   }


	//Sorted.Sort( [](const TPair<uint64, EOpType>& a, const TPair<uint64, EOpType>& b)
 //   {
 //       return a.Key >b.Key;
 //   });

 //   UE_LOGF(LogMutableCore,Log, "Op histogram (%llu ops):", count);
 //   for(int i=0; i<8; ++i)
 //   {
 //       float p = Sorted[i].Key/float(count)*100.0f;
 //       int OpType = (int)Sorted[i].Value;
 //       UE_LOGF(LogMutableCore, Log, "  %5.2f%% : %ls", p, s_opNames[OpType] );
 //   }

	//// Debug log part of the tree
	//Traverse_TopDown_Unique( roots,
	//	[&](const Ptr<ASTOp>& n)
	//	{
	//		if (n->GetOpType() == EOpType::IM_MULTILAYER)
	//		{
	//			Ptr<const ASTOpImageMultiLayer> Typed = static_cast<const ASTOpImageMultiLayer*>(n.get());
	//			Ptr<const ASTOp> Base = Typed->base.child();
	//			Ptr<const ASTOpConstantResource> Constant = static_cast<const ASTOpConstantResource*>(Base.get());

	//			// Log the op
	//			UE_LOGF(LogMutableCore, Log, "Multilayer at %x:", n.get());
	//			UE_LOGF(LogMutableCore, Log, "    base is type %ls:", s_opNames[(int)Base->GetOpType()]);
	//			if (Constant)
	//			{
	//				int Format = int( ((UE::Mutable::Private::FImage*)Constant->GetValue().get())->GetFormat() );
	//				UE_LOGF(LogMutableCore, Log, "    constant image format is %d:", Format);
	//			}
	//			else
	//			{
	//				FGetImageDescContext Context;
	//				FImageDesc Desc = Base->GetImageDesc(true, &Context);
	//				UE_LOGF(LogMutableCore, Log, "    estimated image format is %d:", int(Desc.m_format) );
	//			}
	//		}
	//		return true;
	//	});

}


//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_TopDown_Unique( const TArray<Ptr<ASTOp>>& roots, TFunctionRef<bool(Ptr<ASTOp>&)> f )
{
    TQueue<Ptr<ASTOp>> pending;
	for (const Ptr<ASTOp>& r : roots)
	{
		pending.Enqueue(r);
	}

    TSet<Ptr<const ASTOp>> traversed;

    // We record the parents of all roots as traversed
    for (const Ptr<ASTOp>& r: roots)
    {
        r->ForEachParent( [&]( const ASTOp* parent )
        {
            // If the parent is also a root, we will want to process it.
            if ( !roots.Contains(parent) )
            {
                traversed.Add( parent );
            }
        });
    }

    while (!pending.IsEmpty())
    {
		Ptr<ASTOp> pCurrent;
		pending.Dequeue(pCurrent);
        if (!pCurrent)
        {
            continue;
        }

        // Did we traverse all parents?
        bool parentsTraversed = true;

        pCurrent->ForEachParent( [&]( const ASTOp* parent )
        {
            if (!traversed.Contains(parent))
            {
                // \todo Is the parent in the relevant subtree?


                parentsTraversed = false;
            }
        });

        if (!parentsTraversed)
        {
            pending.Enqueue(pCurrent);
        }
        else if (!traversed.Contains(pCurrent))
        {
            traversed.Add(pCurrent);

            // Process
            bool recurse = f(pCurrent);

            // Recurse children
            if (recurse)
            {
                pCurrent->ForEachChild([&]( ASTChild& c )
                {
                    if (c && !traversed.Contains(c.Child))
                    {
                        pending.Enqueue( c.Child );
                    }
                });
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_TopDown_Unique_Imprecise( const TArray<Ptr<ASTOp>>& roots, TFunctionRef<bool(Ptr<ASTOp>&)> f )
{
    TQueue<Ptr<ASTOp>> pending;
	for (const Ptr<ASTOp>& r : roots)
	{
		pending.Enqueue(r);
	}

    TSet<Ptr<const ASTOp>> traversed;

    while (!pending.IsEmpty())
    {
		Ptr<ASTOp> pCurrent;
		pending.Dequeue(pCurrent);

        // It could have been completed in another branch
        if (pCurrent && !traversed.Contains(pCurrent))
        {
            traversed.Add(pCurrent);

            // Process
            bool recurse = f(pCurrent);

            // Recurse children
            if (recurse)
            {
                pCurrent->ForEachChild([&]( ASTChild& c )
                {
                    if (c && !traversed.Contains(c.Child))
                    {
                        pending.Enqueue( c.Child );
                    }
                });
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_TopRandom_Unique_NonReentrant( const TArray<Ptr<ASTOp>>& roots,
	TFunctionRef<bool(Ptr<ASTOp>&)> f )
{
    ASTOpList pending;

    uint32 traverseIndex = s_lastTraverseIndex++;

    for (const Ptr<ASTOp>& r:roots)
    {
        if (r && r->m_traverseIndex!=traverseIndex )
        {
            r->m_traverseIndex = traverseIndex;
            pending.Add( r );
        }
    }
    for(const Ptr<ASTOp>& p : pending)
    {
        p->m_traverseIndex = traverseIndex-1;
    }

    while (pending.Num())
    {
        Ptr<ASTOp> pCurrent = pending.Pop();

        // It could have been completed in another branch
        if (pCurrent->m_traverseIndex!=traverseIndex)
        {
            pCurrent->m_traverseIndex = traverseIndex;

            // Process
            bool recurse = f(pCurrent);

            // Recurse children
            if (recurse)
            {
                pCurrent->ForEachChild([&]( ASTChild& c )
                {
                    if (c && c.Child->m_traverseIndex!=traverseIndex)
                    {
                        pending.Add( c.Child );
                    }
                });
            }
        }
    }
}



//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void Visitor_TopDown_Unique_Cloning::Traverse( Ptr<ASTOp>& InOutRoot )
{
    // Visit the given root
    if (InOutRoot)
    {
		Pending.Add({ false,InOutRoot });

        Process();

		InOutRoot = GetOldToNew(InOutRoot);
    }
}


void Visitor_TopDown_Unique_Cloning::Process()
{
    while ( Pending.Num() )
    {
        TPair<bool,Ptr<ASTOp>> item = Pending.Pop();

        Ptr<ASTOp> at = item.Value;

		ASTOp::MapChildFunc Identity = [](const Ptr<ASTOp>& o) {return o; };
		
		if (item.Key)
        {
            // Item indicating we finished with all the children of this instruction
			Ptr<ASTOp> cop = at->Clone(Identity);

            // Fix the references to the children
            bool childChanged = false;
            cop->ForEachChild( [&](ASTChild& ref)
            {
				const Ptr<ASTOp>* it = OldToNew.Find(ref.Child);
                if ( ref && it && it->get()!=nullptr )
                {
                    auto oldRef = ref.Child;
                    ref=GetOldToNew(ref.Child);
                    if (ref.Child!=oldRef)
                    {
                        childChanged = true;
                    }
                }
            });

            // If any child changed, we need to replace this instruction
            if ( childChanged )
            {
                OldToNew.Add(at,cop);
            }

        }
        else
        {
			const Ptr<ASTOp>* it = OldToNew.Find(at);
            if (!it)
            {
				Ptr<ASTOp> initialAt = at;

                // Fix the references to the children, possibly adding a new instruction
                {
					Ptr<ASTOp> cop = at->Clone(Identity);
                    bool childChanged = false;
                    cop->ForEachChild( [&](ASTChild& ref)
                    {
						const Ptr<ASTOp>* ito = OldToNew.Find(ref.Child);
                        if ( ref && ito && ito->get()!=nullptr )
                        {
							Ptr<ASTOp> oldRef = ref.Child;
                            ref=GetOldToNew(ref.Child);
                            if (ref.Child!=oldRef)
                            {
                                childChanged = true;
                            }
                        }
                    });

                    // If any child changed, we need to re-add this instruction
                    if ( childChanged )
                    {
						OldToNew.Add(at, cop);
						at = cop;
                    }
                }

                bool processChildren = true;
				Ptr<ASTOp> newAt = Visit( at, processChildren );
				OldToNew.Add(initialAt,newAt);

                // Proceed with children
                if (processChildren)
                {
					Pending.Add({ true, newAt });

                    at->ForEachChild( [&](ASTChild& ref)
                    {
                        if (ref && !OldToNew.Contains(ref.Child))
                        {
							Pending.Add({ false,ref.Child });
                        }
                    });
                }
            }
        }

    }
}


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_BottomUp_Unique_NonReentrant
(
        ASTOpList& roots,
        TFunctionRef<void(Ptr<ASTOp>&)> f
)
{
    uint32 traverseIndex = s_lastTraverseIndex++;

    TArray< std::pair<Ptr<ASTOp>,int32> > pending;
    for (Ptr<ASTOp>& r:roots)
    {
        if (r && r->m_traverseIndex!=traverseIndex )
        {
            r->m_traverseIndex=traverseIndex;
            pending.Add( std::make_pair<>(r,0) );
        }
    }
    for(std::pair<Ptr<ASTOp>, int32>& p : pending)
    {
        p.first->m_traverseIndex = traverseIndex-1;
    }

    while (pending.Num())
    {
        int32 phase = pending.Last().second;
        Ptr<ASTOp> pCurrent = pending.Last().first;
        pending.Pop();

        // It could have been completed in another branch
        if (pCurrent->m_traverseIndex!=traverseIndex)
        {
            if (phase==0)
            {
                // Process this again...
                pending.Add( std::make_pair<>(pCurrent,1) );

                // ...after the children are processed
                pCurrent->ForEachChild([&]( ASTChild& c )
                {
                    if (c && c.Child->m_traverseIndex!=traverseIndex )
                    {
						std::pair<Ptr<ASTOp>, int32> e = std::make_pair<>(c.Child,0);
                        pending.Add( e );
                    }
                });
            }
            else
            {
                pCurrent->m_traverseIndex=traverseIndex;

                // Children have been be completed
                f(pCurrent);
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_BottomUp_Unique_NonReentrant
(
        ASTOpList& roots,
        TFunctionRef<void(Ptr<ASTOp>&)> f,
        TFunctionRef<bool(const ASTOp*)> accept
)
{
    uint32 traverseIndex = s_lastTraverseIndex++;

    TArray< std::pair<Ptr<ASTOp>,int32> > pending;
    for (Ptr<ASTOp>& r:roots)
    {
        if (r && r->m_traverseIndex!=traverseIndex)
        {
            r->m_traverseIndex=traverseIndex;
            pending.Add( std::make_pair<>(r,0) );
        }
    }
    for(std::pair<Ptr<ASTOp>, int32>& p : pending)
    {
        p.first->m_traverseIndex = traverseIndex-1;
    }

    while (pending.Num())
    {
        int32 phase = pending.Last().second;
        Ptr<ASTOp> pCurrent = pending.Last().first;
        pending.Pop();

        // It could have been completed in another branch
        if (pCurrent->m_traverseIndex!=traverseIndex && accept(pCurrent.get()))
        {
            if (phase==0)
            {
                // Process this again...
                pending.Add( std::make_pair<>(pCurrent,1) );

                // ...after the children are processed
                pCurrent->ForEachChild([&]( ASTChild& c )
                {
                    if (c && accept(c.Child.get()) && c.Child->m_traverseIndex!=traverseIndex )
                    {
                        pending.Add( std::make_pair<>(c.Child,0) );
                    }
                });
            }
            else
            {
                pCurrent->m_traverseIndex=traverseIndex;

                // Children have been be completed
                f(pCurrent);
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_BottomUp_Unique
(
        ASTOpList& roots,
        TFunctionRef<void(Ptr<ASTOp>&)> f,
        TFunctionRef<bool(const ASTOp*)> accept
)
{
    TSet<Ptr<ASTOp>> Traversed;
    TArray< std::pair<Ptr<ASTOp>,int32> > Pending;
    for (Ptr<ASTOp>& r:roots)
    {
        if (r)
        {
			std::pair<Ptr<ASTOp>, int32>* It = Pending.FindByPredicate([&](const std::pair<Ptr<ASTOp>, int>& p)
				{
					return r == p.first;
				});
            if ( !It )
            {
                Pending.Add( std::make_pair<>(r,0) );
            }
        }
    }

    while (!Pending.IsEmpty())
    {
        int phase = Pending.Last().second;
        Ptr<ASTOp> pCurrent = Pending.Last().first;
        Pending.Pop();

        // It could have been completed in another branch
        if (accept(pCurrent.get()) && !Traversed.Find(pCurrent))
        {
            if (phase==0)
            {
                // Process this again...
				Pending.Add( std::make_pair<>(pCurrent,1) );

                // ...after the children are processed
                pCurrent->ForEachChild([&]( ASTChild& c )
                {
                    if (c && accept(c.Child.get()) && !Traversed.Find(c.Child) )
                    {
                        Pending.Add( std::make_pair<>(c.Child,0) );
                    }
                });
            }
            else
            {
                Traversed.Add(pCurrent);

                // Children have been be completed
                f(pCurrent);
            }
        }
    }
}


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
void ASTOp::Traverse_BottomUp_Unique
(
        Ptr<ASTOp>& root,
        TFunctionRef<void(Ptr<ASTOp>&)> f,
        TFunctionRef<bool(const ASTOp*)> accept
)
{
    if (root)
    {
        ASTOpList roots;
        roots.Add(root);
        Traverse_BottomUp_Unique(roots,f,accept);
    }
}



//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
int32 ASTOp::GetParentCount() const
{
    int32 result=0;
    ForEachParent( [&](const ASTOp* p)
    {
        if (p!=nullptr) ++result;
    });
    return result;
}


void ASTOp::Replace( const Ptr<ASTOp>& Node, const Ptr<ASTOp>& Other)
{
    if(Other == Node)
    {
        return;
    }

	TArray<ASTOp*, TInlineAllocator<4> > ParentsCopy;
	{
		UE::TScopeLock Lock(Node->ParentsMutex);
		ParentsCopy = Node->Parents;
	}

    for(ASTOp* Parent: ParentsCopy)
    {
        if(Parent)
        {
			Parent->ForEachChild( [=](ASTChild& c)
            {
                if(c.Child==Node)
                {
                    c = Other;
                }
            });
        }
    }
}


FImageDesc ASTOp::GetImageDesc( bool, FGetImageDescContext* ) const
{
    check(false);
    return FImageDesc();
}


FSourceDataDescriptor ASTOp::GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const
{
	ensure(false);
	return {};
}


bool ASTOp::IsImagePlainConstant(FVector4f&) const
{
	// Some image operations don't have this implemented and hit here.
	// \TODO: Optimize for those cases.
	//ensure(false);
    return false;
}


bool ASTOp::IsColorConstant(FVector4f&) const
{
	// Some operations don't have this implemented and hit here.
	// \TODO: It is an opportunity to implement it and help with CO optimization in some cases.
    //ensure(false);
    return false;
}


void ASTOp::GetBlockLayoutSize( uint64, int32*, int32*, FBlockLayoutSizeCache* )
{
    check(false);
}

void ASTOp::GetBlockLayoutSizeCached(uint64 blockId, int32* BlockX, int32* BlockY, FBlockLayoutSizeCache* cache)
{
	FBlockLayoutSizeCache::KeyType key(this,blockId);
	FBlockLayoutSizeCache::ValueType* ValuePtr = cache->Find( key );
    if (ValuePtr)
    {
        *BlockX = ValuePtr->Key;
        *BlockY = ValuePtr->Value;
        return;
    }

    GetBlockLayoutSize( blockId, BlockX, BlockY, cache );

	cache->Add(key, FBlockLayoutSizeCache::ValueType( *BlockX, *BlockY) );
}


void ASTOp::GetLayoutBlockSize( int32*, int32* )
{
    check(false);
}


bool ASTOp::GetNonBlackRect( FImageRect& ) const
{
    return false;
}


ASTOp::EClosedMeshTest ASTOp::IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const
{ 
	// If this is hit, consider implementing it for that subclass.
	return EClosedMeshTest::Unknown; 
}


void ASTOp::LinkRange(FProgram& Program, const FRangeData& Range, FOperation::ADDRESS& RangeSize, uint16& RangeId)
{
	if (Range.rangeSize)
	{
		if (Range.rangeSize->linkedRange < 0)
		{
			check(Program.Ranges.Num() < 255);
			Range.rangeSize->linkedRange = int8(Program.Ranges.Num());
			FRangeDesc RangeData;
			RangeData.Name = Range.rangeName;
			RangeData.UID = Range.rangeUID;

			// Try to see if a parameter directly controls de size of the range. This is used
			// to store hint data for instance generation in tools or randomizers that want to
			// support multilayer, but it is not critical otherwise.
			int32 EstimatedSizeParameter = -1;
			if (Range.rangeSize->GetOpType() == EOpType::SC_PARAMETER ||
				Range.rangeSize->GetOpType() == EOpType::NU_PARAMETER )
			{
				const ASTOpParameter* ParamOp = static_cast<const ASTOpParameter*>(Range.rangeSize.child().get());

				EstimatedSizeParameter = ParamOp->LinkedParameterIndex;
			}

			RangeData.DimensionParameter = EstimatedSizeParameter;

			Program.Ranges.Add(RangeData);
		}

		RangeSize = Range.rangeSize->LinkedAddress;
		RangeId = uint16(Range.rangeSize->linkedRange);
	}
}

} // namespace UE::Mutable::Private
