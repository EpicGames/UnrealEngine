// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpAddExtensionData.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"


namespace UE::Mutable::Private
{


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
ASTOpAddExtensionData::ASTOpAddExtensionData()
	: Instance(this)
	, ExtensionData(this)
{
}


ASTOpAddExtensionData::~ASTOpAddExtensionData()
{
	// Explicit call needed to avoid recursive destruction
	ASTOp::RemoveChildren();
}


EOpType ASTOpAddExtensionData::GetOpType() const
{
	return Type;
}


bool ASTOpAddExtensionData::IsEqual(const ASTOp& OtherUntyped) const
{
	if (OtherUntyped.GetOpType() == GetOpType())
	{
		const ASTOpAddExtensionData* Other = static_cast<const ASTOpAddExtensionData*>(&OtherUntyped);
		return Instance == Other->Instance &&
			ExtensionData == Other->ExtensionData;
	}

	return false;
}

uint32 ASTOpAddExtensionData::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
	
	Result = HashCombineFast(Result, GetTypeHash(Instance));
	Result = HashCombineFast(Result, GetTypeHash(ExtensionData));
	
	return Result;
}

UE::Mutable::Private::Ptr<ASTOp> ASTOpAddExtensionData::Clone(MapChildFuncRef MapChild) const
{
	Ptr<ASTOpAddExtensionData> NewInstance = new ASTOpAddExtensionData();

	NewInstance->Instance = MapChild(Instance.child());
	NewInstance->ExtensionData = MapChild(ExtensionData.child());

	return NewInstance;
}

void ASTOpAddExtensionData::ForEachChild(const TFunctionRef<void(ASTChild&)> F)
{
	F(Instance);
	F(ExtensionData);
}

void ASTOpAddExtensionData::Link(FProgram& Program, FLinkerOptions*)
{
	// Already linked?
	if (LinkedAddress)
	{
		return;
	}

	FOperation::InstanceAddExtensionDataArgs Args;
	FMemory::Memzero(Args);

	if (!Instance || !Instance->LinkedAddress)
	{
		// Can happen if there's no reference skeletal mesh in the first component
		return;
	}

	Args.Instance = Instance->LinkedAddress;

	check(ExtensionData->LinkedAddress);
	Args.ExtensionData = ExtensionData->LinkedAddress;

	++Program.NumOps;
	LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

	AppendCode(Program.ByteCode, GetOpType());
	AppendCode(Program.ByteCode, Args);
}

}
