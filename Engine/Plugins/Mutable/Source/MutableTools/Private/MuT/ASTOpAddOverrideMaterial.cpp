// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpAddOverrideMaterial.h"

#include "MuR/Model.h"
#include "MuT/CodeOptimiser.h"


namespace UE::Mutable::Private
{
	ASTOpAddOverrideMaterial::ASTOpAddOverrideMaterial() :
		Instance(this),
		Material(this)
	{
	}


	ASTOpAddOverrideMaterial::~ASTOpAddOverrideMaterial()
	{
		// Explicit call needed to avoid recursive destruction
		RemoveChildren();
	}


	EOpType ASTOpAddOverrideMaterial::GetOpType() const
	{
		return EOpType::IN_ADDOVERRIDEMATERIAL;
	}


	bool ASTOpAddOverrideMaterial::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpAddOverrideMaterial* Other = static_cast<const ASTOpAddOverrideMaterial*>(&OtherUntyped);
			return Instance == Other->Instance &&
				Material == Other->Material &&
				SlotName == Other->SlotName;
		}

		return false;
	}

	
	uint32 ASTOpAddOverrideMaterial::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Instance));
		Result = HashCombineFast(Result, GetTypeHash(Material));
		Result = HashCombineFast(Result, GetTypeHash(SlotName));
		
		return Result;
	}
	

	Ptr<ASTOp> ASTOpAddOverrideMaterial::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpAddOverrideMaterial> NewInstance = new ASTOpAddOverrideMaterial();

		NewInstance->Instance = MapChild(Instance.child());
		NewInstance->Material = MapChild(Material.child());
		NewInstance->SlotName = SlotName;

		return NewInstance;
	}
	

	void ASTOpAddOverrideMaterial::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Instance);
		Func(Material);
	}
	

	void ASTOpAddOverrideMaterial::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}

		FOperation::InstanceAddOverrideMaterialArgs Args;
		FMemory::Memzero(Args);

		if (Instance)
		{
			Args.Instance = Instance->LinkedAddress;
		}

		if (Material)
		{
			Args.Material = Material->LinkedAddress;
			
			// Find out relevant parameters. \todo: this may be optimised by reusing partial
			// values in a LINK_CONTEXT or similar
			SubtreeRelevantParametersVisitorAST Visitor;
			Visitor.Run(Material.child());
			
			TArray<uint16> Params;
			for (const FString& ParameterName : Visitor.Parameters)
			{
				for (int32 Index = 0; Index < Program.Parameters.Num(); ++Index)
				{
					const FParameterDesc& Param = Program.Parameters[Index];
					if (Param.Name == ParameterName)
					{
						Params.Add(uint16(Index));
						break;
					}
				}
			}

			Params.Sort();

			int32 It = Program.ParameterLists.Find(Params);

			if (It != INDEX_NONE)
			{
				Program.RelevantParameterList.Add(Material->LinkedAddress, It);
			}
			else
			{
				Program.RelevantParameterList.Add(Material->LinkedAddress, Program.ParameterLists.Num());
				Program.ParameterLists.Add(Params);
			}
		}
		
		Args.SlotName = Program.AddConstant(SlotName.ToString());

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}
