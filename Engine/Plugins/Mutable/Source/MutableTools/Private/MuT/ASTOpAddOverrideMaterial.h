// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;
	
	class ASTOpAddOverrideMaterial final : public ASTOp
	{
	public:
		ASTChild Instance;
		
		ASTChild Material;
		
		FName SlotName;

		ASTOpAddOverrideMaterial();
		ASTOpAddOverrideMaterial(const ASTOpAddOverrideMaterial&) = delete;
		virtual ~ASTOpAddOverrideMaterial() override;

		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)> Func) override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void Link(FProgram& Program, FLinkerOptions* Options) override;
	};
}

