// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpAddSkeletalMesh.h"

#include "ASTOpConditional.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuT/CodeOptimiser.h"


namespace UE::Mutable::Private
{
	ASTOpAddSkeletalMesh::ASTOpAddSkeletalMesh() :
		Instance(this),
		SkeletalMesh(this)
	{
	}


	ASTOpAddSkeletalMesh::~ASTOpAddSkeletalMesh()
	{
		RemoveChildren();
	}


	EOpType ASTOpAddSkeletalMesh::GetOpType() const
	{
		return Type;
	}


	bool ASTOpAddSkeletalMesh::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpAddSkeletalMesh* Other = static_cast<const ASTOpAddSkeletalMesh*>(&OtherUntyped);
			return Instance == Other->Instance && SkeletalMesh == Other->SkeletalMesh;
		}

		return false;
	}

		
	uint32 ASTOpAddSkeletalMesh::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Instance));
		Result = HashCombineFast(Result, GetTypeHash(SkeletalMesh));
		
		return Result;
	}

		
	Ptr<ASTOp> ASTOpAddSkeletalMesh::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpAddSkeletalMesh> New = new ASTOpAddSkeletalMesh();

		New->Instance = MapChild(Instance.child());
		New->SkeletalMesh = MapChild(SkeletalMesh.child());

		return New;
	}

	void ASTOpAddSkeletalMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Instance);
		Func(SkeletalMesh);
	}
		

	void ASTOpAddSkeletalMesh::Link(FProgram& Program, FLinkerOptions*)
	{
		if (LinkedAddress)
		{
			return;
		}

		FOperation::InstanceAddArgs Args;
		FMemory::Memzero(Args);

		if (Instance)
		{
			Args.instance = Instance->LinkedAddress;
		}

		if (SkeletalMesh)
		{
			Args.value = SkeletalMesh->LinkedAddress;

			SubtreeResourceRelevantParametersVisitorAST Visitor;
			Visitor.Run(SkeletalMesh.child());

			TArray<uint16> Params;
			for (const FString& ParameterName : Visitor.Parameters)
			{
				for (int32 Index = 0; Index < Program.Parameters.Num(); ++Index)
				{
					const FParameterDesc& ParameterDesc = Program.Parameters[Index];
					if (ParameterDesc.Name == ParameterName)
					{
						Params.Add(static_cast<uint16>(Index));
						break;
					}
				}
			}

			Params.Sort();

			int32 It = Program.ParameterLists.Find(Params);

			if (It != INDEX_NONE)
			{
				Program.RelevantParameterList.Add(SkeletalMesh->LinkedAddress, It);
			}
			else
			{
				Program.RelevantParameterList.Add(SkeletalMesh->LinkedAddress, Program.ParameterLists.Num());
				Program.ParameterLists.Add(Params);
			}
		}
		
		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}
