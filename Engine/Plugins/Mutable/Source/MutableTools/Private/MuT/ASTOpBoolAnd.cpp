// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpBoolAnd.h"
#include "MuT/ASTOpConstantBool.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpBoolAnd::ASTOpBoolAnd()
		: A(this)
		, B(this)
	{
	}


	ASTOpBoolAnd::~ASTOpBoolAnd()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpBoolAnd::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpBoolAnd* Other = static_cast<const ASTOpBoolAnd*>(&OtherUntyped);
			return A == Other->A && B == Other->B;
		}
		return false;
	}


	uint32 ASTOpBoolAnd::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(A));
		Result = HashCombineFast(Result, GetTypeHash(B));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpBoolAnd::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpBoolAnd> New = new ASTOpBoolAnd();
		New->A = MapChild(A.child());
		New->B = MapChild(B.child());
		return New;
	}


	void ASTOpBoolAnd::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(A);
		Func(B);
	}


	void ASTOpBoolAnd::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::BoolBinaryArgs Args;
			FMemory::Memzero(Args);

			if (A) 
			{
				Args.A = A->LinkedAddress;
			}

			if (B)
			{
				Args.B = B->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	ASTOp::FBoolEvalResult ASTOpBoolAnd::EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache* Cache) const
	{
		FEvaluateBoolCache LocalCache;
		if (!Cache)
		{
			Cache = &LocalCache;
		}
		else
		{
			// Is this in the cache?
			FEvaluateBoolCache::iterator it = Cache->find(this);
			if (it != Cache->end())
			{
				return it->second;
			}
		}

		FBoolEvalResult Result = BET_UNKNOWN;

		FBoolEvalResult resultA = BET_UNKNOWN;
		FBoolEvalResult resultB = BET_UNKNOWN;
		for (int32 f = 0; f < Facts.Num(); ++f)
		{
			if (A && resultA == BET_UNKNOWN)
			{
				resultA = A->EvaluateBool(Facts, Cache);

				if (resultA == BET_TRUE && resultB == BET_TRUE)
				{
					Result = BET_TRUE;
					break;
				}
				if (resultA == BET_FALSE || resultB == BET_FALSE)
				{
					Result = BET_FALSE;
					break;
				}
			}
			if (B && resultB == BET_UNKNOWN)
			{
				resultB = B->EvaluateBool(Facts, Cache);

				if (resultA == BET_TRUE && resultB == BET_TRUE)
				{
					Result = BET_TRUE;
					break;
				}
				if (resultA == BET_FALSE || resultB == BET_FALSE)
				{
					Result = BET_FALSE;
					break;
				}
			}
		}

		(*Cache)[this] = Result;

		return Result;
	}


	Ptr<ASTOp> ASTOpBoolAnd::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOp> Result;

		bool bChanged = false;
		if (!A)
		{
			Result = B.child();
			bChanged = true;
		}
		else if (!B)
		{
			Result = A.child();
			bChanged = true;
		}
		else if (A->GetOpType() == EOpType::BO_CONSTANT)
		{
			if (static_cast<const ASTOpConstantBool*>(A.child().get())->bValue)
			{
				Result = B.child();
				bChanged = true;
			}
			else
			{
				Result = A.child();
				bChanged = true;
			}
		}
		else if (B->GetOpType() == EOpType::BO_CONSTANT)
		{
			if (static_cast<const ASTOpConstantBool*>(B.child().get())->bValue)
			{
				Result = A.child();
				bChanged = true;
			}
			else
			{
				Result = B.child();
				bChanged = true;
			}
		}

		// Common cases of repeated branch in children
		else if (A->GetOpType() == EOpType::BO_AND)
		{
			const ASTOpBoolAnd* TypedA = static_cast<const ASTOpBoolAnd*>(A.child().get());
			if (TypedA->A.child() == B.child()
				||
				TypedA->B.child() == B.child())
			{
				Result = A.child();
				bChanged = true;
			}
		}
		else if (B->GetOpType() == EOpType::BO_AND)
		{
			const ASTOpBoolAnd* TypedB = static_cast<const ASTOpBoolAnd*>(B.child().get());
			if (TypedB->A.child() == A.child()
				||
				TypedB->B.child() == A.child())
			{
				Result = B.child();
				bChanged = true;
			}
		}

		else if (A.child() == B.child() || *A.child() == *B.child())
		{
			Result = A.child();
			bChanged = true;
		}

		// if it became null, it means true (neutral AND argument)
		if (bChanged && !Result)
		{
			Result = new ASTOpConstantBool(true);
		}

		return Result;
	}


}
