// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpBoolEqualIntConst.h"

#include "MuT/ASTOpMeshMorph.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpBoolEqualIntConst::ASTOpBoolEqualIntConst()
		: Value(this)
	{
	}


	ASTOpBoolEqualIntConst::~ASTOpBoolEqualIntConst()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpBoolEqualIntConst::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpBoolEqualIntConst* other = static_cast<const ASTOpBoolEqualIntConst*>(&otherUntyped);
			return Value == other->Value && Constant == other->Constant;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpBoolEqualIntConst::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpBoolEqualIntConst> n = new ASTOpBoolEqualIntConst();
		n->Value = mapChild(Value.child());
		n->Constant = Constant;
		return n;
	}


	void ASTOpBoolEqualIntConst::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Value);
	}


	uint32 ASTOpBoolEqualIntConst::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Value));
		Result = HashCombineFast(Result, GetTypeHash(Constant));

		return Result;
	}


	void ASTOpBoolEqualIntConst::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::BoolEqualScalarConstArgs Args;
			FMemory::Memzero(Args);

			Args.Constant = Constant;

			if (Value)
			{
				Args.Value = Value->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	ASTOp::FBoolEvalResult ASTOpBoolEqualIntConst::EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache* Cache) const
	{
		FEvaluateBoolCache LocalCache;
		if (!Cache)
		{
			Cache = &LocalCache;
		}
		else
		{
			// Is this in the cache?
			FEvaluateBoolCache::iterator it = Cache->find(this);
			if (it != Cache->end())
			{
				return it->second;
			}
		}

		FBoolEvalResult Result = BET_UNKNOWN;

		if (!Value)
		{
			return Result;
		}

		bool bIntUnknown = true;
		int32 IntResult = Value->EvaluateInt(Facts, bIntUnknown);
		if (bIntUnknown)
		{
			Result = BET_UNKNOWN;
		}
		else if (IntResult == Constant)
		{
			Result = BET_TRUE;
		}
		else
		{
			Result = BET_FALSE;
		}

		(*Cache)[this] = Result;

		return Result;
	}


}
