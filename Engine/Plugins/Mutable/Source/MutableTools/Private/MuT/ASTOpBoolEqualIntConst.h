// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpBoolEqualIntConst final : public ASTOp
	{
	public:

		ASTChild Value;
		int32 Constant=0;

	public:

		ASTOpBoolEqualIntConst();
		ASTOpBoolEqualIntConst(const ASTOpBoolEqualIntConst&) = delete;
		virtual ~ASTOpBoolEqualIntConst() override;

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::BO_EQUAL_INT_CONST; }
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void Link(FProgram& program, FLinkerOptions*) override;
		virtual FBoolEvalResult EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache*) const override;
	};

}

