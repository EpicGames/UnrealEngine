// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpBoolNot.h"
#include "MuT/ASTOpConstantBool.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpBoolNot::ASTOpBoolNot()
		: A(this)
	{
	}


	ASTOpBoolNot::~ASTOpBoolNot()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpBoolNot::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpBoolNot* Other = static_cast<const ASTOpBoolNot*>(&OtherUntyped);
			return A == Other->A;
		}
		return false;
	}


	uint32 ASTOpBoolNot::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(A));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpBoolNot::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpBoolNot> New = new ASTOpBoolNot();
		New->A = MapChild(A.child());
		return New;
	}


	void ASTOpBoolNot::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(A);
	}


	void ASTOpBoolNot::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::BoolNotArgs Args;
			FMemory::Memzero(Args);

			if (A) 
			{
				Args.A = A->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	ASTOp::FBoolEvalResult ASTOpBoolNot::EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache* Cache) const
	{
		FEvaluateBoolCache LocalCache;
		if (!Cache)
		{
			Cache = &LocalCache;
		}
		else
		{
			// Is this in the cache?
			FEvaluateBoolCache::iterator it = Cache->find(this);
			if (it != Cache->end())
			{
				return it->second;
			}
		}

		FBoolEvalResult Result = BET_UNKNOWN;

		if (A)
		{
			FBoolEvalResult child = A->EvaluateBool(Facts, Cache);
			if (child == BET_TRUE)
			{
				Result = BET_FALSE;
			}
			else if (child == BET_FALSE)
			{
				Result = BET_TRUE;
			}
		}

		(*Cache)[this] = Result;

		return Result;
	}


	Ptr<ASTOp> ASTOpBoolNot::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> Source = A.child();
		if (Source && Source->GetOpType() == EOpType::BO_CONSTANT)
		{
			const ASTOpConstantBool* TypedSource = static_cast<const ASTOpConstantBool*>(Source.get());
			Result = new ASTOpConstantBool(!TypedSource->bValue);
		}

		return Result;
	}


}
