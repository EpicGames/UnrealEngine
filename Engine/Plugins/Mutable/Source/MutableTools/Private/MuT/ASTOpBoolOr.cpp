// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpBoolOr.h"
#include "MuT/ASTOpConstantBool.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpBoolOr::ASTOpBoolOr()
		: A(this)
		, B(this)
	{
	}


	ASTOpBoolOr::~ASTOpBoolOr()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpBoolOr::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpBoolOr* Other = static_cast<const ASTOpBoolOr*>(&OtherUntyped);
			return A == Other->A &&
				B == Other->B;
		}
		return false;
	}


	uint32 ASTOpBoolOr::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(A));
		Result = HashCombineFast(Result, GetTypeHash(B));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpBoolOr::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpBoolOr> New = new ASTOpBoolOr();
		New->A = MapChild(A.child());
		New->B = MapChild(B.child());
		return New;
	}


	void ASTOpBoolOr::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(A);
		Func(B);
	}


	void ASTOpBoolOr::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::BoolBinaryArgs Args;
			FMemory::Memzero(Args);

			if (A) 
			{
				Args.A = A->LinkedAddress;
			}

			if (B) 
			{
				Args.B = B->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	ASTOp::FBoolEvalResult ASTOpBoolOr::EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache* Cache) const
	{
		FEvaluateBoolCache LocalCache;
		if (!Cache)
		{
			Cache = &LocalCache;
		}
		else
		{
			// Is this in the cache?
			FEvaluateBoolCache::iterator it = Cache->find(this);
			if (it != Cache->end())
			{
				return it->second;
			}
		}

		FBoolEvalResult Result = BET_UNKNOWN;

		FBoolEvalResult resultA = BET_UNKNOWN;
		FBoolEvalResult resultB = BET_UNKNOWN;
		for (int32 f = 0; f < Facts.Num(); ++f)
		{
			if (A && resultA == BET_UNKNOWN)
			{
				resultA = A->EvaluateBool(Facts, Cache);

				if (resultA == BET_TRUE || resultB == BET_TRUE)
				{
					Result = BET_TRUE;
					break;
				}
				if (resultA == BET_FALSE && resultB == BET_FALSE)
				{
					Result = BET_FALSE;
					break;
				}
			}
			if (B && resultB == BET_UNKNOWN)
			{
				resultB = B->EvaluateBool(Facts, Cache);

				if (resultA == BET_TRUE || resultB == BET_TRUE)
				{
					Result = BET_TRUE;
					break;
				}
				if (resultA == BET_FALSE && resultB == BET_FALSE)
				{
					Result = BET_FALSE;
					break;
				}
			}
		}

		(*Cache)[this] = Result;

		return Result;
	}


	Ptr<ASTOp> ASTOpBoolOr::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOp> Result;

		bool bChanged = false;
		Ptr<ASTOp> aAt = A.child();
		Ptr<ASTOp> bAt = B.child();
		if (!aAt)
		{
			Result = bAt;
			bChanged = true;
		}
		else if (!bAt)
		{
			Result = aAt;
			bChanged = true;
		}
		else if (aAt->GetOpType() == EOpType::BO_CONSTANT)
		{
			if (static_cast<const ASTOpConstantBool*>(aAt.get())->bValue)
			{
				Result = aAt;
				bChanged = true;
			}
			else
			{
				Result = bAt;
				bChanged = true;
			}
		}
		else if (bAt->GetOpType() == EOpType::BO_CONSTANT)
		{
			if (static_cast<const ASTOpConstantBool*>(bAt.get())->bValue)
			{
				Result = bAt;
				bChanged = true;
			}
			else
			{
				Result = aAt;
				bChanged = true;
			}
		}

		// Common cases of repeated branch in children
		else if (aAt->GetOpType() == EOpType::BO_OR)
		{
			const ASTOpBoolOr* typedA = static_cast<const ASTOpBoolOr*>(aAt.get());
			if (typedA->A.child() == bAt
				||
				typedA->B.child() == bAt)
			{
				Result = aAt;
				bChanged = true;
			}
		}
		else if (bAt->GetOpType() == EOpType::BO_OR)
		{
			const ASTOpBoolOr* typedB = static_cast<const ASTOpBoolOr*>(bAt.get());
			if (typedB->B.child() == aAt
				||
				typedB->B.child() == bAt)
			{
				Result = bAt;
				bChanged = true;
			}
		}

		else if (aAt == bAt || *aAt == *bAt)
		{
			Result = aAt;
			bChanged = true;
		}

		// if it became null, it means false (neutral OR argument)
		if (bChanged && !Result)
		{
			Result = new ASTOpConstantBool(false);
		}

		return Result;
	}


}
