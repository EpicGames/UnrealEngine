// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpBoolOr final : public ASTOp
	{
	public:

		ASTChild A;
		ASTChild B;

	public:

		ASTOpBoolOr();
		ASTOpBoolOr(const ASTOpBoolOr&) = delete;
		~ASTOpBoolOr();

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::BO_OR; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FBoolEvalResult EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache*) const override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;

	};


}

