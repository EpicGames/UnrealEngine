// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpColorArithmetic.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpColorArithmetic::ASTOpColorArithmetic()
		: A(this)
		, B(this)
	{
	}


	ASTOpColorArithmetic::~ASTOpColorArithmetic()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpColorArithmetic::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpColorArithmetic* Other = static_cast<const ASTOpColorArithmetic*>(&OtherUntyped);
			return A == Other->A &&
				B == Other->B &&
				Operation == Other->Operation;
		}
		return false;
	}


	uint32 ASTOpColorArithmetic::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(A));
		Result = HashCombineFast(Result, GetTypeHash(B));
		Result = HashCombineFast(Result, Operation);

		return Result;
	}


	Ptr<ASTOp> ASTOpColorArithmetic::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpColorArithmetic> New = new ASTOpColorArithmetic();
		New->A = MapChild(A.child());
		New->B = MapChild(B.child());
		New->Operation = Operation;
		return New;
	}


	void ASTOpColorArithmetic::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(A);
		Func(B);
	}


	void ASTOpColorArithmetic::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ArithmeticArgs Args;
			FMemory::Memzero(Args);

			if (A) 
			{
				Args.A = A->LinkedAddress;
			}

			if (B)
			{
				Args.B = B->LinkedAddress;
			}

			Args.Operation = Operation;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}
}
