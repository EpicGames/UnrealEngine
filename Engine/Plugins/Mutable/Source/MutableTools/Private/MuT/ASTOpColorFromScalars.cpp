// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpColorFromScalars.h"

#include "Containers/Map.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpSwitch.h"

namespace UE::Mutable::Private
{

	ASTOpColorFromScalars::ASTOpColorFromScalars()
		: V{ ASTChild(this),ASTChild(this),ASTChild(this),ASTChild(this) }
	{
	}


	ASTOpColorFromScalars::~ASTOpColorFromScalars()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpColorFromScalars::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpColorFromScalars* Other = static_cast<const ASTOpColorFromScalars*>(&otherUntyped);
			for (int32 i = 0; i<MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				if (!(V[i] == Other->V[i]))
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}


	uint32 ASTOpColorFromScalars::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(V[0]));
		Result = HashCombineFast(Result, GetTypeHash(V[1]));
		Result = HashCombineFast(Result, GetTypeHash(V[2]));
		Result = HashCombineFast(Result, GetTypeHash(V[3]));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpColorFromScalars::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpColorFromScalars> n = new ASTOpColorFromScalars();
		for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
		{
			n->V[i] = mapChild(V[i].child());
		}
		return n;
	}


	void ASTOpColorFromScalars::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
		{
			f(V[i]);
		}
	}


	void ASTOpColorFromScalars::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ColorFromScalarsArgs Args;
			FMemory::Memzero(Args);

			for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				if (V[i]) 
				{
					Args.V[i] = V[i]->LinkedAddress;
				}
			}			

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

}
