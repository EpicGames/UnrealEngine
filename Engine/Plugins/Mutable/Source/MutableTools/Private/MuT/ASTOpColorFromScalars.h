// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpColorFromScalars final : public ASTOp
	{
	public:

		ASTChild V[MUTABLE_OP_MAX_SWIZZLE_CHANNELS];

	public:

		ASTOpColorFromScalars();
		ASTOpColorFromScalars(const ASTOpColorFromScalars&) = delete;
		~ASTOpColorFromScalars();

		virtual EOpType GetOpType() const override { return EOpType::CO_FROMSCALARS; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
	};

}

