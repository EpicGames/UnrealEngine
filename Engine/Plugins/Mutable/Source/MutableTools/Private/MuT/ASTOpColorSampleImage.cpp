// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpColorSampleImage.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpColorSampleImage::ASTOpColorSampleImage()
		: Image(this)
		, X(this)
		, Y(this)
	{
	}


	ASTOpColorSampleImage::~ASTOpColorSampleImage()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpColorSampleImage::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpColorSampleImage* Other = static_cast<const ASTOpColorSampleImage*>(&OtherUntyped);
			return Image == Other->Image &&
				X == Other->X &&
				Y == Other->Y &&
				Filter == Other->Filter;
		}
		return false;
	}


	uint32 ASTOpColorSampleImage::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Image));
		Result = HashCombineFast(Result, GetTypeHash(X));
		Result = HashCombineFast(Result, GetTypeHash(Y));
		Result = HashCombineFast(Result, GetTypeHash(Filter));

		return Result;
	}


	Ptr<ASTOp> ASTOpColorSampleImage::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpColorSampleImage> New = new ASTOpColorSampleImage();
		New->Image = MapChild(Image.child());
		New->X = MapChild(X.child());
		New->Y = MapChild(Y.child());
		New->Filter = Filter;
		return New;
	}


	void ASTOpColorSampleImage::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Image);
		Func(X);
		Func(Y);
	}


	void ASTOpColorSampleImage::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ColorSampleImageArgs Args;
			FMemory::Memzero(Args);

			if (Image) 
			{
				Args.Image = Image->LinkedAddress;
			}

			if (X) 
			{
				Args.X = X->LinkedAddress;
			}

			if (Y) 
			{
				Args.Y = Y->LinkedAddress;
			}

			Args.Filter = Filter;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}
}
