// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpColorSwizzle.h"

#include "Containers/Map.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpColorFromScalars.h"
#include "MuT/ASTOpSwitch.h"

namespace UE::Mutable::Private
{

	ASTOpColorSwizzle::ASTOpColorSwizzle()
		: Sources{ ASTChild(this),ASTChild(this),ASTChild(this),ASTChild(this) }
	{
	}


	ASTOpColorSwizzle::~ASTOpColorSwizzle()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpColorSwizzle::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpColorSwizzle* Other = static_cast<const ASTOpColorSwizzle*>(&otherUntyped);
			for (int32 i = 0; i<MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				if (!(Sources[i] == Other->Sources[i] && SourceChannels[i] == Other->SourceChannels[i]))
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}


	uint32 ASTOpColorSwizzle::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Sources[0]));
		Result = HashCombineFast(Result, GetTypeHash(Sources[1]));
		Result = HashCombineFast(Result, GetTypeHash(Sources[2]));
		Result = HashCombineFast(Result, GetTypeHash(Sources[3]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[0]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[1]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[2]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[3]));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpColorSwizzle::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpColorSwizzle> n = new ASTOpColorSwizzle();
		for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
		{
			n->Sources[i] = mapChild(Sources[i].child());
			n->SourceChannels[i] = SourceChannels[i];
		}
		return n;
	}


	void ASTOpColorSwizzle::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
		{
			f(Sources[i]);
		}
	}


	void ASTOpColorSwizzle::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ColorSwizzleArgs Args;
			FMemory::Memzero(Args);

			for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				if (Sources[i]) 
				{
					Args.sources[i] = Sources[i]->LinkedAddress;
				}

				Args.sourceChannels[i] = SourceChannels[i];
			}			

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());	

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpColorSwizzle::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOp> at;

		// Optimizations that can be applied per-channel
		{
			Ptr<ASTOpColorSwizzle> NewSwizzle;

			for (int32 ChannelIndex = 0; ChannelIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++ChannelIndex)
			{
				Ptr<ASTOp> Candidate = Sources[ChannelIndex].child();
				if (!Candidate)
				{
					continue;
				}

				switch (Candidate->GetOpType())
				{

				// Swizzle + swizzle = swizzle
				case EOpType::CO_SWIZZLE:
				{
					if (!NewSwizzle)
					{
						NewSwizzle = UE::Mutable::Private::Clone<ASTOpColorSwizzle>(this);
					}
					const ASTOpColorSwizzle* TypedCandidate = static_cast<const ASTOpColorSwizzle*>(Candidate.get());
					int32 CandidateChannel = SourceChannels[ChannelIndex];

					NewSwizzle->Sources[ChannelIndex] = TypedCandidate->Sources[CandidateChannel].child();
					NewSwizzle->SourceChannels[ChannelIndex] = TypedCandidate->SourceChannels[CandidateChannel];
					break;
				}

				default:
					break;

				}
			}

			at = NewSwizzle;
		}

		// Not optimized yet?
		if (!at)
		{
			// Optimizations that depend on all channels.
			bool bAllChannelsSameType = true;
			EOpType AllChannelsType = EOpType::NONE;
			for (int32 ChannelIndex = 0; ChannelIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++ChannelIndex)
			{
				Ptr<ASTOp> Candidate = Sources[ChannelIndex].child();
				if (!Candidate)
				{
					continue;
				}

				if (ChannelIndex == 0)
				{
					AllChannelsType = Candidate->GetOpType();
				}
				else if (Candidate->GetOpType() != AllChannelsType)
				{
					bAllChannelsSameType = false;
					break;
				}
			}

			if (bAllChannelsSameType)
			{
				switch (AllChannelsType)
				{
				case EOpType::CO_FROMSCALARS:
				{
					// We can remove the swizzle and replace it with a new FromScalars actually swizzling the inputs.

					Ptr<ASTOpColorFromScalars> NewAt = new ASTOpColorFromScalars();

					for (int32 ChannelIndex = 0; ChannelIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++ChannelIndex)
					{
						const ASTOp* SelectedSourceGeneric = Sources[ChannelIndex].child().get();
						if (SelectedSourceGeneric)
						{
							const ASTOpColorFromScalars* SelectedSource = static_cast<const ASTOpColorFromScalars*>(SelectedSourceGeneric);
							int32 SelectedChannel = SourceChannels[ChannelIndex];
							Ptr<ASTOp> SelectedFloatInput = SelectedSource->V[SelectedChannel].child();
							NewAt->V[ChannelIndex] = SelectedFloatInput;
						}
					}

					at = NewAt;
					break;
				}

				default: break;
				}
			}
		}

		return at;
	}

}
