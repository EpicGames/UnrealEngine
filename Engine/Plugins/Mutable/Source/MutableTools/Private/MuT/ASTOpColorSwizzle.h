// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpColorSwizzle final : public ASTOp
	{
	public:

		ASTChild Sources[MUTABLE_OP_MAX_SWIZZLE_CHANNELS];

		uint8 SourceChannels[MUTABLE_OP_MAX_SWIZZLE_CHANNELS] = { 0,0,0,0 };

	public:

		ASTOpColorSwizzle();
		ASTOpColorSwizzle(const ASTOpColorSwizzle&) = delete;
		~ASTOpColorSwizzle();

		virtual EOpType GetOpType() const override { return EOpType::CO_SWIZZLE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;
	};

}

