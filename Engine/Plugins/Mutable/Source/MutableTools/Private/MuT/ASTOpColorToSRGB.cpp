// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpColorToSRGB.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpColorToSRGB::ASTOpColorToSRGB()
		: Color(this)
	{
	}


	ASTOpColorToSRGB::~ASTOpColorToSRGB()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpColorToSRGB::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpColorToSRGB* Other = static_cast<const ASTOpColorToSRGB*>(&OtherUntyped);
			return Color == Other->Color;
		}
		return false;
	}


	uint32 ASTOpColorToSRGB::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Color));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpColorToSRGB::Clone(MapChildFuncRef MapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpColorToSRGB> New = new ASTOpColorToSRGB();
		New->Color = MapChild(Color.child());
		return New;
	}


	void ASTOpColorToSRGB::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Color);
	}


	void ASTOpColorToSRGB::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ColorArgs Args;
			FMemory::Memzero(Args);

			if (Color)
			{
				Args.Color = Color->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());	

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

}
