// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpColorToSRGB final : public ASTOp
	{
	public:

		ASTChild Color;

	public:

		ASTOpColorToSRGB();
		ASTOpColorToSRGB(const ASTOpColorToSRGB&) = delete;
		~ASTOpColorToSRGB();

		virtual EOpType GetOpType() const override { return EOpType::CO_LINEARTOSRGB; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& Program, FLinkerOptions* Options) override;
	};

}

