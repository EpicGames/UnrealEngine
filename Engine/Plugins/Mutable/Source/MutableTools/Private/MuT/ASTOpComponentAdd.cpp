// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpComponentAdd.h"

#include "MuT/ASTOpConditional.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/RefCounted.h"


namespace UE::Mutable::Private
{
	ASTOpComponentAdd::ASTOpComponentAdd()
		: Instance(this)
		, Value(this)
	{
	}


	ASTOpComponentAdd::~ASTOpComponentAdd()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	
	EOpType ASTOpComponentAdd::GetOpType() const
	{
		return Type;
	}


	bool ASTOpComponentAdd::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpComponentAdd* other = static_cast<const ASTOpComponentAdd*>(&OtherUntyped);
			return GetOpType() == other->GetOpType() &&
				Instance == other->Instance &&
				Value == other->Value &&
				ExternalId == other->ExternalId;
		}
		
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpComponentAdd::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpComponentAdd> New = new ASTOpComponentAdd();
		New->Instance = MapChild(Instance.child());
		New->Value = MapChild(Value.child());
		New->ExternalId = ExternalId;
		
		return New;
	}


	uint32 ASTOpComponentAdd::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Instance));
		Result = HashCombineFast(Result, GetTypeHash(Value));
		
		return Result;
	}
	

	void ASTOpComponentAdd::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Instance);
		Func(Value);
	}


	void ASTOpComponentAdd::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::InstanceAddComponentArgs Args;
			FMemory::Memzero(Args);
			Args.ExternalId = ExternalId;

			if (Instance)
			{
				Args.Instance = Instance->LinkedAddress;
			}
				
			if (Value)
			{
				Args.Value = Value->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}
}
