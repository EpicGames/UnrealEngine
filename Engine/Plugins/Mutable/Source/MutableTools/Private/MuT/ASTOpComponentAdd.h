// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;


	//---------------------------------------------------------------------------------------------
	//! Operations to add elements to an instance
	//---------------------------------------------------------------------------------------------
	class ASTOpComponentAdd final : public ASTOp
	{
	public:
		static const EOpType Type = EOpType::IN_ADDCOMPONENT;
		
		ASTChild Instance;
		ASTChild Value;
		FComponentId ExternalId = INDEX_NONE;

		ASTOpComponentAdd();
		ASTOpComponentAdd(const ASTOpComponentAdd&) = delete;
		~ASTOpComponentAdd();

		virtual EOpType GetOpType() const override;

		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual uint32 Hash() const override;
		virtual void Link(FProgram& Program, FLinkerOptions* Options) override;
	};
}

