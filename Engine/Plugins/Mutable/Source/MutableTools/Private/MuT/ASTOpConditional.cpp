// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConditional.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConstantBool.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	ASTOpConditional::ASTOpConditional()
		: condition(this)
		, yes(this)
		, no(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpConditional::~ASTOpConditional()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpConditional::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConditional* other = static_cast<const ASTOpConditional*>(&otherUntyped);
			return type == other->type && condition == other->condition &&
				yes == other->yes && no == other->no;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpConditional::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpConditional> n = new ASTOpConditional();
		n->type = type;
		n->condition = mapChild(condition.child());
		n->yes = mapChild(yes.child());
		n->no = mapChild(no.child());
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpConditional::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(condition));
		Result = HashCombineFast(Result, GetTypeHash(yes));
		Result = HashCombineFast(Result, GetTypeHash(no));
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConditional::Assert()
	{
		switch (type)
		{
		case EOpType::NU_CONDITIONAL:
		case EOpType::SC_CONDITIONAL:
		case EOpType::CO_CONDITIONAL:
		case EOpType::IM_CONDITIONAL:
		case EOpType::ME_CONDITIONAL:
		case EOpType::LA_CONDITIONAL:
		case EOpType::IN_CONDITIONAL:
		case EOpType::ED_CONDITIONAL:
		case EOpType::MI_CONDITIONAL:
		case EOpType::SK_CONDITIONAL:
		case EOpType::LD_CONDITIONAL:
		case EOpType::IS_CONDITIONAL:
			break;
		default:
			// Unexpected type
			check(false);
			break;
		}

		ASTOp::Assert();
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConditional::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(condition);
		f(yes);
		f(no);
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConditional::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ConditionalArgs Args;
			FMemory::Memzero(Args);

			if (condition) 
			{
				Args.condition = condition->LinkedAddress;
			}

			if (yes) 
			{
				Args.yes = yes->LinkedAddress;
			}

			if (no) 
			{
				Args.no = no->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());	

			AppendCode(Program.ByteCode, type);
			AppendCode(Program.ByteCode, Args);

		}
	}


	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpConditional::GetImageDesc(bool returnBestOption, class FGetImageDescContext* context) const
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (type == EOpType::IM_CONDITIONAL || type == EOpType::MI_CONDITIONAL)
		{
			// In a conditional we cannot guarantee the size and format.
			// We check both options, and if they are the same we return that.
			// Otherwise, we return an empty descriptor that will force re-formatting of the image.
			// The code optimiser will take care then of moving the format operations down to each
			// branch and remove the unnecessary ones.
			FImageDesc noDesc;
			FImageDesc yesDesc;

			if (no.child())
			{
				noDesc = no->GetImageDesc(returnBestOption, context);
			}
			if (yes.child())
			{
				yesDesc = yes->GetImageDesc(returnBestOption, context);
			}

			if (yesDesc == noDesc || returnBestOption)
			{
				res = yesDesc;
			}
			else
			{
				res.m_format = (yesDesc.m_format == noDesc.m_format) ? yesDesc.m_format : EImageFormat::None;
				res.FormatIfAlpha = (yesDesc.FormatIfAlpha == noDesc.FormatIfAlpha) ? yesDesc.FormatIfAlpha : EImageFormat::None;
				res.m_lods = (yesDesc.m_lods == noDesc.m_lods) ? yesDesc.m_lods : 0;
				res.m_size = (yesDesc.m_size == noDesc.m_size) ? yesDesc.m_size : FImageSize(0, 0);
			}
		}
		else
		{
			checkf(false, TEXT("Instruction not supported"));
		}


		// Cache the result
		if (context)
		{
			context->m_results.Add(this, res);
		}

		return res;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConditional::GetLayoutBlockSize(int32* pBlockX, int32* pBlockY)
	{
		if (type == EOpType::IM_CONDITIONAL)
		{
			yes->GetLayoutBlockSize(pBlockX, pBlockY);

			if (!*pBlockX)
			{
				no->GetLayoutBlockSize(pBlockX, pBlockY);
			}
		}
		else
		{
			checkf(false, TEXT("Instruction not supported"));
		}
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConditional::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache)
	{
		if (type == EOpType::LA_CONDITIONAL)
		{
			yes->GetBlockLayoutSizeCached(BlockId, pBlockX, pBlockY, cache);

			if (*pBlockX == 0)
			{
				no->GetBlockLayoutSizeCached(BlockId, pBlockX, pBlockY, cache);
			}
		}
		else
		{
			checkf(false, TEXT("Instruction not supported"));
		}
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpConditional::GetNonBlackRect(FImageRect& maskUsage) const
	{
		if (type == EOpType::IM_CONDITIONAL)
		{
			FImageRect local;
			bool localValid = false;
			if (yes)
			{
				localValid = yes->GetNonBlackRect(local);
				if (!localValid)
				{
					return false;
				}
			}

			if (no)
			{
				FImageRect noRect;
				bool validNo = no->GetNonBlackRect(noRect);
				if (validNo)
				{
					if (localValid)
					{
						local.Bound(noRect);
					}
					else
					{
						local = noRect;
						localValid = true;
					}
				}
				else
				{
					return false;
				}
			}

			if (localValid)
			{
				maskUsage = local;
				return true;
			}
		}

		return false;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpConditional::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		if (!condition)
		{
			// If there is no expression, we'll assume true.
			return yes.child();
		}

		// If the branches are the same, remove the instruction
		if (yes.child() == no.child())
		{
			return yes.child();
		}

		// Constant condition?
		if (condition->GetOpType() == EOpType::BO_CONSTANT)
		{
			const ASTOpConstantBool* TypedCondition = static_cast<const ASTOpConstantBool*>(condition.child().get());
			if (TypedCondition->bValue)
			{
				return yes.child();
			}

			return no.child();
		}

		else
		{
			// If the yes branch is a conditional with the same condition
			if (yes && yes->GetOpType() == type)
			{
				const ASTOpConditional* typedYes = static_cast<const ASTOpConditional*>(yes.child().get());
				if (condition.child() == typedYes->condition.child()
					||
					*condition.child() == *typedYes->condition.child())
				{
					auto op = UE::Mutable::Private::Clone<ASTOpConditional>(this);
					op->yes = typedYes->yes.child();
					return op;
				}
			}

			// If the no branch is a conditional with the same condition
			else if (no && no->GetOpType() == type)
			{
				const ASTOpConditional* typedNo = static_cast<const ASTOpConditional*>(no.child().get());
				if (condition.child() == typedNo->condition.child()
					||
					*condition.child() == *typedNo->condition.child())
				{
					auto op = UE::Mutable::Private::Clone<ASTOpConditional>(this);
					op->no = typedNo->no.child();
					return op;
				}
			}
		}

		return nullptr;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpConditional::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> pRes = new ImageSizeExpression;
		pRes->type = ImageSizeExpression::ISET_CONDITIONAL;
		pRes->yes = yes->GetImageSizeExpression();
		pRes->no = no->GetImageSizeExpression();

		if (*pRes->yes == *pRes->no)
		{
			pRes = pRes->yes;
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpConditional::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found = Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		if (yes)
		{
			FSourceDataDescriptor SourceDesc = yes->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		if (no)
		{
			FSourceDataDescriptor SourceDesc = no->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		Context->Cache.Add(this, Result);

		return Result;
	}

}
