// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;

	//---------------------------------------------------------------------------------------------
	//! Conditional operation.
	//---------------------------------------------------------------------------------------------
	class ASTOpConditional final : public ASTOp
	{
	public:

		//! Type of switch
		EOpType type = EOpType::NONE;

		//! Boolean expression
		ASTChild condition;

		//! Branches
		ASTChild yes;
		ASTChild no;

	public:

		ASTOpConditional();
		ASTOpConditional(const ASTOpConditional&) = delete;
		~ASTOpConditional() override;

		virtual EOpType GetOpType() const override { return type; }
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual uint32 Hash() const override;
		virtual void Assert() override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)> f) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, class FGetImageDescContext* context) const override;
		virtual void GetLayoutBlockSize(int32* pBlockX, int32* pBlockY) override;
		virtual void GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache) override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;
		virtual bool GetNonBlackRect(FImageRect& maskUsage) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual bool IsConditional() const override { return true; }
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};


}

