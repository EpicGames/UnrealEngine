// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantBool.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpConstantBool::ASTOpConstantBool(bool InValue)
	{
		bValue = InValue;
	}


	void ASTOpConstantBool::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	bool ASTOpConstantBool::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConstantBool* other = static_cast<const ASTOpConstantBool*>(&OtherUntyped);
			return bValue == other->bValue;
		}
		return false;
	}


	uint32 ASTOpConstantBool::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(bValue));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantBool::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpConstantBool> New = new ASTOpConstantBool();
		New->bValue = bValue;
		return New;
	}


	void ASTOpConstantBool::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::BoolConstantArgs Args;
			FMemory::Memzero(Args);
			Args.bValue = bValue;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	ASTOp::FBoolEvalResult ASTOpConstantBool::EvaluateBool(ASTOpList&, FEvaluateBoolCache*) const
	{
		return bValue ? BET_TRUE : BET_FALSE;
	}


	bool ASTOpConstantBool::IsConstantBool(const Ptr<ASTOp>& Op, bool& bOutValue)
	{
		if (Op && Op->GetOpType() == EOpType::BO_CONSTANT)
		{
			const ASTOpConstantBool* TypedOp = static_cast<const ASTOpConstantBool*>(Op.get());
			bOutValue = TypedOp->bValue;

			return true;
		}

		return false;
	}

}
