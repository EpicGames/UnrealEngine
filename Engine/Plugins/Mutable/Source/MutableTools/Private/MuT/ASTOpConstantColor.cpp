// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantColor.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	void ASTOpConstantColor::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	bool ASTOpConstantColor::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConstantColor* other = static_cast<const ASTOpConstantColor*>(&OtherUntyped);

			// HACK: We encode an invalid value (Nan) for table option "None".
			// This nan comparison is needed because some compilers will return that 0.0f is equal to Nan...
			if (FMath::IsNaN(Value[0]) || FMath::IsNaN(other->Value[0]))
			{
				return FMath::IsNaN(Value[0]) == FMath::IsNaN(other->Value[0]);
			}

			return Value == other->Value;
		}
		return false;
	}


	uint32 ASTOpConstantColor::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Value));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantColor::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpConstantColor> New = new ASTOpConstantColor();
		New->Value = Value;
		return New;
	}


	void ASTOpConstantColor::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::ColorConstantArgs Args;
			FMemory::Memzero(Args);
			Args.Value = Value;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	bool ASTOpConstantColor::IsColorConstant(FVector4f& OutColor) const
	{
		OutColor = Value;
		return true;
	}

}
