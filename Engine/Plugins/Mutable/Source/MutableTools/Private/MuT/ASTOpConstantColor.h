// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpConstantColor final : public ASTOp
	{
	public:

		FVector4f Value = FVector4f(0.0f,0.0f,0.0f,0.0f);

	public:

		virtual EOpType GetOpType() const override { return EOpType::CO_CONSTANT; }
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual bool IsColorConstant(FVector4f& OutColor) const override;
	};

}

