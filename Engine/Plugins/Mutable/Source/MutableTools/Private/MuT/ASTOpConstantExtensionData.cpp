// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantExtensionData.h"

#include "HAL/UnrealMemory.h"
#include "MuR/Model.h"


namespace UE::Mutable::Private
{


//-------------------------------------------------------------------------------------------------
bool ASTOpConstantExtensionData::IsEqual(const ASTOp& OtherUntyped) const
{
	if (OtherUntyped.GetOpType() == GetOpType())
	{
		const ASTOpConstantExtensionData* Other = static_cast<const ASTOpConstantExtensionData*>(&OtherUntyped);
		return Other->ExtensionDataId == ExtensionDataId;
	}

	return false;
}


//-------------------------------------------------------------------------------------------------
UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantExtensionData::Clone(MapChildFuncRef MapChild) const
{
	Ptr<ASTOpConstantExtensionData> Result = new ASTOpConstantExtensionData();
	Result->ExtensionDataId = ExtensionDataId;

	return Result;
}


//-------------------------------------------------------------------------------------------------
uint32 ASTOpConstantExtensionData::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
		
	Result = HashCombineFast(Result, GetTypeHash(ExtensionDataId));
		
	return Result;
}


//-------------------------------------------------------------------------------------------------
void ASTOpConstantExtensionData::Link(FProgram& Program, FLinkerOptions* Options)
{
	if (!LinkedAddress)
	{
		FOperation::ExternalDataConstantArgs Args;
		FMemory::Memzero(Args);

		Args.ExternalObjectId = ExtensionDataId;

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}

}
