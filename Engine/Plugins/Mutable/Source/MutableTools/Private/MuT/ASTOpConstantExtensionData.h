// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{

	struct FProgram;

	/** A constant ExtensionData. */
	class ASTOpConstantExtensionData final : public ASTOp
	{
	public:
		PASSTHROUGH_ID ExtensionDataId = PASSTHROUGH_ID_INVALID;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::ED_CONSTANT; }
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override {}
		bool IsEqual(const ASTOp& OtherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		uint32 Hash() const override;
		void Link(FProgram& Program, FLinkerOptions*) override;
	};


}

