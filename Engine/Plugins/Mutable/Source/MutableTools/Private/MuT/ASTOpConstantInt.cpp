// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantInt.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpConstantInt::ASTOpConstantInt(int32 InValue)
	{
		Value = InValue;
	}


	void ASTOpConstantInt::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	bool ASTOpConstantInt::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConstantInt* Other = static_cast<const ASTOpConstantInt*>(&OtherUntyped);
			return Value == Other->Value;
		}
		return false;
	}


	uint32 ASTOpConstantInt::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Value));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantInt::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpConstantInt> New = new ASTOpConstantInt();
		New->Value = Value;
		return New;
	}


	void ASTOpConstantInt::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::IntConstantArgs Args;
			FMemory::Memzero(Args);
			Args.Value = Value;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	int32 ASTOpConstantInt::EvaluateInt(ASTOpList&, bool& bOutUnknown) const
	{
		bOutUnknown =  false;
		return Value;
	}

}
