// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantMatrix.h"

UE::Mutable::Private::ASTOpConstantMatrix::ASTOpConstantMatrix(FMatrix44f InitValue) : value(InitValue)
{
}

uint32 UE::Mutable::Private::ASTOpConstantMatrix::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());

	Result = HashCombineFast(Result, value.ComputeHash());
	
	return Result;
}

bool UE::Mutable::Private::ASTOpConstantMatrix::IsEqual(const ASTOp& otherUntyped) const
{
	if (otherUntyped.GetOpType() == GetOpType())
	{
		const ASTOpConstantMatrix* other = static_cast<const ASTOpConstantMatrix*>(&otherUntyped);
		return value == other->value;
	}
	
	return false;
}

UE::Mutable::Private::Ptr<UE::Mutable::Private::ASTOp> UE::Mutable::Private::ASTOpConstantMatrix::Clone(MapChildFuncRef mapChild) const
{
	Ptr<ASTOpConstantMatrix> n = new ASTOpConstantMatrix();
	n->value = value;
	return n;
}

void UE::Mutable::Private::ASTOpConstantMatrix::Link(FProgram& Program, FLinkerOptions* Options)
{
	if (!LinkedAddress)
	{
		FOperation::MatrixConstantArgs Args;
		FMemory::Memzero(Args);
		Args.value = Program.AddConstant(value);

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}
