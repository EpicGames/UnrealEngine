// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantProjector.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	void ASTOpConstantProjector::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	uint32 ASTOpConstantProjector::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(value.position));
		Result = HashCombineFast(Result, GetTypeHash(value.direction));
		
		return Result;
	}


	bool ASTOpConstantProjector::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConstantProjector* other = static_cast<const ASTOpConstantProjector*>(&otherUntyped);
			return value == other->value;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantProjector::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpConstantProjector> n = new ASTOpConstantProjector();
		n->value = value;
		return n;
	}


	void ASTOpConstantProjector::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::ResourceConstantArgs Args;
			FMemory::Memzero(Args);
			Args.value = Program.AddConstant(value);

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

}
