// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantResource.h"

#include "MuT/CompilerPrivate.h"
#include "MuR/Layout.h"
#include "MuR/Mesh.h"
#include "MuR/Model.h"
#include "MuR/ImagePrivate.h"
#include "MuR/PhysicsBody.h"
#include "MuR/Serialisation.h"
#include "MuR/Skeleton.h"

#include "Containers/Array.h"
#include "Hash/CityHash.h"
#include "Misc/AssertionMacros.h"

#include <inttypes.h> // Required for 64-bit printf macros

namespace UE::Mutable::Private
{

	//-------------------------------------------------------------------------------------------------
	void ASTOpConstantResource::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		/** Image Parameters are evaluated at runtime. */
		for (TPair<FParameterKey, ASTChild >& Element : ImageOperations)
		{
			f(Element.Value);
		}
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpConstantResource::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpConstantResource* Other = static_cast<const ASTOpConstantResource*>(&OtherUntyped);
			return Type == Other->Type && ValueHash == Other->ValueHash &&
				Value == Other->Value &&
				SourceDataDescriptor == Other->SourceDataDescriptor	&&
				ImageOperations == Other->ImageOperations;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantResource::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpConstantResource> n = new ASTOpConstantResource();
		n->Type = Type;
		n->Value = Value;
		n->ValueHash = ValueHash;
		n->SourceDataDescriptor = SourceDataDescriptor;

		for (const TPair<FParameterKey, ASTChild>& ImageOperation : ImageOperations)
		{
			n->ImageOperations.Add(ImageOperation.Key, ASTChild(n, MapChild(ImageOperation.Value.child())));
		}
		
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpConstantResource::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(ValueHash));

		return Result;
	}


	namespace
	{
 		/** Adds a constant mesh data to a Program and returns its constant index. */
		int32 AddConstantMesh(FProgram& Program, const TManagedPtr<const FMesh>& MeshData, FLinkerOptions& Options)
		{
			auto AddMeshToProgram = [&Options, &Program](const TManagedPtr<FMesh>& Mesh)
			{
				// Use a map-based deduplication
				int32 MeshIndex = -1;
				TManagedPtr<const FMesh> MeshKey = Mesh;
				const int32* MeshIndexPtr = Options.MeshConstantMap.Find(MeshKey);
				if (!MeshIndexPtr)
				{
					MeshIndex = Program.ConstantMeshesPermanent.Add(Mesh);
					Options.MeshConstantMap.Add(Mesh, MeshIndex);
				}
				else
				{
					MeshIndex = *MeshIndexPtr;
				}

				check(MeshIndex >= 0)
				return Program.ConstantMeshContentIndices.Add(FConstantResourceIndex{(uint32)MeshIndex, 0});
			};
			
			// Split generated mesh data in 4 parts. Geometry and Pose and Physics and Metadata.
			// Indices for a given rom are sorted by content flag value.
			static_assert(EMeshContentFlags::GeometryData < EMeshContentFlags::PoseData);
			static_assert(EMeshContentFlags::PoseData     < EMeshContentFlags::PhysicsData);
			static_assert(EMeshContentFlags::PhysicsData  < EMeshContentFlags::MetaData);

			int32 FirstIndex = Program.ConstantMeshContentIndices.Num();
			EMeshContentFlags MeshContentFlags = EMeshContentFlags::None;	
			// GeometryMesh
			{
				const EMeshCopyFlags GeometryDataCopyFlags = 
						EMeshCopyFlags::WithSurfaces      | 
						EMeshCopyFlags::WithVertexBuffers |
						EMeshCopyFlags::WithIndexBuffers  |
						EMeshCopyFlags::WithLayouts       |
						EMeshCopyFlags::WithClothData     |
						EMeshCopyFlags::WithMorphData	  |
						EMeshCopyFlags::WithSkinWeightProfileData;

				TManagedPtr<FMesh> MeshGeometryData = MeshData->Clone(GeometryDataCopyFlags);

				// Copy geometry related additional buffers.
				for (const TPair<EMeshBufferType, FMeshBufferSet>& AdditionalBuffer : MeshData->AdditionalBuffers)
				{
					const bool bIsGeometryBufferType = 
							AdditionalBuffer.Key == EMeshBufferType::MeshLaplacianData    ||
							AdditionalBuffer.Key == EMeshBufferType::MeshLaplacianOffsets ||
							AdditionalBuffer.Key == EMeshBufferType::UniqueVertexMap;

					if (bIsGeometryBufferType)
					{
						MeshGeometryData->AdditionalBuffers.Emplace(AdditionalBuffer);
					}
				}

				MeshGeometryData->MeshIDPrefix = 0;
				
				if (MeshGeometryData->ClothSections.Num())
				{
					MeshGeometryData->ClothSections[0].ClothingAsset.Reset();
				}
				
				AddMeshToProgram(MeshGeometryData);	
				EnumAddFlags(MeshContentFlags, EMeshContentFlags::GeometryData);
			}

			// Pose Mesh
			{
				const EMeshCopyFlags PoseDataCopyFlags =
					EMeshCopyFlags::WithPoses |
					EMeshCopyFlags::WithBoneMap;

				TManagedPtr<FMesh> MeshPoseData = MeshData->Clone(PoseDataCopyFlags);

				if (MeshData->Skeleton)
				{
					MUTABLE_CPUPROFILER_SCOPE(ConvertBoneNamesToIds_Pose);

					const TArray<FName>& BoneNames = MeshData->Skeleton->BoneNames;

					for (FBoneIdOrIndex& Bone : MeshPoseData->BoneMap)
					{
						Bone.Id = Program.AddConstant(BoneNames[Bone.Index]);
					}

					for (FMesh::FBonePose& BonePose : MeshPoseData->BonePoses)
					{
						BonePose.BoneId.Id = Program.AddConstant(BoneNames[BonePose.BoneId.Index]);
					}
				}

				for (const TPair<EMeshBufferType, FMeshBufferSet>& AdditionalBuffer : MeshData->AdditionalBuffers)
				{
					const bool bIsPoseBufferType =
						AdditionalBuffer.Key == EMeshBufferType::SkeletonDeformBinding;

					if (bIsPoseBufferType)
					{
						MeshPoseData->AdditionalBuffers.Emplace(AdditionalBuffer);
					}
				}

				MeshPoseData->MeshIDPrefix = 0;
				AddMeshToProgram(MeshPoseData);
				EnumAddFlags(MeshContentFlags, EMeshContentFlags::PoseData);
			}

			// PhysicsMeshData
			{
				const EMeshCopyFlags PhysicsDataCopyFlags = 
						EMeshCopyFlags::WithAdditionalPhysics |
						EMeshCopyFlags::WithPhysicsBody;

				TManagedPtr<FMesh> MeshPhysicsData = MeshData->Clone(PhysicsDataCopyFlags);

				if (MeshPhysicsData->PhysicsBody)
				{
					TManagedPtr<FPhysicsBody> PhysicsBody = StaticCastManagedPtr<const FPhysicsBody>(MeshData->PhysicsBody)->Clone();
					MeshPhysicsData->PhysicsBody = PhysicsBody;

					PhysicsBody->BoneIds.Reset(PhysicsBody->GetBodyCount());
					for (FName BoneName : PhysicsBody->BodiesBoneNames)
					{
						PhysicsBody->BoneIds.Add(Program.AddConstant(BoneName));
					}
				}
			
				// Copy components related additional buffers.
				for (const TPair<EMeshBufferType, FMeshBufferSet>& AdditionalBuffer : MeshData->AdditionalBuffers)
				{
					const bool bIsPhysicsBufferType = 
							AdditionalBuffer.Key == EMeshBufferType::PhysicsBodyDeformBinding   ||
							AdditionalBuffer.Key == EMeshBufferType::PhysicsBodyDeformSelection ||
							AdditionalBuffer.Key == EMeshBufferType::PhysicsBodyDeformOffsets;

					if (bIsPhysicsBufferType)
					{
						MeshPhysicsData->AdditionalBuffers.Emplace(AdditionalBuffer);
					}
				}

				MeshPhysicsData->MeshIDPrefix = 0;
				AddMeshToProgram(MeshPhysicsData);	
				EnumAddFlags(MeshContentFlags, EMeshContentFlags::PhysicsData);
			}

			// MetadataMesh Mesh
			{
				const EMeshCopyFlags MetadataDataCopyFlags = 
						EMeshCopyFlags::WithSurfaces          |
						EMeshCopyFlags::WithMorphMetaData     |
						EMeshCopyFlags::WithSkinWeightProfileMetaData;

				TManagedPtr<FMesh> MeshMetadataData = MeshData->Clone(MetadataDataCopyFlags);
			
				// Add a descriptor MeshBufferSet to the metadata part to have formating info.
				{
					FMeshBufferSet VertexMeshFormat;
					const FMeshBufferSet& VertexBufferSet = MeshData->VertexBuffers;
					
					VertexMeshFormat.ElementCount = VertexBufferSet.ElementCount;

					const int32 NumVertexBuffers = VertexBufferSet.Buffers.Num();
					VertexMeshFormat.Buffers.SetNum(NumVertexBuffers);

					for (int32 BufferIndex = 0; BufferIndex < NumVertexBuffers; ++BufferIndex)
					{
						VertexMeshFormat.Buffers[BufferIndex].Channels = VertexBufferSet.Buffers[BufferIndex].Channels;	
						VertexMeshFormat.Buffers[BufferIndex].ElementSize = VertexBufferSet.Buffers[BufferIndex].ElementSize;	
					}
				
					MeshMetadataData->VertexBuffers = MoveTemp(VertexMeshFormat);
					EnumAddFlags(MeshMetadataData->VertexBuffers.Flags, EMeshBufferSetFlags::IsDescriptor);
				}

				{
					FMeshBufferSet IndexMeshFormat;
					const FMeshBufferSet& IndexBufferSet = MeshData->IndexBuffers;
					
					IndexMeshFormat.ElementCount = IndexBufferSet.ElementCount;

					const int32 NumIndexBuffers = IndexBufferSet.Buffers.Num();
					IndexMeshFormat.Buffers.SetNum(NumIndexBuffers);

					for (int32 BufferIndex = 0; BufferIndex < NumIndexBuffers; ++BufferIndex)
					{
						IndexMeshFormat.Buffers[BufferIndex].Channels = IndexBufferSet.Buffers[BufferIndex].Channels;	
						IndexMeshFormat.Buffers[BufferIndex].ElementSize = IndexBufferSet.Buffers[BufferIndex].ElementSize;	
					}
					
					MeshMetadataData->IndexBuffers = MoveTemp(IndexMeshFormat);
					EnumAddFlags(MeshMetadataData->IndexBuffers.Flags, EMeshBufferSetFlags::IsDescriptor);
				}

				// Add the clothing metadata.
				MeshMetadataData->ClothSections.SetNum(MeshData->ClothSections.Num());
				for (int32 ClothSectionIndex = 0; ClothSectionIndex < MeshMetadataData->ClothSections.Num(); ++ClothSectionIndex)
				{
					MeshMetadataData->ClothSections[ClothSectionIndex].AssetLODIndex = MeshData->ClothSections[ClothSectionIndex].AssetLODIndex;
				}

				MeshMetadataData->MeshIDPrefix = 0;
				AddMeshToProgram(MeshMetadataData);	
				EnumAddFlags(MeshContentFlags, EMeshContentFlags::MetaData);
			}


			// For now empty meshes are not discarded. A mesh rom index will be used even if empty.
			check(Program.ConstantMeshContentIndices.Num() - FirstIndex == 4);

			FMeshContentRange MeshContentRange;

			FMemory::Memzero(MeshContentRange);
			MeshContentRange.SetFirstIndex((uint32)FirstIndex);
			MeshContentRange.SetContentFlags(MeshContentFlags);
			MeshContentRange.MeshIDPrefix = MeshData->MeshIDPrefix;

			return Program.ConstantMeshes.Add(MeshContentRange);
		}   

		/** Adds a constant image data to a Program and returns its constant index. */
		uint32 AddConstantImage(FProgram& Program, const TManagedPtr<const FImage>& Image, const FSourceDataDescriptor& SourceDataDescriptor, FLinkerOptions& Options)
		{
			MUTABLE_CPUPROFILER_SCOPE(AddConstantImage);

			check(Image->GetSizeX() * Image->GetSizeY() > 0);
		
			FImageOperator& ImOp = Options.ImageOperator;

			if (!Options.bSeparateImageMips)
			{	
				// Use a map-based deduplication only if we are splitting mips.
				int32 ImageIndex = Program.ConstantImageLODsPermanent.Add(Image);
				Options.ImageConstantMipMap.Add(Image, ImageIndex);
				
				FImageLODRange LODRange
				{
					.FirstIndex    = Program.ConstantImageLODIndices.Add({ uint32(ImageIndex), 0 }),
					.ImageSizeX    = Image->GetSizeX(),
					.ImageSizeY    = Image->GetSizeY(),
					.LODCount      = (uint8)Image->GetLODCount(),
					.NumLODsInTail = (uint8)Image->GetLODCount(),
					.Flags         = Image->Flags,
					.ImageFormat   = Image->GetFormat(),
				};

				return Program.ConstantImages.Add(LODRange);
			}

			auto AddImageToProgram = [&Program, &Options](const TManagedPtr<FImage>& Image) -> int32
			{
				MUTABLE_CPUPROFILER_SCOPE(Deduplicate);
				const int32* FoundStoreIndex = Options.ImageConstantMipMap.Find(Image);
				if (FoundStoreIndex)
				{
					return Program.ConstantImageLODIndices.Add({ uint32(*FoundStoreIndex), 0 });
				}
				else
				{
					int32 StoreIndex = Program.ConstantImageLODsPermanent.Add(Image);
					Options.ImageConstantMipMap.Add(Image, StoreIndex);

					return Program.ConstantImageLODIndices.Add({ uint32(StoreIndex), 0 });
				}
			};

			// Compute number of LODs to store.
			int32 LODsToStore = Image->GetLODCount();
			if (SourceDataDescriptor.bNoMipmaps)
			{
				LODsToStore = FMath::Min(LODsToStore, 1);
			}
			else if (!(Image->Flags & FImage::IF_CANNOT_BE_SCALED))
			{
				LODsToStore = FImage::GetMipmapCount(Image->GetSizeX(), Image->GetSizeY());
			}
			
			// If the image cannot be scaled or the image does not need to generate mips, 
			// prevent from storing LODs that are not present.
			const int32 FirstLODToStoreInTail = 
					FMath::Min(LODsToStore, Image->DataStorage.ComputeFirstCompactedTailLOD());

			FImageLODRange LODRange
			{
				.FirstIndex    = Program.ConstantImageLODIndices.Num(),
				.ImageSizeX    = Image->GetSizeX(),
				.ImageSizeY    = Image->GetSizeY(),
				.LODCount      = (uint8)LODsToStore,
				// The last image is always considered part of the tail.
				.NumLODsInTail = (uint8)FMath::Max(1, LODsToStore - FirstLODToStoreInTail), 
				.Flags         = Image->Flags,
				.ImageFormat   = Image->GetFormat(),
			};

			constexpr int32 MaxQuality = 4;

			int32 LOD = 0;

			// Special case for the LODs available in Image that will be stored as a single rom per LOD, just a copy. 
			for (; LOD < FirstLODToStoreInTail && LOD < Image->GetLODCount(); ++LOD)
			{
				AddImageToProgram(ImOp.ExtractMip(Image.Get(), LOD));
			}

			if (LOD < LODsToStore)
			{
				EImageFormat ImageUnCompressedFormat = GetUncompressedFormat(Image->GetFormat());

				// Extract Mip does not support RGB images if it needs to resize.
				ImageUnCompressedFormat = ImageUnCompressedFormat == EImageFormat::RGB_UByte
						? EImageFormat::RGBA_UByte
						: ImageUnCompressedFormat;

				TManagedPtr<const FImage> RawImage = Image;
				if (ImageUnCompressedFormat != Image->GetFormat())
				{
					int32 FirstLODToDecompress = FMath::Min(Image->GetLODCount() - 1, LOD);
					RawImage = ImOp.ImagePixelFormat(MaxQuality, RawImage.Get(), ImageUnCompressedFormat, FirstLODToDecompress);
					RawImage = ImOp.ExtractMip(RawImage.Get(), LOD - FirstLODToDecompress);
				}
				else
				{
					RawImage = ImOp.ExtractMip(RawImage.Get(), LOD);
				}

				// Store LODs that will be stored as a single rom per LOD and need to be generated.
				for (; LOD < FirstLODToStoreInTail; ++LOD)
				{
					AddImageToProgram(ImOp.ImagePixelFormat(MaxQuality, RawImage.Get(), Image->GetFormat()));

					// This is wrong, we are losing signal, but compilation performance gets badly affected on
					// models with lots of images if we scale down from the original image.  
					RawImage = ImOp.ExtractMip(RawImage.Get(), 1);
				}

				// Store LODs that will be stored together in a single rom.
				const int32 RawImageLOD = LOD;
				if (LOD < LODsToStore)
				{
					TManagedPtr<FImage> ImageToStore = ImOp.ExtractMip(RawImage.Get(), LOD - RawImageLOD);
					ImageToStore->DataStorage.SetNumLODs(LODRange.NumLODsInTail);
					++LOD;
					
					for (; LOD < LODsToStore; ++LOD)
					{
						TManagedPtr<FImage> SrcLODImage = ImOp.ExtractMip(RawImage.Get(), LOD - RawImageLOD);

						const int32 DestLOD = LOD - FirstLODToStoreInTail;
						
						// Some formats that are not block based will not allocate data when setting the number of LODs.
						// NOTE: We are working with uncompressed formats, probably this is not needed.
						if (ImageToStore->DataStorage.GetLOD(DestLOD).IsEmpty())
						{
							ImageToStore->DataStorage.ResizeLOD(DestLOD, SrcLODImage->DataStorage.GetLOD(0).Num());
						}
						
						TArrayView<uint8> SrcView  = SrcLODImage->DataStorage.GetLOD(0);
						TArrayView<uint8> DestView = ImageToStore->DataStorage.GetLOD(DestLOD);

						check(DestView.Num() == SrcView.Num());
						FMemory::Memcpy(DestView.GetData(), SrcView.GetData(), DestView.Num());	
					}

					AddImageToProgram(ImOp.ImagePixelFormat(MaxQuality, ImageToStore.Get(), Image->GetFormat()));
				}
			}
			return Program.ConstantImages.Add(LODRange);
		}
	}

	void ASTOpConstantResource::Link(FProgram& Program, FLinkerOptions* Options)
	{
		MUTABLE_CPUPROFILER_SCOPE(ASTOpConstantResource_Link);

		if (!LinkedAddress && !bLinkedAndNull)
		{
			switch (Type)
			{
			case EOpType::ME_CONSTANT:
			{
				FOperation::MeshConstantArgs Args;
				FMemory::Memzero(Args);

				TManagedPtr<const FMesh> MeshValue = StaticCastManagedPtr<const FMesh>(Value);
				check(MeshValue);

				Args.Skeleton = -1;
				if (TManagedPtr<const FSkeleton> MeshSkeleton = MeshValue->GetSkeleton())
				{
					Args.Skeleton = Program.AddConstant(MeshSkeleton);
					//MeshValue->Skeleton = nullptr;
				}

				check(!MeshValue->PassthroughObject);
				
				Args.Value = AddConstantMesh(Program, MeshValue, *Options);
				Args.ClothID = MeshValue->ClothSections.Num() ? MeshValue->ClothSections[0].ClothingAsset.GetId() : PASSTHROUGH_ID_INVALID;

				int32 DataDescIndex = Options->AdditionalData.SourceMeshPerConstant.Add(SourceDataDescriptor);
				check(DataDescIndex == Args.Value);

				++Program.NumOps;
				LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
				
				AppendCode(Program.ByteCode, Type);
				AppendCode(Program.ByteCode, Args);

				break;
			}

			case EOpType::IM_CONSTANT:
			{
				FOperation::ResourceConstantArgs Args;
				FMemory::Memzero(Args);
				
				TManagedPtr<const FImage> pTyped = StaticCastManagedPtr<const FImage>(Value);
				check(pTyped);

				if (pTyped->GetSizeX() * pTyped->GetSizeY() == 0)
				{
					// It's an empty or degenerated image, return a null operation.
					// Null op
					LinkedAddress = 0;
					bLinkedAndNull = true;
				}
				else
				{
					check(!pTyped->PassthroughObject);

					Args.value = AddConstantImage(Program, pTyped, SourceDataDescriptor, *Options);
					
					int32 DataDescIndex = Options->AdditionalData.SourceImagePerConstant.Add(SourceDataDescriptor);
					check(DataDescIndex == Args.value);
					
					++Program.NumOps;
					LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
					
					AppendCode(Program.ByteCode, Type);
					AppendCode(Program.ByteCode, Args);
				}

				break;
			}
				
			case EOpType::LA_CONSTANT:
			{
				FOperation::ResourceConstantArgs Args;
				FMemory::Memzero(Args);
				
				TManagedPtr<const FLayout> pTyped = StaticCastManagedPtr<const FLayout>(Value);
				check(pTyped);
				Args.value = Program.AddConstant(pTyped);

				++Program.NumOps;
				LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
				
				AppendCode(Program.ByteCode, Type);
				AppendCode(Program.ByteCode, Args);
				
				break;
			}

			case EOpType::MI_CONSTANT:
			{
				FOperation::MaterialConstantArgs Args;
				FMemory::Memzero(Args);

				TManagedPtr<FMaterial> MaterialValue = StaticCastManagedPtr<const FMaterial>(Value)->Clone();
				check(MaterialValue);

				// Store linked child images into the material constant
				for (const TPair<FParameterKey, ASTChild>& ImageOperation : ImageOperations)
				{
					TVariant<FOperation::ADDRESS, TManagedPtr<const FImage>> NewImage;
					NewImage.Set<FOperation::ADDRESS>(ImageOperation.Value->LinkedAddress);

					FMaterial::FImageParameterData ImageParameterData;
					ImageParameterData.ImageParameter = NewImage;

					MaterialValue->ImageParameters.Add(ImageOperation.Key, ImageParameterData);
				}

				Args.ID = MaterialValue->PassthroughObject.Reset(); // Not serialized but compared. Set to invalid to avoid duplicating it.
				Args.Value = Program.AddConstant(MaterialValue);
				
				++Program.NumOps;
				LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
				
				AppendCode(Program.ByteCode, GetOpType());
				AppendCode(Program.ByteCode, Args);
				
				break;
			}
			default:
				unimplemented();
			}
			
			// Clear stored value to reduce memory usage.
			Value = nullptr;
		}
	}

	FImageDesc ASTOpConstantResource::GetImageDesc(bool, class FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		if (Type == EOpType::IM_CONSTANT)
		{
			// TODO: cache to avoid disk loading
			TManagedPtr<const FImage> ConstImage = StaticCastManagedPtr<const FImage>(Value);
			if (ensure(ConstImage))
			{
				Result.m_format = ConstImage->GetFormat();
				Result.m_lods = ConstImage->GetLODCount();
				Result.m_size = ConstImage->GetSize();
			}
		}
		else if (Type == EOpType::MI_CONSTANT)
		{
			if (Context)
			{
				if (const ASTChild* Image = ImageOperations.Find(Context->ParameterKey))
				{
					check(Image->Child);
					Result = Image->Child->GetImageDesc();
				}
			}
		}
		else
		{
			check(false);
		}

		return Result;
	}

	//-------------------------------------------------------------------------------------------------
	void ASTOpConstantResource::GetBlockLayoutSize(uint64 BlockId, int32* BlockX, int32* BlockY, FBlockLayoutSizeCache*)
	{
		switch (Type)
		{
		case EOpType::LA_CONSTANT:
		{
			TManagedPtr<const FLayout> pLayout = StaticCastManagedPtr<const FLayout>(Value);
			check(pLayout);

			if (pLayout)
			{
				int relId = pLayout->FindBlock(BlockId);
				if (relId >= 0)
				{
					*BlockX = pLayout->Blocks[relId].Size[0];
					*BlockY = pLayout->Blocks[relId].Size[1];
				}
				else
				{
					*BlockX = 0;
					*BlockY = 0;
				}
			}

			break;
		}
		default:
			check(false);
		}
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConstantResource::GetLayoutBlockSize(int32* pBlockX, int32* pBlockY)
	{
		switch (Type)
		{

		case EOpType::IM_CONSTANT:
		{
			// We didn't find any layout.
			*pBlockX = 0;
			*pBlockY = 0;
			break;
		}

		default:
			checkf(false, TEXT("Instruction not supported"));
		}
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpConstantResource::GetNonBlackRect(FImageRect& maskUsage) const
	{
		if (Type == EOpType::IM_CONSTANT)
		{
			// TODO: cache
			TManagedPtr<const FImage> pMask = StaticCastManagedPtr<const FImage>(Value);
			pMask->GetNonBlackRect(maskUsage);
			return true;
		}

		return false;
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpConstantResource::IsImagePlainConstant(FVector4f& color) const
	{
		bool res = false;
		switch (Type)
		{

		case EOpType::IM_CONSTANT:
		{
			TManagedPtr<const FImage> pImage = StaticCastManagedPtr<const FImage>(Value);
			if (pImage->GetSizeX() <= 0 || pImage->GetSizeY() <= 0)
			{
				res = true;
				color = FVector4f(0.0f,0.0f,0.0f,1.0f);
			}
			else if (pImage->Flags & FImage::IF_IS_PLAIN_COLOR_VALID)
			{
				if (pImage->Flags & FImage::IF_IS_PLAIN_COLOR)
				{
					res = true;
					color = pImage->Sample(FVector2f(0, 0));
				}
				else
				{
					res = false;
				}
			}
			else
			{
				if (pImage->IsPlainColor(color))
				{
					res = true;
					pImage->Flags |= FImage::IF_IS_PLAIN_COLOR;
				}

				pImage->Flags |= FImage::IF_IS_PLAIN_COLOR_VALID;
			}
			break;
		}

		default:
			break;
		}

		return res;
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpConstantResource::~ASTOpConstantResource()
	{
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpConstantResource::GetValueHash() const
	{
		return ValueHash;
	}


	//-------------------------------------------------------------------------------------------------
	TManagedPtr<const FResource> ASTOpConstantResource::GetValue() const
	{
		return Value;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpConstantResource::SetValue(const TManagedPtr<const FResource>& InValue)
	{
		MUTABLE_CPUPROFILER_SCOPE(ASTOpConstantResource_SetValue);

		switch (Type)
		{
		case EOpType::IM_CONSTANT:
		{
			TManagedPtr<const FImage> Resource = StaticCastManagedPtr<const FImage>(InValue);
			
			FOutputHashStream stream;
			{
				MUTABLE_CPUPROFILER_SCOPE(Serialize);
				FOutputArchive arch(&stream);
				FImage::Serialise(Resource.Get(), arch);
			}

			ValueHash = stream.GetHash();
			
			Value = Resource;
			break;
		}

		case EOpType::ME_CONSTANT:
		{
			TManagedPtr<const FMesh> Resource = StaticCastManagedPtr<const FMesh>(InValue);

			FOutputHashStream stream;
			{
				MUTABLE_CPUPROFILER_SCOPE(Serialize);
				FOutputArchive arch(&stream);
				FMesh::Serialise(Resource.Get(), arch);
			}

			ValueHash = stream.GetHash();

			Value = Resource;
			break;
		}

		case EOpType::LA_CONSTANT:
		{
			TManagedPtr<const FLayout> Resource = StaticCastManagedPtr<const FLayout>(InValue);

			FOutputHashStream stream;
			{
				MUTABLE_CPUPROFILER_SCOPE(Serialize);
				FOutputArchive arch(&stream);
				FLayout::Serialise(Resource.Get(), arch);
			}

			ValueHash = stream.GetHash();

			Value = Resource;
			break;
		}
		
		case EOpType::MI_CONSTANT:
		{
			TManagedPtr<const FMaterial> Resource = StaticCastManagedPtr<const FMaterial>(InValue);

			FOutputHashStream stream;
			{
				MUTABLE_CPUPROFILER_SCOPE(Serialize);
				FOutputArchive arch(&stream);
				FMaterial::Serialise(Resource.Get(), arch);
			}

			ValueHash = stream.GetHash();
			Value = Resource;

			break;
		}

		default:
			Value = InValue;
			break;
		}
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpConstantResource::GetImageSizeExpression() const
	{
		if (Type==EOpType::IM_CONSTANT)
		{
			Ptr<ImageSizeExpression> Res = new ImageSizeExpression;
			Res->type = ImageSizeExpression::ISET_CONSTANT;
			TManagedPtr<const FImage> Const = StaticCastManagedPtr<const FImage>(Value);
			Res->size = Const->GetSize();
			return Res;
		}

		return nullptr;
	}


	FSourceDataDescriptor ASTOpConstantResource::GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const
	{
		return SourceDataDescriptor;
	}

	
	ASTOp::EClosedMeshTest ASTOpConstantResource::IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const
	{
		if (Cache)
		{
			const ASTOp::EClosedMeshTest* Cached = Cache->Find(this);
			if (Cached)
			{
				return *Cached;
			}
		}

		ASTOp::EClosedMeshTest Result = EClosedMeshTest::Unknown;
		if (Type == EOpType::ME_CONSTANT)
		{
			TManagedPtr<const FMesh> Mesh = StaticCastManagedPtr<const FMesh>(Value);
			if (Mesh)
			{
				if (Mesh->IsClosed())
				{
					Result = EClosedMeshTest::Yes;
				}
				else
				{
					Result = EClosedMeshTest::No;
				}
			}
		}

		if (Cache)
		{
			Cache->Add(this,Result);
		}

		return Result;
	}

}
