// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** A constant mesh, image, volume or layout. */
	class ASTOpConstantResource final : public ASTOp
	{
	private:

		//!
		TManagedPtr<const FResource> Value;

		//! Value hash
		uint64 ValueHash = 0;

		//! We tried to link already but the result is a null op.
		bool bLinkedAndNull = false;

	public:

		//! Type of constant
		EOpType Type;

		/** Source data descriptor. */
		FSourceDataDescriptor SourceDataDescriptor;
		
		/** Image Parameters are evaluated at runtime. */
		TMap<FParameterKey, ASTChild> ImageOperations;

	public:

		~ASTOpConstantResource() override;

		// Own interface

		//! Get a hash of the stored value.
		uint32 GetValueHash() const;

		//! Get a pointer to the stored value.
		TManagedPtr<const FResource> GetValue() const;

		/** Set the value to store in this op.	*/
		void SetValue(const TManagedPtr<const FResource>& Value);


		// ASTOp interface
		virtual EOpType GetOpType() const override { return Type; }
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual uint32 Hash() const override;
		virtual void Link(FProgram& program, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool, class FGetImageDescContext*) const override;
		virtual void GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache) override;
		virtual void GetLayoutBlockSize(int32* pBlockX, int32* pBlockY) override;
		virtual bool GetNonBlackRect(FImageRect& maskUsage) const override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
		virtual EClosedMeshTest IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const override;
	};


}

