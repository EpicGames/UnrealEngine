// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantScalar.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpConstantScalar::ASTOpConstantScalar(float InValue)
	{
		Value = InValue;
	}


	void ASTOpConstantScalar::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	bool ASTOpConstantScalar::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConstantScalar* other = static_cast<const ASTOpConstantScalar*>(&OtherUntyped);
			return Value == other->Value;
		}
		return false;
	}


	uint32 ASTOpConstantScalar::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Value));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantScalar::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpConstantScalar> New = new ASTOpConstantScalar();
		New->Value = Value;
		return New;
	}


	void ASTOpConstantScalar::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::ScalarConstantArgs Args;
			FMemory::Memzero(Args);
			Args.Value = Value;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	int32 ASTOpConstantScalar::EvaluateInt(ASTOpList&, bool& bOutUnknown) const
	{
		bOutUnknown =  false;
		return Value;
	}

}
