// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpConstantScalar final : public ASTOp
	{
	public:

		float Value = 0.0f;

	public:

		ASTOpConstantScalar(float Value = 0.0f);

		virtual EOpType GetOpType() const override { return EOpType::SC_CONSTANT; }
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual int32 EvaluateInt(ASTOpList& Facts, bool& bOutUnknown) const override;
	};

}

