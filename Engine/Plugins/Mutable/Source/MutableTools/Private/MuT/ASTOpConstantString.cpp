// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpConstantString.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	void ASTOpConstantString::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	uint32 ASTOpConstantString::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(value));

		return Result;
	}


	bool ASTOpConstantString::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpConstantString* other = static_cast<const ASTOpConstantString*>(&otherUntyped);
			return value == other->value;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpConstantString::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpConstantString> n = new ASTOpConstantString();
		n->value = value;
		return n;
	}


	void ASTOpConstantString::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::ResourceConstantArgs Args;
			FMemory::Memzero(Args);
			Args.value = Program.AddConstant(value);

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, EOpType::ST_CONSTANT);
			AppendCode(Program.ByteCode, Args);
		}
	}

}
