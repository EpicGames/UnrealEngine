// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpConstantString final : public ASTOp
	{
	public:
		//!
		FString value;

	public:
		EOpType GetOpType() const override { return EOpType::ST_CONSTANT; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
	};


}

