// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpEntry.h"
#include "MuT/ASTOpConstantBool.h"


namespace UE::Mutable::Private
{

	ASTOpEntry::ASTOpEntry()
		: A(this)
	{
	}


	ASTOpEntry::~ASTOpEntry()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	EOpType ASTOpEntry::GetOpType() const
	{
		return EOpType::NONE;
	}


	bool ASTOpEntry::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpEntry* Other = static_cast<const ASTOpEntry*>(&OtherUntyped);
			return A == Other->A;
		}
		
		return false;
	}


	uint32 ASTOpEntry::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(A));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpEntry::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpEntry> New = new ASTOpEntry();
		
		New->A = MapChild(A.child());
		
		return New;
	}


	void ASTOpEntry::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(A);
	}


	void ASTOpEntry::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			if (A)
			{
				LinkedAddress = A->LinkedAddress;
			}
		}
	}
}
