// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpExternal.h"


namespace UE::Mutable::Private
{
	EOpType ASTOpExternal::GetOpType() const
	{
		return Type;
	}


	uint32 ASTOpExternal::Hash() const
	{
		uint32 Result = GetTypeHash(OperationInstancedStruct);
		
		for (const ASTChild& Input : Inputs)
		{
			Result = HashCombineFast(Result, GetTypeHash(Input));
		}
		
		return Result;
	}


	void ASTOpExternal::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		for (ASTChild& Input : Inputs)
		{
			Func(Input);
		}
	}


	bool ASTOpExternal::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpExternal* Other = static_cast<const ASTOpExternal*>(&OtherUntyped);

			if (OperationInstancedStruct != Other->OperationInstancedStruct)
			{
				return false;
			}

			if (Inputs.Num() != Other->Inputs.Num())
			{
				return false;
			}
			
			for (int Index = 0; Index < Inputs.Num(); Index++)
			{
				if (Inputs[Index] != Other->Inputs[Index])
				{
					return false;
				}
			}
			
			return true;
		}
		
		return false;
	}


	Ptr<ASTOp> ASTOpExternal::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpExternal> New = new ASTOpExternal();
		New->Type = Type;
		New->OperationInstancedStruct = OperationInstancedStruct;
		
		for (const ASTChild& Input : Inputs)
		{
			New->Inputs.Emplace(New, MapChild(Input.child()));
		}
		
		return New;
	}


	void ASTOpExternal::Link(FProgram& Program, FLinkerOptions* Options)
	{
		if (!LinkedAddress)
		{
			FOperation::ExternalArgs Args;
			FMemory::Memzero(Args);
			Args.NumOperants = Inputs.Num();

			check(Type != EOpType::NONE);

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(Type, Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, Type);
			AppendCode(Program.ByteCode, Args);

			for (const ASTChild& Input : Inputs)
			{
				AppendCode(Program.ByteCode, Input->LinkedAddress);
			}

			if (Options->ExternalOperations)
			{
				Options->ExternalOperations->Add(LinkedAddress, OperationInstancedStruct);
			}
		}
	}
}
