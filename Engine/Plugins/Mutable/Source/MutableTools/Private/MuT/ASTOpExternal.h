// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"
#include "StructUtils/InstancedStruct.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpExternal final : public ASTOp
	{
	public:
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void Link(FProgram&, FLinkerOptions*) override;

		EOpType Type = EOpType::NONE;
		
		FInstancedStruct OperationInstancedStruct;

		TArray<ASTChild> Inputs;
	};
}

