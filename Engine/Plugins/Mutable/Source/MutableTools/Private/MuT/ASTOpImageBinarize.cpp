// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageBinarize.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

	ASTOpImageBinarize::ASTOpImageBinarize()
		: Base(this),
		Threshold(this)
	{
	}


	ASTOpImageBinarize::~ASTOpImageBinarize()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageBinarize::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageBinarize* Other = static_cast<const ASTOpImageBinarize*>(&InOther);
			return Base == Other->Base &&
				Threshold == Other->Threshold;
		}
		return false;
	}


	uint32 ASTOpImageBinarize::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Threshold));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpImageBinarize::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageBinarize> New = new ASTOpImageBinarize();
		New->Base = MapChild(Base.child());
		New->Threshold = MapChild(Threshold.child());
		return New;
	}


	void ASTOpImageBinarize::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(Threshold);
	}


	void ASTOpImageBinarize::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageBinariseArgs Args;
			FMemory::Memzero(Args);

			if (Base)
			{
				Args.Base = Base->LinkedAddress;
			}
			if (Threshold)
			{
				Args.Threshold = Threshold->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageBinarize::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Base)
		{
			Result = Base->GetImageDesc(bReturnBestOption, Context);
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageBinarize::GetImageSizeExpression() const
	{
		if (Base)
		{
			return Base->GetImageSizeExpression();
		}

		return new ImageSizeExpression;
	}


	FSourceDataDescriptor ASTOpImageBinarize::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
