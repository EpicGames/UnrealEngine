// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageBlankLayout.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"


namespace UE::Mutable::Private
{

	ASTOpImageBlankLayout::ASTOpImageBlankLayout()
		: Layout(this)
	{
	}


	ASTOpImageBlankLayout::~ASTOpImageBlankLayout()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageBlankLayout::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType() == GetOpType())
		{
			const ASTOpImageBlankLayout* Other = static_cast<const ASTOpImageBlankLayout*>(&InOther);
			return Layout == Other->Layout &&
				BlockSize == Other->BlockSize &&
				Format == Other->Format &&
				GenerateMipmaps == Other->GenerateMipmaps &&
				MipmapCount == Other->MipmapCount;
		}
		
		return false;
	}


	uint32 ASTOpImageBlankLayout::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Layout));
		Result = HashCombineFast(Result, GetTypeHash(BlockSize));
		Result = HashCombineFast(Result, GetTypeHash(Format));
		Result = HashCombineFast(Result, GetTypeHash(GenerateMipmaps));
		Result = HashCombineFast(Result, GetTypeHash(MipmapCount));

		return Result;
	}


	Ptr<ASTOp> ASTOpImageBlankLayout::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageBlankLayout> n = new ASTOpImageBlankLayout();
		n->Layout = MapChild(Layout.child());
		n->BlockSize = BlockSize;
		n->Format = Format;
		n->GenerateMipmaps = GenerateMipmaps;
		n->MipmapCount = MipmapCount;
		return n;
	}


	void ASTOpImageBlankLayout::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Layout);
	}


	void ASTOpImageBlankLayout::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageBlankLayoutArgs Args;
			FMemory::Memzero(Args);

			if (Layout)
			{
				Args.Layout = Layout->LinkedAddress;
			}

			Args.BlockSize[0] = BlockSize[0];
			Args.BlockSize[1] = BlockSize[1];
			Args.Format = Format;
			Args.GenerateMipmaps = GenerateMipmaps;
			Args.MipmapCount = MipmapCount;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FImageDesc ASTOpImageBlankLayout::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
	   // TODO: We would need to process the layout to find the grid size, and then use
		// the block size with it.
		Result.m_size = FImageSize(0, 0);
		Result.m_format = Format;

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageBlankLayout::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> Res = new ImageSizeExpression;
		Res->type = ImageSizeExpression::ISET_LAYOUTFACTOR;
		Res->layout = Layout.child();
		Res->factor[0] = BlockSize[0];
		Res->factor[1] = BlockSize[1];
		return Res;
	}


	void ASTOpImageBlankLayout::GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY)
	{
		*OutBlockX = int32(BlockSize[0]);
		*OutBlockY = int32(BlockSize[1]);
	}


	bool ASTOpImageBlankLayout::IsImagePlainConstant(FVector4f& OutColor) const
	{
		OutColor[0] = 0;
		OutColor[1] = 0;
		OutColor[2] = 0;
		OutColor[3] = 0;
		return true;
	}

}
