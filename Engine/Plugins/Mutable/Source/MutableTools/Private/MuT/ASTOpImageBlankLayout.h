// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageBlankLayout final : public ASTOp
	{
	public:

		ASTChild Layout;

		/** Size of a layout block in pixels. */
		UE::Math::TIntVector2<uint16> BlockSize;

		EImageFormat Format = EImageFormat::None;

		/** If true, generate mipmaps. */
		uint8 GenerateMipmaps = 0;

		/** Mipmaps to generate if mipmaps are to be generated. 0 means all. */
		uint8 MipmapCount = 0;

	public:

		ASTOpImageBlankLayout();
		ASTOpImageBlankLayout(const ASTOpImageBlankLayout&) = delete;
		~ASTOpImageBlankLayout();

		virtual EOpType GetOpType() const override { return EOpType::IM_BLANKLAYOUT; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool bReturnBestOption, FGetImageDescContext*) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual void GetLayoutBlockSize(int* OutBlockX, int* OutBlockY) override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;

	};


}

