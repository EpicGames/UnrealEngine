// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageColorMap.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

	ASTOpImageColorMap::ASTOpImageColorMap()
		: Base(this),
		Mask(this),
		Map(this)
	{
	}


	ASTOpImageColorMap::~ASTOpImageColorMap()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageColorMap::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageColorMap* Other = static_cast<const ASTOpImageColorMap*>(&InOther);
			return Base == Other->Base &&
				Mask == Other->Mask &&
				Map == Other->Map;
		}
		return false;
	}


	uint32 ASTOpImageColorMap::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Mask));
		Result = HashCombineFast(Result, GetTypeHash(Map));

		return Result;
	}


	Ptr<ASTOp> ASTOpImageColorMap::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageColorMap> New = new ASTOpImageColorMap();
		New->Base = MapChild(Base.child());
		New->Mask = MapChild(Mask.child());
		New->Map = MapChild(Map.child());
		return New;
	}


	void ASTOpImageColorMap::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(Mask);
		Func(Map);
	}


	void ASTOpImageColorMap::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageColorMapArgs Args;
			FMemory::Memzero(Args);

			if (Base)
			{
				Args.Base = Base->LinkedAddress;
			}
			if (Mask)
			{
				Args.Mask = Mask->LinkedAddress;
			}
			if (Map)
			{
				Args.Map = Map->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageColorMap::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Base)
		{
			Result = Base->GetImageDesc(bReturnBestOption, Context);
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageColorMap::GetImageSizeExpression() const
	{
		if (Base)
		{
			return Base->GetImageSizeExpression();
		}

		return new ImageSizeExpression;
	}


	bool ASTOpImageColorMap::IsImagePlainConstant(FVector4f& OutColor) const
	{
		// Because we cannot calculate the OutColor
		bool bResult = false;
		return bResult;
	}


	FSourceDataDescriptor ASTOpImageColorMap::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
