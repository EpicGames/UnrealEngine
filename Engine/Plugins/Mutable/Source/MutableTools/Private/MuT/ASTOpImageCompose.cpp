// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageCompose.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "MuR/Layout.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/ImagePrivate.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{


//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
//-------------------------------------------------------------------------------------------------
ASTOpImageCompose::ASTOpImageCompose()
    : Layout(this)
    , Base(this)
	, BlockImage(this)
	, Mask(this)
{
}


//-------------------------------------------------------------------------------------------------
ASTOpImageCompose::~ASTOpImageCompose()
{
    // Explicit call needed to avoid recursive destruction
    ASTOp::RemoveChildren();
}


//-------------------------------------------------------------------------------------------------
bool ASTOpImageCompose::IsEqual(const ASTOp& otherUntyped) const
{
	if (otherUntyped.GetOpType() == GetOpType())
    {
		const ASTOpImageCompose* other = static_cast<const ASTOpImageCompose*>(&otherUntyped);
		return Layout == other->Layout &&
			Base ==other->Base &&
			BlockImage == other->BlockImage &&
			Mask == other->Mask &&
			BlockId == other->BlockId;
    }
    return false;
}


//-------------------------------------------------------------------------------------------------
uint32 ASTOpImageCompose::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
	
    Result = HashCombineFast(Result, GetTypeHash(Layout));
	Result = HashCombineFast(Result, GetTypeHash(Base));
	Result = HashCombineFast(Result, GetTypeHash(BlockImage));
	Result = HashCombineFast(Result, GetTypeHash(Mask));
	Result = HashCombineFast(Result, BlockId);
	
	return Result;
}


//-------------------------------------------------------------------------------------------------
UE::Mutable::Private::Ptr<ASTOp> ASTOpImageCompose::Clone(MapChildFuncRef mapChild) const
{
	UE::Mutable::Private::Ptr<ASTOpImageCompose> n = new ASTOpImageCompose();
    n->Layout = mapChild(Layout.child());
	n->Base = mapChild(Base.child());
	n->BlockImage = mapChild(BlockImage.child());
	n->Mask = mapChild(Mask.child());
	n->BlockId = BlockId;
    return n;
}


//-------------------------------------------------------------------------------------------------
void ASTOpImageCompose::ForEachChild(const TFunctionRef<void(ASTChild&)> f )
{
	f(Layout);
	f(Base);
	f(BlockImage);
	f(Mask);
}


//-------------------------------------------------------------------------------------------------
void ASTOpImageCompose::Link( FProgram& Program, FLinkerOptions*)
{
    // Already linked?
    if (!LinkedAddress)
    {
        FOperation::ImageComposeArgs Args;
		FMemory::Memzero(Args);

		if (Layout) 
		{
			Args.layout = Layout->LinkedAddress;
		}

		if (Base) 
		{
			Args.base = Base->LinkedAddress;
		}

		if (BlockImage) 
		{
			Args.blockImage = BlockImage->LinkedAddress;
		}

		if (Mask) 
		{
			Args.mask = Mask->LinkedAddress;
		}

        Args.BlockId = BlockId;

        ++Program.NumOps;
        LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
        
        AppendCode(Program.ByteCode, GetOpType());
        AppendCode(Program.ByteCode,Args);
    }

}


FImageDesc ASTOpImageCompose::GetImageDesc( bool returnBestOption, FGetImageDescContext* Context) const
{
    FImageDesc Result;

    // Local context in case it is necessary
    FGetImageDescContext LocalContext;
    if (!Context)
    {
		Context = &LocalContext;
    }
    else
    {
        // Cached result?
		FImageDesc* PtrValue = Context->m_results.Find(this);
		if (PtrValue)
		{
			return *PtrValue;
		}
	}

    // Actual work
    if (Base)
    {
		Result = Base->GetImageDesc( returnBestOption, Context);
    }

	if (BlockImage)
	{
		FImageDesc BlockDesc = BlockImage->GetImageDesc(returnBestOption, Context);
		Result.m_format = GetMostGenericFormat(Result.m_format,BlockDesc.m_format);
		Result.FormatIfAlpha = GetMostGenericFormat(Result.FormatIfAlpha,BlockDesc.FormatIfAlpha);
	}

	// Format overriden by current op.
	Context->bIsFormatFromImageParameter = false;

    // Cache th result
	Context->m_results.Add(this, Result);

    return Result;
}


UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageCompose::GetImageSizeExpression() const
{
    if ( Base )
    {
        return Base->GetImageSizeExpression();
    }

    return nullptr;
}


bool ASTOpImageCompose::IsImagePlainConstant(FVector4f& color) const
{
	bool res = false;

	if (BlockImage.child())
	{
		BlockImage->IsImagePlainConstant(color);
	}

	if (res && Base.child())
	{
		FVector4f baseColor;
		res = Base->IsImagePlainConstant(baseColor);
		res &= (color == baseColor);
	}

	return res;
}


void ASTOpImageCompose::GetLayoutBlockSize(int32* pBlockX, int32* pBlockY)
{
	// Try to follow the base image of the compose, which is the most stable.
	if (Base)
	{
		Base->GetLayoutBlockSize(pBlockX,pBlockY);
	}

	// We can only follow the block if the base is empty, since the first block will set the block size.	
	if (*pBlockX == 0)
	{
		// Let's try the block approach: We need the block size and the layout blocks
		int32 layoutBlocksX = 0;
		int32 layoutBlocksY = 0;
		if (Layout.child())
		{
			MUTABLE_CPUPROFILER_SCOPE(GetLayoutBlockSize_GetBlockLayoutSize);
			FBlockLayoutSizeCache cache;
			Layout->GetBlockLayoutSizeCached(BlockId, &layoutBlocksX, &layoutBlocksY, &cache);
		}

		if (layoutBlocksX > 0 && layoutBlocksY > 0 && BlockImage.child())
		{
			FImageDesc blockDesc = BlockImage->GetImageDesc();

			*pBlockX = blockDesc.m_size[0] / layoutBlocksX;
			*pBlockY = blockDesc.m_size[1] / layoutBlocksY;
		}
		else
		{
			*pBlockX = 0;
			*pBlockY = 0;
		}
	}
}


UE::Mutable::Private::Ptr<ASTOp> ASTOpImageCompose::OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const
{
	UE::Mutable::Private::Ptr<ASTOp> at;

	UE::Mutable::Private::Ptr<ASTOp> baseAt = Base.child();
	UE::Mutable::Private::Ptr<ASTOp> blockAt = BlockImage.child();
	UE::Mutable::Private::Ptr<ASTOp> layoutAt = Layout.child();
	if (layoutAt
		&&
		layoutAt->GetOpType() == EOpType::LA_CONSTANT
		&&
		baseAt
		&&
		blockAt)
	{
		const ASTOpConstantResource* typedLayout = static_cast<const ASTOpConstantResource*>(layoutAt.get());
		const UE::Mutable::Private::FLayout* pLayout = static_cast<const UE::Mutable::Private::FLayout*>(typedLayout->GetValue().Get());

		// Constant single-block full layout?
		if (pLayout->GetBlockCount() == 1
			&&
			pLayout->Blocks[0].Min == FIntVector2(0, 0)
			&&
			pLayout->Blocks[0].Size == pLayout->Size
			&&
			pLayout->Blocks[0].Id == BlockId
			)
		{
			// We could only take the block, but we must make sure it will have the format
			// and size of the base.
			UE::Mutable::Private::FImageDesc BaseDesc = baseAt->GetImageDesc(true);
			UE::Mutable::Private::FImageDesc BlockDesc = blockAt->GetImageDesc(true);

			at = blockAt;

			// \todo isn't this a common operation?
			if (BaseDesc.m_format != BlockDesc.m_format
				&&
				BaseDesc.m_format != EImageFormat::None)
			{
				UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> Reformat = new ASTOpImagePixelFormat;
				Reformat->Format = BaseDesc.m_format;
				Reformat->FormatIfAlpha = BaseDesc.FormatIfAlpha;
				Reformat->Source = at;
				at = Reformat;
			}

			if (BaseDesc.m_size != BlockDesc.m_size
				&&
				BaseDesc.m_size[0] != 0
				&&
				BaseDesc.m_size[1] != 0)
			{
				UE::Mutable::Private::Ptr<ASTOpImageResize> Resize = new ASTOpImageResize;
				Resize->Size[0] = BaseDesc.m_size[0];
				Resize->Size[1] = BaseDesc.m_size[1];
				Resize->Source = at;
				at = Resize;
			}
		}
	}

	return at;
}


FSourceDataDescriptor ASTOpImageCompose::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
{
	// Cache management
	TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
	if (!Context)
	{
		LocalContext.Reset(new FGetSourceDataDescriptorContext);
		Context = LocalContext.Get();
	}

	FSourceDataDescriptor* Found = Context->Cache.Find(this);
	if (Found)
	{
		return *Found;
	}

	// Not cached: calculate
	FSourceDataDescriptor Result;

	if (Base)
	{
		FSourceDataDescriptor SourceDesc = Base->GetSourceDataDescriptor(Context);
		Result.CombineWith(SourceDesc);
	}

	if (BlockImage)
	{
		FSourceDataDescriptor SourceDesc = BlockImage->GetSourceDataDescriptor(Context);
		Result.CombineWith(SourceDesc);
	}

	if (Mask)
	{
		FSourceDataDescriptor SourceDesc = Mask->GetSourceDataDescriptor(Context);
		Result.CombineWith(SourceDesc);
	}

	Context->Cache.Add(this, Result);

	return Result;
}


}
