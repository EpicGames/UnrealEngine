// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageConform.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageBlankLayout.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpImageInvert.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CompilerPrivate.h"

namespace UE::Mutable::Private
{

	ASTOpImageConform::ASTOpImageConform()
		: Source(this)
	{
	}


	ASTOpImageConform::~ASTOpImageConform()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageConform::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpImageConform* Other = static_cast<const ASTOpImageConform*>(&OtherUntyped);
			return Source == Other->Source && ConformDescriptor == Other->ConformDescriptor;
		}

		return false;
	}


	uint32 ASTOpImageConform::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(ConformDescriptor));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageConform::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpImageConform> New = new ASTOpImageConform();
		New->Source = mapChild(Source.child());
		New->ConformDescriptor = ConformDescriptor;

		return New;
	}


	void ASTOpImageConform::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
	}


	void ASTOpImageConform::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageConformArgs Args;
			FMemory::Memzero(Args);

			// Manual member copy to avoid adding unset bits to the program.
			Args.ConformDescriptor.m_size = ConformDescriptor.m_size;
			Args.ConformDescriptor.m_format = ConformDescriptor.m_format;
			Args.ConformDescriptor.FormatIfAlpha = ConformDescriptor.FormatIfAlpha;

			Args.ConformDescriptor.m_lods = ConformDescriptor.m_lods;

			if (Source) 
			{
				Args.Source = Source->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}

	FImageDesc ASTOpImageConform::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (Source.child())
		{
			Result = Source.child()->GetImageDesc(bReturnBestOption, Context);
		}

		if (ConformDescriptor.FormatIfAlpha != EImageFormat::None &&
			GetImageFormatData(Result.m_format).Channels > 3)
		{
			Result.m_format = ConformDescriptor.FormatIfAlpha;
		}
		else if (ConformDescriptor.m_format != EImageFormat::None)
		{
			Result.m_format = ConformDescriptor.m_format;
		}

		if (ConformDescriptor.FormatIfAlpha != EImageFormat::None)
		{
			Result.FormatIfAlpha = ConformDescriptor.FormatIfAlpha;
		}

		if (ConformDescriptor.m_size != FImageSize(0, 0))
		{
			Result.m_size = ConformDescriptor.m_size;
		}

		if (ConformDescriptor.m_lods != 0)
		{
			if (ConformDescriptor.m_lods == 0xFF)
			{
				Result.m_lods = FImage::GetMipmapCount(Result.m_size.X, Result.m_size.Y);
			}
			else
			{
				Result.m_lods = ConformDescriptor.m_lods;
			}
		}

		// Format overriden by current op.
		Context->bIsFormatFromImageParameter = false;

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	void ASTOpImageConform::GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY)
	{
		// We didn't find any layout yet.
		*OutBlockX = 0;
		*OutBlockY = 0;

		// Try the source
		if (Source)
		{
			Source->GetLayoutBlockSize(OutBlockX, OutBlockY);

			if (ConformDescriptor.m_size != FImageSize(0,0) && *OutBlockX > 0 && *OutBlockY > 0)
			{
				FImageDesc SourceDesc = Source->GetImageDesc(false);
				if (SourceDesc.m_size[0] > 0 && SourceDesc.m_size[1] > 0)
				{
					float FactorX = float(ConformDescriptor.m_size.X) / float(SourceDesc.m_size[0]);
					float FactorY = float(ConformDescriptor.m_size.Y) / float(SourceDesc.m_size[1]);
					*OutBlockX = int32(*OutBlockX * FactorX);
					*OutBlockY = int32(*OutBlockY * FactorY);
				}
				else
				{
					*OutBlockX = 0;
					*OutBlockY = 0;
				}
			}
		}

	}


	bool ASTOpImageConform::IsImagePlainConstant(FVector4f& color) const
	{
		bool res = false;
		if (Source.child())
		{
			return Source.child()->IsImagePlainConstant(color);
		}
		return res;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageConform::GetImageSizeExpression() const
	{
		UE::Mutable::Private::Ptr<ImageSizeExpression> Result;

		if (ConformDescriptor.m_size == FImageSize(0,0))
		{
			if (Source.child())
			{
				Result = Source.child()->GetImageSizeExpression();
			}
			else
			{
				Result = new ImageSizeExpression;
			}
		}
		else
		{
			Result = new ImageSizeExpression;
			Result->type = ImageSizeExpression::ISET_CONSTANT;
			Result->size[0] = ConformDescriptor.m_size.X;
			Result->size[1] = ConformDescriptor.m_size.Y;
		}

		return Result;
	}


	FSourceDataDescriptor ASTOpImageConform::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
