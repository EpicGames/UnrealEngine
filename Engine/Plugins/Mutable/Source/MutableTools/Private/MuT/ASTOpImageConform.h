// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageConform final : public ASTOp
	{
	public:

		ASTChild Source;
		FImageDesc ConformDescriptor;

	public:

		ASTOpImageConform();
		ASTOpImageConform(const ASTOpImageConform&) = delete;
		~ASTOpImageConform();

		virtual EOpType GetOpType() const override { return EOpType::IM_CONFORM; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		//virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const override;
		//virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		virtual void GetLayoutBlockSize(int* pBlockX, int* pBlockY) override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

