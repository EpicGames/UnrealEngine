// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageCrop.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"


namespace UE::Mutable::Private
{

	ASTOpImageCrop::ASTOpImageCrop()
		: Source(this)
	{
	}


	ASTOpImageCrop::~ASTOpImageCrop()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageCrop::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageCrop* other = static_cast<const ASTOpImageCrop*>(&InOther);
			return Source == other->Source &&
				Min == other->Min &&
				Size == other->Size;
		}
		return false;
	}


	uint32 ASTOpImageCrop::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Min));
		Result = HashCombineFast(Result, GetTypeHash(Size));

		return Result;
	}


	Ptr<ASTOp> ASTOpImageCrop::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageCrop> n = new ASTOpImageCrop();
		n->Source = MapChild(Source.child());
		n->Min = Min;
		n->Size = Size;
		return n;
	}


	void ASTOpImageCrop::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
	}


	void ASTOpImageCrop::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageCropArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.source = Source->LinkedAddress;
			}

			Args.minX = Min[0];
			Args.minY = Min[1];
			Args.sizeX = Size[0];
			Args.sizeY = Size[1];

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FImageDesc ASTOpImageCrop::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Source)
		{
			Result = Source->GetImageDesc(bReturnBestOption, Context);
			//Result.m_lods = 1;
			Result.m_size[0] = Size[0];
			Result.m_size[1] = Size[1];
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageCrop::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> Res = new ImageSizeExpression;
		Res->type = ImageSizeExpression::ISET_CONSTANT;
		Res->size[0] = Size[0];
		Res->size[1] = Size[1];
		return Res;
	}


	void ASTOpImageCrop::GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY)
	{
		// We didn't find any layout yet.
		*OutBlockX = 0;
		*OutBlockY = 0;

		// Try the source
		if (Source)
		{
			Source->GetLayoutBlockSize( OutBlockX, OutBlockY );
		}
	}


	FSourceDataDescriptor ASTOpImageCrop::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	Ptr<ASTOp> ASTOpImageCrop::OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> SourceAt = Source.child();

		// The instruction can be sunk
		EOpType sourceType = SourceAt->GetOpType();
		switch (sourceType)
		{
		case EOpType::IM_PLAINCOLOR:
		{
			Ptr<ASTOpImagePlainColor> NewOp = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(SourceAt.get());
			NewOp->Size[0] = Size[0];
			NewOp->Size[1] = Size[1];
			NewOp->LODs = 1; // TODO
			Result = NewOp;
			break;
		}

		default:
			break;

		}

		return Result;
	}


	Ptr<ASTOp> ASTOpImageCrop::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext& context) const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> SourceAt = Source.child();

		switch (SourceAt->GetOpType())
		{
		// In case we have other operations with special optimisation rules.
		case EOpType::NONE:
			break;

		default:
		{
			Result = context.ImageCropSinker.Apply(this);

			break;
		} // default

		}

		return Result;
	}


	Ptr<ASTOp> Sink_ImageCropAST::Apply(const ASTOpImageCrop* InRoot)
	{
		check(InRoot->GetOpType() == EOpType::IM_CROP);

		Root = InRoot;
		OldToNew.Reset();

		InitialSource = InRoot->Source.child();
		Ptr<ASTOp> newSource = Visit(InitialSource, InRoot);

		// If there is any change, it is the new root.
		if (newSource != InitialSource)
		{
			return newSource;
		}

		return nullptr;
	}


	Ptr<ASTOp> Sink_ImageCropAST::Visit(Ptr<ASTOp> at, const ASTOpImageCrop* currentCropOp)
	{
		if (!at) return nullptr;

		// Already visited?
		const Ptr<ASTOp>* Cached = OldToNew.Find({ at, currentCropOp });
		if (Cached)
		{
			return *Cached;
		}

		bool skipSinking = false;
		Ptr<ASTOp> newAt = at;
		switch (at->GetOpType())
		{

		case EOpType::IM_CONDITIONAL:
		{
			// We move down the two paths
			Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
			newOp->yes = Visit(newOp->yes.child(), currentCropOp);
			newOp->no = Visit(newOp->no.child(), currentCropOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_SWITCH:
		{
			// We move down all the paths
			Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
			newOp->Default = Visit(newOp->Default.child(), currentCropOp);
			for (auto& c : newOp->Cases)
			{
				c.Branch = Visit(c.Branch.child(), currentCropOp);
			}
			newAt = newOp;
			break;
		}

		case EOpType::IM_PIXELFORMAT:
		{
			Ptr<ASTOpImagePixelFormat> nop = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(at);
			nop->Source = Visit(nop->Source.child(), currentCropOp);
			newAt = nop;
			break;
		}

		case EOpType::IM_PATCH:
		{
			const ASTOpImagePatch* typedPatch = static_cast<const ASTOpImagePatch*>(at.get());

			Ptr<ASTOp> rectOp = typedPatch->patch.child();
			ASTOp::FGetImageDescContext context;
			FImageDesc patchDesc = rectOp->GetImageDesc(false, &context);
			box<FIntVector2> patchBox;
			patchBox.min[0] = typedPatch->location[0];
			patchBox.min[1] = typedPatch->location[1];
			patchBox.size[0] = patchDesc.m_size[0];
			patchBox.size[1] = patchDesc.m_size[1];

			box<FIntVector2> cropBox;
			cropBox.min[0] = currentCropOp->Min[0];
			cropBox.min[1] = currentCropOp->Min[1];
			cropBox.size[0] = currentCropOp->Size[0];
			cropBox.size[1] = currentCropOp->Size[1];

			if (!patchBox.IntersectsExclusive(cropBox))
			{
				// We can ignore the patch
				newAt = Visit(typedPatch->base.child(), currentCropOp);
			}
			else
			{
				// Crop the base with the full crop, and the patch with the intersected part,
				// adapting the patch origin
				Ptr<ASTOpImagePatch> newOp = UE::Mutable::Private::Clone<ASTOpImagePatch>(at);
				newOp->base = Visit(newOp->base.child(), currentCropOp);

				box<FIntVector2> ibox = patchBox.Intersect2i(cropBox);
				check(ibox.size[0] > 0 && ibox.size[1] > 0);

				Ptr<ASTOpImageCrop> patchCropOp = UE::Mutable::Private::Clone<ASTOpImageCrop>(currentCropOp);
				patchCropOp->Min[0] = ibox.min[0] - patchBox.min[0];
				patchCropOp->Min[1] = ibox.min[1] - patchBox.min[1];
				patchCropOp->Size[0] = ibox.size[0];
				patchCropOp->Size[1] = ibox.size[1];
				newOp->patch = Visit(newOp->patch.child(), patchCropOp.get());

				newOp->location[0] = ibox.min[0] - cropBox.min[0];
				newOp->location[1] = ibox.min[1] - cropBox.min[1];
				newAt = newOp;
			}

			break;
		}

		case EOpType::IM_CROP:
		{
			// We can combine the two crops into a possibly smaller crop
			const ASTOpImageCrop* childCrop = static_cast<const ASTOpImageCrop*>(at.get());

			box<FIntVector2> childCropBox;
			childCropBox.min[0] = childCrop->Min[0];
			childCropBox.min[1] = childCrop->Min[1];
			childCropBox.size[0] = childCrop->Size[0];
			childCropBox.size[1] = childCrop->Size[1];

			box<FIntVector2> cropBox;
			cropBox.min[0] = currentCropOp->Min[0];
			cropBox.min[1] = currentCropOp->Min[1];
			cropBox.size[0] = currentCropOp->Size[0];
			cropBox.size[1] = currentCropOp->Size[1];

			// Compose the crops: in the final image the child crop is applied first and the
			// current ctop is applied to the result. So the final crop box would be:
			box<FIntVector2> ibox;
			ibox.min = childCropBox.min + cropBox.min;
			ibox.size[0] = FMath::Min(cropBox.size[0], childCropBox.size[0]);
			ibox.size[1] = FMath::Min(cropBox.size[1], childCropBox.size[1]);
			//check(cropBox.min.AllSmallerOrEqualThan(childCropBox.size));
			//check((cropBox.min + cropBox.size).AllSmallerOrEqualThan(childCropBox.size));
			//check((ibox.min + ibox.size).AllSmallerOrEqualThan(childCropBox.min + childCropBox.size));

			// This happens more often that one would think
			if (ibox == childCropBox)
			{
				// the parent crop is not necessary
				skipSinking = true;
			}
			else if (ibox == cropBox)
			{
				// The child crop is not necessary
				Ptr<ASTOp> childSource = childCrop->Source.child();
				newAt = Visit(childSource, currentCropOp);
			}
			else
			{
				// combine into one crop
				Ptr<ASTOpImageCrop> newCropOp = UE::Mutable::Private::Clone<ASTOpImageCrop>(currentCropOp);
				newCropOp->Min[0] = ibox.min[0];
				newCropOp->Min[1] = ibox.min[1];
				newCropOp->Size[0] = ibox.size[0];
				newCropOp->Size[1] = ibox.size[1];

				Ptr<ASTOp> childSource = childCrop->Source.child();
				newAt = Visit(childSource, newCropOp.get());
			}
			break;
		}

		case EOpType::IM_LAYER:
		{
			// We move the op down the arguments
			Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(at);

			Ptr<ASTOp> aOp = nop->base.child();
			nop->base = Visit(aOp, currentCropOp);

			Ptr<ASTOp> bOp = nop->blend.child();
			nop->blend = Visit(bOp, currentCropOp);

			Ptr<ASTOp> mOp = nop->mask.child();
			nop->mask = Visit(mOp, currentCropOp);

			newAt = nop;
			break;
		}

		case EOpType::IM_LAYERCOLOR:
		{
			// We move the op down the arguments
			Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(at);

			Ptr<ASTOp> aOp = nop->base.child();
			nop->base = Visit(aOp, currentCropOp);

			Ptr<ASTOp> mOp = nop->mask.child();
			nop->mask = Visit(mOp, currentCropOp);

			newAt = nop;
			break;
		}

		case EOpType::IM_DISPLACE:
		{
			// We move the op down the arguments
			Ptr<ASTOpImageDisplace> nop = UE::Mutable::Private::Clone<ASTOpImageDisplace>(at);

			Ptr<ASTOp> aOp = nop->Source.child();
			nop->Source = Visit(aOp, currentCropOp);

			Ptr<ASTOp> bOp = nop->DisplacementMap.child();
			nop->DisplacementMap = Visit(bOp, currentCropOp);

			newAt = nop;
			break;
		}

		case EOpType::IM_RASTERMESH:
		{
			// We add cropping data to the raster mesh if it doesn't have any
			// \TODO: Is is possible to hit 2 crops on a raster mesh? Combine the crop.
			Ptr<ASTOpImageRasterMesh> nop = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(at);

			bool bRasterHasCrop = nop->UncroppedSizeX != 0;
			if (!bRasterHasCrop)
			{
				box<FIntVector2> cropBox;
				cropBox.min[0] = currentCropOp->Min[0];
				cropBox.min[1] = currentCropOp->Min[1];
				cropBox.size[0] = currentCropOp->Size[0];
				cropBox.size[1] = currentCropOp->Size[1];

				nop->UncroppedSizeX = nop->SizeX;
				nop->UncroppedSizeY = nop->SizeY;
				nop->CropMinX = cropBox.min[0];
				nop->CropMinY = cropBox.min[1];
				nop->SizeX = cropBox.size[0];
				nop->SizeY = cropBox.size[1];

				newAt = nop;
			}
			break;
		}

		case EOpType::IM_INTERPOLATE:
		{
			// Move the op  down all the paths
			Ptr<ASTOpImageInterpolate> NewOp = UE::Mutable::Private::Clone<ASTOpImageInterpolate>(at);

			for (int32 v = 0; v < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++v)
			{
				Ptr<ASTOp> child = NewOp->Targets[v].child();
				Ptr<ASTOp> bOp = Visit(child, currentCropOp);
				NewOp->Targets[v] = bOp;
			}

			newAt = NewOp;
			break;
		}

		default:
			break;
		}

		// end on line, replace with crop
		if (at == newAt && at != InitialSource && !skipSinking)
		{
			Ptr<ASTOpImageCrop> newOp = UE::Mutable::Private::Clone<ASTOpImageCrop>(currentCropOp);
			check(newOp->GetOpType() == EOpType::IM_CROP);

			newOp->Source = at;

			newAt = newOp;
		}

		OldToNew.Add({ at, currentCropOp }, newAt);

		return newAt;
	}


}
