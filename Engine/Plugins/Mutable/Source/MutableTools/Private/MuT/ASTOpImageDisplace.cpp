// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageDisplace.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

	ASTOpImageDisplace::ASTOpImageDisplace()
		: Source(this),
		DisplacementMap(this)
	{
	}


	ASTOpImageDisplace::~ASTOpImageDisplace()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageDisplace::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageDisplace* Other = static_cast<const ASTOpImageDisplace*>(&InOther);
			return Source == Other->Source &&
				DisplacementMap == Other->DisplacementMap;
		}
		return false;
	}


	uint32 ASTOpImageDisplace::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(DisplacementMap));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpImageDisplace::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageDisplace> New = new ASTOpImageDisplace();
		New->Source = MapChild(Source.child());
		New->DisplacementMap = MapChild(DisplacementMap.child());
		return New;
	}


	void ASTOpImageDisplace::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
		Func(DisplacementMap);
	}


	void ASTOpImageDisplace::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageDisplaceArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}
			if (DisplacementMap)
			{
				Args.DisplacementMap = DisplacementMap->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageDisplace::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Source)
		{
			Result = Source->GetImageDesc(bReturnBestOption, Context);
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageDisplace::GetImageSizeExpression() const
	{
		if (Source)
		{
			return Source->GetImageSizeExpression();
		}

		return new ImageSizeExpression;
	}


	FSourceDataDescriptor ASTOpImageDisplace::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	Ptr<ASTOp> ASTOpImageDisplace::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext& context) const
	{
		Ptr<ASTOp> at;

		Ptr<ASTOp> OriginalAt = at;
		Ptr<ASTOp> sourceAt = Source.child();
		Ptr<ASTOp> displaceMapAt = DisplacementMap.child();

		switch (sourceAt->GetOpType())
		{

		case EOpType::IM_CONDITIONAL:
		{
			if (displaceMapAt->GetOpType() == EOpType::IM_CONDITIONAL)
			{
				const ASTOpConditional* typedSource = static_cast<const ASTOpConditional*>(sourceAt.get());
				const ASTOpConditional* typedDisplacementMap = static_cast<const ASTOpConditional*>(displaceMapAt.get());

				if (typedSource->condition == typedDisplacementMap->condition)
				{
					Ptr<ASTOpConditional> nop = UE::Mutable::Private::Clone<ASTOpConditional>(sourceAt);

					Ptr<ASTOpImageDisplace> aOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(this);
					aOp->Source = typedSource->yes.child();
					aOp->DisplacementMap = typedDisplacementMap->yes.child();
					nop->yes = aOp;

					Ptr<ASTOpImageDisplace> bOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(this);
					bOp->Source = typedSource->no.child();
					bOp->DisplacementMap = typedDisplacementMap->no.child();
					nop->no = bOp;

					at = nop;
				}
			}
			break;
		}

		case EOpType::IM_SWITCH:
		{
			if (displaceMapAt->GetOpType() == EOpType::IM_SWITCH)
			{
				const ASTOpSwitch* typedSource = static_cast<const ASTOpSwitch*>(sourceAt.get());
				const ASTOpSwitch* typedDisplacementMap = static_cast<const ASTOpSwitch*>(displaceMapAt.get());

				if (typedSource->IsCompatibleWith(typedDisplacementMap))
				{
					// Move the format down all the paths
					auto nop = UE::Mutable::Private::Clone<ASTOpSwitch>(sourceAt);

					if (nop->Default)
					{
						Ptr<ASTOpImageDisplace> defOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(this);
						defOp->Source = typedSource->Default.child();
						defOp->DisplacementMap = typedDisplacementMap->Default.child();
						nop->Default = defOp;
					}

					// We need to copy the options because we change them
					for (size_t v = 0; v < nop->Cases.Num(); ++v)
					{
						if (nop->Cases[v].Branch)
						{
							Ptr<ASTOpImageDisplace> bOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(this);
							bOp->Source = typedSource->Cases[v].Branch.child();
							bOp->DisplacementMap = typedDisplacementMap->FindBranch(typedSource->Cases[v].Condition);
							nop->Cases[v].Branch = bOp;
						}
					}

					at = nop;
				}
			}

			// If we didn't optimize already, try to simply sink the source.
			if (OriginalAt == at)
			{
				// Make a project for every path
				Ptr<ASTOpSwitch> nop = UE::Mutable::Private::Clone<ASTOpSwitch>(sourceAt.get());

				if (nop->Default)
				{
					Ptr<ASTOpImageDisplace> defOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(this);
					defOp->Source = nop->Default.child();
					nop->Default = defOp;
				}

				// We need to copy the options because we change them
				for (size_t o = 0; o < nop->Cases.Num(); ++o)
				{
					if (nop->Cases[o].Branch)
					{
						Ptr<ASTOpImageDisplace> bOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(this);
						bOp->Source = nop->Cases[o].Branch.child();
						nop->Cases[o].Branch = bOp;
					}
				}

				at = nop;
			}

			break;
		}

		default:
			break;
		}

		return at;
	}

}
