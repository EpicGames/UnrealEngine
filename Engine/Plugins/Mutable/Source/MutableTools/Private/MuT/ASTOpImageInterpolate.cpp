// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageInterpolate.h"

#include "Containers/Map.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpSwitch.h"

namespace UE::Mutable::Private
{

	ASTOpImageInterpolate::ASTOpImageInterpolate()
		: Factor(this), Targets{ ASTChild(this),ASTChild(this),ASTChild(this),ASTChild(this),ASTChild(this),ASTChild(this) }
	{
	}


	ASTOpImageInterpolate::~ASTOpImageInterpolate()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageInterpolate::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageInterpolate* Other = static_cast<const ASTOpImageInterpolate*>(&OtherUntyped);
			for (int32 i = 0; i<MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++i)
			{
				if (!(Targets[i] == Other->Targets[i]))
				{
					return false;
				}
			}
			return Factor == Other->Factor;
		}
		return false;
	}


	uint32 ASTOpImageInterpolate::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Targets[0]));
		Result = HashCombineFast(Result, GetTypeHash(Targets[1]));
		Result = HashCombineFast(Result, GetTypeHash(Targets[2]));
		Result = HashCombineFast(Result, GetTypeHash(Targets[3]));
		Result = HashCombineFast(Result, GetTypeHash(Targets[4]));
		Result = HashCombineFast(Result, GetTypeHash(Targets[5]));
		Result = HashCombineFast(Result, GetTypeHash(Factor));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageInterpolate::Clone(MapChildFuncRef MapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpImageInterpolate> n = new ASTOpImageInterpolate();
		n->Factor = MapChild(Factor.child());
		for (int32 i = 0; i < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++i)
		{
			n->Targets[i] = MapChild(Targets[i].child());
		}
		n->Factor = Factor.child();
		return n;
	}


	void ASTOpImageInterpolate::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Factor);
		for (int32 i = 0; i < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++i)
		{
			Func(Targets[i]);
		}
	}


	void ASTOpImageInterpolate::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageInterpolateArgs Args;
			FMemory::Memzero(Args);

			if (Factor)
			{
				Args.Factor = Factor->LinkedAddress;
			}

			for (int32 i = 0; i < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++i)
			{
				if (Targets[i]) 
				{
					Args.Targets[i] = Targets[i]->LinkedAddress;
				}
			}			

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//!
	FImageDesc ASTOpImageInterpolate::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!Context)
		{
			Context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (Targets[0])
		{
			res = Targets[0]->GetImageDesc(bReturnBestOption, Context);
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, res);
		}

		return res;
	}


	void ASTOpImageInterpolate::GetLayoutBlockSize(int32* pBlockX, int32* pBlockY)
	{
		if (Targets[0].child())
		{
			// Assume the block size of the biggest mip
			Targets[0].child()->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageInterpolate::GetImageSizeExpression() const
	{
		UE::Mutable::Private::Ptr<ImageSizeExpression> pRes;

		if (Targets[0].child())
		{
			pRes = Targets[0].child()->GetImageSizeExpression();
		}
		else
		{
			pRes = new ImageSizeExpression;
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpImageInterpolate::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found= Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		for( int32 TargetIndex=0; TargetIndex<MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++TargetIndex)
		{ 
			if (Targets[TargetIndex])
			{
				FSourceDataDescriptor TargetDesc = Targets[TargetIndex]->GetSourceDataDescriptor(Context);
				Result.CombineWith(TargetDesc);
			}
		}

		Context->Cache.Add(this,Result);

		return Result;
	}


}
