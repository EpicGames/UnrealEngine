// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageInvert.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

	ASTOpImageInvert::ASTOpImageInvert()
		: Base(this)
	{
	}


	ASTOpImageInvert::~ASTOpImageInvert()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageInvert::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageInvert* Other = static_cast<const ASTOpImageInvert*>(&InOther);
			return Base == Other->Base;
		}
		return false;
	}


	uint32 ASTOpImageInvert::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Base));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpImageInvert::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageInvert> New = new ASTOpImageInvert();
		New->Base = MapChild(Base.child());
		return New;
	}


	void ASTOpImageInvert::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
	}


	void ASTOpImageInvert::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageInvertArgs Args;
			FMemory::Memzero(Args);

			if (Base)
			{
				Args.Base = Base->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageInvert::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Base)
		{
			Result = Base->GetImageDesc(bReturnBestOption, Context);
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageInvert::GetImageSizeExpression() const
	{
		if (Base)
		{
			return Base->GetImageSizeExpression();
		}

		return new ImageSizeExpression;
	}


	bool ASTOpImageInvert::IsImagePlainConstant(FVector4f& OutColor) const
	{
		bool bResult = true;
		OutColor = FVector4f(1.0f,1.0f,1.0f,1.0f);

		if (Base)
		{
			bResult = Base->IsImagePlainConstant(OutColor);
			if (bResult)
			{
				OutColor[0] = 1.0f - OutColor[0];
				OutColor[1] = 1.0f - OutColor[1];
				OutColor[2] = 1.0f - OutColor[2];
			}
		}

		return bResult;
	}


	FSourceDataDescriptor ASTOpImageInvert::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
