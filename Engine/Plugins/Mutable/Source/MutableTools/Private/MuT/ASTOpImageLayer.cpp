// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageLayer.h"

#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageCrop.h"
#include "MuT/ASTOpImageResize.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpSwitch.h"
#include "MuR/Model.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"


namespace UE::Mutable::Private
{

	ASTOpImageLayer::ASTOpImageLayer()
		: base(this)
		, blend(this)
		, mask(this)
	{
	}


	ASTOpImageLayer::~ASTOpImageLayer()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageLayer::IsEqual(const ASTOp& InOtherUntyped) const
	{
		if (InOtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpImageLayer* Other = static_cast<const ASTOpImageLayer*>(&InOtherUntyped);
			return base == Other->base &&
				blend == Other->blend &&
				mask == Other->mask &&
				blendType == Other->blendType &&
				blendTypeAlpha == Other->blendTypeAlpha &&
				BlendAlphaSourceChannel == Other->BlendAlphaSourceChannel &&
				Flags == Other->Flags;
		}
		return false;
	}


	uint32 ASTOpImageLayer::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(base));
		Result = HashCombineFast(Result, GetTypeHash(blend));
		Result = HashCombineFast(Result, GetTypeHash(mask));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageLayer::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpImageLayer> n = new ASTOpImageLayer();
		n->base = mapChild(base.child());
		n->blend = mapChild(blend.child());
		n->mask = mapChild(mask.child());
		n->blendType = blendType;
		n->blendTypeAlpha = blendTypeAlpha;
		n->BlendAlphaSourceChannel = BlendAlphaSourceChannel;
		n->Flags = Flags;
		return n;
	}


	void ASTOpImageLayer::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(base);
		f(blend);
		f(mask);
	}


	void ASTOpImageLayer::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageLayerArgs Args;
			FMemory::Memzero(Args);

			Args.blendType = (uint8)blendType;
			Args.blendTypeAlpha = (uint8)blendTypeAlpha;
			Args.BlendAlphaSourceChannel = BlendAlphaSourceChannel;
			Args.flags = Flags;

			if (base) 
			{
				Args.base = base->LinkedAddress;
			}

			if (blend) 
			{
				Args.blended = blend->LinkedAddress;
			}

			if (mask)
			{
				Args.mask = mask->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageLayer::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const 
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (base)
		{
			res = base->GetImageDesc(returnBestOption, context);
		}


		// Cache the result
		if (context)
		{
			context->m_results.Add(this, res);
		}

		return res;
	}


	void ASTOpImageLayer::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (base)
		{
			base->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageLayer::GetImageSizeExpression() const
	{
		if (base)
		{
			return base->GetImageSizeExpression();
		}

		return nullptr;
	}


	Ptr<ASTOp> ASTOpImageLayer::OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const
	{
		Ptr<ASTOp> at;

		Ptr<ASTOp> baseAt = base.child();
		Ptr<ASTOp> blendAt = blend.child();
		Ptr<ASTOp> maskAt = mask.child();

		if (!baseAt)
		{
			return at;
		}

		// Convert to image layer color if blend is plain
		if (!at && blendAt && blendAt->GetOpType() == EOpType::IM_PLAINCOLOR)
		{
			// TODO: May some flags be supported?
			if (Flags == 0)
			{
				const ASTOpImagePlainColor* BlendPlainColor = static_cast<const ASTOpImagePlainColor*>(blendAt.get());

				Ptr<ASTOpImageLayerColor> NewLayerColor = new ASTOpImageLayerColor;
				NewLayerColor->base = baseAt;
				NewLayerColor->mask = maskAt;
				NewLayerColor->blendType = blendType;
				NewLayerColor->blendTypeAlpha = blendTypeAlpha;
				NewLayerColor->BlendAlphaSourceChannel = BlendAlphaSourceChannel;
				NewLayerColor->color = BlendPlainColor->Color.child();
				at = NewLayerColor;
			}
		}

		// Plain masks optimization
		if (!at && maskAt)
		{
			FVector4f color;
			if (maskAt->IsImagePlainConstant(color))
			{
				// For masks we only use one channel
				if (FMath::IsNearlyZero(color[0]))
				{
					// If the mask is black, we can skip the entire operation
					at = base.child();
				}
				else if (FMath::IsNearlyEqual(color[0], 1, UE_SMALL_NUMBER))
				{
					// If the mask is white, we can remove it
					Ptr<ASTOpImageLayer> NewOp = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
					NewOp->mask = nullptr;
					at = NewOp;
				}
			}
		}

		// See if the mask is actually already in the alpha channel of the blended. In that case,
		// remove the mask and enable the flag to use the alpha from blended.
		// This sounds very specific but, experimentally, it seems to happen often.
		if (!at && maskAt && blendAt 
			&& 
			Flags==0
			)
		{
			// Traverse down the expression while the expressions match
			Ptr<ASTOp> CurrentMask = maskAt;
			Ptr<ASTOp> CurrentBlend = blendAt;
			
			bool bMatchingAlphaExpression = true;
			while (bMatchingAlphaExpression)
			{
				bMatchingAlphaExpression = false;

				// Skip blend ops that wouldn't change the alpha
				bool bUpdated = true;
				while (bUpdated)
				{
					bUpdated = false;
					if (!CurrentBlend)
					{
						break;
					}

					switch (CurrentBlend->GetOpType())
					{
					case EOpType::IM_LAYERCOLOR:
					{
						const ASTOpImageLayerColor* BlendLayer = static_cast<const ASTOpImageLayerColor*>(CurrentBlend.get());
						if (BlendLayer->blendTypeAlpha == EBlendType::BT_NONE)
						{
							CurrentBlend = BlendLayer->base.child();
							bUpdated = true;
						}
						break;
					}

					case EOpType::IM_LAYER:
					{
						const ASTOpImageLayer* BlendLayer = static_cast<const ASTOpImageLayer*>(CurrentBlend.get());
						if (BlendLayer->blendTypeAlpha == EBlendType::BT_NONE)
						{
							CurrentBlend = BlendLayer->base.child();
							bUpdated = true;
						}
						break;
					}

					default:
						break;
					}
				}

				// Matching ops?
				if (CurrentMask->GetOpType() != CurrentBlend->GetOpType())
				{
					break;
				}

				switch (CurrentMask->GetOpType())
				{

				case EOpType::IM_DISPLACE:
				{
					const ASTOpImageDisplace* MaskDisplace = static_cast<const ASTOpImageDisplace*>(CurrentMask.get());
					const ASTOpImageDisplace* BlendDisplace = static_cast<const ASTOpImageDisplace*>(CurrentBlend.get());
					if (MaskDisplace && BlendDisplace 
						&&
						MaskDisplace->DisplacementMap.child()
						==
						BlendDisplace->DisplacementMap.child())
					{
						CurrentMask = MaskDisplace->Source.child();
						CurrentBlend = BlendDisplace->Source.child();
						bMatchingAlphaExpression = true;
					}
					break;
				}

				case EOpType::IM_RASTERMESH:
				{
					const ASTOpImageRasterMesh* MaskRaster = static_cast<const ASTOpImageRasterMesh*>(CurrentMask.get());
					const ASTOpImageRasterMesh* BlendRaster = static_cast<const ASTOpImageRasterMesh*>(CurrentBlend.get());
					if (MaskRaster && BlendRaster
						&&
						MaskRaster->mesh.child() == BlendRaster->mesh.child()
						&&
						MaskRaster->projector.child() == BlendRaster->projector.child()
						&&
						MaskRaster->mask.child() == BlendRaster->mask.child()
						&&
						MaskRaster->angleFadeProperties.child() == BlendRaster->angleFadeProperties.child()
						&&
						MaskRaster->BlockId == BlendRaster->BlockId
						&&
						MaskRaster->LayoutIndex == BlendRaster->LayoutIndex
						)
					{
						CurrentMask = MaskRaster->image.child();
						CurrentBlend = BlendRaster->image.child();
						bMatchingAlphaExpression = true;
					}
					break;
				}

				case EOpType::IM_RESIZE:
				{
					const ASTOpImageResize* MaskResize = static_cast<const ASTOpImageResize*>(CurrentMask.get());
					const ASTOpImageResize* BlendResize = static_cast<const ASTOpImageResize*>(CurrentBlend.get());
					if (MaskResize && BlendResize 
						&&
						MaskResize->Size[0] == BlendResize->Size[0]
						&&
						MaskResize->Size[1] == BlendResize->Size[1])
					{
						CurrentMask = MaskResize->Source.child();
						CurrentBlend = BlendResize->Source.child();
						bMatchingAlphaExpression = true;
					}
					break;
				}

				default:
					// Case not supported, so don't optimize.
					break;
				}
			}

			if (CurrentMask && CurrentMask->GetOpType() == EOpType::IM_SWIZZLE)
			{
				// End of the possible mask expression chain match should have a swizzle selecting the alpha.
				const ASTOpImageSwizzle* MaskSwizzle = static_cast<const ASTOpImageSwizzle*>(CurrentMask.get());
				if (MaskSwizzle->SourceChannels[0] == 3
					&&
					!MaskSwizzle->SourceChannels[1]
					&&
					!MaskSwizzle->SourceChannels[2]
					&&
					!MaskSwizzle->SourceChannels[3]
					&&
					MaskSwizzle->Sources[0].child() == CurrentBlend
					)
				{
					// we can do something good here
					Ptr<ASTOpImageLayer> NewLayer = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
					NewLayer->mask = nullptr;
					NewLayer->Flags |= FOperation::ImageLayerArgs::F_USE_MASK_FROM_BLENDED;
					at = NewLayer;
				}
			}
		}

		// Try to avoid child swizzle
		if (!at)
		{
			// Is the base a swizzle expanding alpha from a texture?
			if (baseAt->GetOpType() == EOpType::IM_SWIZZLE)
			{
				const ASTOpImageSwizzle* TypedBase = static_cast<const ASTOpImageSwizzle*>(baseAt.get());
				bool bAreAllAlpha = true;
				for (int32 c=0; c<MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (TypedBase->Sources[c] && TypedBase->SourceChannels[c] != 3)
					{
						bAreAllAlpha = false;
						break;
					}
				}

				if (bAreAllAlpha)
				{
					// TODO
				}
			}

			// Is the mask a swizzle expanding alpha from a texture?
			if (!at && maskAt && maskAt->GetOpType() == EOpType::IM_SWIZZLE)
			{
				const ASTOpImageSwizzle* TypedBase = static_cast<const ASTOpImageSwizzle*>(maskAt.get());
				bool bAreAllAlpha = true;
				for (int32 c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (TypedBase->Sources[c] && TypedBase->SourceChannels[c] != 3)
					{
						bAreAllAlpha = false;
						break;
					}
				}

				if (bAreAllAlpha)
				{
					// TODO
				}
			}

			// Is the blend a swizzle selecting alpha?
			if (Pass>0 && !at && blendAt && blendAt->GetOpType() == EOpType::IM_SWIZZLE && Flags==0)
			{
				const ASTOpImageSwizzle* TypedBlend = static_cast<const ASTOpImageSwizzle*>(blendAt.get());
				if (TypedBlend->Format==EImageFormat::L_UByte
					&&
					TypedBlend->SourceChannels[0]==3)
				{
					const FImageDesc SourceImageDesc = TypedBlend->Sources[0]->GetImageDesc(true);
					if (ensure(SourceImageDesc.m_format == EImageFormat::RGBA_UByte))
					{
						Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
						nop->Flags = Flags | FOperation::ImageLayerArgs::FLAGS::F_BLENDED_RGB_FROM_ALPHA;
						nop->blend = TypedBlend->Sources[0].child();
						at = nop;
					}
				}
			}
		}

		// Introduce crop if mask is constant and smaller than the base
		if (!at && maskAt)
		{
			FImageRect sourceMaskUsage;
			FImageDesc maskDesc;

			bool validUsageRect = false;
			{
				//MUTABLE_CPUPROFILER_SCOPE(EvaluateAreasForCrop);

				validUsageRect = maskAt->GetNonBlackRect(sourceMaskUsage);
				if (validUsageRect)
				{
					check(sourceMaskUsage.size[0] > 0);
					check(sourceMaskUsage.size[1] > 0);

					FGetImageDescContext context;
					maskDesc = maskAt->GetImageDesc(false, &context);
				}
			}

			if (validUsageRect)
			{
				// Adjust for compressed blocks (4), and some extra mips (2 more mips, which is 4)
				// \TODO: block size may be different in ASTC
				constexpr int blockSize = 4 * 4;

				FImageRect maskUsage;
				maskUsage.min[0] = (sourceMaskUsage.min[0] / blockSize) * blockSize;
				maskUsage.min[1] = (sourceMaskUsage.min[1] / blockSize) * blockSize;
				FImageSize minOffset = sourceMaskUsage.min - maskUsage.min;
				maskUsage.size[0] = ((sourceMaskUsage.size[0] + minOffset[0] + blockSize - 1) / blockSize) * blockSize;
				maskUsage.size[1] = ((sourceMaskUsage.size[1] + minOffset[1] + blockSize - 1) / blockSize) * blockSize;

				// Is it worth?
				float ratio = float(maskUsage.size[0] * maskUsage.size[1])
					/ float(maskDesc.m_size[0] * maskDesc.m_size[1]);
				float acceptableCropRatio = options.AcceptableCropRatio;
				if (ratio < acceptableCropRatio)
				{
					check(maskUsage.size[0] > 0);
					check(maskUsage.size[1] > 0);

					Ptr<ASTOpImageCrop> cropMask = new ASTOpImageCrop();
					cropMask->Source = mask.child();
					cropMask->Min[0] = maskUsage.min[0];
					cropMask->Min[1] = maskUsage.min[1];
					cropMask->Size[0] = maskUsage.size[0];
					cropMask->Size[1] = maskUsage.size[1];

					Ptr<ASTOpImageCrop> cropBlended = new ASTOpImageCrop();
					cropBlended->Source = blend.child();
					cropBlended->Min[0] = maskUsage.min[0];
					cropBlended->Min[1] = maskUsage.min[1];
					cropBlended->Size[0] = maskUsage.size[0];
					cropBlended->Size[1] = maskUsage.size[1];

					Ptr<ASTOpImageCrop> cropBase = new ASTOpImageCrop();
					cropBase->Source = base.child();
					cropBase->Min[0] = maskUsage.min[0];
					cropBase->Min[1] = maskUsage.min[1];
					cropBase->Size[0] = maskUsage.size[0];
					cropBase->Size[1] = maskUsage.size[1];

					Ptr<ASTOpImageLayer> newLayer = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
					newLayer->base = cropBase;
					newLayer->blend = cropBlended;
					newLayer->mask = cropMask;

					Ptr<ASTOpImagePatch> patch = new ASTOpImagePatch();
					patch->base = baseAt;
					patch->patch = newLayer;
					patch->location[0] = maskUsage.min[0];
					patch->location[1] = maskUsage.min[1];
					at = patch;
				}
			}
		}

		return at;
	}


	Ptr<ASTOp> ASTOpImageLayer::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		Ptr<ASTOp> at;

		// Layer effects may be worth sinking down switches and conditionals, to be able
		// to apply extra optimisations
		Ptr<ASTOp> baseAt = base.child();
		Ptr<ASTOp> blendAt = blend.child();
		Ptr<ASTOp> maskAt = mask.child();

		if (!baseAt)
		{
			return at;
		}

		// If we failed to optimize so far, see if it is worth optimizing the blended branch only.
		if (!at && blendAt)
		{
			EOpType BlendType = blendAt->GetOpType();
			switch (BlendType)
			{


			case EOpType::IM_SWITCH:
			{
				const ASTOpSwitch* BlendSwitch = static_cast<const ASTOpSwitch*>(blendAt.get());

				// If at least a switch option is a plain color, sink the layer into the switch
				bool bWorthSinking = false;
				for (int32 v = 0; v < BlendSwitch->Cases.Num(); ++v)
				{
					if (BlendSwitch->Cases[v].Branch)
					{
						// \TODO: Use the smarter query function to detect plain images?
						if (BlendSwitch->Cases[v].Branch->GetOpType() == EOpType::IM_PLAINCOLOR)
						{
							bWorthSinking = true;
							break;
						}
					}
				}

				if (bWorthSinking)
				{
					bool bMaskIsCompatibleSwitch = false;
					const ASTOpSwitch* MaskSwitch = nullptr;
					if (maskAt && maskAt->GetOpType()== EOpType::IM_SWITCH)
					{
						MaskSwitch = static_cast<const ASTOpSwitch*>(maskAt.get());
						bMaskIsCompatibleSwitch = MaskSwitch->IsCompatibleWith(BlendSwitch);
					}

					Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(BlendSwitch);

					if (NewSwitch->Default)
					{
						Ptr<ASTOpImageLayer> defOp = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
						defOp->blend = BlendSwitch->Default.child();
						if (bMaskIsCompatibleSwitch)
						{
							defOp->mask = MaskSwitch->Default.child();
						}
						NewSwitch->Default = defOp;
					}

					for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
					{
						if (NewSwitch->Cases[v].Branch)
						{
							Ptr<ASTOpImageLayer> BranchOp = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
							BranchOp->blend = BlendSwitch->Cases[v].Branch.child();
							if (bMaskIsCompatibleSwitch)
							{
								BranchOp->mask = MaskSwitch->Cases[v].Branch.child();
							}
							NewSwitch->Cases[v].Branch = BranchOp;
						}
					}

					at = NewSwitch;
				}

				break;
			}

			default:
				break;

			}
		}

		// If we failed to optimize so far, see if it is worth optimizing the mask branch only.
		if (!at && maskAt)
		{
			EOpType MaskType = maskAt->GetOpType();
			switch (MaskType)
			{

			case EOpType::IM_SWITCH:
			{
				const ASTOpSwitch* MaskSwitch = static_cast<const ASTOpSwitch*>(maskAt.get());

				// If at least a switch option is a plain color, sink the layer into the switch
				bool bWorthSinking = false;
				for (int32 v = 0; v < MaskSwitch->Cases.Num(); ++v)
				{
					if (MaskSwitch->Cases[v].Branch)
					{
						// \TODO: Use the smarter query function to detect plain images?
						if (MaskSwitch->Cases[v].Branch->GetOpType() == EOpType::IM_PLAINCOLOR)
						{
							bWorthSinking = true;
							break;
						}
					}
				}

				if (bWorthSinking)
				{
					Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(MaskSwitch);

					if (NewSwitch->Default)
					{
						Ptr<ASTOpImageLayer> defOp = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
						defOp->mask = MaskSwitch->Default.child();
						NewSwitch->Default = defOp;
					}

					for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
					{
						if (NewSwitch->Cases[v].Branch)
						{
							Ptr<ASTOpImageLayer> BranchOp = UE::Mutable::Private::Clone<ASTOpImageLayer>(this);
							BranchOp->mask = MaskSwitch->Cases[v].Branch.child();
							NewSwitch->Cases[v].Branch = BranchOp;
						}
					}

					at = NewSwitch;
				}

				break;
			}

			default:
				break;

			}
		}

		return at;
	}


	FSourceDataDescriptor ASTOpImageLayer::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found = Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		if (base)
		{
			FSourceDataDescriptor SourceDesc = base->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		if (blend)
		{
			FSourceDataDescriptor SourceDesc = blend->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		if (mask)
		{
			FSourceDataDescriptor SourceDesc = mask->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		Context->Cache.Add(this, Result);

		return Result;
	}

}
