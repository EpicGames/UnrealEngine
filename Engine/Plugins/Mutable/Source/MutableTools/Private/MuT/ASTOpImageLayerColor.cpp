// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageLayerColor.h"

#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpConstantColor.h"
#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpImageLayerColor::ASTOpImageLayerColor()
		: base(this)
		, color(this)
		, mask(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpImageLayerColor::~ASTOpImageLayerColor()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpImageLayerColor::IsEqual(const ASTOp& InOtherUntyped) const
	{
		if (InOtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageLayerColor* Other = static_cast<const ASTOpImageLayerColor*>(&InOtherUntyped);
			return base == Other->base &&
				color == Other->color &&
				mask == Other->mask &&
				blendType == Other->blendType &&
				blendTypeAlpha == Other->blendTypeAlpha &&
				BlendAlphaSourceChannel == Other->BlendAlphaSourceChannel &&
				Flags == Other->Flags;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpImageLayerColor::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(base));
		Result = HashCombineFast(Result, GetTypeHash(color));
		Result = HashCombineFast(Result, GetTypeHash(mask));
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageLayerColor::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpImageLayerColor> n = new ASTOpImageLayerColor();
		n->base = mapChild(base.child());
		n->color = mapChild(color.child());
		n->mask = mapChild(mask.child());
		n->blendType = blendType;
		n->blendTypeAlpha = blendTypeAlpha;
		n->BlendAlphaSourceChannel = BlendAlphaSourceChannel;
		n->Flags = Flags;
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageLayerColor::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(base);
		f(color);
		f(mask);
	}


	void ASTOpImageLayerColor::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageLayerColorArgs Args;
			FMemory::Memzero(Args);

			Args.blendType = (uint8)blendType;
			Args.blendTypeAlpha = (uint8)blendTypeAlpha;
			Args.BlendAlphaSourceChannel = BlendAlphaSourceChannel;
			Args.flags = Flags;

			check(base);
			if (base)
			{
				Args.base = base->LinkedAddress;
			}

			if (color) 
			{
				Args.color = color->LinkedAddress;
			}

			if (mask) 
			{
				Args.mask = mask->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpImageLayerColor::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const 
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (base)
		{
			res = base->GetImageDesc(returnBestOption, context);
		}


		// Cache the result
		if (context)
		{
			context->m_results.Add(this, res);
		}

		return res;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageLayerColor::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (base)
		{
			base->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageLayerColor::GetImageSizeExpression() const
	{
		if (base)
		{
			return base->GetImageSizeExpression();
		}

		return nullptr;
	}


	//---------------------------------------------------------------------------------------------
	Ptr<ASTOp> ASTOpImageLayerColor::OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const
	{
		Ptr<ASTOp> at;

		// Plain masks optimization
		if (mask.child() && !(Flags & FOperation::ImageLayerArgs::F_USE_MASK_FROM_BLENDED) )
		{
			FVector4f Color;
			if (mask.child()->IsImagePlainConstant(Color))
			{
				// For masks we only use one channel
				if (FMath::IsNearlyZero(Color[0]))
				{
					// If the mask is black, we can skip the entire operation
					at = base.child();
				}
				else if (FMath::IsNearlyEqual(Color[0], 1, UE_SMALL_NUMBER))
				{
					// If the mask is white, we can remove it
					Ptr<ASTOpImageLayerColor> NewOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(this);
					NewOp->mask = nullptr;
					at = NewOp;
				}
			}
		}

		// Layer operations with constants that do nothing.
		if (!at)
		{
			bool bRGBUnchanged = blendType == EBlendType::BT_NONE;
			bool bAlphaUnchanged = blendTypeAlpha == EBlendType::BT_NONE;

			FVector4f ColorConst(0,0,0,1);
			if (!color || color.child()->GetOpType()==EOpType::CO_CONSTANT)
			{
				if (color)
				{
					const ASTOpConstantColor* TypedColor = static_cast<const ASTOpConstantColor*>(color.child().get());
					ColorConst = TypedColor->Value;
				}

				if (!bAlphaUnchanged)
				{
					// TODO: Update when alpha may come from alpha in the color.
					switch (blendTypeAlpha)
					{
					case EBlendType::BT_LIGHTEN: bAlphaUnchanged = FMath::IsNearlyEqual(ColorConst[BlendAlphaSourceChannel], 0.0f); break;
					case EBlendType::BT_MULTIPLY: bAlphaUnchanged = FMath::IsNearlyEqual(ColorConst[BlendAlphaSourceChannel], 1.0f); break;
					default: break;
					}
				}

				if (!bRGBUnchanged)
				{
					if (Flags & FOperation::ImageLayerArgs::FLAGS::F_BASE_RGB_FROM_ALPHA)
					{
						switch (blendType)
						{
						case EBlendType::BT_LIGHTEN: bRGBUnchanged = FMath::IsNearlyZero(ColorConst[3]); break;
						case EBlendType::BT_MULTIPLY: bRGBUnchanged = FMath::IsNearlyEqual(ColorConst[3],1.0f); break;
						default: break;
						}
					}
					else
					{
						// How many channels are there in the base?
						FImageDesc BaseDesc = base->GetImageDesc();
						const FImageFormatData& FormatDesc = GetImageFormatData(BaseDesc.m_format);
						if (FormatDesc.Channels == 1)
						{
							// We only need to check R
							switch (blendType)
							{
							case EBlendType::BT_LIGHTEN: bRGBUnchanged = FMath::IsNearlyZero(ColorConst[0]); break;
							case EBlendType::BT_MULTIPLY: bRGBUnchanged = FMath::IsNearlyEqual(ColorConst[0],1.0); break;
							default: break;
							}
						}
						else
						{
							// Check RGB
							switch (blendType)
							{
							case EBlendType::BT_LIGHTEN: bRGBUnchanged = ColorConst.IsNearlyZero3(UE_SMALL_NUMBER); break;
							case EBlendType::BT_MULTIPLY: bRGBUnchanged = FVector3f(ColorConst).Equals(FVector3f(1, 1, 1)); break;
							default: break;
							}
						}
					}
				}
			}

			if (bRGBUnchanged && bAlphaUnchanged)
			{
				// Skip this operation.
				at = base.child();
			}
		}


		// Try to avoid child swizzle
		if (!at)
		{
			// Is the base a swizzle getting expanding alpha from the same texture?
			if (base.child()->GetOpType() == EOpType::IM_SWIZZLE)
			{
				const ASTOpImageSwizzle* TypedBase = static_cast<const ASTOpImageSwizzle*>(base.child().get());
				bool bAreAllSameAlpha = true;
				Ptr<ASTOp> Source = nullptr;
				for (int32 c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					Ptr<ASTOp> ThisSource = TypedBase->Sources[c].child();
					if (!Source)
					{
						Source = ThisSource;
					}
					else if (ThisSource && ThisSource!=Source)
					{
						bAreAllSameAlpha = false;
						break;
					}


					if (ThisSource && TypedBase->SourceChannels[c] != 3)
					{
						bAreAllSameAlpha = false;
						break;
					}
				}

				if (bAreAllSameAlpha)
				{
					Ptr<ASTOpImageLayerColor> NewOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(this);
					NewOp->Flags |= FOperation::ImageLayerArgs::FLAGS::F_BASE_RGB_FROM_ALPHA;
					NewOp->base = Source;
					at = NewOp;
				}
			}
		}

		return at;
	}


	//-------------------------------------------------------------------------------------------------
	Ptr<ASTOp> ASTOpImageLayerColor::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		Ptr<ASTOp> at;

		// Layer effects may be worth sinking down switches and conditionals, to be able
		// to apply extra optimisations
		auto baseAt = base.child();
		auto maskAt = mask.child();

		// Promote conditions from the base
		EOpType baseType = baseAt->GetOpType();
		switch (baseType)
		{
			// Seems to cause operation explosion in optimizer in bandit model.
			// moved to generic sink in the default.
//            case EOpType::IM_CONDITIONAL:
//            {
//                bModified = true;

//                OP op = Program.m_code[baseAt];

//                OP aOp = Program.m_code[at];
//                aOp.Args.ImageLayerColor.base = Program.m_code[baseAt].Args.Conditional.yes;
//                op.args.Conditional.yes = Program.AddOp( aOp );

//                OP bOp = Program.m_code[at];
//                bOp.Args.ImageLayerColor.base = Program.m_code[baseAt].Args.Conditional.no;
//                op.args.Conditional.no = Program.AddOp( bOp );

//                at = Program.AddOp( op );
//                break;
//            }

		case EOpType::IM_SWITCH:
		{
			// Warning:
			// It seems to cause data explosion in optimizer in some models. Because
			// all switch branches become unique constants

			// See if the blended has an identical switch, to optimise it too
			const ASTOpSwitch* baseSwitch = static_cast<const ASTOpSwitch*>(baseAt.get());

			// Mask not supported yet
			if (maskAt)
			{
				break;
			}

			// Move the layer operation down base paths
			Ptr<ASTOpSwitch> nop = UE::Mutable::Private::Clone<ASTOpSwitch>(baseSwitch);

			if (nop->Default)
			{
				Ptr<ASTOpImageLayerColor> defOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(this);
				defOp->base = nop->Default.child();
				nop->Default = defOp;
			}

			for (int32 v = 0; v < nop->Cases.Num(); ++v)
			{
				if (nop->Cases[v].Branch)
				{
					Ptr<ASTOpImageLayerColor> bOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(this);
					bOp->base = nop->Cases[v].Branch.child();
					nop->Cases[v].Branch = bOp;
				}
			}

			at = nop;
			break;
		}

		case EOpType::IM_DISPLACE:
		{
			// Mask not supported yet. If there is a mask it wouldn't be correct to sink
			// unless the mask was a similar displace.
			if (maskAt)
			{
				break;
			}

			Ptr<ASTOpImageDisplace> NewDisplace = UE::Mutable::Private::Clone<ASTOpImageDisplace>(baseAt);

			Ptr<ASTOp> SourceOp = NewDisplace->Source.child();
			Ptr<ASTOpImageLayerColor> NewSource = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(this);
			NewSource->base = SourceOp;
			NewDisplace->Source = NewSource;

			at = NewDisplace;
			break;
		}

		case EOpType::IM_RASTERMESH:
		{
			// Mask not supported yet. If there is a mask it wouldn't be correct to sink.				
			if (maskAt)
			{
				break;
			}

			Ptr<ASTOpImageRasterMesh> NewRaster = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(baseAt);

			Ptr<ASTOp> sourceOp = NewRaster->image.child();
			Ptr<ASTOpImageLayerColor> NewSource = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(this);
			NewSource->base = sourceOp;
			NewRaster->image = NewSource;

			at = NewRaster;
			break;
		}


		default:
			break;

		}

		return at;
	}


	FSourceDataDescriptor ASTOpImageLayerColor::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found = Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		if (base)
		{
			FSourceDataDescriptor SourceDesc = base->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		if (mask)
		{
			FSourceDataDescriptor SourceDesc = mask->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		Context->Cache.Add(this, Result);

		return Result;
	}

}
