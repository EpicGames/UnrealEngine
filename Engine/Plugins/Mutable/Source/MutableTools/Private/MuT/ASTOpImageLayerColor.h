// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpImageLayerColor final : public ASTOp
	{
	public:

		ASTChild base;
		ASTChild color;
		ASTChild mask;

		/** Blend type used for the color channels. */
		EBlendType blendType = EBlendType::BT_NONE;

		/** Blend type used for the alpha channel if any. This will be applied to the alpha with the channel BlendAlphaSourceChannel of the color. */
		EBlendType blendTypeAlpha = EBlendType::BT_NONE;

		/** Channel to use from the source color argument to apply blendTypeAlpha, if any. */
		uint8 BlendAlphaSourceChannel = 0;

		/** See ImageLayerArgs::Flags .*/
		uint8 Flags = 0;

	public:

		ASTOpImageLayerColor();
		ASTOpImageLayerColor(const ASTOpImageLayerColor&) = delete;
		~ASTOpImageLayerColor();

		virtual EOpType GetOpType() const override { return EOpType::IM_LAYERCOLOR; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		virtual void GetLayoutBlockSize(int* pBlockX, int* pBlockY) override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const override;
		virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

