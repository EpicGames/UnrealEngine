// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageMakeGrowMap.h"

#include "Containers/Map.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{

	ASTOpImageMakeGrowMap::ASTOpImageMakeGrowMap()
		: Mask(this)
	{
	}


	ASTOpImageMakeGrowMap::~ASTOpImageMakeGrowMap()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageMakeGrowMap::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageMakeGrowMap* other = static_cast<const ASTOpImageMakeGrowMap*>(&otherUntyped);
			return Mask == other->Mask &&
				Border == other->Border;
		}
		return false;
	}


	uint32 ASTOpImageMakeGrowMap::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());		

		Result = HashCombineFast(Result, GetTypeHash(Mask));
		Result = HashCombineFast(Result, GetTypeHash(Border));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMakeGrowMap::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpImageMakeGrowMap> n = new ASTOpImageMakeGrowMap();
		n->Mask = mapChild(Mask.child());
		n->Border = Border;
		return n;
	}


	void ASTOpImageMakeGrowMap::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Mask);
	}


	void ASTOpImageMakeGrowMap::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageMakeGrowMapArgs Args;
			FMemory::Memzero(Args);

			Args.border = Border;
			if (Mask) 
			{
				Args.mask = Mask->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMakeGrowMap::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		UE::Mutable::Private::Ptr<ASTOp> at;

		switch (Mask.child()->GetOpType())
		{

		case EOpType::IM_CONDITIONAL:
		{
			// We move the format down the two paths
			Ptr<ASTOpConditional> nop = UE::Mutable::Private::Clone<ASTOpConditional>(Mask.child());

			Ptr<ASTOpImageMakeGrowMap> aOp = UE::Mutable::Private::Clone<ASTOpImageMakeGrowMap>(this);
			aOp->Mask = nop->yes.child();
			nop->yes = aOp;

			Ptr<ASTOpImageMakeGrowMap> bOp = UE::Mutable::Private::Clone<ASTOpImageMakeGrowMap>(this);
			bOp->Mask = nop->no.child();
			nop->no = bOp;

			at = nop;
			break;
		}

		case EOpType::IM_SWITCH:
		{
			// Move the format down all the paths
			Ptr<ASTOpSwitch> nop = UE::Mutable::Private::Clone<ASTOpSwitch>(Mask.child());

			if (nop->Default)
			{
				Ptr<ASTOpImageMakeGrowMap> defOp = UE::Mutable::Private::Clone<ASTOpImageMakeGrowMap>(this);
				defOp->Mask = nop->Default.child();
				nop->Default = defOp;
			}

			// We need to copy the options because we change them
			for (size_t v = 0; v < nop->Cases.Num(); ++v)
			{
				if (nop->Cases[v].Branch)
				{
					Ptr<ASTOpImageMakeGrowMap> bOp = UE::Mutable::Private::Clone<ASTOpImageMakeGrowMap>(this);
					bOp->Mask = nop->Cases[v].Branch.child();
					nop->Cases[v].Branch = bOp;
				}
			}

			at = nop;
			break;
		}

		default:
			break;
		}


		return at;
	}


	//!
	FImageDesc ASTOpImageMakeGrowMap::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (Mask.child())
		{
			res = Mask.child()->GetImageDesc(returnBestOption, context);
		}

		// Cache the result
		context->m_results.Add(this, res);

		return res;
	}


	void ASTOpImageMakeGrowMap::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (Mask.child())
		{
			// Assume the block size of the biggest mip
			Mask.child()->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	bool ASTOpImageMakeGrowMap::IsImagePlainConstant(FVector4f& color) const
	{
		bool res = false;
		if (Mask.child())
		{
			Mask.child()->IsImagePlainConstant(color);
		}
		return res;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageMakeGrowMap::GetImageSizeExpression() const
	{
		UE::Mutable::Private::Ptr<ImageSizeExpression> pRes;

		if (Mask.child())
		{
			pRes = Mask.child()->GetImageSizeExpression();
		}
		else
		{
			pRes = new ImageSizeExpression;
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpImageMakeGrowMap::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mask)
		{
			return Mask->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}

