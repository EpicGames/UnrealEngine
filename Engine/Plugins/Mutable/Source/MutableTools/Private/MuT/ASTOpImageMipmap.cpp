// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageMipmap.h"

#include "Containers/Map.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageConform.h"
#include "MuT/ASTOpImageBlankLayout.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{

	ASTOpImageMipmap::ASTOpImageMipmap()
		: Source(this)
	{
	}


	ASTOpImageMipmap::~ASTOpImageMipmap()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageMipmap::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageMipmap* other = static_cast<const ASTOpImageMipmap*>(&otherUntyped);
			return Source == other->Source &&
				Levels == other->Levels &&
				BlockLevels == other->BlockLevels &&
				bOnlyTail == other->bOnlyTail &&
				bPreventSplitTail == other->bPreventSplitTail &&
				AddressMode == other->AddressMode &&
				FilterType == other->FilterType;
		}
		return false;
	}


	uint32 ASTOpImageMipmap::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, Levels);

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMipmap::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpImageMipmap> n = new ASTOpImageMipmap();
		n->Source = mapChild(Source.child());
		n->Levels = Levels;
		n->BlockLevels = BlockLevels;
		n->bOnlyTail = bOnlyTail;
		n->bPreventSplitTail = bPreventSplitTail;
		n->AddressMode = AddressMode;
		n->FilterType = FilterType;

		return n;
	}


	void ASTOpImageMipmap::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
	}


	void ASTOpImageMipmap::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageMipmapArgs Args;
			FMemory::Memzero(Args);

			Args.levels = Levels;
			Args.blockLevels = BlockLevels;
			Args.onlyTail = bOnlyTail;
			Args.AddressMode = AddressMode;
			Args.FilterType = FilterType;

			if (Source)
			{
				Args.source = Source->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMipmap::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		UE::Mutable::Private::Ptr<ASTOp> at;

		// \TODO: This seems to fail with the bandit test model.
		//if (Source.child())
		//{
		//	ASTOp::FGetImageDescContext context;
		//	FImageDesc ChildDesc = Source.child()->GetImageDesc(false, &context);

		//	if (ChildDesc.m_lods>0 && ChildDesc.m_lods>=Levels)
		//	{
		//		// We can skip the mipmaps, because the child will always contain all lods anyway.
		//		at = Source.child();
		//	}
		//}

		return at;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMipmap::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext& context) const
	{
		UE::Mutable::Private::Ptr<ASTOp> at;

		UE::Mutable::Private::Ptr<ASTOp> sourceAt = Source.child();
		switch (sourceAt->GetOpType())
		{

		case EOpType::IM_BLANKLAYOUT:
		{
			// Set the mipmap generation on the blank layout operation
			Ptr<ASTOpImageBlankLayout> NewOp = UE::Mutable::Private::Clone<ASTOpImageBlankLayout>(sourceAt);
			NewOp->GenerateMipmaps = 1;    // true
			NewOp->MipmapCount = Levels;
			at = NewOp;
			break;
		}

		case EOpType::IM_PLAINCOLOR:
		{
			// Set the mipmap generation on the plaincolor operation
			Ptr<ASTOpImagePlainColor> mop = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(sourceAt);
			mop->LODs = Levels;
			at = mop;
			break;
		}

		case EOpType::IM_PIXELFORMAT:
		{
			// Swap unless the mipmap operation builds only the tail or is compressed.
			// Otherwise, we could fall in a loop of swapping mipmaps and pixelformats.
			UE::Mutable::Private::Ptr<ASTOpImageMipmap> mop = UE::Mutable::Private::Clone<ASTOpImageMipmap>(this);
			UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> fop = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(sourceAt);
			bool isCompressedFormat = IsCompressedFormat(fop->Format);
			if (isCompressedFormat && !mop->bOnlyTail)
			{
				mop->Source = fop->Source.child();
				fop->Source = mop;

				at = fop;
			}
			break;
		}

		case EOpType::IM_CONFORM:
		{
			UE::Mutable::Private::Ptr<ASTOpImageConform> ConformOp = UE::Mutable::Private::Clone<ASTOpImageConform>(sourceAt);
	
			// Differently to the ImageMipmap op where a value of 0 mips means generate all lods, in an ImageConform operation 0 value means 
			// keep the LODs that come form source and 0xFF generate all LODs, so we need to translate the value semantics.
			ConformOp->ConformDescriptor.m_lods = Levels == 0 ? 0xFF : Levels;
			at = ConformOp;

			break;
		}

		default:
		{
			at = context.ImageMipmapSinker.Apply(this);

			break;
		} // mipmap source default

		} // mipmap source type switch


		return at;
	}


	//!
	FImageDesc ASTOpImageMipmap::GetImageDesc(bool returnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (Source.child())
		{
			Result = Source.child()->GetImageDesc(returnBestOption, Context);
		}

		int mipLevels = Levels;

		if (mipLevels == 0)
		{
			mipLevels = FMath::Max(
				(int)ceilf(logf((float)Result.m_size[0]) / logf(2.0f)),
				(int)ceilf(logf((float)Result.m_size[1]) / logf(2.0f))
			);
		}

		Result.m_lods = FMath::Max(Result.m_lods, (uint8)mipLevels);

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	void ASTOpImageMipmap::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (Source.child())
		{
			// Assume the block size of the biggest mip
			Source.child()->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	bool ASTOpImageMipmap::IsImagePlainConstant(FVector4f& color) const
	{
		bool res = false;
		if (Source.child())
		{
			Source.child()->IsImagePlainConstant(color);
		}
		return res;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageMipmap::GetImageSizeExpression() const
	{
		UE::Mutable::Private::Ptr<ImageSizeExpression> pRes;

		if (Source.child())
		{
			pRes = Source.child()->GetImageSizeExpression();
		}
		else
		{
			pRes = new ImageSizeExpression;
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpImageMipmap::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_ImageMipmapAST::Apply(const ASTOpImageMipmap* InRoot)
	{
		Root = InRoot;
		OldToNew.Empty();

		if (Root->bOnlyTail)
		{
			return nullptr;
		}

		UE::Mutable::Private::Ptr<ASTOp> newSource;
		InitialSource = Root->Source.child();

		// Before sinking, see if it is worth splitting into miptail and mip.
		// We always need to leave the root mipmap operation in case any intermediate operation fails to generate all mips.
		if (!Root->bPreventSplitTail)
		{
			// the block mipmaps can be done before composition is done.
			UE::Mutable::Private::Ptr<ASTOpImageMipmap> newMip = UE::Mutable::Private::Clone<ASTOpImageMipmap>(Root);
			newMip->Levels = Root->BlockLevels;
			newMip->BlockLevels = Root->BlockLevels;
			newMip->bOnlyTail = false;
			newMip->bPreventSplitTail = true;

			// the smallest mipmaps after the composition is done.
			UE::Mutable::Private::Ptr<ASTOpImageMipmap> topMipOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(Root);
			topMipOp->bOnlyTail = true;
			topMipOp->bPreventSplitTail = true;

			// Proceed
			topMipOp->Source = Visit(InitialSource, newMip.get());

			newSource = topMipOp;
		}
		else
		{
			// Proceed
			newSource = Visit(InitialSource, Root);
		}

		Root = nullptr;

		// If there is any change, it is the new root.
		if (newSource != InitialSource)
		{
			return newSource;
		}

		return nullptr;
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_ImageMipmapAST::Visit(UE::Mutable::Private::Ptr<ASTOp> at, const ASTOpImageMipmap* currentMipmapOp)
	{
		if (!at) return nullptr;

		// Already visited?
		const Ptr<ASTOp>* Cached = OldToNew.Find({ at,currentMipmapOp });
		if (Cached)
		{
			return *Cached;
		}

		UE::Mutable::Private::Ptr<ASTOp> newAt = at;
		switch (at->GetOpType())
		{

		case EOpType::IM_CONDITIONAL:
		{
			// We move the op down the two paths
			Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
			newOp->yes = Visit(newOp->yes.child(), currentMipmapOp);
			newOp->no = Visit(newOp->no.child(), currentMipmapOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_SWITCH:
		{
			// We move the op down all the paths
			Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
			newOp->Default = Visit(newOp->Default.child(), currentMipmapOp);
			for (ASTOpSwitch::FCase& c : newOp->Cases)
			{
				c.Branch = Visit(c.Branch.child(), currentMipmapOp);
			}
			newAt = newOp;
			break;
		}

		case EOpType::IM_COMPOSE:
		{
			const ASTOpImageCompose* typedAt = static_cast<const ASTOpImageCompose*>(at.get());
			if (!currentMipmapOp->bOnlyTail
				&&
				// Don't move the mipmapping if we are composing with a mask.
				// TODO: allow mipmapping in the masks, RLE formats, etc.
				!typedAt->Mask
				)
			{
				Ptr<ASTOpImageCompose> newOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(at);

				Ptr<ASTOp> baseOp = newOp->Base.child();
				newOp->Base = Visit(baseOp, currentMipmapOp);

				Ptr<ASTOp> blockOp = newOp->BlockImage.child();
				newOp->BlockImage = Visit(blockOp, currentMipmapOp);

				newAt = newOp;
			}

			break;
		}

		case EOpType::IM_MULTICOMPOSE:
		{
			const ASTOpImageMultiCompose* typedAt = static_cast<const ASTOpImageMultiCompose*>(at.get());
			if (!currentMipmapOp->bOnlyTail
				//&&
				// Don't move the mipmapping if we are composing with a mask.
				// TODO: allow mipmapping in the masks, RLE formats, etc.
				//!typedAt->Mask
				)
			{
				Ptr<ASTOpImageMultiCompose> NewOp = UE::Mutable::Private::Clone<ASTOpImageMultiCompose>(at);

				Ptr<ASTOp> BaseOp = NewOp->Base.child();
				NewOp->Base = Visit(BaseOp, currentMipmapOp);

				Ptr<ASTOp> SourceImageOp = NewOp->SourceImage.child();
				NewOp->SourceImage = Visit(SourceImageOp, currentMipmapOp);

				newAt = NewOp;
			}

			break;
		}

		case EOpType::IM_PATCH:
		{
			if (!currentMipmapOp->bOnlyTail)
			{
				// Special case: we propagate the mipmapping down the patch up to
				// the level allowed by the patch size and placement. We then leave
				// a top-level  mipmapping operation to generate the smallest
				// mipmaps after the patch is done.

				const ASTOpImagePatch* typedSource = static_cast<const ASTOpImagePatch*>(at.get());
				Ptr<ASTOp> rectOp = typedSource->patch.child();
				ASTOp::FGetImageDescContext context;
				FImageDesc patchDesc = rectOp->GetImageDesc(false, &context);

				// Calculate the mip levels that can be calculated for the patch
				uint8 PatchBlockLevels = 0;
				{
					uint16 minX = typedSource->location[0];
					uint16 minY = typedSource->location[1];
					uint16 sizeX = patchDesc.m_size[0];
					uint16 sizeY = patchDesc.m_size[1];
					while (minX && minY && sizeX && sizeY
						&&
						minX % 2 == 0 && minY % 2 == 0 && sizeX % 2 == 0 && sizeY % 2 == 0)
					{
						PatchBlockLevels++;
						minX /= 2;
						minY /= 2;
						sizeX /= 2;
						sizeY /= 2;
					}
				}

				if (currentMipmapOp->Levels!= PatchBlockLevels || currentMipmapOp->BlockLevels!= PatchBlockLevels)
				{
					UE::Mutable::Private::Ptr<ASTOpImageMipmap> newMip = UE::Mutable::Private::Clone<ASTOpImageMipmap>(currentMipmapOp);
					newMip->Levels = PatchBlockLevels;
					newMip->BlockLevels = PatchBlockLevels;
					newMip->bOnlyTail = false;

					Ptr<ASTOpImagePatch> newOp = UE::Mutable::Private::Clone<ASTOpImagePatch>(at);
					newOp->base = Visit(newOp->base.child(), newMip.get());
					newOp->patch = Visit(newOp->patch.child(), newMip.get());
					newAt = newOp;

					if (currentMipmapOp->Levels != currentMipmapOp->BlockLevels						
						// If the current levels are all of them, we will want to rebuild the mips after patch.
						// This happens if ignoring layouts, in which case there is no top-most mipmap to ensure the tail already.
						|| currentMipmapOp->BlockLevels == 0 
						)
					{
						// We need to add a mipmap on top to finish the mipmapping
						UE::Mutable::Private::Ptr<ASTOpImageMipmap> topMipOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(currentMipmapOp);
						topMipOp->Source = newOp;
						topMipOp->bOnlyTail = true;
						newAt = topMipOp;
					}
				}
				else
				{
					// The patch supports the same amount of mips that we are currently sinking.
					Ptr<ASTOpImagePatch> newOp = UE::Mutable::Private::Clone<ASTOpImagePatch>(at);
					newOp->base = Visit(newOp->base.child(), currentMipmapOp);
					newOp->patch = Visit(newOp->patch.child(), currentMipmapOp);
					newAt = newOp;
				}
			}

			break;
		}

		default:
			break;
		}

		// end on line, replace with mipmap
		if (at == newAt && at != InitialSource)
		{
			UE::Mutable::Private::Ptr<ASTOpImageMipmap> newOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(currentMipmapOp);
			check(newOp->GetOpType() == EOpType::IM_MIPMAP);

			newOp->Source = at;

			newAt = newOp;
		}

		OldToNew.Add({ at,currentMipmapOp }, newAt);

		return newAt;
	}

}
