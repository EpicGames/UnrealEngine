// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageMipmap final : public ASTOp
	{
	public:

		ASTChild Source;

		uint8 Levels = 0;

		//! Number of mipmaps that can be generated for a single layout block.
		uint8 BlockLevels = 0;

		//! This is true if this operation is supposed to build only the tail mipmaps.
		//! It is used during the code optimisation phase, and to validate the code.
		bool bOnlyTail = false;

		/** If this is enabled, at optimize time, the mip operation will not be split in top and bottom mip (for compose tails). */
		bool bPreventSplitTail = false;

		//! Mipmap generation settings. 
		EAddressMode AddressMode = EAddressMode::None;
		EMipmapFilterType FilterType = EMipmapFilterType::SimpleAverage;

	public:

		ASTOpImageMipmap();
		ASTOpImageMipmap(const ASTOpImageMipmap&) = delete;
		~ASTOpImageMipmap();

		virtual EOpType GetOpType() const override { return EOpType::IM_MIPMAP; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual UE::Mutable::Private::Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const override;
		virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		virtual void GetLayoutBlockSize(int* pBlockX, int* pBlockY) override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

