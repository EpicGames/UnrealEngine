// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageMultiCompose.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "MuR/Layout.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/ImagePrivate.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

ASTOpImageMultiCompose::ASTOpImageMultiCompose()
    : Layout(this)
    , Base(this)
	, SourceLayout(this)
	, SourceImage(this)
{
}


ASTOpImageMultiCompose::~ASTOpImageMultiCompose()
{
    // Explicit call needed to avoid recursive destruction
    ASTOp::RemoveChildren();
}


bool ASTOpImageMultiCompose::IsEqual(const ASTOp& OtherUntyped) const
{
	if (OtherUntyped.GetOpType() == GetOpType())
    {
		const ASTOpImageMultiCompose* Other = static_cast<const ASTOpImageMultiCompose*>(&OtherUntyped);
		return 
			Layout == Other->Layout &&
			Base == Other->Base &&
			SourceLayout == Other->SourceLayout &&
			SourceImage == Other->SourceImage &&
			LayoutBlockSizeInPixelsX == Other->LayoutBlockSizeInPixelsX &&
			LayoutBlockSizeInPixelsY == Other->LayoutBlockSizeInPixelsY &&
			SourceSizeX == Other->SourceSizeX &&
			SourceSizeY == Other->SourceSizeY;
			
    }

    return false;
}


uint32 ASTOpImageMultiCompose::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
	
	Result = HashCombineFast(Result, GetTypeHash(Layout));
	Result = HashCombineFast(Result, GetTypeHash(Base));
	Result = HashCombineFast(Result, GetTypeHash(SourceLayout));
	Result = HashCombineFast(Result, GetTypeHash(SourceImage));

	return Result;
}


UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMultiCompose::Clone(MapChildFuncRef MapChild) const
{
	UE::Mutable::Private::Ptr<ASTOpImageMultiCompose> Clone = new ASTOpImageMultiCompose();
    Clone->Layout = MapChild(Layout.child());
	Clone->Base = MapChild(Base.child());
	Clone->SourceLayout = MapChild(SourceLayout.child());
	Clone->SourceImage = MapChild(SourceImage.child());

	Clone->LayoutBlockSizeInPixelsX = LayoutBlockSizeInPixelsX;
	Clone->LayoutBlockSizeInPixelsY = LayoutBlockSizeInPixelsY;

	Clone->SourceSizeX = SourceSizeX;
	Clone->SourceSizeY = SourceSizeY;

    return Clone;
}


void ASTOpImageMultiCompose::ForEachChild(const TFunctionRef<void(ASTChild&)> Function)
{
	Function(Layout);
	Function(Base);
	Function(SourceLayout);
	Function(SourceImage);
}


void ASTOpImageMultiCompose::Link( FProgram& Program, FLinkerOptions*)
{
    // Already linked?
    if (!LinkedAddress)
    {
        FOperation::ImageMultiComposeArgs Args;
		FMemory::Memzero(Args);

		if (Layout) 
		{
			Args.Layout = Layout->LinkedAddress;
		}

		if (Base) 
		{
			Args.Base = Base->LinkedAddress;
		}

		if (SourceLayout) 
		{
			Args.SourceLayout = SourceLayout->LinkedAddress;
		}

		if (SourceImage) 
		{
			Args.SourceImage = SourceImage->LinkedAddress;
		}

		Args.LayoutBlockSizeInPixelsX = LayoutBlockSizeInPixelsX;
		Args.LayoutBlockSizeInPixelsY = LayoutBlockSizeInPixelsY;
		
		Args.SourceSizeX = SourceSizeX;
		Args.SourceSizeY = SourceSizeY;

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}

}


FImageDesc ASTOpImageMultiCompose::GetImageDesc( bool bReturnBestOption, FGetImageDescContext* Context ) const
{
    FImageDesc Result;

    // Local context in case it is necessary
    FGetImageDescContext LocalContext;
    if (!Context)
    {
		Context = &LocalContext;
    }
    else
    {
        // Cached result?
		FImageDesc* PtrValue = Context->m_results.Find(this);
		if (PtrValue)
		{
			return *PtrValue;
		}
	}

    // Actual work
    if (Base)
    {
		Result = Base->GetImageDesc(bReturnBestOption, Context);
    }

	if (SourceImage)
	{
		FImageDesc SourceImageDesc = SourceImage->GetImageDesc(bReturnBestOption, Context);
		Result.m_format = GetMostGenericFormat(Result.m_format, SourceImageDesc.m_format);
	}

    // Cache th result
    if (Context)
    {
		Context->m_results.Add(this, Result);
	}

    return Result;
}


UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageMultiCompose::GetImageSizeExpression() const
{
    if ( Base )
    {
        return Base->GetImageSizeExpression();
    }

    return nullptr;
}


bool ASTOpImageMultiCompose::IsImagePlainConstant(FVector4f& Color) const
{
	bool bIsImagePlainConstant = false;

	if (SourceImage.child())
	{
		bIsImagePlainConstant = SourceImage->IsImagePlainConstant(Color);
	}

	if (bIsImagePlainConstant && Base.child())
	{
		FVector4f BaseColor;
		bIsImagePlainConstant = Base->IsImagePlainConstant(BaseColor);
		bIsImagePlainConstant &= (Color == BaseColor);
	}

	return bIsImagePlainConstant;
}


void ASTOpImageMultiCompose::GetLayoutBlockSize(int32* BlockX, int32* BlockY)
{
	// Try to follow the base image of the compose, which is the most stable.
	if (Base)
	{
		Base->GetLayoutBlockSize(BlockX, BlockY);
	}

	// We can only follow the block if the base is empty, since the first block will set the block size.	
	if (*BlockX == 0)
	{
		*BlockX = static_cast<int32>(LayoutBlockSizeInPixelsX);
		*BlockY = static_cast<int32>(LayoutBlockSizeInPixelsY);
	}
}


FSourceDataDescriptor ASTOpImageMultiCompose::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
{
	// Cache management
	TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
	if (!Context)
	{
		LocalContext.Reset(new FGetSourceDataDescriptorContext);
		Context = LocalContext.Get();
	}

	FSourceDataDescriptor* Found = Context->Cache.Find(this);
	if (Found)
	{
		return *Found;
	}

	// Not cached: calculate
	FSourceDataDescriptor Result;

	if (Base)
	{
		FSourceDataDescriptor SourceDesc = Base->GetSourceDataDescriptor(Context);
		Result.CombineWith(SourceDesc);
	}

	if (SourceImage)
	{
		FSourceDataDescriptor SourceDesc = SourceImage->GetSourceDataDescriptor(Context);
		Result.CombineWith(SourceDesc);
	}

	//if (Mask)
	//{
	//	FSourceDataDescriptor SourceDesc = Mask->GetSourceDataDescriptor(Context);
	//	Result.CombineWith(SourceDesc);
	//}

	Context->Cache.Add(this, Result);

	return Result;
}


}
