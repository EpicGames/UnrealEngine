// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageMultiCompose final : public ASTOp
	{
	public:

		ASTChild Layout;
		ASTChild Base;
		ASTChild SourceLayout;
		ASTChild SourceImage;
		//ASTChild Mask;
		
		/** Used to find the size of the image to create. */
		uint16 LayoutBlockSizeInPixelsX = 0;
		uint16 LayoutBlockSizeInPixelsY = 0;

		uint16 SourceSizeX = 0;
		uint16 SourceSizeY = 0;

	public:

		ASTOpImageMultiCompose();
		ASTOpImageMultiCompose(const ASTOpImageMultiCompose&) = delete;
		~ASTOpImageMultiCompose();

		virtual EOpType GetOpType() const override { return EOpType::IM_MULTICOMPOSE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		//virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const override;
		//virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		virtual void GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY) override;
		virtual bool IsImagePlainConstant(FVector4f& OutColor) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

