// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageMultiLayer.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	ASTOpImageMultiLayer::ASTOpImageMultiLayer()
		: base(this)
		, blend(this)
		, mask(this)
		, range(this, nullptr, FString(), FString())
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpImageMultiLayer::~ASTOpImageMultiLayer()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpImageMultiLayer::IsEqual(const ASTOp& InOtherUntyped) const
	{
		if (InOtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageMultiLayer* Other = static_cast<const ASTOpImageMultiLayer*>(&InOtherUntyped);
			return base == Other->base &&
				blend == Other->blend &&
				mask == Other->mask &&
				range == Other->range &&
				blendType == Other->blendType &&
				blendTypeAlpha == Other->blendTypeAlpha &&
				BlendAlphaSourceChannel == Other->BlendAlphaSourceChannel &&
				bUseMaskFromBlended == Other->bUseMaskFromBlended;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpImageMultiLayer::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(base));
		Result = HashCombineFast(Result, GetTypeHash(blend));
		Result = HashCombineFast(Result, GetTypeHash(mask));
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageMultiLayer::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpImageMultiLayer> n = new ASTOpImageMultiLayer();
		n->base = mapChild(base.child());
		n->blend = mapChild(blend.child());
		n->mask = mapChild(mask.child());
		n->range.rangeName = range.rangeName;
		n->range.rangeUID = range.rangeUID;
		n->range.rangeSize = mapChild(range.rangeSize.child());
		n->blendType = blendType;
		n->blendTypeAlpha = blendTypeAlpha;
		n->BlendAlphaSourceChannel = BlendAlphaSourceChannel;
		n->bUseMaskFromBlended = bUseMaskFromBlended;
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageMultiLayer::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(base);
		f(blend);
		f(mask);
		f(range.rangeSize);
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageMultiLayer::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageMultiLayerArgs Args;
			FMemory::Memzero(Args);

			Args.blendType = (uint8)blendType;
			Args.blendTypeAlpha = (uint8)blendTypeAlpha;
			Args.BlendAlphaSourceChannel = BlendAlphaSourceChannel;
			Args.bUseMaskFromBlended = bUseMaskFromBlended;

			if (base) 
			{
				Args.base = base->LinkedAddress;
			}

			if (blend) 
			{
				Args.blended = blend->LinkedAddress;
			}

			if (mask) 
			{
				Args.mask = mask->LinkedAddress;
			}

			if (range.rangeSize)
			{
				LinkRange(Program, range, Args.rangeSize, Args.rangeId);
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpImageMultiLayer::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const 
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (base)
		{
			res = base->GetImageDesc(returnBestOption, context);
		}


		// Cache the result
		if (context)
		{
			context->m_results.Add(this, res);
		}

		return res;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageMultiLayer::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (base)
		{
			base->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageMultiLayer::GetImageSizeExpression() const
	{
		if (base)
		{
			return base->GetImageSizeExpression();
		}

		return nullptr;
	}

}
