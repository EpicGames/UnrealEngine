// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpImageMultiLayer final : public ASTOp
	{
	public:

		ASTChild base;
		ASTChild blend;
		ASTChild mask;
		FRangeData range;

		/** Blend type used for the color channels. */
		EBlendType blendType = EBlendType::BT_NONE;

		/** Blend type used for the alpha channel if any. */
		EBlendType blendTypeAlpha = EBlendType::BT_NONE;

		/** Channel to use from the source color argument to apply blendTypeAlpha, if any. */
		uint8 BlendAlphaSourceChannel = 0;

		/** If true, use the alpha channel of the blended image as mask. Mask should be null.*/
		bool bUseMaskFromBlended = false;

	public:

		ASTOpImageMultiLayer();
		ASTOpImageMultiLayer(const ASTOpImageMultiLayer&) = delete;
		~ASTOpImageMultiLayer();

		EOpType GetOpType() const override { return EOpType::IM_MULTILAYER; }
		uint32 Hash() const override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		void GetLayoutBlockSize(int* pBlockX, int* pBlockY) override;
		Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
	};

}

