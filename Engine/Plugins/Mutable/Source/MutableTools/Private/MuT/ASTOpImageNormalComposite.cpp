// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageNormalComposite.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	ASTOpImageNormalComposite::ASTOpImageNormalComposite()
		: Base(this)
		, Normal(this)
		, Mode(ECompositeImageMode::CIM_Disabled)
		, Power(1.0f)
	{
	}


	ASTOpImageNormalComposite::~ASTOpImageNormalComposite()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageNormalComposite::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageNormalComposite* Other = static_cast<const ASTOpImageNormalComposite*>(&OtherUntyped);
			return Base == Other->Base 
				&& Normal == Other->Normal 
				&& Power == Other->Power 
				&& Mode == Other->Mode;
		}

		return false;
	}


	uint32 ASTOpImageNormalComposite::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Normal));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageNormalComposite::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpImageNormalComposite> N = new ASTOpImageNormalComposite();
		N->Base = mapChild(Base.child());
		N->Normal = mapChild(Normal.child());
		N->Mode = Mode;
		N->Power = Power;

		return N;
	}


	void ASTOpImageNormalComposite::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Base);
		f(Normal);
	}


	void ASTOpImageNormalComposite::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageNormalCompositeArgs Args;
			FMemory::Memzero(Args);

			if (Base)
			{
				Args.base = Base->LinkedAddress;
			}

			if (Normal)
			{
				Args.normal = Normal->LinkedAddress;
			}

			Args.power = Power;
			Args.mode = Mode;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FImageDesc ASTOpImageNormalComposite::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (Base)
		{
			res = Base->GetImageDesc(returnBestOption, context);
		}

		if (context)
		{
			context->m_results.Add(this, res);
		}

		return res;
	}

	void ASTOpImageNormalComposite::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (Base)
		{
			Base->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}

	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageNormalComposite::GetImageSizeExpression() const
	{
		if (Base)
		{
			return Base->GetImageSizeExpression();
		}

		return nullptr;
	}


	FSourceDataDescriptor ASTOpImageNormalComposite::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found = Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		if (Base)
		{
			FSourceDataDescriptor SourceDesc = Base->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		if (Normal)
		{
			FSourceDataDescriptor SourceDesc = Normal->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		Context->Cache.Add(this, Result);

		return Result;
	}

}
