// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;

	class ASTOpImageNormalComposite final : public ASTOp
	{
	public:
		ASTChild Base;
		ASTChild Normal;
		ECompositeImageMode Mode;
		float Power;

	public:

		ASTOpImageNormalComposite();
		ASTOpImageNormalComposite(const ASTOpImageNormalComposite&) = delete;
		~ASTOpImageNormalComposite();

		EOpType GetOpType() const override { return EOpType::IM_NORMALCOMPOSITE; }
		uint32 Hash() const override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		void GetLayoutBlockSize(int* pBlockX, int* pBlockY) override;
		Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}

