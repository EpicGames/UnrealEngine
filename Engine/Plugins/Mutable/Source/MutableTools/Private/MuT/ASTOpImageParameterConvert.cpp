// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageParameterConvert.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "MuR/Model.h"


namespace UE::Mutable::Private
{
	ASTOpImageParameterConvert::ASTOpImageParameterConvert() :
		ImageParameter(this)
	{
	}

	ASTOpImageParameterConvert::~ASTOpImageParameterConvert()
	{
		// Explicit call needed to avoid recursive destruction
		RemoveChildren();
	}


	void ASTOpImageParameterConvert::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(ImageParameter);
	}


	bool ASTOpImageParameterConvert::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpImageParameterConvert* Other = static_cast<const ASTOpImageParameterConvert*>(&OtherUntyped);
			return Type == Other->Type &&
				ImageParameter == Other->ImageParameter &&
				ImageDescriptor == Other->ImageDescriptor;
		}
		
		return false;
	}


	Ptr<ASTOp> ASTOpImageParameterConvert::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageParameterConvert> New = new ASTOpImageParameterConvert();
		New->Type = Type;
		New->ImageParameter =  ASTChild(New, MapChild(ImageParameter.child()));
		New->ImageDescriptor = ImageDescriptor;

		return New;
	}


	EOpType ASTOpImageParameterConvert::GetOpType() const 
	{ 
		return Type; 
	}


	uint32 ASTOpImageParameterConvert::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(ImageParameter));

		return Result;
	}


	void ASTOpImageParameterConvert::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::ImageParameterConvertArgs Args;
			FMemory::Memzero(Args);

			check(ImageParameter->LinkedAddress);
			Args.ImageParameter = ImageParameter->LinkedAddress;
		
			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

	
	FImageDesc ASTOpImageParameterConvert::GetImageDesc(bool, FGetImageDescContext* Context) const
	{
		if (Context)
		{
			Context->bIsFormatFromImageParameter = true;
		}

		return ImageDescriptor;
	}
}
