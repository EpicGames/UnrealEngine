// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;
	
	class ASTOpImageParameterConvert final : public ASTOp
	{
	public:
		EOpType Type = EOpType::IM_PARAMETER_CONVERT;

		ASTChild ImageParameter;

		FImageDesc ImageDescriptor;
		
		int32 LinkedParameterIndex = -1;

		ASTOpImageParameterConvert();
		virtual ~ASTOpImageParameterConvert() override;

		// ASTOp interface.
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool, FGetImageDescContext*) const override;
	};
}

