// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImagePatch.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpImagePatch::ASTOpImagePatch()
		: base(this)
		, patch(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpImagePatch::~ASTOpImagePatch()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpImagePatch::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImagePatch* other = static_cast<const ASTOpImagePatch*>(&otherUntyped);
			return base == other->base &&
				patch == other->patch &&
				location == other->location;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpImagePatch::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(base));
		Result = HashCombineFast(Result, GetTypeHash(patch));

		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpImagePatch::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpImagePatch> n = new ASTOpImagePatch();
		n->base = mapChild(base.child());
		n->patch = mapChild(patch.child());
		n->location = location;
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImagePatch::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(base);
		f(patch);
	}


	void ASTOpImagePatch::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImagePatchArgs Args;
			FMemory::Memzero(Args);

			if (base)
			{
				Args.base = base->LinkedAddress;
			}

			if (patch) 
			{
				Args.patch = patch->LinkedAddress;
			}

			Args.minX = location[0];
			Args.minY = location[1];

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpImagePatch::GetImageDesc(bool returnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (base)
		{
			Result = base->GetImageDesc(returnBestOption, Context);
		}

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImagePatch::GetImageSizeExpression() const
	{
		if (base)
		{
			return base->GetImageSizeExpression();
		}

		return nullptr;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImagePatch::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		// We didn't find any layout yet.
		*pBlockX = 0;
		*pBlockY = 0;

		// Try the source
		if (base)
		{
			base->GetLayoutBlockSize( pBlockX, pBlockY );
		}

		if (patch && *pBlockX == 0 && *pBlockY == 0)
		{
			patch->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	FSourceDataDescriptor ASTOpImagePatch::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found = Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		if (base)
		{
			FSourceDataDescriptor SourceDesc = base->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		if (patch)
		{
			FSourceDataDescriptor SourceDesc = patch->GetSourceDataDescriptor(Context);
			Result.CombineWith(SourceDesc);
		}

		Context->Cache.Add(this, Result);

		return Result;
	}

}
