// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImagePixelFormat.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImageConform.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageBlankLayout.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpImageInvert.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CompilerPrivate.h"

namespace UE::Mutable::Private
{

	template <class SCALAR> class vec4;

	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	ASTOpImagePixelFormat::ASTOpImagePixelFormat()
		: Source(this)
	{
	}


	ASTOpImagePixelFormat::~ASTOpImagePixelFormat()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImagePixelFormat::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImagePixelFormat* other = static_cast<const ASTOpImagePixelFormat*>(&otherUntyped);
			return Source == other->Source && Format == other->Format && FormatIfAlpha == other->FormatIfAlpha;
		}
		return false;
	}


	uint32 ASTOpImagePixelFormat::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Format));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImagePixelFormat::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> n = new ASTOpImagePixelFormat();
		n->Source = mapChild(Source.child());
		n->Format = Format;
		n->FormatIfAlpha = FormatIfAlpha;
		return n;
	}


	void ASTOpImagePixelFormat::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
	}


	void ASTOpImagePixelFormat::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImagePixelFormatArgs Args;
			FMemory::Memzero(Args);

			Args.format = Format;
			Args.formatIfAlpha = FormatIfAlpha;
			if (Source) 
			{
				Args.source = Source->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	Ptr<ASTOp> ASTOpImagePixelFormat::OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const
	{
		UE::Mutable::Private::Ptr<ASTOp> NewOp;

		// Skip this operation if the source op format if fixed and is already the one we want.
		if (Source)
		{
			FGetImageDescContext Context;
			FImageDesc SourceDesc = Source->GetImageDesc(false, &Context);

			const bool bIsFixedFormat = SourceDesc.m_format == SourceDesc.FormatIfAlpha || SourceDesc.FormatIfAlpha == EImageFormat::None;
			if (!Context.bIsFormatFromImageParameter && bIsFixedFormat && SourceDesc.m_format == Format)
			{
				NewOp = Source.child();
			}
		}

		return NewOp;
	}


	Ptr<ASTOp> ASTOpImagePixelFormat::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		Ptr<ASTOp> at;

		Ptr<ASTOp> sourceAt = Source.child();

		EImageFormat format = Format;
		bool bIsCompressedFormat = IsCompressedFormat(Format);
		//bool isBlockFormat = GetImageFormatData( format ).PixelsPerBlockX!=0;

		// The instruction can be sunk
		EOpType sourceType = sourceAt ? sourceAt->GetOpType() : EOpType::NONE;
		switch (sourceType)
		{
		case EOpType::IM_PIXELFORMAT:
		{
			// Keep only the top pixel format
			const ASTOpImagePixelFormat* typedSource = static_cast<const ASTOpImagePixelFormat*>(sourceAt.get());
			UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> formatOp = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(this);
			formatOp->Source = typedSource->Source.child();
			at = formatOp;
			break;
		}

		case EOpType::IM_DISPLACE:
		{
			// This op doesn't support compressed formats
			if (!bIsCompressedFormat)
			{
				UE::Mutable::Private::Ptr<ASTOpImageDisplace> newOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(sourceAt);

				UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> fop = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(this);
				fop->Source = newOp->Source.child();
				newOp->Source = fop;

				at = newOp;
			}
			break;
		}

		case EOpType::IM_INVERT:
		{
			// This op doesn't support compressed formats
			if (!bIsCompressedFormat)
			{
				UE::Mutable::Private::Ptr<ASTOpImageInvert> newOp = UE::Mutable::Private::Clone<ASTOpImageInvert>(sourceAt);

				UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> fop = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(this);
				fop->Source = newOp->Base.child();
				newOp->Base = fop;

				at = newOp;
			}
			break;
		}

		case EOpType::IM_RASTERMESH:
		{
			// This op doesn't support compressed formats
			if (!bIsCompressedFormat
				&&
				static_cast<const ASTOpImageRasterMesh*>(sourceAt.get())->image)
			{
				Ptr<ASTOpImageRasterMesh> newOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(sourceAt);

				UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> fop = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(this);
				fop->Source = newOp->image.child();
				newOp->image = fop;

				at = newOp;
			}
			break;
		}

		case EOpType::IM_BLANKLAYOUT:
		{
			// Just make sure the layout format is the right one and forget the op
			Ptr<ASTOpImageBlankLayout> NewOp = UE::Mutable::Private::Clone<ASTOpImageBlankLayout>(sourceAt);

			EImageFormat layoutFormat = NewOp->Format;
			if (FormatIfAlpha != EImageFormat::None
				&&
				GetImageFormatData(layoutFormat).Channels > 3)
			{
				format = FormatIfAlpha;
			}

			NewOp->Format = format;
			at = NewOp;
			break;
		}

		case EOpType::IM_PLAINCOLOR:
		{
			// Just make sure the format is the right one and forget the op
			Ptr<ASTOpImagePlainColor> NewOp = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(sourceAt);

			EImageFormat LayoutFormat = NewOp->Format;
			if (FormatIfAlpha != EImageFormat::None
				&&
				GetImageFormatData(LayoutFormat).Channels > 3)
			{
				format = FormatIfAlpha;
			}

			NewOp->Format = format;
			at = NewOp;
			break;
		}

		case EOpType::IM_CONFORM:
		{
			Ptr<ASTOpImageConform> NewOp = UE::Mutable::Private::Clone<ASTOpImageConform>(sourceAt);

			NewOp->ConformDescriptor.m_format = format;
			NewOp->ConformDescriptor.FormatIfAlpha = FormatIfAlpha;

			at = NewOp;
			break;
		}
		default:
		{

			at = context.ImagePixelFormatSinker.Apply(this);

			break;
		} // pixelformat source default

		}

		return at;
	}


	//!
	FImageDesc ASTOpImagePixelFormat::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		if (Source.child())
		{
			res = Source.child()->GetImageDesc(returnBestOption, context);
		}

		if (FormatIfAlpha != EImageFormat::None &&
			GetImageFormatData(res.m_format).Channels > 3)
		{
			res.m_format = FormatIfAlpha;
		}
		else
		{
			res.m_format = Format;
			check(res.m_format != EImageFormat::None);
		}

		if (FormatIfAlpha != EImageFormat::None &&
			res.FormatIfAlpha != EImageFormat::None)
		{
			res.FormatIfAlpha = FormatIfAlpha;
		}
		else
		{
			res.FormatIfAlpha = EImageFormat::None;
		}

		// Format overriden by current op.
		context->bIsFormatFromImageParameter = false;

		// Cache the result
		context->m_results.Add(this, res);

		return res;
	}


	void ASTOpImagePixelFormat::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (Source.child())
		{
			Source.child()->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	bool ASTOpImagePixelFormat::IsImagePlainConstant(FVector4f& color) const
	{
		bool res = false;
		if (Source.child())
		{
			Source.child()->IsImagePlainConstant(color);
		}
		return res;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImagePixelFormat::GetImageSizeExpression() const
	{
		UE::Mutable::Private::Ptr<ImageSizeExpression> pRes;

		if (Source.child())
		{
			pRes = Source.child()->GetImageSizeExpression();
		}
		else
		{
			pRes = new ImageSizeExpression;
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpImagePixelFormat::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_ImagePixelFormatAST::Apply(const ASTOpImagePixelFormat* root)
	{
		Root = root;
		OldToNew.Reset();

		check(root->GetOpType() == EOpType::IM_PIXELFORMAT);

		InitialSource = Root->Source.child();
		UE::Mutable::Private::Ptr<ASTOp> newSource = Visit(InitialSource, Root);

		Root = nullptr;

		// If there is any change, it is the new root.
		if (newSource != InitialSource)
		{
			return newSource;
		}

		return nullptr;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_ImagePixelFormatAST::Visit(UE::Mutable::Private::Ptr<ASTOp> at, const ASTOpImagePixelFormat* currentFormatOp)
	{
		if (!at) return nullptr;

		EImageFormat format = currentFormatOp->Format;
		bool bIsCompressedFormat = IsCompressedFormat(format);
		bool isBlockFormat = GetImageFormatData(format).PixelsPerBlockX != 0;

		// Already visited?
		const Ptr<ASTOp>* Cached = OldToNew.Find({ at,currentFormatOp });
		if (Cached)
		{
			return *Cached;
		}

		UE::Mutable::Private::Ptr<ASTOp> newAt = at;
		switch (at->GetOpType())
		{

		case EOpType::IM_CONDITIONAL:
		{
			// We move the op down the two paths
			Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
			newOp->yes = Visit(newOp->yes.child(), currentFormatOp);
			newOp->no = Visit(newOp->no.child(), currentFormatOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_SWITCH:
		{
			// We move the op down all the paths
			Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
			newOp->Default = Visit(newOp->Default.child(), currentFormatOp);
			for (ASTOpSwitch::FCase& c : newOp->Cases)
			{
				c.Branch = Visit(c.Branch.child(), currentFormatOp);
			}
			newAt = newOp;
			break;
		}

		case EOpType::IM_COMPOSE:
		{
			if (isBlockFormat)
			{
				// Since blocks can be resized at runtime anyway, push the format down and rely on reformatting on the fly if necessary.

				// We move the format down the two paths
				Ptr<ASTOpImageCompose> newOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(at);

				// TODO: We have to make sure we don't end up with two different formats if
				// there is an formatIfAlpha

				Ptr<ASTOp> baseOp = newOp->Base.child();
				newOp->Base = Visit(baseOp, currentFormatOp);

				Ptr<ASTOp> blockOp = newOp->BlockImage.child();
				newOp->BlockImage = Visit(blockOp, currentFormatOp);

				newAt = newOp;
			}

			break;
		}

		case EOpType::IM_MULTICOMPOSE:
		{
			if (isBlockFormat)
			{
				// Since blocks can be resized at runtime anyway, push the format down and rely on reformatting on the fly if necessary.

				// We move the format down the two paths
				Ptr<ASTOpImageMultiCompose> newOp = UE::Mutable::Private::Clone<ASTOpImageMultiCompose>(at);

				// TODO: We have to make sure we don't end up with two different formats if
				// there is an formatIfAlpha

				Ptr<ASTOp> BaseOp = newOp->Base.child();
				newOp->Base = Visit(BaseOp, currentFormatOp);

				// TODO: Do not force a format to the Source Image it's from a TextureParameter
				Ptr<ASTOp> SourceImageOp = newOp->SourceImage.child();
				newOp->SourceImage = Visit(SourceImageOp, currentFormatOp);

				newAt = newOp;
			}

			break;
		}

		case EOpType::IM_PATCH:
		{
			if (isBlockFormat)
			{
				// We move the format down the two paths
				Ptr<ASTOpImagePatch> newOp = UE::Mutable::Private::Clone<ASTOpImagePatch>(at);

				newOp->base = Visit(newOp->base.child(), currentFormatOp);
				newOp->patch = Visit(newOp->patch.child(), currentFormatOp);

				newAt = newOp;
			}

			break;
		}

		case EOpType::IM_MIPMAP:
		{
			const ASTOpImageMipmap* typedSource = static_cast<const ASTOpImageMipmap*>(at.get());

			// If its a compressed format, only sink formats on mipmap operations that
			// generate the tail. To avoid optimization loop.
			if (!bIsCompressedFormat || typedSource->bOnlyTail)
			{
				Ptr<ASTOpImageMipmap> newOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(typedSource);

				Ptr<ASTOp> baseOp = newOp->Source.child();
				newOp->Source = Visit(baseOp, currentFormatOp);

				newAt = newOp;
			}
			break;
		}

		case EOpType::IM_INTERPOLATE:
		{
			// This op doesn't support compressed formats
			if (!bIsCompressedFormat)
			{
				// Move the format down all the paths
				Ptr<ASTOpImageInterpolate> newOp = UE::Mutable::Private::Clone<ASTOpImageInterpolate>(at);

				for (int v = 0; v < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++v)
				{
					Ptr<ASTOp> Child = newOp->Targets[v].child();
					Ptr<ASTOp> FormattedChild = Visit(Child, currentFormatOp);
					newOp->Targets[v] = FormattedChild;
				}

				newAt = newOp;
			}
			break;
		}

		case EOpType::IM_LAYER:
		{
			if (GetOpToolsDesc(at->GetOpType()).bSupportedBasePixelFormats[(size_t)format])
			{
				// We move the format down the two paths
				Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(at);

				nop->base = Visit(nop->base.child(), currentFormatOp);
				nop->blend = Visit(nop->blend.child(), currentFormatOp);

				newAt = nop;
			}
			break;
		}

		case EOpType::IM_LAYERCOLOR:
		{
			if (GetOpToolsDesc(at->GetOpType()).bSupportedBasePixelFormats[(size_t)format])
			{
				// We move the format down the base
				Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(at);

				nop->base = Visit(nop->base.child(), currentFormatOp);

				newAt = nop;
			}
			break;
		}

		case EOpType::IM_CONFORM:
		{
			Ptr<ASTOpImageConform> NewOp = UE::Mutable::Private::Clone<ASTOpImageConform>(at);

			NewOp->ConformDescriptor.m_format = format;
			NewOp->ConformDescriptor.FormatIfAlpha = currentFormatOp->FormatIfAlpha;

			newAt = NewOp;
			break;
		}

		default:
			break;
		}

		// end on tree branch, replace with format
		if (at == newAt && at != InitialSource)
		{
			UE::Mutable::Private::Ptr<ASTOpImagePixelFormat> newOp = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(currentFormatOp);
			check(newOp->GetOpType() == EOpType::IM_PIXELFORMAT);

			newOp->Source = at;

			newAt = newOp;
		}

		OldToNew.Add({ at, currentFormatOp }, newAt);

		return newAt;
	}

}
