// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImagePlainColor.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageInterpolate.h"


namespace UE::Mutable::Private
{

	ASTOpImagePlainColor::ASTOpImagePlainColor()
		: Color(this)
	{
	}


	ASTOpImagePlainColor::~ASTOpImagePlainColor()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImagePlainColor::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImagePlainColor* other = static_cast<const ASTOpImagePlainColor*>(&InOther);
			return Color == other->Color &&
				Format == other->Format &&
				Size == other->Size &&
				LODs == other->LODs;
		}
		return false;
	}


	uint32 ASTOpImagePlainColor::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Color));
		Result = HashCombineFast(Result, GetTypeHash(Size));
		Result = HashCombineFast(Result, GetTypeHash(Format));
		Result = HashCombineFast(Result, GetTypeHash(LODs));

		return Result;
	}


	Ptr<ASTOp> ASTOpImagePlainColor::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImagePlainColor> New = new ASTOpImagePlainColor();
		New->Color = MapChild(Color.child());
		New->Format = Format;
		New->Size = Size;
		New->LODs = LODs;
		return New;
	}


	void ASTOpImagePlainColor::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Color);
	}


	void ASTOpImagePlainColor::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImagePlainColorArgs Args;
			FMemory::Memzero(Args);

			if (Color)
			{
				Args.Color = Color->LinkedAddress;
			}

			Args.Format = Format;
			Args.Size[0] = Size[0];
			Args.Size[1] = Size[1];
			Args.LODs = LODs;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FImageDesc ASTOpImagePlainColor::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		Result.m_format = Format;
		Result.m_size[0] = Size[0];
		Result.m_size[1] = Size[1];
		Result.m_lods = LODs;
		check(Result.m_format != EImageFormat::None);

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImagePlainColor::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> Res = new ImageSizeExpression;
		Res->type = ImageSizeExpression::ISET_CONSTANT;
		Res->size[0] = Size[0];
		Res->size[1] = Size[1];
		return Res;
	}


	void ASTOpImagePlainColor::GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY)
	{
		// We didn't find any layout yet.
		*OutBlockX = 0;
		*OutBlockY = 0;
	}


	bool ASTOpImagePlainColor::IsImagePlainConstant(FVector4f& OutColor) const
	{
		bool bResult = true;
		OutColor = FVector4f( 0, 0, 0, 1 );

		if (Color)
		{
			bResult = Color->IsColorConstant(OutColor);
		}

		return bResult;
	}


	FSourceDataDescriptor ASTOpImagePlainColor::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		return {};
	}

}
