// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageRasterMesh.h"

#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpMeshProject.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "Containers/Map.h"
#include "HAL/PlatformMath.h"


namespace UE::Mutable::Private
{


	ASTOpImageRasterMesh::ASTOpImageRasterMesh()
		: mesh(this)
		, image(this)
		, angleFadeProperties(this)
		, mask(this)
		, projector(this)
	{
		BlockId = FLayoutBlock::InvalidBlockId;
		LayoutIndex = -1;
		SizeX = SizeY = 0;
		SourceSizeX = SourceSizeY = 0;
		CropMinX = CropMinY = 0;
		UncroppedSizeX = UncroppedSizeY = 0;

		bIsRGBFadingEnabled = 1;
		bIsAlphaFadingEnabled = 1;
		SamplingMethod = ESamplingMethod::Point;
		MinFilterMethod = EMinFilterMethod::None;
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpImageRasterMesh::~ASTOpImageRasterMesh()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpImageRasterMesh::IsEqual(const ASTOp& InOtherUntyped) const
	{
		if (InOtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageRasterMesh* Other = static_cast<const ASTOpImageRasterMesh*>(&InOtherUntyped);
			return mesh == Other->mesh &&
				image == Other->image &&
				angleFadeProperties == Other->angleFadeProperties &&
				mask == Other->mask &&
				projector == Other->projector &&
				BlockId == Other->BlockId &&
				LayoutIndex == Other->LayoutIndex &&
				SizeX == Other->SizeX &&
				SizeY == Other->SizeY &&
				SourceSizeX == Other->SourceSizeX &&
				SourceSizeY == Other->SourceSizeY &&
				CropMinX == Other->CropMinX &&
				CropMinY == Other->CropMinY &&
				UncroppedSizeX == Other->UncroppedSizeX &&
				UncroppedSizeY == Other->UncroppedSizeY &&
				bIsRGBFadingEnabled == Other->bIsRGBFadingEnabled &&
				bIsAlphaFadingEnabled == Other->bIsAlphaFadingEnabled &&
				SamplingMethod == Other->SamplingMethod &&
				MinFilterMethod == Other->MinFilterMethod;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpImageRasterMesh::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(mesh));
		Result = HashCombineFast(Result, GetTypeHash(image));
		Result = HashCombineFast(Result, GetTypeHash(angleFadeProperties));
		Result = HashCombineFast(Result, GetTypeHash(mask));
		Result = HashCombineFast(Result, GetTypeHash(projector));
		Result = HashCombineFast(Result, GetTypeHash(BlockId));
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageRasterMesh::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpImageRasterMesh> n = new ASTOpImageRasterMesh();
		n->mesh = mapChild(mesh.child());
		n->image = mapChild(image.child());
		n->angleFadeProperties = mapChild(angleFadeProperties.child());
		n->mask = mapChild(mask.child());
		n->projector = mapChild(projector.child());
		n->BlockId = BlockId;
		n->LayoutIndex = LayoutIndex;
		n->SizeX = SizeX;
		n->SizeY = SizeY;
		n->CropMinX = CropMinX;
		n->CropMinY = CropMinY;
		n->UncroppedSizeX = UncroppedSizeX;
		n->UncroppedSizeY = UncroppedSizeY;
		n->SourceSizeX = SourceSizeX;
		n->SourceSizeY = SourceSizeY;
		n->bIsRGBFadingEnabled = bIsRGBFadingEnabled;
		n->bIsAlphaFadingEnabled = bIsAlphaFadingEnabled;
		n->SamplingMethod = SamplingMethod;
		n->MinFilterMethod = MinFilterMethod;
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageRasterMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(mesh);
		f(image);
		f(angleFadeProperties);
		f(mask);
		f(projector);
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageRasterMesh::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageRasterMeshArgs Args;
			FMemory::Memzero(Args);

			Args.BlockId = BlockId;
			Args.LayoutIndex = LayoutIndex;
			Args.sizeX = SizeX;
			Args.sizeY = SizeY;
			Args.SourceSizeX = SourceSizeX;
			Args.SourceSizeY = SourceSizeY;
			Args.CropMinX = CropMinX;
			Args.CropMinY = CropMinY;
			Args.UncroppedSizeX = UncroppedSizeX;
			Args.UncroppedSizeY = UncroppedSizeY;
			Args.bIsRGBFadingEnabled = bIsRGBFadingEnabled;
			Args.bIsAlphaFadingEnabled = bIsAlphaFadingEnabled;
			Args.SamplingMethod = static_cast<uint8>(SamplingMethod);
			Args.MinFilterMethod = static_cast<uint8>(MinFilterMethod);

			if (mesh) 
			{
				Args.mesh = mesh->LinkedAddress;
			}

			if (image) 
			{
				Args.image = image->LinkedAddress;
			}

			if (angleFadeProperties) 
			{
				Args.angleFadeProperties = angleFadeProperties->LinkedAddress;
			}

			if (mask) 
			{
				Args.mask = mask->LinkedAddress;
			}

			if (projector) 
			{
				Args.projector = projector->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpImageRasterMesh::GetImageDesc(bool returnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (image)
		{
			Result = image->GetImageDesc( returnBestOption, Context);
			Result.m_size[0] = SizeX;
			Result.m_size[1] = SizeY;
		}


		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	Ptr<ImageSizeExpression> ASTOpImageRasterMesh::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> pRes = new ImageSizeExpression;
		pRes->type = ImageSizeExpression::ISET_CONSTANT;
		pRes->size[0] = SizeX ? SizeX : 256;
		pRes->size[1] = SizeY ? SizeY : 256;
		return pRes;
	}


	//---------------------------------------------------------------------------------------------
	Ptr<ASTOp> ASTOpImageRasterMesh::OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const
	{
		Ptr<ASTOp> at;

		// TODO

		return at;
	}


	//-------------------------------------------------------------------------------------------------
	Ptr<ASTOp> ASTOpImageRasterMesh::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		Ptr<ASTOp> at;

		Ptr<ASTOp> OriginalAt = at;
		Ptr<ASTOp> sourceAt = mesh.child();
		Ptr<ASTOp> imageAt = image.child();

		EOpType sourceType = sourceAt->GetOpType();
		switch (sourceType)
		{

		case EOpType::ME_PROJECT:
		{
			// If we are rastering just the UV layout (to create a mask) we don't care about
			// mesh project operations, which modify only the positions.
			// This optimisation helps with states removing fake dependencies on projector
			// parameters that may be runtime.
			if (!imageAt)
			{
				// We remove the project from the raster children
				const ASTOpMeshProject* MeshProjectOp = static_cast<const ASTOpMeshProject*>(sourceAt.get());
				Ptr<ASTOpImageRasterMesh> NewOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
				NewOp->mesh = MeshProjectOp->Mesh.child();
				at = NewOp;
			}
			break;
		}

		case EOpType::ME_MORPH:
		{
			// TODO: should be sink only if no imageAt?
			const ASTOpMeshMorph* typedSource = static_cast<const ASTOpMeshMorph*>(sourceAt.get());
			Ptr<ASTOpImageRasterMesh> rasterOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
			rasterOp->mesh = typedSource->Base.child();
			at = rasterOp;
			break;
		}

		case EOpType::ME_ADDMETADATA:
		{
			// Ignore metadata
			const ASTOpMeshAddMetadata* typedSource = static_cast<const ASTOpMeshAddMetadata*>(sourceAt.get());
			Ptr<ASTOpImageRasterMesh> rasterOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
			rasterOp->mesh = typedSource->Source.child();
			at = rasterOp;
			break;
		}

		case EOpType::ME_CONDITIONAL:
		{
			auto nop = UE::Mutable::Private::Clone<ASTOpConditional>(sourceAt.get());
			nop->type = EOpType::IM_CONDITIONAL;

			Ptr<ASTOpImageRasterMesh> aOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
			aOp->mesh = nop->yes.child();
			nop->yes = aOp;

			Ptr<ASTOpImageRasterMesh> bOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
			bOp->mesh = nop->no.child();
			nop->no = bOp;

			at = nop;
			break;
		}

		case EOpType::ME_SWITCH:
		{
			// Make an image for every path
			auto nop = UE::Mutable::Private::Clone<ASTOpSwitch>(sourceAt.get());
			nop->Type = EOpType::IM_SWITCH;

			if (nop->Default)
			{
				Ptr<ASTOpImageRasterMesh> defOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
				defOp->mesh = nop->Default.child();
				nop->Default = defOp;
			}

			// We need to copy the options because we change them
			for (size_t o = 0; o < nop->Cases.Num(); ++o)
			{
				if (nop->Cases[o].Branch)
				{
					Ptr<ASTOpImageRasterMesh> bOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
					bOp->mesh = nop->Cases[o].Branch.child();
					nop->Cases[o].Branch = bOp;
				}
			}

			at = nop;
			break;
		}

		default:
			break;
		}

		// If we didn't optimize the mesh child, try to optimize the image child.
		if (OriginalAt == at && imageAt)
		{
			EOpType imageType = imageAt->GetOpType();
			switch (imageType)
			{

				// TODO: Implement for image conditionals.
				//case EOpType::ME_CONDITIONAL:
				//{
				//	auto nop = UE::Mutable::Private::Clone<ASTOpConditional>(sourceAt.get());
				//	nop->type = EOpType::IM_CONDITIONAL;

				//	Ptr<ASTOpFixed> aOp = UE::Mutable::Private::Clone<ASTOpFixed>(this);
				//	aOp->SetChild(aOp->op.Args.ImageRasterMesh.mesh, nop->yes);
				//	nop->yes = aOp;

				//	Ptr<ASTOpFixed> bOp = UE::Mutable::Private::Clone<ASTOpFixed>(this);
				//	bOp->SetChild(bOp->op.Args.ImageRasterMesh.mesh, nop->no);
				//	nop->no = bOp;

				//	at = nop;
				//	break;
				//}

			case EOpType::IM_SWITCH:
			{
				// TODO: Do this only if the projector is constant?

				// Make a project for every path
				auto nop = UE::Mutable::Private::Clone<ASTOpSwitch>(imageAt.get());

				if (nop->Default)
				{
					Ptr<ASTOpImageRasterMesh> defOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
					defOp->image = nop->Default.child();
					nop->Default = defOp;
				}

				// We need to copy the options because we change them
				for (size_t o = 0; o < nop->Cases.Num(); ++o)
				{
					if (nop->Cases[o].Branch)
					{
						Ptr<ASTOpImageRasterMesh> bOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(this);
						bOp->image = nop->Cases[o].Branch.child();
						nop->Cases[o].Branch = bOp;
					}
				}

				at = nop;
				break;
			}

			default:
				break;
			}
		}

		return at;
	}


	FSourceDataDescriptor ASTOpImageRasterMesh::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (image)
		{
			return image->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
