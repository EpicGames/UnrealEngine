// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageRasterMesh final : public ASTOp
	{
	public:

		ASTChild mesh;
		ASTChild image;
		ASTChild angleFadeProperties;
		ASTChild mask;
		ASTChild projector;

		/** If layouts are used, this can indicate a single layout block that we want to raster. */
		uint64 BlockId;
		int8 LayoutIndex;

		/** Size of the image to generate by rasterization of the mesh. */
		uint16 SizeX, SizeY;

		/** Expected size of the image that we want to project. */
		uint16 SourceSizeX, SourceSizeY;

		/** Sub-rect to raster, ignoring all the rest.
		 * Only valid if any UncroppedSizeX is greater than 0. 
		 */
		uint16 CropMinX, CropMinY;
		uint16 UncroppedSizeX, UncroppedSizeY;

		uint8 bIsRGBFadingEnabled : 1;
		uint8 bIsAlphaFadingEnabled : 1;
		ESamplingMethod SamplingMethod;
		EMinFilterMethod MinFilterMethod;

	public:

		ASTOpImageRasterMesh();
		ASTOpImageRasterMesh(const ASTOpImageRasterMesh&) = delete;
		~ASTOpImageRasterMesh();

		virtual EOpType GetOpType() const override { return EOpType::IM_RASTERMESH; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions& options, int32 Pass) const override;
		virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

