// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageResize.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageConform.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpImageInvert.h"


namespace UE::Mutable::Private
{

	ASTOpImageResize::ASTOpImageResize()
		: Source(this)
	{
	}


	ASTOpImageResize::~ASTOpImageResize()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageResize::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageResize* Other = static_cast<const ASTOpImageResize*>(&InOther);
			return Source == Other->Source &&
				Size == Other->Size;
		}
		return false;
	}


	uint32 ASTOpImageResize::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Size));

		return Result;
	}


	Ptr<ASTOp> ASTOpImageResize::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageResize> New = new ASTOpImageResize();
		New->Source = MapChild(Source.child());
		New->Size = Size;
		return New;
	}


	void ASTOpImageResize::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
	}


	void ASTOpImageResize::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageResizeArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}

			Args.Size[0] = Size[0];
			Args.Size[1] = Size[1];

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageResize::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Source)
		{
			Result = Source->GetImageDesc(bReturnBestOption, Context);
		}

		Result.m_size[0] = Size[0];
		Result.m_size[1] = Size[1];

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageResize::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> Res = new ImageSizeExpression;
		Res->type = ImageSizeExpression::ISET_CONSTANT;
		Res->size[0] = Size[0];
		Res->size[1] = Size[1];
		return Res;
	}


	void ASTOpImageResize::GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY)
	{
		// We didn't find any layout yet.
		*OutBlockX = 0;
		*OutBlockY = 0;

		// Try the source
		if (Source)
		{
			Source->GetLayoutBlockSize( OutBlockX, OutBlockY );

			if (*OutBlockX > 0 && *OutBlockY > 0)
			{
				FImageDesc SourceDesc = Source->GetImageDesc(false);
				if (SourceDesc.m_size[0] > 0 && SourceDesc.m_size[1] > 0)
				{
					float FactorX = float(Size[0]) / float(SourceDesc.m_size[0]);
					float FactorY = float(Size[1]) / float(SourceDesc.m_size[1]);
					*OutBlockX = int32(*OutBlockX * FactorX);
					*OutBlockY = int32(*OutBlockY * FactorY);
				}
				else
				{
					*OutBlockX = 0;
					*OutBlockY = 0;
				}
			}
		}
	}


	bool ASTOpImageResize::IsImagePlainConstant(FVector4f& OutColor) const
	{
		bool bResult = true;
		OutColor = FVector4f(0.0f,0.0f,0.0f,1.0f);

		if (Source)
		{
			bResult = Source->IsImagePlainConstant(OutColor);
		}

		return bResult;
	}


	FSourceDataDescriptor ASTOpImageResize::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	Ptr<ASTOp> ASTOpImageResize::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> SourceAt = Source.child();

		// The instruction can be sunk
		EOpType SourceType = SourceAt->GetOpType();
		switch (SourceType)
		{
		case EOpType::IM_PLAINCOLOR:
		{
			Ptr<ASTOpImagePlainColor> NewOp = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(SourceAt.get());
			NewOp->Size[0] = Size[0];
			NewOp->Size[1] = Size[1];
			NewOp->LODs = 1; // TODO
			Result = NewOp;
			break;
		}

		default:
			break;

		}

		return Result;
	}


	Ptr<ASTOp> ASTOpImageResize::OptimiseSize() const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> SourceAt = Source.child();

		EOpType SourceType = SourceAt->GetOpType();
		switch (SourceType)
		{
		case EOpType::IM_RESIZE:
		{
			// Keep top resize
			Ptr<const ASTOpImageResize> SourceOp = static_cast<const ASTOpImageResize*>(SourceAt.get());

			Ptr<ASTOpImageResize> NewOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			NewOp->Source = SourceOp->Source.child();

			Result = NewOp;
			break;
		}

		case EOpType::IM_CONFORM:
		{
			// Integrate the resize into the conform and remove resize. 
			Ptr<const ASTOpImageConform> SourceOp = static_cast<const ASTOpImageConform*>(SourceAt.get());

			Ptr<ASTOpImageConform> NewOp = UE::Mutable::Private::Clone<ASTOpImageConform>(SourceOp);
			NewOp->ConformDescriptor.m_size = FImageSize(Size[0], Size[1]);

			Result = NewOp;
			break;
		}

		case EOpType::IM_PLAINCOLOR:
		{
			// Set the size in the children and remove resize
			Ptr<ASTOpImagePlainColor> SourceOp = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(SourceAt.get());
			SourceOp->Size[0] = Size[0];
			SourceOp->Size[1] = Size[1];
			SourceOp->LODs = 1; // TODO
			Result = SourceOp;
			break;
		}

		case EOpType::IM_TRANSFORM:
		{
			// Set the size in the children and remove resize
			Ptr<ASTOpImageTransform> SourceOp = UE::Mutable::Private::Clone<ASTOpImageTransform>(SourceAt.get());
			SourceOp->SizeX = Size[0];
			SourceOp->SizeY = Size[1];
			Result = SourceOp;
			break;
		}

		case EOpType::IM_CONDITIONAL:
		{
			// We move the resize down the two paths
			Ptr<ASTOpConditional> NewOp = UE::Mutable::Private::Clone<ASTOpConditional>(SourceAt);

			Ptr<ASTOpImageResize> AOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			AOp->Source = NewOp->yes.child();
			NewOp->yes = AOp;

			Ptr<ASTOpImageResize> BOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			BOp->Source = NewOp->no.child();
			NewOp->no = BOp;

			Result = NewOp;
			break;
		}

		case EOpType::IM_SWITCH:
		{
			// Move the resize down all the paths
			Ptr<ASTOpSwitch> NewOp = UE::Mutable::Private::Clone<ASTOpSwitch>(SourceAt);

			if (NewOp->Default)
			{
				Ptr<ASTOpImageResize> DefOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				DefOp->Source = NewOp->Default.child();
				NewOp->Default = DefOp;
			}

			for (ASTOpSwitch::FCase& Case : NewOp->Cases)
			{
				if (Case.Branch)
				{
					Ptr<ASTOpImageResize> CaseOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
					CaseOp->Source = Case.Branch.child();
					Case.Branch = CaseOp;
				}
			}

			Result = NewOp;
			break;
		}

		case EOpType::IM_SWIZZLE:
		{
			Ptr<ASTOpImageSwizzle> NewOp = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(SourceAt);
			for (int32 ChannelIndex = 0; ChannelIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++ChannelIndex)
			{
				Ptr<ASTOp> OldChannelOp = NewOp->Sources[ChannelIndex].child();
				if (OldChannelOp)
				{
					Ptr<ASTOpImageResize> ChannelResize = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
					ChannelResize->Source = OldChannelOp;
					NewOp->Sources[ChannelIndex] = ChannelResize;
				}
			}
			Result = NewOp;
			break;
		}

		case EOpType::IM_COMPOSE:
		{
			Ptr<ASTOpImageCompose> NewOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(SourceAt);

			Ptr<ASTOpImageResize> BaseOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			BaseOp->Source = NewOp->Base.child();
			NewOp->Base = BaseOp;

			Ptr<ASTOpImageResize> BlockOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			BlockOp->Source = NewOp->BlockImage.child();
			NewOp->BlockImage = BlockOp;

			if (NewOp->Mask)
			{
				Ptr<ASTOpImageResize> MaskOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				MaskOp->Source = NewOp->Mask.child();
				NewOp->Mask = MaskOp;
			}

			Result = NewOp;
			break;
		}

		case EOpType::IM_MULTICOMPOSE:
		{
			Ptr<ASTOpImageMultiCompose> NewOp = UE::Mutable::Private::Clone<ASTOpImageMultiCompose>(SourceAt);

			Ptr<ASTOpImageResize> BaseOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			BaseOp->Source = NewOp->Base.child();
			NewOp->Base = BaseOp;

			Ptr<ASTOpImageResize> SourceImageOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			SourceImageOp->Source = NewOp->SourceImage.child();
			NewOp->SourceImage = SourceImageOp;

			//if (NewOp->Mask)
			//{
			//	Ptr<ASTOpImageResize> MaskOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			//	MaskOp->Source = NewOp->Mask.child();
			//	NewOp->Mask = MaskOp;
			//}

			Result = NewOp;
			break;
		}

		case EOpType::IM_RASTERMESH:
		{
			Ptr<ASTOpImageRasterMesh> NewOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(SourceAt);

			//if ( NewOp->op.args.ImageRasterMesh.sizeX != op.args.ImageResize.size[0]
			//     ||
			//     NewOp->op.args.ImageRasterMesh.sizeY != op.args.ImageResize.size[1] )
			{
				NewOp->SizeX = Size[0];
				NewOp->SizeY = Size[1];

				if (NewOp->mask)
				{
					Ptr<ASTOpImageResize> MaskOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
					MaskOp->Source = NewOp->mask.child();
					NewOp->mask = MaskOp;
				}

				// Don't apply absolute resizes to the image to raster: it could even enlarge it.
				// This should only be scaled with relative resizes, which come from LOD biases, etc.
				//if (NewOp->op.args.ImageRasterMesh.image)
				//{
				//	Ptr<ASTOpImageResize> mop = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				//	mop->SetChild(mop->op.args.ImageResize.source, NewOp->children[NewOp->op.args.ImageRasterMesh.image].child());
				//	NewOp->SetChild(NewOp->op.args.ImageRasterMesh.image, mop);
				//}
			}

			Result = NewOp;
			break;
		}

		case EOpType::IM_INTERPOLATE:
		{
			Ptr<ASTOpImageInterpolate> NewOp = UE::Mutable::Private::Clone<ASTOpImageInterpolate>(SourceAt);

			for (int32 TargetIndex = 0; TargetIndex < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++TargetIndex)
			{
				Ptr<ASTOp> TargetAt = NewOp->Targets[TargetIndex].child();
				if (TargetAt)
				{
					Ptr<ASTOpImageResize> SourceOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
					SourceOp->Source = TargetAt;
					NewOp->Targets[TargetIndex] = SourceOp;
				}
			}

			Result = NewOp;
			break;
		}

		case EOpType::IM_INVERT:
		{
			Ptr<ASTOpImageInvert> NewOp = UE::Mutable::Private::Clone<ASTOpImageInvert>(SourceAt);
			Ptr<ASTOp> BaseAt = NewOp->Base.child();

			Ptr<ASTOpImageResize> NewBase = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			NewBase->Source = BaseAt;

			NewOp->Base = NewBase;

			Result = NewOp;
			break;
		}

		case EOpType::IM_PIXELFORMAT:
		{
			// \todo: only if shrinking?

			// Only sink the resize if we know that the pixelformat source image is uncompressed.
			Ptr<ASTOpImagePixelFormat> SourceTyped = static_cast<ASTOpImagePixelFormat*>(SourceAt.get());
			FImageDesc PixelFormatSourceDesc = SourceTyped->Source->GetImageDesc();
			if (PixelFormatSourceDesc.m_format != EImageFormat::None
				&&
				!UE::Mutable::Private::IsCompressedFormat(PixelFormatSourceDesc.m_format))
			{
				Ptr<ASTOpImagePixelFormat> NewOp = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(SourceAt);
				Ptr<ASTOp> BaseAt = NewOp->Source.child();

				Ptr<ASTOpImageResize> NewBase = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				NewBase->Source = BaseAt;

				NewOp->Source = NewBase;

				Result = NewOp;
			}
			break;
		}

		case EOpType::IM_LAYER:
		{
			Ptr<ASTOpImageLayer> NewOp = UE::Mutable::Private::Clone<ASTOpImageLayer>(SourceAt);

			Ptr<ASTOpImageResize> BaseOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			BaseOp->Source = NewOp->base.child();
			NewOp->base = BaseOp;

			Ptr<ASTOpImageResize> blendOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			blendOp->Source = NewOp->blend.child();
			NewOp->blend = blendOp;

			Ptr<ASTOp> MaskAt = NewOp->mask.child();
			if (MaskAt)
			{
				Ptr<ASTOpImageResize> ResizedMaskOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				ResizedMaskOp->Source = MaskAt;
				NewOp->mask = ResizedMaskOp;
			}

			Result = NewOp;
			break;
		}

		case EOpType::IM_LAYERCOLOR:
		{
			Ptr<ASTOpImageLayerColor> NewOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(SourceAt);

			Ptr<ASTOpImageResize> BaseOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
			BaseOp->Source = NewOp->base.child();
			NewOp->base = BaseOp;

			Ptr<ASTOp> MaskAt = NewOp->mask.child();
			if (MaskAt)
			{
				Ptr<ASTOpImageResize> ResizedMaskOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				ResizedMaskOp->Source = MaskAt;
				NewOp->mask = ResizedMaskOp;
			}

			Result = NewOp;
			break;
		}

		case EOpType::IM_DISPLACE:
		{
			// In the size optimization phase we can optimize the resize with the displace
			// because the constants have not been collapsed yet.
			// We will still check it and sink the size directly below the IM_MAKEGROWMAP op
			Ptr<ASTOpImageDisplace> SourceTyped = static_cast<ASTOpImageDisplace*>(SourceAt.get());
			Ptr<ASTOp> OriginalDisplacementMapOp = SourceTyped->DisplacementMap.child();
			if (OriginalDisplacementMapOp->GetOpType() == EOpType::IM_MAKEGROWMAP)
			{
				Ptr<ASTOpImageDisplace> NewOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(SourceAt);

				Ptr<ASTOpImageResize> BaseOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				BaseOp->Source = NewOp->Source.child();
				NewOp->Source = BaseOp;

				// Clone the map op and replace its children
				Ptr<ASTOpImageMakeGrowMap> MapOp = UE::Mutable::Private::Clone<ASTOpImageMakeGrowMap>(OriginalDisplacementMapOp);
				NewOp->DisplacementMap = MapOp;

				Ptr<ASTOpImageResize> MapSourceOp = UE::Mutable::Private::Clone<ASTOpImageResize>(this);
				MapSourceOp->Source = MapOp->Mask.child();
				MapOp->Mask = MapSourceOp;

				Result = NewOp;
			}
			break;
		}

		default:
			break;
		}

		return Result;
	}

}
