// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageResize final : public ASTOp
	{
	public:

		ASTChild Source;
		UE::Math::TIntVector2<uint16> Size = UE::Math::TIntVector2<uint16>(0, 0);

	public:

		ASTOpImageResize();
		ASTOpImageResize(const ASTOpImageResize&) = delete;
		~ASTOpImageResize();

		virtual EOpType GetOpType() const override { return EOpType::IM_RESIZE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool bReturnBestOption, FGetImageDescContext*) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual void GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY) override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;
		virtual Ptr<ASTOp> OptimiseSize() const override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};


}

