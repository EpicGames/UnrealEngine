// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageResizeLike.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

	ASTOpImageResizeLike::ASTOpImageResizeLike()
		: Source(this),
		SizeSource(this)
	{
	}


	ASTOpImageResizeLike::~ASTOpImageResizeLike()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageResizeLike::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageResizeLike* Other = static_cast<const ASTOpImageResizeLike*>(&InOther);
			return Source == Other->Source &&
				SizeSource == Other->SizeSource;
		}
		return false;
	}


	uint32 ASTOpImageResizeLike::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(SizeSource));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpImageResizeLike::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageResizeLike> New = new ASTOpImageResizeLike();
		New->Source = MapChild(Source.child());
		New->SizeSource = MapChild(SizeSource.child());
		return New;
	}


	void ASTOpImageResizeLike::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
		Func(SizeSource);
	}


	void ASTOpImageResizeLike::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageResizeLikeArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}

			if (SizeSource)
			{
				Args.SizeSource = SizeSource->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageResizeLike::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Source)
		{
			Result = Source->GetImageDesc(bReturnBestOption, Context);
		}

		if (SizeSource)
		{
			FImageDesc SizeResult = SizeSource->GetImageDesc(bReturnBestOption, Context);
			Result.m_size = SizeResult.m_size;
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageResizeLike::GetImageSizeExpression() const
	{
		if (SizeSource)
		{
			return SizeSource->GetImageSizeExpression();
		}

		return new ImageSizeExpression;
	}


	bool ASTOpImageResizeLike::IsImagePlainConstant(FVector4f& OutColor) const
	{
		bool bResult = true;
		OutColor = FVector4f(0.0f,0.0f,0.0f,1.0f);

		if (Source)
		{
			bResult = Source->IsImagePlainConstant(OutColor);
		}

		return bResult;
	}


	FSourceDataDescriptor ASTOpImageResizeLike::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	Ptr<ASTOp> ASTOpImageResizeLike::OptimiseSize() const
	{
		Ptr<ASTOp> Result;

		if (!Source || !SizeSource)
		{
			return Result;
		}

		Ptr<ImageSizeExpression> SourceSize = Source->GetImageSizeExpression();
		Ptr<ImageSizeExpression> SizeSourceSize = SizeSource->GetImageSizeExpression();

		if (*SourceSize == *SizeSourceSize)
		{
			Result = Source.child();
		}

		else if (SizeSourceSize->type == ImageSizeExpression::ISET_CONSTANT)
		{
			Ptr<ASTOpImageResize> NewOp = new ASTOpImageResize;
			NewOp->Source = Source.child();
			NewOp->Size[0] = SizeSourceSize->size[0];
			NewOp->Size[1] = SizeSourceSize->size[1];
			Result = NewOp;
		}
		else if (SizeSourceSize->type == ImageSizeExpression::ISET_LAYOUTFACTOR)
		{
			// TODO
			// Skip intermediate ops until the layout
//                    if ( Program.m_code[sizeSourceAt].type != EOpType::IM_BLANKLAYOUT )
//                    {
//                        m_modified = true;

//                        OP blackLayoutOp;
//                        blackLayoutOp.type = EOpType::IM_BLANKLAYOUT;
//                        blackLayoutOp.args.ImageBlankLayout.layout = sizeSourceSize->layout;
//                        blackLayoutOp.args.ImageBlankLayout.blockSize[0] = sizeSourceSize->factor[0];
//                        blackLayoutOp.args.ImageBlankLayout.blockSize[1] = sizeSourceSize->factor[1];
//                        blackLayoutOp.args.ImageBlankLayout.format = L_UByte;

//                        OP opResize = Program.m_code[at];
//                        opResize.args.ImageResizeLike.sizeSource = Program.AddOp( blackLayoutOp );
//                        Result = Program.AddOp( opResize );
//                    }
		}

		return Result;
	}

}
