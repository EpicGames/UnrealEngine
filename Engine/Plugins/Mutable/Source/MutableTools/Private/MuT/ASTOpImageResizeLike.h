// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageResizeLike final : public ASTOp
	{
	public:

		ASTChild Source;
		ASTChild SizeSource;

	public:

		ASTOpImageResizeLike();
		ASTOpImageResizeLike(const ASTOpImageResizeLike&) = delete;
		~ASTOpImageResizeLike();

		virtual EOpType GetOpType() const override { return EOpType::IM_RESIZELIKE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool bReturnBestOption, FGetImageDescContext*) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;
		virtual Ptr<ASTOp> OptimiseSize() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};


}

