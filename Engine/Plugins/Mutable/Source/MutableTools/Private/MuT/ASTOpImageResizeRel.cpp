// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageResizeRel.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageConform.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImageBlankLayout.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImageSaturate.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpImageInvert.h"


namespace UE::Mutable::Private
{

	ASTOpImageResizeRel::ASTOpImageResizeRel()
		: Source(this)
	{
	}


	ASTOpImageResizeRel::~ASTOpImageResizeRel()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageResizeRel::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageResizeRel* Other = static_cast<const ASTOpImageResizeRel*>(&InOther);
			return Source == Other->Source &&
				Factor == Other->Factor;
		}
		return false;
	}


	uint32 ASTOpImageResizeRel::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Factor));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpImageResizeRel::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageResizeRel> New = new ASTOpImageResizeRel();
		New->Source = MapChild(Source.child());
		New->Factor = Factor;
		return New;
	}


	void ASTOpImageResizeRel::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
	}


	void ASTOpImageResizeRel::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageResizeRelArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}

			Args.Factor[0] = Factor[0];
			Args.Factor[1] = Factor[1];

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageResizeRel::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Source)
		{
			Result = Source->GetImageDesc(bReturnBestOption, Context);
		}

		Result.m_size[0] = uint16(Result.m_size[0] * Factor[0]);
		Result.m_size[1] = uint16(Result.m_size[1] * Factor[1]);

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageResizeRel::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> Res;

		if (Source)
		{
			Res = Source->GetImageSizeExpression();
			if (Res->type == ImageSizeExpression::ISET_CONSTANT)
			{
				Res->size[0] = uint16(Res->size[0] * Factor[0]);
				Res->size[1] = uint16(Res->size[1] * Factor[1]);
			}
			else
			{
				// TODO: Proportional factor
				Res = new ImageSizeExpression();
				Res->type = ImageSizeExpression::ISET_UNKNOWN;
			}
		}
		else
		{
			Res = new ImageSizeExpression;
		}

		return Res;
	}


	void ASTOpImageResizeRel::GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY)
	{
		// We didn't find any layout yet.
		*OutBlockX = 0;
		*OutBlockY = 0;

		// Try the source
		if (Source)
		{
			Source->GetLayoutBlockSize( OutBlockX, OutBlockY );

			if (*OutBlockX > 0 && *OutBlockY > 0)
			{
				FImageDesc SourceDesc = Source->GetImageDesc(false);
				if (SourceDesc.m_size[0] > 0 && SourceDesc.m_size[1] > 0)
				{
					*OutBlockX = int32(*OutBlockX * Factor[0]);
					*OutBlockY = int32(*OutBlockY * Factor[1]);
				}
				else
				{
					*OutBlockX = 0;
					*OutBlockY = 0;
				}
			}
		}
	}


	bool ASTOpImageResizeRel::IsImagePlainConstant(FVector4f& OutColor) const
	{
		bool bResult = true;
		OutColor = FVector4f(0.0f,0.0f,0.0f,1.0f);

		if (Source)
		{
			bResult = Source->IsImagePlainConstant(OutColor);
		}

		return bResult;
	}


	FSourceDataDescriptor ASTOpImageResizeRel::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	Ptr<ASTOp> ASTOpImageResizeRel::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> SourceAt = Source.child();

		// The instruction can be sunk
		EOpType SourceType = SourceAt->GetOpType();
		switch (SourceType)
		{
		case EOpType::IM_RESIZE:
		{
			Ptr<ASTOpImageResize> NewOp = UE::Mutable::Private::Clone<ASTOpImageResize>(SourceAt.get());
			NewOp->Size[0] = int16(NewOp->Size[0] * Factor[0]);
			NewOp->Size[1] = int16(NewOp->Size[1] * Factor[1]);

			Result = NewOp;
			break;
		}
		case EOpType::IM_CONFORM:
		{
			// The optimization can only be placed if we know the size.
			// TODO: implement the sinker visitor optimization.
			Ptr<ASTOpImageConform> NewOp = UE::Mutable::Private::Clone<ASTOpImageConform>(SourceAt.get());

			if (NewOp->ConformDescriptor.m_size != FImageSize(0,0))
			{
				FImageSize ConformSize = NewOp->ConformDescriptor.m_size;
				NewOp->ConformDescriptor.m_size = FImageSize(int16(ConformSize.X * Factor[0]), int16(ConformSize.Y * Factor[1]));

				Result = NewOp;
			}
			break;
		}


		default:
			break;

		}

		return Result;
	}


	class Sink_ImageResizeRelAST
	{
	public:

		Ptr<ASTOp> Apply(const ASTOpImageResizeRel* InRoot)
		{
			Root = InRoot;
			OldToNew.Empty();

			InitialSource = InRoot->Source.child();
			Ptr<ASTOp> newSource = Visit(InitialSource, InRoot);

			// If there is any change, it is the new root.
			if (newSource != InitialSource)
			{
				return newSource;
			}

			return nullptr;
		}

	protected:

		const ASTOp* Root;
		Ptr<ASTOp> InitialSource;
		//! For each operation we sink, the map from old instructions to new instructions.
		TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

		Ptr<ASTOp> Visit(Ptr<ASTOp> at, const ASTOpImageResizeRel* CurrentSinkingOp)
		{
			if (!at) return nullptr;

			// Already visited?
			const Ptr<ASTOp>* Cached = OldToNew.Find({ at, CurrentSinkingOp });
			if (Cached)
			{
				return *Cached;
			}

			float ScaleX = CurrentSinkingOp->Factor[0];
			float ScaleY = CurrentSinkingOp->Factor[1];

			Ptr<ASTOp> Result = at;
			switch (at->GetOpType())
			{

			case EOpType::IM_CONDITIONAL:
			{
				// We move the mask creation down the two paths
				auto NewOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
				NewOp->yes = Visit(NewOp->yes.child(), CurrentSinkingOp);
				NewOp->no = Visit(NewOp->no.child(), CurrentSinkingOp);
				Result = NewOp;
				break;
			}

			case EOpType::IM_PIXELFORMAT:
			{
				// \todo: only if shrinking?
				Ptr<ASTOpImagePixelFormat> NewOp = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(at);
				NewOp->Source = Visit(NewOp->Source.child(), CurrentSinkingOp);
				Result = NewOp;
				break;
			}

			case EOpType::IM_SWITCH:
			{
				// We move the mask creation down all the paths
				Ptr<ASTOpSwitch> NewOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
				NewOp->Default = Visit(NewOp->Default.child(), CurrentSinkingOp);
				for (ASTOpSwitch::FCase& c : NewOp->Cases)
				{
					c.Branch = Visit(c.Branch.child(), CurrentSinkingOp);
				}
				Result = NewOp;
				break;
			}

			case EOpType::IM_SWIZZLE:
			{
				Ptr<ASTOpImageSwizzle> NewOp = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(at);
				for (int32 s = 0; s < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++s)
				{
					Ptr<ASTOp> channelOp = NewOp->Sources[s].child();
					if (channelOp)
					{
						NewOp->Sources[s] = Visit(channelOp, CurrentSinkingOp);
					}
				}
				Result = NewOp;
				break;
			}

			case EOpType::IM_COMPOSE:
			{
				// We can only optimise if the layout grid blocks size in pixels is
				// still an integer after relative scale
				bool acceptable = false;
				{
					const ASTOpImageCompose* typedAt = static_cast<const ASTOpImageCompose*>(at.get());
					Ptr<ASTOp> originalBaseOp = typedAt->Base.child();

					// \todo: recursion-proof cache?
					int32 layoutBlockPixelsX = 0;
					int32 layoutBlockPixelsY = 0;
					originalBaseOp->GetLayoutBlockSize(&layoutBlockPixelsX, &layoutBlockPixelsY);

					int32 scaledLayoutBlockPixelsX = int32(layoutBlockPixelsX * ScaleX);
					int32 scaledLayoutBlockPixelsY = int32(layoutBlockPixelsY * ScaleY);
					int32 unscaledLayoutBlockPixelsX = int32(scaledLayoutBlockPixelsX / ScaleX);
					int32 unscaledLayoutBlockPixelsY = int32(scaledLayoutBlockPixelsY / ScaleY);
					acceptable =
						(layoutBlockPixelsX != 0 && layoutBlockPixelsY != 0)
						&&
						(layoutBlockPixelsX == unscaledLayoutBlockPixelsX)
						&&
						(layoutBlockPixelsY == unscaledLayoutBlockPixelsY);
				}

				if (acceptable)
				{
					Ptr<ASTOpImageCompose> NewOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(at);

					Ptr<ASTOp> baseOp = NewOp->Base.child();
					NewOp->Base = Visit(baseOp, CurrentSinkingOp);

					Ptr<ASTOp> blockOp = NewOp->BlockImage.child();
					NewOp->BlockImage = Visit(blockOp, CurrentSinkingOp);

					Result = NewOp;
				}

				break;
			}

			case EOpType::IM_MULTICOMPOSE:
			{
				// We can only optimise if the layout grid blocks size in pixels is
				// still an integer after relative scale
				bool bIsAcceptable = false;
				{
					const ASTOpImageMultiCompose* TypedAt = static_cast<const ASTOpImageMultiCompose*>(at.get());
					Ptr<ASTOp> OriginalBaseOp = TypedAt->Base.child();

					// \todo: recursion-proof cache?
					int32 LayoutBlockPixelsX = 0;
					int32 LayoutBlockPixelsY = 0;
					OriginalBaseOp->GetLayoutBlockSize(&LayoutBlockPixelsX, &LayoutBlockPixelsY);

					int32 ScaledLayoutBlockPixelsX = int32(LayoutBlockPixelsX * ScaleX);
					int32 ScaledLayoutBlockPixelsY = int32(LayoutBlockPixelsY * ScaleY);
					int32 UnscaledLayoutBlockPixelsX = int32(ScaledLayoutBlockPixelsX / ScaleX);
					int32 UnscaledLayoutBlockPixelsY = int32(ScaledLayoutBlockPixelsY / ScaleY);
					bIsAcceptable =
						(LayoutBlockPixelsX != 0 && LayoutBlockPixelsY != 0)
						&&
						(LayoutBlockPixelsX == UnscaledLayoutBlockPixelsX)
						&&
						(LayoutBlockPixelsY == UnscaledLayoutBlockPixelsY);
				}

				if (bIsAcceptable)
				{
					Ptr<ASTOpImageMultiCompose> NewOp = UE::Mutable::Private::Clone<ASTOpImageMultiCompose>(at);

					Ptr<ASTOp> baseOp = NewOp->Base.child();
					NewOp->Base = Visit(baseOp, CurrentSinkingOp);

					Ptr<ASTOp> SourceImageOp = NewOp->SourceImage.child();
					NewOp->SourceImage = Visit(SourceImageOp, CurrentSinkingOp);

					Result = NewOp;
				}

				break;
			}

			case EOpType::IM_PATCH:
			{
				Ptr<ASTOpImagePatch> NewOp = UE::Mutable::Private::Clone<ASTOpImagePatch>(at);

				NewOp->base = Visit(NewOp->base.child(), CurrentSinkingOp);
				NewOp->patch = Visit(NewOp->patch.child(), CurrentSinkingOp);

				// todo: review if this is always correct, or we need some "divisible" check
				NewOp->location[0] = uint16(NewOp->location[0] * ScaleX);
				NewOp->location[1] = uint16(NewOp->location[1] * ScaleY);

				Result = NewOp;

				break;
			}

			case EOpType::IM_MIPMAP:
			{
				Ptr<ASTOpImageMipmap> NewOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(at.get());
				Ptr<ASTOp> baseOp = NewOp->Source.child();
				NewOp->Source = Visit(baseOp, CurrentSinkingOp);
				Result = NewOp;
				break;
			}

			case EOpType::IM_INTERPOLATE:
			{
				Ptr<ASTOpImageInterpolate> NewOp = UE::Mutable::Private::Clone<ASTOpImageInterpolate>(at);

				for (int32 v = 0; v < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++v)
				{
					Ptr<ASTOp> Child = NewOp->Targets[v].child();
					Ptr<ASTOp> bOp = Visit(Child, CurrentSinkingOp);
					NewOp->Targets[v] = bOp;
				}

				Result = NewOp;
				break;
			}

			case EOpType::IM_MULTILAYER:
			{
				Ptr<ASTOpImageMultiLayer> nop = UE::Mutable::Private::Clone<ASTOpImageMultiLayer>(at);
				nop->base = Visit(nop->base.child(), CurrentSinkingOp);
				nop->mask = Visit(nop->mask.child(), CurrentSinkingOp);
				nop->blend = Visit(nop->blend.child(), CurrentSinkingOp);
				Result = nop;
				break;
			}

			case EOpType::IM_LAYER:
			{
				Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(at);

				Ptr<ASTOp> aOp = nop->base.child();
				nop->base = Visit(aOp, CurrentSinkingOp);

				Ptr<ASTOp> bOp = nop->blend.child();
				nop->blend = Visit(bOp, CurrentSinkingOp);

				Ptr<ASTOp> mOp = nop->mask.child();
				nop->mask = Visit(mOp, CurrentSinkingOp);

				Result = nop;
				break;
			}

			case EOpType::IM_LAYERCOLOR:
			{
				Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(at);

				Ptr<ASTOp> aOp = nop->base.child();
				nop->base = Visit(aOp, CurrentSinkingOp);

				Ptr<ASTOp> mOp = nop->mask.child();
				nop->mask = Visit(mOp, CurrentSinkingOp);

				Result = nop;
				break;
			}

			case EOpType::IM_DISPLACE:
			{
				Ptr<ASTOpImageDisplace> nop = UE::Mutable::Private::Clone<ASTOpImageDisplace>(at);

				Ptr<ASTOp> sourceOp = nop->Source.child();
				nop->Source = Visit(sourceOp, CurrentSinkingOp);

				Ptr<ASTOp> mapOp = nop->DisplacementMap.child();
				nop->DisplacementMap = Visit(mapOp, CurrentSinkingOp);

				// Make sure we don't scale a constant displacement map, which is very wrong.
				if (mapOp->GetOpType() == EOpType::IM_MAKEGROWMAP)
				{
					Result = nop;
				}
				else
				{
					// We cannot resize an already calculated displacement map.
				}

				break;
			}

			case EOpType::IM_MAKEGROWMAP:
			{
				Ptr<ASTOpImageMakeGrowMap> nop = UE::Mutable::Private::Clone<ASTOpImageMakeGrowMap>(at);
				Ptr<ASTOp> maskOp = nop->Mask.child();
				nop->Mask = Visit(maskOp, CurrentSinkingOp);
				Result = nop;
				break;
			}

			case EOpType::IM_INVERT:
			{
				Ptr<ASTOpImageInvert> nop = UE::Mutable::Private::Clone<ASTOpImageInvert>(at);
				Ptr<ASTOp> maskOp = nop->Base.child();
				nop->Base = Visit(maskOp, CurrentSinkingOp);
				Result = nop;
				break;
			}

			case EOpType::IM_SATURATE:
			{
				Ptr<ASTOpImageSaturate> NewOp = UE::Mutable::Private::Clone<ASTOpImageSaturate>(at);
				NewOp->Base = Visit(NewOp->Base.child(), CurrentSinkingOp);
				Result = NewOp;
				break;
			}

			case EOpType::IM_TRANSFORM:
			{
				// It can only sink in the transform if it doesn't have it's own size.
				const ASTOpImageTransform* TypedAt = static_cast<const ASTOpImageTransform*>(at.get());
				if (TypedAt->SizeX == 0 && TypedAt->SizeY == 0)
				{
					Ptr<ASTOpImageTransform> NewOp = UE::Mutable::Private::Clone<ASTOpImageTransform>(at);
					Ptr<ASTOp> MaskOp = NewOp->Base.child();
					NewOp->Base = Visit(MaskOp, CurrentSinkingOp);
					Result = NewOp;
				}

				break;
			}

			case EOpType::IM_RASTERMESH:
			{
				Ptr<ASTOpImageRasterMesh> nop = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(at);
				Ptr<ASTOp> maskOp = nop->mask.child();
				nop->mask = Visit(maskOp, CurrentSinkingOp);

				// Resize the image to project as well, assuming that since the target has a different resolution
				// it make sense for the source image to have a similar resize.
				// Actually, don't do it because the LODBias will be applied separetely at graph generation time.
				//auto imageOp = nop->image.child();
				//nop->image = Visit(imageOp, CurrentSinkingOp);

				nop->SizeX = uint16(nop->SizeX * ScaleX + 0.5f);
				nop->SizeY = uint16(nop->SizeY * ScaleY + 0.5f);
				Result = nop;
				break;
			}

			default:
				break;
			}

			// end on line, replace with sinking op
			if (at == Result && at != InitialSource)
			{
				Ptr<ASTOpImageResizeRel> NewOp = UE::Mutable::Private::Clone<ASTOpImageResizeRel>(CurrentSinkingOp);
				NewOp->Source = at;

				Result = NewOp;
			}

			OldToNew.Add({ at, CurrentSinkingOp }, Result);

			return Result;
		}
	};


	Ptr<ASTOp> ASTOpImageResizeRel::OptimiseSize() const
	{
		Ptr<ASTOp> Result;

		Ptr<ASTOp> SourceAt = Source.child();

		// The instruction can be sunk
		EOpType SourceType = SourceAt->GetOpType();
		switch (SourceType)
		{

		case EOpType::IM_BLANKLAYOUT:
		{
			Ptr<ASTOpImageBlankLayout> NewOp = UE::Mutable::Private::Clone<ASTOpImageBlankLayout>(SourceAt);
			NewOp->BlockSize[0] = uint16(NewOp->BlockSize[0] * Factor[0] + 0.5f);
			NewOp->BlockSize[1] = uint16(NewOp->BlockSize[1] * Factor[1] + 0.5f);
			Result = NewOp;
			break;
		}

		case EOpType::IM_PLAINCOLOR:
		{
			Ptr<ASTOpImagePlainColor> NewOp = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(SourceAt);
			NewOp->Size[0] = FMath::CeilToInt(NewOp->Size[0] * Factor[0]);
			NewOp->Size[1] = FMath::CeilToInt(NewOp->Size[1] * Factor[1]);
			NewOp->LODs = 1; // TODO
			Result = NewOp;
			break;
		}

		case EOpType::IM_TRANSFORM:
		{
			// We can only optimize here if we know the transform result size, otherwise, we will sink the op in the sinker.
			const ASTOpImageTransform* typedAt = static_cast<const ASTOpImageTransform*>(SourceAt.get());
			if (typedAt->SizeX != 0 && typedAt->SizeY != 0)
			{
				// Set the size in the children and remove resize
				Ptr<ASTOpImageTransform> sourceOp = UE::Mutable::Private::Clone<ASTOpImageTransform>(SourceAt.get());
				sourceOp->SizeX = FMath::CeilToInt32(sourceOp->SizeX * Factor[0]);
				sourceOp->SizeY = FMath::CeilToInt32(sourceOp->SizeY * Factor[1]);
				Result = sourceOp;
			}
			break;
		}

		// Don't combine. ResizeRel sometimes can resize more children than Resize can do. (see RasterMesh)
		// It can be combined in an optimization step further in the process, when normal sizes may have been 
		// optimized already (see OptimizeSemantic)
//                case EOpType::IM_RESIZE:
//                {
//                    break;
//                }


		default:
		{
			Sink_ImageResizeRelAST sinker;
			Result = sinker.Apply(this);

			break;
		}

		}

		return Result;
	}

}
