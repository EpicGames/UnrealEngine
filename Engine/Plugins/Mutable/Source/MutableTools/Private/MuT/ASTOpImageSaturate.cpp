// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageSaturate.h"

#include "Containers/Map.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuR/ImagePrivate.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageResize.h"


namespace UE::Mutable::Private
{

	ASTOpImageSaturate::ASTOpImageSaturate()
		: Base(this),
		Factor(this)
	{
	}


	ASTOpImageSaturate::~ASTOpImageSaturate()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageSaturate::IsEqual(const ASTOp& InOther) const
	{
		if (InOther.GetOpType()==GetOpType())
		{
			const ASTOpImageSaturate* Other = static_cast<const ASTOpImageSaturate*>(&InOther);
			return Base == Other->Base &&
				Factor == Other->Factor;
		}
		return false;
	}


	uint32 ASTOpImageSaturate::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Factor));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpImageSaturate::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageSaturate> New = new ASTOpImageSaturate();
		New->Base = MapChild(Base.child());
		New->Factor = MapChild(Factor.child());
		return New;
	}


	void ASTOpImageSaturate::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(Factor);
	}


	void ASTOpImageSaturate::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageSaturateArgs Args;
			FMemory::Memzero(Args);

			if (Base)
			{
				Args.Base = Base->LinkedAddress;
			}

			if (Factor)
			{
				Args.Factor = Factor->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageSaturate::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// Actual work
		if (Base)
		{
			Result = Base->GetImageDesc(bReturnBestOption, Context);
		}

		// Cache the result
		if (Context)
		{
			Context->m_results.Add(this, Result);
		}

		return Result;
	}


	Ptr<ImageSizeExpression> ASTOpImageSaturate::GetImageSizeExpression() const
	{
		if (Base)
		{
			return Base->GetImageSizeExpression();
		}

		return new ImageSizeExpression;
	}


	bool ASTOpImageSaturate::IsImagePlainConstant(FVector4f& OutColor) const
	{
		bool bResult = true;
		OutColor = FVector4f(0.0f,0.0f,0.0f,1.0f);

		if (Base)
		{
			bResult = Base->IsImagePlainConstant(OutColor);
		}

		return bResult;
	}


	FSourceDataDescriptor ASTOpImageSaturate::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
