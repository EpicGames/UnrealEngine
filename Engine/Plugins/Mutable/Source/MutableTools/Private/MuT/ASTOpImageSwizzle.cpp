// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageSwizzle.h"

#include "Containers/Map.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageConform.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImageSaturate.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpImageInvert.h"
#include "MuT/ASTOpColorSwizzle.h"
#include "MuT/ASTOpSwitch.h"

namespace UE::Mutable::Private
{

	ASTOpImageSwizzle::ASTOpImageSwizzle()
		: Sources{ ASTChild(this),ASTChild(this),ASTChild(this),ASTChild(this) }
	{
	}


	ASTOpImageSwizzle::~ASTOpImageSwizzle()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpImageSwizzle::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpImageSwizzle* Other = static_cast<const ASTOpImageSwizzle*>(&otherUntyped);
			for (int32 i = 0; i<MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				if (!(Sources[i] == Other->Sources[i] && SourceChannels[i] == Other->SourceChannels[i]))
				{
					return false;
				}
			}
			return Format == Other->Format;
		}
		return false;
	}


	uint32 ASTOpImageSwizzle::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Sources[0]));
		Result = HashCombineFast(Result, GetTypeHash(Sources[1]));
		Result = HashCombineFast(Result, GetTypeHash(Sources[2]));
		Result = HashCombineFast(Result, GetTypeHash(Sources[3]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[0]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[1]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[2]));
		Result = HashCombineFast(Result, GetTypeHash(SourceChannels[3]));
		Result = HashCombineFast(Result, GetTypeHash(Format));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageSwizzle::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpImageSwizzle> n = new ASTOpImageSwizzle();
		for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
		{
			n->Sources[i] = mapChild(Sources[i].child());
			n->SourceChannels[i] = SourceChannels[i];
		}
		n->Format = Format;
		return n;
	}


	void ASTOpImageSwizzle::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
		{
			f(Sources[i]);
		}
	}


	void ASTOpImageSwizzle::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageSwizzleArgs Args;
			FMemory::Memzero(Args);

			Args.format = Format;
			for (int32 i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				if (Sources[i])
				{
					Args.sources[i] = Sources[i]->LinkedAddress;
				}

				Args.sourceChannels[i] = SourceChannels[i];
			}			

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageSwizzle::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		Ptr<ASTOpImageSwizzle> sat;

		for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
		{
			Ptr<ASTOp> candidate = Sources[c].child();
			if (!candidate)
			{
				continue;
			}

			switch (candidate->GetOpType())
			{
			// Swizzle
			case EOpType::IM_SWIZZLE:
			{
				if (!sat)
				{
					sat = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
				}
				const ASTOpImageSwizzle* typedCandidate = static_cast<const ASTOpImageSwizzle*>(candidate.get());
				int candidateChannel = SourceChannels[c];

				sat->Sources[c] = typedCandidate->Sources[candidateChannel].child();
				sat->SourceChannels[c] = typedCandidate->SourceChannels[candidateChannel];

				break;
			}

			// Format
			case EOpType::IM_PIXELFORMAT:
			{
				// We can remove the format if its source is already an uncompressed format
				ASTOpImagePixelFormat* typedCandidate = static_cast<ASTOpImagePixelFormat*>(candidate.get());
				Ptr<ASTOp> formatSource = typedCandidate->Source.child();

				if (formatSource)
				{
					FImageDesc desc = formatSource->GetImageDesc();
					if (desc.m_format != EImageFormat::None && !IsCompressedFormat(desc.m_format))
					{
						if (!sat)
						{
							sat = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
						}
						sat->Sources[c] = formatSource;
					}
				}

				break;
			}

			default:
				break;
			}

		}

		return sat;
	}

	namespace
	{
		//---------------------------------------------------------------------------------------------
		//! Set al the non-null sources of an image swizzle operation to the given value
		//---------------------------------------------------------------------------------------------
		void ReplaceAllSources(Ptr<ASTOpImageSwizzle>& op, Ptr<ASTOp>& value)
		{
			check(op->GetOpType() == EOpType::IM_SWIZZLE);
			for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
			{
				if (op->Sources[c])
				{
					op->Sources[c] = value;
				}
			}
		}
	}

	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageSwizzle::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		MUTABLE_CPUPROFILER_SCOPE(OptimiseSwizzleAST);

		//! Basic optimisation first
		Ptr<ASTOp> at = OptimiseSemantic(options, 0);
		if (at)
		{
			return at;
		}

		// If all sources are the same, we can sink the instruction
		bool bAllChannelsAreTheSame = true;
		bool bAllChannelsAreTheSameType = true;
		Ptr<ASTOp> channelSourceAt;
		for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
		{
			Ptr<ASTOp> candidate = Sources[c].child();
			if (candidate)
			{
				if (!channelSourceAt)
				{
					channelSourceAt = candidate;
				}
				else
				{
					bAllChannelsAreTheSame = bAllChannelsAreTheSame && (channelSourceAt == candidate);
					bAllChannelsAreTheSameType = bAllChannelsAreTheSameType && (channelSourceAt->GetOpType() == candidate->GetOpType());
				}
			}
		}

		if (!channelSourceAt)
		{
			return at;
		}

		// If we are not changing channel order, just remove the swizzle and adjust the format.
		bool bSameChannelOrder = true;
		int32 NumChannelsInFormat = GetImageFormatData(Format).Channels;
		for (int32 c = 0; c < NumChannelsInFormat; ++c)
		{
			if (Sources[c] && SourceChannels[c] != c)
			{
				bSameChannelOrder = false;
			}
		}
		
		// If all channels are the same, and in the same order, and the source format is the same that we are
		// setting in the swizzle, then the swizzle won't do anything.
		if (bAllChannelsAreTheSame && bSameChannelOrder)
		{
			FImageDesc SourceDesc = channelSourceAt->GetImageDesc();
			if (SourceDesc.m_format == Format)
			{
				return channelSourceAt;
			}
		}

		EOpType sourceType = channelSourceAt->GetOpType();

		if (bAllChannelsAreTheSame)
		{
			at = context.ImageSwizzleSinker.Apply(this);
		}

		if (!at && bAllChannelsAreTheSameType)
		{
			// Maybe we can still sink the instruction in some cases

			// If we have RGB being the same IM_MULTILAYER, and alpha a compatible IM_MULTILAYER we can optimize with
			// a special multilayer blend mode. This happens often because of higher level group projector nodes.
			if (!at
				&&
				Format == EImageFormat::RGBA_UByte
				&&
				Sources[0] == Sources[1] && Sources[0] == Sources[2]
				&&
				Sources[0] && Sources[0]->GetOpType() == EOpType::IM_MULTILAYER
				&&
				Sources[3] && Sources[3]->GetOpType() == EOpType::IM_MULTILAYER
				&&
				SourceChannels[0] == 0 && SourceChannels[1] == 1 && SourceChannels[2] == 2 && SourceChannels[3] == 0
				)
			{
				const ASTOpImageMultiLayer* ColorMultiLayer = static_cast<const ASTOpImageMultiLayer*>(Sources[0].child().get());
				check(ColorMultiLayer);
				const ASTOpImageMultiLayer* AlphaMultiLayer = static_cast<const ASTOpImageMultiLayer*>(Sources[3].child().get());
				check(AlphaMultiLayer);

				bool bIsSpecialMultiLayer = !AlphaMultiLayer->mask
					&&
					ColorMultiLayer->range == AlphaMultiLayer->range;

				if (bIsSpecialMultiLayer)
				{
					// We can combine the 2 multilayers into the composite blend+lighten mode

					Ptr<ASTOpImageSwizzle> NewBase = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					NewBase->Sources[0] = ColorMultiLayer->base.child();
					NewBase->Sources[1] = ColorMultiLayer->base.child();
					NewBase->Sources[2] = ColorMultiLayer->base.child();
					NewBase->Sources[3] = AlphaMultiLayer->base.child();

					Ptr<ASTOpImageSwizzle> NewBlended = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					NewBlended->Sources[0] = ColorMultiLayer->blend.child();
					NewBlended->Sources[1] = ColorMultiLayer->blend.child();
					NewBlended->Sources[2] = ColorMultiLayer->blend.child();
					NewBlended->Sources[3] = AlphaMultiLayer->blend.child();

					Ptr<ASTOpImageMultiLayer> NewMultiLayer = UE::Mutable::Private::Clone<ASTOpImageMultiLayer>(ColorMultiLayer);
					NewMultiLayer->blendTypeAlpha = AlphaMultiLayer->blendType;
					NewMultiLayer->BlendAlphaSourceChannel = 3;
					NewMultiLayer->base = NewBase;
					NewMultiLayer->blend = NewBlended;

					if ( NewMultiLayer->mask.child() == AlphaMultiLayer->blend.child()
						&&
						NewBlended->Format==EImageFormat::RGBA_UByte
						)
					{
						// Additional optimization is possible here.
						NewMultiLayer->bUseMaskFromBlended = true;
						NewMultiLayer->mask = nullptr;
					}

					at = NewMultiLayer;
				}
			}

			// If we have RGB being the same IM_LAYER, and alpha a compatible IM_LAYER we can optimize with a special layer blend mode.
			if (!at
				&&
				Format == EImageFormat::RGBA_UByte
				&&
				Sources[0] == Sources[1] && (Sources[0] == Sources[2] || !Sources[2])
				&&
				Sources[0] && Sources[0]->GetOpType() == EOpType::IM_LAYER
				&&
				Sources[3] && Sources[3]->GetOpType() == EOpType::IM_LAYER
				&&
				SourceChannels[0] == 0 && SourceChannels[1] == 1 && (SourceChannels[2] == 2 || !Sources[2] ) && SourceChannels[3] == 0
				)
			{
				const ASTOpImageLayer* ColorLayer = static_cast<const ASTOpImageLayer*>(Sources[0].child().get());
				check(ColorLayer);
				const ASTOpImageLayer* AlphaLayer = static_cast<const ASTOpImageLayer*>(Sources[3].child().get());
				check(AlphaLayer);

				bool bIsSpecialMultiLayer = !AlphaLayer->mask && !ColorLayer->Flags && !AlphaLayer->Flags;

				if (bIsSpecialMultiLayer)
				{
					// We can combine the 2 image_layers into the composite blend+lighten mode

					Ptr<ASTOpImageSwizzle> NewBase = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					NewBase->Sources[0] = ColorLayer->base.child();
					NewBase->Sources[1] = ColorLayer->base.child();
					NewBase->Sources[2] = Sources[2] ? ColorLayer->base.child() : nullptr;
					NewBase->Sources[3] = AlphaLayer->base.child();

					Ptr<ASTOpImageSwizzle> NewBlended = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					NewBlended->Sources[0] = ColorLayer->blend.child();
					NewBlended->Sources[1] = ColorLayer->blend.child();
					NewBlended->Sources[2] = Sources[2] ? ColorLayer->blend.child() : nullptr;
					NewBlended->Sources[3] = AlphaLayer->blend.child();

					Ptr<ASTOpImageLayer> NewLayer = UE::Mutable::Private::Clone<ASTOpImageLayer>(ColorLayer);
					NewLayer->blendTypeAlpha = AlphaLayer->blendType;
					NewLayer->BlendAlphaSourceChannel = 3;
					NewLayer->base = NewBase;
					NewLayer->blend = NewBlended;

					if (NewLayer->mask.child() == AlphaLayer->blend.child()
						&&
						NewBlended->Format == EImageFormat::RGBA_UByte
						)
					{
						// Additional optimization is possible here.
						NewLayer->Flags |= FOperation::ImageLayerArgs::FLAGS::F_USE_MASK_FROM_BLENDED;
						NewLayer->mask = nullptr;
					}

					at = NewLayer;
				}
			}


			// If the channels are compatible switches, we can still sink the swizzle.
			if (!at && sourceType == EOpType::IM_SWITCH)
			{
				const ASTOpSwitch* FirstSwitch = static_cast<const ASTOpSwitch*>(Sources[0].child().get());
				check(FirstSwitch);

				bool bAreAllSwitchesCompatible = true;
				for (int32 c = 1; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (Sources[c])
					{
						const ASTOpSwitch* Typed = static_cast<const ASTOpSwitch*>(Sources[c].child().get());
						check(Typed);
						if (!Typed->IsCompatibleWith(FirstSwitch))
						{
							bAreAllSwitchesCompatible = false;
							break;
						}
					}
				}

				if (bAreAllSwitchesCompatible)
				{
					// Move the swizzle down all the paths
					Ptr<ASTOpSwitch> nop = UE::Mutable::Private::Clone<ASTOpSwitch>(channelSourceAt);

					if (nop->Default)
					{
						Ptr<ASTOpImageSwizzle> defOp = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
						for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
						{
							const ASTOpSwitch* ChannelSwitch = static_cast<const ASTOpSwitch*>(Sources[c].child().get());
							if (ChannelSwitch)
							{
								defOp->Sources[c] = ChannelSwitch->Default.child();
							}
						}
						nop->Default = defOp;
					}

					for (int32 v = 0; v < nop->Cases.Num(); ++v)
					{
						if (nop->Cases[v].Branch)
						{
							Ptr<ASTOpImageSwizzle> branchOp = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
							for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
							{
								const ASTOpSwitch* ChannelSwitch = static_cast<const ASTOpSwitch*>(Sources[c].child().get());
								if (ChannelSwitch)
								{
									branchOp->Sources[c] = ChannelSwitch->Cases[v].Branch.child();
								}
							}
							nop->Cases[v].Branch = branchOp;
						}
					}

					at = nop;
				}
			}

			// Swizzle down compatible displaces.
			if (!at && sourceType == EOpType::IM_DISPLACE)
			{
				const ASTOpImageDisplace* FirstDisplace = static_cast<const ASTOpImageDisplace*>(Sources[0].child().get());
				check(FirstDisplace);

				bool bAreAllDisplacesCompatible = true;
				for (int32 c = 1; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (Sources[c])
					{
						const ASTOpImageDisplace* Typed = static_cast<const ASTOpImageDisplace*>(Sources[c].child().get());
						check(Typed);
						if (FirstDisplace->DisplacementMap != Typed->DisplacementMap)
						{
							bAreAllDisplacesCompatible = false;
							break;
						}
					}
				}

				if (bAreAllDisplacesCompatible)
				{
					// Move the swizzle down all the paths
					Ptr<ASTOpImageDisplace> NewDisplace = UE::Mutable::Private::Clone<ASTOpImageDisplace>(FirstDisplace);

					Ptr<ASTOpImageSwizzle> SourceOp = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
					{
						const ASTOpImageDisplace* ChannelDisplace = static_cast<const ASTOpImageDisplace*>(Sources[c].child().get());
						if (ChannelDisplace)
						{
							SourceOp->Sources[c] = ChannelDisplace->Source.child();
						}
					}

					NewDisplace->Source = SourceOp;

					at = NewDisplace;
				}

			}

			// Swizzle down compatible raster meshes.
			if (!at && sourceType == EOpType::IM_RASTERMESH)
			{
				const ASTOpImageRasterMesh* FirstRasterMesh = static_cast<const ASTOpImageRasterMesh*>(Sources[0].child().get());
				check(FirstRasterMesh);

				bool bAreAllRasterMeshesCompatible = true;
				for (int32 c = 1; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (Sources[c])
					{
						const ASTOpImageRasterMesh* Typed = static_cast<const ASTOpImageRasterMesh*>(Sources[c].child().get());
						check(Typed);

						// Compare all Args but the source image
						if (Typed->mesh.child() != FirstRasterMesh->mesh.child()
							|| Typed->angleFadeProperties.child() != FirstRasterMesh->angleFadeProperties.child()
							|| Typed->mask.child() != FirstRasterMesh->mask.child()
							|| Typed->projector.child() != FirstRasterMesh->projector.child()
							|| Typed->BlockId != FirstRasterMesh->BlockId
							|| Typed->LayoutIndex != FirstRasterMesh->LayoutIndex
							|| Typed->SizeX != FirstRasterMesh->SizeX
							|| Typed->SizeY != FirstRasterMesh->SizeY
							|| Typed->UncroppedSizeX != FirstRasterMesh->UncroppedSizeX
							|| Typed->UncroppedSizeY != FirstRasterMesh->UncroppedSizeY
							|| Typed->CropMinX != FirstRasterMesh->CropMinX
							|| Typed->CropMinY != FirstRasterMesh->CropMinY
							// Also ignore the fading flags. They are dealt with below.
							//|| Typed->bIsRGBFadingEnabled != FirstRasterMesh->bIsRGBFadingEnabled
							//|| Typed->bIsAlphaFadingEnabled != FirstRasterMesh->bIsAlphaFadingEnabled
							)
						{
							bAreAllRasterMeshesCompatible = false;
							break;
						}
					}
				}

				if (bAreAllRasterMeshesCompatible)
				{
					// Move the swizzle down all the paths
					Ptr<ASTOpImageRasterMesh> NewRaster = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(FirstRasterMesh);

					Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
					{
						const ASTOpImageRasterMesh* ChannelRaster = static_cast<const ASTOpImageRasterMesh*>(Sources[c].child().get());
						if (ChannelRaster)
						{
							NewSwizzle->Sources[c] = ChannelRaster->image.child();
						}
					}

					NewRaster->image = NewSwizzle;

					// If we are swapping rgb and alphas, we need to correct some flags
					{
						// We should only find these two cases
						if (SourceChannels[0] == 3 || SourceChannels[1] == 3 || SourceChannels[2] == 3)
						{
							NewRaster->bIsRGBFadingEnabled = NewRaster->bIsAlphaFadingEnabled;
						}
						else if (Sources[3] && SourceChannels[3] < 3)
						{
							const ASTOpImageRasterMesh* ChannelRaster = static_cast<const ASTOpImageRasterMesh*>(Sources[3].child().get());
							NewRaster->bIsAlphaFadingEnabled = ChannelRaster->bIsRGBFadingEnabled;
						}
					}

					at = NewRaster;
				}

			}

			// Swizzle down compatible image transforms.
			if (!at && sourceType == EOpType::IM_TRANSFORM)
			{
				const ASTOpImageTransform* FirstTransform = static_cast<const ASTOpImageTransform*>(Sources[0].child().get());
				check(FirstTransform);

				bool bAreAllTransformsCompatible = true;
				for (int32 c = 1; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (Sources[c])
					{
						const ASTOpImageTransform* Typed = static_cast<const ASTOpImageTransform*>(Sources[c].child().get());
						check(Typed);

						// Compare all Args but the source image
						if (Typed->ScaleX.child() != FirstTransform->ScaleX.child()
							|| Typed->ScaleY.child() != FirstTransform->ScaleY.child()
							|| Typed->OffsetX.child() != FirstTransform->OffsetX.child()
							|| Typed->OffsetY.child() != FirstTransform->OffsetY.child()
							|| Typed->Rotation.child() != FirstTransform->Rotation.child()
							|| Typed->SizeX != FirstTransform->SizeX
							|| Typed->SizeY != FirstTransform->SizeY
							|| Typed->SourceSizeX != FirstTransform->SourceSizeX
							|| Typed->SourceSizeY != FirstTransform->SourceSizeY
							|| Typed->AddressMode != FirstTransform->AddressMode
							|| Typed->bKeepAspectRatio != FirstTransform->bKeepAspectRatio
							)
						{
							bAreAllTransformsCompatible = false;
							break;
						}
					}
				}

				if (bAreAllTransformsCompatible)
				{
					// Move the swizzle down all the paths
					Ptr<ASTOpImageTransform> NewTransform = UE::Mutable::Private::Clone<ASTOpImageTransform>(FirstTransform);

					Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
					{
						const ASTOpImageTransform* ChannelTransform = static_cast<const ASTOpImageTransform*>(Sources[c].child().get());
						if (ChannelTransform)
						{
							NewSwizzle->Sources[c] = ChannelTransform->Base.child();
						}
					}

					NewTransform->Base = NewSwizzle;

					at = NewTransform;
				}
			}

			// Swizzle down compatible resizes.
			//if (!at && sourceType == EOpType::IM_RESIZE)
			//{
			//	const ASTOpFixed* FirstResize = static_cast<const ASTOpFixed*>(Sources[0].child().get());
			//	check(FirstResize);

			//	bool bAreAllResizesCompatible = true;
			//	for (int32 c = 1; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
			//	{
			//		if (Sources[c])
			//		{
			//			const ASTOpFixed* Typed = static_cast<const ASTOpFixed*>(Sources[c].child().get());
			//			check(Typed);
			//			// Compare all Args but the source image
			//			FOperation::ImageResizeArgs ArgCopy = FirstResize->op.args.ImageResize;
			//			ArgCopy.source = Typed->op.args.ImageResize.source;

			//			if (FMemory::Memcmp(&ArgCopy, &Typed->op.args.ImageResize, sizeof(FOperation::ImageResizeArgs)) != 0)
			//			{
			//				bAreAllResizesCompatible = false;
			//				break;
			//			}
			//		}
			//	}

			//	if (bAreAllResizesCompatible)
			//	{
			//		// Move the swizzle down all the paths
			//		Ptr<ASTOpFixed> NewResize = UE::Mutable::Private::Clone<ASTOpFixed>(FirstResize);

			//		Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
			//		for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
			//		{
			//			const ASTOpFixed* ChannelResize = static_cast<const ASTOpFixed*>(Sources[c].child().get());
			//			if (ChannelResize)
			//			{
			//				NewSwizzle->Sources[c] = ChannelResize->children[ChannelResize->op.args.ImageResize.source].child();
			//			}
			//		}

			//		NewResize->SetChild(NewResize->op.args.ImageResize.source, NewSwizzle);

			//		at = NewResize;
			//	}

			//}

			// Swizzle down compatible pixelformats.
			if (!at && sourceType == EOpType::IM_PIXELFORMAT && bSameChannelOrder)
			{
				const ASTOpImagePixelFormat* FirstFormat = static_cast<const ASTOpImagePixelFormat*>(Sources[0].child().get());
				check(FirstFormat);

				bool bAreAllFormatsCompatible = true;
				for (int32 c = 1; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (Sources[c])
					{
						const ASTOpImagePixelFormat* Typed = static_cast<const ASTOpImagePixelFormat*>(Sources[c].child().get());
						check(Typed);

						if (Typed->Source.child() != FirstFormat->Source.child())
						{
							bAreAllFormatsCompatible = false;
							break;
						}
					}
				}

				if (bAreAllFormatsCompatible)
				{
					// Move the swizzle down all the paths
					Ptr<ASTOpImagePixelFormat> NewFormat = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(FirstFormat);
					NewFormat->Format = Format;
					at = NewFormat;
				}
			}

			// Swizzle down plaincolors.
			if (!at && sourceType == EOpType::IM_PLAINCOLOR)
			{
				Ptr<ASTOpImagePlainColor> NewPlain = UE::Mutable::Private::Clone<ASTOpImagePlainColor>(channelSourceAt);
				Ptr<ASTOpColorSwizzle> NewSwizzle = new ASTOpColorSwizzle;
				for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
				{
					if (Sources[c])
					{
						const ASTOpImagePlainColor* TypedPlain = static_cast<const ASTOpImagePlainColor*>(Sources[c].child().get());
						NewSwizzle->Sources[c] = TypedPlain->Color.child();
					}
					NewSwizzle->SourceChannels[c] = SourceChannels[c];
				}
				NewPlain->Color = NewSwizzle;
				NewPlain->Format = Format;
				at = NewPlain;
			}

		}

		// TODO \warning: probably wrong because it doesn't check if the layer color is doing a separated alpha operation.
		// Swizzle of RGB from a source + A from a layer color
		// This can be optimized to apply the layer color on-base directly to the alpha channel to skip the swizzle
		//if ( !at 
		//	&&
		//	Sources[0] && Sources[0]==Sources[1] && Sources[0]==Sources[2]
		//	&&
		//	Sources[3] && Sources[3]->GetOpType()==EOpType::IM_LAYERCOLOR
		//	)
		//{
		//	// Move the swizzle down all the paths
		//	Ptr<ASTOpImageLayerColor> NewLayerColor = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(Sources[3].child());

		//	Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
		//	NewSwizzle->Sources[3] = NewLayerColor->base.child();

		//	NewLayerColor->blendTypeAlpha = NewLayerColor->blendType;
		//	NewLayerColor->BlendAlphaSourceChannel = SourceChannels[3];
		//	NewLayerColor->blendType = EBlendType::BT_NONE;
		//	NewLayerColor->base = NewSwizzle;

		//	at = NewLayerColor;

		//}

		// Swizzle of RGB from a source + A from a layer
		// This can be optimized to apply the layer on-base directly to the alpha channel to skip the swizzle
		// \TODO: wrong: the new layer color will always use the alpha from the color, instead of the channel that the swizzle is selecting.
		// \TODO: wrong: it ignores the possibility of separate alpha operation
		//if (!at
		//	&&
		//	Sources[0] && Sources[0] == Sources[1] && Sources[0] == Sources[2]
		//	&&
		//	Sources[3] && Sources[3]->GetOpType() == EOpType::IM_LAYER)
		//{
		//	// Move the swizzle down all the paths
		//	Ptr<ASTOpImageLayer> NewLayer = UE::Mutable::Private::Clone<ASTOpImageLayer>(Sources[3].child());

		//	Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
		//	NewSwizzle->Sources[3] = NewLayer->base.child();

		//	NewLayer->blendTypeAlpha = NewLayer->blendType;
		//	NewLayer->BlendAlphaSourceChannel = SourceChannels[3];
		//	NewLayer->blendType = EBlendType::BT_NONE;
		//	NewLayer->base = NewSwizzle;

		//	at = NewLayer;
		//}

		// Swizzle of RGB from a layer color + A from a different source
		// This can be optimized to apply the layer color on-base directly to the rgb channel to skip the swizzle
		if (!at
			&&
			Sources[0] 
			&& 
			(!Sources[1] || Sources[0] == Sources[1]) 
			&& 
			(!Sources[2] || Sources[0] == Sources[2])
			&&
			Sources[0]->GetOpType() == EOpType::IM_LAYERCOLOR
			&& 
			!(Sources[3]==Sources[0]) )
		{
			// Move the swizzle down all the rgb path
			Ptr<ASTOpImageLayerColor> NewLayerColor = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(Sources[0].child());
			check(NewLayerColor);

			Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
			NewSwizzle->Sources[0] = NewLayerColor->base.child();
			NewSwizzle->Sources[1] = Sources[1] ? NewLayerColor->base.child() : nullptr;
			NewSwizzle->Sources[2] = Sources[2] ? NewLayerColor->base.child() : nullptr;

			NewLayerColor->blendTypeAlpha = EBlendType::BT_NONE;
			NewLayerColor->base = NewSwizzle;

			at = NewLayerColor;
		}

		// Swizzle getting an A from a saturate
		// The saturate doesn't affect A channel so it can be removed.
		if (!at)
		{
			Ptr<ASTOpImageSwizzle> NewSwizzle;

			for (int32 Channel = 0; Channel < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++Channel)
			{
				if (Sources[Channel] && SourceChannels[Channel]==3 && Sources[Channel]->GetOpType() == EOpType::IM_SATURATE)
				{
					// Remove the saturate for this channel
					if (!NewSwizzle)
					{
						NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
					}

					const ASTOpImageSaturate* OldSaturate = static_cast<const ASTOpImageSaturate*>(Sources[Channel].child().get());
					Ptr<ASTOp> OldSaturateBase = OldSaturate->Base.child();

					NewSwizzle->Sources[Channel] = OldSaturateBase;
					at = NewSwizzle;
				}
			}
		}

		// Swizzle of RGB from a saturate + A from a different source
		// This can be optimized to apply the saturate after the swizzle, since it doesn't touch A
		if (!at
			&&
			Sources[0]
			&&
			Sources[0]->GetOpType() == EOpType::IM_SATURATE 
			&&
			(Sources[0] == Sources[1]) && (Sources[0] == Sources[2])
			&&
			// Actually it would be enough with all the RGB channels to be present in any order
			SourceChannels[0]==0 && SourceChannels[1] == 1 && SourceChannels[2] == 2
			)
		{
			// Move the swizzle down 
			Ptr<ASTOpImageSaturate> NewSaturate = UE::Mutable::Private::Clone<ASTOpImageSaturate>(Sources[0].child());
			check(NewSaturate);

			Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
			Ptr<ASTOp> OldSaturateBase = NewSaturate->Base.child();
			NewSwizzle->Sources[0] = OldSaturateBase;
			NewSwizzle->Sources[1] = OldSaturateBase;
			NewSwizzle->Sources[2] = OldSaturateBase;

			// Remove the saturate from the alpha if it is there.
			if (Sources[3] == Sources[0] && SourceChannels[3] == 3)
			{
				NewSwizzle->Sources[3] = OldSaturateBase;
			}

			NewSaturate->Base = NewSwizzle;

			at = NewSaturate;
		}
		
		// Swizzle with the same op as identity in RGB, a Layer op in the A that has one of the operands matching 
		// the one in the swizzle RGB, but using its A. 
		// The Layer operation can be flagged as alpha only and moved up the swizzle, then the swizzle is identity 
		// and can be removed, so remove it here anyway.
		// This is another very specific optimization case that happens with certain combination of operations.
		// from:
		//- SWIZZLE
		//	r -> r from A
		//	g -> g from A
		//	b -> b from A
		//	a -> (r or a) from LAYER
		//		- 3 from A (on alpha only using flags)
		//		- B
		// to:
		//- LAYER (on alpha only)
		//	- A
		//	- B
		// In addition, if the blend operation done by LAYER is commutative, see if X is 3 from I instead.
		if (!at
			&&
			(Sources[0] == Sources[1]) && (Sources[0] == Sources[2])
			&&
			SourceChannels[0] == 0 && SourceChannels[1] == 1 && SourceChannels[2] == 2
			&&
			Sources[3] && Sources[3]->GetOpType() == EOpType::IM_LAYER
			)
		{
			const ASTOpImageLayer* OldLayer = static_cast<const ASTOpImageLayer*>(Sources[3].child().get());

			Ptr<ASTOp> SwizzleRGBOp = Sources[0].child();
			Ptr<ASTOp> OldLayerBlendOp = OldLayer->blend.child();
			{
				auto DiscardNeutralOps = [](Ptr<ASTOp> Op)
				{
					bool bUpdated = true;
					while (bUpdated)
					{
						bUpdated = false;
						switch (Op->GetOpType())
						{
						case EOpType::IM_PIXELFORMAT:
						{
							const ASTOpImagePixelFormat* Typed = static_cast<const ASTOpImagePixelFormat*>(Op.get());
							Op = Typed->Source.child();
							bUpdated = true;
							break;
						}

						default: break;
						}
					}
					return Op;
				};

				SwizzleRGBOp = DiscardNeutralOps(SwizzleRGBOp);
				OldLayerBlendOp = DiscardNeutralOps(OldLayerBlendOp);
			}
			bool bOldLayerBlendIsCompatibleWithSwizzleRGBs = OldLayerBlendOp == SwizzleRGBOp;

			// For now just check the case that we are observing in the working data: 
			// A is in the blended of a multiply, and we take its alpha channel
			// \TODO: Implement the other cases when we find instances of them.
			if ( OldLayer->Flags==FOperation::ImageLayerArgs::FLAGS::F_BLENDED_RGB_FROM_ALPHA
				&&
				bOldLayerBlendIsCompatibleWithSwizzleRGBs
				&& 
				OldLayer->blendType==EBlendType::BT_MULTIPLY 
				&&
				OldLayer->blendTypeAlpha == EBlendType::BT_NONE 
				&&
				SourceChannels[3]==0)
			{
				// The new base needs to have the format of the root swizzle
				Ptr<ASTOpImagePixelFormat> NewBase = new ASTOpImagePixelFormat;
				NewBase->Source = Sources[0].child();
				NewBase->Format = Format;

				Ptr<ASTOpImageLayer> NewLayer = UE::Mutable::Private::Clone<ASTOpImageLayer>(OldLayer);
				NewLayer->blend = OldLayer->base.child();
				NewLayer->base = NewBase;
				NewLayer->blendTypeAlpha = NewLayer->blendType;
				NewLayer->blendType = EBlendType::BT_NONE;
				NewLayer->BlendAlphaSourceChannel = 0;
				NewLayer->Flags = 0;

				at = NewLayer;
			}
		}


		// If we have an alpha channel that has as children something that expands a single channel texture
		// skip the expansion,. since we know we just want one channel.
		// Very specific, based on observed code patterns.
		// \TODO: Make more general.
		if (!at
			&&
			Sources[3] && Sources[3]->GetOpType() == EOpType::IM_LAYER
			)
		{
			ASTOpImageLayer* OldLayer = static_cast<ASTOpImageLayer*>(Sources[3].child().get());

			// For now just check the case that we are observing in the working data: 
			if (OldLayer->Flags == 0
				&&
				SourceChannels[3] == 0
				&&
				OldLayer->blend->GetOpType()==EOpType::IM_PIXELFORMAT )
			{
				const ASTOpImagePixelFormat* OldFormat = static_cast<const ASTOpImagePixelFormat*>(OldLayer->blend.child().get());
				if (OldFormat->Source->GetOpType() == EOpType::IM_SWIZZLE
					&&
					(
						OldFormat->Format == EImageFormat::RGB_UByte
						||
						OldFormat->Format == EImageFormat::RGBA_UByte
						) 
					)
				{
					const ASTOpImageSwizzle* OldChildSwizzle = static_cast<const ASTOpImageSwizzle*>(OldFormat->Source.child().get());
					if (OldChildSwizzle->Format == EImageFormat::L_UByte)
					{
						Ptr<ASTOpImageSwizzle> NewBaseSwizzle = new ASTOpImageSwizzle;
						NewBaseSwizzle->Format = EImageFormat::L_UByte;
						NewBaseSwizzle->Sources[0] = OldLayer->base.child();
						NewBaseSwizzle->SourceChannels[0] = SourceChannels[3];

						Ptr<ASTOpImageSwizzle> NewBlendSwizzle = new ASTOpImageSwizzle;
						NewBlendSwizzle->Format = EImageFormat::L_UByte;
						NewBlendSwizzle->Sources[0] = OldLayer->blend.child();
						NewBlendSwizzle->SourceChannels[0] = SourceChannels[3];

						Ptr<ASTOpImageLayer> NewLayer = UE::Mutable::Private::Clone<ASTOpImageLayer>(OldLayer);
						NewLayer->base = NewBaseSwizzle;
						NewLayer->blend = NewBlendSwizzle;

						Ptr<ASTOpImageSwizzle> NewSwizzle = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(this);
						NewSwizzle->Sources[3] = NewLayer;

						at = NewSwizzle;
					}
				}
			}
		}


		return at;
	}



	//!
	FImageDesc ASTOpImageSwizzle::GetImageDesc(bool returnBestOption, FGetImageDescContext* context) const
	{
		FImageDesc res;

		// Local context in case it is necessary
		FGetImageDescContext localContext;
		if (!context)
		{
			context = &localContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		int32 FirstValidSourceIndex = -1;
		for (int32 SourceIndex = 0; SourceIndex < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex)
		{
			if (Sources[SourceIndex].child())
			{
				FirstValidSourceIndex = SourceIndex;
				break;	
			}
		}

		if (FirstValidSourceIndex >= 0)
		{
			res = Sources[FirstValidSourceIndex]->GetImageDesc(returnBestOption, context);
			res.m_format = Format;
			check(res.m_format != EImageFormat::None);
		}

		// Format overriden by current op.
		context->bIsFormatFromImageParameter = false;

		// Cache the result
		context->m_results.Add(this, res);

		return res;
	}


	void ASTOpImageSwizzle::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (Sources[0].child())
		{
			// Assume the block size of the biggest mip
			Sources[0].child()->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	bool ASTOpImageSwizzle::IsImagePlainConstant(FVector4f& color) const
	{
		// TODO: Maybe something could be done here.
		return false;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageSwizzle::GetImageSizeExpression() const
	{
		UE::Mutable::Private::Ptr<ImageSizeExpression> pRes;

		if (Sources[0].child())
		{
			pRes = Sources[0].child()->GetImageSizeExpression();
		}
		else
		{
			pRes = new ImageSizeExpression;
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpImageSwizzle::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found= Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		for( int32 SourceIndex=0; SourceIndex<MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++SourceIndex )
		{ 
			if (Sources[SourceIndex])
			{
				FSourceDataDescriptor SourceDesc = Sources[SourceIndex]->GetSourceDataDescriptor(Context);				
				Result.CombineWith(SourceDesc);
			}
		}

		Context->Cache.Add(this,Result);

		return Result;
	}



	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_ImageSwizzleAST::Apply(const ASTOpImageSwizzle* InRoot)
	{
		Root = InRoot;
		OldToNew.Reset();

		check(Root->GetOpType() == EOpType::IM_SWIZZLE);

		// This sinker only works assuming all swizzle channels come from the same image operation.
		bool bAllChannelsAreTheSame = true;
		Ptr<ASTOp> Source;
		for (int c = 0; c < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++c)
		{
			Ptr<ASTOp> Candidate = Root->Sources[c].child();
			if (Candidate)
			{
				if (!Source)
				{
					Source = Candidate;
				}
				else
				{
					bAllChannelsAreTheSame = bAllChannelsAreTheSame && (Source == Candidate);
				}
			}
		}

		if (!bAllChannelsAreTheSame || !Source)
		{
			return nullptr;
		}

		InitialSource = Source;
		UE::Mutable::Private::Ptr<ASTOp> NewSource = Visit(InitialSource, Root);

		Root = nullptr;

		// If there is any change, it is the new root.
		if (NewSource != InitialSource)
		{
			return NewSource;
		}

		return nullptr;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_ImageSwizzleAST::Visit(UE::Mutable::Private::Ptr<ASTOp> at, const ASTOpImageSwizzle* CurrentSwizzleOp)
	{
		if (!at) return nullptr;

		// Already visited?
		const Ptr<ASTOp>* Cached = OldToNew.Find({ at,CurrentSwizzleOp });
		if (Cached)
		{
			return *Cached;
		}

		UE::Mutable::Private::Ptr<ASTOp> newAt = at;
		switch (at->GetOpType())
		{

		case EOpType::IM_CONDITIONAL:
		{
			// We move the op down the two paths
			auto newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
			newOp->yes = Visit(newOp->yes.child(), CurrentSwizzleOp);
			newOp->no = Visit(newOp->no.child(), CurrentSwizzleOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_SWITCH:
		{
			// We move the op down all the paths
			Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
			newOp->Default = Visit(newOp->Default.child(), CurrentSwizzleOp);
			for (ASTOpSwitch::FCase& c : newOp->Cases)
			{
				c.Branch = Visit(c.Branch.child(), CurrentSwizzleOp);
			}
			newAt = newOp;
			break;
		}

		case EOpType::IM_COMPOSE:
		{
			Ptr<ASTOpImageCompose> newOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(at);
			newOp->Base = Visit(newOp->Base.child(), CurrentSwizzleOp);
			newOp->BlockImage = Visit(newOp->BlockImage.child(), CurrentSwizzleOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_MULTICOMPOSE:
		{
			Ptr<ASTOpImageMultiCompose> newOp = UE::Mutable::Private::Clone<ASTOpImageMultiCompose>(at);
			newOp->Base = Visit(newOp->Base.child(), CurrentSwizzleOp);
			newOp->SourceImage = Visit(newOp->SourceImage.child(), CurrentSwizzleOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_PATCH:
		{
			Ptr<ASTOpImagePatch> newOp = UE::Mutable::Private::Clone<ASTOpImagePatch>(at);
			newOp->base = Visit(newOp->base.child(), CurrentSwizzleOp);
			newOp->patch = Visit(newOp->patch.child(), CurrentSwizzleOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_MIPMAP:
		{
			Ptr<ASTOpImageMipmap> newOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(at);
			newOp->Source = Visit(newOp->Source.child(), CurrentSwizzleOp);
			newAt = newOp;
			break;
		}

		case EOpType::IM_INTERPOLATE:
		{
			Ptr<ASTOpImageInterpolate> NewOp = UE::Mutable::Private::Clone<ASTOpImageInterpolate>(at);

			for (int32 v = 0; v < MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++v)
			{
				NewOp->Targets[v] = Visit(NewOp->Targets[v].child(), CurrentSwizzleOp);;
			}

			newAt = NewOp;
			break;
		}

		case EOpType::IM_LAYER:
		{
			Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(at);
			nop->base = Visit(nop->base.child(), CurrentSwizzleOp);
			nop->blend = Visit(nop->blend.child(), CurrentSwizzleOp);
			newAt = nop;
			break;
		}

		case EOpType::IM_LAYERCOLOR:
		{
			Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(at);
			nop->base = Visit(nop->base.child(), CurrentSwizzleOp);

			// We need to swizzle the color too
			Ptr<ASTOpColorSwizzle> NewColorOp = new ASTOpColorSwizzle;
			for (int i = 0; i < MUTABLE_OP_MAX_SWIZZLE_CHANNELS; ++i)
			{
				NewColorOp->Sources[i] = nop->color.child();
				NewColorOp->SourceChannels[i] = CurrentSwizzleOp->SourceChannels[i];
			}
			nop->color = NewColorOp;

			newAt = nop;
			break;
		}

		case EOpType::IM_DISPLACE:
		{
			Ptr<ASTOpImageDisplace> NewOp = UE::Mutable::Private::Clone<ASTOpImageDisplace>(at);

			Ptr<ASTOp> Child = NewOp->Source.child();
			Ptr<ASTOp> NewSource = Visit(Child, CurrentSwizzleOp);
			NewOp->Source = NewSource;

			newAt = NewOp;
			break;
		}

		case EOpType::IM_INVERT:
		{
			Ptr<ASTOpImageInvert> NewOp = UE::Mutable::Private::Clone<ASTOpImageInvert>(at);

			Ptr<ASTOp> Child = NewOp->Base.child();
			Ptr<ASTOp> NewSource = Visit(Child, CurrentSwizzleOp);
			NewOp->Base = NewSource;

			newAt = NewOp;
			break;
		}

		case EOpType::IM_RASTERMESH:
		{
			Ptr<ASTOpImageRasterMesh> NewOp = UE::Mutable::Private::Clone<ASTOpImageRasterMesh>(at);

			NewOp->image = Visit(NewOp->image.child(), CurrentSwizzleOp);

			// If we are swapping rgb and alphas, we need to correct some flags
			{
				// We should only find these two cases
				if (CurrentSwizzleOp->SourceChannels[0] == 3 || CurrentSwizzleOp->SourceChannels[1] == 3 || CurrentSwizzleOp->SourceChannels[2] == 3)
				{
					NewOp->bIsRGBFadingEnabled = NewOp->bIsAlphaFadingEnabled;
				}
				else if (CurrentSwizzleOp->Sources[3] && CurrentSwizzleOp->SourceChannels[3] < 3)
				{
					NewOp->bIsAlphaFadingEnabled = NewOp->bIsRGBFadingEnabled;
				}
			}

			newAt = NewOp;
			break;
		}

		case EOpType::IM_TRANSFORM:
		{
			Ptr<ASTOpImageTransform> nop = UE::Mutable::Private::Clone<ASTOpImageTransform>(at);
			nop->Base = Visit(nop->Base.child(), CurrentSwizzleOp);
			newAt = nop;
			break;
		}

		case EOpType::IM_PIXELFORMAT:
		{
			// If we are not changing channel order, just remove the swizzle and adjust the format.
			bool bSameChannelOrder = true;
			int32 NumChannelsInFormat = GetImageFormatData(CurrentSwizzleOp->Format).Channels;
			for (int32 c = 0; c < NumChannelsInFormat; ++c)
			{
				if (CurrentSwizzleOp->Sources[c] && CurrentSwizzleOp->SourceChannels[c] != c)
				{
					bSameChannelOrder = false;
				}
			}

			if (bSameChannelOrder)
			{
				Ptr<ASTOpImagePixelFormat> NewOp = UE::Mutable::Private::Clone<ASTOpImagePixelFormat>(at);
				NewOp->Format = CurrentSwizzleOp->Format;
				newAt = NewOp;
			}
			break;
		}

		case EOpType::IM_CONFORM:
		{
			// If we are not changing channel order, just remove the swizzle and adjust the format.
			bool bSameChannelOrder = true;
			int32 NumChannelsInFormat = GetImageFormatData(CurrentSwizzleOp->Format).Channels;
			for (int32 C = 0; C < NumChannelsInFormat; ++C)
			{
				if (CurrentSwizzleOp->Sources[C] && CurrentSwizzleOp->SourceChannels[C] != C)
				{
					bSameChannelOrder = false;
				}
			}

			if (bSameChannelOrder)
			{
				Ptr<ASTOpImageConform> NewOp = UE::Mutable::Private::Clone<ASTOpImageConform>(at);
				NewOp->ConformDescriptor.m_format = CurrentSwizzleOp->Format;
				NewOp->ConformDescriptor.FormatIfAlpha = EImageFormat::None;
				newAt = NewOp;
			}
			break;
		}
		case EOpType::IM_BLANKLAYOUT:
		{
			// We can remove the swizzle entirely.
			// It is not 100% equivalent, because blank layouts are initialized with 0,0,0,1 so the result could be
			// different, but those pixels shouldn't be used anyway.
			newAt = UE::Mutable::Private::Clone<ASTOp>(at);
			break;
		}

		default:
			break;
		}

		// end on tree branch, replace with format
		if (at == newAt && at != InitialSource)
		{
			UE::Mutable::Private::Ptr<ASTOpImageSwizzle> NewOp = UE::Mutable::Private::Clone<ASTOpImageSwizzle>(CurrentSwizzleOp);
			check(NewOp->GetOpType() == EOpType::IM_SWIZZLE);

			ReplaceAllSources(NewOp, at);

			newAt = NewOp;
		}

		OldToNew.Add({ at, CurrentSwizzleOp }, newAt);

		return newAt;
	}


}
