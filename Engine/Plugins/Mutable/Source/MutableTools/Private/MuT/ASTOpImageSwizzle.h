// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpImageSwizzle final : public ASTOp
	{
	public:

		ASTChild Sources[MUTABLE_OP_MAX_SWIZZLE_CHANNELS];

		uint8 SourceChannels[MUTABLE_OP_MAX_SWIZZLE_CHANNELS] = { 0,0,0,0 };

		EImageFormat Format = EImageFormat::None;

	public:

		ASTOpImageSwizzle();
		ASTOpImageSwizzle(const ASTOpImageSwizzle&) = delete;
		~ASTOpImageSwizzle();

		virtual EOpType GetOpType() const override { return EOpType::IM_SWIZZLE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;
		virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FImageDesc GetImageDesc(bool bReturnBestOption, FGetImageDescContext*) const override;
		virtual void GetLayoutBlockSize(int32* OutBlockX, int32* OutBlockY) override;
		virtual bool IsImagePlainConstant(FVector4f& OutColor) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

