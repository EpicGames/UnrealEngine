// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpImageTransform.h"

#include "Containers/Map.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	//-------------------------------------------------------------------------------------------------
	ASTOpImageTransform::ASTOpImageTransform()
		: Base(this)
		, OffsetX(this)
		, OffsetY(this)
		, ScaleX(this)
		, ScaleY(this)
		, Rotation(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpImageTransform::~ASTOpImageTransform()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpImageTransform::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			auto Other = static_cast<const ASTOpImageTransform*>(&OtherUntyped);
			return 
				Base 	 == Other->Base &&
				OffsetX  == Other->OffsetX &&
				OffsetY  == Other->OffsetY &&
				ScaleX 	 == Other->ScaleX &&
				ScaleY 	 == Other->ScaleY &&
				Rotation == Other->Rotation &&
				AddressMode == Other->AddressMode &&
				SizeX == Other->SizeX &&
				SizeY == Other->SizeY &&
				SourceSizeX == Other->SourceSizeX &&
				SourceSizeY == Other->SourceSizeY &&
				bKeepAspectRatio == Other->bKeepAspectRatio;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpImageTransform::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(OffsetX));
		Result = HashCombineFast(Result, GetTypeHash(OffsetY));
		Result = HashCombineFast(Result, GetTypeHash(ScaleX));
		Result = HashCombineFast(Result, GetTypeHash(ScaleY));
		Result = HashCombineFast(Result, GetTypeHash(Rotation));
		Result = HashCombineFast(Result, GetTypeHash(AddressMode));
		Result = HashCombineFast(Result, GetTypeHash(SizeX));
		Result = HashCombineFast(Result, GetTypeHash(SizeY));
		Result = HashCombineFast(Result, GetTypeHash(SourceSizeX));
		Result = HashCombineFast(Result, GetTypeHash(SourceSizeY));
		Result = HashCombineFast(Result, GetTypeHash(bKeepAspectRatio));
			
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageTransform::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageTransform> NewOp = new ASTOpImageTransform();
		NewOp->Base     = MapChild(Base.child());
		NewOp->OffsetX  = MapChild(OffsetX.child());
		NewOp->OffsetY  = MapChild(OffsetY.child());
		NewOp->ScaleX   = MapChild(ScaleX.child());
		NewOp->ScaleY   = MapChild(ScaleY.child());
		NewOp->Rotation = MapChild(Rotation.child());
		NewOp->AddressMode = AddressMode;
		NewOp->SizeX = SizeX;
		NewOp->SizeY = SizeY;
		NewOp->SourceSizeX = SourceSizeX;
		NewOp->SourceSizeY = SourceSizeY;
		NewOp->bKeepAspectRatio = bKeepAspectRatio;
		return NewOp;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageTransform::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(OffsetX);
		Func(OffsetY);
		Func(ScaleX);
		Func(ScaleY);
		Func(Rotation);
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageTransform::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::ImageTransformArgs Args;
			FMemory::Memzero(Args);

			Args.Base     = Base     ? Base->LinkedAddress     : 0;
			Args.OffsetX  = OffsetX  ? OffsetX->LinkedAddress  : 0;
			Args.OffsetY  = OffsetY  ? OffsetY->LinkedAddress  : 0;
			Args.ScaleX   = ScaleX   ? ScaleX->LinkedAddress   : 0;
			Args.ScaleY   = ScaleY   ? ScaleY->LinkedAddress   : 0;
			Args.Rotation = Rotation ? Rotation->LinkedAddress : 0;
			Args.AddressMode = static_cast<uint32>(AddressMode);
			Args.bKeepAspectRatio = bKeepAspectRatio;
			Args.SizeX = SizeX;
			Args.SizeY = SizeY;
			Args.SourceSizeX = SourceSizeX;
			Args.SourceSizeY = SourceSizeY;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpImageTransform::GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}


		// Actual work
		if (Base)
		{
			Result = Base->GetImageDesc(bReturnBestOption, Context);
			
			Result.m_format = GetUncompressedFormat(Result.m_format); 
			Result.FormatIfAlpha = GetUncompressedFormat(Result.FormatIfAlpha); 
			Result.m_lods = 1;
			
			if (!(SizeX == 0 && SizeY == 0))
			{
				Result.m_size = FImageSize(SizeX, SizeY);
			}
		}

		// Format overriden by current op.
		Context->bIsFormatFromImageParameter = false;

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpImageTransform::GetLayoutBlockSize(int* pBlockX, int* pBlockY)
	{
		if (Base)
		{
			Base->GetLayoutBlockSize(pBlockX, pBlockY);
		}
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpImageTransform::GetImageSizeExpression() const
	{
		if (Base)
		{
			if (!(SizeX == 0 && SizeY == 0))
			{
				Ptr<ImageSizeExpression> SizeExpr = new ImageSizeExpression;
				SizeExpr->type = ImageSizeExpression::ISET_CONSTANT;
				SizeExpr->size[0] = SizeX;
				SizeExpr->size[1] = SizeY;

				return SizeExpr;
			}
			else
			{
				return Base->GetImageSizeExpression();
			}
		}

		return nullptr;
	}


	FSourceDataDescriptor ASTOpImageTransform::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
