// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpImageTransform final : public ASTOp
	{
	public:

		ASTChild Base;
		ASTChild OffsetX;
		ASTChild OffsetY;
		ASTChild ScaleX;
		ASTChild ScaleY;
		ASTChild Rotation;

		uint16 SizeX = 0;
		uint16 SizeY = 0;

		uint16 SourceSizeX = 0;
		uint16 SourceSizeY = 0;

		EAddressMode AddressMode = EAddressMode::Wrap;
		bool bKeepAspectRatio = false;

	public:

		ASTOpImageTransform();
		ASTOpImageTransform(const ASTOpImageTransform&) = delete;
		~ASTOpImageTransform();

		virtual EOpType GetOpType() const override { return EOpType::IM_TRANSFORM; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& Program, FLinkerOptions* Options) override;
		virtual FImageDesc GetImageDesc(bool bReturnBestOption, FGetImageDescContext* Context) const override;
		virtual void GetLayoutBlockSize(int* pBlockX, int* pBlockY) override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}

