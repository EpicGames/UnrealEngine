// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpLODNew.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"


namespace UE::Mutable::Private
{
	ASTOpLODNew::ASTOpLODNew()
	{
	}
	

	ASTOpLODNew::~ASTOpLODNew()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	
	EOpType ASTOpLODNew::GetOpType() const
	{
		return EOpType::LD_NEW;
	}


	bool ASTOpLODNew::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpLODNew* Other = static_cast<const ASTOpLODNew*>(&OtherUntyped);
			return Meshes == Other->Meshes;
		}
		return false;
	}


	uint32 ASTOpLODNew::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Meshes));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpLODNew::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpLODNew> New = new ASTOpLODNew();
		
		for (const ASTChild& LOD : Meshes)
		{
			New->Meshes.Emplace(New, MapChild(LOD.child()));
		}
		
		return New;
	}


	void ASTOpLODNew::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		for (ASTChild& LOD : Meshes)
		{
			Func(LOD);
		}
	}


	void ASTOpLODNew::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}

		FOperation::FLODNewArgs Args;
		FMemory::Memzero(Args);
		
		TArray<FOperation::ADDRESS> LODAddresses;
		for (const ASTChild& Mesh : Meshes)
		{
			if (Mesh)
			{
				LODAddresses.Add(Mesh->LinkedAddress);
			}
		}
		
		Args.Meshes = Program.AddConstant(LODAddresses);

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), (FOperation::ADDRESS)Program.ByteCode.Num());

		AppendCode(Program.ByteCode, GetOpType()); 
		AppendCode(Program.ByteCode, Args);
	}
}
