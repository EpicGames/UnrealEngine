// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpLayoutFromMesh.h"

#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Mesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshFormat.h"
#include "MuT/ASTOpMeshApplyShape.h"
#include "MuT/ASTOpMeshBindShape.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpMeshMerge.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpLayoutMerge.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	ASTOpLayoutFromMesh::ASTOpLayoutFromMesh()
		: Mesh(this)
	{
	}


	ASTOpLayoutFromMesh::~ASTOpLayoutFromMesh()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpLayoutFromMesh::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpLayoutFromMesh* Other = static_cast<const ASTOpLayoutFromMesh*>(&OtherUntyped);
			return Mesh == Other->Mesh && LayoutIndex == Other->LayoutIndex;
		}
		return false;
	}


	uint32 ASTOpLayoutFromMesh::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(LayoutIndex));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpLayoutFromMesh::Clone(MapChildFuncRef MapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpLayoutFromMesh> New = new ASTOpLayoutFromMesh();
		New->Mesh = MapChild(Mesh.child());
		New->LayoutIndex = LayoutIndex;
		return New;
	}


	void ASTOpLayoutFromMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Mesh);
	}


	void ASTOpLayoutFromMesh::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::LayoutFromMeshArgs Args;
			FMemory::Memzero(Args);

			Args.LayoutIndex = LayoutIndex;
			if (Mesh) 
			{
				Args.Mesh = Mesh->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	void ASTOpLayoutFromMesh::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache)
	{
		// This shouldn't happen for this operation because it is always in a branch of layout operations that is not the main one.
		check(false);
	}


	namespace
	{

		/** Handle the optimization of a ASTOpLayoutFromMesh operation by moving it down its subtree. */
		class FSinkLayoutFromMesh
		{
		public:

			// \TODO This is recursive and may cause stack overflows in big models.
			UE::Mutable::Private::Ptr<ASTOp> Apply(const ASTOpLayoutFromMesh* InRoot)
			{
				InitialRoot = InRoot;
				check(InitialRoot);

				OldToNew.Empty();

				InitialSource = InitialRoot->Mesh.child();
				UE::Mutable::Private::Ptr<ASTOp> NewSource = Visit(InitialSource, InitialRoot);

				// If there is any change, it is the new root.
				if (NewSource != InitialSource)
				{
					return NewSource;
				}

				return nullptr;
			}


		protected:

			const class ASTOpLayoutFromMesh* InitialRoot = nullptr;
			Ptr<ASTOp> InitialSource;
			TMap<FSinkerOldToNewKey, Ptr<ASTOp>> OldToNew;

			UE::Mutable::Private::Ptr<ASTOp> Visit(const UE::Mutable::Private::Ptr<ASTOp>& at, const ASTOpLayoutFromMesh* CurrentSinkingOp)
			{
				if (!at) return nullptr;

				// Already visited?
				const Ptr<ASTOp>* Cached = OldToNew.Find({ at,CurrentSinkingOp });
				if (Cached)
				{
					return *Cached;
				}

				UE::Mutable::Private::Ptr<ASTOp> newAt = at;
				switch (at->GetOpType())
				{

				case EOpType::ME_MORPH:
				{
					// Sink, ignoring the op
					const ASTOpMeshMorph* Typed = static_cast<const ASTOpMeshMorph*>(at.get());
					newAt = Visit(Typed->Base.child(), CurrentSinkingOp);
					break;
				}

				case EOpType::ME_FORMAT:
				{
					// Sink, ignoring the op
					const ASTOpMeshFormat* Typed = static_cast<const ASTOpMeshFormat*>(at.get());
					newAt = Visit(Typed->Source.child(), CurrentSinkingOp);
					break;
				}

				case EOpType::ME_APPLYSHAPE:
				{
					// Sink, ignoring the op
					const ASTOpMeshApplyShape* Typed = static_cast<const ASTOpMeshApplyShape*>(at.get());
					newAt = Visit(Typed->Mesh.child(), CurrentSinkingOp);
					break;
				}

				case EOpType::ME_BINDSHAPE:
				{
					// Sink, ignoring the op
					const ASTOpMeshBindShape* Typed = static_cast<const ASTOpMeshBindShape*>(at.get());
					newAt = Visit(Typed->Mesh.child(), CurrentSinkingOp);
					break;
				}

				case EOpType::ME_ADDMETADATA:
				{
					// Sink, ignoring the op
					const ASTOpMeshAddMetadata* Typed = static_cast<const ASTOpMeshAddMetadata*>(at.get());
					newAt = Visit(Typed->Source.child(), CurrentSinkingOp);
					break;
				}

				case EOpType::ME_CONDITIONAL:
				{
					Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(at);
					NewConditional->type = EOpType::LA_CONDITIONAL;
					NewConditional->yes = Visit(NewConditional->yes.child(), CurrentSinkingOp);
					NewConditional->no = Visit(NewConditional->no.child(), CurrentSinkingOp);
					newAt = NewConditional;
					break;
				}

				case EOpType::ME_SWITCH:
				{
					Ptr<ASTOpSwitch> NewOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
					NewOp->Type = EOpType::LA_SWITCH;
					NewOp->Default = Visit(NewOp->Default.child(), CurrentSinkingOp);
					for (ASTOpSwitch::FCase& c : NewOp->Cases)
					{
						c.Branch = Visit(c.Branch.child(), CurrentSinkingOp);
					}
					newAt = NewOp;
					break;
				}

				case EOpType::ME_MERGE:
				{
					const ASTOpMeshMerge* Typed = static_cast<const ASTOpMeshMerge*>(at.get());

					Ptr<ASTOpLayoutMerge> NewMerge = new ASTOpLayoutMerge;
					NewMerge->Base = Visit(Typed->Base.child(), CurrentSinkingOp);
					NewMerge->Added = Visit(Typed->Added.child(), CurrentSinkingOp);
					newAt = NewMerge;
					break;
				}

				default:
					if (at != InitialSource)
					{
						UE::Mutable::Private::Ptr<ASTOpLayoutFromMesh> NewOp = UE::Mutable::Private::Clone<ASTOpLayoutFromMesh>(CurrentSinkingOp);
						NewOp->Mesh = at;
						newAt = NewOp;
					}
					break;

				}

				OldToNew.Add({ at,CurrentSinkingOp }, newAt);

				return newAt;
			}

		};

	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpLayoutFromMesh::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
	{
		MUTABLE_CPUPROFILER_SCOPE(ASTOpLayoutFromMesh_Sink);

		FSinkLayoutFromMesh Sinker;
		UE::Mutable::Private::Ptr<ASTOp> at = Sinker.Apply(this);

		return at;
	}

}

