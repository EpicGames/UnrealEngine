// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;
	
	class ASTOpLayoutFromMesh final : public ASTOp
	{
	public:

		ASTChild Mesh;
		uint8_t LayoutIndex = 0;

	public:

		ASTOpLayoutFromMesh();
		ASTOpLayoutFromMesh(const ASTOpLayoutFromMesh&) = delete;
		~ASTOpLayoutFromMesh();

		EOpType GetOpType() const override { return EOpType::LA_FROMMESH; }
		uint32 Hash() const override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const override;
		void GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache) override;

	};

}

