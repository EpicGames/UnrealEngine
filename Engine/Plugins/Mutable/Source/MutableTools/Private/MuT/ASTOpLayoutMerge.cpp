// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpLayoutMerge.h"

#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Mesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{


	ASTOpLayoutMerge::ASTOpLayoutMerge()
		: Base(this)
		, Added(this)
	{
	}


	ASTOpLayoutMerge::~ASTOpLayoutMerge()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpLayoutMerge::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpLayoutMerge* other = static_cast<const ASTOpLayoutMerge*>(&otherUntyped);
			return Base == other->Base && Added == other->Added;
		}
		return false;
	}


	uint32 ASTOpLayoutMerge::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Added));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpLayoutMerge::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpLayoutMerge> n = new ASTOpLayoutMerge();
		n->Base = mapChild(Base.child());
		n->Added = mapChild(Added.child());
		return n;
	}


	void ASTOpLayoutMerge::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Base);
		f(Added);
	}


	void ASTOpLayoutMerge::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::LayoutMergeArgs Args;
			FMemory::Memzero(Args);

			if (Base) 
			{
				Args.Base = Base->LinkedAddress;
			}

			if (Added) 
			{
				Args.Added = Added->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	void ASTOpLayoutMerge::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache)
	{
		if (Base)
		{
			Base->GetBlockLayoutSize(BlockId, pBlockX, pBlockY, cache);
		}

		if (!*pBlockX && Added)
		{
			Added->GetBlockLayoutSize(BlockId, pBlockX, pBlockY, cache);
		}
	}


}
