// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;
	
	class ASTOpLayoutMerge final : public ASTOp
	{
	public:

		ASTChild Base;
		ASTChild Added;

	public:

		ASTOpLayoutMerge();
		ASTOpLayoutMerge(const ASTOpLayoutMerge&) = delete;
		~ASTOpLayoutMerge();

		EOpType GetOpType() const override { return EOpType::LA_MERGE; }
		uint32 Hash() const override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		void GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache) override;

	};

}

