// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpLayoutPack.h"

#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Mesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{

	ASTOpLayoutPack::ASTOpLayoutPack()
		: Source(this)
	{
	}


	ASTOpLayoutPack::~ASTOpLayoutPack()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpLayoutPack::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpLayoutPack* other = static_cast<const ASTOpLayoutPack*>(&otherUntyped);
			return Source == other->Source;
		}
		return false;
	}


	uint32 ASTOpLayoutPack::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpLayoutPack::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpLayoutPack> n = new ASTOpLayoutPack();
		n->Source = mapChild(Source.child());
		return n;
	}


	void ASTOpLayoutPack::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
	}


	void ASTOpLayoutPack::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::LayoutRemoveBlocksArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	void ASTOpLayoutPack::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache)
	{
		if (Source)
		{
			Source->GetBlockLayoutSize(BlockId, pBlockX, pBlockY, cache);
		}
	}

}
