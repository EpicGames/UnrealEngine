// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpLayoutRemoveBlocks.h"

#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Mesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{

	ASTOpLayoutRemoveBlocks::ASTOpLayoutRemoveBlocks()
		: Source(this)
		, ReferenceLayout(this)
	{
	}


	ASTOpLayoutRemoveBlocks::~ASTOpLayoutRemoveBlocks()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpLayoutRemoveBlocks::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpLayoutRemoveBlocks* other = static_cast<const ASTOpLayoutRemoveBlocks*>(&otherUntyped);
			return Source == other->Source && ReferenceLayout == other->ReferenceLayout;
		}
		return false;
	}


	uint32 ASTOpLayoutRemoveBlocks::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(ReferenceLayout));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpLayoutRemoveBlocks::Clone(MapChildFuncRef mapChild) const
	{
		UE::Mutable::Private::Ptr<ASTOpLayoutRemoveBlocks> n = new ASTOpLayoutRemoveBlocks();
		n->Source = mapChild(Source.child());
		n->ReferenceLayout = mapChild(ReferenceLayout.child());
		return n;
	}


	void ASTOpLayoutRemoveBlocks::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
		f(ReferenceLayout);
	}


	void ASTOpLayoutRemoveBlocks::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::LayoutRemoveBlocksArgs Args;
			FMemory::Memzero(Args);

			if (Source) 
			{
				Args.Source = Source->LinkedAddress;
			}

			if (ReferenceLayout) 
			{
				Args.ReferenceLayout = ReferenceLayout->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	void ASTOpLayoutRemoveBlocks::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache)
	{
		if (Source)
		{
			Source->GetBlockLayoutSize(BlockId, pBlockX, pBlockY, cache);
		}
	}

}
