// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMaterialBreak.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{
	ASTOpMaterialBreak::ASTOpMaterialBreak()
		: Material(this)
	{
	}

	ASTOpMaterialBreak::~ASTOpMaterialBreak()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	void ASTOpMaterialBreak::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Material);
	}


	bool ASTOpMaterialBreak::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMaterialBreak* Other = static_cast<const ASTOpMaterialBreak*>(&OtherUntyped);
			return Type == Other->Type &&
				Material == Other->Material &&
				ParameterKey == Other->ParameterKey;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMaterialBreak::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMaterialBreak> Result = new ASTOpMaterialBreak();
		Result->Type = Type;
		Result->ParameterKey = ParameterKey;
		Result->Material = MapChild(Material.child());

		return Result;
	}


	EOpType ASTOpMaterialBreak::GetOpType() const
	{ 
		return Type; 
	}


	uint32 ASTOpMaterialBreak::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
	
		Result = HashCombineFast(Result, GetTypeHash(ParameterKey));
		Result = HashCombineFast(Result, GetTypeHash(Material));
		
		return Result;
	}


	void ASTOpMaterialBreak::Assert()
	{
		ASTOp::Assert();
	}


	void ASTOpMaterialBreak::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{

			FOperation::MaterialBreakArgs Args;
			FMemory::Memzero(Args);

			if (Material)
			{
				Args.Material = Material->LinkedAddress;
			}

			Args.ParameterName = Program.AddConstant(ParameterKey.ParameterName);
			Args.LayerIndex = ParameterKey.LayerIndex;


			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpMaterialBreak::GetImageDesc(bool returnBestOption, FGetImageDescContext* Context) const
	{
		check(Type == EOpType::IM_MATERIAL_BREAK);

		FImageDesc Result;

		if (Material)
		{
			// Local context in case it is necessary
			FGetImageDescContext LocalContext;
			if (!Context)
			{
				Context = &LocalContext;
			}

			Context->ParameterKey = ParameterKey;

			Result = Material->GetImageDesc(returnBestOption, Context);
		}

		return Result;
	}

	FSourceDataDescriptor ASTOpMaterialBreak::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Material)
		{
			return Material->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
