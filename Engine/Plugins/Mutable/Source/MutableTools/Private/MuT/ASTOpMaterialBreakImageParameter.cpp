// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMaterialBreakImageParameter.h"

#include "MuR/Types.h"


namespace UE::Mutable::Private
{
	ASTOpImageFromMaterialParameter::ASTOpImageFromMaterialParameter()
	{
	}

	ASTOpImageFromMaterialParameter::~ASTOpImageFromMaterialParameter()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	void ASTOpImageFromMaterialParameter::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
	}


	bool ASTOpImageFromMaterialParameter::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpImageFromMaterialParameter* Other = static_cast<const ASTOpImageFromMaterialParameter*>(&OtherUntyped);
			return MaterialParameter == Other->MaterialParameter &&
				ParameterKey == Other->ParameterKey;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpImageFromMaterialParameter::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpImageFromMaterialParameter> Result = new ASTOpImageFromMaterialParameter();
		Result->ParameterKey = ParameterKey;
		Result->MaterialParameter = MaterialParameter;

		return Result;
	}


	EOpType ASTOpImageFromMaterialParameter::GetOpType() const
	{ 
		return EOpType::IM_PARAMETER_FROM_MATERIAL;
	}


	uint32 ASTOpImageFromMaterialParameter::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(ParameterKey));
		Result = HashCombineFast(Result, GetTypeHash(MaterialParameter.Name));
		
		return Result;
	}


	void ASTOpImageFromMaterialParameter::Assert()
	{
		ASTOp::Assert();
	}


	void ASTOpImageFromMaterialParameter::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			int32 LinkedParameterIndex = Program.Parameters.Find(MaterialParameter);
			check(LinkedParameterIndex != INDEX_NONE); // check ASTOpParameter.cpp if this happens

			FOperation::MaterialBreakImageParameterArgs Args;
			FMemory::Memzero(Args);

			Args.MaterialParameter = FOperation::ADDRESS(LinkedParameterIndex);
			Args.ParameterName = Program.AddConstant(ParameterKey.ParameterName);
			Args.LayerIndex = ParameterKey.LayerIndex;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FImageDesc ASTOpImageFromMaterialParameter::GetImageDesc(bool, FGetImageDescContext*) const
	{
		check(GetOpType() == EOpType::IM_PARAMETER_FROM_MATERIAL);
		return FImageDesc();
	}
}
