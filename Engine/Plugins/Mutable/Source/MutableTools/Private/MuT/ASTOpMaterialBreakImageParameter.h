// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** Parameter operation. */
	class ASTOpImageFromMaterialParameter final : public ASTOp
	{
	public:

		FParameterDesc MaterialParameter;
		FParameterKey ParameterKey;

	public:

		ASTOpImageFromMaterialParameter();
		ASTOpImageFromMaterialParameter(const ASTOpImageFromMaterialParameter&) = delete;
		~ASTOpImageFromMaterialParameter() override;

		// ASTOp interface.
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Assert() override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool, FGetImageDescContext*) const override;

	};

}

