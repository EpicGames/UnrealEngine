// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMaterialModify.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "HAL/PlatformMath.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/CodeOptimiser.h"


namespace UE::Mutable::Private
{
	ASTOpMaterialModify::ASTOpMaterialModify()
		: Material(this)
	{
	}

	ASTOpMaterialModify::~ASTOpMaterialModify()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	void ASTOpMaterialModify::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Material);

		for (TPair<FParameterKey, ASTChild>& Parameter : ParametersToModify)
		{
			f(Parameter.Value);
		}
	}


	bool ASTOpMaterialModify::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMaterialModify* Other = static_cast<const ASTOpMaterialModify*>(&OtherUntyped);
			return GetOpType() == Other->GetOpType() &&
				Material == Other->Material &&
				ParametersToModify == Other->ParametersToModify &&
				ImagePropertyIndexMap == Other->ImagePropertyIndexMap;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMaterialModify::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMaterialModify> Result = new ASTOpMaterialModify();
		Result->Material = MapChild(Material.child());

		//Clone array of parameters
		Result->ParametersToModify.Reserve(ParametersToModify.Num());
		for (const TPair<FParameterKey, ASTChild>& Parameter : ParametersToModify)
		{
			ASTChild& ClonedParam = Result->ParametersToModify.Emplace(Parameter.Key, Result.get());
			ClonedParam = MapChild(Parameter.Value.child());
		}

		Result->ImagePropertyIndexMap = ImagePropertyIndexMap;
		
		return Result;
	}


	EOpType ASTOpMaterialModify::GetOpType() const
	{ 
		return EOpType::MI_MODIFY;
	}


	uint32 ASTOpMaterialModify::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Material));

		for (const TPair<FParameterKey, ASTChild>& Parameter : ParametersToModify)
		{
			Result = HashCombineFast(Result, GetTypeHash(Parameter.Key));
			Result = HashCombineFast(Result, GetTypeHash(Parameter.Value));
		}

		for (const TPair<FParameterKey, int32>& Parameter : ImagePropertyIndexMap)
		{
			Result = HashCombineFast(Result, GetTypeHash(Parameter.Key));
			Result = HashCombineFast(Result, GetTypeHash(Parameter.Value));
		}

		return Result;
	}


	void ASTOpMaterialModify::Assert()
	{
		ASTOp::Assert();
	}


	void ASTOpMaterialModify::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MaterialModifyArgs Args;
			FMemory::Memzero(Args);

			if (Material)
			{
				Args.Material = Material->LinkedAddress;
			}

			Args.NumParameters = ParametersToModify.Num();

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);

			//Store parameters info directly into the bytecode
			for (const TPair<FParameterKey, ASTChild>& Parameter : ParametersToModify)
			{
				//Parameter Name
				AppendCode(Program.ByteCode, Program.AddConstant(Parameter.Key.ParameterName));
				
				// Layer Index
				AppendCode(Program.ByteCode, Parameter.Key.LayerIndex);

				// Check if exists a child operation
				if (Parameter.Value)
				{
					AppendCode(Program.ByteCode, Parameter.Value->LinkedAddress);

					// Only images need this step
					if (GetOpDataType(Parameter.Value.Child->GetOpType()) == EDataType::Image)
					{
						int32* ImagePropertyIndex = ImagePropertyIndexMap.Find(Parameter.Key);
						check(ImagePropertyIndex) // All images must have this index

						AppendCode(Program.ByteCode, *ImagePropertyIndex);

						// Code from the removed operation IN_ADDIMAGE:
						// Find out relevant parameters. \todo: this may be optimised by reusing partial
						// values in a LINK_CONTEXT or similar
						SubtreeRelevantParametersVisitorAST visitor;
						visitor.Run(Parameter.Value.child());

						TArray<uint16> params;
						for (const FString& paramName : visitor.Parameters)
						{
							for (int32 i = 0; i < Program.Parameters.Num(); ++i)
							{
								const auto& param = Program.Parameters[i];
								if (param.Name == paramName)
								{
									params.Add(uint16(i));
									break;
								}
							}
						}

						params.Sort();

						auto it = Program.ParameterLists.Find(params);

						if (it != INDEX_NONE)
						{
							Program.RelevantParameterList.Add(Parameter.Value->LinkedAddress, it);
						}
						else
						{
							Program.RelevantParameterList.Add(Parameter.Value->LinkedAddress, Program.ParameterLists.Num());
							Program.ParameterLists.Add(params);
						}
					}
				}
				else // If the child is not valid, means that we want to delete the already existing value of the parameter
				{
					AppendCode(Program.ByteCode, 0);
				}
			}
		}
	}


	FImageDesc ASTOpMaterialModify::GetImageDesc(bool returnBestOption, FGetImageDescContext* Context) const
	{
		check(GetOpType() == EOpType::MI_MODIFY);

		FImageDesc Result;

		if (Context)
		{
			if (const ASTChild* Image = ParametersToModify.Find(Context->ParameterKey))
			{
				if (Image->Child)
				{
					check(GetOpDataType(Image->Child->GetOpType()) == EDataType::Image);
					Result = Image->Child->GetImageDesc();
				}
			}
		}

		return Result;
	}


	FSourceDataDescriptor ASTOpMaterialModify::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Material)
		{
			return Material->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}

