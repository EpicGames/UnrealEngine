// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/AST.h"

namespace UE::Mutable::Private
{
	class ASTOpMaterialModify final : public ASTOp
	{
	public:

		/** Parent Material */
		ASTChild Material;

		/** Parameters to modify. A nullptr value means that, if this operation is linked to a previous Modify operation,
			the value set in that operation will be reset to its default value (the value set in the material)	*/
		TMap<FParameterKey, ASTChild> ParametersToModify;

		/** Index to the ImageProperties array of the model resources */
		TMap<FParameterKey, int32> ImagePropertyIndexMap; //Cannot create a structure due to the ASTChild constructor restrictions

	public:

		ASTOpMaterialModify();
		ASTOpMaterialModify(const ASTOpMaterialModify&) = delete;
		~ASTOpMaterialModify() override;

		// ASTOp interface.
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Assert() override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool, FGetImageDescContext*) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	
	};
}
