// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpMaterialSkeletalMeshBreak.h"


namespace UE::Mutable::Private
{
	ASTOpMaterialSkeletalMeshBreak::ASTOpMaterialSkeletalMeshBreak()
		: SkeletalMesh(this)
	{
		
	}

	ASTOpMaterialSkeletalMeshBreak::~ASTOpMaterialSkeletalMeshBreak()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	EOpType ASTOpMaterialSkeletalMeshBreak::GetOpType() const
	{
		return EOpType::MI_SKELETALMESH_BREAK;
	}

	uint32 ASTOpMaterialSkeletalMeshBreak::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(SlotName));
		Result = HashCombineFast(Result, GetTypeHash(SkeletalMesh));
		
		return Result;
	}

	void ASTOpMaterialSkeletalMeshBreak::ForEachChild(const TFunctionRef<void(ASTChild&)> F)
	{
		F(SkeletalMesh);
	}

	bool ASTOpMaterialSkeletalMeshBreak::IsEqual(const ASTOp& InOtherUUntyped) const
	{
		if (InOtherUUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMaterialSkeletalMeshBreak* Other = static_cast<const ASTOpMaterialSkeletalMeshBreak*>(&InOtherUUntyped);
			return SlotName == Other->SlotName && SkeletalMesh == Other->SkeletalMesh;
		}

		return false;
	}

	Ptr<ASTOp> ASTOpMaterialSkeletalMeshBreak::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMaterialSkeletalMeshBreak> New = new ASTOpMaterialSkeletalMeshBreak();
		New->SkeletalMesh = MapChild(SkeletalMesh.child());
		New->SlotName = SlotName;
		
		return New;
	}

	void ASTOpMaterialSkeletalMeshBreak::Link(FProgram& Program, FLinkerOptions* LinkerOptions)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}
		
		FOperation::MaterialSkeletalMeshBreakArgs Args;
		FMemory::Memzero(Args);

		Args.SlotName = Program.AddConstant(SlotName);
		
		if (SkeletalMesh)
		{
			Args.SkeletalMesh = SkeletalMesh->LinkedAddress;
		}

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}


