// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	class NodeSkeletalMesh;
	struct FProgram;
	
	class ASTOpMaterialSkeletalMeshBreak final : public ASTOp
	{
	public:

		/** The name of the material slot to get from the given mesh. */
		FName SlotName;

		/** The skeletal mesh to use as source when looking for the material with the target "SlotName"*/
		ASTChild SkeletalMesh;

		ASTOpMaterialSkeletalMeshBreak();
		virtual ~ASTOpMaterialSkeletalMeshBreak() override;

		// ASTOp interface.
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
	};
}

