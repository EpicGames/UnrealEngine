// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpMaterialSkeletalMeshObjectBreak.h"


namespace UE::Mutable::Private
{
	ASTOpMaterialSkeletalMeshObjectBreak::ASTOpMaterialSkeletalMeshObjectBreak()
		: SkeletalMeshObject(this)
	{
		
	}

	ASTOpMaterialSkeletalMeshObjectBreak::~ASTOpMaterialSkeletalMeshObjectBreak()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	EOpType ASTOpMaterialSkeletalMeshObjectBreak::GetOpType() const
	{
		return EOpType::MI_SKELETALMESHOBJECT_BREAK;
	}

	uint32 ASTOpMaterialSkeletalMeshObjectBreak::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(SlotName));
		Result = HashCombineFast(Result, GetTypeHash(SkeletalMeshObject));
		
		return Result;
	}

	void ASTOpMaterialSkeletalMeshObjectBreak::ForEachChild(const TFunctionRef<void(ASTChild&)> F)
	{
		F(SkeletalMeshObject);
	}

	bool ASTOpMaterialSkeletalMeshObjectBreak::IsEqual(const ASTOp& InOtherUUntyped) const
	{
		if (InOtherUUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMaterialSkeletalMeshObjectBreak* Other = static_cast<const ASTOpMaterialSkeletalMeshObjectBreak*>(&InOtherUUntyped);
			return SlotName == Other->SlotName && SkeletalMeshObject == Other->SkeletalMeshObject;
		}

		return false;
	}

	Ptr<ASTOp> ASTOpMaterialSkeletalMeshObjectBreak::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMaterialSkeletalMeshObjectBreak> New = new ASTOpMaterialSkeletalMeshObjectBreak();
		New->SkeletalMeshObject = MapChild(SkeletalMeshObject.child());
		New->SlotName = SlotName;
		
		return New;
	}

	void ASTOpMaterialSkeletalMeshObjectBreak::Link(FProgram& Program, FLinkerOptions* LinkerOptions)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}
		
		FOperation::MaterialSkeletalMeshObjectBreakArgs Args;
		FMemory::Memzero(Args);

		Args.SlotName = Program.AddConstant(SlotName);
		
		if (SkeletalMeshObject)
		{
			Args.SkeletalMeshObject = SkeletalMeshObject->LinkedAddress;
		}

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}


