// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshAddMetadata.h"

#include "MuT/ASTOpMeshMorph.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpMeshAddMetadata::ASTOpMeshAddMetadata()
		: Source(this)
	{
	}


	ASTOpMeshAddMetadata::~ASTOpMeshAddMetadata()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshAddMetadata::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshAddMetadata* Other = static_cast<const ASTOpMeshAddMetadata*>(&OtherUntyped);
			return
				Source == Other->Source &&
				GameplayTags == Other->GameplayTags &&
				AssetUserDataIds == Other->AssetUserDataIds &&
				AnimationSlots == Other->AnimationSlots &&
				SkeletonId == Other->SkeletonId &&
				BonePosePriority == Other->BonePosePriority &&
				SocketPriority == Other->SocketPriority &&
				Sockets == Other->Sockets && 
				PhysicsAssetId == Other->PhysicsAssetId &&
				AdditionalPhysicsAssetIds == Other->AdditionalPhysicsAssetIds &&
				RealTimeMorphNames == Other->RealTimeMorphNames;
		}

		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshAddMetadata::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshAddMetadata> NewOp = new ASTOpMeshAddMetadata();
		NewOp->Source = MapChild(Source.child());
		NewOp->SkeletonId = SkeletonId;
		NewOp->GameplayTags = GameplayTags;
		NewOp->AssetUserDataIds = AssetUserDataIds;
		NewOp->AnimationSlots = AnimationSlots;
		NewOp->BonePosePriority = BonePosePriority;
		NewOp->SocketPriority = SocketPriority;
		NewOp->Sockets = Sockets;
		NewOp->PhysicsAssetId = PhysicsAssetId;
		NewOp->AdditionalPhysicsAssetIds = AdditionalPhysicsAssetIds;
		NewOp->RealTimeMorphNames = RealTimeMorphNames;
		return NewOp;
	}


	void ASTOpMeshAddMetadata::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
	}


	uint32 ASTOpMeshAddMetadata::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));

		return Result;
	}


	void ASTOpMeshAddMetadata::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{	
			FOperation::MeshAddMetadataArgs Args;
			FMemory::Memzero(Args);
		
			Args.Source = Source ? Source->LinkedAddress : 0;
			
			using OpEnumFlags = FOperation::MeshAddMetadataArgs::EnumFlags;

			Args.Flags = OpEnumFlags::None;
			
			if (GameplayTags.Num())
			{
				EnumAddFlags(Args.Flags, OpEnumFlags::HasGameplayTags);

				if (GameplayTags.Num() > 1)
				{
					EnumAddFlags(Args.Flags, OpEnumFlags::IsGameplayTagList);
					Args.GameplayTags.ListIndex = Program.AddConstant(GameplayTags);
				}
				else
				{
					Args.GameplayTags.NameIndex = Program.AddConstant(GameplayTags[0]);
				}
			}

			if (AssetUserDataIds.Num())
			{
				EnumAddFlags(Args.Flags, OpEnumFlags::HasAssetUserData);

				if (AssetUserDataIds.Num() > 1)
				{
					EnumAddFlags(Args.Flags, OpEnumFlags::IsAssetUserDataList);
					Args.AssetUserDataIds.ListAddress = Program.AddConstant(AssetUserDataIds);
				}
				else
				{
					Args.AssetUserDataIds.PassthroughId = AssetUserDataIds[0];
				}
			}
			
			if (AnimationSlots.Num())
			{
				EnumAddFlags(Args.Flags, OpEnumFlags::HasAnimationSlots);

				if (AnimationSlots.Num() > 1)
				{
					TArray<FName> AnimSlotNames;
					TArray<PASSTHROUGH_ID> AnimInstances;
					
					for (const TPair<FName, PASSTHROUGH_ID>& AnimationSlot : AnimationSlots)
					{
						AnimSlotNames.Add(AnimationSlot.Key);
						AnimInstances.Add(AnimationSlot.Value);
					}
					
					EnumAddFlags(Args.Flags, OpEnumFlags::IsAnimationSlotList);
					Args.AnimSlotNames.ListIndex = Program.AddConstant(AnimSlotNames);
					Args.AnimInstances.ListAddress = Program.AddConstant(AnimInstances);
					
				}
				else
				{
					Args.AnimSlotNames.NameIndex = Program.AddConstant(AnimationSlots[0].Key);
					Args.AnimInstances.PassthroughId = AnimationSlots[0].Value;
				}
			}

			if (RealTimeMorphNames.Num())
			{
				EnumAddFlags(Args.Flags, OpEnumFlags::HasRealTimeMorphNames);

				if (RealTimeMorphNames.Num() > 1)
				{
					EnumAddFlags(Args.Flags, OpEnumFlags::IsRealTimeMorphNamesList);
					Args.RealTimeMorphNames.ListAddress = Program.AddConstant(RealTimeMorphNames);
				}
				else
				{
					Args.RealTimeMorphNames.StringAddress = Program.AddConstant(RealTimeMorphNames[0]);
				}
			}

			Args.SkeletonId = SkeletonId;
			Args.BonePosePriority = BonePosePriority;
			Args.SocketPriority = SocketPriority;

			if (Sockets.Num())
			{
				EnumAddFlags(Args.Flags, OpEnumFlags::HasSockets);

				if (Sockets.Num() > 1)
				{
					EnumAddFlags(Args.Flags, OpEnumFlags::IsSocketsList);
					Args.Sockets.ListAddress = Program.AddConstant(Sockets);
				}
				else
				{
					Args.Sockets.SocketId = Program.AddConstant(Sockets[0]);
				}
			}

			Args.PhysicsAssetId = PhysicsAssetId;

			if (AdditionalPhysicsAssetIds.Num())
			{
				EnumAddFlags(Args.Flags, OpEnumFlags::HasAdditionalPhysicsAsset);
				if (AdditionalPhysicsAssetIds.Num() > 1)
				{
					EnumAddFlags(Args.Flags, OpEnumFlags::IsAdditionalPhysicsAssetList);
					Args.AdditionalPhysicsAssetsIds.ListAddress = Program.AddConstant(AdditionalPhysicsAssetIds);
				}
				else
				{
					Args.AdditionalPhysicsAssetsIds.PassthroughId = AdditionalPhysicsAssetIds[0];
				}
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FSourceDataDescriptor ASTOpMeshAddMetadata::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
