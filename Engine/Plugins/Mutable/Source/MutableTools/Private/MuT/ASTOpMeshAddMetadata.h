// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"



namespace UE::Mutable::Private
{
	struct FProgram;

	/** Add metadata to a mesh. */
	class ASTOpMeshAddMetadata final : public ASTOp
	{
	public:

		/** Source mesh to add tags to. */
		ASTChild Source;

		/** Tags to add. */
		TArray<FName> GameplayTags;
	
		TArray<PASSTHROUGH_ID> AssetUserDataIds;

		TArray<TPair<FName, PASSTHROUGH_ID>> AnimationSlots;
		
		PASSTHROUGH_ID AnimInstanceId = PASSTHROUGH_ID_INVALID;
		
		/** SkeletonId to add. */
		PASSTHROUGH_ID SkeletonId = PASSTHROUGH_ID_INVALID;

		TArray<FString> RealTimeMorphNames;

		/** Bone Pose priority when merging two meshes. */
		int8 BonePosePriority = 0;

		/** Socket priority when merging two meshes. */
		int8 SocketPriority = 0;

		/** Array of sockets. */
		TArray<FMeshSocket> Sockets;

		/** PhysicsAsset to add */
		PASSTHROUGH_ID PhysicsAssetId = PASSTHROUGH_ID_INVALID;

		/** Additional PhysicsAssets */
		TArray<PASSTHROUGH_ID> AdditionalPhysicsAssetIds;
	
		ASTOpMeshAddMetadata();
		ASTOpMeshAddMetadata(const ASTOpMeshAddMetadata&) = delete;
		virtual ~ASTOpMeshAddMetadata() override;

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::ME_ADDMETADATA; }
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void Link(FProgram& Program, FLinkerOptions*) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}

