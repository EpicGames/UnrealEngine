// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshApplyLayout.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpMeshAddMetadata.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpMeshApplyLayout::ASTOpMeshApplyLayout()
		: Mesh(this)
		, Layout(this)
	{
	}


	ASTOpMeshApplyLayout::~ASTOpMeshApplyLayout()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshApplyLayout::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshApplyLayout* Other = static_cast<const ASTOpMeshApplyLayout*>(&OtherUntyped);
			return Mesh == Other->Mesh &&
				Layout == Other->Layout &&
				Channel == Other->Channel;
		}
		return false;
	}


	uint32 ASTOpMeshApplyLayout::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(Layout));
		Result = HashCombineFast(Result, GetTypeHash(Channel));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpMeshApplyLayout::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshApplyLayout> New = new ASTOpMeshApplyLayout();
		New->Mesh = MapChild(Mesh.child());
		New->Layout = MapChild(Layout.child());
		New->Channel = Channel;
		return New;
	}


	void ASTOpMeshApplyLayout::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Mesh);
		Func(Layout);
	}


	void ASTOpMeshApplyLayout::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshApplyLayoutArgs Args;
			FMemory::Memzero(Args);

			if (Mesh) 
			{
				Args.Mesh = Mesh->LinkedAddress;
			}

			if (Layout) 
			{
				Args.Layout = Layout->LinkedAddress;
			}

			Args.Channel = Channel;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	Ptr<ASTOp> ASTOpMeshApplyLayout::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		Ptr<ASTOp> MeshAt = Mesh.child();

		if (!MeshAt)
		{
			return nullptr;
		}

		EOpType MeshOpType = MeshAt->GetOpType();
		
		bool bLayoutIsConstant = !Layout || Layout->GetOpType() == EOpType::LA_CONSTANT;

		switch (MeshOpType)
		{
			case EOpType::ME_ADDMETADATA:
			{
				Ptr<ASTOpMeshAddMetadata> New = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(MeshAt);
				if (New->Source)
				{
					Ptr<ASTOpMeshApplyLayout> NewApplyPose = UE::Mutable::Private::Clone<ASTOpMeshApplyLayout>(this);
					NewApplyPose->Mesh = New->Source.child();
					New->Source = NewApplyPose;
				}

				NewOp = New;
				break;
			}

			case EOpType::ME_SWITCH:
			{
				if (bLayoutIsConstant)
				{
					// Move the apply down all the paths
					Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(MeshAt);

					if (NewSwitch->Default)
					{
						Ptr<ASTOpMeshApplyLayout> defOp = UE::Mutable::Private::Clone<ASTOpMeshApplyLayout>(this);
						defOp->Mesh = NewSwitch->Default.child();
						NewSwitch->Default = defOp;
					}

					// We need to copy the options because we change them
					for (int32 CaseIndex = 0; CaseIndex < NewSwitch->Cases.Num(); ++CaseIndex)
					{
						if (NewSwitch->Cases[CaseIndex].Branch)
						{
							Ptr<ASTOpMeshApplyLayout> bOp = UE::Mutable::Private::Clone<ASTOpMeshApplyLayout>(this);
							bOp->Mesh = NewSwitch->Cases[CaseIndex].Branch.child();
							NewSwitch->Cases[CaseIndex].Branch = bOp;
						}
					}

					NewOp = NewSwitch;
				}
				break;
			}


			default:
				break;
		}

		return NewOp;
	}


	FSourceDataDescriptor ASTOpMeshApplyLayout::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
