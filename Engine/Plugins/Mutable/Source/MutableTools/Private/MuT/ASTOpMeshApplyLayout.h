// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpMeshApplyLayout final : public ASTOp
	{
	public:

		ASTChild Mesh;
		ASTChild Layout;
		uint16 Channel=0;

	public:

		ASTOpMeshApplyLayout();
		ASTOpMeshApplyLayout(const ASTOpMeshApplyLayout&) = delete;
		~ASTOpMeshApplyLayout();

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::ME_APPLYLAYOUT; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};


}

