// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshApplyPose.h"
#include "MuT/ASTOpMeshAddMetadata.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpMeshApplyPose::ASTOpMeshApplyPose()
		: Base(this)
		, Pose(this)
	{
	}


	ASTOpMeshApplyPose::~ASTOpMeshApplyPose()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshApplyPose::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshApplyPose* Other = static_cast<const ASTOpMeshApplyPose*>(&OtherUntyped);
			return Base == Other->Base &&
				Pose == Other->Pose;
		}
		return false;
	}


	uint32 ASTOpMeshApplyPose::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Pose));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpMeshApplyPose::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshApplyPose> New = new ASTOpMeshApplyPose();
		New->Base = MapChild(Base.child());
		New->Pose = MapChild(Pose.child());
		return New;
	}


	void ASTOpMeshApplyPose::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(Pose);
	}


	void ASTOpMeshApplyPose::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshApplyPoseArgs Args;
			FMemory::Memzero(Args);

			if (Base) 
			{
				Args.base = Base->LinkedAddress;
			}

			if (Pose) 
			{
				Args.pose = Pose->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	Ptr<ASTOp> ASTOpMeshApplyPose::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		Ptr<ASTOp> MeshAt = Base.child();

		if (!MeshAt)
		{
			return nullptr;
		}

		EOpType MeshType = MeshAt->GetOpType();

		switch (MeshType)
		{
			case EOpType::ME_ADDMETADATA:
			{
				Ptr<ASTOpMeshAddMetadata> New = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(MeshAt);
				if (New->Source)
				{
					Ptr<ASTOpMeshApplyPose> NewApplyPose = UE::Mutable::Private::Clone<ASTOpMeshApplyPose>(this);
					NewApplyPose->Base = New->Source.child();
					New->Source = NewApplyPose;
				}

				NewOp = New;
				break;
			}

			default:
				break;
		}

		return NewOp;
	}

	FSourceDataDescriptor ASTOpMeshApplyPose::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
