// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshApplyShape.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	ASTOpMeshApplyShape::ASTOpMeshApplyShape()
		: Mesh(this)
		, Shape(this)
		, bRecomputeNormals(false)
		, bReshapeSkeleton(false)
		, bReshapePhysicsVolumes(false)
		, bReshapeVertices(true)
		, bApplyLaplacian(false)
	{
	}


	ASTOpMeshApplyShape::~ASTOpMeshApplyShape()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshApplyShape::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshApplyShape* Other = static_cast<const ASTOpMeshApplyShape*>(&OtherUntyped);

			const bool bSameFlags =
				bRecomputeNormals == Other->bRecomputeNormals &&
				bReshapePhysicsVolumes == Other->bReshapePhysicsVolumes &&
				bReshapeSkeleton == Other->bReshapeSkeleton &&
				bReshapeVertices == Other->bReshapeVertices &&
				bApplyLaplacian == Other->bApplyLaplacian;

			return Mesh == Other->Mesh && Shape == Other->Shape && bSameFlags;
		}
		return false;
	}


	uint32 ASTOpMeshApplyShape::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(Shape));
		Result = HashCombineFast(Result, GetTypeHash(bRecomputeNormals));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeSkeleton));
		Result = HashCombineFast(Result, GetTypeHash(bReshapePhysicsVolumes));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeVertices));
		Result = HashCombineFast(Result, GetTypeHash(bApplyLaplacian));

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshApplyShape::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshApplyShape> NewOp = new ASTOpMeshApplyShape();
		NewOp->Mesh = MapChild(Mesh.child());
		NewOp->Shape = MapChild(Shape.child());
		NewOp->bRecomputeNormals = bRecomputeNormals;
		NewOp->bReshapeSkeleton = bReshapeSkeleton;
		NewOp->bReshapePhysicsVolumes = bReshapePhysicsVolumes;
		NewOp->bReshapeVertices = bReshapeVertices;
		NewOp->bApplyLaplacian = bApplyLaplacian;
		return NewOp;
	}


	void ASTOpMeshApplyShape::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Mesh);
		Func(Shape);
	}


	void ASTOpMeshApplyShape::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshApplyShapeArgs Args;
			FMemory::Memzero(Args);

			constexpr EMeshBindShapeFlags NoFlags = EMeshBindShapeFlags::None;
			EMeshBindShapeFlags BindFlags = NoFlags;
			EnumAddFlags(BindFlags, bRecomputeNormals ? EMeshBindShapeFlags::RecomputeNormals : NoFlags);
			EnumAddFlags(BindFlags, bReshapeSkeleton ? EMeshBindShapeFlags::ReshapeSkeleton : NoFlags);
			EnumAddFlags(BindFlags, bReshapePhysicsVolumes ? EMeshBindShapeFlags::ReshapePhysicsVolumes : NoFlags);
			EnumAddFlags(BindFlags, bReshapeVertices ? EMeshBindShapeFlags::ReshapeVertices : NoFlags);
			EnumAddFlags(BindFlags, bApplyLaplacian ? EMeshBindShapeFlags::ApplyLaplacian : NoFlags);

			Args.flags = static_cast<uint32>(BindFlags);

			Args.mesh = Mesh ? Mesh->LinkedAddress : 0;
			Args.shape = Shape ? Shape->LinkedAddress : 0;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FSourceDataDescriptor ASTOpMeshApplyShape::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
