// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;

	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpMeshApplyShape final : public ASTOp
	{
	public:

		ASTChild Mesh;
		ASTChild Shape;

		uint32 bRecomputeNormals	  : 1;
		uint32 bReshapeSkeleton       : 1;
		uint32 bReshapePhysicsVolumes : 1;
		uint32 bReshapeVertices       : 1;
		uint32 bApplyLaplacian        : 1;
	public:

		ASTOpMeshApplyShape();
		ASTOpMeshApplyShape(const ASTOpMeshApplyShape&) = delete;
		~ASTOpMeshApplyShape();

		EOpType GetOpType() const override { return EOpType::ME_APPLYSHAPE; }
		uint32 Hash() const override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}

