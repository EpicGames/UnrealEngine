// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshBindShape.h"

#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpMeshAddMetadata.h"


namespace UE::Mutable::Private
{
	ASTOpMeshBindShape::ASTOpMeshBindShape()
		: Mesh(this)
		, Shape(this)
		, bRecomputeNormals(false)
		, bReshapeSkeleton(false)
		, bReshapePhysicsVolumes(false)
		, bReshapeVertices(false)
		, bApplyLaplacian(false)
	{
	}


	ASTOpMeshBindShape::~ASTOpMeshBindShape()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshBindShape::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshBindShape* Other = static_cast<const ASTOpMeshBindShape*>(&OtherUntyped);

			const bool bSameFlags =
				bRecomputeNormals == Other->bRecomputeNormals &&
				bReshapeSkeleton == Other->bReshapeSkeleton	&&
				bReshapePhysicsVolumes == Other->bReshapePhysicsVolumes &&
				bReshapeVertices == Other->bReshapeVertices &&
				bApplyLaplacian == Other->bApplyLaplacian &&
				bReshapeSkeletonInvertSelection == Other->bReshapeSkeletonInvertSelection &&
				bReshapePhysicsVolumesInvertSelection == Other->bReshapePhysicsVolumesInvertSelection;

			return bSameFlags &&
				Mesh == Other->Mesh &&
				Shape == Other->Shape &&
				BonesToDeform == Other->BonesToDeform &&
				PhysicsToDeform == Other->PhysicsToDeform &&
				BindingMethod == Other->BindingMethod &&
				RChannelUsage == Other->RChannelUsage && 
				GChannelUsage == Other->GChannelUsage && 
				BChannelUsage == Other->BChannelUsage && 
				AChannelUsage == Other->AChannelUsage;
		}

		return false;
	}


	uint32 ASTOpMeshBindShape::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(Shape));
		Result = HashCombineFast(Result, GetTypeHash(bRecomputeNormals));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeSkeleton));
		Result = HashCombineFast(Result, GetTypeHash(bReshapePhysicsVolumes));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeVertices));
		Result = HashCombineFast(Result, GetTypeHash(bApplyLaplacian));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeSkeletonInvertSelection));
		Result = HashCombineFast(Result, GetTypeHash(bReshapePhysicsVolumesInvertSelection));
		Result = HashCombineFast(Result, GetTypeHash(BindingMethod));
		Result = HashCombineFast(Result, GetTypeHash(RChannelUsage));
		Result = HashCombineFast(Result, GetTypeHash(GChannelUsage));
		Result = HashCombineFast(Result, GetTypeHash(BChannelUsage));
		Result = HashCombineFast(Result, GetTypeHash(AChannelUsage));

		for (FName Bone : BonesToDeform)
		{
			Result = HashCombineFast(Result, GetTypeHash(Bone));
		}

		for (FName PhysicsBone : PhysicsToDeform)
		{
			Result = HashCombineFast(Result, GetTypeHash(PhysicsBone));
		}

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshBindShape::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshBindShape> NewOp = new ASTOpMeshBindShape();
		NewOp->Mesh = mapChild(Mesh.child());
		NewOp->Shape = mapChild(Shape.child());
		NewOp->bRecomputeNormals = bRecomputeNormals;
		NewOp->bReshapeSkeleton	= bReshapeSkeleton;
		NewOp->bReshapePhysicsVolumes = bReshapePhysicsVolumes;
		NewOp->bReshapeVertices = bReshapeVertices;
		NewOp->bApplyLaplacian = bApplyLaplacian;
		NewOp->bReshapeSkeletonInvertSelection = bReshapeSkeletonInvertSelection;
		NewOp->bReshapePhysicsVolumesInvertSelection = bReshapePhysicsVolumesInvertSelection;
		NewOp->BonesToDeform = BonesToDeform;
		NewOp->PhysicsToDeform = PhysicsToDeform;
		NewOp->BindingMethod = BindingMethod;

		NewOp->RChannelUsage = RChannelUsage;
		NewOp->GChannelUsage = GChannelUsage;
		NewOp->BChannelUsage = BChannelUsage;
		NewOp->AChannelUsage = AChannelUsage;

		return NewOp;
	}


	void ASTOpMeshBindShape::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Mesh);
		Func(Shape);
	}


	void ASTOpMeshBindShape::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshBindShapeArgs Args;
			FMemory::Memzero(Args);

			constexpr EMeshBindShapeFlags NoFlags = EMeshBindShapeFlags::None;
			EMeshBindShapeFlags BindFlags = NoFlags;
			EnumAddFlags(BindFlags, bRecomputeNormals ? EMeshBindShapeFlags::RecomputeNormals : NoFlags);
			EnumAddFlags(BindFlags, bReshapeSkeleton ? EMeshBindShapeFlags::ReshapeSkeleton : NoFlags);
			EnumAddFlags(BindFlags, bReshapePhysicsVolumes ? EMeshBindShapeFlags::ReshapePhysicsVolumes : NoFlags);
			EnumAddFlags(BindFlags, bReshapeVertices ? EMeshBindShapeFlags::ReshapeVertices : NoFlags);
			EnumAddFlags(BindFlags, bApplyLaplacian ? EMeshBindShapeFlags::ApplyLaplacian : NoFlags);
			EnumAddFlags(BindFlags, bReshapeSkeletonInvertSelection ? EMeshBindShapeFlags::ReshapeSkeletonInvertSelection : NoFlags);
			EnumAddFlags(BindFlags, bReshapePhysicsVolumesInvertSelection ? EMeshBindShapeFlags::ReshapePhysicsVolumesInvertSelection : NoFlags);

			{
				auto ConvertColorUsage = [](EVertexColorUsage Usage)
				{
					switch (Usage)
					{
					case EVertexColorUsage::None:			   return EMeshBindColorChannelUsage::None;
					case EVertexColorUsage::ReshapeClusterId:  return EMeshBindColorChannelUsage::ClusterId;
					case EVertexColorUsage::ReshapeMaskWeight: return EMeshBindColorChannelUsage::MaskWeight;
					default: check(false); return EMeshBindColorChannelUsage::None;
					};
				};
	
				const FMeshBindColorChannelUsages ColorUsages = {
					ConvertColorUsage(RChannelUsage),
					ConvertColorUsage(GChannelUsage),
					ConvertColorUsage(BChannelUsage),
					ConvertColorUsage(AChannelUsage) };

				FMemory::Memcpy(&Args.ColorUsage, &ColorUsages, sizeof(Args.ColorUsage));
				static_assert(sizeof(Args.ColorUsage) == sizeof(ColorUsages));
			}

			Args.flags = static_cast<uint32>(BindFlags);

			Args.bindingMethod = BindingMethod;

			if (Mesh)
			{
				Args.mesh = Mesh->LinkedAddress;
			}

			if (Shape)
			{
				Args.shape = Shape->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);

			AppendCode(Program.ByteCode, (int32)BonesToDeform.Num());
			for (FName Bone : BonesToDeform)
			{
				AppendCode(Program.ByteCode, Program.AddConstant(Bone));
			}

			AppendCode(Program.ByteCode, (int32)PhysicsToDeform.Num());
			for (FName PhysicsBone : PhysicsToDeform)
			{
				AppendCode(Program.ByteCode, Program.AddConstant(PhysicsBone));
			}
		}
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshBindShape::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		Ptr<ASTOp> MeshAt = Mesh.child();
		if (!MeshAt)
		{
			return nullptr;
		}

		Ptr<ASTOp> ShapeAt = Shape.child();
		if (!ShapeAt)
		{
			return nullptr;
		}

		EOpType MeshType = MeshAt->GetOpType();
		EOpType ShapeType = ShapeAt->GetOpType();

		// See if both mesh and shape have an operation that can be optimized in a combined way
		if (MeshType == ShapeType)
		{
			switch (MeshType)
			{

			case EOpType::ME_SWITCH:
			{
				// If the switch variable and structure is the same
				const ASTOpSwitch* MeshSwitch = static_cast<const ASTOpSwitch*>(MeshAt.get());
				const ASTOpSwitch* ShapeSwitch = static_cast<const ASTOpSwitch*>(ShapeAt.get());
				bool bIsSimilarSwitch = MeshSwitch->IsCompatibleWith(ShapeSwitch);
				if (!bIsSimilarSwitch)
				{
					break;
				}

				// Move the operation down all the paths
				Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(MeshAt);

				if (NewSwitch->Default)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = MeshSwitch->Default.child();
					NewBind->Shape = ShapeSwitch->Default.child();
					NewSwitch->Default = NewBind;
				}

				for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
				{
					if (NewSwitch->Cases[v].Branch)
					{
						Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
						NewBind->Mesh = MeshSwitch->Cases[v].Branch.child();
						NewBind->Shape = ShapeSwitch->FindBranch(MeshSwitch->Cases[v].Condition);
						NewSwitch->Cases[v].Branch = NewBind;
					}
				}

				NewOp = NewSwitch;
				break;
			}


			case EOpType::ME_CONDITIONAL:
			{
				const ASTOpConditional* MeshConditional = static_cast<const ASTOpConditional*>(MeshAt.get());
				const ASTOpConditional* ShapeConditional = static_cast<const ASTOpConditional*>(ShapeAt.get());
				bool bIsSimilar = MeshConditional->condition == ShapeConditional->condition;
				if (!bIsSimilar)
				{
					break;
				}

				Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(MeshAt);

				if (NewConditional->yes)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = MeshConditional->yes.child();
					NewBind->Shape = ShapeConditional->yes.child();
					NewConditional->yes = NewBind;
				}

				if (NewConditional->no)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = MeshConditional->no.child();
					NewBind->Shape = ShapeConditional->no.child();
					NewConditional->no = NewBind;
				}

				NewOp = NewConditional;
				break;
			}


			default:
				break;

			}
		}


		// If not already optimized
		if (!NewOp)
		{
			// Optimize only the mesh parameter
			switch (MeshType)
			{

			case EOpType::ME_SWITCH:
			{
				// Move the operation down all the paths
				Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(MeshAt);

				if (NewSwitch->Default)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = NewSwitch->Default.child();
					NewSwitch->Default = NewBind;
				}

				for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
				{
					if (NewSwitch->Cases[v].Branch)
					{
						Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
						NewBind->Mesh = NewSwitch->Cases[v].Branch.child();
						NewSwitch->Cases[v].Branch = NewBind;
					}
				}

				NewOp = NewSwitch;
				break;
			}

			case EOpType::ME_CONDITIONAL:
			{
				// Move the operation down all the paths
				Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(MeshAt);

				if (NewConditional->yes)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = NewConditional->yes.child();
					NewConditional->yes = NewBind;
				}

				if (NewConditional->no)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = NewConditional->no.child();
					NewConditional->no = NewBind;
				}

				NewOp = NewConditional;
				break;
			}

			case EOpType::ME_REMOVEMASK:
			{
				// We bind something that could have a part removed: we can reorder to bind the entire mesh
				// and apply remove later at runtime.

				Ptr<ASTOpMeshRemoveMask> NewRemove = UE::Mutable::Private::Clone<ASTOpMeshRemoveMask>(MeshAt);
				if (NewRemove->source)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = NewRemove->source.child();
					NewRemove->source = NewBind;
				}

				NewOp = NewRemove;
				break;
			}

			case EOpType::ME_ADDMETADATA:
			{
				Ptr<ASTOpMeshAddMetadata> New = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(MeshAt);
				if (New->Source)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = New->Source.child();
					New->Source = NewBind;
				}

				NewOp = New;
				break;
			}

			case EOpType::ME_PREPARELAYOUT:
			{
				Ptr<ASTOpMeshPrepareLayout> New = UE::Mutable::Private::Clone<ASTOpMeshPrepareLayout>(MeshAt);
				if (New->Mesh)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Mesh = New->Mesh.child();
					New->Mesh = NewBind;
				}

				NewOp = New;
				break;
			}

			default:
				break;

			}
		}

		// If not already optimized
		if (!NewOp)
		{
			// Optimize only the shape parameter
			switch (ShapeType)
			{

			case EOpType::ME_SWITCH:
			{
				// Move the operation down all the paths
				Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(ShapeAt);

				if (NewSwitch->Default)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Shape = NewSwitch->Default.child();
					NewSwitch->Default = NewBind;
				}

				for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
				{
					if (NewSwitch->Cases[v].Branch)
					{
						Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
						NewBind->Shape = NewSwitch->Cases[v].Branch.child();
						NewSwitch->Cases[v].Branch = NewBind;
					}
				}

				NewOp = NewSwitch;
				break;
			}

			case EOpType::ME_CONDITIONAL:
			{
				// Move the operation down all the paths
				Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(ShapeAt);

				if (NewConditional->yes)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Shape = NewConditional->yes.child();
					NewConditional->yes = NewBind;
				}

				if (NewConditional->no)
				{
					Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
					NewBind->Shape = NewConditional->no.child();
					NewConditional->no = NewBind;
				}

				NewOp = NewConditional;
				break;
			}

			case EOpType::ME_ADDMETADATA:
			{
				// Ignore the tags in the shape
				Ptr<ASTOpMeshBindShape> NewBind = UE::Mutable::Private::Clone<ASTOpMeshBindShape>(this);
				const ASTOpMeshAddMetadata* New = static_cast<const ASTOpMeshAddMetadata*>(ShapeAt.get());
				NewBind->Shape = New->Source.child();
				NewOp = NewBind;
				break;
			}

			default:
				break;

			}
		}


		return NewOp;
	}


	FSourceDataDescriptor ASTOpMeshBindShape::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
