// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/Skeleton.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpMeshBindShape final : public ASTOp
	{
	public:

		ASTChild Mesh;
		ASTChild Shape;

		TArray<FName> BonesToDeform;
		TArray<FName> PhysicsToDeform;

		uint32 BindingMethod = 0;
		
		uint32 bRecomputeNormals	  					: 1;
		uint32 bReshapeSkeleton	      					: 1;
		uint32 bReshapePhysicsVolumes 					: 1;
		uint32 bReshapeVertices       					: 1;
		uint32 bApplyLaplacian        					: 1;
		uint32 bReshapeSkeletonInvertSelection			: 1;
		uint32 bReshapePhysicsVolumesInvertSelection	: 1;

		EVertexColorUsage RChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage GChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage BChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage AChannelUsage = EVertexColorUsage::None;

		ASTOpMeshBindShape();
		ASTOpMeshBindShape(const ASTOpMeshBindShape&) = delete;
		virtual ~ASTOpMeshBindShape() override;

		virtual EOpType GetOpType() const override { return EOpType::ME_BINDSHAPE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};
}

