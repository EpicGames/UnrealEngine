// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshClipDeform.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	ASTOpMeshClipDeform::ASTOpMeshClipDeform()
		: Mesh(this)
		, ClipShape(this)
	{
	}


	ASTOpMeshClipDeform::~ASTOpMeshClipDeform()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshClipDeform::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshClipDeform* Other = static_cast<const ASTOpMeshClipDeform*>(&OtherUntyped);
			return Mesh == Other->Mesh && ClipShape == Other->ClipShape && FaceCullStrategy == Other->FaceCullStrategy;
		}

		return false;
	}


	uint32 ASTOpMeshClipDeform::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(ClipShape));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshClipDeform::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshClipDeform> n = new ASTOpMeshClipDeform();
		n->Mesh = mapChild(Mesh.child());
		n->ClipShape = mapChild(ClipShape.child());
		n->FaceCullStrategy = FaceCullStrategy;
		return n;
	}


	void ASTOpMeshClipDeform::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Mesh);
		f(ClipShape);
	}


	void ASTOpMeshClipDeform::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshClipDeformArgs Args;
			FMemory::Memzero(Args);

			Args.FaceCullStrategy = FaceCullStrategy;

			if (Mesh)
			{
				Args.mesh = Mesh->LinkedAddress;
			}

			if (ClipShape)
			{
				Args.clipShape = ClipShape->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FSourceDataDescriptor ASTOpMeshClipDeform::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
