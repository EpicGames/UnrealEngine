// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshClipMorphPlane.h"

#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "HAL/PlatformMath.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	ASTOpMeshClipMorphPlane::ASTOpMeshClipMorphPlane()
		: Source(this)
	{
	}


	ASTOpMeshClipMorphPlane::~ASTOpMeshClipMorphPlane()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshClipMorphPlane::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshClipMorphPlane* other = static_cast<const ASTOpMeshClipMorphPlane*>(&otherUntyped);

			return Source == other->Source &&
				MorphShape == other->MorphShape &&
				SelectionShape == other->SelectionShape &&
				VertexSelectionBone == other->VertexSelectionBone &&
				VertexSelectionType == other->VertexSelectionType &&
				FaceCullStrategy == other->FaceCullStrategy &&
				Dist == other->Dist &&
				Factor == other->Factor &&
				VertexSelectionBoneMaxRadius == other->VertexSelectionBoneMaxRadius;
		}
		return false;
	}


	uint32 ASTOpMeshClipMorphPlane::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Factor));
		Result = HashCombineFast(Result, GetTypeHash(Dist));
		
		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshClipMorphPlane::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshClipMorphPlane> n = new ASTOpMeshClipMorphPlane();
		n->Source = mapChild(Source.child());
		n->MorphShape = MorphShape;
		n->SelectionShape = SelectionShape;
		n->VertexSelectionBone = VertexSelectionBone;
		n->VertexSelectionType = VertexSelectionType;
		n->FaceCullStrategy = FaceCullStrategy;
		n->Dist = Dist;
		n->Factor = Factor;
		n->VertexSelectionBoneMaxRadius = VertexSelectionBoneMaxRadius;
		return n;
	}


	void ASTOpMeshClipMorphPlane::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
	}


	void ASTOpMeshClipMorphPlane::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshClipMorphPlaneArgs Args;
			FMemory::Memzero(Args);

			if (Source) 
			{
				Args.Source = Source->LinkedAddress;
			}

			Args.MorphShape = Program.AddConstant(MorphShape);

			Args.FaceCullStrategy = FaceCullStrategy;

			Args.VertexSelectionType = VertexSelectionType;
			if (VertexSelectionType == EClipVertexSelectionType::BoneHierarchy)
			{
				Args.VertexSelectionShapeOrBone = Program.AddConstant(VertexSelectionBone);
			}
			else if (VertexSelectionType == EClipVertexSelectionType::Shape)
			{
				Args.VertexSelectionShapeOrBone = Program.AddConstant(SelectionShape);
			}

			Args.Dist = Dist;
			Args.Factor = Factor;
			Args.MaxBoneRadius = VertexSelectionBoneMaxRadius;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshClipMorphPlane::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		if (!Source.child())
		{
			return nullptr;
		}

		EOpType SourceType = Source.child()->GetOpType();

		// Optimize only the mesh parameter
		switch (SourceType)
		{

		case EOpType::ME_ADDMETADATA:
		{
			Ptr<ASTOpMeshAddMetadata> NewAddMetadata = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(Source.child());

			if (NewAddMetadata->Source)
			{
				Ptr<ASTOpMeshClipMorphPlane> New = UE::Mutable::Private::Clone<ASTOpMeshClipMorphPlane>(this);
				New->Source = NewAddMetadata->Source.child();
				NewAddMetadata->Source = New;
			}

			NewOp = NewAddMetadata;
			break;
		}

		default:
			break;

		}

		return NewOp;
	}


	FSourceDataDescriptor ASTOpMeshClipMorphPlane::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
