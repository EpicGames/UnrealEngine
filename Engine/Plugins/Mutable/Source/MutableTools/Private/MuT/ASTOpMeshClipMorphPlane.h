// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"

namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpMeshClipMorphPlane final : public ASTOp
	{
	public:

		ASTChild Source;

		FShape MorphShape;
		FShape SelectionShape;
		FName VertexSelectionBone = NAME_None;

		EClipVertexSelectionType VertexSelectionType = EClipVertexSelectionType::None;
		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

		float Dist = 0.f, Factor = 0.f, VertexSelectionBoneMaxRadius = -1.f;

	public:

		ASTOpMeshClipMorphPlane();
		ASTOpMeshClipMorphPlane(const ASTOpMeshClipMorphPlane&) = delete;
		virtual ~ASTOpMeshClipMorphPlane();

		virtual EOpType GetOpType() const override { return EOpType::ME_CLIPMORPHPLANE; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual UE::Mutable::Private::Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};


}

