// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshClipWithMesh.h"
#include "MuT/ASTOpMeshAddMetadata.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpMeshClipWithMesh::ASTOpMeshClipWithMesh()
		: Source(this)
		, ClipMesh(this)
	{
	}


	ASTOpMeshClipWithMesh::~ASTOpMeshClipWithMesh()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshClipWithMesh::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshClipWithMesh* Other = static_cast<const ASTOpMeshClipWithMesh*>(&OtherUntyped);
			return Source == Other->Source &&
				ClipMesh == Other->ClipMesh &&
				FaceCullStrategy == Other->FaceCullStrategy;
		}
		return false;
	}


	uint32 ASTOpMeshClipWithMesh::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(ClipMesh));
		Result = HashCombineFast( Result, GetTypeHash(FaceCullStrategy));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpMeshClipWithMesh::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshClipWithMesh> New = new ASTOpMeshClipWithMesh();
		New->Source = MapChild(Source.child());
		New->ClipMesh = MapChild(ClipMesh.child());
		New->FaceCullStrategy = FaceCullStrategy;
		return New;
	}


	void ASTOpMeshClipWithMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
		Func(ClipMesh);
	}


	void ASTOpMeshClipWithMesh::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshClipWithMeshArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}

			if (ClipMesh)
			{
				Args.ClipMesh = ClipMesh->LinkedAddress;
			}

			Args.FaceCullStrategy = FaceCullStrategy;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	Ptr<ASTOp> ASTOpMeshClipWithMesh::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		Ptr<ASTOp> MeshAt = Source.child();

		if (!MeshAt)
		{
			return nullptr;
		}

		EOpType MeshType = MeshAt->GetOpType();

		switch (MeshType)
		{
			case EOpType::ME_ADDMETADATA:
			{
				Ptr<ASTOpMeshAddMetadata> New = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(MeshAt);
				if (New->Source)
				{
					Ptr<ASTOpMeshClipWithMesh> NewApplyPose = UE::Mutable::Private::Clone<ASTOpMeshClipWithMesh>(this);
					NewApplyPose->Source = New->Source.child();
					New->Source = NewApplyPose;
				}

				NewOp = New;
				break;
			}

			default:
				break;
		}

		return NewOp;
	}


	FSourceDataDescriptor ASTOpMeshClipWithMesh::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
