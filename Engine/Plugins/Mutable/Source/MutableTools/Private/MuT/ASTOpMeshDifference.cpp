// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshDifference.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshAddMetadata.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	ASTOpMeshDifference::ASTOpMeshDifference()
		: Base(this), Target(this)
	{
	}


	//---------------------------------------------------------------------------------------------
	ASTOpMeshDifference::~ASTOpMeshDifference()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//---------------------------------------------------------------------------------------------
	bool ASTOpMeshDifference::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshDifference* Other = static_cast<const ASTOpMeshDifference*>(&otherUntyped);

			return Base == Other->Base && Target == Other->Target
				&& bIgnoreTextureCoords == Other->bIgnoreTextureCoords
				&& Channels == Other->Channels;
		}
		return false;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshDifference::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshDifference> n = new ASTOpMeshDifference();
		n->Base = mapChild(Base.child());
		n->Target = mapChild(Target.child());
		n->bIgnoreTextureCoords = bIgnoreTextureCoords;
		n->Channels = Channels;		
		return n;
	}


	//---------------------------------------------------------------------------------------------
	void ASTOpMeshDifference::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Base);
		f(Target);
	}


	//---------------------------------------------------------------------------------------------
	uint32 ASTOpMeshDifference::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Target));
		
		return Result;
	}


	void ASTOpMeshDifference::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			AppendCode(Program.ByteCode, GetOpType());

			FOperation::ADDRESS BaseAt = Base ? Base->LinkedAddress : 0;
			AppendCode(Program.ByteCode, BaseAt);

			FOperation::ADDRESS TargetAt = Target ? Target->LinkedAddress : 0;
			AppendCode(Program.ByteCode, TargetAt);

			AppendCode(Program.ByteCode, (uint8)bIgnoreTextureCoords);

			AppendCode(Program.ByteCode, (uint8)Channels.Num());
			for (const FChannel& b : Channels)
			{
				AppendCode(Program.ByteCode, b.Semantic);
				AppendCode(Program.ByteCode, b.SemanticIndex);
			}
		}
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshDifference::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		Ptr<ASTOp> BaseAt = Base.child();
		if (!BaseAt)
		{
			return nullptr;
		}

		Ptr<ASTOp> TargetAt = Target.child();
		if (!TargetAt)
		{
			return nullptr;
		}

		EOpType BaseType = BaseAt->GetOpType();
		EOpType TargetType = TargetAt->GetOpType();

		// See if both base and target have an operation that can be optimized in a combined way
		if (BaseType == TargetType)
		{
			switch (BaseType)
			{

			case EOpType::ME_SWITCH:
			{
				// If the switch variable and structure is the same
				const ASTOpSwitch* BaseSwitch = static_cast<const ASTOpSwitch*>(BaseAt.get());
				const ASTOpSwitch* TargetSwitch = static_cast<const ASTOpSwitch*>(TargetAt.get());
				bool bIsSimilarSwitch = BaseSwitch->IsCompatibleWith(TargetSwitch);
				if (!bIsSimilarSwitch)
				{
					break;
				}

				// Move the operation down all the paths
				Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(BaseAt);

				if (NewSwitch->Default)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = BaseSwitch->Default.child();
					NewDiff->Target = TargetSwitch->Default.child();
					NewSwitch->Default = NewDiff;
				}

				for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
				{
					if (NewSwitch->Cases[v].Branch)
					{
						Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
						NewDiff->Base = BaseSwitch->Cases[v].Branch.child();
						NewDiff->Target = TargetSwitch->FindBranch(BaseSwitch->Cases[v].Condition);
						NewSwitch->Cases[v].Branch = NewDiff;
					}
				}

				NewOp = NewSwitch;
				break;
			}


			case EOpType::ME_CONDITIONAL:
			{
				const ASTOpConditional* BaseConditional = static_cast<const ASTOpConditional*>(BaseAt.get());
				const ASTOpConditional* TargetConditional = static_cast<const ASTOpConditional*>(TargetAt.get());
				bool bIsSimilar = BaseConditional->condition == TargetConditional->condition;
				if (!bIsSimilar)
				{
					break;
				}

				Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(BaseAt);

				if (NewConditional->yes)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = BaseConditional->yes.child();
					NewDiff->Target = TargetConditional->yes.child();
					NewConditional->yes = NewDiff;
				}

				if (NewConditional->no)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = BaseConditional->no.child();
					NewDiff->Target = TargetConditional->no.child();
					NewConditional->no = NewDiff;
				}

				NewOp = NewConditional;
				break;
			}


			default:
				break;

			}
		}


		// If not already optimized
		if (!NewOp)
		{
			// Optimize only the mesh parameter
			switch (BaseType)
			{

			case EOpType::ME_SWITCH:
			{
				// Move the operation down all the paths
				Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(BaseAt);

				if (NewSwitch->Default)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = NewSwitch->Default.child();
					NewSwitch->Default = NewDiff;
				}

				for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
				{
					if (NewSwitch->Cases[v].Branch)
					{
						Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
						NewDiff->Base = NewSwitch->Cases[v].Branch.child();
						NewSwitch->Cases[v].Branch = NewDiff;
					}
				}

				NewOp = NewSwitch;
				break;
			}

			case EOpType::ME_CONDITIONAL:
			{
				// Move the operation down all the paths
				Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(BaseAt);

				if (NewConditional->yes)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = NewConditional->yes.child();
					NewConditional->yes = NewDiff;
				}

				if (NewConditional->no)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = NewConditional->no.child();
					NewConditional->no = NewDiff;
				}

				NewOp = NewConditional;
				break;
			}

			case EOpType::ME_ADDMETADATA:
			{
				Ptr<ASTOpMeshAddMetadata> NewAdd = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(BaseAt);

				if (NewAdd->Source)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Base = NewAdd->Source.child();
					NewAdd->Source = NewDiff;
				}

				NewOp = NewAdd;
				break;
			}

			default:
				break;

			}
		}

		// If not already optimized
		if (!NewOp)
		{
			// Optimize only the shape parameter
			switch (TargetType)
			{

			case EOpType::ME_SWITCH:
			{
				// Move the operation down all the paths
				Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(TargetAt);

				if (NewSwitch->Default)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Target = NewSwitch->Default.child();
					NewSwitch->Default = NewDiff;
				}

				for (int32 v = 0; v < NewSwitch->Cases.Num(); ++v)
				{
					if (NewSwitch->Cases[v].Branch)
					{
						Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
						NewDiff->Target = NewSwitch->Cases[v].Branch.child();
						NewSwitch->Cases[v].Branch = NewDiff;
					}
				}

				NewOp = NewSwitch;
				break;
			}

			case EOpType::ME_CONDITIONAL:
			{
				// Move the operation down all the paths
				Ptr<ASTOpConditional> NewConditional = UE::Mutable::Private::Clone<ASTOpConditional>(TargetAt);

				if (NewConditional->yes)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Target = NewConditional->yes.child();
					NewConditional->yes = NewDiff;
				}

				if (NewConditional->no)
				{
					Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
					NewDiff->Target = NewConditional->no.child();
					NewConditional->no = NewDiff;
				}

				NewOp = NewConditional;
				break;
			}

			case EOpType::ME_ADDMETADATA:
			{
				// Ignore tags in this branch
				const ASTOpMeshAddMetadata* Add = static_cast<const ASTOpMeshAddMetadata*>(TargetAt.get());

				Ptr<ASTOpMeshDifference> NewDiff = UE::Mutable::Private::Clone<ASTOpMeshDifference>(this);
				NewDiff->Target = Add->Source.child();
				NewOp = NewDiff;
				break;
			}

			default:
				break;

			}
		}


		return NewOp;
	}


	//-------------------------------------------------------------------------------------------------
	FSourceDataDescriptor ASTOpMeshDifference::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
