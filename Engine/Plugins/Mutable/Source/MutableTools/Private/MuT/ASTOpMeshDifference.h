// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** Calculate the difference between two meshes (usually to calculate a morph delta). */
	class ASTOpMeshDifference final : public ASTOp
	{
	public:

		/** Base mesh of the morph. */
		ASTChild Base;

		/** Target mesh of the morph. */
		ASTChild Target;

		/** If true the texture coordinates will not be morphed.This is only relevant if the list in channelSemantic is empty. */
		bool bIgnoreTextureCoords;

		/** Channels to diff. */
		struct FChannel
		{
			uint8 Semantic=0;
			uint8 SemanticIndex=0;

			inline bool operator==( const FChannel& Other ) const
			{
				return Semantic == Other.Semantic && SemanticIndex == Other.SemanticIndex;
			}
		};
		TArray<FChannel> Channels;

	public:

		ASTOpMeshDifference();
		ASTOpMeshDifference(const ASTOpMeshDifference&) = delete;
		~ASTOpMeshDifference() override;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::ME_DIFFERENCE; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Link(FProgram& program, FLinkerOptions*) override;
		Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

