// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshExtractLayoutBlocks.h"

#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshApplyPose.h"
#include "MuT/ASTOpMeshApplyLayout.h"
#include "MuT/ASTOpMeshMerge.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "Misc/AssertionMacros.h"

namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	//---------------------------------------------------------------------------------------------
	ASTOpMeshExtractLayoutBlocks::ASTOpMeshExtractLayoutBlocks()
		: Source(this)
	{
	}


	ASTOpMeshExtractLayoutBlocks::~ASTOpMeshExtractLayoutBlocks()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshExtractLayoutBlocks::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshExtractLayoutBlocks* other = static_cast<const ASTOpMeshExtractLayoutBlocks*>(&otherUntyped);
			return Source == other->Source && LayoutIndex == other->LayoutIndex && Blocks == other->Blocks;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshExtractLayoutBlocks::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshExtractLayoutBlocks> n = new ASTOpMeshExtractLayoutBlocks();
		n->Source = mapChild(Source.child());
		n->LayoutIndex = LayoutIndex;
		n->Blocks = Blocks;
		return n;
	}


	void ASTOpMeshExtractLayoutBlocks::Assert()
	{
		check(Blocks.Num() < std::numeric_limits<uint16>::max());
		ASTOp::Assert();
	}


	void ASTOpMeshExtractLayoutBlocks::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
	}


	uint32 ASTOpMeshExtractLayoutBlocks::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));

		return Result;
	}


	void ASTOpMeshExtractLayoutBlocks::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());	
			
			AppendCode(Program.ByteCode, GetOpType());

			FOperation::ADDRESS sourceAt = Source ? Source->LinkedAddress : 0;
			AppendCode(Program.ByteCode, sourceAt);
			AppendCode(Program.ByteCode, (uint16)LayoutIndex);
			AppendCode(Program.ByteCode, (uint16)Blocks.Num());

			for (uint64 Id : Blocks)
			{
				AppendCode(Program.ByteCode, Id);
			}
		}
	}

	 
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshExtractLayoutBlocks::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext& Context) const
	{
		UE::Mutable::Private::Ptr<ASTOp> NewOp = Context.MeshExtractLayoutBlocksSinker.Apply(this);
		return NewOp;
	}


	FSourceDataDescriptor ASTOpMeshExtractLayoutBlocks::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> Sink_MeshExtractLayoutBlocksAST::Apply(const ASTOpMeshExtractLayoutBlocks* root)
	{
		Root = root;

		OldToNew.Reset();

		InitialSource = Root->Source.child();
		UE::Mutable::Private::Ptr<ASTOp> newSource = Visit(InitialSource, Root);

		// If there is any change, it is the new root.
		if (newSource != InitialSource)
		{
			return newSource;
		}

		return nullptr;
	}

	UE::Mutable::Private::Ptr<ASTOp> Sink_MeshExtractLayoutBlocksAST::Visit(const UE::Mutable::Private::Ptr<ASTOp>& at, const ASTOpMeshExtractLayoutBlocks* currentSinkOp)
	{
		if (!at) return nullptr;

		// Already visited?
		const Ptr<ASTOp>* Cached = OldToNew.Find({ at, currentSinkOp });
		if (Cached)
		{
			return *Cached;
		}

		UE::Mutable::Private::Ptr<ASTOp> newAt = at;
		switch (at->GetOpType())
		{

		case EOpType::ME_APPLYLAYOUT:
		{
			Ptr<ASTOpMeshApplyLayout> NewOp = UE::Mutable::Private::Clone<ASTOpMeshApplyLayout>(at);
			NewOp->Mesh = Visit(NewOp->Mesh.child(), currentSinkOp);
			newAt = NewOp;
			break;
		}

		case EOpType::ME_PREPARELAYOUT:
		{
			ASTOpMeshPrepareLayout* SourcePrepareOp = static_cast<ASTOpMeshPrepareLayout*>(at.get());

			bool bHandled = false;

			// If the "prepare" operation is for an unrelated layout, sink the "extract" down the "prepare" mesh.
			if (SourcePrepareOp->LayoutChannel!= currentSinkOp->LayoutIndex)
			{
				newAt = Visit(SourcePrepareOp->Mesh.child(), currentSinkOp);
				bHandled = true;
			}
			// If the "prepare" operation is for a this layout channel...
			else if (SourcePrepareOp->LayoutChannel == currentSinkOp->LayoutIndex)
			{
				// ...and the layout is constant...
				if (SourcePrepareOp->Layout && SourcePrepareOp->Layout->GetOpType() == EOpType::LA_CONSTANT)
				{
					ASTOpConstantResource* LayoutOp = static_cast<ASTOpConstantResource*>(SourcePrepareOp->Layout.child().get());
					const FLayout* Layout = static_cast<const FLayout*>(LayoutOp->GetValue().Get());

					// ...and it is a single block covering all the layout.
					if (Layout && Layout->IsSingleBlockAndFull())
					{
						// We can omit the "extract" operation because it will get the whole mesh anyway
						newAt = SourcePrepareOp;
						bHandled = true;
					}
				}
			}
			
			if (!bHandled)
			{
				if (at != InitialSource)
				{
					// We didn't optimize it, so create the end operation as in the switch "default" case.
					UE::Mutable::Private::Ptr<ASTOpMeshExtractLayoutBlocks> newOp = UE::Mutable::Private::Clone<ASTOpMeshExtractLayoutBlocks>(currentSinkOp);
					newOp->Source = at;
					newAt = newOp;
				}
			}
			break;
		}

		case EOpType::ME_ADDMETADATA:
		{
			Ptr<ASTOpMeshAddMetadata> NewOp = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(at);
			NewOp->Source = Visit(NewOp->Source.child(), currentSinkOp);
			newAt = NewOp;
			break;
		}

		case EOpType::ME_CLIPMORPHPLANE:
		{
			Ptr<ASTOpMeshClipMorphPlane> NewOp = UE::Mutable::Private::Clone<ASTOpMeshClipMorphPlane>(at);
			NewOp->Source = Visit(NewOp->Source.child(), currentSinkOp);
			newAt = NewOp;
			break;
		}

		case EOpType::ME_MORPH:
		{
			Ptr<ASTOpMeshMorph> NewOp = UE::Mutable::Private::Clone<ASTOpMeshMorph>(at);
			NewOp->Base = Visit(NewOp->Base.child(), currentSinkOp);
			newAt = NewOp;
			break;
		}

		case EOpType::ME_MERGE:
		{
			Ptr<ASTOpMeshMerge> NewOp = UE::Mutable::Private::Clone<ASTOpMeshMerge>(at);
			NewOp->Base = Visit(NewOp->Base.child(), currentSinkOp);
			NewOp->Added = Visit(NewOp->Added.child(), currentSinkOp);
			newAt = NewOp;
			break;
		}

		case EOpType::ME_APPLYPOSE:
		{
			Ptr<ASTOpMeshApplyPose> NewOp = UE::Mutable::Private::Clone<ASTOpMeshApplyPose>(at);
			NewOp->Base = Visit(NewOp->Base.child(), currentSinkOp);
			newAt = NewOp;
			break;
		}

		case EOpType::ME_REMOVEMASK:
		{
			// TODO: Make mask smaller?
			Ptr<ASTOpMeshRemoveMask> newOp = UE::Mutable::Private::Clone<ASTOpMeshRemoveMask>(at);
			newOp->source = Visit(newOp->source.child(), currentSinkOp);
			newAt = newOp;
			break;
		}

		case EOpType::ME_CONDITIONAL:
		{
			Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
			newOp->yes = Visit(newOp->yes.child(), currentSinkOp);
			newOp->no = Visit(newOp->no.child(), currentSinkOp);
			newAt = newOp;
			break;
		}

		case EOpType::ME_SWITCH:
		{
			Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
			newOp->Default = Visit(newOp->Default.child(), currentSinkOp);
			for (ASTOpSwitch::FCase& c : newOp->Cases)
			{
				c.Branch = Visit(c.Branch.child(), currentSinkOp);
			}
			newAt = newOp;
			break;
		}

		// If we reach here it means the operation type has not bee optimized.
		default:
			if (at != InitialSource)
			{
				UE::Mutable::Private::Ptr<ASTOpMeshExtractLayoutBlocks> newOp = UE::Mutable::Private::Clone<ASTOpMeshExtractLayoutBlocks>(currentSinkOp);
				newOp->Source = at;
				newAt = newOp;
			}
			break;

		}

		OldToNew.Add({ at, currentSinkOp }, newAt);

		return newAt;
	}


}
