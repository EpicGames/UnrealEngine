// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;


	//---------------------------------------------------------------------------------------------
	//! Variable sized mesh block extract operation.
	//---------------------------------------------------------------------------------------------
	class ASTOpMeshExtractLayoutBlocks final : public ASTOp
	{
	public:

		/** Source mesh to extract block from. */
		ASTChild Source;

		/** Layout to use to select the blocks. */
		uint16 LayoutIndex = 0;

		/** Block Ids to include in the resulting mesh. If this is empty all vertices with any valid block assigned will be included. */
		TArray<uint64> Blocks;

	public:

		ASTOpMeshExtractLayoutBlocks();
		ASTOpMeshExtractLayoutBlocks(const ASTOpMeshExtractLayoutBlocks&) = delete;
		~ASTOpMeshExtractLayoutBlocks() override;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::ME_EXTRACTLAYOUTBLOCK; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Assert() override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		UE::Mutable::Private::Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};


}

