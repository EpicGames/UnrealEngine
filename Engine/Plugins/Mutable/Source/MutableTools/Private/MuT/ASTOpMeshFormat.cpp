// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshFormat.h"

#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Mesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshTransformWithBoundingMesh.h"
#include "MuT/ASTOpMeshTransformWithBone.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpMeshApplyPose.h"
#include "MuT/ASTOpMeshApplyLayout.h"
#include "MuT/ASTOpMeshMerge.h"
#include "MuT/ASTOpSwitch.h"

#include "GPUSkinPublicDefs.h"

namespace UE::Mutable::Private
{


//-------------------------------------------------------------------------------------------------
ASTOpMeshFormat::ASTOpMeshFormat()
    : Source(this)
    , Format(this)
{
}


ASTOpMeshFormat::~ASTOpMeshFormat()
{
    // Explicit call needed to avoid recursive destruction
    ASTOp::RemoveChildren();
}


bool ASTOpMeshFormat::IsEqual(const ASTOp& otherUntyped) const
{
	if (otherUntyped.GetOpType() == GetOpType())
	{
		const ASTOpMeshFormat* other = static_cast<const ASTOpMeshFormat*>(&otherUntyped);
        return Source==other->Source && Format==other->Format && Flags ==other->Flags;
    }
    return false;
}


uint32 ASTOpMeshFormat::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());

	Result = HashCombineFast(Result, GetTypeHash(Source));
    Result = HashCombineFast(Result, GetTypeHash(Format));
    
	return Result;
}


UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshFormat::Clone(MapChildFuncRef mapChild) const
{
	UE::Mutable::Private::Ptr<ASTOpMeshFormat> n = new ASTOpMeshFormat();
    n->Source = mapChild(Source.child());
    n->Format = mapChild(Format.child());
	n->Flags = Flags;
	n->bOptimizeBuffers = bOptimizeBuffers;
    return n;
}


void ASTOpMeshFormat::ForEachChild(const TFunctionRef<void(ASTChild&)> f )
{
    f( Source );
    f( Format );
}


void ASTOpMeshFormat::Link( FProgram& Program, FLinkerOptions* )
{
    // Already linked?
    if (!LinkedAddress)
    {
        FOperation::MeshFormatArgs Args;
		FMemory::Memzero(Args);

		Args.Flags = Flags;
		if (bOptimizeBuffers)
		{
			Args.Flags = Args.Flags | FOperation::MeshFormatArgs::OptimizeBuffers;
		}

		if (Source) 
		{
			Args.source = Source->LinkedAddress;
		}

		if (Format) 
		{
			Args.format = Format->LinkedAddress;
		}

        ++Program.NumOps;
        LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
        
        AppendCode(Program.ByteCode, GetOpType());
        AppendCode(Program.ByteCode, Args);
    }

}


UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshFormat::OptimiseSink(const FModelOptimizationOptions& options, FOptimizeSinkContext& context) const
{
	UE::Mutable::Private::Ptr<ASTOp> at = context.MeshFormatSinker.Apply(this);
	return at;
}


FSourceDataDescriptor ASTOpMeshFormat::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
{
	if (Source)
	{
		return Source->GetSourceDataDescriptor(Context);
	}

	return {};
}


UE::Mutable::Private::Ptr<ASTOp> Sink_MeshFormatAST::Apply(const ASTOpMeshFormat* root)
{
	Root = root;

	OldToNew.Reset();

	InitialSource = Root->Source.child();
	UE::Mutable::Private::Ptr<ASTOp> newSource = Visit(InitialSource, Root);

	// If there is any change, it is the new root.
	if (newSource != InitialSource)
	{
		return newSource;
	}

	return nullptr;
}


namespace
{
	TManagedPtr<const FMesh> FindBaseMeshConstant(UE::Mutable::Private::Ptr<ASTOp> at)
	{
		TManagedPtr<const FMesh> Result;

		switch (at->GetOpType())
		{
		case EOpType::ME_CONSTANT:
		{
			const ASTOpConstantResource* typed = static_cast<const ASTOpConstantResource*>(at.get());
			Result = StaticCastManagedPtr<const FMesh>(typed->GetValue());
			break;
		}

		default:
			check(false);
		}

		check(Result);

		return Result;
	}

    TManagedPtr<const FMesh> EnsureFormatHasSkinningBuffers(TManagedPtr<const FMesh>& FormatMesh)
    {
        const FMeshBufferSet& FormatMeshVertexBuffers = FormatMesh->GetVertexBuffers();
    
        int32 SourceSkinningBufferIndex = -1;         
        int32 SourceSkinningChannelIndex = -1;

        // Assume bone indices implies it also has weights.
        FormatMeshVertexBuffers.FindChannel(EMeshBufferSemantic::BoneIndices, 0, &SourceSkinningBufferIndex, &SourceSkinningChannelIndex);

        bool bSourceHasSkinningData = SourceSkinningBufferIndex != -1;
			
        if (bSourceHasSkinningData)
        {
            return FormatMesh;
        }

		TManagedPtr<FMesh> NewMesh = FormatMesh->Clone();
        FMeshBufferSet& MeshBuffers = NewMesh->GetVertexBuffers();
        
        FMeshBuffer& Buffer = MeshBuffers.Buffers.AddDefaulted_GetRef();

        FMeshBufferChannel BoneIndices;
        BoneIndices.Semantic = EMeshBufferSemantic::BoneIndices;
        BoneIndices.Format = EMeshBufferFormat::UInt16;
        BoneIndices.SemanticIndex = 0;
        BoneIndices.Offset = 0;
        BoneIndices.ComponentCount = MAX_TOTAL_INFLUENCES;

        FMeshBufferChannel BoneWeights;
        BoneWeights.Semantic = EMeshBufferSemantic::BoneWeights;
        BoneWeights.Format = EMeshBufferFormat::NUInt16;
        BoneWeights.SemanticIndex = 0;
        BoneWeights.Offset = MAX_TOTAL_INFLUENCES*2;
        BoneWeights.ComponentCount = MAX_TOTAL_INFLUENCES;


        Buffer.ElementSize = MAX_TOTAL_INFLUENCES*4;
        Buffer.Channels.Add(BoneIndices);
        Buffer.Channels.Add(BoneWeights);

        return NewMesh;
    }

}


UE::Mutable::Private::Ptr<ASTOp> Sink_MeshFormatAST::Visit(const UE::Mutable::Private::Ptr<ASTOp>& at, const ASTOpMeshFormat* currentFormatOp)
{
	if (!at)
	{
		return nullptr;
	}

	// Already visited?
	const Ptr<ASTOp>* Cached = OldToNew.Find({ at,currentFormatOp });
	if (Cached)
	{
		return *Cached;
	}

	UE::Mutable::Private::Ptr<ASTOp> newAt = at;
	switch (at->GetOpType())
	{

	case EOpType::ME_APPLYLAYOUT:
	{
		Ptr<ASTOpMeshApplyLayout> NewOp = UE::Mutable::Private::Clone<ASTOpMeshApplyLayout>(at);
		NewOp->Mesh = Visit(NewOp->Mesh.child(), currentFormatOp);
		newAt = NewOp;
		break;
	}

	case EOpType::ME_ADDMETADATA:
	{
		Ptr<ASTOpMeshAddMetadata> newOp = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(at);
		newOp->Source = Visit(newOp->Source.child(), currentFormatOp);
		newAt = newOp;
		break;
	}

	case EOpType::ME_CLIPMORPHPLANE:
	{
		Ptr<ASTOpMeshClipMorphPlane> newOp = UE::Mutable::Private::Clone<ASTOpMeshClipMorphPlane>(at);
		newOp->Source = Visit(newOp->Source.child(), currentFormatOp);
		newAt = newOp;
		break;
	}

	case EOpType::ME_TRANSFORMWITHMESH:
	{
		Ptr<ASTOpMeshTransformWithBoundingMesh> NewOp = UE::Mutable::Private::Clone<ASTOpMeshTransformWithBoundingMesh>(at);
		NewOp->source = Visit(NewOp->source.child(), currentFormatOp);

		// Don't transform the bounding mesh: it should be optimized with a different specific format elsewhere (TODO).
		// NewOp->boundingMesh = Visit(NewOp->boundingMesh.child(), currentFormatOp);

		newAt = NewOp;
		break;
	}

	case EOpType::ME_TRANSFORMWITHBONE:
	{
		Ptr<ASTOpMeshTransformWithBone> NewOp = UE::Mutable::Private::Clone<ASTOpMeshTransformWithBone>(at);
		NewOp->SourceMesh = Visit(NewOp->SourceMesh.child(), currentFormatOp);
		newAt = NewOp;
		break;
	}

	case EOpType::ME_MORPH:
	{
		// Move the format down the base of the morph
		Ptr<ASTOpMeshMorph> NewOp = UE::Mutable::Private::Clone<ASTOpMeshMorph>(at);
		NewOp->Base = Visit(NewOp->Base.child(), currentFormatOp);

		newAt = NewOp;
		break;
	}

	case EOpType::ME_MERGE:
	{
		Ptr<ASTOpMeshMerge> NewOp = UE::Mutable::Private::Clone<ASTOpMeshMerge>(at);
		NewOp->Base = Visit(NewOp->Base.child(), currentFormatOp);
		NewOp->Added = Visit(NewOp->Added.child(), currentFormatOp);
		newAt = NewOp;
		break;
	}

	case EOpType::ME_APPLYPOSE:
	{
		TManagedPtr<const FMesh> TargetFormat = FindBaseMeshConstant(currentFormatOp->Format.child());
        TargetFormat = EnsureFormatHasSkinningBuffers(TargetFormat);

		Ptr<ASTOpMeshApplyPose> NewOp = UE::Mutable::Private::Clone<ASTOpMeshApplyPose>(at);
        UE::Mutable::Private::Ptr<ASTOpMeshFormat> NewFormat = UE::Mutable::Private::Clone<ASTOpMeshFormat>(currentFormatOp);
        
		UE::Mutable::Private::Ptr<ASTOpConstantResource> NewFormatConstant = new ASTOpConstantResource();
		NewFormatConstant->Type = EOpType::ME_CONSTANT;
		NewFormatConstant->SetValue(TargetFormat);
		NewFormatConstant->SourceDataDescriptor = at->GetSourceDataDescriptor();
	
        NewFormat->Flags = NewFormat->Flags | FOperation::MeshFormatArgs::OptimizeBuffers;

        // TODO: Optimize, in case no skinning data is found in the format mesh a generic buffer that can represent
        // all possible skinning formats is added. This is not optimal, we may want to add a flag to the format op
        // to indicate it should copy the skinning from the base mesh.
        NewFormat->Format = NewFormatConstant;

        NewOp->Base = Visit(NewOp->Base.child(), NewFormat.get());
		
		newAt = NewOp;
		break;
	}

	case EOpType::ME_REMOVEMASK:
	{
		Ptr<ASTOpMeshRemoveMask> newOp = UE::Mutable::Private::Clone<ASTOpMeshRemoveMask>(at);
		newOp->source = Visit(newOp->source.child(), currentFormatOp);
		newAt = newOp;
		break;
	}

	case EOpType::ME_CONDITIONAL:
	{
		Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
		newOp->yes = Visit(newOp->yes.child(), currentFormatOp);
		newOp->no = Visit(newOp->no.child(), currentFormatOp);
		newAt = newOp;
		break;
	}

	case EOpType::ME_SWITCH:
	{
		Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
		newOp->Default = Visit(newOp->Default.child(), currentFormatOp);
		for (ASTOpSwitch::FCase& c : newOp->Cases)
		{
			c.Branch = Visit(c.Branch.child(), currentFormatOp);
		}
		newAt = newOp;
		break;
	}

	case EOpType::ME_FORMAT:
		// TODO: The child format can be removed. 
		// Unless channels are removed and re-added, which would change their content?
		break;


	// This operation should not be optimized.
	case EOpType::ME_DIFFERENCE:

	// If we reach here it means the operation type has not bee optimized.
	default:
		if (at != InitialSource)
		{
			UE::Mutable::Private::Ptr<ASTOpMeshFormat> newOp = UE::Mutable::Private::Clone<ASTOpMeshFormat>(currentFormatOp);
			newOp->Source = at;
			newAt = newOp;
		}
		break;

	}

	OldToNew.Add({ at,currentFormatOp }, newAt);

	return newAt;
}

}
