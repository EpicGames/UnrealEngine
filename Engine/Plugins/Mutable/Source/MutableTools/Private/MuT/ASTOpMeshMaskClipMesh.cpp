// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshMaskClipMesh.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	ASTOpMeshMaskClipMesh::ASTOpMeshMaskClipMesh()
		: source(this)
		, clip(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpMeshMaskClipMesh::~ASTOpMeshMaskClipMesh()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpMeshMaskClipMesh::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshMaskClipMesh* other = static_cast<const ASTOpMeshMaskClipMesh*>(&otherUntyped);
			return source == other->source && clip == other->clip;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpMeshMaskClipMesh::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(source));
		Result = HashCombineFast(Result, GetTypeHash(clip));
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMaskClipMesh::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshMaskClipMesh> n = new ASTOpMeshMaskClipMesh();
		n->source = mapChild(source.child());
		n->clip = mapChild(clip.child());
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpMeshMaskClipMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(source);
		f(clip);
	}


	void ASTOpMeshMaskClipMesh::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshMaskClipMeshArgs Args;
			FMemory::Memzero(Args);

			if (source) 
			{
				Args.source = source->LinkedAddress;
			}

			if (clip) 
			{
				Args.clip = clip->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	namespace
	{
		class Sink_MeshMaskClipMeshSource
		{
		public:

			// \TODO This is recursive and may cause stack overflows in big models.

			UE::Mutable::Private::Ptr<ASTOp> Apply(const ASTOpMeshMaskClipMesh* root)
			{
				Root = root;
				m_oldToNew.Empty();

				InitialSource = root->source.child();
				UE::Mutable::Private::Ptr<ASTOp> NewSource = Visit(InitialSource);

				// If there is any change, it is the new root.
				if (NewSource != InitialSource)
				{
					return NewSource;
				}

				return nullptr;
			}

		protected:

			const ASTOpMeshMaskClipMesh* Root;
			UE::Mutable::Private::Ptr<ASTOp> InitialSource;
			TMap<UE::Mutable::Private::Ptr<ASTOp>, UE::Mutable::Private::Ptr<ASTOp>> m_oldToNew;
			TArray<UE::Mutable::Private::Ptr<ASTOp>> m_newOps;

			UE::Mutable::Private::Ptr<ASTOp> Visit(const UE::Mutable::Private::Ptr<ASTOp>& at)
			{
				if (!at) return nullptr;

				// Newly created?
				if (m_newOps.Find(at) != INDEX_NONE)
				{
					return at;
				}

				// Already visited?
				UE::Mutable::Private::Ptr<ASTOp>* cacheIt = m_oldToNew.Find(at);
				if (cacheIt)
				{
					return *cacheIt;
				}

				UE::Mutable::Private::Ptr<ASTOp> newAt = at;
				switch (at->GetOpType())
				{

				// This cannot be sunk since the result is different. Since the clipping is now correctly
				// generated at the end of the chain when really necessary, this wrong optimisation is no 
				// longer needed.
				//case EOpType::ME_MORPH:
		        //{
		        //    break;
		        //}

				case EOpType::ME_REMOVEMASK:
				{
					// Remove this op:
					// This may lead to the mask being bigger than needed since it will include
					// faces removed by the ignored removemask, but it is ok

					// TODO: Swap instead of ignore, and implement removemask on a mask?
					const ASTOpMeshRemoveMask* typedAt = static_cast<const ASTOpMeshRemoveMask*>(at.get());
					newAt = Visit(typedAt->source.child());
					break;
				}

				case EOpType::ME_PREPARELAYOUT:
				{
					// Ignore the prepare op: it doesn't contribute to the mask generation.
					const ASTOpMeshPrepareLayout* typedAt = static_cast<const ASTOpMeshPrepareLayout*>(at.get());
					newAt = Visit(typedAt->Mesh.child());
					break;
				}

				case EOpType::ME_ADDMETADATA:
				{
					Ptr<ASTOpMeshAddMetadata> newOp = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(at);
					newOp->Source = Visit(newOp->Source.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_CONDITIONAL:
				{
					// We move the mask creation down the two paths
					// It always needs to be a clone because otherwise we could be modifying an
					// instruction that shouldn't if the parent was a ME_REMOVEMASK above and we
					// skipped the cloning for the parent.
					Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
					newOp->yes = Visit(newOp->yes.child());
					newOp->no = Visit(newOp->no.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_SWITCH:
				{
					// We move the mask creation down all the paths
					Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
					newOp->Default = Visit(newOp->Default.child());
					for (ASTOpSwitch::FCase& c : newOp->Cases)
					{
						c.Branch = Visit(c.Branch.child());
					}
					newAt = newOp;
					break;
				}

				// This cannot be sunk since the result is different. Since the clipping is now correctly
				// generated at the end of the chain when really necessary, this wrong optimisation is no 
				// longer needed.
				//case EOpType::ME_CLIPMORPHPLANE:
	   //         {
	   //             // We move the mask creation down the source
	   //             auto typedAt = dynamic_cast<const ASTOpMeshClipMorphPlane*>(at.get());
	   //             newAt = Visit(typedAt->source.child());
	   //             break;
	   //         }

				default:
				{
					//
					if (at != InitialSource)
					{
						Ptr<ASTOpMeshMaskClipMesh> newOp = UE::Mutable::Private::Clone<ASTOpMeshMaskClipMesh>(Root);
						newOp->source = at;
						newAt = newOp;
					}
					break;
				}

				}

				m_oldToNew.Add(at, newAt);

				return newAt;
			}
		};


		class Sink_MeshMaskClipMeshClip
		{
		public:

			// \TODO This is recursive and may cause stack overflows in big models.

			UE::Mutable::Private::Ptr<ASTOp> Apply(const ASTOpMeshMaskClipMesh* root)
			{
				Root = root;
				m_oldToNew.Empty();

				m_initialClip = root->clip.child();
				UE::Mutable::Private::Ptr<ASTOp> NewClip = Visit(m_initialClip);

				// If there is any change, it is the new root.
				if (NewClip != m_initialClip)
				{
					return NewClip;
				}

				return nullptr;
			}

		protected:

			const ASTOpMeshMaskClipMesh* Root;
			UE::Mutable::Private::Ptr<ASTOp> m_initialClip;
			TMap<UE::Mutable::Private::Ptr<ASTOp>, UE::Mutable::Private::Ptr<ASTOp>> m_oldToNew;
			TArray<UE::Mutable::Private::Ptr<ASTOp>> m_newOps;

			UE::Mutable::Private::Ptr<ASTOp> Visit(const UE::Mutable::Private::Ptr<ASTOp>& at)
			{
				if (!at) return nullptr;

				// Newly created?
				if (m_newOps.Contains(at))
				{
					return at;
				}

				// Already visited?
				UE::Mutable::Private::Ptr<ASTOp>* cacheIt = m_oldToNew.Find(at);
				if (cacheIt)
				{
					return *cacheIt;
				}

				UE::Mutable::Private::Ptr<ASTOp> newAt = at;
				switch (at->GetOpType())
				{

				case EOpType::ME_CONDITIONAL:
				{
					// We move the mask creation down the two paths
					// It always needs to be a clone because otherwise we could be modifying an
					// instruction that shouldn't if the parent was a ME_REMOVEMASK above and we
					// skipped the cloning for the parent.
					Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
					newOp->yes = Visit(newOp->yes.child());
					newOp->no = Visit(newOp->no.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_SWITCH:
				{
					// We move the mask creation down all the paths
					Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
					newOp->Default = Visit(newOp->Default.child());
					for (ASTOpSwitch::FCase& c : newOp->Cases)
					{
						c.Branch = Visit(c.Branch.child());
					}
					newAt = newOp;
					break;
				}

				case EOpType::ME_ADDMETADATA:
				{
					// Ignore tags in the clip mesh branch
					const ASTOpMeshAddMetadata* AddOp = static_cast<const ASTOpMeshAddMetadata*>(at.get());
					newAt = Visit(AddOp->Source.child());
					break;
				}

				default:
				{
					//
					if (at != m_initialClip)
					{
						Ptr<ASTOpMeshMaskClipMesh> newOp = UE::Mutable::Private::Clone<ASTOpMeshMaskClipMesh>(Root);
						newOp->clip = at;
						newAt = newOp;
					}
					break;
				}

				}

				m_oldToNew.Add(at, newAt);

				return newAt;
			}
		};

	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMaskClipMesh::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		// \TODO: Add logic state to the sinkers to avoid explosion with switches in both branches and similar cases.

		// This will sink the operation down the source
		Sink_MeshMaskClipMeshSource SinkerSource;
		UE::Mutable::Private::Ptr<ASTOp> at = SinkerSource.Apply(this);

		// If we didn't sink it.
		if (!at || at == this)
		{
			// This will sink the operation down the clip child
			Sink_MeshMaskClipMeshClip SinkerClip;
			at = SinkerClip.Apply(this);
		}

		return at;
	}


	//-------------------------------------------------------------------------------------------------
	FSourceDataDescriptor ASTOpMeshMaskClipMesh::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (source)
		{
			return source->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
 
