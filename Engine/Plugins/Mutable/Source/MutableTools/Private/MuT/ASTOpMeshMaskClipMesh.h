// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpMeshMaskClipMesh final : public ASTOp
	{
	public:

		ASTChild source;
		ASTChild clip;

	public:

		ASTOpMeshMaskClipMesh();
		ASTOpMeshMaskClipMesh(const ASTOpMeshMaskClipMesh&) = delete;
		~ASTOpMeshMaskClipMesh();

		EOpType GetOpType() const override { return EOpType::ME_MASKCLIPMESH; }
		uint32 Hash() const override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

