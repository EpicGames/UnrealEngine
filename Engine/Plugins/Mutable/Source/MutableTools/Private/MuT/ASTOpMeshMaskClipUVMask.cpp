// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshMaskClipUVMask.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpSwitch.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	ASTOpMeshMaskClipUVMask::ASTOpMeshMaskClipUVMask()
		: Source(this)
		, UVSource(this)
		, MaskImage(this)
		, MaskLayout(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpMeshMaskClipUVMask::~ASTOpMeshMaskClipUVMask()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpMeshMaskClipUVMask::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshMaskClipUVMask* other = static_cast<const ASTOpMeshMaskClipUVMask*>(&otherUntyped);
			return Source == other->Source
				&& UVSource == other->UVSource
				&& MaskImage == other->MaskImage
				&& MaskLayout == other->MaskLayout 
				&& LayoutIndex==other->LayoutIndex;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpMeshMaskClipUVMask::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(UVSource));
		Result = HashCombineFast(Result, GetTypeHash(MaskImage));
		Result = HashCombineFast(Result, GetTypeHash(MaskLayout));

		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMaskClipUVMask::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshMaskClipUVMask> n = new ASTOpMeshMaskClipUVMask();
		n->Source = mapChild(Source.child());
		n->UVSource = mapChild(UVSource.child());
		n->MaskImage = mapChild(MaskImage.child());
		n->MaskLayout = mapChild(MaskLayout.child());
		n->LayoutIndex = LayoutIndex;
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpMeshMaskClipUVMask::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Source);
		f(UVSource);
		f(MaskImage);
		f(MaskLayout);
	}


	void ASTOpMeshMaskClipUVMask::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshMaskClipUVMaskArgs Args;
			FMemory::Memzero(Args);

			if (Source) 
			{
				Args.Source = Source->LinkedAddress;
			}

			if (UVSource) 
			{
				Args.UVSource = UVSource->LinkedAddress;
			}

			if (MaskImage) 
			{
				Args.MaskImage = MaskImage->LinkedAddress;
			}

			if (MaskLayout) 
			{
				Args.MaskLayout = MaskLayout->LinkedAddress;
			}

			Args.LayoutIndex = LayoutIndex;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	namespace
	{
		class Sink_MeshMaskClipUVMaskSource
		{
		public:

			// \TODO This is recursive and may cause stack overflows in big models.

			UE::Mutable::Private::Ptr<ASTOp> Apply(const ASTOpMeshMaskClipUVMask* root)
			{
				Root = root;
				m_oldToNew.Empty();

				InitialSource = root->Source.child();
				UE::Mutable::Private::Ptr<ASTOp> NewSource = Visit(InitialSource);

				// If there is any change, it is the new root.
				if (NewSource != InitialSource)
				{
					return NewSource;
				}

				return nullptr;
			}

		protected:

			const ASTOpMeshMaskClipUVMask* Root;
			UE::Mutable::Private::Ptr<ASTOp> InitialSource;
			TMap<UE::Mutable::Private::Ptr<ASTOp>, UE::Mutable::Private::Ptr<ASTOp>> m_oldToNew;
			TArray<UE::Mutable::Private::Ptr<ASTOp>> m_newOps;

			UE::Mutable::Private::Ptr<ASTOp> Visit(const UE::Mutable::Private::Ptr<ASTOp>& at)
			{
				if (!at) return nullptr;

				// Newly created?
				if (m_newOps.Find(at) != INDEX_NONE)
				{
					return at;
				}

				// Already visited?
				UE::Mutable::Private::Ptr<ASTOp>* cacheIt = m_oldToNew.Find(at);
				if (cacheIt)
				{
					return *cacheIt;
				}

				UE::Mutable::Private::Ptr<ASTOp> newAt = at;
				switch (at->GetOpType())
				{

				// This cannot be sunk since the result is different. Since the clipping is now correctly
				// generated at the end of the chain when really necessary, this wrong optimisation is no 
				// longer needed.
				//case EOpType::ME_MORPH:
		        //{
		        //    break;
		        //}

				case EOpType::ME_REMOVEMASK:
				{
					// Remove this op:
					// This may lead to the mask being bigger than needed since it will include
					// faces removed by the ignored removemask, but it is ok

					// TODO: Swap instead of ignore, and implement removemask on a mask?
					const ASTOpMeshRemoveMask* typedAt = static_cast<const ASTOpMeshRemoveMask*>(at.get());
					newAt = Visit(typedAt->source.child());
					break;
				}

				case EOpType::ME_ADDMETADATA:
				{
					Ptr<ASTOpMeshAddMetadata> newOp = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(at);
					newOp->Source = Visit(newOp->Source.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_PREPARELAYOUT:
				{
					// Ignore the prepare op in the source: it doesn't contribute to the mask generation (the one in the SourceUV is used for UVs).
					const ASTOpMeshPrepareLayout* typedAt = static_cast<const ASTOpMeshPrepareLayout*>(at.get());
					newAt = Visit(typedAt->Mesh.child());
					break;
				}

				case EOpType::ME_CONDITIONAL:
				{
					// We move the mask creation down the two paths
					// It always needs to be a clone because otherwise we could be modifying an
					// instruction that shouldn't if the parent was a ME_REMOVEMASK above and we
					// skipped the cloning for the parent.
					Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
					newOp->yes = Visit(newOp->yes.child());
					newOp->no = Visit(newOp->no.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_SWITCH:
				{
					// We move the mask creation down all the paths
					Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
					newOp->Default = Visit(newOp->Default.child());
					for (ASTOpSwitch::FCase& c : newOp->Cases)
					{
						c.Branch = Visit(c.Branch.child());
					}
					newAt = newOp;
					break;
				}

				// This cannot be sunk since the result is different. Since the clipping is now correctly
				// generated at the end of the chain when really necessary, this wrong optimisation is no 
				// longer needed.
				//case EOpType::ME_CLIPMORPHPLANE:
	   //         {
	   //             // We move the mask creation down the Source
	   //             auto typedAt = dynamic_cast<const ASTOpMeshClipMorphPlane*>(at.get());
	   //             newAt = Visit(typedAt->Source.child());
	   //             break;
	   //         }

				default:
				{
					//
					if (at != InitialSource)
					{
						Ptr<ASTOpMeshMaskClipUVMask> newOp = UE::Mutable::Private::Clone<ASTOpMeshMaskClipUVMask>(Root);
						newOp->Source = at;
						newAt = newOp;
					}
					break;
				}

				}

				m_oldToNew.Add(at, newAt);

				return newAt;
			}
		};


		class Sink_MeshMaskClipUVMaskClip
		{
		public:

			// \TODO This is recursive and may cause stack overflows in big models.

			UE::Mutable::Private::Ptr<ASTOp> Apply(const ASTOpMeshMaskClipUVMask* root)
			{
				Root = root;
				m_oldToNew.Empty();

				m_initialClip = root->MaskImage.child();
				UE::Mutable::Private::Ptr<ASTOp> NewClip = Visit(m_initialClip);

				// If there is any change, it is the new root.
				if (NewClip != m_initialClip)
				{
					return NewClip;
				}

				return nullptr;
			}

		protected:

			const ASTOpMeshMaskClipUVMask* Root;
			UE::Mutable::Private::Ptr<ASTOp> m_initialClip;
			TMap<UE::Mutable::Private::Ptr<ASTOp>, UE::Mutable::Private::Ptr<ASTOp>> m_oldToNew;
			TArray<UE::Mutable::Private::Ptr<ASTOp>> m_newOps;

			UE::Mutable::Private::Ptr<ASTOp> Visit(const UE::Mutable::Private::Ptr<ASTOp>& at)
			{
				if (!at) return nullptr;

				// Newly created?
				if (m_newOps.Contains(at))
				{
					return at;
				}

				// Already visited?
				UE::Mutable::Private::Ptr<ASTOp>* cacheIt = m_oldToNew.Find(at);
				if (cacheIt)
				{
					return *cacheIt;
				}

				UE::Mutable::Private::Ptr<ASTOp> newAt = at;
				switch (at->GetOpType())
				{

				case EOpType::ME_CONDITIONAL:
				{
					// We move the mask creation down the two paths
					// It always needs to be a clone because otherwise we could be modifying an
					// instruction that shouldn't if the parent was a ME_REMOVEMASK above and we
					// skipped the cloning for the parent.
					Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
					newOp->yes = Visit(newOp->yes.child());
					newOp->no = Visit(newOp->no.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_SWITCH:
				{
					// We move the mask creation down all the paths
					Ptr<ASTOpSwitch> newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
					newOp->Default = Visit(newOp->Default.child());
					for (ASTOpSwitch::FCase& c : newOp->Cases)
					{
						c.Branch = Visit(c.Branch.child());
					}
					newAt = newOp;
					break;
				}


				default:
				{
					//
					if (at != m_initialClip)
					{
						Ptr<ASTOpMeshMaskClipUVMask> newOp = UE::Mutable::Private::Clone<ASTOpMeshMaskClipUVMask>(Root);
						newOp->MaskImage = at;
						newAt = newOp;
					}
					break;
				}

				}

				m_oldToNew.Add(at, newAt);

				return newAt;
			}
		};

	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMaskClipUVMask::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		// \TODO: Add logic state to the sinkers to avoid explosion with switches in both branches and similar cases.

		// This will sink the operation down the Source
		Sink_MeshMaskClipUVMaskSource SinkerSource;
		UE::Mutable::Private::Ptr<ASTOp> at = SinkerSource.Apply(this);

		// If we didn't sink it.
		if (!at || at == this)
		{
			// This will sink the operation down the Mask child
			Sink_MeshMaskClipUVMaskClip SinkerClip;
			at = SinkerClip.Apply(this);
		}

		return at;
	}


	//-------------------------------------------------------------------------------------------------
	FSourceDataDescriptor ASTOpMeshMaskClipUVMask::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}

