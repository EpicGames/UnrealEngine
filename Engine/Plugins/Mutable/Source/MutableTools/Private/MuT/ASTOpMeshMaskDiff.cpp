// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshMaskDiff.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "HAL/PlatformMath.h"

namespace UE::Mutable::Private
{

	ASTOpMeshMaskDiff::ASTOpMeshMaskDiff()
		: Source(this)
		, Fragment(this)
	{
	}


	ASTOpMeshMaskDiff::~ASTOpMeshMaskDiff()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshMaskDiff::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshMaskDiff* Other = static_cast<const ASTOpMeshMaskDiff*>(&OtherUntyped);
			return Source == Other->Source &&
				Fragment == Other->Fragment;
		}
		return false;
	}


	uint32 ASTOpMeshMaskDiff::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Fragment));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpMeshMaskDiff::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshMaskDiff> New = new ASTOpMeshMaskDiff();
		New->Source = MapChild(Source.child());
		New->Fragment = MapChild(Fragment.child());
		return New;
	}


	void ASTOpMeshMaskDiff::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
		Func(Fragment);
	}


	void ASTOpMeshMaskDiff::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshMaskDiffArgs Args;
			FMemory::Memzero(Args);

			if (Source) 
			{
				Args.Source = Source->LinkedAddress;
			}

			if (Fragment) 
			{
				Args.Fragment = Fragment->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMaskDiff::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		if (!Fragment)
		{
			return nullptr;
		}

		UE::Mutable::Private::Ptr<ASTOp> ResultOp;

		EOpType FragmentType = Fragment->GetOpType();
		switch (FragmentType)
		{

		case EOpType::ME_ADDMETADATA:
		{
			// Tags in the fragment can be ignored.
			const ASTOpMeshAddMetadata* Add = static_cast<const ASTOpMeshAddMetadata*>(Fragment.child().get());

			Ptr<ASTOpMeshMaskDiff> NewAt = UE::Mutable::Private::Clone<ASTOpMeshMaskDiff>(this);
			NewAt->Fragment = Add->Source.child();
			ResultOp = NewAt;
			break;
		}

		case EOpType::ME_PREPARELAYOUT:
		{
			// Ignore the prepare op in the source: it doesn't contribute to the diff generation.
			const ASTOpMeshPrepareLayout* Prepare = static_cast<const ASTOpMeshPrepareLayout*>(Source.child().get());

			Ptr<ASTOpMeshMaskDiff> NewAt = UE::Mutable::Private::Clone<ASTOpMeshMaskDiff>(this);
			NewAt->Source = Prepare->Mesh.child();
			ResultOp = NewAt;
			break;
		}

		case EOpType::ME_SWITCH:
		{
			// Move the mask diff down all the paths
			Ptr<ASTOpSwitch> NewOp = UE::Mutable::Private::Clone<ASTOpSwitch>(Fragment.child().get());

			if (NewOp->Default)
			{
				Ptr<ASTOpMeshMaskDiff> defOp = UE::Mutable::Private::Clone<ASTOpMeshMaskDiff>(this);
				defOp->Fragment = NewOp->Default.child();
				NewOp->Default = defOp;
			}

			// We need to copy the options because we change them
			for (int32 v = 0; v < NewOp->Cases.Num(); ++v)
			{
				if (NewOp->Cases[v].Branch)
				{
					Ptr<ASTOpMeshMaskDiff> bOp = UE::Mutable::Private::Clone<ASTOpMeshMaskDiff>(this);
					bOp->Fragment = NewOp->Cases[v].Branch.child();
					NewOp->Cases[v].Branch = bOp;
				}
			}

			ResultOp = NewOp;
			break;
		}

		default:
			break;
		}

		// If we didn't optimize the fragment, see ifg we can optimize the source.
		if (!ResultOp && Source)
		{
			EOpType SourceType = Source->GetOpType();
			switch (SourceType)
			{

			case EOpType::ME_ADDMETADATA:
			{
				// Tags in the source can be ignored for mask generation.
				const ASTOpMeshAddMetadata* Add = static_cast<const ASTOpMeshAddMetadata*>(Source.child().get());

				Ptr<ASTOpMeshMaskDiff> NewAt = UE::Mutable::Private::Clone<ASTOpMeshMaskDiff>(this);
				NewAt->Source = Add->Source.child();
				ResultOp = NewAt;
				break;
			}

			case EOpType::ME_PREPARELAYOUT:
			{
				// Ignore the prepare op in the source: it doesn't contribute to the diff generation.
				const ASTOpMeshPrepareLayout* Prepare = static_cast<const ASTOpMeshPrepareLayout*>(Source.child().get());

				Ptr<ASTOpMeshMaskDiff> NewAt = UE::Mutable::Private::Clone<ASTOpMeshMaskDiff>(this);
				NewAt->Source = Prepare->Mesh.child();
				ResultOp = NewAt;
				break;
			}

			default:
				break;
			}
		}

		return ResultOp;
	}


	FSourceDataDescriptor ASTOpMeshMaskDiff::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
