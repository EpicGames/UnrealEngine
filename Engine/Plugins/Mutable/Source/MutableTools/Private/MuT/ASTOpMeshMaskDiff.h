// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"

namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpMeshMaskDiff final : public ASTOp
	{
	public:

		ASTChild Source;
		ASTChild Fragment;

	public:

		ASTOpMeshMaskDiff();
		ASTOpMeshMaskDiff(const ASTOpMeshMaskDiff&) = delete;
		~ASTOpMeshMaskDiff();

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::ME_MASKDIFF; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}
