// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshMerge.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpMeshMerge::ASTOpMeshMerge()
		: Base(this)
		, Added(this)
	{
	}


	ASTOpMeshMerge::~ASTOpMeshMerge()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshMerge::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshMerge* Other = static_cast<const ASTOpMeshMerge*>(&OtherUntyped);
			return Base == Other->Base &&
				Added == Other->Added &&
				NewSurfaceID == Other->NewSurfaceID;
		}
		return false;
	}


	uint32 ASTOpMeshMerge::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(Added));
		Result = HashCombineFast(Result, GetTypeHash(NewSurfaceID));

		return Result;
	}


	Ptr<ASTOp> ASTOpMeshMerge::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshMerge> New = new ASTOpMeshMerge();
		New->Base = MapChild(Base.child());
		New->Added = MapChild(Added.child());
		New->NewSurfaceID = NewSurfaceID;
		return New;
	}


	void ASTOpMeshMerge::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(Added);
	}


	void ASTOpMeshMerge::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshMergeArgs Args;
			FMemory::Memzero(Args);

			if (Base) 
			{
				Args.Base = Base->LinkedAddress;
			}

			if (Added) 
			{
				Args.Added = Added->LinkedAddress;
			}

			Args.NewSurfaceID = NewSurfaceID;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FSourceDataDescriptor ASTOpMeshMerge::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
