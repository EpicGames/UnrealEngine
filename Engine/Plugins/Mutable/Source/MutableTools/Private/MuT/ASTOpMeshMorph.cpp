// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshMorph.h"

#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "HAL/PlatformMath.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	ASTOpMeshMorph::ASTOpMeshMorph()
		: Factor(this), Base(this)
	{
	}


	//---------------------------------------------------------------------------------------------
	ASTOpMeshMorph::~ASTOpMeshMorph()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//---------------------------------------------------------------------------------------------
	bool ASTOpMeshMorph::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshMorph* Other = static_cast<const ASTOpMeshMorph*>(&OtherUntyped);
			return Name == Other->Name && Factor == Other->Factor && Base == Other->Base;
		}
		return false;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMorph::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshMorph> New = new ASTOpMeshMorph();
		New->Name = Name;
		New->Factor = MapChild(Factor.child());
		New->Base = MapChild(Base.child());
		return New;
	}


	//---------------------------------------------------------------------------------------------
	void ASTOpMeshMorph::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Factor);
		Func(Base);
	}


	//---------------------------------------------------------------------------------------------
	uint32 ASTOpMeshMorph::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Name));
		Result = HashCombineFast(Result, GetTypeHash(Factor));
		Result = HashCombineFast(Result, GetTypeHash(Base));
		
		return Result;
	}


	//---------------------------------------------------------------------------------------------
	void ASTOpMeshMorph::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshMorphArgs Args;
			FMemory::Memzero(Args);


			Args.Name = Program.AddConstant(Name.ToString());
			Args.Base = Base ? Base->LinkedAddress : 0;
			Args.Factor = Factor ? Factor->LinkedAddress : 0;

			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
			
		}
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshMorph::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		if (!Base.child())
		{
			return nullptr;
		}

		// Base optimizations
		EOpType BaseType = Base.child()->GetOpType();
		switch (BaseType)
		{

		case EOpType::ME_ADDMETADATA:
		{
			// Add the base tags after morphing
			Ptr<ASTOpMeshAddMetadata> NewAddMetadata = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(Base.child());

			if (NewAddMetadata->Source)
			{
				Ptr<ASTOpMeshMorph> New = UE::Mutable::Private::Clone<ASTOpMeshMorph>(this);
				New->Base = NewAddMetadata->Source.child();
				NewAddMetadata->Source = New;
			}

			NewOp = NewAddMetadata;
			break;
		}

		default:
			break;

		}

		return NewOp;
	}


	//-------------------------------------------------------------------------------------------------
	FSourceDataDescriptor ASTOpMeshMorph::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
