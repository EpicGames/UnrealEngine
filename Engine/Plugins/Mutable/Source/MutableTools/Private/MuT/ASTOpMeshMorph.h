// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	//---------------------------------------------------------------------------------------------
	//! From a source mesh, remove a list of fragments with a condition.
	//---------------------------------------------------------------------------------------------
	class ASTOpMeshMorph final : public ASTOp
	{
	public:
		FName Name;
		
		//! Factor selecting what morphs to apply and with what weight.
		ASTChild Factor;

		//! Base mesh to morph.
		ASTChild Base;

	public:

		ASTOpMeshMorph();
		ASTOpMeshMorph(const ASTOpMeshMorph&) = delete;
		~ASTOpMeshMorph() override;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::ME_MORPH; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Link(FProgram& program, FLinkerOptions*) override;
		Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

	};

}

