// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpMeshAddMetadata.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpMeshPrepareLayout::ASTOpMeshPrepareLayout()
		: Mesh(this)
		, Layout(this)
	{
	}


	ASTOpMeshPrepareLayout::~ASTOpMeshPrepareLayout()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshPrepareLayout::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshPrepareLayout* Other = static_cast<const ASTOpMeshPrepareLayout*>(&OtherUntyped);
			return Mesh == Other->Mesh &&
				Layout == Other->Layout &&
				LayoutChannel == Other->LayoutChannel &&
				bUseAbsoluteBlockIds == Other->bUseAbsoluteBlockIds &&
				bNormalizeUVs == Other->bNormalizeUVs &&
				bClampUVIslands == Other->bClampUVIslands &&
				bEnsureAllVerticesHaveLayoutBlock == Other->bEnsureAllVerticesHaveLayoutBlock;
		}
		return false;
	}


	uint32 ASTOpMeshPrepareLayout::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(Layout));
		Result = HashCombineFast(Result, GetTypeHash(bUseAbsoluteBlockIds));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpMeshPrepareLayout::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshPrepareLayout> New = new ASTOpMeshPrepareLayout();
		New->Mesh = MapChild(Mesh.child());
		New->Layout = MapChild(Layout.child());
		New->LayoutChannel = LayoutChannel;
		New->bUseAbsoluteBlockIds = bUseAbsoluteBlockIds;
		New->bNormalizeUVs = bNormalizeUVs;
		New->bClampUVIslands = bClampUVIslands;
		New->bEnsureAllVerticesHaveLayoutBlock = bEnsureAllVerticesHaveLayoutBlock;
		return New;
	}


	void ASTOpMeshPrepareLayout::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Mesh);
		Func(Layout);
	}


	void ASTOpMeshPrepareLayout::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshPrepareLayoutArgs Args;
			FMemory::Memzero(Args);

			if (Mesh) 
			{
				Args.Mesh = Mesh->LinkedAddress;
			}

			if (Layout)
			{
				Args.Layout = Layout->LinkedAddress;
			}

			Args.LayoutChannel = LayoutChannel;
			Args.bUseAbsoluteBlockIds = bUseAbsoluteBlockIds;
			Args.bNormalizeUVs = bNormalizeUVs;
			Args.bClampUVIslands = bClampUVIslands;
			Args.bEnsureAllVerticesHaveLayoutBlock = bEnsureAllVerticesHaveLayoutBlock;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}


	FSourceDataDescriptor ASTOpMeshPrepareLayout::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
