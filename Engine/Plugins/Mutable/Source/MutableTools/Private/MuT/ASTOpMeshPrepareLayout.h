// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpMeshPrepareLayout final : public ASTOp
	{
	public:

		ASTChild Mesh;
		ASTChild Layout;
		uint8 LayoutChannel = 0;
		bool bUseAbsoluteBlockIds = false;
		bool bNormalizeUVs = false;
		bool bClampUVIslands = false;
		bool bEnsureAllVerticesHaveLayoutBlock = false;

	public:

		ASTOpMeshPrepareLayout();
		ASTOpMeshPrepareLayout(const ASTOpMeshPrepareLayout&) = delete;
		~ASTOpMeshPrepareLayout();

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::ME_PREPARELAYOUT; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};


}

