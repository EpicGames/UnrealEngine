// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshProject.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpMeshProject::ASTOpMeshProject()
		: Mesh(this)
		, Projector(this)
	{
	}


	ASTOpMeshProject::~ASTOpMeshProject()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshProject::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType()==GetOpType())
		{
			const ASTOpMeshProject* Other = static_cast<const ASTOpMeshProject*>(&OtherUntyped);
			return Mesh == Other->Mesh &&
				Projector == Other->Projector &&
				LayoutIndex == Other->LayoutIndex;
		}
		return false;
	}


	uint32 ASTOpMeshProject::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, GetTypeHash(Projector));
		Result = HashCombineFast(Result, GetTypeHash(LayoutIndex));

		return Result;
	}


	Ptr<ASTOp> ASTOpMeshProject::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshProject> New = new ASTOpMeshProject();
		New->Mesh = MapChild(Mesh.child());
		New->Projector = MapChild(Projector.child());
		New->LayoutIndex = LayoutIndex;
		return New;
	}


	void ASTOpMeshProject::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Mesh);
		Func(Projector);
	}


	void ASTOpMeshProject::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshProjectArgs Args;
			FMemory::Memzero(Args);

			if (Mesh) 
			{
				Args.Mesh = Mesh->LinkedAddress;
			}

			if (Projector)
			{
				Args.Projector = Projector->LinkedAddress;
			}

			Args.LayoutIndex = LayoutIndex;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	Ptr<ASTOp> ASTOpMeshProject::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		Ptr<ASTOp> SourceAt = Mesh.child();
		Ptr<ASTOp> ProjectorAt = Projector.child();

		if (!SourceAt || !ProjectorAt)
		{
			return NewOp;
		}

		EOpType SourceType = SourceAt->GetOpType();
		switch (SourceType)
		{

		case EOpType::ME_CONDITIONAL:
		{
			if (ProjectorAt->GetOpType() == EOpType::PR_CONSTANT)
			{
				// We move the project down the two paths
				Ptr<ASTOpConditional> nop = UE::Mutable::Private::Clone<ASTOpConditional>(SourceAt);

				Ptr<ASTOpMeshProject> aOp = UE::Mutable::Private::Clone<ASTOpMeshProject>(this);
				aOp->Mesh = nop->yes.child();
				nop->yes = aOp;

				Ptr<ASTOpMeshProject> bOp = UE::Mutable::Private::Clone<ASTOpMeshProject>(this);
				bOp->Mesh = nop->no.child();
				nop->no = bOp;

				NewOp = nop;
			}
			break;
		}

		case EOpType::ME_SWITCH:
		{
			if (ProjectorAt->GetOpType() == EOpType::PR_CONSTANT)
			{
				// Move the format down all the paths
				Ptr<ASTOpSwitch> nop = UE::Mutable::Private::Clone<ASTOpSwitch>(SourceAt);

				if (nop->Default)
				{
					Ptr<ASTOpMeshProject> defOp = UE::Mutable::Private::Clone<ASTOpMeshProject>(this);
					defOp->Mesh = nop->Default.child();
					nop->Default = defOp;
				}

				// We need to copy the options because we change them
				for (int32 CaseIndex = 0; CaseIndex < nop->Cases.Num(); ++CaseIndex)
				{
					if (nop->Cases[CaseIndex].Branch)
					{
						Ptr<ASTOpMeshProject> bOp = UE::Mutable::Private::Clone<ASTOpMeshProject>(this);
						bOp->Mesh = nop->Cases[CaseIndex].Branch.child();
						nop->Cases[CaseIndex].Branch = bOp;
					}
				}

				NewOp = nop;
			}
			break;
		}

		case EOpType::ME_ADDMETADATA:
		{
			// Add the tags after layout
			Ptr<ASTOpMeshAddMetadata> NewAddMetadata = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(SourceAt);

			if (NewAddMetadata->Source)
			{
				Ptr<ASTOpMeshProject> NewAt = UE::Mutable::Private::Clone<ASTOpMeshProject>(this);
				NewAt->Mesh = NewAddMetadata->Source.child();
				NewAddMetadata->Source = NewAt;
			}

			NewOp = NewAddMetadata;
			break;
		}

		default:
			break;
		}

		return NewOp;
	}


	FSourceDataDescriptor ASTOpMeshProject::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
