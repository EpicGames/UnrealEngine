// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshRemoveMask.h"

#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	//---------------------------------------------------------------------------------------------
	ASTOpMeshRemoveMask::ASTOpMeshRemoveMask()
		: source(this)
	{
	}


	//---------------------------------------------------------------------------------------------
	ASTOpMeshRemoveMask::~ASTOpMeshRemoveMask()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//---------------------------------------------------------------------------------------------
	void ASTOpMeshRemoveMask::AddRemove(const Ptr<ASTOp>& condition, const Ptr<ASTOp>& mask)
	{
		removes.Add({ ASTChild(this,condition), ASTChild(this,mask) });
	}


	//---------------------------------------------------------------------------------------------
	bool ASTOpMeshRemoveMask::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshRemoveMask* other = static_cast<const ASTOpMeshRemoveMask*>(&otherUntyped);
			return source == other->source && removes == other->removes && FaceCullStrategy==other->FaceCullStrategy;
		}
		return false;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshRemoveMask::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshRemoveMask> n = new ASTOpMeshRemoveMask();
		n->source = mapChild(source.child());
		n->FaceCullStrategy = FaceCullStrategy;
		for (const TPair<ASTChild, ASTChild>& r : removes)
		{
			n->removes.Add({ ASTChild(n,mapChild(r.Key.child())), ASTChild(n,mapChild(r.Value.child())) });
		}
		return n;
	}


	//---------------------------------------------------------------------------------------------
	void ASTOpMeshRemoveMask::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(source);
		for (TPair<ASTChild, ASTChild>& r : removes)
		{
			f(r.Key);
			f(r.Value);
		}
	}


	//---------------------------------------------------------------------------------------------
	uint32 ASTOpMeshRemoveMask::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(source));
		
		for (const TPair<ASTChild, ASTChild>& Remove : removes)
		{
			Result = HashCombineFast(Result, GetTypeHash(Remove.Key));
			Result = HashCombineFast(Result, GetTypeHash(Remove.Value));
		}
		
		return Result;
	}


	//---------------------------------------------------------------------------------------------
	void ASTOpMeshRemoveMask::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());

			FOperation::ADDRESS sourceAt = source ? source->LinkedAddress : 0;
			AppendCode(Program.ByteCode, sourceAt);

			AppendCode(Program.ByteCode, FaceCullStrategy);

			AppendCode(Program.ByteCode, (uint16)removes.Num());
			for (const TPair<ASTChild, ASTChild>& b : removes)
			{
				FOperation::ADDRESS condition = b.Key ? b.Key->LinkedAddress : 0;
				AppendCode(Program.ByteCode, condition);

				FOperation::ADDRESS remove = b.Value ? b.Value->LinkedAddress : 0;
				AppendCode(Program.ByteCode, remove);
			}
		}
	}


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	namespace
	{
		class Sink_MeshRemoveMaskAST
		{
		public:

			// \TODO This is recursive and may cause stack overflows in big models.

			UE::Mutable::Private::Ptr<ASTOp> Apply(const ASTOpMeshRemoveMask* root)
			{
				Root = root;
				m_oldToNew.Empty();

				InitialSource = root->source.child();
				UE::Mutable::Private::Ptr<ASTOp> newSource = Visit(InitialSource);

				// If there is any change, it is the new root.
				if (newSource != InitialSource)
				{
					return newSource;
				}

				return nullptr;
			}

		protected:

			const ASTOpMeshRemoveMask* Root;
			UE::Mutable::Private::Ptr<ASTOp> InitialSource;
			TMap<UE::Mutable::Private::Ptr<ASTOp>, UE::Mutable::Private::Ptr<ASTOp>> m_oldToNew;
			TArray<UE::Mutable::Private::Ptr<ASTOp>> m_newOps;

			UE::Mutable::Private::Ptr<ASTOp> Visit(const UE::Mutable::Private::Ptr<ASTOp>& at)
			{
				if (!at) return nullptr;

				// Newly created?
				if (m_newOps.Contains(at))
				{
					return at;
				}

				// Already visited?
				Ptr<ASTOp>* cacheIt = m_oldToNew.Find(at);
				if (cacheIt)
				{
					return *cacheIt;
				}

				UE::Mutable::Private::Ptr<ASTOp> newAt = at;
				switch (at->GetOpType())
				{

				case EOpType::ME_MORPH:
				{
					UE::Mutable::Private::Ptr<ASTOpMeshMorph> newOp = UE::Mutable::Private::Clone<ASTOpMeshMorph>(at);
					newOp->Base = Visit(newOp->Base.child());
					newAt = newOp;
					break;
				}

				case EOpType::ME_ADDMETADATA:
				{
					UE::Mutable::Private::Ptr<ASTOpMeshAddMetadata> newOp = UE::Mutable::Private::Clone<ASTOpMeshAddMetadata>(at);
					newOp->Source = Visit(newOp->Source.child());
					newAt = newOp;
					break;
				}

				// disabled to avoid code explosion (or bug?) TODO
//            case EOpType::ME_CONDITIONAL:
//            {
//                Ptr<ASTOpConditional> newOp = UE::Mutable::Private::Clone<ASTOpConditional>(at);
//                newOp->yes = Visit(newOp->yes.child());
//                newOp->no = Visit(newOp->no.child());
//                newAt = newOp;
//                break;
//            }

//            case EOpType::ME_SWITCH:
//            {
//                auto newOp = UE::Mutable::Private::Clone<ASTOpSwitch>(at);
//                newOp->def = Visit(newOp->def.child());
//                for( auto& c:newOp->cases )
//                {
//                    c.branch = Visit(c.branch.child());
//                }
//                newAt = newOp;
//                break;
//            }

				default:
				{
					//
					if (at != InitialSource)
					{
						Ptr<ASTOpMeshRemoveMask> newOp = UE::Mutable::Private::Clone<ASTOpMeshRemoveMask>(Root);
						newOp->source = at;
						newAt = newOp;
					}
					break;
				}

				}

				m_oldToNew.Add(at, newAt);

				return newAt;
			}
		};
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshRemoveMask::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Sink_MeshRemoveMaskAST sinker;
		UE::Mutable::Private::Ptr<ASTOp> at = sinker.Apply(this);

		// If not optimized already, see if we can optimize the "remove" branches
		if (!at)
		{
			Ptr<ASTOpMeshRemoveMask> NewOp;

			for (int32 RemoveIndex=0; RemoveIndex<removes.Num(); ++RemoveIndex)
			{
				if (!removes[RemoveIndex].Value)
				{
					continue;
				}

				EOpType RemoveType = removes[RemoveIndex].Value->GetOpType();
				switch (RemoveType)
				{
				case EOpType::ME_ADDMETADATA:
				{
					// It can be ignored.
					if (!NewOp)
					{
						NewOp = UE::Mutable::Private::Clone<ASTOpMeshRemoveMask>(this);
					}

					const ASTOpMeshAddMetadata* Add = static_cast<const ASTOpMeshAddMetadata*>(removes[RemoveIndex].Value.child().get());
					NewOp->removes[RemoveIndex].Value = Add->Source.child();

					break;
				}

				default:
					break;
				}
			}

			at = NewOp;
		}

		return at;
	}


	//-------------------------------------------------------------------------------------------------
	FSourceDataDescriptor ASTOpMeshRemoveMask::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (source)
		{
			return source->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
