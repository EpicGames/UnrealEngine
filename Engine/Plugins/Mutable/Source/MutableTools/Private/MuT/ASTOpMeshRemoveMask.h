// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** Remove a list of mesh fragments with a condition from a source mesh. */
	class ASTOpMeshRemoveMask final : public ASTOp
	{
	public:

		//! Source mesh to remove from.
		ASTChild source;

		//! Pairs of remove candidates: condition + mesh to remove
		TArray< TPair<ASTChild, ASTChild> > removes;

		/** Strategy to decide when to cull a face. */
		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;

	public:

		ASTOpMeshRemoveMask();
		ASTOpMeshRemoveMask(const ASTOpMeshRemoveMask&) = delete;
		~ASTOpMeshRemoveMask() override;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::ME_REMOVEMASK; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Link(FProgram& program, FLinkerOptions*) override;
		Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

		// Own interface
		void AddRemove(const Ptr<ASTOp>& condition, const Ptr<ASTOp>& mask);
	};

}

