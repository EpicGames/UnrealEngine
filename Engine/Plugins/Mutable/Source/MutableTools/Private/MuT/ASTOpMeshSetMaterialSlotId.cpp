// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshSetMaterialSlotId.h"

#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	//-------------------------------------------------------------------------------------------------
	ASTOpMeshSetMaterialSlotId::ASTOpMeshSetMaterialSlotId()
		: Mesh(this)
	{
	}


	//-------------------------------------------------------------------------------------------------
	ASTOpMeshSetMaterialSlotId::~ASTOpMeshSetMaterialSlotId()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpMeshSetMaterialSlotId::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshSetMaterialSlotId* other = static_cast<const ASTOpMeshSetMaterialSlotId*>(&OtherUntyped);
			return Mesh == other->Mesh && MaterialSlotId == other->MaterialSlotId;
		}

		return false;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshSetMaterialSlotId::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshSetMaterialSlotId> NewOp = new ASTOpMeshSetMaterialSlotId();
		NewOp->Mesh = mapChild(Mesh.child());
		NewOp->MaterialSlotId = MaterialSlotId;

		return NewOp;
	}


	EOpType ASTOpMeshSetMaterialSlotId::GetOpType() const
	{
		return EOpType::ME_SETMATERIALSLOTID;
	}

	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpMeshSetMaterialSlotId::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Mesh));
		Result = HashCombineFast(Result, MaterialSlotId);
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpMeshSetMaterialSlotId::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(Mesh);
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpMeshSetMaterialSlotId::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshSetMaterialSlotIdArgs Args;
			FMemory::Memzero(Args);

			if (Mesh) 
			{
				Args.Mesh = Mesh->LinkedAddress;
			}

			Args.MaterialSlotId = MaterialSlotId;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());	

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

	//-------------------------------------------------------------------------------------------------
	FSourceDataDescriptor ASTOpMeshSetMaterialSlotId::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Mesh)
		{
			return Mesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}

}
