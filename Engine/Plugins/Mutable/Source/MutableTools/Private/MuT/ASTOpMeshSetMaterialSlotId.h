// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;

/**
 * Set a MaterialSlotId to an FMesh.
 */
class ASTOpMeshSetMaterialSlotId final : public ASTOp
{
public:

	ASTChild Mesh;
	uint32 MaterialSlotId = 0;

public:

	ASTOpMeshSetMaterialSlotId();
	ASTOpMeshSetMaterialSlotId(const ASTOpMeshSetMaterialSlotId&) = delete;
	~ASTOpMeshSetMaterialSlotId() override;

	// ASTOp interface
	EOpType GetOpType() const override;
	uint32 Hash() const override;
	void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
	bool IsEqual(const ASTOp& OtherUntyped) const override;
	Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
	void Link(FProgram& Program, FLinkerOptions*) override;
	virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
};

}

