// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshSkeletalMeshBreak.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "MuR/Model.h"


namespace UE::Mutable::Private
{
	ASTOpMeshSkeletalMeshBreak::ASTOpMeshSkeletalMeshBreak() :
		SkeletalMeshParameter(this)
	{
	}

	ASTOpMeshSkeletalMeshBreak::~ASTOpMeshSkeletalMeshBreak()
	{
		// Explicit call needed to avoid recursive destruction
		RemoveChildren();
	}


	void ASTOpMeshSkeletalMeshBreak::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(SkeletalMeshParameter);
	}


	bool ASTOpMeshSkeletalMeshBreak::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshSkeletalMeshBreak* Other = static_cast<const ASTOpMeshSkeletalMeshBreak*>(&OtherUntyped);
			return Type == Other->Type &&
				BaseLODIndex == Other->BaseLODIndex &&
				BaseSectionIndex == Other->BaseSectionIndex &&
				LODBias == Other->LODBias &&
				ConversionFlags == Other->ConversionFlags &&
				MeshID == Other->MeshID &&
				SkeletalMeshParameter == Other->SkeletalMeshParameter;
		}
		
		return false;
	}


	Ptr<ASTOp> ASTOpMeshSkeletalMeshBreak::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpMeshSkeletalMeshBreak> New = new ASTOpMeshSkeletalMeshBreak();
		New->Type = Type;
		New->BaseLODIndex = BaseLODIndex;
		New->BaseSectionIndex = BaseSectionIndex;
		New->LODBias = LODBias;
		New->ConversionFlags = ConversionFlags;
		New->MeshID = MeshID;
		New->SkeletalMeshParameter = ASTChild(New, MapChild(SkeletalMeshParameter.child()));

		return New;
	}


	EOpType ASTOpMeshSkeletalMeshBreak::GetOpType() const 
	{ 
		return Type; 
	}


	uint32 ASTOpMeshSkeletalMeshBreak::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(SkeletalMeshParameter));
		Result = HashCombineFast(Result, GetTypeHash(BaseLODIndex));
		Result = HashCombineFast(Result, GetTypeHash(BaseSectionIndex));
		Result = HashCombineFast(Result, GetTypeHash(LODBias));
		Result = HashCombineFast(Result, GetTypeHash(ConversionFlags));
		Result = HashCombineFast(Result, GetTypeHash(MeshID));

		return Result;
	}


	void ASTOpMeshSkeletalMeshBreak::Assert()
	{
		ASTOp::Assert();
	}


	void ASTOpMeshSkeletalMeshBreak::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{

			FOperation::MeshSkeletalMeshBreakArgs Args;
			FMemory::Memzero(Args);

			check(SkeletalMeshParameter->LinkedAddress);
			Args.SkeletalMeshParameter = SkeletalMeshParameter->LinkedAddress;

			Args.BaseLOD = BaseLODIndex;
			Args.BaseSection = BaseSectionIndex;
			Args.LODBias = LODBias;
			Args.Flags = ConversionFlags;
			Args.MeshID = MeshID;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(Type, Program.ByteCode.Num());

			AppendCode(Program.ByteCode, Type);
			AppendCode(Program.ByteCode, Args);
		}
	}
}
