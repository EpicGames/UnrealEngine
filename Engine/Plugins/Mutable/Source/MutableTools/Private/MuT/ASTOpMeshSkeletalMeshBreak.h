// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpMeshSkeletalMeshBreak final : public ASTOp
	{
	public:
		EOpType Type = EOpType::ME_SKELETALMESH_BREAK;

		/** Used by some parameter types (Mesh) to specify which subdata from the actual parameter value should be used in the operation. */
		uint8 BaseLODIndex = 0;
		uint8 BaseSectionIndex = 0;
		uint8 LODBias = 0;
		uint8 ConversionFlags = 0;
		uint32 MeshID = 0;

		ASTChild SkeletalMeshParameter;

		int32 LinkedParameterIndex = -1;

		ASTOpMeshSkeletalMeshBreak();
		virtual ~ASTOpMeshSkeletalMeshBreak() override;

		// ASTOp interface
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Assert() override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
	};
}

