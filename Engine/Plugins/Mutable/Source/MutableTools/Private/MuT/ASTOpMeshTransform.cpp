// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshTransform.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	ASTOpMeshTransform::ASTOpMeshTransform()
		: source(this)
	{
	}


	ASTOpMeshTransform::~ASTOpMeshTransform()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpMeshTransform::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpMeshTransform* other = static_cast<const ASTOpMeshTransform*>(&otherUntyped);
			return source == other->source &&
				matrix == other->matrix;
		}
		return false;
	}


	uint32 ASTOpMeshTransform::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(source));
		Result = HashCombineFast(Result, matrix.ComputeHash());

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpMeshTransform::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpMeshTransform> n = new ASTOpMeshTransform();
		n->matrix = matrix;
		n->source = mapChild(source.child());
		return n;
	}


	void ASTOpMeshTransform::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(source);
	}


	void ASTOpMeshTransform::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::MeshTransformArgs Args;
			FMemory::Memzero(Args);

			if (source) 
			{
				Args.source = source->LinkedAddress;
			}

			Args.matrix = Program.AddConstant(matrix);

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}


	FSourceDataDescriptor ASTOpMeshTransform::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (source)
		{
			return source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	ASTOp::EClosedMeshTest ASTOpMeshTransform::IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const
	{
		if (source)
		{
			return source->IsClosedMesh(Cache);
		}
		return EClosedMeshTest::Unknown;
	}

}
