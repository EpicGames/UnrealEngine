// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpMeshTransform final : public ASTOp
	{
	public:

		ASTChild source;

		FMatrix44f matrix;

	public:

		ASTOpMeshTransform();
		ASTOpMeshTransform(const ASTOpMeshTransform&) = delete;
		~ASTOpMeshTransform();

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::ME_TRANSFORM; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
		virtual EClosedMeshTest IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const override;

	};


}

