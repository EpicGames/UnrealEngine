// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshTransformWithBone.h"

#include "ASTOpImageTransform.h"


UE::Mutable::Private::ASTOpMeshTransformWithBone::ASTOpMeshTransformWithBone()
	: SourceMesh(this)
	, Matrix(this)
{
}

UE::Mutable::Private::ASTOpMeshTransformWithBone::~ASTOpMeshTransformWithBone()
{
	// Explicit call needed to avoid recursive destruction
	ASTOp::RemoveChildren();
}

uint32 UE::Mutable::Private::ASTOpMeshTransformWithBone::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
	
	Result = HashCombineFast(Result, GetTypeHash(SourceMesh));
	Result = HashCombineFast(Result, GetTypeHash(Matrix));
	Result = HashCombineFast(Result, GetTypeHash(BoneName));
	Result = HashCombineFast(Result, GetTypeHash(ThresholdFactor));

	return Result;
}

bool UE::Mutable::Private::ASTOpMeshTransformWithBone::IsEqual(const ASTOp& OtherUntyped) const
{
	if (GetOpType() == OtherUntyped.GetOpType())
	{
		const ASTOpMeshTransformWithBone& Other = static_cast<const ASTOpMeshTransformWithBone&>(OtherUntyped);
		return SourceMesh == Other.SourceMesh
			&& BoneName == Other.BoneName
			&& Matrix == Other.Matrix
			&& ThresholdFactor == Other.ThresholdFactor;
	}
	return false;
}

UE::Mutable::Private::Ptr<UE::Mutable::Private::ASTOp> UE::Mutable::Private::ASTOpMeshTransformWithBone::Clone(MapChildFuncRef MapChild) const
{
	Ptr<ASTOpMeshTransformWithBone> NewASTOp = new ASTOpMeshTransformWithBone();
	NewASTOp->SourceMesh = MapChild(SourceMesh.child());
	NewASTOp->Matrix = MapChild(Matrix.child());
	NewASTOp->BoneName = BoneName;
	NewASTOp->ThresholdFactor = ThresholdFactor;
	return NewASTOp;
}

void UE::Mutable::Private::ASTOpMeshTransformWithBone::ForEachChild(const TFunctionRef<void(ASTChild&)> Function)
{
	Function(SourceMesh);
	Function(Matrix);
}

void UE::Mutable::Private::ASTOpMeshTransformWithBone::Link(FProgram& Program, FLinkerOptions* Options)
{
	if (!LinkedAddress)
	{
		FOperation::MeshTransformWithBoneArgs Args;
		FMemory::Memzero(Args);
		if (SourceMesh)
		{
			Args.SourceMesh = SourceMesh->LinkedAddress;
		}

		if (Matrix)
		{
			Args.Matrix = Matrix->LinkedAddress;
		}

		Args.BoneId = Program.AddConstant(BoneName);
		Args.ThresholdFactor = ThresholdFactor;

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}

UE::Mutable::Private::FSourceDataDescriptor UE::Mutable::Private::ASTOpMeshTransformWithBone::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
{
	if (SourceMesh)
	{
		return SourceMesh->GetSourceDataDescriptor(Context);
	}

	return {};
}
