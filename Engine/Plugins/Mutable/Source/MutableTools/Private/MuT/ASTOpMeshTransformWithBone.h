// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/Skeleton.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;


//---------------------------------------------------------------------------------------------
//!
//---------------------------------------------------------------------------------------------
class ASTOpMeshTransformWithBone final : public ASTOp
{
public:
	ASTChild SourceMesh;
	ASTChild Matrix;
	FName BoneName = NAME_None;
	float ThresholdFactor = 0.05f;
public:

	ASTOpMeshTransformWithBone();
	ASTOpMeshTransformWithBone(const ASTOpMeshTransformWithBone&) = delete;
	~ASTOpMeshTransformWithBone() override;

	EOpType GetOpType() const override { return EOpType::ME_TRANSFORMWITHBONE; }
	uint32 Hash() const override;
	bool IsEqual(const ASTOp& otherUntyped) const override;
	Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
	void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
	void Link(FProgram& program, FLinkerOptions* Options) override;
	virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

};

}

