// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpMeshTransformWithBoundingMesh.h"

#include "ASTOpImageTransform.h"


UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::ASTOpMeshTransformWithBoundingMesh()
	: source(this)
	, boundingMesh(this)
	, matrix(this)
{
}

UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::~ASTOpMeshTransformWithBoundingMesh()
{
	// Explicit call needed to avoid recursive destruction
	ASTOp::RemoveChildren();
}

uint32 UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
	
	Result = HashCombineFast(Result, GetTypeHash(source));
	Result = HashCombineFast(Result, GetTypeHash(boundingMesh));
	Result = HashCombineFast(Result, GetTypeHash(matrix));

	return Result;
}

bool UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::IsEqual(const ASTOp& otherUntyped) const
{
	if (GetOpType() == otherUntyped.GetOpType())
	{
		const ASTOpMeshTransformWithBoundingMesh& other = static_cast<const ASTOpMeshTransformWithBoundingMesh&>(otherUntyped);
		return source == other.source && boundingMesh == other.boundingMesh && matrix == other.matrix; 
	}
	return false;
}

UE::Mutable::Private::Ptr<UE::Mutable::Private::ASTOp> UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::Clone(MapChildFuncRef mapChild) const
{
	Ptr<ASTOpMeshTransformWithBoundingMesh> n = new ASTOpMeshTransformWithBoundingMesh();
	n->source = mapChild(source.child());
	n->boundingMesh = mapChild(boundingMesh.child());
	n->matrix = mapChild(matrix.child());
	return n;
}

void UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
{
	f(source);
	f(boundingMesh);
	f(matrix);
}

void UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::Link(FProgram& Program, FLinkerOptions* Options)
{
	if (!LinkedAddress)
	{
		FOperation::MeshTransformWithinMeshArgs Args;
		FMemory::Memzero(Args);

		if (source)
		{
			Args.sourceMesh = source->LinkedAddress;
		}
		if (boundingMesh)
		{
			Args.boundingMesh = boundingMesh->LinkedAddress;
		}
		if (matrix)
		{
			Args.matrix = matrix->LinkedAddress;
		}
	
		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}
}

UE::Mutable::Private::FSourceDataDescriptor UE::Mutable::Private::ASTOpMeshTransformWithBoundingMesh::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
{
	if (source)
	{
		return source->GetSourceDataDescriptor(Context);
	}

	return {};
}
