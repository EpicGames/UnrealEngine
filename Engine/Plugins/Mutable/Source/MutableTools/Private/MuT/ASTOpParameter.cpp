// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpParameter.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{
	ASTOpParameter::~ASTOpParameter()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	void ASTOpParameter::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		for (FRangeData& c : Ranges)
		{
			f(c.rangeSize);
		}
		
		for (TPair<FParameterKey, ASTChild>& Element : ImageOperations)
		{
			f(Element.Value);
		}
	}


	bool ASTOpParameter::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpParameter* Other = static_cast<const ASTOpParameter*>(&OtherUntyped);
			return Type == Other->Type &&
				Parameter == Other->Parameter &&
				Ranges == Other->Ranges &&
				ImageOperations == Other->ImageOperations;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpParameter::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpParameter> n = new ASTOpParameter();
		n->Type = Type;
		n->Parameter = Parameter;

		for (const FRangeData& c : Ranges)
		{
			n->Ranges.Emplace(n.get(), MapChild(c.rangeSize.child()), c.rangeName, c.rangeUID);
		}
		
		for (const TPair<FParameterKey, ASTChild>& ImageOperation : ImageOperations)
		{
			n->ImageOperations.Add(ImageOperation.Key, ASTChild(n, MapChild(ImageOperation.Value.child())));
		}

		return n;
	}


	EOpType ASTOpParameter::GetOpType() const 
	{ 
		return Type; 
	}


	uint32 ASTOpParameter::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Parameter.Type));
		Result = HashCombineFast(Result, GetTypeHash(Parameter.Name.Len()));

		return Result;
	}


	void ASTOpParameter::Assert()
	{
		ASTOp::Assert();
	}


	void ASTOpParameter::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			LinkedParameterIndex = Program.Parameters.Find(Parameter);

			// If this fails, it means an ASTOpParameter was created at code generation time, but not registered into the
			// parameters map in the CodeGenerator_FirstPass.
			check(LinkedParameterIndex != INDEX_NONE);

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(Type, Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, Type);
		
			if (Type == EOpType::MI_PARAMETER)
			{
				FOperation::MaterialParameterArgs Args;
				FMemory::Memzero(Args);
				Args.Variable = (FOperation::ADDRESS)LinkedParameterIndex;
				
				// Images
				TArray<FName> ImageParameterNames;
				TArray<uint32> ImageParameterOperations;
				for (const TPair<FParameterKey, ASTChild>& Element : ImageOperations)
				{
					ImageParameterNames.Add(Element.Key.ParameterName);
					ImageParameterOperations.Add(Element.Value->LinkedAddress);
		 		}

				Args.ImageParameterNames = Program.AddConstant(ImageParameterNames);
				Args.ImageParameterAddress = Program.AddConstant(ImageParameterOperations);

				// Ranges
				for (const FRangeData& d : Ranges)
				{
					FOperation::ADDRESS sizeAt = 0;
					uint16 rangeId = 0;
					LinkRange(Program, d, sizeAt, rangeId);
					Program.Parameters[LinkedParameterIndex].Ranges.Add(rangeId);
				}

				AppendCode(Program.ByteCode, Args);

				for (const TPair<FParameterKey, ASTChild>& Element : ImageOperations)
				{
					AppendCode(Program.ByteCode, Element.Key.LayerIndex);
				}
			}

			else
			{
				FOperation::ParameterArgs Args;
				FMemory::Memzero(Args);
				Args.variable = (FOperation::ADDRESS)LinkedParameterIndex;

				for (const FRangeData& d : Ranges)
				{
					FOperation::ADDRESS sizeAt = 0;
					uint16 rangeId = 0;
					LinkRange(Program, d, sizeAt, rangeId);
					Program.Parameters[LinkedParameterIndex].Ranges.Add(rangeId);
				}

				AppendCode(Program.ByteCode, Args);
			}

		}
	}


	int32 ASTOpParameter::EvaluateInt(ASTOpList& facts, bool& bOutUnknown) const
	{
		bOutUnknown = true;

		// Check the facts, in case we have the value for our parameter.
		for (const auto& f : facts)
		{
			if (f->GetOpType() == EOpType::BO_EQUAL_INT_CONST)
			{
				const ASTOpBoolEqualIntConst* typedFact = static_cast<const ASTOpBoolEqualIntConst*>(f.get());
				Ptr<ASTOp> Value = typedFact->Value.child();
				if (Value.get() == this)
				{
					bOutUnknown = false;
					return typedFact->Constant;
				}
				else
				{
					// We could try something more if it was an expression and it had the parameter
					// somewhere in it.
				}
			}
		}

		return 0;
	}


	ASTOp::FBoolEvalResult ASTOpParameter::EvaluateBool(ASTOpList& /*facts*/, FEvaluateBoolCache*) const
	{
		check(Type == EOpType::BO_PARAMETER);
		return BET_UNKNOWN;
	}


	FImageDesc ASTOpParameter::GetImageDesc(bool, FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		if (Type == EOpType::MI_PARAMETER)
		{
			if (Context)
			{
				if (const ASTChild* Image = ImageOperations.Find(Context->ParameterKey))
				{
					check(Image->Child);
					Result = Image->Child->GetImageDesc();
				}
			}
		}

		return Result;
	}


	bool ASTOpParameter::IsColorConstant(FVector4f&) const
	{
		return false;
	}
}
