// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;


	/** Parameter operation. */
	class ASTOpParameter final : public ASTOp
	{
	public:

		//! Type of parameter
		EOpType Type = EOpType::NONE;

		//!
		FParameterDesc Parameter;

		//** Ranges adding dimensions to this parameter. */
		TArray<FRangeData> Ranges;

		/** Image Parameters which are evaluated at runtime. */
		TMap<FParameterKey, ASTChild> ImageOperations;

		/** Index of the parameter in the final program parameter list. 
		* This is generated at link time, because the parameters may be reordered. 
		*/
		int32 LinkedParameterIndex = -1;

		virtual ~ASTOpParameter() override;

		// ASTOp interface.
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Assert() override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual int EvaluateInt(ASTOpList& Facts, bool& bOutUnknown) const override;
		virtual FBoolEvalResult EvaluateBool(ASTOpList& Facts, FEvaluateBoolCache* = nullptr) const override;
		virtual FImageDesc GetImageDesc(bool, FGetImageDescContext*) const override;
		virtual bool IsColorConstant(FVector4f& OutColor) const override;
	};
}

