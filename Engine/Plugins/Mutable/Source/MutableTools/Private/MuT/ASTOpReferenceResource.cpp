// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpReferenceResource.h"

#include "MuT/ASTOpConstantResource.h"
#include "Containers/Array.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{

	void ASTOpReferenceResource::ForEachChild(const TFunctionRef<void(ASTChild&)>)
	{
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpReferenceResource::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpReferenceResource* other = static_cast<const ASTOpReferenceResource*>(&otherUntyped);
			return Type == other->Type && ID == other->ID && bForceLoad == other->bForceLoad && ImageDesc == other->ImageDesc;
		}
		return false;
	}


	//-------------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> ASTOpReferenceResource::Clone(MapChildFuncRef) const
	{
		Ptr<ASTOpReferenceResource> n = new ASTOpReferenceResource();
		n->Type = Type;
		n->ID = ID;
		n->bForceLoad = bForceLoad;
		n->ImageDesc = ImageDesc;
		return n;
	}


	//-------------------------------------------------------------------------------------------------
	uint32 ASTOpReferenceResource::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(ID));
		
		return Result;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpReferenceResource::Link(FProgram& Program, FLinkerOptions* Options)
	{
		if (!LinkedAddress)
		{
			FOperation::ResourceReferenceArgs Args;
			FMemory::Memzero(Args);
			Args.ID = ID;
			Args.ForceLoad = bForceLoad ? 1 : 0;
			Args.ImageDesc = ImageDesc;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(Type, Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, Type);
			AppendCode(Program.ByteCode, Args);
		}
	}


	//-------------------------------------------------------------------------------------------------
	FImageDesc ASTOpReferenceResource::GetImageDesc(bool, class FGetImageDescContext*) const
	{
		return ImageDesc;
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpReferenceResource::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache*)
	{
		switch (Type)
		{

		case EOpType::IM_REFERENCE:
		{
			*pBlockX = 0;
			*pBlockY = 0;
			break;
		}

		default:
			checkf(false, TEXT("Instruction not supported"));
		}
	}


	//-------------------------------------------------------------------------------------------------
	void ASTOpReferenceResource::GetLayoutBlockSize(int32* pBlockX, int32* pBlockY)
	{
		switch (Type)
		{

		case EOpType::IM_REFERENCE:
		{
			// We didn't find any layout.
			*pBlockX = 0;
			*pBlockY = 0;
			break;
		}

		default:
			checkf(false, TEXT("Instruction not supported"));
		}
	}


	//-------------------------------------------------------------------------------------------------
	bool ASTOpReferenceResource::GetNonBlackRect(FImageRect&) const
	{
		return false;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpReferenceResource::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> pRes = new ImageSizeExpression;
		pRes->type = ImageSizeExpression::ISET_UNKNOWN;

		return pRes;
	}


	FSourceDataDescriptor ASTOpReferenceResource::GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const
	{
		return SourceDataDescriptor;
	}


}
