// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;
	template <class SCALAR> class vec4;


	//---------------------------------------------------------------------------------------------
	//! A reference to an engine image (or other resources in the future)
	//---------------------------------------------------------------------------------------------
	class ASTOpReferenceResource final : public ASTOp
	{
	public:

		//! Type of switch
		EOpType Type = EOpType::NONE;
		
		/** */
		bool bForceLoad = false;

		/** Externally provided ID to identify the resource. */
		PASSTHROUGH_ID ID = PASSTHROUGH_ID_INVALID;

		/** */
		FImageDesc ImageDesc;

		/** Source data descriptor. */
		FSourceDataDescriptor SourceDataDescriptor;

	public:

		// Own interface

		// ASTOp interface
		virtual EOpType GetOpType() const override { return Type; }
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual uint32 Hash() const override;
		virtual void Link(FProgram& program, FLinkerOptions*) override;
		virtual FImageDesc GetImageDesc(bool, class FGetImageDescContext*) const override;
		virtual void GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache) override;
		virtual void GetLayoutBlockSize(int32* pBlockX, int32* pBlockY) override;
		virtual bool GetNonBlackRect(FImageRect& maskUsage) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};


}

