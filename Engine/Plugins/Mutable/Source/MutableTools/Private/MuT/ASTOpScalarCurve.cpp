// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpScalarCurve.h"

#include "HAL/PlatformMath.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{


	ASTOpScalarCurve::ASTOpScalarCurve()
		: time(this)
	{
	}


	ASTOpScalarCurve::~ASTOpScalarCurve()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	void ASTOpScalarCurve::ForEachChild(const TFunctionRef<void(ASTChild&)> f)
	{
		f(time);
	}


	uint32 ASTOpScalarCurve::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Curve));
		
		return Result;
	}


	bool ASTOpScalarCurve::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpScalarCurve* other = static_cast<const ASTOpScalarCurve*>(&otherUntyped);
			return time == other->time && Curve == other->Curve;
		}
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpScalarCurve::Clone(MapChildFuncRef mapChild) const
	{
		Ptr<ASTOpScalarCurve> n = new ASTOpScalarCurve();
		n->Curve = Curve;
		n->time = mapChild(time.child());
		return n;
	}


	void ASTOpScalarCurve::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::ScalarCurveArgs Args;
			FMemory::Memzero(Args);

			Args.time = time ? time->LinkedAddress : 0;
			Args.curve = Program.AddConstant(Curve);

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}

}

