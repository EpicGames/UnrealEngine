// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"
#include "Curves/RichCurve.h"
#include "HAL/Platform.h"


namespace UE::Mutable::Private
{
struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpScalarCurve final : public ASTOp
	{
	public:

		ASTChild time;

		//!
		FRichCurve Curve;

	public:

		ASTOpScalarCurve();
		ASTOpScalarCurve(const ASTOpScalarCurve&) = delete;
		~ASTOpScalarCurve() override;

		EOpType GetOpType() const override { return EOpType::SC_CURVE; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& otherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		void Link(FProgram& program, FLinkerOptions* Options) override;
	};


}

