// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshClipWithMesh.h"
#include "MuT/ASTOpMeshAddMetadata.h"

#include "MuR/Model.h"
#include "MuR/RefCounted.h"

namespace UE::Mutable::Private
{

	ASTOpSkeletalMeshClipWithSkeletalMesh::ASTOpSkeletalMeshClipWithSkeletalMesh()
		: Source(this)
		, Clip(this)
	{
	}


	ASTOpSkeletalMeshClipWithSkeletalMesh::~ASTOpSkeletalMeshClipWithSkeletalMesh()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpSkeletalMeshClipWithSkeletalMesh::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshClipWithSkeletalMesh* Other = static_cast<const ASTOpSkeletalMeshClipWithSkeletalMesh*>(&OtherUntyped);
			return Source == Other->Source &&
				Clip == Other->Clip &&
				FaceCullStrategy == Other->FaceCullStrategy;
		}
		return false;
	}


	uint32 ASTOpSkeletalMeshClipWithSkeletalMesh::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, GetTypeHash(Clip));
		Result = HashCombineFast(Result, GetTypeHash(FaceCullStrategy));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpSkeletalMeshClipWithSkeletalMesh::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshClipWithSkeletalMesh> New = new ASTOpSkeletalMeshClipWithSkeletalMesh();
		New->Source = MapChild(Source.child());
		New->Clip = MapChild(Clip.child());
		New->FaceCullStrategy = FaceCullStrategy;
		return New;
	}


	void ASTOpSkeletalMeshClipWithSkeletalMesh::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
		Func(Clip);
	}


	void ASTOpSkeletalMeshClipWithSkeletalMesh::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::FSkeletalMeshClipMeshWithMeshArgs Args;
			FMemory::Memzero(Args);

			if (Source)
			{
				Args.Source = Source->LinkedAddress;
			}

			if (Clip)
			{
				Args.Clip = Clip->LinkedAddress;
			}

			Args.FaceCullStrategy = FaceCullStrategy;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}

	}

	FSourceDataDescriptor ASTOpSkeletalMeshClipWithSkeletalMesh::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
