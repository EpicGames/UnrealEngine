// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Types.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/** */
	class ASTOpSkeletalMeshClipWithSkeletalMesh final : public ASTOp
	{
	public:

		ASTChild Source;
		ASTChild Clip;

		EFaceCullStrategy FaceCullStrategy = EFaceCullStrategy::AllVerticesCulled;
		
	public:

		ASTOpSkeletalMeshClipWithSkeletalMesh();
		ASTOpSkeletalMeshClipWithSkeletalMesh(const ASTOpSkeletalMeshClipWithSkeletalMesh&) = delete;
		~ASTOpSkeletalMeshClipWithSkeletalMesh();

		// ASTOp interface
		virtual EOpType GetOpType() const override { return EOpType::SK_CLIPMESHWITHMESH; }
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};


}

