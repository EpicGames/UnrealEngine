// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshConvert.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "MuR/Model.h"


namespace UE::Mutable::Private
{
	ASTOpSkeletalMeshConvert::ASTOpSkeletalMeshConvert() :
		SkeletalMeshObject(this)
	{
	}

	ASTOpSkeletalMeshConvert::~ASTOpSkeletalMeshConvert()
	{
		// Explicit call needed to avoid recursive destruction
		RemoveChildren();
	}


	void ASTOpSkeletalMeshConvert::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(SkeletalMeshObject);
	}


	bool ASTOpSkeletalMeshConvert::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshConvert* Other = static_cast<const ASTOpSkeletalMeshConvert*>(&OtherUntyped);

			return
				ConversionFlags == Other->ConversionFlags &&
				MeshID == Other->MeshID &&
				SkeletalMeshObject == Other->SkeletalMeshObject;
		}
		
		return false;
	}


	Ptr<ASTOp> ASTOpSkeletalMeshConvert::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshConvert> New = new ASTOpSkeletalMeshConvert();

		New->ConversionFlags = ConversionFlags;
		New->MeshID = MeshID;
		New->SkeletalMeshObject = ASTChild(New, MapChild(SkeletalMeshObject.child()));

		return New;
	}

	uint32 ASTOpSkeletalMeshConvert::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(ConversionFlags));
		Result = HashCombineFast(Result, GetTypeHash(MeshID));
		Result = HashCombineFast(Result, GetTypeHash(SkeletalMeshObject));

		return Result;
	}


	void ASTOpSkeletalMeshConvert::Assert()
	{
		ASTOp::Assert();
	}

	void ASTOpSkeletalMeshConvert::Link(FProgram& Program, FLinkerOptions*)
	{
		if (!LinkedAddress)
		{
			FOperation::FSkeletalMeshConvertArgs Args;
			FMemory::Memzero(Args);

			check(SkeletalMeshObject->LinkedAddress);
			Args.SkeletalMeshObject = SkeletalMeshObject->LinkedAddress;

			Args.ConversionFlags = ConversionFlags;
			Args.MeshID = MeshID;

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}
}
