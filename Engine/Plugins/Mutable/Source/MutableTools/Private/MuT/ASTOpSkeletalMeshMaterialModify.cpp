// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshMaterialModify.h"

#include "MuT/ASTOpBoolEqualIntConst.h"
#include "HAL/PlatformMath.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Model.h"
#include "MuR/Parameters.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/CodeOptimiser.h"


namespace UE::Mutable::Private
{
	ASTOpSkeletalMeshMaterialModify::ASTOpSkeletalMeshMaterialModify()
		: SkeletalMesh(this)
		, NewMaterial(this)
	{
	}

	ASTOpSkeletalMeshMaterialModify::~ASTOpSkeletalMeshMaterialModify()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	void ASTOpSkeletalMeshMaterialModify::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(SkeletalMesh);
		Func(NewMaterial);
	}


	bool ASTOpSkeletalMeshMaterialModify::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshMaterialModify* Other = static_cast<const ASTOpSkeletalMeshMaterialModify*>(&OtherUntyped);

			return GetOpType() == Other->GetOpType() &&
				MaterialSlotNameToModify == Other->MaterialSlotNameToModify &&
				SkeletalMesh == Other->SkeletalMesh &&
				NewMaterial == Other->NewMaterial;
		}

		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSkeletalMeshMaterialModify::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshMaterialModify> Result = new ASTOpSkeletalMeshMaterialModify();

		Result->MaterialSlotNameToModify = MaterialSlotNameToModify;
		Result->SkeletalMesh = MapChild(SkeletalMesh.child());
		Result->NewMaterial = MapChild(NewMaterial.child());

		return Result;
	}


	EOpType ASTOpSkeletalMeshMaterialModify::GetOpType() const
	{ 
		return EOpType::SK_MATERIALMODIFY;
	}

	uint32 ASTOpSkeletalMeshMaterialModify::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(MaterialSlotNameToModify));
		Result = HashCombineFast(Result, GetTypeHash(SkeletalMesh));
		Result = HashCombineFast(Result, GetTypeHash(NewMaterial));
		
		return Result;
	}


	void ASTOpSkeletalMeshMaterialModify::Assert()
	{
		ASTOp::Assert();
	}


	void ASTOpSkeletalMeshMaterialModify::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::FSkeletalMeshMaterialModifyArgs Args;
			FMemory::Memzero(Args);

			Args.MaterialSlotName = Program.AddConstant(MaterialSlotNameToModify); 

			if (SkeletalMesh)
			{
				Args.SkeletalMesh = SkeletalMesh->LinkedAddress;
			}

			if (NewMaterial)
			{
				Args.NewMaterial = NewMaterial->LinkedAddress;
			}

			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

	FSourceDataDescriptor ASTOpSkeletalMeshMaterialModify::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (SkeletalMesh)
		{
			return SkeletalMesh->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}

