// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/AST.h"

namespace UE::Mutable::Private
{
	class ASTOpSkeletalMeshMaterialModify final : public ASTOp
	{
	public:

		FName MaterialSlotNameToModify;
		
		/** SkeletalMesh to Modify */
		ASTChild SkeletalMesh;

		/** New material to slot in MaterialSlotNameToModify */
		ASTChild NewMaterial;


	public:

		ASTOpSkeletalMeshMaterialModify();
		ASTOpSkeletalMeshMaterialModify(const ASTOpSkeletalMeshMaterialModify&) = delete;
		~ASTOpSkeletalMeshMaterialModify() override;

		// ASTOp interface.
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp&) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef ) const override;
		virtual void Assert() override;
		virtual void Link(FProgram&, FLinkerOptions*) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	
	};
}
