// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshMerge.h"

#include "CodeOptimiser.h"
#include "MuR/Model.h"
#include "MuR/RefCounted.h"


namespace UE::Mutable::Private
{
	ASTOpSkeletalMeshMerge::ASTOpSkeletalMeshMerge()
		: BaseMesh(this)
		, AddedMesh(this)
	{
	}
	

	ASTOpSkeletalMeshMerge::~ASTOpSkeletalMeshMerge()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	
	EOpType ASTOpSkeletalMeshMerge::GetOpType() const
	{
		return EOpType::SK_MERGE;
	}


	bool ASTOpSkeletalMeshMerge::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshMerge* Other = static_cast<const ASTOpSkeletalMeshMerge*>(&OtherUntyped);
			return BaseMesh == Other->BaseMesh &&
				AddedMesh == Other->AddedMesh;
		}
		return false;
	}


	uint32 ASTOpSkeletalMeshMerge::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(BaseMesh));
		Result = HashCombineFast(Result, GetTypeHash(AddedMesh));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpSkeletalMeshMerge::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshMerge> New = new ASTOpSkeletalMeshMerge();
		New->BaseMesh = MapChild(BaseMesh.child());
		New->AddedMesh = MapChild(AddedMesh.child());
		return New;
	}


	void ASTOpSkeletalMeshMerge::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(BaseMesh);
		Func(AddedMesh);
	}


	void ASTOpSkeletalMeshMerge::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}

		FOperation::FSkeletalMeshMergeArgs Args;
		FMemory::Memzero(Args);
		
		if (BaseMesh)
		{
			Args.BaseMesh = BaseMesh->LinkedAddress;
		}

		if (AddedMesh)
		{
			Args.AddedMesh = AddedMesh->LinkedAddress;
		}
		
		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), (FOperation::ADDRESS)Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType()); 
		AppendCode(Program.ByteCode, Args);
	}


	Ptr<ASTOp> ASTOpSkeletalMeshMerge::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		if (!AddedMesh)
		{
			return BaseMesh.child();
		}

		if (!BaseMesh)
		{
			return AddedMesh.child();
		}

		return nullptr;
	}
}
