// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshMorph.h"

namespace UE::Mutable::Private
{

	ASTOpSkeletalMeshMorph::ASTOpSkeletalMeshMorph()
		: Factor(this)
		, Base(this)
	{
	}


	ASTOpSkeletalMeshMorph::~ASTOpSkeletalMeshMorph()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpSkeletalMeshMorph::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshMorph* Other = static_cast<const ASTOpSkeletalMeshMorph*>(&OtherUntyped);
			return MorphName == Other->MorphName && Factor == Other->Factor && Base == Other->Base;
		}

		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSkeletalMeshMorph::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshMorph> New = new ASTOpSkeletalMeshMorph();
		New->MorphName = MorphName;
		New->Factor = MapChild(Factor.child());
		New->Base = MapChild(Base.child());

		return New;
	}

	void ASTOpSkeletalMeshMorph::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Factor);
		Func(Base);
	}


	uint32 ASTOpSkeletalMeshMorph::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(MorphName));
		Result = HashCombineFast(Result, GetTypeHash(Factor));
		Result = HashCombineFast(Result, GetTypeHash(Base));
		
		return Result;
	}

	void ASTOpSkeletalMeshMorph::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::FSkeletalMeshMorphArgs Args;
			FMemory::Memzero(Args);

			Args.MorphName = Program.AddConstant(MorphName.ToString());
			Args.Base = Base ? Base->LinkedAddress : 0;
			Args.Factor = Factor ? Factor->LinkedAddress : 0;

			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);
		}
	}

	FSourceDataDescriptor ASTOpSkeletalMeshMorph::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
