// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/**
	 * Apply a morph to a SkeletalMesh.
	 */
	class ASTOpSkeletalMeshMorph final : public ASTOp
	{
	public:
		/** Morph name to apply. */
		FName MorphName;
		
		/** Factor selecting what morphs to apply and with what weight. */
		ASTChild Factor;

		/** Base mesh to morph. */
		ASTChild Base;

	public:

		ASTOpSkeletalMeshMorph();
		ASTOpSkeletalMeshMorph(const ASTOpSkeletalMeshMorph&) = delete;
		~ASTOpSkeletalMeshMorph() override;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::SK_MORPH; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& OtherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		void Link(FProgram& Program, FLinkerOptions*) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}

