// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshNew.h"

#include "CodeOptimiser.h"
#include "MuR/Model.h"


namespace UE::Mutable::Private
{
	ASTOpSkeletalMeshNew::ASTOpSkeletalMeshNew()
	{
	}
	

	ASTOpSkeletalMeshNew::~ASTOpSkeletalMeshNew()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	
	EOpType ASTOpSkeletalMeshNew::GetOpType() const
	{
		return EOpType::SK_NEW;
	}


	bool ASTOpSkeletalMeshNew::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshNew* Other = static_cast<const ASTOpSkeletalMeshNew*>(&OtherUntyped);
			return MaterialSlotMaterials == Other->MaterialSlotMaterials &&
				MaterialSlotNames == Other->MaterialSlotNames &&
				MaterialSlotIds == Other->MaterialSlotIds &&
				LODs == Other->LODs;
		}
		return false;
	}


	uint32 ASTOpSkeletalMeshNew::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(MaterialSlotMaterials));
		Result = HashCombineFast(Result, GetTypeHash(MaterialSlotNames));
		Result = HashCombineFast(Result, GetTypeHash(MaterialSlotIds));
		Result = HashCombineFast(Result, GetTypeHash(LODs));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpSkeletalMeshNew::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshNew> New = new ASTOpSkeletalMeshNew();
		
		for (const ASTChild& MaterialSlot : MaterialSlotMaterials)
		{
			New->MaterialSlotMaterials.Emplace(New, MapChild(MaterialSlot.child()));
		}
		
		New->MaterialSlotNames = MaterialSlotNames;
		New->MaterialSlotIds = MaterialSlotIds;
		
		for (const ASTChild& LOD : LODs)
		{
			New->LODs.Emplace(New, MapChild(LOD.child()));
		}
		
		return New;
	}


	void ASTOpSkeletalMeshNew::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		for (ASTChild& Material : MaterialSlotMaterials)
		{
			Func(Material);
		}
		
		for (ASTChild& LOD : LODs)
		{
			Func(LOD);
		}
	}


	void ASTOpSkeletalMeshNew::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}

		FOperation::FSkeletalMeshNewArgs Args;
		FMemory::Memzero(Args);
		
		TArray<FOperation::ADDRESS> MaterialAddresses;
		MaterialAddresses.Reserve(MaterialSlotMaterials.Num());
		
		for (const ASTChild& Material : MaterialSlotMaterials)
		{
			if (!Material)
			{
				continue;
			}
			
			MaterialAddresses.Add(Material->LinkedAddress);
			
			// Find out relevant parameters. \todo: this may be optimised by reusing partial
			// values in a LINK_CONTEXT or similar
			SubtreeRelevantParametersVisitorAST Visitor;
			Visitor.Run(Material.child());
			
			TArray<uint16> Params;
			for (const FString& ParameterName : Visitor.Parameters)
			{
				for (int32 Index = 0; Index < Program.Parameters.Num(); ++Index)
				{
					const FParameterDesc& Param = Program.Parameters[Index];
					if (Param.Name == ParameterName)
					{
						Params.Add(uint16(Index));
						break;
					}
				}
			}

			Params.Sort();

			int32 It = Program.ParameterLists.Find(Params);

			if (It != INDEX_NONE)
			{
				Program.RelevantParameterList.Add(Material->LinkedAddress, It);
			}
			else
			{
				Program.RelevantParameterList.Add(Material->LinkedAddress, Program.ParameterLists.Num());
				Program.ParameterLists.Add(Params);
			}
		} 
		
		Args.MaterialSlotMaterials = Program.AddConstant(MaterialAddresses);
		Args.MaterialSlotNames = Program.AddConstant(MaterialSlotNames);
		Args.MaterialSlotIds = Program.AddConstant(MaterialSlotIds);
			
		TArray<FOperation::ADDRESS> LODAddresses;
		LODAddresses.SetNum(LODs.Num());
		for (int32 LODIndex = 0; LODIndex < LODs.Num(); ++LODIndex)
		{
			const ASTChild& LOD = LODs[LODIndex];
			if (LOD)
			{
				LODAddresses[LODIndex] = LOD->LinkedAddress;
			}
		}
		
		Args.LODs = Program.AddConstant(LODAddresses);
		
		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), (FOperation::ADDRESS)Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType()); 
		AppendCode(Program.ByteCode, Args);
	}
}
