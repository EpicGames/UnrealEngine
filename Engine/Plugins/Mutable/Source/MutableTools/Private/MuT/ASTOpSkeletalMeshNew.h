// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpSkeletalMeshNew final : public ASTOp
	{
	public:
		/** Key is the Material Slot. */
		TArray<ASTChild> MaterialSlotMaterials;

		/** Key is the Material Slot. */
		TArray<FName> MaterialSlotNames;
		
		/** Key is the Material Slot. */
		TArray<uint32> MaterialSlotIds;
		
		/** Key is the LOD Index. */
		TArray<ASTChild> LODs;
		
		ASTOpSkeletalMeshNew();
		ASTOpSkeletalMeshNew(const ASTOpSkeletalMeshNew&) = delete;
		virtual ~ASTOpSkeletalMeshNew() override;

		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void Link(FProgram& Program, FLinkerOptions* Options) override;
	};
}

