// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshObjectConvert.h"

#include "CodeOptimiser.h"
#include "MuT/LODInfo.h"
#include "MuR/Model.h"


namespace UE::Mutable::Private
{
	ASTOpSkeletalMeshObjectConvert::ASTOpSkeletalMeshObjectConvert() :
		SkeletalMesh(this)
	{
	}
	

	ASTOpSkeletalMeshObjectConvert::~ASTOpSkeletalMeshObjectConvert()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	
	EOpType ASTOpSkeletalMeshObjectConvert::GetOpType() const
	{
		return EOpType::SKO_CONVERT;
	}


	bool ASTOpSkeletalMeshObjectConvert::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshObjectConvert* Other = static_cast<const ASTOpSkeletalMeshObjectConvert*>(&OtherUntyped);
			return SkeletalMesh == Other->SkeletalMesh &&
				Name == Other->Name &&
				NumLODs == Other->NumLODs &&
				FirstLODAvailable == Other->FirstLODAvailable &&
				FirstLODResident == Other->FirstLODResident &&
				MinLODs == Other->MinLODs &&
				MinQualityLevelLODs == Other->MinQualityLevelLODs &&
				LODInfos == Other->LODInfos;
		}
		return false;
	}


	uint32 ASTOpSkeletalMeshObjectConvert::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(SkeletalMesh));
		Result = HashCombineFast(Result, GetTypeHash(Name));
		Result = HashCombineFast(Result, GetTypeHash(NumLODs));
		Result = HashCombineFast(Result, GetTypeHash(FirstLODAvailable));
		Result = HashCombineFast(Result, GetTypeHash(FirstLODResident));
		Result = HashCombineFast(Result, GetTypeHash(MinLODs));
		Result = HashCombineFast(Result, GetTypeHash(MinQualityLevelLODs));
		Result = HashCombineFast(Result, GetTypeHash(LODInfos));
		
		return Result;
	}


	Ptr<ASTOp> ASTOpSkeletalMeshObjectConvert::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshObjectConvert> New = new ASTOpSkeletalMeshObjectConvert();
		
		New->SkeletalMesh = MapChild(SkeletalMesh.child());
		New->Name = Name;
		New->NumLODs = NumLODs;
		New->FirstLODAvailable = FirstLODAvailable;
		New->FirstLODResident = FirstLODResident;
		New->MinLODs = MinLODs;
		New->MinQualityLevelLODs = MinQualityLevelLODs;
		New->LODInfos = LODInfos;
		
		return New;
	}


	void ASTOpSkeletalMeshObjectConvert::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(SkeletalMesh);
	}


	void ASTOpSkeletalMeshObjectConvert::Link(FProgram& Program, FLinkerOptions*)
	{
		if (LinkedAddress)
		{
			return;
		}

		FOperation::FSkeletalMeshObjectConvertArgs Args;
		FMemory::Memzero(Args);
		
		if (SkeletalMesh)
		{
			Args.SkeletalMesh = SkeletalMesh->LinkedAddress;
		}
		
		Args.Name = Program.AddConstant(Name);
		Args.NumLODs = NumLODs;
		Args.FirstLODAvailable = FirstLODAvailable;
		Args.FirstLODResident = FirstLODResident;
		
		{
			TArray<FName> Keys;
			Keys.Reserve(MinLODs.PerPlatform.Num());

			TArray<int32> Values;
			Values.Reserve(MinLODs.PerPlatform.Num());

			for (const TTuple<FName, int32>& Pair : MinLODs.PerPlatform)
			{
				Keys.Add(Pair.Key);
				Values.Add(Pair.Value);
			}
		
			Args.MinLODsDefault = MinLODs.Default;
			Args.MinLODsKeys = Program.AddConstant(Keys);
			Args.MinLODsValues = Program.AddConstant(Values);
		}
		
		{
			TArray<int32> Keys;
			Keys.Reserve(MinQualityLevelLODs.PerQuality.Num());

			TArray<int32> Values;
			Values.Reserve(MinQualityLevelLODs.PerQuality.Num());

			for (const TTuple<int32, int32>& Pair : MinQualityLevelLODs.PerQuality)
			{
				Keys.Add(Pair.Key);
				Values.Add(Pair.Value);
			}
		
			Args.MinQualityLevelLODsDefault = MinQualityLevelLODs.Default;
			Args.MinQualityLevelLODsKeys = Program.AddConstant(Keys);
			Args.MinQualityLevelLODsValues = Program.AddConstant(Values);
		}
		
		{
			TArray<float> ScreenSize;
			ScreenSize.Reserve(LODInfos.Num());

			TArray<float> LODHysteresis;
			ScreenSize.Reserve(LODHysteresis.Num());

			TArray<bool> bSupportUniformlyDistributedSampling;
			ScreenSize.Reserve(bSupportUniformlyDistributedSampling.Num());

			TArray<bool> bAllowCPUAccess;
			ScreenSize.Reserve(bAllowCPUAccess.Num());
			
			for (int32 Index = 0; Index < LODInfos.Num(); ++Index)
			{
				ScreenSize.Add(LODInfos[Index].ScreenSize);
				LODHysteresis.Add(LODInfos[Index].LODHysteresis);
				bSupportUniformlyDistributedSampling.Add(LODInfos[Index].bSupportUniformlyDistributedSampling);
				bAllowCPUAccess.Add(LODInfos[Index].bAllowCPUAccess);
			}
			
			Args.ScreenSize = Program.AddConstant(ScreenSize);
			Args.LODHysteresis = Program.AddConstant(LODHysteresis);
			Args.bSupportUniformlyDistributedSampling = Program.AddConstant(bSupportUniformlyDistributedSampling);
			Args.bAllowCPUAccess = Program.AddConstant(bAllowCPUAccess);
		}
		
		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), (FOperation::ADDRESS)Program.ByteCode.Num());
		
		AppendCode(Program.ByteCode, GetOpType()); 
		AppendCode(Program.ByteCode, Args);
	}
}
