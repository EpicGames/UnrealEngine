// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"
#include "MuT/LODInfo.h"


namespace UE::Mutable::Private
{
	struct FProgram;
	
	
	class ASTOpSkeletalMeshObjectConvert final : public ASTOp
	{
	public:
		ASTChild SkeletalMesh;
		
		FName Name;
		uint8 NumLODs = 0;
		uint8 FirstLODAvailable = 0;
		uint8 FirstLODResident = 0;
		
		FPerPlatformInt MinLODs;
		FPerQualityLevelInt MinQualityLevelLODs;
		
		TArray<FLODInfo> LODInfos;
		
		ASTOpSkeletalMeshObjectConvert();
		ASTOpSkeletalMeshObjectConvert(const ASTOpSkeletalMeshObjectConvert&) = delete;
		virtual ~ASTOpSkeletalMeshObjectConvert() override;

		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& OtherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void Link(FProgram& Program, FLinkerOptions* Options) override;
	};
}

