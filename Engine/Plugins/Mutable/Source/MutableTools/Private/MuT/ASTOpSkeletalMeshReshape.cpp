// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshReshape.h"

namespace UE::Mutable::Private
{

	ASTOpSkeletalMeshReshape::ASTOpSkeletalMeshReshape()
		: Base(this)
		, BaseShape(this)
		, TargetShape(this)
	{
	}


	ASTOpSkeletalMeshReshape::~ASTOpSkeletalMeshReshape()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpSkeletalMeshReshape::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (OtherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshReshape* Other = static_cast<const ASTOpSkeletalMeshReshape*>(&OtherUntyped);

			const bool bSameFlags =
				bRecomputeNormals == Other->bRecomputeNormals &&
				bReshapeSkeleton == Other->bReshapeSkeleton	&&
				bReshapePhysicsVolumes == Other->bReshapePhysicsVolumes &&
				bReshapeVertices == Other->bReshapeVertices &&
				bApplyLaplacian == Other->bApplyLaplacian &&
				bReshapeSkeletonInvertSelection == Other->bReshapeSkeletonInvertSelection &&
				bReshapePhysicsVolumesInvertSelection == Other->bReshapePhysicsVolumesInvertSelection;

			return bSameFlags &&
				Base == Other->Base &&
				BaseShape == Other->BaseShape &&
				TargetShape == Other->TargetShape &&
				BonesToDeform == Other->BonesToDeform &&
				PhysicsToDeform == Other->PhysicsToDeform &&
				BindingMethod == Other->BindingMethod &&
				RChannelUsage == Other->RChannelUsage && 
				GChannelUsage == Other->GChannelUsage && 
				BChannelUsage == Other->BChannelUsage && 
				AChannelUsage == Other->AChannelUsage;
		}

		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSkeletalMeshReshape::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshReshape> New = new ASTOpSkeletalMeshReshape();
		New->Base = MapChild(Base.child());
		New->BaseShape = MapChild(BaseShape.child());
		New->TargetShape = MapChild(TargetShape.child());

		New->BonesToDeform = BonesToDeform;
		New->PhysicsToDeform = PhysicsToDeform;

		New->BindingMethod = BindingMethod;

		New->bRecomputeNormals = bRecomputeNormals;
		New->bReshapeSkeleton = bReshapeSkeleton;
		New->bReshapePhysicsVolumes = bReshapePhysicsVolumes;
		New->bReshapeVertices = bReshapeVertices;
		New->bApplyLaplacian = bApplyLaplacian;
		New->bReshapeSkeletonInvertSelection = bReshapeSkeletonInvertSelection;
		New->bReshapePhysicsVolumesInvertSelection = bReshapePhysicsVolumesInvertSelection;

		New->RChannelUsage = RChannelUsage;
		New->GChannelUsage = GChannelUsage;
		New->BChannelUsage = BChannelUsage;
		New->AChannelUsage = AChannelUsage;

		return New;
	}

	void ASTOpSkeletalMeshReshape::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Base);
		Func(BaseShape);
		Func(TargetShape);
	}


	uint32 ASTOpSkeletalMeshReshape::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		Result = HashCombineFast(Result, GetTypeHash(Base));
		Result = HashCombineFast(Result, GetTypeHash(BaseShape));
		Result = HashCombineFast(Result, GetTypeHash(TargetShape));
		Result = HashCombineFast(Result, GetTypeHash(bRecomputeNormals));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeSkeleton));
		Result = HashCombineFast(Result, GetTypeHash(bReshapePhysicsVolumes));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeVertices));
		Result = HashCombineFast(Result, GetTypeHash(bApplyLaplacian));
		Result = HashCombineFast(Result, GetTypeHash(bReshapeSkeletonInvertSelection));
		Result = HashCombineFast(Result, GetTypeHash(bReshapePhysicsVolumesInvertSelection));
		Result = HashCombineFast(Result, GetTypeHash(BindingMethod));
		Result = HashCombineFast(Result, GetTypeHash(RChannelUsage));
		Result = HashCombineFast(Result, GetTypeHash(GChannelUsage));
		Result = HashCombineFast(Result, GetTypeHash(BChannelUsage));
		Result = HashCombineFast(Result, GetTypeHash(AChannelUsage));

		return Result;
	}

	void ASTOpSkeletalMeshReshape::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			FOperation::FSkeletalMeshReshapeArgs Args;
			FMemory::Memzero(Args);

			constexpr EMeshBindShapeFlags NoFlags = EMeshBindShapeFlags::None;
			EMeshBindShapeFlags BindFlags = NoFlags;
			EnumAddFlags(BindFlags, bRecomputeNormals ? EMeshBindShapeFlags::RecomputeNormals : NoFlags);
			EnumAddFlags(BindFlags, bReshapeSkeleton ? EMeshBindShapeFlags::ReshapeSkeleton : NoFlags);
			EnumAddFlags(BindFlags, bReshapePhysicsVolumes ? EMeshBindShapeFlags::ReshapePhysicsVolumes : NoFlags);
			EnumAddFlags(BindFlags, bReshapeVertices ? EMeshBindShapeFlags::ReshapeVertices : NoFlags);
			EnumAddFlags(BindFlags, bApplyLaplacian ? EMeshBindShapeFlags::ApplyLaplacian : NoFlags);
			EnumAddFlags(BindFlags, bReshapeSkeletonInvertSelection ? EMeshBindShapeFlags::ReshapeSkeletonInvertSelection : NoFlags);
			EnumAddFlags(BindFlags, bReshapePhysicsVolumesInvertSelection ? EMeshBindShapeFlags::ReshapePhysicsVolumesInvertSelection : NoFlags);

			{
				auto ConvertColorUsage = [](EVertexColorUsage Usage)
				{
					switch (Usage)
					{
					case EVertexColorUsage::None:			   return EMeshBindColorChannelUsage::None;
					case EVertexColorUsage::ReshapeClusterId:  return EMeshBindColorChannelUsage::ClusterId;
					case EVertexColorUsage::ReshapeMaskWeight: return EMeshBindColorChannelUsage::MaskWeight;
					default: check(false); return EMeshBindColorChannelUsage::None;
					};
				};
	
				const FMeshBindColorChannelUsages ColorUsages = {
					ConvertColorUsage(RChannelUsage),
					ConvertColorUsage(GChannelUsage),
					ConvertColorUsage(BChannelUsage),
					ConvertColorUsage(AChannelUsage) };

				FMemory::Memcpy(&Args.ColorUsage, &ColorUsages, sizeof(Args.ColorUsage));
				static_assert(sizeof(Args.ColorUsage) == sizeof(ColorUsages));
			}

			Args.Flags = static_cast<uint32>(BindFlags);

			Args.BindingMethod = BindingMethod;

			Args.Base = Base ? Base->LinkedAddress : 0;
			Args.BaseShape = BaseShape ? BaseShape->LinkedAddress : 0;
			Args.TargetShape = TargetShape ? TargetShape->LinkedAddress : 0;

			Args.NumBones = BonesToDeform.Num();
			Args.NumPhysics = PhysicsToDeform.Num();

			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			AppendCode(Program.ByteCode, GetOpType());
			AppendCode(Program.ByteCode, Args);

			for (FName Bone : BonesToDeform)
			{
				AppendCode(Program.ByteCode, Program.AddConstant(Bone));
			}

			for (FName PhysicsBone : PhysicsToDeform)
			{
				AppendCode(Program.ByteCode, Program.AddConstant(PhysicsBone));
			}

		}
	}

	FSourceDataDescriptor ASTOpSkeletalMeshReshape::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Base)
		{
			return Base->GetSourceDataDescriptor(Context);
		}

		return {};
	}
}
