// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	/**
	 * Apply a reshape to a SkeletalMesh.
	 */
	class ASTOpSkeletalMeshReshape final : public ASTOp
	{
	public:	
		ASTChild Base;
		ASTChild BaseShape;
		ASTChild TargetShape;

		TArray<FName> BonesToDeform;
		TArray<FName> PhysicsToDeform;

		uint32 BindingMethod = 0;
		
		uint32 bRecomputeNormals	  					: 1 = false;
		uint32 bReshapeSkeleton	      					: 1 = false;
		uint32 bReshapePhysicsVolumes 					: 1 = false;
		uint32 bReshapeVertices       					: 1 = false;
		uint32 bApplyLaplacian        					: 1 = false;
		uint32 bReshapeSkeletonInvertSelection			: 1 = false;
		uint32 bReshapePhysicsVolumesInvertSelection	: 1 = false;

		EVertexColorUsage RChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage GChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage BChannelUsage = EVertexColorUsage::None;
		EVertexColorUsage AChannelUsage = EVertexColorUsage::None;

	public:

		ASTOpSkeletalMeshReshape();
		ASTOpSkeletalMeshReshape(const ASTOpSkeletalMeshReshape&) = delete;
		~ASTOpSkeletalMeshReshape() override;

		// ASTOp interface
		EOpType GetOpType() const override { return EOpType::SK_RESHAPE; }
		uint32 Hash() const override;
		void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		bool IsEqual(const ASTOp& OtherUntyped) const override;
		Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		void Link(FProgram& Program, FLinkerOptions*) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};

}

