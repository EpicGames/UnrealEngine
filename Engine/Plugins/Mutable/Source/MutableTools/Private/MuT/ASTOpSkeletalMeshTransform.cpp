// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshTransform.h"

#include "MuR/Model.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{
	ASTOpSkeletalMeshTransform::ASTOpSkeletalMeshTransform()
		: Source(this)
	{
	}


	ASTOpSkeletalMeshTransform::~ASTOpSkeletalMeshTransform()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}

	
	EOpType ASTOpSkeletalMeshTransform::GetOpType() const
	{
		return EOpType::SK_TRANSFORM;
	}


	bool ASTOpSkeletalMeshTransform::IsEqual(const ASTOp& otherUntyped) const
	{
		if (otherUntyped.GetOpType() == GetOpType())
		{
			const ASTOpSkeletalMeshTransform* Other = static_cast<const ASTOpSkeletalMeshTransform*>(&otherUntyped);
			return Source == Other->Source &&
				Matrix == Other->Matrix;
		}
		
		return false;
	}


	uint32 ASTOpSkeletalMeshTransform::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());
		
		Result = HashCombineFast(Result, GetTypeHash(Source));
		Result = HashCombineFast(Result, Matrix.ComputeHash());

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSkeletalMeshTransform::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSkeletalMeshTransform> n = new ASTOpSkeletalMeshTransform();

		n->Matrix = Matrix;
		n->Source = MapChild(Source.child());
		
		return n;
	}


	void ASTOpSkeletalMeshTransform::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Source);
	}


	void ASTOpSkeletalMeshTransform::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (LinkedAddress)
		{
			return;
		}

		FOperation::FSkeletalMeshTransformArgs Args;
		FMemory::Memzero(Args);

		if (Source) 
		{
			Args.Source = Source->LinkedAddress;
		}

		Args.Matrix = Program.AddConstant(Matrix);

		++Program.NumOps;
		LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());

		AppendCode(Program.ByteCode, GetOpType());
		AppendCode(Program.ByteCode, Args);
	}


	FSourceDataDescriptor ASTOpSkeletalMeshTransform::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		if (Source)
		{
			return Source->GetSourceDataDescriptor(Context);
		}

		return {};
	}


	ASTOp::EClosedMeshTest ASTOpSkeletalMeshTransform::IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const
	{
		if (Source)
		{
			return Source->IsClosedMesh(Cache);
		}
		return EClosedMeshTest::Unknown;
	}
}
