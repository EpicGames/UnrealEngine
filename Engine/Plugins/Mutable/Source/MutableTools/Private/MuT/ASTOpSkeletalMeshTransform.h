// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;

	class ASTOpSkeletalMeshTransform final : public ASTOp
	{
	public:
		ASTChild Source;

		FMatrix44f Matrix;

		ASTOpSkeletalMeshTransform();
		ASTOpSkeletalMeshTransform(const ASTOpSkeletalMeshTransform&) = delete;
		virtual ~ASTOpSkeletalMeshTransform() override;

		// ASTOp interface
		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef MapChild) const override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
		virtual EClosedMeshTest IsClosedMesh(TMap<const ASTOp*, EClosedMeshTest>* Cache) const override;
	};
}

