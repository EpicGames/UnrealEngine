// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSkeletalMeshTransformWithBone.h"

#include "ASTOpImageTransform.h"


UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::ASTOpSkeletalMeshTransformWithBone()
	: SourceSkeletalMesh(this)
	, Matrix(this)
{
}


UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::~ASTOpSkeletalMeshTransformWithBone()
{
	// Explicit call needed to avoid recursive destruction
	ASTOp::RemoveChildren();
}

UE::Mutable::Private::EOpType UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::GetOpType() const
{
	return EOpType::SK_TRANSFORMWITHBONE;
}


uint32 UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::Hash() const
{
	uint32 Result = GetTypeHash(GetOpType());
	
	Result = HashCombineFast(Result, GetTypeHash(SourceSkeletalMesh));
	Result = HashCombineFast(Result, GetTypeHash(Matrix));
	Result = HashCombineFast(Result, GetTypeHash(BoneName));
	Result = HashCombineFast(Result, GetTypeHash(ThresholdFactor));

	return Result;
}


bool UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::IsEqual(const ASTOp& OtherUntyped) const
{
	if (GetOpType() == OtherUntyped.GetOpType())
	{
		const ASTOpSkeletalMeshTransformWithBone& Other = static_cast<const ASTOpSkeletalMeshTransformWithBone&>(OtherUntyped);
		return SourceSkeletalMesh == Other.SourceSkeletalMesh &&
			BoneName == Other.BoneName &&
			Matrix == Other.Matrix &&
			ThresholdFactor == Other.ThresholdFactor;
	}
	
	return false;
}


UE::Mutable::Private::Ptr<UE::Mutable::Private::ASTOp> UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::Clone(MapChildFuncRef MapChild) const
{
	Ptr<ASTOpSkeletalMeshTransformWithBone> NewASTOp = new ASTOpSkeletalMeshTransformWithBone();
	
	NewASTOp->SourceSkeletalMesh = MapChild(SourceSkeletalMesh.child());
	NewASTOp->Matrix = MapChild(Matrix.child());
	NewASTOp->BoneName = BoneName;
	NewASTOp->ThresholdFactor = ThresholdFactor;

	return NewASTOp;
}

void UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::ForEachChild(const TFunctionRef<void(ASTChild&)> Function)
{
	Function(SourceSkeletalMesh);
	Function(Matrix);
}


void UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::Link(FProgram& Program, FLinkerOptions* Options)
{
	if (LinkedAddress)
	{
		return;
	}
	
	FOperation::FSkeletalMeshTransformWithBoneArgs Args;
	FMemory::Memzero(Args);
	if (SourceSkeletalMesh)
	{
		Args.SourceSkeletalMesh = SourceSkeletalMesh->LinkedAddress;
	}

	if (Matrix)
	{
		Args.Matrix = Matrix->LinkedAddress;
	}

	Args.BoneId = Program.AddConstant(BoneName);
	Args.ThresholdFactor = ThresholdFactor;

	++Program.NumOps;
	LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
		
	AppendCode(Program.ByteCode, GetOpType());
	AppendCode(Program.ByteCode, Args);
}


UE::Mutable::Private::FSourceDataDescriptor UE::Mutable::Private::ASTOpSkeletalMeshTransformWithBone::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
{
	if (SourceSkeletalMesh)
	{
		return SourceSkeletalMesh->GetSourceDataDescriptor(Context);
	}

	return {};
}
