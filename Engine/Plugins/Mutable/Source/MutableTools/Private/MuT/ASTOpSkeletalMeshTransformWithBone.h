// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	struct FProgram;


	//---------------------------------------------------------------------------------------------
	//!
	//---------------------------------------------------------------------------------------------
	class ASTOpSkeletalMeshTransformWithBone final : public ASTOp
	{
	public:
		ASTChild SourceSkeletalMesh;
		ASTChild Matrix;
		FName BoneName = NAME_None;
		float ThresholdFactor = 0.05f;

		ASTOpSkeletalMeshTransformWithBone();
		ASTOpSkeletalMeshTransformWithBone(const ASTOpSkeletalMeshTransformWithBone&) = delete;
		virtual ~ASTOpSkeletalMeshTransformWithBone() override;

		virtual EOpType GetOpType() const override;
		virtual uint32 Hash() const override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)>) override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;
	};
}

