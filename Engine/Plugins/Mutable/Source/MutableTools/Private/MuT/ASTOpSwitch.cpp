// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ASTOpSwitch.h"

#include "ASTOpBoolEqualIntConst.h"
#include "ASTOpConditional.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ASTOpConstantInt.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "Containers/Map.h"
#include "Misc/AssertionMacros.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Model.h"
#include "MuR/MutableMath.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	ASTOpSwitch::ASTOpSwitch()
		: Variable(this)
		, Default(this)
	{
	}


	ASTOpSwitch::~ASTOpSwitch()
	{
		// Explicit call needed to avoid recursive destruction
		ASTOp::RemoveChildren();
	}


	bool ASTOpSwitch::IsEqual(const ASTOp& OtherUntyped) const
	{
		if (GetOpType() == OtherUntyped.GetOpType())
		{
			const ASTOpSwitch* Other = static_cast<const ASTOpSwitch*>(&OtherUntyped);

			return Type     == Other->Type     && 
				   Variable == Other->Variable &&
				   Cases    == Other->Cases    && 
				   Default  == Other->Default;
		}

		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSwitch::Clone(MapChildFuncRef MapChild) const
	{
		Ptr<ASTOpSwitch> New = new ASTOpSwitch();

		New->Type = Type;
		New->Variable = MapChild(Variable.child());
		New->Default = MapChild(Default.child());
		for (const FCase& Case : Cases)
		{
			New->Cases.Emplace(Case.Condition, New, MapChild(Case.Branch.child()));
		}

		return New;
	}


	void ASTOpSwitch::Assert()
	{
		switch (Type)
		{
		case EOpType::NU_SWITCH:
		case EOpType::SC_SWITCH:
		case EOpType::CO_SWITCH:
		case EOpType::IM_SWITCH:
		case EOpType::ME_SWITCH:
		case EOpType::LA_SWITCH:
		case EOpType::IN_SWITCH:
		case EOpType::ED_SWITCH:
		case EOpType::SK_SWITCH:
		case EOpType::IS_SWITCH:
		case EOpType::MI_SWITCH:
		case EOpType::LD_SWITCH:
			break;
		default:
			// Unexpected Type
			check(false);
			break;
		}

		ASTOp::Assert();
	}


	uint32 ASTOpSwitch::Hash() const
	{
		uint32 Result = GetTypeHash(GetOpType());

		for (const FCase& Case : Cases)
		{
			Result = HashCombineFast(Result, GetTypeHash(Case.Condition));
			Result = HashCombineFast(Result, GetTypeHash(Case.Branch));
		}

		return Result;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSwitch::GetFirstValidValue()
	{
		for (const FCase& Case : Cases)
		{
			if (Case.Branch)
			{
				return Case.Branch.child();
			}
		}

		return nullptr;
	}


	bool ASTOpSwitch::IsCompatibleWith(const ASTOpSwitch* Other) const
	{
		if (!Other)
		{
			return false;
		}

		if (Variable.child() != Other->Variable.child())
		{
			return false;
		}

		if (Cases.Num() != Other->Cases.Num())
		{
			return false;
		}

		for (const FCase& Case : Cases)
		{
			bool bFound = false;
			for (const FCase& OtherCase : Other->Cases)
			{
				if (Case.Condition == OtherCase.Condition)
				{
					bFound = true;
					break;
				}
			}

			if (!bFound)
			{
				return false;
			}
		}

		return true;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSwitch::FindBranch(int32 Condition) const
	{
		for (const FCase& Case : Cases)
		{
			if (Case.Condition == Condition)
			{
				return Case.Branch.child();
			}
		}

		return Default.child();
	}


	void ASTOpSwitch::ForEachChild(const TFunctionRef<void(ASTChild&)> Func)
	{
		Func(Variable);
		Func(Default);

		for (FCase& Case : Cases)
		{
			Func(Case.Branch);
		}
	}


	void ASTOpSwitch::Link(FProgram& Program, FLinkerOptions*)
	{
		// Already linked?
		if (!LinkedAddress)
		{
			++Program.NumOps;
			LinkedAddress = MakeProgramAddress(GetOpType(), Program.ByteCode.Num());
			
			AppendCode(Program.ByteCode, Type);

			FOperation::ADDRESS VarAddress = Variable ? Variable->LinkedAddress : 0;
			FOperation::ADDRESS DefAddress = Default ? Default->LinkedAddress : 0;

			AppendCode(Program.ByteCode, VarAddress);
			AppendCode(Program.ByteCode, DefAddress);

			const int32 CaseCount = Cases.Num();
			int32 CasesInRange = 0;
			for (int32 CaseIndex = 0; CaseIndex < CaseCount; ++CaseIndex)
			{
				for (; CaseIndex < CaseCount; ++CaseIndex)
				{
					int32 CandidateCaseIndex = CaseIndex + 1;

					if (Cases.IsValidIndex(CandidateCaseIndex) &&
						Cases[CandidateCaseIndex].Branch == Cases[CandidateCaseIndex - 1].Branch &&
						Cases[CandidateCaseIndex].Condition == Cases[CandidateCaseIndex - 1].Condition + 1)
					{
						++CasesInRange;
					}
					else
					{
						break;
					}
				}				
			}

			bool bUseRanges = CasesInRange >= CaseCount / 2;

			if (!bUseRanges)
			{
				FOperation::FSwitchCaseDescriptor CaseDesc;
				CaseDesc.Count = Cases.Num();
				CaseDesc.bUseRanges = false;
				AppendCode(Program.ByteCode, CaseDesc);

				for (const FCase& Case : Cases)
				{
					FOperation::ADDRESS CaseBranchAddress = Case.Branch ? Case.Branch->LinkedAddress : 0;
					AppendCode(Program.ByteCode, Case.Condition);
					AppendCode(Program.ByteCode, CaseBranchAddress);
				}
			}
			else
			{
				FOperation::FSwitchCaseDescriptor CaseDesc;
				CaseDesc.Count = Cases.Num() - CasesInRange;
				CaseDesc.bUseRanges = true;

				AppendCode(Program.ByteCode, CaseDesc);

				for (int32 CaseIndex = 0; CaseIndex < CaseCount; ++CaseIndex)
				{
					int32 RangeStart = CaseIndex;
					uint32 RangeSize = 1;
					for (; CaseIndex < CaseCount; ++CaseIndex)
					{
						int32 CandidateCaseIndex = CaseIndex + 1;
						if (Cases.IsValidIndex(CandidateCaseIndex) &&
							Cases[CandidateCaseIndex].Branch == Cases[CandidateCaseIndex - 1].Branch &&
							Cases[CandidateCaseIndex].Condition == Cases[CandidateCaseIndex - 1].Condition + 1)
						{
							++RangeSize;
						}
						else
						{
							break;
						}
					}

					const FCase& FirstCase = Cases[RangeStart];
					FOperation::ADDRESS CaseBranchAddress = FirstCase.Branch ? FirstCase.Branch->LinkedAddress : 0;
					AppendCode(Program.ByteCode, FirstCase.Condition);
					AppendCode(Program.ByteCode, RangeSize);
					AppendCode(Program.ByteCode, CaseBranchAddress);
				}
			}
		}
	}


	FImageDesc ASTOpSwitch::GetImageDesc(bool bReturnBestOption, class FGetImageDescContext* Context) const
	{
		FImageDesc Result;

		// Local context in case it is necessary
		FGetImageDescContext LocalContext;
		if (!Context)
		{
			Context = &LocalContext;
		}
		else
		{
			// Cached result?
			FImageDesc* PtrValue = Context->m_results.Find(this);
			if (PtrValue)
			{
				return *PtrValue;
			}
		}

		// In a switch we cannot guarantee the size and format.
		// We check all the options, and if they are the same we return that.
		// Otherwise, we return a descriptor with empty fields in the conflicting ones, size or format.
		// In some places this will force re-formatting of the image.
		// The code optimiser will take care then of moving the format operations down to each
		// Branch and remove the unnecessary ones.
		FImageDesc Candidate;

		bool bSameSize = true;
		bool bSameFormat = true;
		bool bSameLods = true;
		bool bFirst = true;
		bool bIsFormatFromImageParameter = false;

		if (Default)
		{
			FImageDesc ChildDesc = Default->GetImageDesc(bReturnBestOption, Context);
			Candidate = ChildDesc;
			bFirst = false;
			bIsFormatFromImageParameter = Context->bIsFormatFromImageParameter;
		}

		for (int32 CaseIndex = 0; CaseIndex < Cases.Num(); ++CaseIndex)
		{
			if (Cases[CaseIndex].Branch)
			{
				FImageDesc ChildDesc = Cases[CaseIndex].Branch->GetImageDesc(bReturnBestOption, Context);
				bIsFormatFromImageParameter = bIsFormatFromImageParameter || Context->bIsFormatFromImageParameter;

				if (bFirst)
				{
					Candidate = ChildDesc;
					bFirst = false;
				}
				else
				{
					bSameSize = bSameSize && (Candidate.m_size == ChildDesc.m_size);
					bSameFormat = bSameFormat && (Candidate.m_format == ChildDesc.m_format);
					bSameFormat = bSameFormat && (Candidate.FormatIfAlpha == ChildDesc.FormatIfAlpha);
					bSameLods = bSameLods && (Candidate.m_lods == ChildDesc.m_lods);

					if (bReturnBestOption)
					{
						Candidate.m_format = GetMostGenericFormat(Candidate.m_format, ChildDesc.m_format);
						Candidate.FormatIfAlpha = GetMostGenericFormat(Candidate.FormatIfAlpha, ChildDesc.FormatIfAlpha);

						// Return the biggest size
						Candidate.m_size[0] = FMath::Max(Candidate.m_size[0], ChildDesc.m_size[0]);
						Candidate.m_size[1] = FMath::Max(Candidate.m_size[1], ChildDesc.m_size[1]);
					}
				}
			}
		}

		Result = Candidate;

		// In case of ReturnBestOption the first valid case will be used to determine size and lods.
		// Format will be the most generic from all Cases.
		if (!bSameFormat && !bReturnBestOption)
		{
			Result.m_format = EImageFormat::None;
			Result.FormatIfAlpha = EImageFormat::None;
		}

		if (!bSameSize && !bReturnBestOption)
		{
			Result.m_size = FImageSize(0, 0);
		}

		if (!bSameLods && !bReturnBestOption)
		{
			Result.m_lods = 0;
		}

		// Format overriden by current op.
		Context->bIsFormatFromImageParameter = bIsFormatFromImageParameter;

		// Cache the result
		Context->m_results.Add(this, Result);

		return Result;
	}


	void ASTOpSwitch::GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache)
	{
		switch (Type)
		{
		case EOpType::LA_SWITCH:
		{
			Ptr<ASTOp> child = GetFirstValidValue();
			if (!child)
			{
				child = Default.child();
			}

			if (child)
			{
				child->GetBlockLayoutSizeCached(BlockId, pBlockX, pBlockY, cache);
			}
			else
			{
				*pBlockX = 0;
				*pBlockY = 0;
			}
			break;
		}

		default:
			check(false);
			break;
		}
	}


	void ASTOpSwitch::GetLayoutBlockSize(int32* pBlockX, int32* pBlockY)
	{
		switch (Type)
		{
		case EOpType::IM_SWITCH:
		{
			Ptr<ASTOp> child = GetFirstValidValue();
			if (!child)
			{
				child = Default.child();
			}

			if (child)
			{
				child->GetLayoutBlockSize(pBlockX, pBlockY);
			}
			else
			{
				checkf(false, TEXT("Image switch had no options."));
			}
			break;
		}

		default:
			checkf(false, TEXT("Instruction not supported"));
		}
	}


	bool ASTOpSwitch::GetNonBlackRect(FImageRect& maskUsage) const
	{
		if (Type == EOpType::IM_SWITCH)
		{
			FImageRect local;
			bool localValid = false;
			if (Default)
			{
				localValid = Default->GetNonBlackRect(local);
				if (!localValid)
				{
					return false;
				}
			}

			for (const FCase& c : Cases)
			{
				if (c.Branch)
				{
					FImageRect branchRect;
					bool validBranch = c.Branch->GetNonBlackRect(branchRect);
					if (validBranch)
					{
						if (localValid)
						{
							local.Bound(branchRect);
						}
						else
						{
							local = branchRect;
							localValid = true;
						}
					}
					else
					{
						return false;
					}
				}
			}

			if (localValid)
			{
				maskUsage = local;
				return true;
			}
		}

		return false;
	}


	bool ASTOpSwitch::IsImagePlainConstant(FVector4f&) const
	{
		// We could check if every option is plain and exactly the same color, but probably it is
		// not worth.
		return false;
	}


	UE::Mutable::Private::Ptr<ASTOp> ASTOpSwitch::OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const
	{
		// Constant Condition?
		if (Variable->GetOpType() == EOpType::NU_CONSTANT)
		{
			Ptr<ASTOp> Branch = Default.child();

			const ASTOpConstantInt* typedCondition = static_cast<const ASTOpConstantInt*>(Variable.child().get());
			for (int32 o = 0; o < Cases.Num(); ++o)
			{
				if (Cases[o].Branch &&
					typedCondition->Value == (int32)Cases[o].Condition)
				{
					Branch = Cases[o].Branch.child();
					break;
				}
			}

			return Branch;
		}

		else if (Variable->GetOpType() == EOpType::NU_PARAMETER)
		{
			// If all the branches for the possible values are the same op remove the instruction
			const ASTOpParameter* ParamOp = static_cast<const ASTOpParameter*>(Variable.child().get());
			check(ParamOp);
			if(ParamOp->Parameter.PossibleValues.IsEmpty())
			{
				return nullptr;
			}

			bool bFirstValue = true;
			bool bAllSame = true;
			Ptr<ASTOp> SameBranch = nullptr;
			for (const FParameterDesc::FIntValueDesc& Value : ParamOp->Parameter.PossibleValues)
			{
				// Look for the switch Branch it would take
				Ptr<ASTOp> Branch = Default.child();
				for (const FCase& Case : Cases)
				{
					if (Case.Condition == Value.Value)
					{
						Branch = Case.Branch.child();
						break;
					}
				}

				if (bFirstValue)
				{
					bFirstValue = false;
					SameBranch = Branch;
				}
				else
				{
					if (SameBranch != Branch)
					{
						bAllSame = false;
						SameBranch = nullptr;
						break;
					}
				}
			}

	        if (bAllSame)
	        {
				return SameBranch;
	        }
		}

		// Remove conditionals with the same condition as the switch.
		{
			Ptr<ASTOpSwitch> NewSwitch = nullptr;

			for (int32 CaseIndex = 0; CaseIndex < Cases.Num(); ++CaseIndex)
			{
				const FCase& Case = Cases[CaseIndex];
				
				if (!Case.Branch || !Case.Branch->IsConditional())
				{
					continue;
				}
				
				ASTOpConditional* Conditional = static_cast<ASTOpConditional*>(Case.Branch.child().get());
				if (!Conditional->condition || Conditional->condition->GetOpType() != EOpType::BO_EQUAL_INT_CONST)
				{
					continue;
				}
				
				ASTOpBoolEqualIntConst* Condition = static_cast<ASTOpBoolEqualIntConst*>(Conditional->condition.child().get());
				if (Condition->Value != Variable)
				{
					continue;
				}
				
				if (!NewSwitch)
				{
					NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(this);
				}
				
				NewSwitch->Cases[CaseIndex].Branch = Conditional->yes.child();
			}
			
			if (NewSwitch)
			{
				return NewSwitch;
			}
		}

		return nullptr;
	}


	Ptr<ASTOp> ASTOpSwitch::OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const
	{
		Ptr<ASTOp> NewOp;

		// Detect if all Cases are the same op Type or they are null (same op with some branches being null).
		EOpType BranchOpType = EOpType::NONE;
		bool bSameOpTypeOrNull = true;

		if (Default)
		{
			BranchOpType = Default->GetOpType();
		}

		for (const FCase& Case : Cases)
		{
			if (!Case.Branch)
			{
				continue;
			}

			if (BranchOpType==EOpType::NONE)
			{
				BranchOpType = Case.Branch->GetOpType();
			}
			else if (Case.Branch->GetOpType() != BranchOpType)
			{
				bSameOpTypeOrNull = false;
				break;
			}
		}

		if (bSameOpTypeOrNull)
		{
			switch (BranchOpType)
			{
			case EOpType::ME_ADDMETADATA:
			{
				// Move the metadata out of the switch if all metadatas are the same
				bool bAllMetadataIsTheSame = true;
				TArray<FName> GameplayTags;
				TArray<PASSTHROUGH_ID> AssetUserDataIds;
				TArray<TPair<FName, PASSTHROUGH_ID>> AnimationSlots;
				PASSTHROUGH_ID SkeletonId = PASSTHROUGH_ID_INVALID;
				int16 BonePosePriority = 0;
				PASSTHROUGH_ID PhysicsAssetId = PASSTHROUGH_ID_INVALID;
				TArray<uint32> AdditionalPhysicsAssetIds;

				if (Default)
				{
					check(Default->GetOpType() == EOpType::ME_ADDMETADATA);
					const ASTOpMeshAddMetadata* Typed = static_cast<const ASTOpMeshAddMetadata*>(Default.child().get());
					GameplayTags = Typed->GameplayTags;
					AssetUserDataIds = Typed->AssetUserDataIds;
					AnimationSlots = Typed->AnimationSlots;
					SkeletonId = Typed->SkeletonId;
					BonePosePriority = Typed->BonePosePriority;
					PhysicsAssetId = Typed->PhysicsAssetId;
					AdditionalPhysicsAssetIds = Typed->AdditionalPhysicsAssetIds;
				}

				for (const FCase& Case : Cases)
				{
					if (!Case.Branch)
					{
						continue;
					}

					check(Case.Branch->GetOpType() == EOpType::ME_ADDMETADATA);
					const ASTOpMeshAddMetadata* Typed = static_cast<const ASTOpMeshAddMetadata*>(Case.Branch.child().get());
					if (GameplayTags.IsEmpty() && AssetUserDataIds.IsEmpty() &&  SkeletonId == PASSTHROUGH_ID_INVALID && PhysicsAssetId == PASSTHROUGH_ID_INVALID && AdditionalPhysicsAssetIds.IsEmpty())
					{
						GameplayTags = Typed->GameplayTags;
						AssetUserDataIds = Typed->AssetUserDataIds;
						AnimationSlots = Typed->AnimationSlots;
						SkeletonId = Typed->SkeletonId;
						BonePosePriority = Typed->BonePosePriority;
						PhysicsAssetId = Typed->PhysicsAssetId;
						AdditionalPhysicsAssetIds = Typed->AdditionalPhysicsAssetIds;
					}
					else if (
							Typed->GameplayTags != GameplayTags || 
							Typed->AssetUserDataIds != AssetUserDataIds || 
							Typed->AnimationSlots != AnimationSlots || 
							Typed->SkeletonId != SkeletonId ||
							Typed->BonePosePriority != BonePosePriority ||
							Typed->PhysicsAssetId != PhysicsAssetId ||
							Typed->AdditionalPhysicsAssetIds != AdditionalPhysicsAssetIds)
					{
						bAllMetadataIsTheSame = false;
						break;
					}
				}

				if (bAllMetadataIsTheSame)
				{
					Ptr<ASTOpMeshAddMetadata> New = new ASTOpMeshAddMetadata();
					New->GameplayTags = GameplayTags;
					New->AssetUserDataIds = AssetUserDataIds;
					New->AnimationSlots = AnimationSlots;
					New->SkeletonId = SkeletonId;
					New->BonePosePriority= BonePosePriority;
					New->PhysicsAssetId = PhysicsAssetId;
					New->AdditionalPhysicsAssetIds = AdditionalPhysicsAssetIds;

					{
						Ptr<ASTOpSwitch> NewSwitch = UE::Mutable::Private::Clone<ASTOpSwitch>(this);

						// Replace all branches removing the "add tags" operation.
						if (Default)
						{
							check(Default->GetOpType() == EOpType::ME_ADDMETADATA);
							const ASTOpMeshAddMetadata* Typed = static_cast<const ASTOpMeshAddMetadata*>(Default.child().get());
							NewSwitch->Default = Typed->Source.child();
						}

						for (int32 CaseIndex = 0; CaseIndex < Cases.Num(); ++CaseIndex)
						{
							const FCase& SourceCase = Cases[CaseIndex];
							if (!SourceCase.Branch)
							{
								continue;
							}

							FCase& NewCase = NewSwitch->Cases[CaseIndex];

							check(SourceCase.Branch->GetOpType() == EOpType::ME_ADDMETADATA);
							const ASTOpMeshAddMetadata* Typed = static_cast<const ASTOpMeshAddMetadata*>(SourceCase.Branch.child().get());
							NewCase.Branch = Typed->Source.child();
						}

						New->Source = NewSwitch;
					}

					NewOp = New;
				}
				break;
			}

			default:
				break;
			}
		}

		return NewOp;
	}


	UE::Mutable::Private::Ptr<ImageSizeExpression> ASTOpSwitch::GetImageSizeExpression() const
	{
		Ptr<ImageSizeExpression> pRes = new ImageSizeExpression;

		bool first = true;
		for (const FCase& c : Cases)
		{
			if (c.Branch)
			{
				if (first)
				{
					pRes = c.Branch->GetImageSizeExpression();
				}
				else
				{
					Ptr<ImageSizeExpression> pOther = c.Branch->GetImageSizeExpression();
					if (!(*pOther == *pRes))
					{
						pRes->type = ImageSizeExpression::ISET_UNKNOWN;
						break;
					}
				}
			}
		}

		return pRes;
	}


	FSourceDataDescriptor ASTOpSwitch::GetSourceDataDescriptor(FGetSourceDataDescriptorContext* Context) const
	{
		// Cache management
		TUniquePtr<FGetSourceDataDescriptorContext> LocalContext;
		if (!Context)
		{
			LocalContext.Reset(new FGetSourceDataDescriptorContext);
			Context = LocalContext.Get();
		}

		FSourceDataDescriptor* Found = Context->Cache.Find(this);
		if (Found)
		{
			return *Found;
		}

		// Not cached: calculate
		FSourceDataDescriptor Result;

		for (const FCase& Case : Cases)
		{
			if (Case.Branch)
			{
				FSourceDataDescriptor SourceDesc = Case.Branch->GetSourceDataDescriptor(Context);
				Result.CombineWith(SourceDesc);
			}
		}

		Context->Cache.Add(this, Result);

		return Result;
	}


}
