// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "MuR/Image.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
struct FProgram;
template <class SCALAR> class vec4;


	//---------------------------------------------------------------------------------------------
	//! Variable sized switch operation.
	//---------------------------------------------------------------------------------------------
	class ASTOpSwitch final : public ASTOp
	{
	public:

		//! Type of switch
		EOpType Type;

		//! Variable whose value will be used to choose the switch branch.
		ASTChild Variable;

		//! Default branch in case none matches the value in the variable.
		ASTChild Default;

		struct FCase
		{
			struct FInvalidTag {};

			explicit FCase(FInvalidTag) : Branch(ASTChild::FInvalidTag()) {}
			
			FCase(int32 cond, Ptr<ASTOp> parent, Ptr<ASTOp> b)
				: Condition(cond)
				, Branch(parent.get(), b)
			{
			}

			int32 Condition;
			ASTChild Branch;

			bool operator==(const FCase& other) const = default;
		};

		TArray<FCase> Cases;

	public:

		ASTOpSwitch();
		ASTOpSwitch(const ASTOpSwitch&) = delete;
		~ASTOpSwitch() override;

		// ASTOp interface
		virtual EOpType GetOpType() const override { return Type; }
		virtual uint32 Hash() const override;
		virtual void ForEachChild(const TFunctionRef<void(ASTChild&)> f) override;
		virtual bool IsEqual(const ASTOp& otherUntyped) const override;
		virtual Ptr<ASTOp> Clone(MapChildFuncRef mapChild) const override;
		virtual void Assert() override;
		virtual void Link(FProgram& program, FLinkerOptions* Options) override;
		virtual FImageDesc GetImageDesc(bool returnBestOption, class FGetImageDescContext* context) const  override;
		virtual void GetBlockLayoutSize(uint64 BlockId, int32* pBlockX, int32* pBlockY, FBlockLayoutSizeCache* cache) override;
		virtual void GetLayoutBlockSize(int32* pBlockX, int32* pBlockY) override;
		virtual Ptr<ASTOp> OptimiseSemantic(const FModelOptimizationOptions&, int32 Pass) const override;
		Ptr<ASTOp> OptimiseSink(const FModelOptimizationOptions&, FOptimizeSinkContext&) const override;
		virtual bool GetNonBlackRect(FImageRect& maskUsage) const override;
		virtual bool IsImagePlainConstant(FVector4f& color) const override;
		virtual Ptr<ImageSizeExpression> GetImageSizeExpression() const override;
		virtual bool IsSwitch() const override { return true; }
		virtual FSourceDataDescriptor GetSourceDataDescriptor(FGetSourceDataDescriptorContext*) const override;

		// Own interface

		//!
		Ptr<ASTOp> GetFirstValidValue();

		//! Return true if the two switches have the same condition, amd case value (but not
		//! necessarily branches) or operation type.
		bool IsCompatibleWith(const ASTOpSwitch* other) const;

		//!
		Ptr<ASTOp> FindBranch(int32 condition) const;

	};


}

