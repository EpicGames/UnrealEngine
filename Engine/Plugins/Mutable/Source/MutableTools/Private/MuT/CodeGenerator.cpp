// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/CodeGenerator.h"

#include "ASTOpAddOverlayMaterial.h"
#include "ASTOpAddOverrideMaterial.h"
#include "ASTOpAddSkeletalMesh.h"
#include "ASTOpEntry.h"
#include "ASTOpComponentAdd.h"
#include "Containers/Array.h"
#include "Logging/LogMacros.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathUtility.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Layout.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Platform.h"
#include "MuT/ASTOpLODNew.h"
#include "MuT/ASTOpAddExtensionData.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantBool.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpBoolAnd.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpMeshBindShape.h"
#include "MuT/ASTOpMeshClipDeform.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshMaskClipMesh.h"
#include "MuT/ASTOpMeshMaskClipUVMask.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshDifference.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshExtractLayoutBlocks.h"
#include "MuT/ASTOpMeshApplyLayout.h"
#include "MuT/ASTOpMeshMerge.h"
#include "MuT/ASTOpMeshSetMaterialSlotId.h"
#include "MuT/ASTOpMeshMaskDiff.h"
#include "MuT/ASTOpMeshTransformWithBoundingMesh.h"
#include "MuT/ASTOpMeshTransformWithBone.h"
#include "MuT/ASTOpLayoutRemoveBlocks.h"
#include "MuT/ASTOpLayoutFromMesh.h"
#include "MuT/ASTOpLayoutMerge.h"
#include "MuT/ASTOpLayoutPack.h"
#include "MuT/CodeGenerator_SecondPass.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeComponentSwitch.h"
#include "MuT/NodeComponentVariation.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeImageConstant.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeSurfaceModifierMeshClipDeform.h"
#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"
#include "MuT/NodeSurfaceModifierMeshClipWithMesh.h"
#include "MuT/NodeSurfaceModifierMeshClipWithUVMask.h"
#include "MuT/NodeSurfaceModifierMeshTransformInMesh.h"
#include "MuT/NodeSurfaceModifierMeshTransformWithBone.h"
#include "MuT/NodeSurfaceModifierSurfaceEdit.h"
#include "MuT/NodeObject.h"
#include "MuT/NodeObjectNew.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeRangeFromScalar.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/NodeScalarEnumParameter.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeSurfaceVariation.h"
#include "MuT/NodeSurfaceSwitch.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{
	TAutoConsoleVariable<bool> CVarAllowMulticompose(
		TEXT("mutable.AllowImageMulticompose"),
		true,
		TEXT("If set to true, enables the use of multicompose."),
		ECVF_Default);

	CodeGenerator::CodeGenerator(CompilerOptions::Private* Options, TFunction<void()>& InWaitCallback)
		: WaitCallback(InWaitCallback),
		LocalPipe(TEXT("CodeGeneratorPipe")),
		GenerateMeshConstantPipe(TEXT("GenerateMeshConstantPipe"))
	{
		CompilerOptions = Options;

		// Create the message log
		ErrorLog = MakeShared<FErrorLog>();
	}

	
	CodeGenerator::~CodeGenerator()
	{
		if (LocalPipe.HasWork())
		{
			const FString LocalPipeWithWorkLog = FString::Printf(TEXT("The '%s' pipe still has work to do on CodeGenerator destruction."), LocalPipe.GetDebugName());
			ErrorLog->Add(LocalPipeWithWorkLog, ELMT_ERROR, nullptr);
		}
		
		if (GenerateMeshConstantPipe.HasWork())
		{
			const FString MeshConstantGenerationPipeWithWorkLog = FString::Printf(TEXT("The '%s' pipe still has work to do on CodeGenerator destruction."), GenerateMeshConstantPipe.GetDebugName());
			ErrorLog->Add(MeshConstantGenerationPipeWithWorkLog, ELMT_ERROR, nullptr);
		}
	}


	void CodeGenerator::GenerateRoot(const Ptr<const Node> InNode)
	{
		MUTABLE_CPUPROFILER_SCOPE(Generate);

		// First pass
		FirstPass.Generate(ErrorLog, InNode.get(), CompilerOptions->bIgnoreStates, this);

		// Second pass
		SecondPassGenerator SecondPass(&FirstPass, CompilerOptions);
		bool bSuccess = SecondPass.Generate(ErrorLog, InNode.get());
		if (!bSuccess)
		{
			return;
		}

		// Main pass for each state
		{
			MUTABLE_CPUPROFILER_SCOPE(MainPass);

            int32 CurrentStateIndex = 0;
			for (const FObjectState& State : FirstPass.States)
			{
				MUTABLE_CPUPROFILER_SCOPE(MainPassState);

				FGenericGenerationOptions Options;
				Options.State = CurrentStateIndex;

				FGenericGenerationResult Result;
				GenerateGeneric(Options, Result, InNode.get());

				Ptr<ASTOp> StateRoot = Result.Op;

				Ptr<ASTOpEntry> EntryOp = new ASTOpEntry(); // Dummy op. Required to be able to replace real roots (ASTOp::Replace requires the operation to have a parent).
				EntryOp->A = StateRoot;
				
				States.Emplace(State, EntryOp);

				++CurrentStateIndex;
			}
		}
	}


	// This function is a very bad idea. Someday we will hang due to oversubscription.
	// We need to convert all recursive CodeGenerator to Tasks and avoid doing Wait/GetResult.
	void CodeGenerator::WaitTask(Tasks::FTask Task)
	{
		if (IsInGameThread())
		{
			while (!Task.IsCompleted())
			{
				WaitCallback();
			}
		}
		else
		{
			Task.Wait();
		}
	}


	void CodeGenerator::GenerateGeneric(const FGenericGenerationOptions& Options, FGenericGenerationResult& OutResult, const Ptr<const Node> InNode )
	{
		if (!InNode)
		{
			return;
		}

        // Type-specific generation
		if (InNode->GetType()->IsA(NodeObject::GetStaticType()))
		{
			const NodeObject* ObjectNode = static_cast<const NodeObject*>(InNode.get());
			FObjectGenerationOptions ObjectOptions;
			ObjectOptions.ActiveTags = Options.ActiveTags;
			ObjectOptions.bIsImage = Options.bIsImage;
			ObjectOptions.State = Options.State;
			FObjectGenerationResult ObjectResult;
			GenerateObject(ObjectOptions, ObjectResult, ObjectNode);
			OutResult.Op = ObjectResult.Op;
			return;
		}

		else if (InNode->GetType()->IsA(NodeScalar::GetStaticType()))
		{
			const NodeScalar* ScalarNode = static_cast<const NodeScalar*>(InNode.get());
			FScalarGenerationResult ScalarResult;
			GenerateScalar(ScalarResult, Options, ScalarNode);
			OutResult.Op = ScalarResult.op;
			return;
		}

		else if (InNode->GetType()->IsA(NodeColor::GetStaticType()))
		{
			const NodeColor* ColorNode = static_cast<const NodeColor*>(InNode.get());
			FColorGenerationResult Result;
			GenerateColor(Result, Options, ColorNode);
			OutResult.Op = Result.op;
			return;
		}

		else if (InNode->GetType()->IsA(NodeProjector::GetStaticType()))
		{
			const NodeProjector* projNode = static_cast<const NodeProjector*>(InNode.get());
			FProjectorGenerationResult ProjResult;
			GenerateProjector(ProjResult, Options, projNode);
			OutResult.Op = ProjResult.op;
			return;
		}

		else if (InNode->GetType()->IsA(NodeSurfaceNew::GetStaticType()))
		{
			// This no longer happens with the current tools.
			check(false);
			return;
		}

		else if (InNode->GetType()->IsA(NodeSurfaceVariation::GetStaticType()))
		{
			// This happens only if we generate a node graph that has a NodeSurfaceVariation at the root.
			return;
		}

		else if (InNode->GetType()->IsA(NodeSurfaceSwitch::GetStaticType()))
		{
			// This happens only if we generate a node graph that has a NodeSurfaceSwitch at the root.
			return;
		}

		else if (InNode->GetType()->IsA(NodeSurfaceModifier::GetStaticType()))
		{
			// This happens only if we generate a node graph that has a modifier at the root.
			return;
		}

		else if (InNode->GetType()->IsA(NodeComponent::GetStaticType()))
		{
			const NodeComponent* ComponentNode = static_cast<const NodeComponent*>(InNode.get());
			FComponentGenerationOptions ComponentOptions(Options, nullptr);
			GenerateComponent(ComponentOptions, OutResult, ComponentNode);
			return;
		}

		else
		{
			// Unsupported node.
			check(false);
		}

	}


	void CodeGenerator::GenerateObject(const FObjectGenerationOptions& InOptions, FObjectGenerationResult& OutResult, const NodeObject* InUntypedNode)
	{
		if (!InUntypedNode)
		{
			OutResult = {};
			return;
		}

		// See if it was already generated
		FGeneratedObjectCacheKey Key;
		Key.Node = InUntypedNode;
		Key.Options = InOptions;
		FGeneratedObjectsMap::ValueType* Found = GeneratedObjects.Find(Key);
		if (Found)
		{
			OutResult = *Found;
			return;
		}

		// Generate for each different type of node
		const FNodeType* Type = InUntypedNode->GetType();
		if (Type == NodeObjectNew::GetStaticType())
		{
			GenerateObject_New(InOptions, OutResult, static_cast<const NodeObjectNew*>(InUntypedNode));
		}
		else if (Type == NodeObjectGroup::GetStaticType())
		{
			GenerateObject_Group(InOptions, OutResult, static_cast<const NodeObjectGroup*>(InUntypedNode));
		}
		else
		{
			check(false);
		}

		// Cache the result
		GeneratedObjects.Add(Key, OutResult);
	}


	void CodeGenerator::GenerateRange(FRangeGenerationResult& Result, const FGenericGenerationOptions& Options, Ptr<const NodeRange> Untyped)
	{
		if (!Untyped)
		{
			Result = FRangeGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;

		{
			UE::TUniqueLock Lock(GeneratedRanges.Mutex);
			FGeneratedRangeMap::ValueType* Found = GeneratedRanges.Map.Find(Key);
			if (Found)
			{
				Result = *Found;
				return;
			}
		}

		// Generate for each different type of node
		if (Untyped->GetType()==NodeRangeFromScalar::GetStaticType())
		{
			const NodeRangeFromScalar* FromScalar = static_cast<const NodeRangeFromScalar*>(Untyped.get());

			Result = FRangeGenerationResult();
			Result.rangeName = FromScalar->Name;

			FScalarGenerationResult ChildResult;
			GenerateScalar(ChildResult, Options, FromScalar->Size);
			Result.sizeOp = ChildResult.op;
		}
		else
		{
			check(false);
		}

		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedRanges.Mutex);
			GeneratedRanges.Map.Add(Key, Result);
		}
	}


	Ptr<NodeScalar> CodeGenerator::GenerateTableVariableNode(const Node& InNode, const FTableCacheKey& CacheKey, bool bAddNoneOption, const FString& DefaultRowName)
	{
		Ptr<NodeScalarEnumParameter> Result = new NodeScalarEnumParameter;

		FString ParamName = CacheKey.ParameterName;
		if (ParamName.Len() == 0)
		{
			ParamName = CacheKey.Table->GetName();
		}

		Result->Name = ParamName;
		Result->DefaultValue = 0;
		Result->SetMessageContext(InNode.GetMessageContext());

		int32 CurrentRow = 0;
		int32 RowCount = CacheKey.Table->GetPrivate()->Rows.Num();
		check(RowCount < MAX_int16) // max FIntValueDesc allows

		if (bAddNoneOption)
		{
			Result->Options.SetNum(RowCount);
			Result->Options[CurrentRow].Value = -1;
			Result->Options[CurrentRow].Name = "None";
		}
		else
		{
			Result->Options.SetNum(RowCount-1);
		}

		// Add the possible values
		{
			// See if there is a string column. If there is one, we will use it as names for the
			// options. Only the first string column will be used.
			int32 NameCol = -1;
			int32 NumCols = CacheKey.Table->GetPrivate()->Columns.Num();
			for (int32 ColumnIndex=0; ColumnIndex<NumCols && NameCol<0; ++ColumnIndex)
			{
				if (CacheKey.Table->GetPrivate()->Columns[ColumnIndex].Type == ETableColumnType::String)
				{
					NameCol = ColumnIndex;
				}
			}

			// Skip "None" option (first row) if it's not required
			int32 StarCount = bAddNoneOption ? 0 : 1;

			for (int32 RowIndex = StarCount; RowIndex < RowCount; ++RowIndex)
			{
				FString ValueName;
				if (NameCol > -1)
				{
					ValueName = CacheKey.Table->GetPrivate()->Rows[RowIndex].Values[NameCol].String;
				}

				Result->Options[CurrentRow].Value = RowIndex;
				Result->Options[CurrentRow].Name = ValueName;


				// Set the first row or the selected row as the default one.
				if(RowIndex == StarCount || ValueName == DefaultRowName)
				{
					Result->DefaultValue = RowIndex;
				}

				// Set the selected row as default (if exists)
				if (ValueName == DefaultRowName)
				{
					Result->DefaultValue = RowIndex;
				}

				++CurrentRow;
			}
		}

		return Result;
	}


	TManagedPtr<const FLayout> CodeGenerator::GenerateLayout(Ptr<const NodeLayout> SourceLayout, uint32 MeshIDPrefix)
	{
		// This can run in any thread.
		UE::TUniqueLock Lock(GenerateLayoutConstantState.Mutex);
		
		TManagedPtr<const FLayout>* CachedLayout = GenerateLayoutConstantState.GeneratedLayouts.Find({ SourceLayout,MeshIDPrefix });

		if (CachedLayout)
		{
			return *CachedLayout;
		}

		TManagedPtr<FLayout> GeneratedLayout = SourceLayout->BuildRuntimeLayout(MeshIDPrefix);

		check(GeneratedLayout->Blocks.IsEmpty() || GeneratedLayout->Blocks[0].Id != FLayoutBlock::InvalidBlockId);
		GenerateLayoutConstantState.GeneratedLayouts.Add({ SourceLayout,MeshIDPrefix }, GeneratedLayout);

		return GeneratedLayout;
	}


	Ptr<ASTOp> CodeGenerator::GenerateImageBlockPatch(Ptr<ASTOp> InBlockOp,
		const NodeSurfaceModifierSurfaceEdit::FTexture& Patch,
		TManagedPtr<FImage> PatchMask,
		Ptr<ASTOp> conditionAd,
		const FImageGenerationOptions& ImageOptions )
	{
		// Blend operation
		Ptr<ASTOp> FinalOp;
		{
			MUTABLE_CPUPROFILER_SCOPE(PatchBlend);

			Ptr<ASTOpImageLayer> LayerOp = new ASTOpImageLayer();
			LayerOp->blendType = Patch.PatchBlendType;
			LayerOp->base = InBlockOp;

			// When we patch from edit nodes, we want to apply it to all the channels.
			// \todo: since we can choose the patch function, maybe we want to be able to select this as well.
			LayerOp->Flags = Patch.bPatchApplyToAlpha ? FOperation::ImageLayerArgs::F_APPLY_TO_ALPHA : 0;

			NodeImage* ImageNode = Patch.PatchImage.get();
			Ptr<ASTOp> BlendOp;
			if (ImageNode)
			{
				FImageGenerationResult BlendResult;
				GenerateImage(ImageOptions, BlendResult, ImageNode);
				BlendOp = BlendResult.op;
			}
			else
			{
				BlendOp = GenerateMissingImageCode(TEXT("Patch top image"), EImageFormat::RGB_UByte, nullptr, ImageOptions);
			}
			BlendOp = GenerateImageFormat(BlendOp, InBlockOp->GetImageDesc().m_format);
			BlendOp = GenerateImageSize(BlendOp, ImageOptions.RectSize);
			LayerOp->blend = BlendOp;

			// Create the rect mask constant
			Ptr<ASTOp> RectConstantOp;
			{
				Ptr<NodeImageConstant> pNode = new NodeImageConstant();
				pNode->Image = PatchMask;

				FImageGenerationOptions ConstantOptions(-1,-1);
				FImageGenerationResult ConstantResult;
				GenerateImage(ConstantOptions, ConstantResult, pNode);
				RectConstantOp = ConstantResult.op;
			}

			NodeImage* MaskNode = Patch.PatchMask.get();
			Ptr<ASTOp> MaskOp;
			if (MaskNode)
			{
				// Combine the block rect mask with the user provided mask.

				FImageGenerationResult MaskResult;
				GenerateImage(ImageOptions, MaskResult, MaskNode);
				MaskOp = MaskResult.op;

				Ptr<ASTOpImageLayer> PatchCombineOp = new ASTOpImageLayer;
				PatchCombineOp->base = MaskOp;
				PatchCombineOp->blend = RectConstantOp;
				PatchCombineOp->blendType = EBlendType::BT_MULTIPLY;
				MaskOp = PatchCombineOp;
			}
			else
			{
				MaskOp = RectConstantOp;
			}
			MaskOp = GenerateImageFormat(MaskOp, EImageFormat::L_UByte);
			MaskOp = GenerateImageSize(MaskOp, ImageOptions.RectSize);
			LayerOp->mask = MaskOp;

			FinalOp = LayerOp;
		}

		// Condition to enable this patch
		if (conditionAd)
		{
			Ptr<ASTOp> conditionalAd;
			{
				Ptr<ASTOpConditional> op = new ASTOpConditional();
				op->type = EOpType::IM_CONDITIONAL;
				op->no = InBlockOp;
				op->yes = FinalOp;
				op->condition = conditionAd;
				conditionalAd = op;
			}

			FinalOp = conditionalAd;
		}

		return FinalOp;
	}
	

	//---------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateComponent(const FComponentGenerationOptions& InOptions, FGenericGenerationResult& OutResult, const NodeComponent* InUntypedNode)
	{
		if (!InUntypedNode)
		{
			OutResult = FGenericGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedComponentCacheKey Key;
		Key.Node = InUntypedNode;
		Key.Options = InOptions;
		FGeneratedComponentMap::ValueType* it = GeneratedComponents.Find(Key);
		if (it)
		{
			OutResult = *it;
			return;
		}

		// Generate for each different type of node
		const FNodeType* Type = InUntypedNode->GetType();
		if (Type == NodeComponentNew::GetStaticType())
		{
			GenerateComponent_New(InOptions, OutResult, static_cast<const NodeComponentNew*>(InUntypedNode));
		}
		else if (Type == NodeComponentSwitch::GetStaticType())
		{
			GenerateComponent_Switch(InOptions, OutResult, static_cast<const NodeComponentSwitch*>(InUntypedNode));
		}
		else if (Type == NodeComponentVariation::GetStaticType())
		{
			GenerateComponent_Variation(InOptions, OutResult, static_cast<const NodeComponentVariation*>(InUntypedNode));
		}
		else
		{
			check(false);
		}

		// Cache the result
		GeneratedComponents.Add(Key, OutResult);
	}

	
	void CodeGenerator::GenerateComponent_New(const FComponentGenerationOptions& Options, FGenericGenerationResult& Result, const NodeComponentNew* InNode)
	{
		FSkeletalMeshObjectGenerationResult SkeletalMeshResult;
		FSkeletalMeshGenerationOptions SkeletalMeshOptions(Options, InNode);
		GenerateSkeletalMeshObject(SkeletalMeshResult, SkeletalMeshOptions, InNode->SkeletalMeshObject.get());

		Ptr<ASTOpAddSkeletalMesh> AddSkeletalMeshOp = new ASTOpAddSkeletalMesh();
		AddSkeletalMeshOp->SkeletalMesh = SkeletalMeshResult.SkeletalMeshOp;

		Ptr<ASTOp> LastInstOp = AddSkeletalMeshOp;

		// Launch the task that generates the component
		// There could be more concurrency here, but it doesn't look like it is relevant yet.
		FComponentTask ComponentTask = LocalPipe.Launch(TEXT("MutableComponentNew"),
			[
				LastInstOp, InNode, Options,
				this
			]
			() mutable
			{
				FGenericGenerationResult Result;
					
				if (Ptr<NodeMaterial> MaterialNode = InNode->OverlayMaterial)
				{
					Ptr<ASTOpAddOverlayMaterial> AddOverlayMaterialOp = new ASTOpAddOverlayMaterial();
					AddOverlayMaterialOp->Instance = LastInstOp;
					AddOverlayMaterialOp->SlotName = NAME_None;

					// Material
					FMaterialGenerationOptions MaterialOptions;
					FMaterialGenerationResult MaterialResult;
					GenerateMaterial(MaterialOptions, MaterialResult, MaterialNode);
					AddOverlayMaterialOp->Material = MaterialResult.op;

					LastInstOp = AddOverlayMaterialOp;
				}
				
				for (const TPair<const FName, Ptr<NodeMaterial>>& NameAndMaterial : InNode->OverrideMaterials)
				{
					Ptr<ASTOpAddOverrideMaterial> AddOverrideMaterialOp = new ASTOpAddOverrideMaterial();
					AddOverrideMaterialOp->SlotName = NameAndMaterial.Key;
					AddOverrideMaterialOp->Instance = LastInstOp;

					FMaterialGenerationOptions MaterialOptions;
					FMaterialGenerationResult MaterialResult;
					GenerateMaterial(MaterialOptions, MaterialResult, NameAndMaterial.Value);
					AddOverrideMaterialOp->Material = MaterialResult.op;

					LastInstOp = AddOverrideMaterialOp;
				}
					
				for (const TPair<const FName, Ptr<NodeMaterial>>& NameAndMaterial : InNode->OverlayMaterials)
				{ 
					Ptr<ASTOpAddOverlayMaterial> AddOverlayMaterialOp = new ASTOpAddOverlayMaterial();
					AddOverlayMaterialOp->SlotName = NameAndMaterial.Key;
					AddOverlayMaterialOp->Instance = LastInstOp;

					FMaterialGenerationOptions MaterialOptions;
					FMaterialGenerationResult MaterialResult;
					GenerateMaterial(MaterialOptions, MaterialResult, NameAndMaterial.Value);
					AddOverlayMaterialOp->Material = MaterialResult.op;

					LastInstOp = AddOverlayMaterialOp;
				}

				Ptr<ASTOpComponentAdd> InstanceOp = new ASTOpComponentAdd();
				InstanceOp->Instance = Options.BaseInstance;
				InstanceOp->Value = LastInstOp;
				InstanceOp->ExternalId = InNode->Id;

				Result.Op = InstanceOp;

				// Add a conditional if this component has conditions
				for (const FirstPassGenerator::FComponent& Component : FirstPass.Components)
				{
					if (Component.Component != InNode)
					{
						continue;
					}

					if (Component.ComponentCondition || Component.ObjectCondition)
					{
						// TODO: This could be done earlier?
						Ptr<ASTOpBoolAnd> ConditionOp = new ASTOpBoolAnd();
						ConditionOp->A = Component.ObjectCondition ? Component.ObjectCondition : new ASTOpConstantBool(true);
						ConditionOp->B = Component.ComponentCondition ? Component.ComponentCondition : new ASTOpConstantBool(true);

						Ptr<ASTOpConditional> IfOp = new ASTOpConditional();
						IfOp->type = EOpType::IN_CONDITIONAL;
						IfOp->no = Options.BaseInstance;
						IfOp->yes = Result.Op;
						IfOp->condition = ConditionOp;

						Result.Op = IfOp;
					}
				}

				return Result;
			});

		WaitTask(ComponentTask);
		Result = ComponentTask.GetResult();
	}


	void CodeGenerator::GenerateComponent_Switch(const FComponentGenerationOptions& Options, FGenericGenerationResult& Result, const NodeComponentSwitch* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeComponentSwitch);

		if (Node->Options.Num() == 0)
		{
			// No options in the switch!
			Result.Op = Options.BaseInstance;
			return;
		}

		Ptr<ASTOpSwitch> Op = new ASTOpSwitch();
		Op->Type = EOpType::IN_SWITCH;

		// Variable value
		if (Node->Parameter)
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options, Node->Parameter.get());
			Op->Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
			Op->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Node->GetMessageContext());
		}

		// Options
		for (int32 OptionIndex = 0; OptionIndex < Node->Options.Num(); ++OptionIndex)
		{
			Ptr<ASTOp> Branch;

			if (Node->Options[OptionIndex])
			{
				FGenericGenerationResult BaseResult;
				GenerateComponent(Options, BaseResult, Node->Options[OptionIndex].get());
				Branch = BaseResult.Op;
			}
			else
			{
				// This argument is not required
				Branch = Options.BaseInstance;
			}

			Op->Cases.Emplace(OptionIndex, Op, Branch);
		}

		Result.Op = Op;
	}


	void CodeGenerator::GenerateComponent_Variation(const FComponentGenerationOptions& Options, FGenericGenerationResult& Result, const NodeComponentVariation* Node)
	{
		Ptr<ASTOp> CurrentMeshOp = Options.BaseInstance;

		// Default case
		if (Node->DefaultComponent)
		{
			FGenericGenerationResult BranchResults;

			GenerateComponent(Options, BranchResults, Node->DefaultComponent.get());
			CurrentMeshOp = BranchResults.Op;
		}

		// Process variations in reverse order, since conditionals are built bottom-up.
		for (int32 VariationIndex = Node->Variations.Num() - 1; VariationIndex >= 0; --VariationIndex)
		{
			int32 TagIndex = -1;
			const FString& Tag = Node->Variations[VariationIndex].Tag;
			for (int32 i = 0; i < FirstPass.Tags.Num(); ++i)
			{
				if (FirstPass.Tags[i].Tag == Tag)
				{
					TagIndex = i;
				}
			}

			if (TagIndex < 0)
			{
				ErrorLog->Add(
					FString::Printf(TEXT("Unknown tag found in component variation [%s]."), *Tag),
					ELMT_WARNING,
					Node->GetMessageContext(),
					ELMSB_UNKNOWN_TAG
				);
				continue;
			}

			Ptr<ASTOp> VariationMeshOp = Options.BaseInstance;
			if (Node->Variations[VariationIndex].Component)
			{
				FGenericGenerationResult BranchResults;
				GenerateComponent(Options, BranchResults, Node->Variations[VariationIndex].Component.get());

				VariationMeshOp = BranchResults.Op;
			}

			Ptr<ASTOpConditional> Conditional = new ASTOpConditional;
			Conditional->type = EOpType::IN_CONDITIONAL;
			Conditional->no = CurrentMeshOp;
			Conditional->yes = VariationMeshOp;
			Conditional->condition = FirstPass.Tags[TagIndex].GenericCondition;

			CurrentMeshOp = Conditional;
		}

		Result.Op = CurrentMeshOp;
	}


	TManagedPtr<FImage> CodeGenerator::GenerateImageBlockPatchMask(const NodeSurfaceModifierSurfaceEdit::FTexture& Patch, FIntPoint GridSize, int32 BlockPixelsX, int32 BlockPixelsY, box<FIntVector2> RectInCells )
	{
		// Create a patching mask for the block
		TManagedPtr<FImage> PatchMask;

		FIntVector2 SourceTextureSize = { GridSize[0] * BlockPixelsX, GridSize[1] * BlockPixelsY };

		FInt32Rect BlockRectInPixels;
		BlockRectInPixels.Min = { RectInCells.min[0] * BlockPixelsX, RectInCells.min[1] * BlockPixelsY };
		BlockRectInPixels.Max = { (RectInCells.min[0] + RectInCells.size[0]) * BlockPixelsX, (RectInCells.min[1] + RectInCells.size[1]) * BlockPixelsY };

		for (const FBox2f& PatchRect : Patch.PatchBlocks)
		{
			// Does the patch rect intersects the current block at all?
			FInt32Rect PatchRectInPixels;
			PatchRectInPixels.Min = { int32(PatchRect.Min[0] * SourceTextureSize[0]), int32(PatchRect.Min[1] * SourceTextureSize[1]) };
			PatchRectInPixels.Max = { int32(PatchRect.Max[0] * SourceTextureSize[0]), int32(PatchRect.Max[1] * SourceTextureSize[1]) };

			FInt32Rect BlockPatchRect = PatchRectInPixels;
			BlockPatchRect.Clip(BlockRectInPixels);

			if (BlockPatchRect.Area() > 0)
			{
				FInt32Point BlockSize = BlockRectInPixels.Size();
				if (!PatchMask)
				{
					PatchMask = MakeManaged<FImage>(BlockSize[0], BlockSize[1], 1, UE::Mutable::Private::EImageFormat::L_UByte, UE::Mutable::Private::EInitializationType::Black);
				}

				uint8* Pixels = PatchMask->GetMipData(0);
				FInt32Point BlockPatchOffset = BlockPatchRect.Min - BlockRectInPixels.Min;
				FInt32Point BlockPatchSize = BlockPatchRect.Size();
				for (int32 RowIndex = BlockPatchOffset[1]; RowIndex < BlockPatchOffset[1]+BlockPatchSize[1]; ++RowIndex)
				{
					uint8* RowPixels = Pixels + RowIndex * BlockSize[0] + BlockPatchOffset[0];
					FMemory::Memset(RowPixels, 255, BlockPatchSize[0]);
				}
			}
		}

		return PatchMask;
	}


	FSurfaceTask CodeGenerator::GenerateSurface( const FSurfaceGenerationOptions& Options, Ptr<const NodeSurfaceNew> SurfaceNode, FLODTask PreviousLODTask )
    {
        MUTABLE_CPUPROFILER_SCOPE(GenerateSurface);

        // Generate the mesh
        //------------------------------------------------------------------------

		// We don't add the mesh here, since it will be added directly at the top of the
		// component expression in the NodeComponentNew generator with the right merges
		// and conditions.
		// But we store it to be used then.

		// Do we need to generate the mesh? Or was it already generated for state conditions 
		// accepting the current state?
		TArray<FirstPassGenerator::FSurface*> TargetSurfaces;
		TargetSurfaces.Reserve(FirstPass.Surfaces.Num());

		for (FirstPassGenerator::FSurface& Surface : FirstPass.Surfaces)
		{
			if (Surface.Node != SurfaceNode)
			{
				continue;
			}

            // Check state conditions
            const bool bSurfaceValidForThisState = 
					Options.State >= Surface.StateCondition.Num() ||
                    Surface.StateCondition[Options.State];

			if (!bSurfaceValidForThisState)
			{
				continue;
			}

			if (Surface.ResultSurfaceTask.IsValid())
			{
				// Reuse the entire surface
				return Surface.ResultSurfaceTask;
			}
			else
			{
				// Not already generated, we will generate this
				TargetSurfaces.Add( &Surface );
			}
		}

        if (TargetSurfaces.IsEmpty())
        {
            return UE::Tasks::MakeCompletedTask<FSurfaceGenerationResult>();
        }

		// Gather all modifiers that apply to this surface
		TArray<FirstPassGenerator::FModifier> Modifiers;
		constexpr bool bModifiersForBeforeOperations = false;

		// Store the data necessary to apply modifiers for the pre-normal operations stage.
		// TODO: Should we merge with currently active tags from the InOptions?
		int32 ComponentId = Options.Component ? Options.Component->Id : -1;
		GetModifiersFor(ComponentId, SurfaceNode->Tags, bModifiersForBeforeOperations, Modifiers);

		// This pass on the modifiers is only to detect errors that cannot be detected at the point they are applied.
		//CheckModifiersForSurface(*SurfaceNode, Modifiers, Options.LODIndex);
		
        // Generate the mesh
		FMeshGenerationStaticOptions MeshStaticOptions(ComponentId, Options.LODIndex);
		MeshStaticOptions.ActiveTags = SurfaceNode->Tags;
		MeshStaticOptions.State = Options.State;
		FMeshGenerationDynamicOptions MeshDynamicOptions;
		MeshDynamicOptions.bLayouts = true;

		// Normalize UVs if we're going to work with images and layouts.
		// TODO: This should come from per-layout settings!
		const bool bNormalizeUVs = false; // !SurfaceNode->Images.IsEmpty();
		MeshDynamicOptions.bNormalizeUVs = bNormalizeUVs;

		// The options depenend on the shared surface being generated, so we need to add the previous lod dependency
		FMeshOptionsTask MeshOptionsTask = LocalPipe.Launch(TEXT("MutableSurfaceMeshOptions"),
			[MeshDynamicOptions, BaseSurfaceGuid = SurfaceNode->SurfaceGuid, this]() mutable
			{	
				// This assumes that the lods are processed in order.

				UE::TUniqueLock Lock(BaseMeshOptions.Mutex);

				check(BaseSurfaceGuid.IsValid());
				const FMeshGenerationResult* BaseMeshResult = BaseMeshOptions.Map.Find(BaseSurfaceGuid);

				// If this is true, we will reuse the surface properties from a higher LOD, so we can skip the generation of material properties and images.
				if (BaseMeshResult)
				{
					// Override the layouts with the ones from the surface we share
					MeshDynamicOptions.OverrideLayouts = BaseMeshResult->GeneratedLayouts;
				}

				// Ensure UV islands remain within their main layout block to fix small displacements on vertices
				// that may cause them to fall on a different block.
				MeshDynamicOptions.bClampUVIslands = true; // bShareSurface;

				return MeshDynamicOptions;
			},
			PreviousLODTask
		);

        FMeshTask MeshTask = GenerateMesh(MeshStaticOptions, MeshOptionsTask, SurfaceNode->Mesh);

		// Apply the modifier for the post-normal operations stage.
		MeshTask = ApplyMeshModifiers(Modifiers,
			MeshStaticOptions, MeshOptionsTask,
			MeshTask, SurfaceNode->SurfaceGuid, SurfaceNode->GetMessageContext(), nullptr);

		FSurfaceTask SurfaceTask = LocalPipe.Launch(TEXT("MutableSurface"),
			[
				MeshTask,
					SurfaceNode,
					ComponentId, Options, 
					Modifiers,
					TargetSurfaces,
					// We need to call some local methods which should be fine since. 
					this
			] 
			() mutable
			{
				FMeshGenerationResult MeshResults = MeshTask.GetResult();

				// Base mesh is allowed to be missing, aggregate all layouts and operations per layout indices in the
				// generated mesh, base and extends.
				TArray<FGeneratedLayout> SurfaceReferenceLayouts;
				TArray<Ptr<ASTOp>> SurfaceLayoutOps;

				{	// This assumes that the lods are processed in order.

					UE::TUniqueLock Lock(BaseMeshOptions.Mutex);

					// Store the base mesh layouts to reuse them in higher LODs.
					check(SurfaceNode->SurfaceGuid.IsValid());
					if (!BaseMeshOptions.Map.Contains(SurfaceNode->SurfaceGuid))
					{
						FMeshGenerationResult BaseMeshResults;
						BaseMeshResults.GeneratedLayouts = MeshResults.GeneratedLayouts;
						BaseMeshOptions.Map.Add(SurfaceNode->SurfaceGuid, MoveTemp(BaseMeshResults));
					}
				}

				FGuid SharedSurfaceGuid = SurfaceNode->SurfaceGuid;

				int32 MaxLayoutNum = MeshResults.GeneratedLayouts.Num();
				for (const FMeshGenerationResult::FExtraLayouts& ExtraLayoutData : MeshResults.ExtraMeshLayouts)
				{
					MaxLayoutNum = FMath::Max(MaxLayoutNum, ExtraLayoutData.GeneratedLayouts.Num());
					SharedSurfaceGuid = FGuid::Combine(SharedSurfaceGuid, ExtraLayoutData.EditGuid);
				}

				SurfaceReferenceLayouts.SetNum(MaxLayoutNum);
				SurfaceLayoutOps.SetNum(MaxLayoutNum);

				TBitArray<> LayoutFromExtension;
				LayoutFromExtension.Init(false, MaxLayoutNum);

				// Scope for access control to shared data
				bool bIsBaseForSharedSurface = false;
				const FSharedSurfaceResultOptions::FSharedSurfaceResult* SharedSurfaceResults = nullptr;

				{
					UE::TUniqueLock Lock(SharedSurfaceOptions.Mutex);

					FSharedSurfaceResultOptions::FSharedSurfaceResultKey SharedResultKey;
					SharedResultKey.BaseSurfaceGuid = SurfaceNode->SurfaceGuid;
					SharedResultKey.CombinedGuid = SharedSurfaceGuid;

					// Do we have the surface we need to share it with?
					SharedSurfaceResults = SharedSurfaceOptions.Map.Find(SharedResultKey);
					bIsBaseForSharedSurface = SharedSurfaceResults == nullptr;

					// Add layouts form the base mesh.	
					for (int32 LayoutIndex = 0; LayoutIndex < MeshResults.GeneratedLayouts.Num(); ++LayoutIndex)
					{
						if (!MeshResults.GeneratedLayouts[LayoutIndex].Layout)
						{
							continue;
						}

						SurfaceReferenceLayouts[LayoutIndex] = MeshResults.GeneratedLayouts[LayoutIndex];

						bool bSharedHasThisLayout = SharedSurfaceResults
							&&
							SharedSurfaceResults->LayoutOps.IsValidIndex(LayoutIndex)
							&&
							SharedSurfaceResults->LayoutOps[LayoutIndex];

						if (bSharedHasThisLayout)
						{
							SurfaceLayoutOps[LayoutIndex] = SharedSurfaceResults->LayoutOps[LayoutIndex];
						}
						else
						{
							Ptr<ASTOpConstantResource> ConstantLayoutOp = new ASTOpConstantResource();
							ConstantLayoutOp->Type = EOpType::LA_CONSTANT;

							ConstantLayoutOp->SetValue(SurfaceReferenceLayouts[LayoutIndex].Layout);
							SurfaceLayoutOps[LayoutIndex] = ConstantLayoutOp;
						}
					}
				}

				// Add extra layouts. In case there is a missing reference layout, the first visited will
				// take the role.
				for (const FMeshGenerationResult::FExtraLayouts& ExtraLayoutsData : MeshResults.ExtraMeshLayouts)
				{
					if (!ExtraLayoutsData.MeshFragment)
					{
						// No mesh to add, we assume there are no layouts to add either.
						check(ExtraLayoutsData.GeneratedLayouts.IsEmpty());
						continue;
					}

					const TArray<FGeneratedLayout>& ExtraGeneratedLayouts = ExtraLayoutsData.GeneratedLayouts;
					for (int32 LayoutIndex = 0; LayoutIndex < ExtraGeneratedLayouts.Num(); ++LayoutIndex)
					{
						if (!ExtraGeneratedLayouts[LayoutIndex].Layout)
						{
							continue;
						}

						bool bLayoutSetByThisExtension = false;
						if (!SurfaceReferenceLayouts[LayoutIndex].Layout)
						{
							// This Layout slot is not set by the base surface, set it as reference.
							SurfaceReferenceLayouts[LayoutIndex] = ExtraGeneratedLayouts[LayoutIndex];
							bLayoutSetByThisExtension = true;

							LayoutFromExtension[LayoutIndex] = bLayoutSetByThisExtension;
						}

						if (SharedSurfaceResults)
						{
							if (!SurfaceLayoutOps[LayoutIndex] && bLayoutSetByThisExtension)
							{
								check(SharedSurfaceResults->LayoutOps.IsValidIndex(LayoutIndex));
								SurfaceLayoutOps[LayoutIndex] = SharedSurfaceResults->LayoutOps[LayoutIndex];
							}
						}
						else
						{
							Ptr<ASTOpConstantResource> LayoutFragmentConstantOp = new ASTOpConstantResource();
							LayoutFragmentConstantOp->Type = EOpType::LA_CONSTANT;

							LayoutFragmentConstantOp->SetValue(ExtraLayoutsData.GeneratedLayouts[LayoutIndex].Layout);

							Ptr<ASTOpLayoutMerge> LayoutMergeOp = new ASTOpLayoutMerge();
							// Base may be null if the base does not have  a mesh with a layout at LayoutIndex.
							// In that case, when applying the condition this can generate null layouts.
							LayoutMergeOp->Base = SurfaceLayoutOps[LayoutIndex];
							LayoutMergeOp->Added = LayoutFragmentConstantOp;

							if (ExtraLayoutsData.Condition)
							{
								Ptr<ASTOpConditional> ConditionalOp = new ASTOpConditional();
								ConditionalOp->type = EOpType::LA_CONDITIONAL;
								ConditionalOp->no = SurfaceLayoutOps[LayoutIndex];
								ConditionalOp->yes = LayoutMergeOp;
								ConditionalOp->condition = ExtraLayoutsData.Condition;

								SurfaceLayoutOps[LayoutIndex] = ConditionalOp;
							}
							else
							{
								SurfaceLayoutOps[LayoutIndex] = LayoutMergeOp;
							}
						}
					}
				}

				Ptr<ASTOp> LastMeshOp = MeshResults.MeshOp;

				check(SurfaceReferenceLayouts.Num() == SurfaceLayoutOps.Num());
				for (int32 LayoutIndex = 0; LayoutIndex < SurfaceReferenceLayouts.Num(); ++LayoutIndex)
				{
					if (!SurfaceReferenceLayouts[LayoutIndex].Layout)
					{
						continue;
					}

					if (SurfaceReferenceLayouts[LayoutIndex].Layout->GetLayoutPackingStrategy() == UE::Mutable::Private::EPackStrategy::Overlay)
					{
						continue;
					}

					// Add layout packing instructions
					if (!SharedSurfaceResults)
					{
						// Make sure we removed unnecessary blocks
						Ptr<ASTOpLayoutFromMesh> ExtractOp = new ASTOpLayoutFromMesh();
						ExtractOp->Mesh = LastMeshOp;
						check(LayoutIndex < 256);
						ExtractOp->LayoutIndex = uint8(LayoutIndex);

						Ptr<ASTOpLayoutRemoveBlocks> RemoveOp = new ASTOpLayoutRemoveBlocks();
						RemoveOp->Source = SurfaceLayoutOps[LayoutIndex];
						RemoveOp->ReferenceLayout = ExtractOp;
						SurfaceLayoutOps[LayoutIndex] = RemoveOp;

						// Pack uv blocks
						Ptr<ASTOpLayoutPack> LayoutPackOp = new ASTOpLayoutPack();
						LayoutPackOp->Source = SurfaceLayoutOps[LayoutIndex];
						SurfaceLayoutOps[LayoutIndex] = LayoutPackOp;
					}

					// Create the expression to apply the layout to the mesh
					{
						Ptr<ASTOpMeshApplyLayout> ApplyLayoutOp = new ASTOpMeshApplyLayout();
						ApplyLayoutOp->Mesh = LastMeshOp;
						ApplyLayoutOp->Layout = SurfaceLayoutOps[LayoutIndex];
						ApplyLayoutOp->Channel = (uint16)LayoutIndex;

						LastMeshOp = ApplyLayoutOp;
					}
				}

				MeshResults.GeneratedLayouts = MoveTemp(SurfaceReferenceLayouts);
				MeshResults.LayoutOps = MoveTemp(SurfaceLayoutOps);

				// Store in the surface for later use.
				for (FirstPassGenerator::FSurface* TargetSurface : TargetSurfaces)
				{
					TargetSurface->ResultMeshOp = LastMeshOp;
				}

				// Build a series of operations to assemble the surface
				//-------------------------------------------------------

				Ptr<ASTOp> LastMaterialOp;

				// Create the expression for each surface parameter, if we are not reusing the surface from another LOD.
				//------------------------------------------------------------------------
				if (!SharedSurfaceResults)
				{
					// Create the expression for the Material. This material will be the base material of the instance
					//------------------------------------------------------------------------
					check(SurfaceNode->Material)

					FMaterialGenerationOptions MaterialOptions;
					MaterialOptions.GenericOptions.State = Options.State;
					MaterialOptions.ComponentId = ComponentId;
					MaterialOptions.LODIndex = Options.LODIndex;
					MaterialOptions.Tags = SurfaceNode->Tags;
					MaterialOptions.Modifiers = Modifiers;
					MaterialOptions.MeshResults = MeshResults;
					MaterialOptions.LayoutFromExtension = LayoutFromExtension;

					FMaterialGenerationResult MaterialResult;
					GenerateMaterial(MaterialOptions, MaterialResult, SurfaceNode->Material);

					LastMaterialOp = MaterialResult.op;
				}
				else
				{
					LastMaterialOp = SharedSurfaceResults->LastMaterialOp;
				}

				FSurfaceGenerationResult SurfaceResult;
				SurfaceResult.MaterialOp = LastMaterialOp;
				SurfaceResult.SharedSurfaceGuid = SharedSurfaceGuid;

				// If we are going to share this surface properties, remember it.
				if (bIsBaseForSharedSurface)
				{
					{
						FSharedSurfaceResultOptions::FSharedSurfaceResult SharedResult;
						SharedResult.LayoutOps = MeshResults.LayoutOps;
						SharedResult.LastMaterialOp = LastMaterialOp;

						UE::TUniqueLock Lock(SharedSurfaceOptions.Mutex);

						FSharedSurfaceResultOptions::FSharedSurfaceResultKey SharedResultKey;
						SharedResultKey.BaseSurfaceGuid = SurfaceNode->SurfaceGuid;
						SharedResultKey.CombinedGuid = SharedSurfaceGuid;
						
						check(!SharedSurfaceOptions.Map.Contains(SharedResultKey));
						SharedSurfaceOptions.Map.Add(SharedResultKey, SharedResult);
					}

					{
						UE::TUniqueLock Lock(ExtraLayoutsOptions.Mutex);

						FExtraLayoutsResultOptions::FExtraLayoutsKey ExtraMeshLayoutKey;
						ExtraMeshLayoutKey.BaseSurfaceGuid = SurfaceNode->SurfaceGuid;
						
						for (const FMeshGenerationResult::FExtraLayouts& ExtraMeshLayout : MeshResults.ExtraMeshLayouts)
						{
							ExtraMeshLayoutKey.EditGuid = ExtraMeshLayout.EditGuid;

							// This assumes that the lods are processed in order. It checks it this way because some platforms may have empty lods at the top.
							if (!ExtraLayoutsOptions.Map.Contains(ExtraMeshLayoutKey))
							{
								ExtraLayoutsOptions.Map.Add(ExtraMeshLayoutKey, ExtraMeshLayout);
							}
						}
					}
				}

				return SurfaceResult;
			}
			, 
			UE::Tasks::Prerequisites(MeshTask, MeshOptionsTask, PreviousLODTask)
		);

		for (FirstPassGenerator::FSurface* TargetSurface : TargetSurfaces)
		{
			TargetSurface->ResultSurfaceTask = SurfaceTask;
		}

		return SurfaceTask;
	}


	FLODTask CodeGenerator::GenerateLOD(const FLODGenerationOptions& Options, const NodeLOD* InNode, FLODTask PreviousLODTask )
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateLOD);

        // Look for all surfaces that belong to this component
		TArray<int32> SurfaceIndices;
		SurfaceIndices.Reserve(Options.SurfaceIndices.Num());
		TArray<FSurfaceTask> SurfaceTasks;
		SurfaceTasks.Reserve(Options.SurfaceIndices.Num());
		for (int32 SurfaceIndex : Options.SurfaceIndices)
		{
			const FirstPassGenerator::FSurface& SurfaceData = FirstPass.Surfaces[SurfaceIndex];
			
			// Apply state conditions: only generate it if it enabled in this state
			{
				bool bEnabledInThisState = true;
				if (SurfaceData.StateCondition.Num() && Options.State >= 0)
				{
					bEnabledInThisState =
						(Options.State < SurfaceData.StateCondition.Num())
						&&
						(SurfaceData.StateCondition[Options.State]);
				}
				if (!bEnabledInThisState)
				{
					continue;
				}
			}

			FSurfaceGenerationOptions SurfaceOptions(Options);
			FSurfaceTask SurfaceTask = GenerateSurface(SurfaceOptions, SurfaceData.Node, PreviousLODTask);

			SurfaceTasks.Add(SurfaceTask);
			SurfaceIndices.Add(SurfaceIndex);
		}

		TArray<UE::Tasks::FTask> Requisites;
		Requisites.Reserve( SurfaceTasks.Num() + 1);
		Requisites.Append(SurfaceTasks);
		if (PreviousLODTask.IsValid())
		{
			Requisites.Add(PreviousLODTask);
		}

		FLODTask LODTask = LocalPipe.Launch(TEXT("MutableLOD"),
			[
				SurfaceTasks, SurfaceIndices, 
					this
			]
			() mutable
			{
				// Build a series of operations to assemble the component
				FLODGenerationResult Result;
				
				Ptr<ASTOpLODNew> LODOp = new ASTOpLODNew();

				struct FSurfaceOpsKey
				{
					uint32 SurfaceId = 0;
					Ptr<ASTOp> MeshOp;
					Ptr<ASTOp> FinalCondition;

					bool operator==(const FSurfaceOpsKey& Other) const = default;
				};

				TArray<FSurfaceOpsKey> GeneratedOps;
				GeneratedOps.Reserve(SurfaceTasks.Num());

				for (int32 SelectedSurfaceIndex=0; SelectedSurfaceIndex < SurfaceTasks.Num(); ++SelectedSurfaceIndex)
				{
					FSurfaceTask& SurfaceTask = SurfaceTasks[SelectedSurfaceIndex];
					FSurfaceGenerationResult SurfaceGenerationResult = SurfaceTask.GetResult();

					const FirstPassGenerator::FSurface& SurfaceData = FirstPass.Surfaces[SurfaceIndices[SelectedSurfaceIndex]];
					
					if (!SurfaceGenerationResult.MaterialOp || !SurfaceData.ResultMeshOp)
					{
						continue;
					}
					
					const uint32 SurfaceId = GetTypeHash(SurfaceGenerationResult.SharedSurfaceGuid);

					FSurfaceOpsKey SurfaceOpsKey(SurfaceId, SurfaceData.ResultMeshOp, SurfaceData.FinalCondition);
					if (GeneratedOps.Find(SurfaceOpsKey) != INDEX_NONE)
					{
						continue;
					}

					GeneratedOps.Add(SurfaceOpsKey);

					Ptr<ASTOp> MaterialOp = SurfaceGenerationResult.MaterialOp;
					
					if (SurfaceData.FinalCondition)
					{
						Ptr<ASTOpConditional> Op = new ASTOpConditional();
						Op->type = EOpType::MI_CONDITIONAL;
						Op->no = nullptr;
						Op->yes = MaterialOp;
						Op->condition = SurfaceData.FinalCondition;
						MaterialOp = Op;
					}
				
					FLODGenerationResult::FMaterialSlot MaterialSlot;
					MaterialSlot.Material = MaterialOp;
					MaterialSlot.SlotName = SurfaceData.Node->Name;
					MaterialSlot.Id = SurfaceId > 0 ? SurfaceId : 1; // It cannot be 0 because it is a special case for the merge operation.
					MaterialSlot.ExternalId = SurfaceData.Node->ExternalId;
					
					Result.MaterialSlots.Add(MaterialSlot);

					// Add the mesh with its condition
					Ptr<ASTOp> MeshOp;

					Ptr<ASTOpMeshSetMaterialSlotId> MeshAddSlotIdOp = new ASTOpMeshSetMaterialSlotId();
					MeshAddSlotIdOp->Mesh = SurfaceData.ResultMeshOp;
					MeshAddSlotIdOp->MaterialSlotId = MaterialSlot.Id;
					MeshOp = MeshAddSlotIdOp;

					if (SurfaceData.FinalCondition)
					{
						Ptr<ASTOpConditional> op = new ASTOpConditional();
						op->type = EOpType::ME_CONDITIONAL;
						op->no = nullptr;
						op->yes = MeshOp;
						op->condition = SurfaceData.FinalCondition;
						MeshOp = op;
					}

					LODOp->Meshes.Add(ASTChild(LODOp, MeshOp));

					Result.LODOp = LODOp;
				}

				return Result;
			},
			Requisites
		);

		return LODTask;
    }


	void CodeGenerator::GenerateObject_New(const FObjectGenerationOptions& Options, FObjectGenerationResult& OutResult, const NodeObjectNew* InNode)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeObjectNew);

		// There is always at least a null parent
		bool bIsChildObject = Options.ParentObjectNode != nullptr;

		// Add this object as current parent
		FObjectGenerationOptions ChildOptions = Options;
		ChildOptions.ParentObjectNode = InNode;

        // Parse the child objects first, which will accumulate operations in the patching lists
        for ( int32 ChildIndex=0; ChildIndex < InNode->Children.Num(); ++ChildIndex)
        {
            if ( const NodeObject* pChildNode = InNode->Children[ChildIndex].get() )
            {
				// If there are parent objects, the condition of this object depends on the condition of the parent object
				FObjectGenerationOptions ThisChildOptions = ChildOptions;
				if (!ThisChildOptions.CurrentObjectCondition)
				{
					// In case there is no group node, we generate a constant true condition
					// This condition will be overwritten by the group nodes.
					ThisChildOptions.CurrentObjectCondition = new ASTOpConstantBool(true);
				}

                // This op is ignored: everything is stored as patches to apply to the parent when it is compiled.
				FObjectGenerationResult ThisResult;
                GenerateObject(ThisChildOptions, ThisResult, pChildNode);
				OutResult.AdditionalComponents.Append(ThisResult.AdditionalComponents);
            }
        }

        // Create the expression adding all the components
		Ptr<ASTOp> LastCompOp;
		Ptr<ASTOp> PlaceholderOp;
		if (bIsChildObject)
		{
			PlaceholderOp = new ASTOpEntry;
			LastCompOp = PlaceholderOp;
		}

		// Add the components in this node
        for ( int32 ComponentIndex=0; ComponentIndex < InNode->Components.Num(); ++ComponentIndex)
        {
			const NodeComponent* ComponentNode = InNode->Components[ComponentIndex].get();
            if (ComponentNode)
            {
				FComponentGenerationOptions ComponentOptions( Options, LastCompOp );
				FGenericGenerationResult ComponentResult;
				GenerateComponent(ComponentOptions, ComponentResult, ComponentNode);
				LastCompOp = ComponentResult.Op;
            }
        }

		// If we didn't generate anything, make sure we don't use the placeholder.
		if (LastCompOp == PlaceholderOp)
		{
			LastCompOp = nullptr;
			PlaceholderOp = nullptr;
		}

		// Add the components from child objects
		FObjectGenerationResult::FAdditionalComponentKey ThisKey;
		ThisKey.ObjectNode = InNode;
		TArray<TArray<FObjectGenerationResult::FAdditionalComponentData>> MultiAdditionalComponents;
		OutResult.AdditionalComponents.MultiFind(ThisKey, MultiAdditionalComponents, true);

		if (!MultiAdditionalComponents.IsEmpty())
		{
			for (TArray<FObjectGenerationResult::FAdditionalComponentData> ThisAdditionalComponents : MultiAdditionalComponents)
			{
				for (const FObjectGenerationResult::FAdditionalComponentData& Additional : ThisAdditionalComponents)
				{
					check(Additional.PlaceholderOp);
					ASTOp::Replace(Additional.PlaceholderOp, LastCompOp);
					LastCompOp = Additional.ComponentOp;
				}
			}
		}

		// Store this chain of components for use in parent objects if necessary
		if (LastCompOp && bIsChildObject)
		{
			FObjectGenerationResult::FAdditionalComponentKey ParentKey;
			ParentKey.ObjectNode = Options.ParentObjectNode;

			FObjectGenerationResult::FAdditionalComponentData Data;
			Data.ComponentOp = LastCompOp;
			Data.PlaceholderOp = PlaceholderOp;
			OutResult.AdditionalComponents.FindOrAdd(ParentKey).Add(Data);
		}

		// Add an ASTOpAddExtensionData for each connected ExtensionData node
		for (const Ptr<NodeExtensionData>& Node : InNode->ExtensionDataNodes)
		{
			if (!Node.get())
			{
				// No node connected
				continue;
			}

			FExtensionDataGenerationResult ChildResult;
			GenerateExtensionData(ChildResult, Options, Node);

			if (!ChildResult.Op.get())
			{
				// Failed to generate anything for this node
				continue;
			}

			FConditionalExtensionDataOp& SavedOp = ConditionalExtensionDataOps.AddDefaulted_GetRef();
			SavedOp.Condition = Options.CurrentObjectCondition;
			SavedOp.ExtensionDataOp = ChildResult.Op;
		}

		Ptr<ASTOp> RootOp = LastCompOp;

		if (!Options.ParentObjectNode)
		{
			for (const FConditionalExtensionDataOp& SavedOp : ConditionalExtensionDataOps)
			{
				Ptr<ASTOpAddExtensionData> ExtensionPinOp = new ASTOpAddExtensionData();
				ExtensionPinOp->Instance = ASTChild(ExtensionPinOp, RootOp);
				ExtensionPinOp->ExtensionData = ASTChild(ExtensionPinOp, SavedOp.ExtensionDataOp);

				if (SavedOp.Condition.get())
				{
					Ptr<ASTOpConditional> ConditionOp = new ASTOpConditional();
					ConditionOp->type = EOpType::IN_CONDITIONAL;
					ConditionOp->no = RootOp;
					ConditionOp->yes = ExtensionPinOp;
					ConditionOp->condition = ASTChild(ConditionOp, SavedOp.Condition);
					
					RootOp = ConditionOp;
				}
				else
				{
					RootOp = ExtensionPinOp;
				}
			}
		}

		OutResult.Op = RootOp;
    }


	void CodeGenerator::GenerateObject_Group(const FObjectGenerationOptions& Options, FObjectGenerationResult& OutResult, const NodeObjectGroup* Node)
	{
		TArray<FString> UsedNames;

        // Parse the child objects first, which will accumulate operations in the patching lists
        for ( int32 ChildIndex=0; ChildIndex < Node->Children.Num(); ++ChildIndex)
        {
            if ( const NodeObject* pChildNode = Node->Children[ChildIndex].get() )
            {
				// Look for the child condition in the first pass
                Ptr<ASTOp> ConditionOp;
				bool bFound = false;
                for( int32 ObjectIndex = 0; !bFound && ObjectIndex<FirstPass.Objects.Num(); ObjectIndex++ )
				{
					FirstPassGenerator::FObject& Candidate = FirstPass.Objects[ObjectIndex];
					if (Candidate.Node == pChildNode)
					{
						bFound = true;
						ConditionOp = Candidate.Condition;
					}
				}

				FObjectGenerationOptions ChildOptions = Options;
				ChildOptions.CurrentObjectCondition = ConditionOp;

                // The result op is ignored: everything is stored as data to apply when the parent is compiled.
				FObjectGenerationResult ChildResult;
                GenerateObject(ChildOptions, ChildResult, pChildNode );
				OutResult.AdditionalComponents.Append(ChildResult.AdditionalComponents);

				// Check for duplicated child names
				FString ChildName = pChildNode->GetName();
				if (UsedNames.Contains(ChildName))
				{
					FString Msg = FString::Printf(TEXT("Object group has more than one children with the same name [%s]."), *ChildName );
					ErrorLog->Add(Msg, ELMT_WARNING, Node->GetMessageContext());
				}
				else
				{
					UsedNames.Add(ChildName);
				}
            }
        }
    }


    //---------------------------------------------------------------------------------------------
    Ptr<ASTOp> CodeGenerator::GenerateMissingBoolCode(const TCHAR* Where, bool Value, const void* ErrorContext )
    {
        // Log a warning
		FString Msg = FString::Printf(TEXT("Required connection not found: %s"), Where);
        ErrorLog->Add( Msg, ELMT_ERROR, ErrorContext);

        // Create a constant node
        Ptr<NodeBoolConstant> pNode = new NodeBoolConstant();
        pNode->Value = Value;

		FBoolGenerationResult ChildResult;
		FGenericGenerationOptions Options;
        GenerateBool(ChildResult,Options, pNode );
		return ChildResult.op;
    }


	//---------------------------------------------------------------------------------------------
	void CodeGenerator::GetModifiersFor(
		int32 ComponentId,
		const TArray<FString>& SurfaceTags,
		bool bModifiersForBeforeOperations,
		TArray<FirstPassGenerator::FModifier>& OutModifiers) const
	{
        MUTABLE_CPUPROFILER_SCOPE(GetModifiersFor);

		if (SurfaceTags.IsEmpty())
		{
			return;
		}

		for (const FirstPassGenerator::FModifier& Modifier: FirstPass.Modifiers)
		{
			if (!Modifier.Node)
			{
				continue;
			}

			// Correct stage?
			if (Modifier.Node->bApplyBeforeNormalOperations != bModifiersForBeforeOperations)
			{
				continue;
			}

			// Correct component?
			if (Modifier.Node->RequiredComponentId>=0 && Modifier.Node->RequiredComponentId!=ComponentId)
			{
				continue;
			}

			// Already there?
			bool bAlreadyAdded = 
				OutModifiers.FindByPredicate( [&Modifier](const FirstPassGenerator::FModifier& c) {return c.Node == Modifier.Node; })
				!= 
				nullptr;

			if (bAlreadyAdded)
			{
				continue;
			}

			// Matching tags?
			bool bApply = false;

			switch (Modifier.Node->MultipleTagsPolicy)
			{
			case EMutableMultipleTagPolicy::OnlyOneRequired:
			{
				for (const FString& Tag: Modifier.Node->RequiredTags)
				{
					if (SurfaceTags.Contains(Tag))
					{
						bApply = true;
						break;
					}
				}
				break;
			}

			case EMutableMultipleTagPolicy::AllRequired:
			{
				bApply = !Modifier.Node->RequiredTags.IsEmpty();
				for (const FString& Tag : Modifier.Node->RequiredTags)
				{
					if (!SurfaceTags.Contains(Tag))
					{
						bApply = false;
						break;
					}
				}
			}
			}

			if (bApply)
			{
				OutModifiers.Add(Modifier);
			}
		}
	}

	//---------------------------------------------------------------------------------------------
	FMeshTask CodeGenerator::ApplyMeshModifiers(
		const TArray<FirstPassGenerator::FModifier>& Modifiers,
		const FMeshGenerationStaticOptions& StaticOptions, 
		FMeshOptionsTask Options,
		FMeshTask BaseTask,
		FGuid BaseSurfaceGuid,
		const void* ErrorContext,
		const NodeMesh* OriginalMeshNode)
	{
		FMeshTask LastMeshTask = BaseTask;
		FMeshTask PreModifiersTask = BaseTask;

		int32 CurrentLOD = StaticOptions.LODIndex;
		check(CurrentLOD>=0);

		// Process mesh extend modifiers (from edit modifiers)
		int32 EditIndex = 0;
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Modifier.Node);

				bool bAffectsCurrentLOD = Edit->LODs.IsValidIndex(CurrentLOD);
				if (bAffectsCurrentLOD && Edit->LODs[CurrentLOD].MeshAdd)
				{
					Ptr<NodeMesh> pAdd = Edit->LODs[CurrentLOD].MeshAdd;

					// Store the data necessary to apply modifiers for the pre-normal operations stage.
					FMeshGenerationStaticOptions MergedMeshStaticOptions(StaticOptions);
					MergedMeshStaticOptions.ActiveTags = Edit->EnableTags; // TODO: Append to current?
					MergedMeshStaticOptions.ModifiersToIgnore.Add(Modifier);

					FMeshOptionsTask MergedMeshOptionsTask = UE::Tasks::Launch(TEXT("MutableMergedMeshOptions"),
						[Options, BaseSurfaceGuid, EditGuid = Edit->ModifierGuid, MergedMeshStaticOptions, this]() mutable
						{	
							// This assumes that the lods are processed in order.

							FMeshGenerationDynamicOptions Result = Options.GetResult();
							Result.bEnsureAllVerticesHaveLayoutBlock = false;

							UE::TUniqueLock Lock(ExtraLayoutsOptions.Mutex);
							
							check(BaseSurfaceGuid.IsValid() && EditGuid.IsValid());
							FExtraLayoutsResultOptions::FExtraLayoutsKey ExtraLayoutKey;
							ExtraLayoutKey.BaseSurfaceGuid = BaseSurfaceGuid;
							ExtraLayoutKey.EditGuid = EditGuid;

							// Reuse layouts from previous LODs if possible.
							if (const FMeshGenerationResult::FExtraLayouts* ExtraLayouts = ExtraLayoutsOptions.Map.Find(ExtraLayoutKey))
							{
								Result.OverrideLayouts = ExtraLayouts->GeneratedLayouts;
							}
							else
							{
								Result.OverrideLayouts.Empty();
							}

							return Result;
						},
						Options);

					FMeshTask AddBaseTask = GenerateMesh(MergedMeshStaticOptions, MergedMeshOptionsTask, pAdd);

					FMeshTask AddTask = UE::Tasks::Launch(TEXT("MutableMergedMeshAdd"),
						[AddBaseTask, LastMeshTask, ErrorLog=ErrorLog,Edit,ErrorContext]() mutable
						{
							FMeshGenerationResult AddResults = AddBaseTask.GetResult();
							FMeshGenerationResult BaseMeshResult = LastMeshTask.GetResult();

							// Warn about discrepancies on layout strategy between the added and the base
							int32 LayoutIndexThatHasBlocks = -1;
							{
								if (BaseMeshResult.GeneratedLayouts.Num() != AddResults.GeneratedLayouts.Num())
								{
									// When extending a mesh section the added mesh section will use the layout strategy of the base one
									const FString Msg = FString::Printf(TEXT("Extended mesh section layout count is different than the mesh being extended."));
									ErrorLog->Add(Msg, ELMT_INFO, Edit->GetMessageContext(), ErrorContext);
								}

								for (int32 LayoutIndex = 0; LayoutIndex < BaseMeshResult.GeneratedLayouts.Num(); ++LayoutIndex)
								{
									if (BaseMeshResult.GeneratedLayouts[LayoutIndex].Layout
										&&
										BaseMeshResult.GeneratedLayouts[LayoutIndex].Layout->Strategy != EPackStrategy::Overlay)
									{
										LayoutIndexThatHasBlocks = LayoutIndex;
									}

									if (BaseMeshResult.GeneratedLayouts[LayoutIndex].Layout
										&&
										AddResults.GeneratedLayouts.IsValidIndex(LayoutIndex)
										&&
										AddResults.GeneratedLayouts[LayoutIndex].Layout
										&&
										BaseMeshResult.GeneratedLayouts[LayoutIndex].Layout->Strategy != AddResults.GeneratedLayouts[LayoutIndex].Layout->Strategy)
									{
										// When extending a mesh section the added mesh section will use the layout strategy of the base one
										FString Msg = FString::Printf(TEXT("Extended mesh section layout [%d] is using a different strategy than the section being extended. The base strategy will be used."), LayoutIndex);
										ErrorLog->Add(Msg, ELMT_INFO, Edit->GetMessageContext(), ErrorContext);
									}
								}
							}

							// Add the operation to extract the relevant layout blocks if necessary
							// TODO: Handle multiple layouts defining blocks: what to extract?
							if (LayoutIndexThatHasBlocks >= 0)
							{
								Ptr<ASTOpMeshExtractLayoutBlocks> ExtractOp = new ASTOpMeshExtractLayoutBlocks();
								ExtractOp->Source = AddResults.MeshOp;
								ExtractOp->LayoutIndex = LayoutIndexThatHasBlocks;

								AddResults.MeshOp = ExtractOp;
							}

							return AddResults;
						},
						UE::Tasks::Prerequisites(AddBaseTask, LastMeshTask)
						);

					// Apply the modifiers for the post-normal operations stage to the added mesh
					FMeshGenerationStaticOptions ModifierOptions(StaticOptions);
					ModifierOptions.ActiveTags = Edit->EnableTags;
					ModifierOptions.ModifiersToIgnore.AddUnique(Modifier);

					TArray<FirstPassGenerator::FModifier> ChildModifiers;
					constexpr bool bModifiersForBeforeOperations = false;
					GetModifiersFor(StaticOptions.ComponentId, ModifierOptions.ActiveTags, bModifiersForBeforeOperations, ChildModifiers);

					FMeshTask AddWithModifiersTask = ApplyMeshModifiers(ChildModifiers, ModifierOptions, Options, AddTask, BaseSurfaceGuid, ErrorContext, nullptr);

					LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshMergeModifier"),
						[AddBaseTask, AddWithModifiersTask, LastMeshTask, Modifier, EditIndex, EditGuid = Edit->ModifierGuid]() mutable
						{
							FMeshGenerationResult AddResults = AddBaseTask.GetResult();
							FMeshGenerationResult AddFinalResults = AddWithModifiersTask.GetResult();
							FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();

							FMeshGenerationResult::FExtraLayouts Data;
							Data.GeneratedLayouts = AddResults.GeneratedLayouts;
							Data.Condition = Modifier.FinalCondition;
							Data.MeshFragment = AddFinalResults.MeshOp;
							Data.EditGuid = EditGuid;
							if (LastMeshResults.ExtraMeshLayouts.Num() <= EditIndex)
							{
								LastMeshResults.ExtraMeshLayouts.SetNum(EditIndex+1);
							}
							LastMeshResults.ExtraMeshLayouts[EditIndex] = Data;

							Ptr<ASTOpMeshMerge> MergeOp = new ASTOpMeshMerge;
							MergeOp->Base = LastMeshResults.MeshOp;
							MergeOp->Added = AddFinalResults.MeshOp;
							// will merge the meshes under the same surface
							MergeOp->NewSurfaceID = 0;

							// Condition to apply
							if (Modifier.FinalCondition)
							{
								Ptr<ASTOpConditional> ConditionalOp = new ASTOpConditional();
								ConditionalOp->type = EOpType::ME_CONDITIONAL;
								ConditionalOp->no = LastMeshResults.MeshOp;
								ConditionalOp->yes = MergeOp;
								ConditionalOp->condition = Modifier.FinalCondition;
								LastMeshResults.MeshOp = ConditionalOp;
							}
							else
							{
								LastMeshResults.MeshOp = MergeOp;
							}

							return LastMeshResults;
						},						
						UE::Tasks::Prerequisites(AddWithModifiersTask, LastMeshTask)
					);
				}

				++EditIndex;
			}

		}


		// Process mesh remove modifiers (from edit modifiers)
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Modifier.Node);

				bool bAffectsCurrentLOD = Edit->LODs.IsValidIndex(CurrentLOD);

				// Apply mesh removes from child objects "edit surface" nodes.
				// "Removes" need to come after "Adds" because some removes may refer to added meshes,
				// and not the base.
				// \TODO: Apply base removes first, and then "added meshes" removes here. It may have lower memory footprint during generation.
				if (bAffectsCurrentLOD && Edit->LODs[CurrentLOD].MeshRemove)
				{
					Ptr<NodeMesh> MeshRemove = Edit->LODs[CurrentLOD].MeshRemove;

					FMeshGenerationStaticOptions RemoveMeshStaticOptions(StaticOptions);
					RemoveMeshStaticOptions.ActiveTags = Edit->EnableTags;
					RemoveMeshStaticOptions.ModifiersToIgnore.Add(Modifier);

					FMeshGenerationDynamicOptions RemoveMeshDynamicOptions;
					RemoveMeshDynamicOptions.bLayouts = false;

					FMeshTask RemoveMeshTask = GenerateMesh(RemoveMeshStaticOptions,UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>(RemoveMeshDynamicOptions), MeshRemove);

					LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshMergeModifier"),
						[RemoveMeshTask, LastMeshTask, PreModifiersTask, Edit, Modifier]() mutable
						{
							FMeshGenerationResult RemoveResults = RemoveMeshTask.GetResult();
							FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();
							FMeshGenerationResult BaseResults = PreModifiersTask.GetResult();

							Ptr<ASTOpMeshMaskDiff> MaskOp = new ASTOpMeshMaskDiff;

							// By default, remove from the base
							Ptr<ASTOp> RemoveFrom = BaseResults.BaseMeshOp;
							MaskOp->Source = RemoveFrom;
							MaskOp->Fragment = RemoveResults.MeshOp;

							Ptr<ASTOpMeshRemoveMask> RemoveOp = new ASTOpMeshRemoveMask();
							RemoveOp->source = LastMeshResults.MeshOp;
							RemoveOp->FaceCullStrategy = Edit->FaceCullStrategy;
							RemoveOp->AddRemove(Modifier.FinalCondition, MaskOp);

							LastMeshResults.MeshOp = RemoveOp;

							return LastMeshResults;
						},
						UE::Tasks::Prerequisites(RemoveMeshTask, LastMeshTask)
					);
				}
			}
		}


		// Process mesh morph modifiers (from edit modifiers)
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Modifier.Node);
				if (Edit->MeshMorph.IsEmpty())
				{
					continue;
				}

				check(OriginalMeshNode);

				// Request morphed skeletal mesh
				TSharedPtr< TManagedPtr<FMesh> > ResolveMorphedMesh = MakeShared<TManagedPtr<FMesh>>();

				UE::Tasks::FTask TargetMeshTask;
				if (OriginalMeshNode->GetType() == NodeMeshConstant::GetStaticType())
				{
					FMesh* OriginalMesh = static_cast<const NodeMeshConstant*>(OriginalMeshNode)->Value.Get();
					check(OriginalMesh->IsReference());
					PASSTHROUGH_ID OriginalMeshID = OriginalMesh->PassthroughObject.GetId();

					bool bRunImmediatlyIfPossible = IsInGameThread();
					TargetMeshTask = CompilerOptions->OptimisationOptions.ReferencedMeshResourceProvider(
						OriginalMeshID,
						Edit->MeshMorph,
						ResolveMorphedMesh, 
						bRunImmediatlyIfPossible );
				}
				else
				{
					check(OriginalMeshNode->GetType() == NodeMeshSkeletalMeshObjectBreak::GetStaticType())
					TargetMeshTask = UE::Tasks::MakeCompletedTask<void>();
				}

				check(TargetMeshTask.IsValid())

				// Factor
				Ptr<ASTOp> FactorOp;
				if (Edit->MorphFactor)
				{
					FScalarGenerationResult ChildResult;
					GenerateScalar(ChildResult, StaticOptions, Edit->MorphFactor);
					FactorOp = ChildResult.op;
				}
				else
				{
					Ptr<NodeScalarConstant> auxNode = new NodeScalarConstant();
					auxNode->Value = 1.0f;

					FScalarGenerationResult ChildResult;
					GenerateScalar(ChildResult, StaticOptions, auxNode);
					FactorOp = ChildResult.op;
				}

				Ptr<const NodeMesh> OriginalMeshCopy = OriginalMeshNode;
				LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshMorphModifier"),
					[TargetMeshTask, LastMeshTask, PreModifiersTask, Edit, Modifier, ResolveMorphedMesh, FactorOp, CompilerOptions=CompilerOptions, OriginalMeshCopy]() mutable
					{
						FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();
						FMeshGenerationResult BaseResults = PreModifiersTask.GetResult();

						{
							Ptr<ASTOpMeshMorph> MorphOp; 
							if (OriginalMeshCopy->GetType() == NodeMeshConstant::GetStaticType())
							{
								MorphOp = new ASTOpMeshMorph();
								// Morph operation
								{
									MorphOp->Base = LastMeshResults.MeshOp;
									MorphOp->Factor = FactorOp;
									MorphOp->Name = FName(*Edit->MeshMorph);
								}
							}
							else
							{
								check(OriginalMeshCopy->GetType() == NodeMeshSkeletalMeshObjectBreak::GetStaticType());

								MorphOp = new ASTOpMeshMorph();
								
								MorphOp->Base = LastMeshResults.MeshOp;
								MorphOp->Factor = FactorOp;
								MorphOp->Name = FName(*Edit->MeshMorph);
							}

							// Condition to apply the morph
							if (Modifier.FinalCondition)
							{
								Ptr<ASTOpConditional> ConditionalOp = new ASTOpConditional();
								ConditionalOp->type = EOpType::ME_CONDITIONAL;
								ConditionalOp->no = LastMeshResults.MeshOp;
								ConditionalOp->yes = MorphOp;
								ConditionalOp->condition = Modifier.FinalCondition;
								LastMeshResults.MeshOp = ConditionalOp;
							}
							else
							{
								LastMeshResults.MeshOp = MorphOp;
							}
						}

						return LastMeshResults;
					},
					UE::Tasks::Prerequisites(TargetMeshTask, LastMeshTask)
				);
			}
		}


		// Process clip-with-mesh modifiers
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierMeshClipWithMesh::GetStaticType())
			{
				const NodeSurfaceModifierMeshClipWithMesh* TypedClipNode = static_cast<const NodeSurfaceModifierMeshClipWithMesh*>(Modifier.Node);

				FMeshGenerationDynamicOptions RemoveMeshDynamicOptions;
				RemoveMeshDynamicOptions.bLayouts = false;

				FMeshGenerationStaticOptions ClipStaticOptions(StaticOptions);
				ClipStaticOptions.ModifiersToIgnore.AddUnique(Modifier);
				ClipStaticOptions.ActiveTags.Empty();
				FMeshTask ClipMeshTask = GenerateMesh(ClipStaticOptions, UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>(RemoveMeshDynamicOptions), TypedClipNode->ClipMesh);

				LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshMergeModifier"),
					[ClipMeshTask, LastMeshTask, PreModifiersTask, Modifier, TypedClipNode, ErrorLog=ErrorLog, ErrorContext]() mutable
					{
						FMeshGenerationResult ClipResults = ClipMeshTask.GetResult();
						FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();
						FMeshGenerationResult BaseResults = PreModifiersTask.GetResult();

						if (!ClipResults.MeshOp)
						{
							ErrorLog->Add("Clip mesh has not been generated", ELMT_ERROR, ErrorContext);
							return LastMeshResults;
						}

						Ptr<ASTOpMeshMaskClipMesh> MaskOp = new ASTOpMeshMaskClipMesh();
						MaskOp->source = BaseResults.MeshOp;
						MaskOp->clip = ClipResults.MeshOp;

						Ptr<ASTOpMeshRemoveMask> RemoveOp = new ASTOpMeshRemoveMask();
						RemoveOp->source = LastMeshResults.MeshOp;
						RemoveOp->FaceCullStrategy = TypedClipNode->FaceCullStrategy;
						RemoveOp->AddRemove(Modifier.FinalCondition, MaskOp);

						LastMeshResults.MeshOp = RemoveOp;

						return LastMeshResults;
					},
					UE::Tasks::Prerequisites(ClipMeshTask, LastMeshTask)
				);
			}
		}

		// Process clip-with-mask modifiers
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierMeshClipWithUVMask::GetStaticType())
			{
				// Create a constant mesh with the original UVs required by this modifier.
				// TODO: Optimize, by caching.
				// TODO: Optimize by formatting and keeping only UVs
				check(OriginalMeshNode);

				TSharedPtr< TManagedPtr<FMesh> > ResolveOriginalMesh = MakeShared<TManagedPtr<FMesh>>();

				UE::Tasks::FTask TargetMeshTask;
				if (OriginalMeshNode->GetType() == NodeMeshConstant::GetStaticType())
				{
					FMesh* OriginalMesh = static_cast<const NodeMeshConstant*>(OriginalMeshNode)->Value.Get();
					check(OriginalMesh->IsReference());
					PASSTHROUGH_ID OriginalMeshID = OriginalMesh->PassthroughObject.GetId();

					bool bRunImmediatlyIfPossible = IsInGameThread();
					FString NoMorph;
					TargetMeshTask = CompilerOptions->OptimisationOptions.ReferencedMeshResourceProvider(
						OriginalMeshID,
						NoMorph,
						ResolveOriginalMesh,
						bRunImmediatlyIfPossible);
				}
				else
				{
					check(OriginalMeshNode->GetType() == NodeMeshSkeletalMeshObjectBreak::GetStaticType())
					TargetMeshTask = UE::Tasks::MakeCompletedTask<void>();
				}

				check(TargetMeshTask.IsValid())

				const NodeSurfaceModifierMeshClipWithUVMask* TypedClipNode = static_cast<const NodeSurfaceModifierMeshClipWithUVMask*>(Modifier.Node);

				Ptr<ASTOp> MaskImage;
				TManagedPtr<const FLayout> Layout;
				if (TypedClipNode->ClipMask)
				{
					FImageGenerationOptions ClipOptions(StaticOptions.ComponentId, StaticOptions.LODIndex);
					ClipOptions.ImageLayoutStrategy = CompilerOptions::TextureLayoutStrategy::None;
					ClipOptions.LayoutBlockId = FLayoutBlock::InvalidBlockId;
					ClipOptions.GenericOptions.State = StaticOptions.State;

					FImageGenerationResult ClipMaskResult;
					GenerateImage(ClipOptions, ClipMaskResult, TypedClipNode->ClipMask);

					// It could be IF_L_UBIT, but since this should be optimized out at compile time, leave the most cpu efficient.
					MaskImage = GenerateImageFormat(ClipMaskResult.op, UE::Mutable::Private::EImageFormat::L_UByte);
				}
				else if (TypedClipNode->ClipLayout)
				{
					// Generate the layout with blocks to extract
					Layout = GenerateLayout(TypedClipNode->ClipLayout, 0);
				}

				Ptr<const NodeMesh> OriginalMeshCopy = OriginalMeshNode;
				LastMeshTask = UE::Tasks::Launch(TEXT("MutableModifier"),
					[Layout, MaskImage, ResolveOriginalMesh, LastMeshTask, PreModifiersTask, Modifier, TypedClipNode, CompilerOptions = CompilerOptions, ErrorLog = ErrorLog, ErrorContext, OriginalMeshCopy]() mutable
					{
						FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();
						FMeshGenerationResult BaseResults = PreModifiersTask.GetResult();
						
						if (OriginalMeshCopy->GetType() == NodeMeshConstant::GetStaticType())
						{
							TManagedPtr<FMesh> TargetMesh = *ResolveOriginalMesh;
							if (!TargetMesh)
							{
								return FMeshGenerationResult();
							}

							Ptr<ASTOpConstantResource> UVMeshOp = new ASTOpConstantResource;
							UVMeshOp->Type = EOpType::ME_CONSTANT;
							UVMeshOp->SetValue(TargetMesh->Clone());
							UVMeshOp->SourceDataDescriptor = static_cast<const NodeMeshConstant*>(OriginalMeshCopy.get())->SourceDataDescriptor;

							Ptr<ASTOpMeshMaskClipUVMask> MeshMaskAt = new ASTOpMeshMaskClipUVMask();
							MeshMaskAt->Source = BaseResults.BaseMeshOp;
							MeshMaskAt->UVSource = UVMeshOp;
							MeshMaskAt->LayoutIndex = TypedClipNode->LayoutIndex;

							if (TypedClipNode->ClipMask)
							{
								MeshMaskAt->MaskImage = MaskImage;
								if (!MeshMaskAt->MaskImage)
								{
									ErrorLog->Add("Clip UV mask has not been generated", ELMT_ERROR, ErrorContext);
									return LastMeshResults;
								}
							}

							else if (TypedClipNode->ClipLayout)
							{
								Ptr<ASTOpConstantResource> LayoutOp = new ASTOpConstantResource();
								LayoutOp->Type = EOpType::LA_CONSTANT;
								LayoutOp->SetValue(Layout);
								MeshMaskAt->MaskLayout = LayoutOp;
							}

							else
							{
								// No mask or layout specified to clip. Don't clip anything.
								return LastMeshResults;
							}

							if (MeshMaskAt)
							{
								Ptr<ASTOpMeshRemoveMask> RemoveOp = new ASTOpMeshRemoveMask();
								RemoveOp->source = LastMeshResults.MeshOp;
								RemoveOp->FaceCullStrategy = TypedClipNode->FaceCullStrategy;
								RemoveOp->AddRemove(Modifier.FinalCondition, MeshMaskAt);

								LastMeshResults.MeshOp = RemoveOp;
							}

							return LastMeshResults;
						}
						else
						{
							ErrorLog->Add("NodeModifierMeshClipWithUVMask is not supported with mesh parameters, ignoring operation.", ELMT_WARNING, ErrorContext);
							return LastMeshResults;
						}
					},
					UE::Tasks::Prerequisites(TargetMeshTask, LastMeshTask)
				);
			}
		}

		// Process clip-morph-plane modifiers
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierMeshClipMorphPlane::GetStaticType())
			{
				const NodeSurfaceModifierMeshClipMorphPlane* TypedNode = static_cast<const NodeSurfaceModifierMeshClipMorphPlane*>(Modifier.Node);

				LastMeshTask = UE::Tasks::Launch(TEXT("MutableModifier"),
					[LastMeshTask, PreModifiersTask, Modifier, TypedNode]() mutable
					{
						FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();
						FMeshGenerationResult BaseResults = PreModifiersTask.GetResult();

						Ptr<ASTOpMeshClipMorphPlane> ClipOp = new ASTOpMeshClipMorphPlane();
						ClipOp->Source = LastMeshResults.MeshOp;
						ClipOp->FaceCullStrategy = TypedNode->Parameters.FaceCullStrategy;

						// Morph to an ellipse
						{
							FShape MorphShape;
							MorphShape.type = (uint8)FShape::Type::Ellipse;
							MorphShape.position = TypedNode->Parameters.Origin;
							MorphShape.up = TypedNode->Parameters.Normal;
							// TODO: Move rotation to ellipse rotation reference base instead of passing it directly
							MorphShape.size = FVector3f(TypedNode->Parameters.Radius1, TypedNode->Parameters.Radius2, TypedNode->Parameters.Rotation);

							// Generate a "side" vector.
							// \todo: make generic and move to the vector class
							{
								// Generate vector perpendicular to normal for ellipse rotation reference base
								FVector3f AuxBase(0.f, 1.f, 0.f);

								if (FMath::Abs(FVector3f::DotProduct(TypedNode->Parameters.Normal, AuxBase)) > 0.95f)
								{
									AuxBase = FVector3f(0.f, 0.f, 1.f);
								}

								MorphShape.side = FVector3f::CrossProduct(TypedNode->Parameters.Normal, AuxBase);
							}
							ClipOp->MorphShape = MorphShape;
						}

						// Selection box
						ClipOp->VertexSelectionType = TypedNode->Parameters.VertexSelectionType;
						if (ClipOp->VertexSelectionType == EClipVertexSelectionType::Shape)
						{
							FShape SelectionShape;
							SelectionShape.type = (uint8)FShape::Type::AABox;
							SelectionShape.position = TypedNode->Parameters.SelectionBoxOrigin;
							SelectionShape.size = TypedNode->Parameters.SelectionBoxRadius;
							ClipOp->SelectionShape = SelectionShape;
						}
						else if (ClipOp->VertexSelectionType == EClipVertexSelectionType::BoneHierarchy)
						{
							ClipOp->VertexSelectionBone = TypedNode->Parameters.VertexSelectionBone;
							ClipOp->VertexSelectionBoneMaxRadius = TypedNode->Parameters.MaxEffectRadius;
						}

						ClipOp->Dist = TypedNode->Parameters.DistanceToPlane;
						ClipOp->Factor = TypedNode->Parameters.LinearityFactor;

						Ptr<ASTOpConditional> ConditionalOp = new ASTOpConditional();
						ConditionalOp->type = EOpType::ME_CONDITIONAL;
						ConditionalOp->no = LastMeshResults.MeshOp;
						ConditionalOp->yes = ClipOp;
						ConditionalOp->condition = Modifier.FinalCondition;

						LastMeshResults.MeshOp = ConditionalOp;

						return LastMeshResults;
					},
					UE::Tasks::Prerequisites(LastMeshTask)
				);

			}
		}

    	// Process clip deform modifiers.
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType() == NodeSurfaceModifierMeshClipDeform::GetStaticType())
			{
				const NodeSurfaceModifierMeshClipDeform* TypedClipNode = static_cast<const NodeSurfaceModifierMeshClipDeform*>(Modifier.Node);

				FMeshGenerationStaticOptions ClipStaticOptions(StaticOptions);
				ClipStaticOptions.ActiveTags.Empty();
				ClipStaticOptions.ModifiersToIgnore.AddUnique(Modifier);
				FMeshGenerationDynamicOptions ClipMeshDynamicOptions;
				ClipMeshDynamicOptions.bLayouts = false;
				FMeshTask ClipMeshTask = GenerateMesh(ClipStaticOptions, UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>(ClipMeshDynamicOptions), TypedClipNode->ClipMesh);

				LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshMergeModifier"),
					[ClipMeshTask, LastMeshTask, Modifier, TypedClipNode, ErrorLog = ErrorLog, ErrorContext]() mutable
					{
						FMeshGenerationResult ClipResults = ClipMeshTask.GetResult();
						FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();

						Ptr<ASTOpMeshBindShape>  BindOp = new ASTOpMeshBindShape();
						BindOp->Mesh = LastMeshResults.MeshOp;
						BindOp->Shape = ClipResults.MeshOp;
						BindOp->BindingMethod = static_cast<uint32>(TypedClipNode->BindingMethod);

						Ptr<ASTOpMeshClipDeform> ClipOp = new ASTOpMeshClipDeform();
						ClipOp->FaceCullStrategy = TypedClipNode->FaceCullStrategy;
						ClipOp->ClipShape = ClipResults.MeshOp;
						ClipOp->Mesh = BindOp;

						if (!ClipOp->ClipShape)
						{
							ErrorLog->Add("Clip shape mesh has not been generated", ELMT_ERROR, ErrorContext);
							return LastMeshResults;
						}

						Ptr<ASTOpConditional> Op = new ASTOpConditional();
						Op->type = EOpType::ME_CONDITIONAL;
						Op->no = LastMeshResults.MeshOp;
						Op->yes = ClipOp;
						Op->condition = Modifier.FinalCondition;

						LastMeshResults.MeshOp = Op;

						return LastMeshResults;
					},
					UE::Tasks::Prerequisites(ClipMeshTask, LastMeshTask)
				);
			}
		}
		
		// Process transform mesh within mesh modifiers.
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (StaticOptions.ModifiersToIgnore.Contains(Modifier))
			{
				// Prevent recursion.
				continue;
			}

			if (Modifier.Node->GetType()== NodeSurfaceModifierMeshTransformInMesh::GetStaticType())
			{
				const NodeSurfaceModifierMeshTransformInMesh* TypedTransformNode = static_cast<const NodeSurfaceModifierMeshTransformInMesh*>(Modifier.Node);

				// If a matrix node is not connected, the op won't do anything, so let's not create it at all.
				if (!TypedTransformNode->MatrixNode)
				{
					ErrorLog->Add("Matrix has not been generated", ELMT_ERROR, ErrorContext);
					continue;
				}
				else if (!TypedTransformNode->BoundingMesh)
				{
					ErrorLog->Add("Bounding mesh has not been generated", ELMT_ERROR, ErrorContext);
					continue;
				}
				
				// Transform matrix.
				Ptr<ASTOp> MatrixOp;
				FMatrixGenerationResult ChildResult;
				GenerateMatrix(ChildResult, StaticOptions, TypedTransformNode->MatrixNode);
				MatrixOp = ChildResult.op;

				// Bounding mesh
				FMeshGenerationStaticOptions BoundingMeshStaticOptions(StaticOptions);
				BoundingMeshStaticOptions.ActiveTags.Empty();
				BoundingMeshStaticOptions.ModifiersToIgnore.AddUnique(Modifier);
				FMeshGenerationDynamicOptions BoundingMeshDynamicOptions;
				BoundingMeshDynamicOptions.bLayouts = false;
				FMeshTask BoundingMeshTask = GenerateMesh(BoundingMeshStaticOptions, UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>(BoundingMeshDynamicOptions), TypedTransformNode->BoundingMesh);

				LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshTransformWithBoundingMeshModifier"),
					[BoundingMeshTask, LastMeshTask, Modifier, MatrixOp, ErrorLog = ErrorLog, ErrorContext]() mutable
					{
						FMeshGenerationResult BoundingResults = BoundingMeshTask.GetResult();
						FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();

						Ptr<ASTOpMeshTransformWithBoundingMesh> TransformOp = new ASTOpMeshTransformWithBoundingMesh();
						TransformOp->source = LastMeshResults.MeshOp;
						TransformOp->matrix = MatrixOp;
						TransformOp->boundingMesh = BoundingResults.MeshOp;

						if (TransformOp->boundingMesh)
						{
							ASTOp::EClosedMeshTest IsClosed = TransformOp->boundingMesh->IsClosedMesh();
							if (IsClosed == ASTOp::EClosedMeshTest::No)
							{
								ErrorLog->Add("Mesh used for the mesh transformation is not closed.", ELMT_WARNING, ErrorContext);
							}
						}

						// Condition to apply the transform op
						if (Modifier.FinalCondition)
						{
							Ptr<ASTOpConditional> ConditionalOp = new ASTOpConditional();
							ConditionalOp->type = EOpType::ME_CONDITIONAL;
							ConditionalOp->no = LastMeshResults.MeshOp;
							ConditionalOp->yes = TransformOp;
							ConditionalOp->condition = Modifier.FinalCondition;
							LastMeshResults.MeshOp = ConditionalOp;
						}
						else
						{
							LastMeshResults.MeshOp = TransformOp;
						}

						return LastMeshResults;
					},
					UE::Tasks::Prerequisites(BoundingMeshTask, LastMeshTask)
				);

			}
			else if (Modifier.Node->GetType() == NodeSurfaceModifierMeshTransformWithBone::GetStaticType())
			{
				const NodeSurfaceModifierMeshTransformWithBone* TypedTransformNode = static_cast<const NodeSurfaceModifierMeshTransformWithBone*>(Modifier.Node);

				// If a matrix node is not connected, the op won't do anything, so let's not create it at all.
				if (TypedTransformNode->MatrixNode)
				{
					// Transform matrix.
					Ptr<ASTOp> MatrixOp;
					FMatrixGenerationResult ChildResult;
					GenerateMatrix(ChildResult, StaticOptions, TypedTransformNode->MatrixNode);
					MatrixOp = ChildResult.op;


					LastMeshTask = UE::Tasks::Launch(TEXT("MutableMeshTransformWithBoneModifier"),
						[LastMeshTask, Modifier, MatrixOp, BoneName = TypedTransformNode->BoneName, ThresholdFactor = TypedTransformNode->ThresholdFactor, ErrorLog = ErrorLog, ErrorContext]() mutable
						{
							FMeshGenerationResult LastMeshResults = LastMeshTask.GetResult();

							Ptr<ASTOpMeshTransformWithBone> TransformOp = new ASTOpMeshTransformWithBone();
							TransformOp->SourceMesh = LastMeshResults.MeshOp;
							TransformOp->Matrix = MatrixOp;
							TransformOp->BoneName = BoneName;
							TransformOp->ThresholdFactor = ThresholdFactor;

							// Condition to apply the transform op
							if (Modifier.FinalCondition)
							{
								Ptr<ASTOpConditional> ConditionalOp = new ASTOpConditional();
								ConditionalOp->type = EOpType::ME_CONDITIONAL;
								ConditionalOp->no = LastMeshResults.MeshOp;
								ConditionalOp->yes = TransformOp;
								ConditionalOp->condition = Modifier.FinalCondition;
								LastMeshResults.MeshOp = ConditionalOp;
							}
							else
							{
								LastMeshResults.MeshOp = TransformOp;
							}

							return LastMeshResults;
						},
						UE::Tasks::Prerequisites(LastMeshTask)
					);
				}
			}
		}
		

		return LastMeshTask;
	}


	Ptr<ASTOp> CodeGenerator::ApplyImageBlockModifiers(
		const TArray<FirstPassGenerator::FModifier>& Modifiers,
		const FImageGenerationOptions& Options, Ptr<ASTOp> BaseImageOp, 
		const FParameterKey& ImageParameterKey,
		FIntPoint GridSize,
		const FLayoutBlockDesc& LayoutBlockDesc,
		box< FIntVector2 > RectInCells,
		const void* ErrorContext)
	{
		Ptr<ASTOp> LastImageOp = BaseImageOp;
		int32 CurrentLOD = Options.LODIndex;

		// Process patch image modifiers (from edit modifiers)
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (Modifier.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Modifier.Node);

				bool bAffectsCurrentLOD = Edit->LODs.IsValidIndex(CurrentLOD);

				if (!bAffectsCurrentLOD)
				{
					continue;
				}

				const NodeSurfaceModifierSurfaceEdit::FTexture* MatchingEdit = Edit->LODs[CurrentLOD].Textures.FindByPredicate(
					[&](const NodeSurfaceModifierSurfaceEdit::FTexture& Candidate)
					{
						return (Candidate.MaterialParameterKey == ImageParameterKey);
					});

				if ( !MatchingEdit)
				{
					continue;
				}

				if (MatchingEdit->PatchImage.get())
				{
					// Does the current block need to be patched? Find out by building a mask.
					TManagedPtr<FImage> PatchMask = GenerateImageBlockPatchMask(*MatchingEdit, GridSize, LayoutBlockDesc.BlockPixelsX, LayoutBlockDesc.BlockPixelsY, RectInCells);

					if (PatchMask)
					{
						LastImageOp = GenerateImageBlockPatch(LastImageOp, *MatchingEdit, PatchMask, Modifier.FinalCondition, Options);
					}
				}
			}

			else
			{
				// This modifier doesn't affect the per-block image operations.
			}

		}

		return LastImageOp;
	}


	void CodeGenerator::UpdateLayoutBlockDesc(CodeGenerator::FLayoutBlockDesc& Out, FImageDesc BlockDesc, FIntVector2 LayoutCellSize)
	{
		if (LayoutCellSize.X > 0 && LayoutCellSize.Y > 0)
		{
			if (Out.BlockPixelsX == 0)
			{
				Out.BlockPixelsX = FMath::Max(1, BlockDesc.m_size[0] / LayoutCellSize[0]);
				Out.BlockPixelsY = FMath::Max(1, BlockDesc.m_size[1] / LayoutCellSize[1]);
			}

			if (Out.FinalFormat == EImageFormat::None)
			{
				Out.FinalFormat = BlockDesc.m_format;
				Out.bBlocksHaveMips = BlockDesc.m_lods > 1;
			}
		}
	};


	Ptr<ASTOp> CodeGenerator::ApplyImageExtendModifiers(
		const TArray<FirstPassGenerator::FModifier>& Modifiers,
		const FMeshGenerationStaticOptions& Options,
		const FMeshGenerationResult& BaseMeshResults,
		Ptr<ASTOp> BaseImageOp, 
		CompilerOptions::TextureLayoutStrategy ImageLayoutStrategy,
		int32 LayoutIndex, 
		const FParameterKey& ImageParameterKey,
		FIntPoint GridSize, 
		CodeGenerator::FLayoutBlockDesc& InOutLayoutBlockDesc,
		const void* ModifiedNodeErrorContext)
	{
		Ptr<ASTOp> LastImageOp = BaseImageOp;

		int32 CurrentLOD = Options.LODIndex;
		check(CurrentLOD >= 0);

		// Process mesh extend modifiers (from edit modifiers)
		int32 EditIndex = 0;
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (Modifier.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Modifier.Node);

				int32 ThisEditIndex = EditIndex;
				++EditIndex;

				bool bAffectsCurrentLOD = Edit->LODs.IsValidIndex(CurrentLOD);
				if (!bAffectsCurrentLOD)
				{
					continue;
				}

				const NodeSurfaceModifierSurfaceEdit::FTexture* MatchingEdit = Edit->LODs[CurrentLOD].Textures.FindByPredicate(
					[&](const NodeSurfaceModifierSurfaceEdit::FTexture& Candidate)
					{
						return (Candidate.MaterialParameterKey == ImageParameterKey);
					});

				if (!MatchingEdit || (MatchingEdit && !MatchingEdit->Extend) )
				{
					if (Edit->LODs[CurrentLOD].MeshAdd)
					{
						// When extending a mesh section it is mandatory to provide textures for all section textures handled by Mutable.
						FString Msg = FString::Printf(TEXT("Required texture [%s] is missing when trying to extend a mesh section."), *ImageParameterKey.ParameterName.ToString());

						if (ImageParameterKey.LayerIndex >= 0)
						{
							Msg = FString::Printf(TEXT("Required texture [%s - Layer %d] is missing when trying to extend a mesh section."), *ImageParameterKey.ParameterName.ToString(), ImageParameterKey.LayerIndex);
						}

						ErrorLog->Add(Msg, ELMT_INFO, Edit->GetMessageContext(), ModifiedNodeErrorContext);
					}

					continue;
				}

				if (!BaseMeshResults.ExtraMeshLayouts.IsValidIndex(ThisEditIndex))
				{
					ErrorLog->Add(TEXT("Trying to extend a layout that doesn't exist."), ELMT_INFO, Edit->GetMessageContext(), ModifiedNodeErrorContext);
					continue;
				}

				const TArray<FGeneratedLayout>& ExtraLayouts = BaseMeshResults.ExtraMeshLayouts[ThisEditIndex].GeneratedLayouts;

				if (LayoutIndex >= ExtraLayouts.Num() || !ExtraLayouts[LayoutIndex].Layout)
				{
					ErrorLog->Add(TEXT("Trying to extend a layout that doesn't exist."), ELMT_INFO, Edit->GetMessageContext(), ModifiedNodeErrorContext);
					continue;
				}

				TManagedPtr<const FLayout> ExtendLayout = ExtraLayouts[LayoutIndex].Layout;

				Ptr<ASTOp> LastBase = LastImageOp;

				for (int32 BlockIndex = 0; BlockIndex < ExtendLayout->GetBlockCount(); ++BlockIndex)
				{
					// Generate the image block
					FImageGenerationOptions ImageOptions(Options.ComponentId,Options.LODIndex);
					ImageOptions.GenericOptions.State = Options.State;
					ImageOptions.ImageLayoutStrategy = ImageLayoutStrategy;
					ImageOptions.GenericOptions.ActiveTags = Edit->EnableTags; // TODO: Merge with current tags?
					ImageOptions.RectSize = { 0,0 };
					ImageOptions.LayoutToApply = ExtendLayout;
					ImageOptions.LayoutBlockId = ExtendLayout->Blocks[BlockIndex].Id;
					FImageGenerationResult ExtendResult;
					GenerateImage(ImageOptions, ExtendResult, MatchingEdit->Extend);
					Ptr<ASTOp> FragmentOp = ExtendResult.op;

					// Block in layout grid units
					box< FIntVector2 > RectInCells;
					RectInCells.min = ExtendLayout->Blocks[BlockIndex].Min;
					RectInCells.size = ExtendLayout->Blocks[BlockIndex].Size;

					FImageDesc ExtendDesc = FragmentOp->GetImageDesc();

					// If we don't know the size of a layout block in pixels, calculate it
					UpdateLayoutBlockDesc(InOutLayoutBlockDesc, ExtendDesc, RectInCells.size);

					if (CVarAllowMulticompose.GetValueOnAnyThread() && ExtendResult.bCanMulticompose)
					{
						LastBase = GenerateImageMulticompose(LastBase, BaseMeshResults.LayoutOps[LayoutIndex], ImageOptions, MatchingEdit->Extend, ExtendLayout, InOutLayoutBlockDesc);
						break;
					}

					// Adjust the format and size of the block to be added
					// Actually don't do it, it will be propagated from the top format operation.
					//FragmentOp = GenerateImageFormat(FragmentOp, FinalImageFormat);

					UE::Math::TIntVector2<int32> ExpectedSize;
					ExpectedSize[0] = InOutLayoutBlockDesc.BlockPixelsX * RectInCells.size[0];
					ExpectedSize[1] = InOutLayoutBlockDesc.BlockPixelsY * RectInCells.size[1];
					FragmentOp = GenerateImageSize(FragmentOp, ExpectedSize);

					// Compose operation
					Ptr<ASTOpImageCompose> ComposeOp = new ASTOpImageCompose();
					ComposeOp->Layout = BaseMeshResults.LayoutOps[LayoutIndex];
					ComposeOp->Base = LastBase;
					ComposeOp->BlockImage = FragmentOp;

					// Set the absolute block index.
					check(ExtendLayout->Blocks[BlockIndex].Id != FLayoutBlock::InvalidBlockId);
					ComposeOp->BlockId = ExtendLayout->Blocks[BlockIndex].Id;

					LastBase = ComposeOp;
				}

				// Condition to enable this image extension
				if (Modifier.FinalCondition)
				{
					Ptr<ASTOpConditional> Op = new ASTOpConditional();
					Op->type = EOpType::IM_CONDITIONAL;
					Op->no = LastImageOp;
					Op->yes = LastBase;
					Op->condition = Modifier.FinalCondition;
					LastImageOp = Op;
				}
				else
				{
					LastImageOp = LastBase;
				}
			}
		}

		return LastImageOp;
	}


	void CodeGenerator::CheckModifiersForSurface(const NodeSurfaceNew& Node, const TArray<FirstPassGenerator::FModifier>& Modifiers, int32 LODIndex )
	{
		int32 CurrentLOD = LODIndex;
		check(CurrentLOD >= 0);

		for (const FirstPassGenerator::FModifier& Mod : Modifiers)
		{
			// A mistake in the surface edit modifier usually results in no change visible. Try to detect it.
			if (Mod.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType() && Node.Material->GetType() == NodeMaterialModify::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Mod.Node);
				const NodeMaterialModify* Material = static_cast<const NodeMaterialModify*>(Node.Material.get());

				// If this edit is not affected by the current LOD, then ignore the check.
				// If both arrays are empty means that there are no textures to check
				// If there is a MeshRemove, this edit does not affect textures
				if (!Edit->LODs.IsValidIndex(CurrentLOD) || Edit->LODs[CurrentLOD].MeshRemove || (Material->ImageParameters.IsEmpty() && Edit->LODs[CurrentLOD].Textures.IsEmpty()))
				{
					continue;
				}

				bool bAtLeastSomeTexture = false;

				for (const NodeSurfaceModifierSurfaceEdit::FTexture& Texture : Edit->LODs[CurrentLOD].Textures)
				{
					const FImageParameterData* MatchingEdit = Material->ImageParameters.Find(Texture.MaterialParameterKey);

					if (!MatchingEdit || MatchingEdit->bIsPassthrough)
					{
						ErrorLog->Add(FString::Printf(TEXT("Trying to Modify the Passthrough texture [%s]."), *Texture.MaterialParameterKey.ParameterName.ToString()), ELMT_WARNING, Edit->GetMessageContext(), Node.GetMessageContext());
					}
				}
			}
		}
	}


	bool CodeGenerator::HasImageBlockModifiers(const FParameterKey& ImageKey, const TArray<FirstPassGenerator::FModifier>& Modifiers, int32 LODIndex) const
	{
		check(LODIndex >= 0);

		// Process patch image modifiers (from edit modifiers)
		for (const FirstPassGenerator::FModifier& Modifier : Modifiers)
		{
			if (Modifier.Node->GetType() == NodeSurfaceModifierSurfaceEdit::GetStaticType())
			{
				const NodeSurfaceModifierSurfaceEdit* Edit = static_cast<const NodeSurfaceModifierSurfaceEdit*>(Modifier.Node);

				bool bAffectsLOD = Edit->LODs.IsValidIndex(LODIndex);

				if (!bAffectsLOD)
				{
					continue;
				}

				const NodeSurfaceModifierSurfaceEdit::FTexture* MatchingEdit = Edit->LODs[LODIndex].Textures.FindByPredicate(
					[&](const NodeSurfaceModifierSurfaceEdit::FTexture& Candidate)
					{
						return (Candidate.MaterialParameterKey == ImageKey);
					});

				if (!MatchingEdit)
				{
					continue;
				}

				if (MatchingEdit->PatchImage.get())
				{
					return true;
				}
			}
		}

		return false;
	}

	Ptr<ASTOpImagePixelFormat> CodeGenerator::GenerateFormatExpression(const NodeImageFormat* FormatNode, Ptr<ASTOp> SubExpression, EYesNoQuery CanNormalLAEncode)
	{
		Ptr<ASTOpImagePixelFormat> FormatOp = new ASTOpImagePixelFormat();

		FormatOp->Source = SubExpression;
		FormatOp->Format = FormatNode->Format;
		FormatOp->FormatIfAlpha = FormatNode->FormatIfAlpha;

		// Parameter formats don't need to apply the swizzle. For parameters the data will be cooked using
		// the editor encoder which will do the swizzle if needed.
		// FIXME: This is fragile and does not support sub expressions with a mix of parameters and constants.
		if (FormatNode->bNeedsNormalmapLAEncoding)
		{
			if (CanNormalLAEncode == EYesNoQuery::Yes)
			{
				Ptr<ASTOpImageSwizzle> NormalLASwizzle = new ASTOpImageSwizzle();
				NormalLASwizzle->Sources[0] = SubExpression;
				NormalLASwizzle->Sources[1] = SubExpression;
				NormalLASwizzle->Sources[2] = SubExpression;
				NormalLASwizzle->Sources[3] = SubExpression;
				NormalLASwizzle->SourceChannels[0] = 0;
				NormalLASwizzle->SourceChannels[1] = 0;
				NormalLASwizzle->SourceChannels[2] = 0;
				NormalLASwizzle->SourceChannels[3] = 1;

				NormalLASwizzle->Format = EImageFormat::RGBA_UByte;

				FormatOp->Source = NormalLASwizzle;
			}
			else if (CanNormalLAEncode == EYesNoQuery::Mixed)
			{
				UE_LOGF(LogMutableCore, Log, " While generating format expression, the format needs needs Normal LA encoding but the subexpression has mixed CanNormaLAEncode, swizzle not applied.");
			}
		}

		return FormatOp;
	}
}
