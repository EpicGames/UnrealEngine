// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ASTOpExternal.h"
#include "Containers/Map.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuT/AST.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/Compiler.h"
#include "MuT/ErrorLog.h"
#include "MuT/Node.h"
#include "MuT/NodeBool.h"
#include "MuT/NodeColorExternal.h"
#include "MuT/NodeColorSampleImage.h"
#include "MuT/NodeComponentNew.h"
#include "MuT/NodeComponentSwitch.h"
#include "MuT/NodeExternal.h"
#include "MuT/NodeExtensionData.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeLOD.h"
#include "MuT/NodeLayout.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeObjectGroup.h"
#include "MuT/NodeObjectNew.h"
#include "MuT/NodeProjector.h"
#include "MuT/NodeString.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeMaterialModify.h"
#include "MuT/NodeMeshClipWithMesh.h"
#include "MuT/NodeSurfaceModifierSurfaceEdit.h"
#include "MuT/Table.h"
#include "MuT/TablePrivate.h"
#include "Templates/TypeHash.h"
#include "Tasks/Pipe.h"

namespace UE::Mutable::Private
{
	class NodeMaterialSkeletalMeshBreak;
	class NodeSkeletalMeshClipWithSkeletalMesh;
	class NodeImageObject;
	class NodeSkeletalMeshObjectConvert;
	class NodeSkeletalMeshObjectSwitch;
	class NodeSkeletalMeshConvert;
	class NodeExternalSwitch;
	extern TAutoConsoleVariable<bool> CVarAllowMulticompose;

	class NodeImageConvert;
	class NodeExternalParameter;
	class NodeExternalOperation;
	class NodeSkeletalMeshMorph;
	class NodeSkeletalMeshReshape;
	class NodeSkeletalMeshModify;
	class NodeSkeletalMeshVariation;
	class NodeSkeletalMeshSwitch;
	class NodeMaterialExternal;
	class NodeScalarExternal;
	class NodeMaterialSkeletalMeshObjectBreak;
	class NodeImageExternal;
	class NodeMeshExternal;
	class ASTOpMaterialModify;
	class ASTOpParameter;
	class FLayout;
	class NodeColorArithmeticOperation;
	class NodeColorConstant;
	class NodeColorFromScalars;
	class NodeColorParameter;
	class NodeColorSwitch;
	class NodeColorTable;
	class NodeColorVariation;
	class NodeColorToSRGB;
	class NodeColorMaterialBreak;
	class NodeImageBinarise;
	class NodeImageColorMap;
	class NodeImageConditional;
	class NodeImageConstant;
	class NodeImageFormat;
	class NodeImageConform;
	class NodeImageFromMaterialParameter;
	class NodeImageInterpolate;
	class NodeImageInvert;
	class NodeImageLayer;
	class NodeImageLayerColor;
	class NodeImageLuminance;
	class NodeImageMaterialBreak;
	class NodeImageMipmap;
	class NodeImageMultiLayer;
	class NodeImageNormalComposite;
	class NodeImageObjectParameter;
	class NodeImagePlainColor;
	class NodeImageResize;
	class NodeImageSaturate;
	class NodeImageSwitch;
	class NodeImageSwizzle;
	class NodeImageTable;
	class NodeImageTransform;
	class NodeImageVariation;
	class NodeMaterial;
	class NodeMaterialBreak;
	class NodeMaterialConstant;
	class NodeMaterialModify;
	class NodeMaterialTable;
	class NodeMaterialSwitch;
	class NodeMaterialVariation;
	class NodeMaterialParameter;
	class NodeMeshApplyPose;
	class NodeMeshClipDeform;
	class NodeMeshClipMorphPlane;
	class NodeMeshClipWithMesh;
	class NodeMeshRemoveMesh;
	class NodeMeshConstant;
	class NodeMeshFormat;
	class NodeMeshFragment;
	class NodeMeshMakeMorph;
	class NodeMeshMorph;
	class NodeMeshReshape;
	class NodeMeshSwitch;
	class NodeMeshTransform;
	class NodeMeshVariation;
	class NodeMeshSkeletalMeshObjectBreak;
	class NodeRange;
	class NodeScalarArithmeticOperation;
	class NodeScalarConstant;
	class NodeScalarCurve;
	class NodeScalarEnumParameter;
	class NodeScalarParameter;
	class NodeScalarSwitch;
	class NodeScalarTable;
	class NodeScalarVariation;
	class NodeScalarMaterialBreak;
	class NodeStringConstant;
	class NodeStringParameter;
	class NodeSkeletalMesh;
	class NodeSkeletalMeshObjectParameter;
	class NodeSkeletalMeshMerge;
	class NodeMatrix;
	class NodeMatrixConstant;
	class NodeMatrixParameter;
	class NodeMeshTransformWithBone;
	class NodeMeshTransformInMesh;
	struct FObjectState;
	struct FProgram;

    /** Converts UE::Mutable::Private::Node graphs into ASTOp graphs. */
    class CodeGenerator
    {
		
		friend class FirstPassGenerator;

		struct FLayoutBlockDesc;

    public:

        CodeGenerator( CompilerOptions::Private*, TFunction<void()>& InWaitCallback );
    	~CodeGenerator();
    	
        //! Data will be stored in States
        void GenerateRoot( const Ptr<const Node> );

    	void WaitTask(Tasks::FTask Task);

		/** This function will be called repeatedly in case a synchronization between threads is necessary. 
		* It can be called from any thread.
		*/
		TFunction<void()> WaitCallback;

		struct FGenericGenerationOptions
		{
			friend FORCEINLINE uint32 GetTypeHash(const FGenericGenerationOptions& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.State));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.ActiveTags.Num()));
				return KeyHash;
			}

			friend bool operator==(const FGenericGenerationOptions&, const FGenericGenerationOptions&) = default;

			int32 State = -1;
			TArray<FString> ActiveTags;

			// Variable to identify if an operation will end up generating an image.
			bool bIsImage = false;
			
		};

		struct FObjectGenerationOptions : public FGenericGenerationOptions
		{
			const NodeObjectNew* ParentObjectNode = nullptr;

			/** Condition that enables a specific object. */
			Ptr<ASTOp> CurrentObjectCondition;
		};

		struct FComponentGenerationOptions : public FGenericGenerationOptions
		{
			FComponentGenerationOptions(const FGenericGenerationOptions& BaseOptions, const Ptr<ASTOp>& InBaseInstance )
			{
				BaseInstance = InBaseInstance;
				State = BaseOptions.State;
				ActiveTags = BaseOptions.ActiveTags;
			}

			/** Instance to which the possibly generated components should be added. */
			Ptr<ASTOp> BaseInstance;
		};

    	struct FSkeletalMeshGenerationOptions
    	{
    		FSkeletalMeshGenerationOptions(const FGenericGenerationOptions& BaseOptions, const NodeComponentNew* InComponent)
				: GenericOptions(BaseOptions)
				, Component(InComponent)
    		{
    		}

    		uint8 FirstLODAvailable = 0;
    		uint8 NumLODs = 0;
			FGenericGenerationOptions GenericOptions;
    		const NodeComponentNew* Component = nullptr;
    	};
    	
		struct FLODGenerationOptions : public FGenericGenerationOptions
		{
			FLODGenerationOptions(const FGenericGenerationOptions& BaseOptions, int32 InLODIndex, const NodeComponentNew* InComponent, TConstArrayView<int32> InSurfaceIndices)
			{
				Component = InComponent;
				LODIndex = InLODIndex;
				SurfaceIndices = InSurfaceIndices;
				State = BaseOptions.State;
				ActiveTags = BaseOptions.ActiveTags;
			}

			const NodeComponentNew* Component = nullptr;
			int32 LODIndex;
			TConstArrayView<int32> SurfaceIndices;
		};

		struct FSurfaceGenerationOptions : public FGenericGenerationOptions
		{
			explicit FSurfaceGenerationOptions(const FGenericGenerationOptions& BaseOptions)
			{
				Component = nullptr;
				LODIndex = -1;
				State = BaseOptions.State;
				ActiveTags = BaseOptions.ActiveTags;
			}

			explicit FSurfaceGenerationOptions(const FLODGenerationOptions& BaseOptions)
			{
				Component = BaseOptions.Component;
				LODIndex = BaseOptions.LODIndex;
				State = BaseOptions.State;
				ActiveTags = BaseOptions.ActiveTags;
			}

			const NodeComponentNew* Component = nullptr;
			int32 LODIndex = -1;
		};

		struct FGeneratedCacheKey
		{
			Ptr<const Node> Node;
			FGenericGenerationOptions Options;

			friend FORCEINLINE uint32 GetTypeHash(const FGeneratedCacheKey& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.Node.get()));
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.Options));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FGeneratedCacheKey& Other) const = default;
		};


		void GenerateGeneric(const FGenericGenerationOptions& Options, FGenericGenerationResult& OutResult, const Ptr<const Node> InNode);

		/** Launch the tasks to generate a LOD subgraph. 
		* Since LODs need to be generated in order a dependency task representing the previous LOD generation can be specified. 
		*/
		FLODTask GenerateLOD(const FLODGenerationOptions&, const NodeLOD*, FLODTask PreviousLODTask);

		/** Launch the tasks to generate a Surface subgraph.
		*/
		FSurfaceTask GenerateSurface(const FSurfaceGenerationOptions&, Ptr<const NodeSurfaceNew>, FLODTask PreviousLODTask);


		struct FGeneratedObjectCacheKey
		{
			Ptr<const Node> Node;
			FObjectGenerationOptions Options;

			friend FORCEINLINE uint32 GetTypeHash(const FGeneratedObjectCacheKey& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.Node.get()));
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.Options));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FGeneratedObjectCacheKey& Other) const
			{
				return Node == Other.Node && Options == Other.Options;
			}
		};

		struct FObjectGenerationResult : public FGenericGenerationResult
		{
			/** List of additional components to add to an object that come from child objects.
			 * The index is the object and lod that should receive the components.
			 */
			struct FAdditionalComponentKey
			{
				const NodeObjectNew* ObjectNode = nullptr;

				FORCEINLINE bool operator==(const FAdditionalComponentKey& Other) const
				{
					return ObjectNode == Other.ObjectNode;
				}

				friend FORCEINLINE uint32 GetTypeHash(const FAdditionalComponentKey& InKey)
				{
					uint32 KeyHash = ::GetTypeHash(InKey.ObjectNode);
					return KeyHash;
				}
			};

			struct FAdditionalComponentData
			{
				Ptr<ASTOp> ComponentOp;
				Ptr<ASTOp> PlaceholderOp;
			};

			TMultiMap< FAdditionalComponentKey, TArray<FAdditionalComponentData> > AdditionalComponents;
		};

		typedef TMap<FGeneratedObjectCacheKey, FObjectGenerationResult> FGeneratedObjectsMap;
		FGeneratedObjectsMap GeneratedObjects;

		void GenerateObject(const FObjectGenerationOptions&, FObjectGenerationResult&, const NodeObject*);
		void GenerateObject_New(const FObjectGenerationOptions&, FObjectGenerationResult&, const NodeObjectNew*);
		void GenerateObject_Group(const FObjectGenerationOptions&, FObjectGenerationResult&, const NodeObjectGroup*);

    public:

        /** */
        const CompilerOptions::Private* CompilerOptions = nullptr;

		/** */
		FirstPassGenerator FirstPass;

        /** Used to accumulate all generation error messages. */
        TSharedPtr<FErrorLog> ErrorLog;

        /** After the entire code generation this contains the information about all the states. */
        typedef TArray< TPair<FObjectState, Ptr<ASTOp>> > FStateList;
        FStateList States;

    private:

		/** Any task accessing generic CodeGenerator members or "serial" data should run in this pipe. */
		UE::Tasks::FPipe LocalPipe;

		/** This struct contains the generation state that can only be accessed when generating NodeMeshConstants.
		* It should only be accessed from tasks running in the specific FPipe below.
		*/
		struct FGenerateMeshConstantState
		{
			/** Container of meshes generated to be able to reuse them.
			* They are sorted by a cheap hash to speed up searches.
			*/
			struct FGeneratedConstantMesh
			{
				TManagedPtr<FMesh> Mesh;
				Ptr<ASTOp> LastMeshOp;
			};
			TMap<uint64, TArray<FGeneratedConstantMesh>> GeneratedConstantMeshes;
		};
		FGenerateMeshConstantState GenerateMeshConstantState;


		/** List of already used vertex ID for meshes that must be unique. */
		struct FUniqueMeshIds
		{
			uint32 EnsureUnique(uint32 Id);
		private:
			UE::FMutex Mutex;
			TSet<uint32> Map;
		};
    	
		FUniqueMeshIds UniqueMeshIds;

		/** Any task accessing GenerateMeshConstantState should run in this pipe. */
		UE::Tasks::FPipe GenerateMeshConstantPipe;


		/** The key for generated tables is made of the source table and a parameter name. */
		struct FTableCacheKey
		{
			Ptr<const FTable> Table;
			FString ParameterName;

			friend FORCEINLINE uint32 GetTypeHash(const FTableCacheKey& InKey)
			{
				uint32 KeyHash = ::GetTypeHash(InKey.Table.get());
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.ParameterName));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FTableCacheKey& InKey) const
			{
				if (Table != InKey.Table) return false;
				if (ParameterName != InKey.ParameterName) return false;
				return true;
			}
		};

		struct FGeneratedTableNodes
		{
			UE::FMutex Mutex;
			TMap< FTableCacheKey, Ptr<NodeScalar> > Map;
		};
		FGeneratedTableNodes GeneratedTableNodes;

		struct FConditionalExtensionDataOp
		{
			Ptr<ASTOp> Condition;
			Ptr<ASTOp> ExtensionDataOp;
		};

		TArray<FConditionalExtensionDataOp> ConditionalExtensionDataOps;

		struct FGeneratedComponentCacheKey
		{
			Ptr<const Node> Node;
			FGenericGenerationOptions Options;

			friend FORCEINLINE uint32 GetTypeHash(const FGeneratedComponentCacheKey& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.Node.get()));
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.Options));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FGeneratedComponentCacheKey& Other) const
			{
				return Node == Other.Node && Options == Other.Options;
			}
		};

		typedef TMap<FGeneratedComponentCacheKey, FGenericGenerationResult> FGeneratedComponentMap;
		FGeneratedComponentMap GeneratedComponents;

		void GenerateComponent(const FComponentGenerationOptions&, FGenericGenerationResult&, const NodeComponent*);
		void GenerateComponent_New(const FComponentGenerationOptions&, FGenericGenerationResult&, const NodeComponentNew*);
		void GenerateComponent_Switch(const FComponentGenerationOptions&, FGenericGenerationResult&, const NodeComponentSwitch*);
		void GenerateComponent_Variation(const FComponentGenerationOptions&, FGenericGenerationResult&, const NodeComponentVariation*);

        /** */
		Ptr<NodeScalar> GenerateTableVariableNode(const Node&, const FTableCacheKey&, bool bAddNoneOption, const FString& DefaultRowName);

        //!
        Ptr<ASTOp> GenerateMissingBoolCode(const TCHAR* strWhere, bool value, const void* errorContext );

		enum class EYesNoQuery
		{
			Yes,
			No,
			Mixed,
		};

		Ptr<ASTOpImagePixelFormat> GenerateFormatExpression(const NodeImageFormat* NodeFormat, Ptr<ASTOp> SubExpression, EYesNoQuery CanNormalLAEncode);

        //!
		template<class NODE_TABLE, ETableColumnType TYPE, EOpType OPTYPE, typename F>
		Ptr<ASTOp> GenerateTableSwitch( const NODE_TABLE& node, F&& GenerateOption );

		template<class NODE_TABLE, ETableColumnType TYPE, class NODE_TYPE, class NODE_SWITCH_TYPE, typename F>
		void GenerateTableSwitchNode(const NODE_TABLE& node, Ptr<NODE_SWITCH_TYPE>& OutResult, F&& GenerateOption);


		//-----------------------------------------------------------------------------------------
		// Images
		
		/** Options that affect the generation of images. It is like list of what required data we want while parsing down the image node graph. */
		struct FImageGenerationOptions
		{
			FImageGenerationOptions(int32 InComponentId, int32 InLODIndex)
				: ComponentId(InComponentId)
				, LODIndex(InLODIndex)
			{
				GenericOptions.bIsImage = true;
			}

			FGenericGenerationOptions GenericOptions;

			/** The id of the component that we are currently generating. */
			int32 ComponentId = -1;

			/** The LOD being generated. */
			int32 LODIndex = -1;

			/** */
			CompilerOptions::TextureLayoutStrategy ImageLayoutStrategy = CompilerOptions::TextureLayoutStrategy::None;

			/** If different than {0,0} this is the mandatory size of the image that needs to be generated. */
			UE::Math::TIntVector2<int32> RectSize = {0, 0};

			/** Layout block that we are trying to generate if any. */
			uint64 LayoutBlockId = FLayoutBlock::InvalidBlockId;
			TManagedPtr<const FLayout> LayoutToApply;

			/**(Optional) Parameter Descriptor of the material where this image belongs. */
			FParameterDesc MaterialParameter;

			friend FORCEINLINE uint32 GetTypeHash(const FImageGenerationOptions& InKey)
			{
				uint32 KeyHash = GetTypeHash((FGenericGenerationOptions&)InKey.GenericOptions);
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.ComponentId));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.LODIndex));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.ImageLayoutStrategy));
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.RectSize));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.LayoutBlockId));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.LayoutToApply.Get()));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FImageGenerationOptions& Other) const = default;
		};

		/** */
		struct FImageGenerationResult
		{
			Ptr<ASTOp> op;
			bool Parameter = false;
			bool bCanMulticompose = true;
			EYesNoQuery CanNormalLAEncode = EYesNoQuery::Yes;
		};

		/** */
		struct FGeneratedImageCacheKey
		{
			FGeneratedImageCacheKey(const FImageGenerationOptions& InOptions, const NodeImagePtrConst& InNode)
				: Node(InNode)
				, Options(InOptions)
			{
			}

			Ptr<const Node> Node;
			FImageGenerationOptions Options;

			friend FORCEINLINE uint32 GetTypeHash(const FGeneratedImageCacheKey& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.Node.get()));
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.Options));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FGeneratedImageCacheKey& Other) const = default;
		};

		typedef TMap<FGeneratedImageCacheKey, FImageGenerationResult> FGeneratedImagesMap;
		struct FGeneratedImages
		{
			UE::FMutex Mutex;
			FGeneratedImagesMap Map;
		};
		FGeneratedImages GeneratedImages;

		void GenerateImage(const FImageGenerationOptions&, FImageGenerationResult& result, const NodeImagePtrConst& node);
		void GenerateImage_Constant(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageConstant*);
		void GenerateImage_Interpolate(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageInterpolate*);
		void GenerateImage_Saturate(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageSaturate*);
		void GenerateImage_Table(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageTable*);
		void GenerateImage_Swizzle(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageSwizzle*);
		void GenerateImage_ColorMap(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageColorMap*);
		void GenerateImage_Binarise(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageBinarise*);
		void GenerateImage_Luminance(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageLuminance*);
		void GenerateImage_Layer(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageLayer*);
		void GenerateImage_LayerColor(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageLayerColor*);
		void GenerateImage_Resize(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageResize*);
		void GenerateImage_PlainColor(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImagePlainColor*);
		void GenerateImage_Project(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageProject*);
		void GenerateImage_Mipmap(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageMipmap*);
		void GenerateImage_Switch(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageSwitch*);
		void GenerateImage_Conditional(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageConditional*);
		void GenerateImage_Format(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageFormat*);
		void GenerateImage_Conform(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageConform*);
		void GenerateImage_Parameter(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageObjectParameter*);
		void GenerateImage_MultiLayer(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageMultiLayer*);
		void GenerateImage_Invert(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageInvert*);
		void GenerateImage_Variation(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageVariation*);
		void GenerateImage_NormalComposite(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageNormalComposite*);
		void GenerateImage_Transform(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageTransform*);
		void GenerateImage_MaterialBreak(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageMaterialBreak*);
		void GenerateImage_FromMaterialParameter(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageFromMaterialParameter*);
		void GenerateImage_External(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageExternal*);
		void GenerateImage_ParameterConvert(const FImageGenerationOptions&, FImageGenerationResult&, const NodeImageConvert*);

    	void GenerateCrop(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImage&);

		/** */
		TManagedPtr<FImage> GenerateMissingImage(EImageFormat);

		/** */
		Ptr<ASTOp> GenerateMissingImageCode(const TCHAR* strWhere, EImageFormat, const void* errorContext, const FImageGenerationOptions& Options);

		/** */
		Ptr<ASTOp> GeneratePlainImageCode(const FVector4f& Color, const FImageGenerationOptions& Options);

		/** */
		Ptr<ASTOp> GenerateImageFormat(Ptr<ASTOp>, EImageFormat);

		/** */
		Ptr<ASTOp> GenerateImageSize(Ptr<ASTOp>, UE::Math::TIntVector2<int32>);

		/** */
		Ptr<ASTOp> GenerateImageMulticompose(Ptr<ASTOp> BaseOp, Ptr<ASTOp> LayoutOp, const FImageGenerationOptions& Options, const NodeImagePtrConst& NodeImage, TManagedPtr<const FLayout> SourceLayout, const CodeGenerator::FLayoutBlockDesc& InLayoutBlockDesc );

		/** Generate a layout block-sized image with a mask including all pixels in the blocks defined in the patch node. */
		TManagedPtr<FImage> GenerateImageBlockPatchMask(const NodeSurfaceModifierSurfaceEdit::FTexture&, FIntPoint GridSize, int32 BlockPixelsX, int32 BlockPixelsY, box<FIntVector2> RectInCells);

		/** Generate all the operations to apply the block patching on top of the BlockOp, and masking with PatchMask. */
		Ptr<ASTOp> GenerateImageBlockPatch(Ptr<ASTOp> BlockOp, const NodeSurfaceModifierSurfaceEdit::FTexture&, TManagedPtr<FImage> PatchMask, Ptr<ASTOp> ConditionOp, const FImageGenerationOptions&);

		//-----------------------------------------------------------------------------------------
		// Materials	

		/** Options that affect the generation of Materials. It is like list of what required data we want while parsing down the material node graph. */
		struct FMaterialGenerationOptions
		{
			FGenericGenerationOptions GenericOptions;

			/** The id of the component that we are currently generating. */
			int32 ComponentId = -1;

			/** The LOD being generated. */
			int32 LODIndex = -1;

			/** Array with all the tags for the modifiers. */
			TArray<FString> Tags;

			/** Modifiers to apply. */
			TArray<FirstPassGenerator::FModifier> Modifiers;

			FMeshGenerationResult MeshResults;
			
			TBitArray<> LayoutFromExtension;

			/** */
			CompilerOptions::TextureLayoutStrategy ImageLayoutStrategy = CompilerOptions::TextureLayoutStrategy::None;

			/** If different than {0,0} this is the mandatory size of the image that needs to be generated. */
			UE::Math::TIntVector2<int32> RectSize = { 0, 0 };

			/** Layout block that we are trying to generate if any. */
			uint64 LayoutBlockId = FLayoutBlock::InvalidBlockId;
			TManagedPtr<const FLayout> LayoutToApply;

			friend bool operator==(const FMaterialGenerationOptions&, const FMaterialGenerationOptions&) = default;

			friend FORCEINLINE uint32 GetTypeHash(const FMaterialGenerationOptions& Options)
			{
				uint32 KeyHash = 0;

				// TODO: Use a more representative Hash.
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(Options.GenericOptions));

				return KeyHash;
			}
		};

		/** */
		struct FMaterialGenerationResult
		{
			Ptr<ASTOp> op;
		};
    	
		struct FGeneratedMaterialCacheKey
		{
			Ptr<const Node> Node;
			FMaterialGenerationOptions Options;

			friend FORCEINLINE uint32 GetTypeHash(const FGeneratedMaterialCacheKey& InKey)
			{
				uint32 KeyHash = 0;

				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.Node.get()));
				KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.Options));

				return KeyHash;
			}

			friend bool operator==(const FGeneratedMaterialCacheKey&, const FGeneratedMaterialCacheKey&) = default;
		};
    	
		typedef TMap<FGeneratedMaterialCacheKey, FMaterialGenerationResult> FGeneratedMaterialsMap;
		struct FGeneratedMaterials
		{
			UE::FMutex Mutex;
			FGeneratedMaterialsMap Map;
		};
		FGeneratedMaterials GeneratedMaterials;

		void GenerateMaterial(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const Ptr<const NodeMaterial>& node);
		void GenerateMaterial_Constant(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialConstant* node);
		void GenerateMaterial_Switch(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialSwitch* node);
		void GenerateMaterial_Table(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialTable* node);
		void GenerateMaterial_Variation(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialVariation* node);
		void GenerateMaterial_Parameter(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialParameter* node);
		void GenerateMaterial_External(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialExternal* node);
    	void GenerateMaterial_BreakSkeletalMeshObject(const FMaterialGenerationOptions&, FMaterialGenerationResult& OutResult, const NodeMaterialSkeletalMeshObjectBreak* InNode);
    	void GenerateMaterial_BreakSkeletalMesh(const FMaterialGenerationOptions&, FMaterialGenerationResult& OutResult, const NodeMaterialSkeletalMeshBreak* InNode);
		void GenerateMaterial_Modify(const FMaterialGenerationOptions&, FMaterialGenerationResult& result, const NodeMaterialModify* node);

		/** */
		Ptr<ASTOp> GenerateMissingMaterialCode(const TCHAR* strWhere, const void* ErrorContext);

        //-----------------------------------------------------------------------------------------
        // Meshes

		/** Options that affect the generation of meshes. It is like list of what required data we want
		* while parsing down the mesh node graph.
		*/
		struct FMeshGenerationStaticOptions : public FGenericGenerationOptions
		{
			FMeshGenerationStaticOptions(int32 InComponentId, int32 InLODIndex)
				: ComponentId(InComponentId)
				, LODIndex(InLODIndex)
			{
			}

			/** The id of the component that we are currently generating. */
			int32 ComponentId = -1;

			/** The LOD being generated. */
			int32 LODIndex = -1;
			
			/** List of midifiers that shouldn't be applied in the processing of the current subgraph. 
			* This is usually because they have been processed already in the generation so far.
			*/
			TArray<FirstPassGenerator::FModifier> ModifiersToIgnore;
		};


		/** Options that affect the generation of meshes. It is like list of what required data we want
		* while parsing down the mesh node graph.
		*/
		struct FMeshGenerationDynamicOptions
		{
			/** The meshes at the leaves will need their own layout block data. */
			bool bLayouts = false;

			/** If true, Ensure UV Islands are not split between two or more blocks. UVs shared between multiple
			* layout blocks will be clamped to fit the one with more vertices belonging to the UV island.
			* Mainly used to keep consistent layouts when reusing textures between LODs. 
			*/
			bool bClampUVIslands = true;

			/** If true, UVs will be normalized. Normalize UVs should be done in cases where we operate with Images and Layouts */
			bool bNormalizeUVs = false;

			/** If true, assign vertices without layout to the first block. */
			bool bEnsureAllVerticesHaveLayoutBlock = true;

			/** If this has something the layouts in constant meshes will be ignored, because
			* they are supposed to match some other set of layouts. If the array is empty, layouts
			* are generated normally.
			*/
			TArray<FGeneratedLayout> OverrideLayouts;

			/** Optional context to use instead of the node error context.
			 * Be careful since it is not used everywhere. Check usages before assigning a value to it. */
			TOptional<const void*> OverrideContext;
		};

		using FMeshOptionsTask = UE::Tasks::TTask<CodeGenerator::FMeshGenerationDynamicOptions>;

		struct FBaseMeshResultOptions
		{
			/** Use this mutex to control access to the BaseMeshOptions attribute from any thread.
			* The task dependency will make sure that the actual order for data insert and retrieval is correct, but unrelated data can be added while
			* unrelated data is being retrieved so we need the mutex.
			*/
			UE::FMutex Mutex;

			/** Store the mesh generation data for edits that we intend to share across LODs.
			* The key is the combination of the SharedSurfaceGuid of the surface.
			*/
			TMap<FGuid, FMeshGenerationResult> Map;
		};
		FBaseMeshResultOptions BaseMeshOptions;

		struct FExtraLayoutsResultOptions
		{
			/** Use this mutex to control access to the ExtraLayoutsOptions attribute from any thread.
			* The task dependency will make sure that the actual order for data insert and retrieval is correct, but unrelated data can be added while
			* unrelated data is being retrieved so we need the mutex.
			*/
			UE::FMutex Mutex;

			struct FExtraLayoutsKey
			{
				FGuid BaseSurfaceGuid;
				FGuid EditGuid;

				friend FORCEINLINE uint32 GetTypeHash(const FExtraLayoutsKey& InKey)
				{
					return HashCombineFast(GetTypeHash(InKey.BaseSurfaceGuid), GetTypeHash(InKey.EditGuid));
				}

				FORCEINLINE bool operator==(const FExtraLayoutsKey& Other) const = default;
			};

			/** Store the mesh generation data for edits that we intend to share across LODs.
			* The key is the combination of the BaseSurfaceGuid and the ModifierGuid.
			*/
			TMap<FExtraLayoutsKey, FMeshGenerationResult::FExtraLayouts> Map;
		};

		FExtraLayoutsResultOptions ExtraLayoutsOptions;

		struct FSharedSurfaceResultOptions
		{
			/** Use this mutex to control access to the SharedSurfaceOptions attribute from any thread.
			* The task dependency will make sure that the actual order for data insert and retrieval is correct, but unrelated data can be added while
			* unrelated data is being retrieved so we need the mutex.
			*/
			UE::FMutex Mutex;

			struct FSharedSurfaceResult
			{
				TArray<Ptr<ASTOp>> LayoutOps;
				Ptr<ASTOp> LastMaterialOp;
				uint32 SurfaceId;
			};

			struct FSharedSurfaceResultKey
			{
				FGuid BaseSurfaceGuid;
				FGuid CombinedGuid;

				friend FORCEINLINE uint32 GetTypeHash(const FSharedSurfaceResultKey& InKey)
				{
					return HashCombineFast(GetTypeHash(InKey.BaseSurfaceGuid), GetTypeHash(InKey.CombinedGuid));
				}

				FORCEINLINE bool operator==(const FSharedSurfaceResultKey& Other) const = default;
			};

			/** Store the generated layout and surface operations that we intend to share across LODs.
			* The key is the combination of the BaseSurfaceGuid and the combination of all ModifierGuids 
			* affecting this surface.
			*/
			TMap<FSharedSurfaceResultKey, FSharedSurfaceResult> Map;
		};

		FSharedSurfaceResultOptions SharedSurfaceOptions;

		/** This struct contains the generation state that can only be accessed when generating constant layouts.
		* It should only be accessed after locking the included mutex.
		*/
		struct FGenerateLayoutConstantState
		{
			UE::FMutex Mutex;

			/** Map of layouts found in the code already generated.The map is from the source layout node to the generated layout. */
			struct FGeneratedLayoutKey
			{
				Ptr<const NodeLayout> SourceLayout;
				uint32 MeshIdPrefix;

				friend FORCEINLINE uint32 GetTypeHash(const FGeneratedLayoutKey& InKey)
				{
					uint32 KeyHash = ::GetTypeHash(InKey.SourceLayout.get());
					KeyHash = HashCombineFast(KeyHash, GetTypeHash(InKey.MeshIdPrefix));
					return KeyHash;
				}

				FORCEINLINE bool operator==(const FGeneratedLayoutKey& Other) const = default;
			};
			TMap<FGeneratedLayoutKey, TManagedPtr<const FLayout>> GeneratedLayouts;
		};
		FGenerateLayoutConstantState GenerateLayoutConstantState;

		FMeshTask GenerateMesh(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const Ptr<const NodeMesh>&);
		FMeshTask GenerateMesh_Constant(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshConstant*);
		FMeshTask GenerateMesh_Format(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshFormat*);
		FMeshTask GenerateMesh_Morph(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshMorph*);
		FMeshTask GenerateMesh_MakeMorph(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshMakeMorph*);
        FMeshTask GenerateMesh_Fragment(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshFragment*);
        FMeshTask GenerateMesh_Switch(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshSwitch*);
        FMeshTask GenerateMesh_Transform(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshTransform*);
        FMeshTask GenerateMesh_ClipMorphPlane(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshClipMorphPlane*);
        FMeshTask GenerateMesh_ClipWithMesh(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshClipWithMesh*);
        FMeshTask GenerateMesh_ApplyPose(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshApplyPose*);
        FMeshTask GenerateMesh_RemoveMesh(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshRemoveMesh*);
		FMeshTask GenerateMesh_TransformWithBone(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshTransformWithBone*);
    	FMeshTask GenerateMesh_TransformInMesh(const FMeshGenerationStaticOptions&, const FMeshOptionsTask, const NodeMeshTransformInMesh*);
        FMeshTask GenerateMesh_Variation(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshVariation*);
		FMeshTask GenerateMesh_Table(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshTable*);
		FMeshTask GenerateMesh_Reshape(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshReshape*);
		FMeshTask GenerateMesh_ClipDeform(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshClipDeform*);
		FMeshTask GenerateMesh_SkeletalMeshBreak(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshSkeletalMeshObjectBreak*);
    	FMeshTask GenerateMesh_External(const FMeshGenerationStaticOptions&, FMeshOptionsTask, const NodeMeshExternal*);

		Ptr<ASTOp> GenerateLayoutOpsAndResult(const FMeshGenerationDynamicOptions&, Ptr<ASTOp> LastMeshOp, const TArray<Ptr<NodeLayout>>& OriginalLayouts, uint32 MeshId, FMeshGenerationResult& OutResult);

    	//!
    	TManagedPtr<const FLayout> GenerateLayout(Ptr<const NodeLayout> SourceLayout, uint32 MeshIDPrefix);

    	//-----------------------------------------------------------------------------------------
    	// Skeletal Mesh
    	
    	struct FSkeletalMeshGenerationResult
    	{
    		Ptr<ASTOp> SkeletalMeshOp;
    	};

    	typedef TMap<FGeneratedCacheKey, FSkeletalMeshGenerationResult> FGeneratedSkeletalMeshesMap;

    	struct FGeneratedSkeletalMeshes
    	{
    		UE::FMutex Mutex;
    		FGeneratedSkeletalMeshesMap Map;
    	};
    	FGeneratedSkeletalMeshes GeneratedSkeletalMeshes;
    	
    	void GenerateSkeletalMesh(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const Ptr<const NodeSkeletalMesh>&);
    	void GenerateSkeletalMesh_New(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshNew*);
    	void GenerateSkeletalMesh_Convert(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshConvert*);
		void GenerateSkeletalMesh_Merge(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshMerge*);
    	void GenerateSkeletalMesh_Switch(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshSwitch*);
    	void GenerateSkeletalMesh_Variation(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshVariation*);
		void GenerateSkeletalMesh_Morph(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshMorph*);
		void GenerateSkeletalMesh_Reshape(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshReshape*);
		void GenerateSkeletalMesh_Modify(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshModify*);
		void GenerateSkeletalMesh_Transform(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshTransform*);
		void GenerateSkeletalMesh_TransformWithBone(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshTransformWithBone*);
		void GenerateSkeletalMesh_ClipWithSkeletalMesh(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshClipWithSkeletalMesh*);

    	//-----------------------------------------------------------------------------------------
    	// Skeletal Mesh Object
    	
    	struct FSkeletalMeshObjectGenerationResult
    	{
    		Ptr<ASTOp> SkeletalMeshOp;
    	};

    	typedef TMap<FGeneratedCacheKey, FSkeletalMeshObjectGenerationResult> FGeneratedSkeletalMeshObjectsMap;

    	struct FGeneratedSkeletalMeshObjects
    	{
    		UE::FMutex Mutex;
    		FGeneratedSkeletalMeshObjectsMap Map;
    	};
    	FGeneratedSkeletalMeshObjects GeneratedSkeletalMeshObjects;
    	void GenerateSkeletalMeshObject(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions&, const Ptr<const NodeSkeletalMeshObject>&);
    	void GenerateSkeletalMeshObject_Convert(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshObjectConvert*);
    	void GenerateSkeletalMeshObject_Parameter(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshObjectParameter*);
    	void GenerateSkeletalMeshObject_Switch(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions&, const NodeSkeletalMeshObjectSwitch*);

    	//-----------------------------------------------------------------------------------------
    	struct FExternalGenerationOptions 
    	{
    		// TODO Propagated options could be a TVariant
    		
    		// GenerateMesh
    		TOptional<FMeshGenerationStaticOptions> MeshStaticOptions;
    		TOptional<FMeshOptionsTask> MeshDynamicOptions;

    		// GenerateImage
    		TOptional<FImageGenerationOptions> ImageOptions;

    		// GenerateScalar
    		TOptional<FGenericGenerationOptions> ScalarOptions;
    		
    		// GenerateColor
    		TOptional<FGenericGenerationOptions> ColorOptions;

    		// GenerateMaterial
    		TOptional<FMaterialGenerationOptions> MaterialOptions;

    		bool operator==(const FExternalGenerationOptions& Other) const
    		{
    			return
    				//MeshStaticOptions == Other.MeshStaticOptions && // Meshes are not cached
    				//MeshDynamicOptions == Other.MeshDynamicOptions &&
   					ImageOptions == Other.ImageOptions &&
 					ScalarOptions == Other.ScalarOptions &&
 					ColorOptions == Other.ColorOptions &&
 					MaterialOptions == Other.MaterialOptions;
    		}

    		friend uint32 GetTypeHash(const FExternalGenerationOptions& InKey)
    		{
    			uint32 Hash = 0;

				// Hash = HashCombineFast(Hash, GetTypeHash(InKey.MeshStaticOptions)); // Meshes are not cached
				// Hash = HashCombineFast(Hash, GetTypeHash(InKey.MeshDynamicOptions));
				Hash = HashCombineFast(Hash, GetTypeHash(InKey.ImageOptions));
				Hash = HashCombineFast(Hash, GetTypeHash(InKey.ScalarOptions));
				Hash = HashCombineFast(Hash, GetTypeHash(InKey.MaterialOptions));
    			
    			return Hash;
    		}
    		
    		FGenericGenerationOptions GetGenericOptions() const
    		{
    			if (MeshStaticOptions.IsSet())
    			{
    				return static_cast<FGenericGenerationOptions>(*MeshStaticOptions);
    			}
    			
    			if (ImageOptions.IsSet())
    			{
    				return ImageOptions->GenericOptions;
    			}
    			
    			if (ScalarOptions.IsSet())
    			{
    				return *ScalarOptions;
    			}
    			
    			if (ColorOptions.IsSet())
    			{
    				return *ColorOptions;
    			}
    			
    			if (MaterialOptions.IsSet())
    			{
    				return MaterialOptions->GenericOptions;
    			}
    			
    			return {};
    		}
    	};
    	
    	struct FGeneratedExternalCacheKey
    	{
    		Ptr<const NodeExternal> Node;
    		FExternalGenerationOptions Options;
    		
    		friend uint32 GetTypeHash(const FGeneratedExternalCacheKey& InKey)
    		{
    			uint32 Hash = 0;

    			Hash = HashCombineFast(Hash, ::GetTypeHash(InKey.Node.get()));
    			Hash = HashCombineFast(Hash, GetTypeHash(InKey.Options));
    			
    			return Hash;
    		}

    		bool operator==(const FGeneratedExternalCacheKey& Other) const = default;
    	};
    	
    	typedef TMap<FGeneratedExternalCacheKey, FExternalTask> FGeneratedExternalMap;
    	struct FGeneratedExternal
    	{
    		UE::FMutex Mutex;
    		FGeneratedExternalMap Map;
    	};
    	FGeneratedExternal GeneratedExternal;

    	FExternalTask GenerateExternal(const FExternalGenerationOptions& Options, const Ptr<const NodeExternal>& ExternalNode);
    	FExternalTask GenerateExternal_Operation(const FExternalGenerationOptions& Options, const Ptr<const NodeExternalOperation>& ExternalNode);
    	FExternalTask GenerateExternal_Parameter(const FExternalGenerationOptions& Options, const Ptr<const NodeExternalParameter>& ExtensionNode);
    	FExternalTask GenerateExternal_Switch(const FExternalGenerationOptions& Options, const Ptr<const NodeExternalSwitch>& ExtensionNode);
    	Ptr<ASTOp> GenerateMissingExternalCode(const TCHAR* StrWhere, const void* ErrorContext);
    	
		//-----------------------------------------------------------------------------------------
    	
		struct FExtensionDataGenerationResult
		{
			Ptr<ASTOp> Op;
		};

		typedef const NodeExtensionData* FGeneratedExtensionDataCacheKey;
		typedef TMap<FGeneratedExtensionDataCacheKey, FExtensionDataGenerationResult> FGeneratedExtensionDataMap;
		FGeneratedExtensionDataMap GeneratedExtensionData;

		void GenerateExtensionData(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions&, const Ptr<const NodeExtensionData>&);
		void GenerateExtensionData_Constant(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions&, const class NodeExtensionDataConstant*);
		void GenerateExtensionData_Switch(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions&, const class NodeExtensionDataSwitch*);
		void GenerateExtensionData_Variation(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions&, const class NodeExtensionDataVariation*);
		Ptr<ASTOp> GenerateMissingExtensionDataCode(const TCHAR* StrWhere, const void* ErrorContext);

        //-----------------------------------------------------------------------------------------
        // Projectors
        struct FProjectorGenerationResult
        {
            Ptr<ASTOp> op;
            EProjectorType type;
        };

        typedef TMap<FGeneratedCacheKey,FProjectorGenerationResult> FGeneratedProjectorsMap;
		struct FGeneratedProjectors
		{
			UE::FMutex Mutex;
			FGeneratedProjectorsMap Map;
		};
		FGeneratedProjectors GeneratedProjectors;

        void GenerateProjector( FProjectorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeProjector>& );
        void GenerateProjector_Constant( FProjectorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeProjectorConstant>& );
        void GenerateProjector_Parameter( FProjectorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeProjectorParameter>& );
        void GenerateMissingProjectorCode( FProjectorGenerationResult&, const void* errorContext );

		//-----------------------------------------------------------------------------------------
		// Bools
		struct FBoolGenerationResult
		{
			Ptr<ASTOp> op;
		};

		typedef TMap<FGeneratedCacheKey, FBoolGenerationResult> FGeneratedBoolsMap;
		struct FGeneratedBools
		{
			UE::FMutex Mutex;
			FGeneratedBoolsMap Map;
		};
		FGeneratedBools GeneratedBools;

		void GenerateBool(FBoolGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeBool>&);
		void GenerateBool_Constant(FBoolGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeBoolConstant>&);
		void GenerateBool_Parameter(FBoolGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeBoolParameter>&);
		void GenerateBool_Not(FBoolGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeBoolNot>&);
		void GenerateBool_And(FBoolGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeBoolAnd>&);

		//-----------------------------------------------------------------------------------------
		// Scalars
		struct FScalarGenerationResult
		{
			Ptr<ASTOp> op;
		};

		typedef TMap<FGeneratedCacheKey, FScalarGenerationResult> FGeneratedScalarsMap;

		struct FGeneratedScalars
		{
			UE::FMutex Mutex;
			FGeneratedScalarsMap Map;
		};
		FGeneratedScalars GeneratedScalars;

		void GenerateScalar(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalar>&);
		void GenerateScalar_Constant(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarConstant>&);
		void GenerateScalar_Parameter(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarParameter>&);
		void GenerateScalar_Switch(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarSwitch>&);
		void GenerateScalar_EnumParameter(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarEnumParameter>&);
		void GenerateScalar_Curve(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarCurve>&);
		void GenerateScalar_Arithmetic(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarArithmeticOperation>&);
		void GenerateScalar_Variation(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarVariation>&);
		void GenerateScalar_Table(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarTable>&);
		void GenerateScalar_MaterialBreak(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarMaterialBreak>&);
		void GenerateScalar_External(FScalarGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeScalarExternal>&);
		Ptr<ASTOp> GenerateMissingScalarCode(const TCHAR* strWhere, float value, const void* errorContext);

		//-----------------------------------------------------------------------------------------
		// Colors
		struct FColorGenerationResult
		{
			Ptr<ASTOp> op;
		};

		typedef TMap<FGeneratedCacheKey, FColorGenerationResult> FGeneratedColorsMap;
		struct FGeneratedColors
		{
			UE::FMutex Mutex;
			FGeneratedColorsMap Map;
		};
		FGeneratedColors GeneratedColors;

		void GenerateColor(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColor>&);
		void GenerateColor_Constant(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorConstant>&);
		void GenerateColor_Parameter(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorParameter>&);
		void GenerateColor_Switch(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorSwitch>&);
		void GenerateColor_SampleImage(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorSampleImage>&);
		void GenerateColor_FromScalars(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorFromScalars>&);
		void GenerateColor_Arithmetic(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorArithmeticOperation>&);
		void GenerateColor_Variation(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorVariation>&);
		void GenerateColor_ToSRGB(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorToSRGB>&);
		void GenerateColor_Table(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorTable>&);
		void GenerateColor_MaterialBreak(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorMaterialBreak>&);
		void GenerateColor_External(FColorGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeColorExternal>&);
		Ptr<ASTOp> GenerateMissingColorCode(const TCHAR* strWhere, const void* errorContext);

		//-----------------------------------------------------------------------------------------
		// Strings
		struct FStringGenerationResult
		{
			Ptr<ASTOp> op;
		};

		typedef TMap<FGeneratedCacheKey, FStringGenerationResult> FGeneratedStringsMap;
		struct FGeneratedStrings
		{
			UE::FMutex Mutex;
			FGeneratedStringsMap Map;
		};
		FGeneratedStrings GeneratedStrings;

		void GenerateString(FStringGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeString>&);
		void GenerateString_Constant(FStringGenerationResult&, const FGenericGenerationOptions& Options, const Ptr<const NodeStringConstant>&);
		void GenerateString_Parameter(FStringGenerationResult&, const FGenericGenerationOptions& Options, const Ptr<const NodeStringParameter>&);

    	//-----------------------------------------------------------------------------------------
    	// Transforms
    	struct FMatrixGenerationResult
    	{
    		Ptr<ASTOp> op;
    	};

    	typedef TMap<FGeneratedCacheKey, FMatrixGenerationResult> FGeneratedMatrixMap;
		struct FGeneratedMatrices
		{
			UE::FMutex Mutex;
			FGeneratedMatrixMap Map;
		};
		FGeneratedMatrices GeneratedMatrices;

    	void GenerateMatrix(FMatrixGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeMatrix>&);
    	void GenerateMatrix_Constant(FMatrixGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeMatrixConstant>&);
    	void GenerateMatrix_Parameter(FMatrixGenerationResult&, const FGenericGenerationOptions&, const Ptr<const NodeMatrixParameter>&);

        //-----------------------------------------------------------------------------------------
        // Ranges
        struct FRangeGenerationResult
        {
            //
            Ptr<ASTOp> sizeOp;

            //
            FString rangeName;

            //
			FString rangeUID;
        };

        typedef TMap<FGeneratedCacheKey,FRangeGenerationResult> FGeneratedRangeMap;
		struct FGeneratedRanges
		{
			UE::FMutex Mutex;
			FGeneratedRangeMap Map;
		};
		FGeneratedRanges GeneratedRanges;

        void GenerateRange(FRangeGenerationResult&, const FGenericGenerationOptions&, Ptr<const NodeRange>);

		//-----------------------------------------------------------------------------------------

		struct FLayoutBlockDesc
		{
			EImageFormat FinalFormat = EImageFormat::None;
			uint16 BlockPixelsX = 0;
			uint16 BlockPixelsY = 0;
			bool bBlocksHaveMips = false;
		};

		void UpdateLayoutBlockDesc(FLayoutBlockDesc& Out, FImageDesc BlockDesc, FIntVector2 LayoutCellSize);

		// Get the modifiers that have to be applied to elements with a specific tag.
		void GetModifiersFor(int32 ComponentId, const TArray<FString>& SurfaceTags, bool bModifiersForBeforeOperations, TArray<FirstPassGenerator::FModifier>& OutModifiers) const;

		// Apply the required mesh modifiers to the given operation.
		FMeshTask ApplyMeshModifiers(
			const TArray<FirstPassGenerator::FModifier>&,
			const FMeshGenerationStaticOptions&,
			FMeshOptionsTask DynamicOptionsTask,
			FMeshTask BaseTask,
			FGuid SurfaceGuid,
			const void* ErrorContext,
			const NodeMesh* OriginalMeshNode);

		Ptr<ASTOp> ApplyImageBlockModifiers(
			const TArray<FirstPassGenerator::FModifier>&, 
			const FImageGenerationOptions&, 
			Ptr<ASTOp> BaseImageOp,
			const FParameterKey& ImageParameterKey,
			FIntPoint GridSize,
			const FLayoutBlockDesc& LayoutBlockDesc,
			box< FIntVector2 > RectInCells,
			const void* ErrorContext);

		Ptr<ASTOp> ApplyImageExtendModifiers(
			const TArray<FirstPassGenerator::FModifier>&,
			const FMeshGenerationStaticOptions& Options,
			const FMeshGenerationResult& BaseMeshResults,
			Ptr<ASTOp> ImageAd, 
			CompilerOptions::TextureLayoutStrategy ImageLayoutStrategy,
			int32 LayoutIndex, 
			const FParameterKey& ImageParameterKey,
			FIntPoint GridSize,
			CodeGenerator::FLayoutBlockDesc& InOutLayoutBlockDesc,
			const void* ModifiedNodeErrorContext);

		void CheckModifiersForSurface(const NodeSurfaceNew&, const TArray<FirstPassGenerator::FModifier>&, int32 LODIndex);

		bool HasImageBlockModifiers(const FParameterKey& ImageKey, const TArray<FirstPassGenerator::FModifier>&, int32 LODIndex) const;

		void GenerateMaterialImage(
			const TPair<FParameterKey, FImageParameterData>& ImageData,
			FImageGenerationResult& OutImageGenerated,
			const TArray<FirstPassGenerator::FModifier>& Modifiers,
			const int32 State,
			const int32 ComponentId,
			const int32 LODIndex,
			const TArray<FString>& Tags,
			const FMeshGenerationResult& MeshResults,
			const TBitArray<>& LayoutFromExtension,
			const void* ErrorContext);
    };

	
    //---------------------------------------------------------------------------------------------
    template<class NODE_TABLE, ETableColumnType TYPE, EOpType OPTYPE, typename F>
    Ptr<ASTOp> CodeGenerator::GenerateTableSwitch( const NODE_TABLE& node, F&& GenerateOption )
    {
        Ptr<const FTable> NodeTable = node.Table;
        Ptr<ASTOp> Variable;
		Ptr<NodeScalar> VariableNode;

		FTableCacheKey CacheKey = FTableCacheKey{ node.Table, node.ParameterName };

		{
			UE::TUniqueLock Lock(GeneratedTableNodes.Mutex);
			Ptr<NodeScalar>* Found = GeneratedTableNodes.Map.Find(CacheKey);
			if (Found)
			{
				VariableNode = *Found;
			}

			if (!VariableNode)
			{
				// Create the table variable expression
				VariableNode = GenerateTableVariableNode(node, CacheKey, node.bNoneOption, node.DefaultRowName);
				GeneratedTableNodes.Map.Add(CacheKey, VariableNode);
			}
		}

		FGenericGenerationOptions ScalarOptions;
		FScalarGenerationResult ScalarResult;
		GenerateScalar(ScalarResult, ScalarOptions, VariableNode);
		Variable = ScalarResult.op;

		int32 NumRows = NodeTable->GetPrivate()->Rows.Num();

        // Verify that the table column is the right type
        int32 ColIndex = NodeTable->FindColumn( node.ColumnName );

		if (NumRows == 0)
		{
			ErrorLog->Add("The table has no rows.", ELMT_ERROR, node.GetMessageContext());
			return nullptr;
		}
        else if (ColIndex < 0)
        {
            ErrorLog->Add("Table column not found.", ELMT_INFO, node.GetMessageContext());
            return nullptr;
        }

        if (NodeTable->GetPrivate()->Columns[ ColIndex ].Type != TYPE )
        {
            ErrorLog->Add("Table column type is not the right type.", ELMT_ERROR, node.GetMessageContext());
            return nullptr;
        }

        // Create the switch to cover all the options
        Ptr<ASTOp> lastSwitch;
        Ptr<ASTOpSwitch> SwitchOp = new ASTOpSwitch();
		SwitchOp->Type = OPTYPE;
		SwitchOp->Variable = Variable;
		SwitchOp->Default = nullptr;

		for (int32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
        {
            check(RowIndex <= 0xFFFF);
			auto Condition = (uint16)RowIndex;

            Ptr<ASTOp> Branch = GenerateOption( node, ColIndex, RowIndex, ErrorLog.Get() );

			if (Branch || TYPE != ETableColumnType::Mesh)
			{
				SwitchOp->Cases.Add(ASTOpSwitch::FCase(Condition, SwitchOp, Branch));
			}
        }

        return SwitchOp;
    }

	
    //---------------------------------------------------------------------------------------------
    template<class NODE_TABLE, ETableColumnType TYPE, class NODE_TYPE, class NODE_SWITCH_TYPE, typename F>
    void CodeGenerator::GenerateTableSwitchNode( const NODE_TABLE& node, Ptr<NODE_SWITCH_TYPE>& OutResult, F&& GenerateOption )
    {
		OutResult = nullptr;

        Ptr<const FTable> NodeTable = node.Table;
        Ptr<NodeScalar> Variable;

		FTableCacheKey CacheKey = FTableCacheKey{ node.Table, node.ParameterName };

		{
			UE::TUniqueLock Lock(GeneratedTableNodes.Mutex);
			Ptr<NodeScalar>* Found = GeneratedTableNodes.Map.Find(CacheKey);
			if (Found)
			{
				Variable = *Found;
			}

			if (!Variable)
			{
				// Create the table variable expression
				Variable = GenerateTableVariableNode(node, CacheKey, node.bNoneOption, node.DefaultRowName);

				GeneratedTableNodes.Map.Add(CacheKey, Variable);
			}
		}

		int32 NumRows = NodeTable->GetPrivate()->Rows.Num();

        // Verify that the table column is the right type
        int32 ColIndex = NodeTable->FindColumn( node.ColumnName );

		if (NumRows == 0)
		{
			ErrorLog->Add("The table has no rows.", ELMT_ERROR, node.GetMessageContext());
			return;
		}
        else if (ColIndex < 0)
        {
            ErrorLog->Add("Table column not found.", ELMT_INFO, node.GetMessageContext());
            return;
        }

        if (NodeTable->GetPrivate()->Columns[ ColIndex ].Type != TYPE )
        {
            ErrorLog->Add("Table column type is not the right type.", ELMT_ERROR, node.GetMessageContext());
            return;
        }

        // Create the switch node
        Ptr<NODE_SWITCH_TYPE> SwitchNode = new NODE_SWITCH_TYPE;
		SwitchNode->Parameter = Variable;
		SwitchNode->Options.SetNum(NumRows);

		for (int32 RowIndex = 0; RowIndex < NumRows; ++RowIndex)
        {
            check(RowIndex <= 0xFFFF);
            Ptr<NODE_TYPE> BranchNode = GenerateOption( node, ColIndex, RowIndex, ErrorLog.Get() );
			SwitchNode->Options[RowIndex] = BranchNode;
        }

		OutResult = SwitchNode;
    }


}
