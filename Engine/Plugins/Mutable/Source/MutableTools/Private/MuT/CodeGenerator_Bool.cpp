// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConstantBool.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ASTOpBoolAnd.h"
#include "MuT/ASTOpBoolNot.h"
#include "MuT/CodeGenerator.h"
#include "MuT/NodeBool.h"
#include "MuT/NodeRange.h"


namespace UE::Mutable::Private
{

	void CodeGenerator::GenerateBool(FBoolGenerationResult& Result, const FGenericGenerationOptions& Options, const Ptr<const NodeBool>& Untyped)
	{
		if (!Untyped)
		{
			Result = FBoolGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;
		{
			UE::TUniqueLock Lock(GeneratedBools.Mutex);
			FGeneratedBoolsMap::ValueType* Found = GeneratedBools.Map.Find(Key);
			if (Found)
			{
				Result = *Found;
				return;
			}
		}

		// Generate for each different type of node
		if (Untyped->GetType()==NodeBoolConstant::GetStaticType())
		{
			const NodeBoolConstant* Constant = static_cast<const NodeBoolConstant*>(Untyped.get());
			GenerateBool_Constant(Result, Options, Constant);
		}
		else if (Untyped->GetType() == NodeBoolParameter::GetStaticType())
		{
			const NodeBoolParameter* Param = static_cast<const NodeBoolParameter*>(Untyped.get());
			GenerateBool_Parameter(Result, Options, Param);
		}
		else if (Untyped->GetType() == NodeBoolNot::GetStaticType())
		{
			const NodeBoolNot* Sample = static_cast<const NodeBoolNot*>(Untyped.get());
			GenerateBool_Not(Result, Options, Sample);
		}
		else if (Untyped->GetType() == NodeBoolAnd::GetStaticType())
		{
			const NodeBoolAnd* From = static_cast<const NodeBoolAnd*>(Untyped.get());
			GenerateBool_And(Result, Options, From);
		}
		else
		{
			check(false);
		}

		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedBools.Mutex);
			GeneratedBools.Map.Add(Key, Result);
		}
	}


	void CodeGenerator::GenerateBool_Constant(FBoolGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeBoolConstant>& Node)
	{
		result.op = new ASTOpConstantBool(Node->Value);
	}


	void CodeGenerator::GenerateBool_Parameter(FBoolGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeBoolParameter>& Node)
	{
		Ptr<ASTOpParameter> op;

		// This can run in any thread.
		Ptr<ASTOpParameter>* Found;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);

			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				FParameterDesc param;
				param.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->UID, param.UID);
				check(bParseOk);
				param.Type = EParameterType::Bool;
				param.DefaultValue.Set<FParamBoolType>(Node->DefaultValue);

				op = new ASTOpParameter();
				op->Type = EOpType::BO_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Node->Ranges.Num(); ++a)
			{
				FRangeGenerationResult RangeResult;
				GenerateRange(RangeResult, Options, Node->Ranges[a]);
				op->Ranges.Emplace(op.get(), RangeResult.sizeOp, RangeResult.rangeName, RangeResult.rangeUID);
			}
		}

		result.op = op;
	}


	void CodeGenerator::GenerateBool_Not(FBoolGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeBoolNot>& Node)
	{
		Ptr<ASTOpBoolNot> op = new ASTOpBoolNot();

		FBoolGenerationResult ChildResult;
		GenerateBool(ChildResult, Options, Node->Source);
		op->A = ChildResult.op;

		result.op = op;
	}


	void CodeGenerator::GenerateBool_And(FBoolGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeBoolAnd>& Node)
	{
		Ptr<ASTOpBoolAnd> op = new ASTOpBoolAnd;

		FBoolGenerationResult ChildResult;
		GenerateBool(ChildResult, Options, Node->A);
		op->A = ChildResult.op;

		GenerateBool(ChildResult, Options, Node->B);
		op->B = ChildResult.op;

		result.op = op;
	}


}
