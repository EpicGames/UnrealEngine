// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Image.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Parameters.h"
#include "MuR/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpConstantColor.h"
#include "MuT/ASTOpColorArithmetic.h"
#include "MuT/ASTOpColorSampleImage.h"
#include "MuT/ASTOpColorFromScalars.h"
#include "MuT/ASTOpColorToSRGB.h"
#include "MuT/ASTOpMaterialBreak.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/ErrorLog.h"
#include "MuT/Node.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeColorArithmeticOperation.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeColorMaterialBreak.h"
#include "MuT/NodeColorParameter.h"
#include "MuT/NodeColorSampleImage.h"
#include "MuT/NodeColorSwitch.h"
#include "MuT/NodeColorTable.h"
#include "MuT/NodeColorVariation.h"
#include "MuT/NodeColorToSRGB.h"
#include "MuT/NodeColorExternal.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/Table.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor(FColorGenerationResult& Result, const FGenericGenerationOptions& Options, const Ptr<const NodeColor>& Untyped)
	{
		if (!Untyped)
		{
			Result = FColorGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;
		{
			UE::TUniqueLock Lock(GeneratedColors.Mutex);
			FGeneratedColorsMap::ValueType* Found = GeneratedColors.Map.Find(Key);
			if (Found)
			{
				Result = *Found;
				return;
			}
		}

		// Generate for each different type of node
		if (Untyped->GetType()==NodeColorConstant::GetStaticType())
		{
			const NodeColorConstant* Constant = static_cast<const NodeColorConstant*>(Untyped.get());
			GenerateColor_Constant(Result, Options, Constant);
		}
		else if (Untyped->GetType() == NodeColorParameter::GetStaticType())
		{
			const NodeColorParameter* Param = static_cast<const NodeColorParameter*>(Untyped.get());
			GenerateColor_Parameter(Result, Options, Param);
		}
		else if (Untyped->GetType() == NodeColorSwitch::GetStaticType())
		{
			const NodeColorSwitch* Switch = static_cast<const NodeColorSwitch*>(Untyped.get());
			GenerateColor_Switch(Result, Options, Switch);
		}
		else if (Untyped->GetType() == NodeColorSampleImage::GetStaticType())
		{
			const NodeColorSampleImage* Sample = static_cast<const NodeColorSampleImage*>(Untyped.get());
			GenerateColor_SampleImage(Result, Options, Sample);
		}
		else if (Untyped->GetType() == NodeColorFromScalars::GetStaticType())
		{
			const NodeColorFromScalars* From = static_cast<const NodeColorFromScalars*>(Untyped.get());
			GenerateColor_FromScalars(Result, Options, From);
		}
		else if (Untyped->GetType() == NodeColorArithmeticOperation::GetStaticType())
		{
			const NodeColorArithmeticOperation* Arithmetic = static_cast<const NodeColorArithmeticOperation*>(Untyped.get());
			GenerateColor_Arithmetic(Result, Options, Arithmetic);
		}
		else if (Untyped->GetType() == NodeColorVariation::GetStaticType())
		{
			const NodeColorVariation* Variation = static_cast<const NodeColorVariation*>(Untyped.get());
			GenerateColor_Variation(Result, Options, Variation);
		}
		else if (Untyped->GetType() == NodeColorToSRGB::GetStaticType())
		{
			const NodeColorToSRGB* ToSRGB = static_cast<const NodeColorToSRGB*>(Untyped.get());
			GenerateColor_ToSRGB(Result, Options, ToSRGB);
		}
		else if (Untyped->GetType() == NodeColorTable::GetStaticType())
		{
			const NodeColorTable* Table = static_cast<const NodeColorTable*>(Untyped.get());
			GenerateColor_Table(Result, Options, Table);
		}
		else if (Untyped->GetType() == NodeColorMaterialBreak::GetStaticType())
		{
			const NodeColorMaterialBreak* Table = static_cast<const NodeColorMaterialBreak*>(Untyped.get());
			GenerateColor_MaterialBreak(Result, Options, Table);
		}
		else if (Untyped->GetType() == NodeColorExternal::GetStaticType())
		{
			const NodeColorExternal* Node = static_cast<const NodeColorExternal*>(Untyped.get());
			GenerateColor_External(Result, Options, Node);
		}
		else
		{
			check(false);
		}

		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedColors.Mutex);
			GeneratedColors.Map.Add(Key, Result);
		}
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_Constant(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorConstant>& Typed)
	{
		const NodeColorConstant& node = *Typed;

		Ptr<ASTOpConstantColor> Op = new ASTOpConstantColor;
		Op->Value = node.Value;

		result.op = Op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_Parameter(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorParameter>& Typed)
	{
		const NodeColorParameter& node = *Typed;

		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);

			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Typed);
			if (!Found)
			{
				FParameterDesc param;
				param.Name = node.Name;
				bool bParseOk = FGuid::Parse(node.Uid, param.UID);
				check(bParseOk);
				param.Type = EParameterType::Color;

				FParamColorType Value;
				Value[0] = node.DefaultValue[0];
				Value[1] = node.DefaultValue[1];
				Value[2] = node.DefaultValue[2];
				Value[3] = node.DefaultValue[3];

				param.DefaultValue.Set<FParamColorType>(Value);

				op = new ASTOpParameter();
				op->Type = EOpType::CO_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Typed, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < node.Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options, node.Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		result.op = op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_Switch(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorSwitch>& Typed)
	{
		const NodeColorSwitch& node = *Typed;

		MUTABLE_CPUPROFILER_SCOPE(NodeColorSwitch);

		if (node.Options.Num() == 0)
		{
			// No options in the switch!
			Ptr<ASTOp> missingOp = GenerateMissingColorCode(TEXT("Switch option"), Typed->GetMessageContext());
			result.op = missingOp;
			return;
		}

		Ptr<ASTOpSwitch> op = new ASTOpSwitch();
		op->Type = EOpType::CO_SWITCH;

		// Variable value
		if (node.Parameter)
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options, node.Parameter.get());
			op->Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
			op->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Typed->GetMessageContext());
		}

		// Options
		for (int32 OptionIndex = 0; OptionIndex < node.Options.Num(); ++OptionIndex)
		{
			Ptr<ASTOp> branch;
			if (node.Options[OptionIndex])
			{
				FColorGenerationResult ParamResult;
				GenerateColor(ParamResult, Options, node.Options[OptionIndex].get());
				branch = ParamResult.op;
			}
			else
			{
				// This argument is required
				branch = GenerateMissingColorCode(TEXT("Switch option"), Typed->GetMessageContext());
			}
			op->Cases.Emplace((int16)OptionIndex, op, branch);
		}

		result.op = op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_Variation(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorVariation>& Typed)
	{
		const NodeColorVariation& node = *Typed;

		Ptr<ASTOp> currentOp;

		// Default case
		if (node.DefaultColor)
		{
			FColorGenerationResult BranchResults;
			GenerateColor(BranchResults, Options, node.DefaultColor);
			currentOp = BranchResults.op;
		}

		// Process variations in reverse order, since conditionals are built bottom-up.
		for (int32 t = node.Variations.Num() - 1; t >= 0; --t)
		{
			int32 tagIndex = -1;
			const FString& tag = node.Variations[t].Tag;
			for (int32 i = 0; i < FirstPass.Tags.Num(); ++i)
			{
				if (FirstPass.Tags[i].Tag == tag)
				{
					tagIndex = i;
				}
			}

			if (tagIndex < 0)
			{
				FString Msg = FString::Printf(TEXT("Unknown tag found in color variation [%s]."), *tag);
				ErrorLog->Add(Msg, ELMT_WARNING, Typed->GetMessageContext());
				continue;
			}

			Ptr<ASTOp> variationOp;
			if (node.Variations[t].Color)
			{
				FColorGenerationResult BranchResults;
				GenerateColor(BranchResults,Options,node.Variations[t].Color);
				variationOp = BranchResults.op;
			}
			else
			{
				// This argument is required
				variationOp = GenerateMissingColorCode(TEXT("Variation option"), Typed->GetMessageContext());
			}


			Ptr<ASTOpConditional> conditional = new ASTOpConditional;
			conditional->type = EOpType::CO_CONDITIONAL;
			conditional->no = currentOp;
			conditional->yes = variationOp;
			conditional->condition = FirstPass.Tags[tagIndex].GenericCondition;

			currentOp = conditional;
		}

		result.op = currentOp;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_SampleImage(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorSampleImage>& Typed)
	{
		const NodeColorSampleImage& node = *Typed;

		// Generate the code
		Ptr<ASTOpColorSampleImage> op = new ASTOpColorSampleImage();

		// Source image
		int32 ComponentId = -1; // TODO.
		int32 LODIndex = 0; // TODO.
		FImageGenerationOptions ImageOptions(ComponentId, LODIndex);
		ImageOptions.GenericOptions.State = Options.State;
		ImageOptions.GenericOptions.ActiveTags = Options.ActiveTags;

		Ptr<ASTOp> base;
		if (node.Image)
		{
			// Generate
			FImageGenerationResult MapResult;
			GenerateImage(ImageOptions, MapResult, node.Image);
			base = MapResult.op;
		}
		else
		{
			// This argument is required
			base = GenerateMissingImageCode(TEXT("Sample image"), EImageFormat::RGB_UByte, Typed->GetMessageContext(), ImageOptions);
		}
		base = GenerateImageFormat(base, EImageFormat::RGBA_UByte);
		op->Image = base;

		FScalarGenerationResult ChildResult;

		// X
		if (NodeScalar* pX = node.X.get())
		{
			GenerateScalar(ChildResult, Options, pX);
			op->X = ChildResult.op;
		}
		else
		{
			// Set a constant 0.5 value
			Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
			Node->Value = 0.5f;
			GenerateScalar(ChildResult, Options, Node);
			op->X = ChildResult.op;
		}


		// Y
		if (NodeScalar* pY = node.Y.get())
		{
			GenerateScalar(ChildResult, Options, pY);
			op->Y = ChildResult.op;
		}
		else
		{
			// Set a constant 0.5 value
			Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
			Node->Value = 0.5f;
			GenerateScalar(ChildResult, Options, Node);
			op->Y = ChildResult.op;
		}

		// TODO
		op->Filter = 0;

		result.op = op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_FromScalars(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorFromScalars>& Typed)
	{
		const NodeColorFromScalars& node = *Typed;

		Ptr<ASTOpColorFromScalars> op = new ASTOpColorFromScalars();

		FScalarGenerationResult ChildResult;

		// X
		if (NodeScalar* pX = node.X.get())
		{
			GenerateScalar(ChildResult, Options, pX);
			op->V[0] = ChildResult.op;
		}
		else
		{
			Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
			Node->Value  = 1.0f;
			GenerateScalar(ChildResult, Options, Node);
			op->V[0] = ChildResult.op;
		}

		// Y
		if (NodeScalar* pY = node.Y.get())
		{
			GenerateScalar(ChildResult, Options, pY);
			op->V[1] = ChildResult.op;
		}
		else
		{
			Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
			Node->Value = 1.0f;
			GenerateScalar(ChildResult, Options, Node);
			op->V[1] = ChildResult.op;
		}

		// Z
		if (NodeScalar* pZ = node.Z.get())
		{
			GenerateScalar(ChildResult, Options, pZ);
			op->V[2] = ChildResult.op;
		}
		else
		{
			Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
			Node->Value = 1.0f;
			GenerateScalar(ChildResult, Options, Node);
			op->V[2] = ChildResult.op;
		}

		// W
		if (NodeScalar* pW = node.W.get())
		{
			GenerateScalar(ChildResult, Options, pW);
			op->V[3] = ChildResult.op;
		}
		else
		{
			Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
			Node->Value = 1.0f;
			GenerateScalar(ChildResult, Options, Node);
			op->V[3] = ChildResult.op;
		}

		result.op = op;
	}


	void CodeGenerator::GenerateColor_Arithmetic(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorArithmeticOperation>& Typed)
	{
		const NodeColorArithmeticOperation& node = *Typed;

		Ptr<ASTOpColorArithmetic> op = new ASTOpColorArithmetic;

		switch (node.Operation)
		{
		case NodeColorArithmeticOperation::EOperation::Add: op->Operation = FOperation::ArithmeticArgs::ADD; break;
		case NodeColorArithmeticOperation::EOperation::Subtract: op->Operation = FOperation::ArithmeticArgs::SUBTRACT; break;
		case NodeColorArithmeticOperation::EOperation::Multiply: op->Operation = FOperation::ArithmeticArgs::MULTIPLY; break;
		case NodeColorArithmeticOperation::EOperation::Divide: op->Operation = FOperation::ArithmeticArgs::DIVIDE; break;
		default:
			checkf(false, TEXT("Unknown arithmetic operation."));
			op->Operation = FOperation::ArithmeticArgs::NONE;
			break;
		}

		FColorGenerationResult ChildResult;

		// A
		if (NodeColor* pA = node.A.get())
		{
			GenerateColor(ChildResult, Options, pA );
			op->A = ChildResult.op;
		}
		else
		{
			op->A = GenerateMissingColorCode(TEXT("ColorArithmetic A"), Typed->GetMessageContext());
		}

		// B
		if (NodeColor* pB = node.B.get())
		{
			GenerateColor(ChildResult, Options, pB);
			op->B = ChildResult.op;
		}
		else
		{
			op->B = GenerateMissingColorCode(TEXT("ColorArithmetic B"), Typed->GetMessageContext());
		}

		result.op = op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_ToSRGB(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorToSRGB>& Typed)
	{
		Ptr<ASTOpColorToSRGB> Op = new ASTOpColorToSRGB;
		if (Typed->Color)
		{
			FColorGenerationResult ChildResult;
			GenerateColor(ChildResult, Options, Typed->Color);
			Op->Color = ChildResult.op;
		}

		result.op = Op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_Table(FColorGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorTable>& Typed)
	{
		const NodeColorTable& node = *Typed;

		result.op = GenerateTableSwitch<NodeColorTable, ETableColumnType::Color, EOpType::CO_SWITCH>(node,
			[this, &Options](const NodeColorTable& node, int32 colIndex, int32 row, FErrorLog*)
			{
				Ptr<NodeColorConstant> CellData = new NodeColorConstant();
				FVector4f Color = node.Table->GetPrivate()->Rows[row].Values[colIndex].Color;

				// Break the NaN codification for colors that will end up generating an image
				if (Options.bIsImage && FMath::IsNaN(node.Table->GetPrivate()->Rows[row].Values[colIndex].Color[0]))
				{
					Color = FVector4f(0.0f, 0.0f, 0.0f, 1.0f);
				}

				CellData->Value = Color;
				FColorGenerationResult BranchResults;
				GenerateColor(BranchResults, Options, CellData );
				return BranchResults.op;
			});
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateColor_MaterialBreak(FColorGenerationResult& Result, const FGenericGenerationOptions& Options, const Ptr<const NodeColorMaterialBreak>& Typed)
	{
		Ptr<ASTOpMaterialBreak> MaterialOp = new ASTOpMaterialBreak();

		// Set the key that identifies the break node parameter
		MaterialOp->ParameterKey = Typed->ParameterKey;

		// Generate Material Source
		FMaterialGenerationOptions MaterialSourceOptions;
		FMaterialGenerationResult MaterialSourceResult;

		GenerateMaterial(MaterialSourceOptions, MaterialSourceResult, Typed->MaterialSource);
		MaterialOp->Material = MaterialSourceResult.op;
		MaterialOp->Type = EOpType::CO_MATERIAL_BREAK;

		Result.op = MaterialOp;
	}


	void CodeGenerator::GenerateColor_External(FColorGenerationResult& Result, const FGenericGenerationOptions& InOptions, const Ptr<const NodeColorExternal>& InNode)
	{
		FExternalGenerationOptions Options;
		Options.ColorOptions = InOptions;
					
		FExternalTask ExtensionResult = GenerateExternal(Options, InNode->Node);
		WaitTask(ExtensionResult);
		
		Result.op = ExtensionResult.GetResult().Op;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> CodeGenerator::GenerateMissingColorCode(const TCHAR* StrWhere, const void* ErrorContext)
	{
		// Log a warning
		FString Msg = FString::Printf(TEXT("Required connection not found: %s"), StrWhere);
		ErrorLog->Add(Msg, ELMT_ERROR, ErrorContext);

		// Create a constant color node
		Ptr<NodeColorConstant> pNode = new NodeColorConstant();
		pNode->Value = FVector4f(1, 1, 0, 1);

		FColorGenerationResult Result;
		FGenericGenerationOptions Options;
		GenerateColor(Result, Options, pNode);

		return Result.op;
	}


}
