// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Layout.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantExtensionData.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeExtensionData.h"
#include "MuT/NodeExtensionDataConstant.h"
#include "MuT/NodeExtensionDataSwitch.h"
#include "MuT/NodeExtensionDataVariation.h"


namespace UE::Mutable::Private
{
class Node;

	//---------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateExtensionData(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions& Options, const Ptr<const NodeExtensionData>& InUntypedNode)
	{
		if (!InUntypedNode)
		{
			OutResult = FExtensionDataGenerationResult();
			return;
		}

		// See if it was already generated
		const FGeneratedExtensionDataCacheKey Key = InUntypedNode.get();
		FGeneratedExtensionDataMap::ValueType* CachedResult = GeneratedExtensionData.Find(Key);
		if (CachedResult)
		{
			OutResult = *CachedResult;
			return;
		}

		const NodeExtensionData* Node = InUntypedNode.get();

		// Generate for each different type of node
		switch (Node->GetType()->Type)
		{
			case Node::EType::ExtensionDataConstant:  GenerateExtensionData_Constant(OutResult, Options, static_cast<const NodeExtensionDataConstant*>(Node)); break;
			case Node::EType::ExtensionDataSwitch:    GenerateExtensionData_Switch(OutResult, Options, static_cast<const NodeExtensionDataSwitch*>(Node)); break;
			case Node::EType::ExtensionDataVariation: GenerateExtensionData_Variation(OutResult, Options, static_cast<const NodeExtensionDataVariation*>(Node)); break;
			default: check(false);
		}

		// Cache the result
		GeneratedExtensionData.Add(Key, OutResult);
	}

	//---------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateExtensionData_Constant(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions& Options, const NodeExtensionDataConstant* Constant)
	{
		Ptr<ASTOpConstantExtensionData> Op = new ASTOpConstantExtensionData();
		Op->ExtensionDataId = Constant->ExtensionDataId;

		OutResult.Op = Op;
	}

	//---------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateExtensionData_Switch(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions& Options, const class NodeExtensionDataSwitch* Switch)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeExtensionDataSwitch);

		if (Switch->Options.IsEmpty())
		{
			Ptr<ASTOp> MissingOp = GenerateMissingExtensionDataCode(TEXT("Switch option"), Switch->GetMessageContext());
			OutResult.Op = MissingOp;
			return;
		}

		Ptr<ASTOpSwitch> Op = new ASTOpSwitch;
		Op->Type = EOpType::ED_SWITCH;

		// Variable
		if (Switch->Parameter)
		{
			FScalarGenerationResult ParamResult; 
			GenerateScalar(ParamResult, Options, Switch->Parameter.get());
			Op->Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
			Op->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Switch->GetMessageContext());
		}

		// Options
		for (int32 OptionIndex = 0; OptionIndex < Switch->Options.Num(); ++OptionIndex)
		{
			Ptr<ASTOp> Branch;
			if (Switch->Options[OptionIndex])
			{
				Ptr<NodeExtensionData> SwitchOption = Switch->Options[OptionIndex];

				FExtensionDataGenerationResult OptionResult;
				GenerateExtensionData(OptionResult, Options, SwitchOption);

				Branch = OptionResult.Op;
			}
			else
			{
				// This argument is required
				Branch = GenerateMissingExtensionDataCode(TEXT("Switch option"), Switch->GetMessageContext());
			}
			Op->Cases.Emplace(static_cast<int16_t>(OptionIndex), Op, Branch);
		}

		OutResult.Op = Op;
	}

	//---------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateExtensionData_Variation(FExtensionDataGenerationResult& OutResult, const FGenericGenerationOptions& Options, const class NodeExtensionDataVariation* Variation)
	{
		Ptr<ASTOp> CurrentOp;

		// Default case
		if (Variation->DefaultValue)
		{
			FExtensionDataGenerationResult DefaultResult;
			GenerateExtensionData(DefaultResult, Options, Variation->DefaultValue);

			CurrentOp = DefaultResult.Op;
		}

		// Process variations in reverse order, since conditionals are built bottom-up.
		for (int32 VariationIndex = Variation->Variations.Num() - 1; VariationIndex >= 0; --VariationIndex)
		{
			const FString& Tag = Variation->Variations[VariationIndex].Tag;
			const int32 TagIndex = FirstPass.Tags.IndexOfByPredicate([Tag](const FirstPassGenerator::FTag& CandidateTag)
			{
				return CandidateTag.Tag == Tag;
			});

			if (TagIndex == INDEX_NONE)
			{
				const FString Msg = FString::Printf(TEXT("Unknown tag found in Extension Data variation [%s]"), *Tag);
				ErrorLog->Add(Msg, ELMT_WARNING, Variation->GetMessageContext());
				continue;
			}

			Ptr<ASTOp> VariationOp;
			if (Ptr<NodeExtensionData> VariationValue = Variation->Variations[VariationIndex].Value)
			{
				FExtensionDataGenerationResult VariationResult;
				GenerateExtensionData(VariationResult, Options, VariationValue);

				VariationOp = VariationResult.Op;
			}
			else
			{
				// This argument is required
				VariationOp = GenerateMissingExtensionDataCode(TEXT("Variation option"), Variation->GetMessageContext());
			}

			Ptr<ASTOpConditional> Conditional = new ASTOpConditional;
			Conditional->type = EOpType::ED_CONDITIONAL;
			Conditional->no = CurrentOp;
			Conditional->yes = VariationOp;
			Conditional->condition = FirstPass.Tags[TagIndex].GenericCondition;

			CurrentOp = Conditional;
		}

		OutResult.Op = CurrentOp;
	}

	Ptr<ASTOp> CodeGenerator::GenerateMissingExtensionDataCode(const TCHAR* StrWhere, const void* ErrorContext)
	{
		// Log a warning
		const FString Msg = FString::Printf(TEXT("Required connection not found: %s"), StrWhere);
		ErrorLog->Add(Msg, ELMT_ERROR, ErrorContext);

		// Create a constant extension data
		Ptr<const NodeExtensionDataConstant> Node = new NodeExtensionDataConstant;

		FExtensionDataGenerationResult Result;
		FGenericGenerationOptions Options;
		GenerateExtensionData_Constant(Result, Options, Node.get());

		return Result.Op;
	}
}

