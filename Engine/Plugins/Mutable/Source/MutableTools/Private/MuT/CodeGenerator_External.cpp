// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGenerator.h"
#include "Image/ImageBuilder.h"
#include "MuR/External/FloatAdapter.h"
#include "MuR/External/MaterialAdapter.h"
#include "MuR/External/MeshAdapter.h"
#include "MuR/External/Operation.h"
#include "MuR/External/TextureAdapter.h"
#include "MuR/External/VectorAdapter.h"
#include "MuT/NodeExternalOperation.h"
#include "MuT/NodeExternalParameter.h"
#include "MuT/NodeExternalSwitch.h"


namespace UE::Mutable::Private
{
	FExternalTask CodeGenerator::GenerateExternal(const FExternalGenerationOptions& Options, const Ptr<const NodeExternal>& ExternalNode)
	{
		if (!ExternalNode)
		{
			return Tasks::MakeCompletedTask<FExternalGenerationResult>();
		}

		// See if it was already generated
		FGeneratedExternalCacheKey Key;
		Key.Node = ExternalNode.get();
		Key.Options = Options;
		
		{
			UE::TUniqueLock Lock(GeneratedExternal.Mutex);
			if (FGeneratedExternalMap::ValueType* Found = GeneratedExternal.Map.Find(Key))
			{
				return *Found;
			}
		}
		
		// Generate for each different type of node
		FExternalTask Result;
	
		switch (ExternalNode->GetType()->Type)
		{
		case Node::EType::ExternalOperation:
			{
				Ptr<const NodeExternalOperation> Operation = static_pointer_cast<const NodeExternalOperation>(ExternalNode);
				Result = GenerateExternal_Operation(Options, Operation);
				break;	
			}
			
		case Node::EType::ExternalParameter:
			{
				Ptr<const NodeExternalParameter> Parameter = static_pointer_cast<const NodeExternalParameter>(ExternalNode);
				Result = GenerateExternal_Parameter(Options, Parameter);
				break;
			}
			
		case Node::EType::ExternalSwitch:
			{
				Ptr<const NodeExternalSwitch> Parameter = static_pointer_cast<const NodeExternalSwitch>(ExternalNode);
				Result = GenerateExternal_Switch(Options, Parameter);
				break;
			}
			
		default:
			{
				Result = Tasks::MakeCompletedTask<FExternalGenerationResult>();
			unimplemented();
		}
		}
	
		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedExternal.Mutex);
			GeneratedExternal.Map.Add(Key, Result);
		}
		
		return Result;
	}


	FExternalTask CodeGenerator::GenerateExternal_Operation(const FExternalGenerationOptions& Options, const Ptr<const NodeExternalOperation>& ExternalNode)
	{
		return Tasks::Launch(UE_SOURCE_LOCATION, [Options, ExternalNode, this]()
	{
			FExternalGenerationResult Result;

		Ptr<ASTOpExternal> External = new ASTOpExternal();
		External->OperationInstancedStruct = ExternalNode->OperationInstancedStruct;

		const UScriptStruct* OutputType = ExternalNode->OperationInstancedStruct.GetPtr<FExternalOperation>()->GetOutput().Value;
		
		if (OutputType == FMeshAdapter::StaticStruct())
		{
			External->Type = EOpType::ME_EXTERNAL;
		}
		else if (OutputType == FTextureAdapter::StaticStruct())
		{
			External->Type = EOpType::IM_EXTERNAL;
		}
		else if (OutputType == FFloatAdapter::StaticStruct())
		{
			External->Type = EOpType::SC_EXTERNAL;
		}
		else if (OutputType == FVectorAdapter::StaticStruct())
		{
			External->Type = EOpType::CO_EXTERNAL;
		}
		else if (OutputType == FMaterialAdapter::StaticStruct())
		{
			External->Type = EOpType::MI_EXTERNAL;
		}
		else
		{
			External->Type = EOpType::IS_EXTERNAL;
		}
		
		const int32 NumInputs = ExternalNode->Inputs.Num();
    			
		External->Inputs.Reserve(NumInputs);
		
		for (int32 Index = 0; Index < NumInputs; ++Index)
		{
			const Ptr<Node>& Input = ExternalNode->Inputs[Index];
    				
			if (Input->GetType()->IsA(NodeMesh::GetStaticType()))
			{
				FMeshGenerationStaticOptions MeshStaticOptions = Options.MeshStaticOptions.IsSet() ?
					Options.MeshStaticOptions.GetValue() :
					FMeshGenerationStaticOptions(0, 0);

				FMeshOptionsTask MeshDynamicOptions = Options.MeshDynamicOptions.IsSet() ?
					Options.MeshDynamicOptions.GetValue() :
					UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>();
    					
				FMeshTask Task = GenerateMesh(MeshStaticOptions, MeshDynamicOptions, static_pointer_cast<NodeMesh>(Input));

				WaitTask(Task);
				FMeshGenerationResult MeshResult = Task.GetResult();

					External->Inputs.Emplace(External.get(), MeshResult.MeshOp);
				
					if (!Result.MeshGenerationResult.IsSet())
				{
						Result.MeshGenerationResult = FMeshGenerationResult();
				}
				
				// Merge all generation results from all inputs.
					Result.MeshGenerationResult->GeneratedLayouts.Append(MeshResult.GeneratedLayouts);
					Result.MeshGenerationResult->LayoutOps.Append(MeshResult.LayoutOps);
					Result.MeshGenerationResult->ExtraMeshLayouts.Append(MeshResult.ExtraMeshLayouts);
			}
			else if (Input->GetType()->IsA(NodeImage::GetStaticType()))
			{
				FImageGenerationOptions ImageOptions = Options.ImageOptions.IsSet() ?
					Options.ImageOptions.GetValue() :
					FImageGenerationOptions(0, 0);
    					
				FImageGenerationResult InputResult;
				GenerateImage(ImageOptions, InputResult, static_pointer_cast<NodeImage>(Input));

					External->Inputs.Emplace(External.get(), InputResult.op);
			}
			else if (Input->GetType()->IsA(NodeScalar::GetStaticType()))
			{
				FGenericGenerationOptions ScalarOptions = Options.ScalarOptions.IsSet() ?
					Options.ScalarOptions.GetValue() :
					FGenericGenerationOptions();
    					
				FScalarGenerationResult InputResult;
				GenerateScalar(InputResult, ScalarOptions, static_pointer_cast<NodeScalar>(Input));

					External->Inputs.Emplace(External.get(), InputResult.op);
			}
			else if (Input->GetType()->IsA(NodeColor::GetStaticType()))
			{
				 FGenericGenerationOptions ColorOptions = Options.ColorOptions.IsSet() ?
					Options.ColorOptions.GetValue() :
					FGenericGenerationOptions();
    					
				FColorGenerationResult InputResult;
				GenerateColor(InputResult, ColorOptions, static_pointer_cast<NodeColor>(Input));

					External->Inputs.Emplace(External.get(), InputResult.op);
			}
			else if (Input->GetType()->IsA(NodeMaterial::GetStaticType()))
			{
				FMaterialGenerationOptions MaterialOptions = Options.MaterialOptions.IsSet() ?
				   Options.MaterialOptions.GetValue() :
				   FMaterialGenerationOptions();
    					
				FMaterialGenerationResult InputResult;
				GenerateMaterial(MaterialOptions, InputResult, static_pointer_cast<NodeMaterial>(Input));

					External->Inputs.Emplace(External.get(), InputResult.op);
			}
			else if (Input->GetType()->IsA(NodeExternal::GetStaticType()))
			{
				FExternalTask InputResult = GenerateExternal(Options, static_pointer_cast<NodeExternal>(Input));

				External->Inputs.Emplace(ASTChild::FInvalidTag());

				Tasks::AddNested(Tasks::Launch(UE_SOURCE_LOCATION, [InputResult, External, Index]() mutable
				{
					External->Inputs[Index] = ASTChild(External.get(), InputResult.GetResult().Op);
				},
				InputResult));
			}
			else
			{
				External->Inputs.Emplace(External.get());
				unimplemented(); // GenerateMutableSourceExtension has compiled an input type that is not supported.
			}
		}
		
		if (Result.MeshGenerationResult.IsSet())
		{
				Result.MeshGenerationResult->MeshOp = External;	
				Result.MeshGenerationResult->BaseMeshOp = External;			
		}
		
			Result.Op = External;
			
			return Result;
		});
	}


	FExternalTask CodeGenerator::GenerateExternal_Parameter(const FExternalGenerationOptions& Options,	const Ptr<const NodeExternalParameter>& ExternalNode)
	{
		Ptr<ASTOpParameter> Op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(ExternalNode.get());
			if (!Found)
			{
				FParameterDesc Param;
				Param.Name = ExternalNode->Name;
				bool bParseOk = FGuid::Parse(ExternalNode->UID, Param.UID);
				check(bParseOk);
				Param.Type = EParameterType::InstancedStruct;
				Param.DefaultValue.Set<FParamInstancedStructType>(ExternalNode->DefaultValue);

				Op = new ASTOpParameter();
				Op->Type = EOpType::IS_PARAMETER;
				Op->Parameter = Param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(ExternalNode.get(), Op);
			}
			else
			{
				Op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 RangeIndex = 0; RangeIndex < ExternalNode->Ranges.Num(); ++RangeIndex)
			{
				FGenericGenerationOptions GenericOptions;

				if (Options.MeshStaticOptions.IsSet())
				{
					GenericOptions = Options.MeshStaticOptions.GetValue();
				}
				else if (Options.ColorOptions.IsSet())
				{
					GenericOptions = Options.ColorOptions.GetValue();
				}
				else if (Options.ImageOptions.IsSet())
				{
					GenericOptions = Options.ImageOptions.GetValue().GenericOptions;
				}
				else if (Options.MaterialOptions.IsSet())
				{
					GenericOptions = Options.MaterialOptions.GetValue().GenericOptions;
				}
				else if (Options.ScalarOptions.IsSet())
				{
					GenericOptions = Options.ScalarOptions.GetValue();
				}
				
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, GenericOptions, ExternalNode->Ranges[RangeIndex]);
				Op->Ranges.Emplace(Op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		FExternalGenerationResult Result;
		Result.Op = Op;
		
		return Tasks::MakeCompletedTask<FExternalGenerationResult>(Result);
	}


	FExternalTask CodeGenerator::GenerateExternal_Switch(const FExternalGenerationOptions& Options, const Ptr<const NodeExternalSwitch>& ExternalNode)
	{
		return Tasks::Launch(UE_SOURCE_LOCATION, [Options, ExternalNode, this]()
	{
		if (ExternalNode->Options.Num() == 0)
		{
				FExternalGenerationResult Result;
		
			// No options in the switch!
			Ptr<ASTOp> MissingOp = GenerateMissingExternalCode(TEXT("Switch option"),
				ExternalNode->GetMessageContext());
			
				Result.Op = MissingOp;

				return Result;
		}

		Ptr<ASTOpSwitch> Op = new ASTOpSwitch();
		Op->Type = EOpType::IS_SWITCH;

		// Variable value
		if (ExternalNode->Parameter)
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GetGenericOptions(), ExternalNode->Parameter.get());
			Op->Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
			Op->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, ExternalNode->GetMessageContext());
		}

			// Options
			const int32 NumOptions = ExternalNode->Options.Num();
			Op->Cases.Reserve(NumOptions);

			for (int32 OptionIndex = 0; OptionIndex < ExternalNode->Options.Num(); ++OptionIndex)
			{
				Ptr<ASTOp> Branch;
				if (ExternalNode->Options[OptionIndex])
				{
						FExternalTask ChildResult = GenerateExternal(Options, ExternalNode->Options[OptionIndex].get());

						Op->Cases.Emplace(ASTOpSwitch::FCase::FInvalidTag());
				
						Tasks::AddNested(Tasks::Launch(UE_SOURCE_LOCATION, [ChildResult, Op, OptionIndex]() mutable
						{
							FExternalGenerationResult Result = ChildResult.GetResult();
							Op->Cases[OptionIndex] = ASTOpSwitch::FCase(static_cast<int16_t>(OptionIndex), Op, Result.Op);
						}, 
						ChildResult));
				}
				else
				{
					Ptr<ASTOp> Result = GenerateMissingExternalCode(TEXT("Switch option"), ExternalNode->GetMessageContext());
					Op->Cases.Emplace(static_cast<int16_t>(OptionIndex), Op, Result);
				}
			}
			
			FExternalGenerationResult Result;
			Result.Op = Op;

			return Result;
		});
	}
	
	
	Ptr<ASTOp> CodeGenerator::GenerateMissingExternalCode(const TCHAR* StrWhere, const void* ErrorContext)
	{
		// Log a warning
		const FString Msg = FString::Printf(TEXT("Required connection not found: %s"), StrWhere);
		ErrorLog->Add(Msg, ELMT_ERROR, ErrorContext);
		
		return {};
	}
}
