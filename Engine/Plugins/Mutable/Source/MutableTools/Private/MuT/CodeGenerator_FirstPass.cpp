// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/CodeGenerator_FirstPass.h"

#include "MuT/CodeGenerator.h"
#include "Misc/AssertionMacros.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConstantBool.h"
#include "MuT/ASTOpBoolEqualIntConst.h"
#include "MuT/ASTOpBoolAnd.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeMaterialConstant.h"
#include "MuT/NodeMaterialExternal.h"
#include "MuT/NodeMaterialParameter.h"
#include "MuT/NodeMaterialSkeletalMeshBreak.h"
#include "MuT/NodeMaterialSkeletalMeshObjectBreak.h"
#include "MuT/NodeMaterialSwitch.h"
#include "MuT/NodeMaterialTable.h"
#include "MuT/NodeMaterialVariation.h"
#include "MuT/NodeObject.h"
#include "MuT/NodeObjectNew.h"
#include "MuT/NodeSkeletalMeshConvert.h"
#include "MuT/NodeSurface.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeSurfaceVariation.h"
#include "MuT/NodeSurfaceSwitch.h"
#include "MuT/NodeSkeletalMeshNew.h"
#include "MuT/NodeSkeletalMeshMerge.h"
#include "MuT/NodeSkeletalMeshModify.h"
#include "MuT/NodeSkeletalMeshMorph.h"
#include "MuT/NodeSkeletalMeshReshape.h"
#include "MuT/NodeSkeletalMeshObjectConvert.h"
#include "MuT/NodeSkeletalMeshObjectSwitch.h"
#include "MuT/NodeSkeletalMeshTransform.h"
#include "MuT/NodeSkeletalMeshTransformWithBone.h"
#include "MuT/NodeSkeletalMeshClipWithSkeletalMesh.h"


namespace UE::Mutable::Private
{

	//---------------------------------------------------------------------------------------------
	FirstPassGenerator::FirstPassGenerator()
	{
		// Default conditions when there is no restriction accumulated.
		CurrentStateCondition.Add(StateCondition());
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate(TSharedPtr<FErrorLog> InErrorLog,
		const Node* Root,
		bool bIgnoreStates,
		CodeGenerator* InGenerator)
	{
		MUTABLE_CPUPROFILER_SCOPE(FirstPassGenerate);

		Generator = InGenerator;
		ErrorLog = InErrorLog;

		// Step 1: collect all objects, surfaces and object conditions
		if (Root)
		{
			Generate_Generic(Root);
		}

		// Step 2: Collect all tags and a list of the surfaces and modifiers that activate them
		for (int32 s = 0; s < Surfaces.Num(); ++s)
		{
			// Collect the tags in new surfaces
			for (int32 t = 0; t < Surfaces[s].Node->Tags.Num(); ++t)
			{
				int32 tag = -1;
				const FString& tagStr = Surfaces[s].Node->Tags[t];
				for (int32 i = 0; i < Tags.Num() && tag < 0; ++i)
				{
					if (Tags[i].Tag == tagStr)
					{
						tag = i;
					}
				}

				// New tag?
				if (tag < 0)
				{
					tag = Tags.Num();
					FTag newTag;
					newTag.Tag = tagStr;
					Tags.Add(newTag);
				}

				if (Tags[tag].Surfaces.Find(s) == INDEX_NONE)
				{
					Tags[tag].Surfaces.Add(s);
				}
			}
		}

		// TODO: Modifier's enabling tags?
		for (int32 ModifierIndex = 0; ModifierIndex < Modifiers.Num(); ++ModifierIndex)
		{
			// Collect the tags in the modifiers
			for (const FString& ModifierTag : Modifiers[ModifierIndex].Node->EnableTags)
			{
				int32 TagIndex = Tags.IndexOfByPredicate([&](const FTag& Candidate)
					{
						return Candidate.Tag == ModifierTag;
					});

				// New tag?
				if (TagIndex < 0)
				{
					FTag newTag;
					newTag.Tag = ModifierTag;
					TagIndex = Tags.Add(newTag);
				}

				if (Tags[TagIndex].Modifiers.Find(ModifierIndex) == INDEX_NONE)
				{
					Tags[TagIndex].Modifiers.Add(ModifierIndex);
				}
			}
		}

		// Step 3: Create default state if necessary
		if (bIgnoreStates)
		{
			States.Empty();
		}

		if (States.IsEmpty())
		{
			FObjectState Data;
			Data.Name = "Default";
			States.Add(Data);
		}
	}


	void FirstPassGenerator::Generate_Generic(const Node* Root)
	{
		if (!Root)
		{
			return;
		}

		if (Root->GetType() == NodeSurfaceNew::GetStaticType())
		{
			Generate_SurfaceNew(static_cast<const NodeSurfaceNew*>(Root));
		}
		else if (Root->GetType() == NodeSurfaceVariation::GetStaticType())
		{
			Generate_SurfaceVariation(static_cast<const NodeSurfaceVariation*>(Root));
		}
		else if (Root->GetType() == NodeSurfaceSwitch::GetStaticType())
		{
			Generate_SurfaceSwitch(static_cast<const NodeSurfaceSwitch*>(Root));
		}
		else if (Root->GetType() == NodeComponentNew::GetStaticType())
		{
			Generate_ComponentNew(static_cast<const NodeComponentNew*>(Root));
		}
		else if (Root->GetType() == NodeModifierSkeletalMeshMerge::GetStaticType())
		{
			Generate_ModifierSkeletalMeshMerge(static_cast<const NodeModifierSkeletalMeshMerge*>(Root));
		}
		else if (Root->GetType() == NodeComponentSwitch::GetStaticType())
		{
			Generate_ComponentSwitch(static_cast<const NodeComponentSwitch*>(Root));
		}
		else if (Root->GetType() == NodeComponentVariation::GetStaticType())
		{
			Generate_ComponentVariation(static_cast<const NodeComponentVariation*>(Root));
		}
		else if (Root->GetType() == NodeObjectNew::GetStaticType())
		{
			Generate_ObjectNew(static_cast<const NodeObjectNew*>(Root));
		}
		else if (Root->GetType() == NodeObjectGroup::GetStaticType())
		{
			Generate_ObjectGroup(static_cast<const NodeObjectGroup*>(Root));
		}
		else if (Root->GetType() == NodeLOD::GetStaticType())
		{
			Generate_LOD(static_cast<const NodeLOD*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshNew::GetStaticType())
		{
			Generate_SkeletalMeshNew(static_cast<const NodeSkeletalMeshNew*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshConvert::GetStaticType())
		{
			Generate_SkeletalMeshConvert(static_cast<const NodeSkeletalMeshConvert*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshObjectParameter::GetStaticType())
		{
			// Do nothing.
		}
		else if (Root->GetType() == NodeSkeletalMeshMerge::GetStaticType())
		{
			Generate_SkeletalMeshMerge(static_cast<const NodeSkeletalMeshMerge*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshSwitch::GetStaticType())
		{
			Generate_SkeletalMeshSwitch(static_cast<const NodeSkeletalMeshSwitch*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshVariation::GetStaticType())
		{
			Generate_SkeletalMeshVariation(static_cast<const NodeSkeletalMeshVariation*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshObjectConvert::GetStaticType())
		{
			Generate_SkeletalMeshObjectConvert(static_cast<const NodeSkeletalMeshObjectConvert*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshMorph::GetStaticType())
		{
			Generate_SkeletalMeshMorph(static_cast<const NodeSkeletalMeshMorph*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshReshape::GetStaticType())
		{
			Generate_SkeletalMeshReshape(static_cast<const NodeSkeletalMeshReshape*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshModify::GetStaticType())
		{
			Generate_SkeletalMeshModify(static_cast<const NodeSkeletalMeshModify*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshObjectSwitch::GetStaticType())
		{
			Generate_SkeletalMeshObjectSwitch(static_cast<const NodeSkeletalMeshObjectSwitch*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshTransformWithBone::GetStaticType())
		{
			Generate_SkeletalMeshTransformWithBone(static_cast<const NodeSkeletalMeshTransformWithBone*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshClipWithSkeletalMesh::GetStaticType())
		{
			Generate_SkeletalMeshClipSkeletalMesh(static_cast<const NodeSkeletalMeshClipWithSkeletalMesh*>(Root));
		}
		else if (Root->GetType() == NodeSkeletalMeshTransform::GetStaticType())
		{
			Generate_SkeletalMeshTransform(static_cast<const NodeSkeletalMeshTransform*>(Root));
		}
		else if (Root->GetType()->IsA(NodeMaterialModify::GetStaticType()))
		{
			Generate_MaterialModify(static_cast<const NodeMaterialModify*>(Root));
		}
		else if (Root->GetType()->IsA(NodeMaterialSwitch::GetStaticType()))
		{
			Generate_MaterialSwitch(static_cast<const NodeMaterialSwitch*>(Root));
		}
		else if (Root->GetType()->IsA(NodeMaterialVariation::GetStaticType()))
		{
			Generate_MaterialVariation(static_cast<const NodeMaterialVariation*>(Root));
		}
		else if (Root->GetType()->IsA(NodeMaterialConstant::GetStaticType()))
		{
			// Do nothing.
		}
		else if (Root->GetType()->IsA(NodeMaterialParameter::GetStaticType()))
		{
			// Do nothing.
		}
		else if (Root->GetType()->IsA(NodeMaterialTable::GetStaticType()))
		{
			// Do nothing.
		}
		else if (Root->GetType()->IsA(NodeMaterialExternal::GetStaticType()))
		{
			// Do nothing.
		}
		else if (Root->GetType()->IsA(NodeMaterialSkeletalMeshObjectBreak::GetStaticType()))
		{
			// Do nothing.
		}
		else if (Root->GetType()->IsA(NodeMaterialSkeletalMeshBreak::GetStaticType()))
		{
			Generate_MaterialSkeletalMeshBreak(static_cast<const NodeMaterialSkeletalMeshBreak*>(Root));
		}
		else if (Root->GetType()->IsA(NodeSurfaceModifier::GetStaticType())) // Cast to super type.
		{
			Generate_SurfaceModifier(static_cast<const NodeSurfaceModifier*>(Root));
		}
		else
		{
			// This node type is not supported in this pass.
			check(false);
		}
	}

	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_SurfaceModifier(const NodeSurfaceModifier* InNode)
	{
		// Add the data about this modifier
		FModifier ThisData;
		ThisData.Node = InNode;
		Ptr<ASTOp> Dummy;
		ConditionContext.GetCurrent(ThisData.ObjectCondition, Dummy);
		ThisData.StateCondition = CurrentStateCondition.Last();
		ThisData.PositiveTags = CurrentPositiveTags;
		ThisData.NegativeTags = CurrentNegativeTags;
		Modifiers.Add(ThisData);
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_SurfaceNew(const NodeSurfaceNew* InNode)
	{
		// Add the data about this surface
		FSurface ThisData;
		ThisData.Node = InNode;
		ThisData.Component = CurrentComponent;
		ThisData.LOD = CurrentLOD;
		Ptr<ASTOp> Dummy;
		ConditionContext.GetCurrent(ThisData.ObjectCondition, ThisData.MeshCondition);
		ThisData.StateCondition = CurrentStateCondition.Last();
		ThisData.PositiveTags = CurrentPositiveTags;
		ThisData.NegativeTags = CurrentNegativeTags;
		const int32 SurfaceIndex = Surfaces.AddUnique(ThisData);

		if (ensure(SkeletalMeshes.Num() && CurrentLOD >= 0))
		{
			if (!SkeletalMeshes.Last().SurfacesPerLOD.IsValidIndex(CurrentLOD))
			{
				SkeletalMeshes.Last().SurfacesPerLOD.SetNum(CurrentLOD + 1);
			}

			SkeletalMeshes.Last().SurfacesPerLOD[CurrentLOD].Add(SurfaceIndex);
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_SurfaceVariation(const NodeSurfaceVariation* InNode)
	{
		switch (InNode->Type)
		{

		case NodeSurfaceVariation::VariationType::Tag:
		{
			// Process the default case

			// Make the default case nodes not active if any of the variation tags is enabled.
			TArray<FString> OldNegativeTags = CurrentNegativeTags;
			{
				for (int32 VariationIndex = 0; VariationIndex < InNode->Variations.Num(); ++VariationIndex)
				{
					CurrentNegativeTags.Add(InNode->Variations[VariationIndex].Tag);
				}

				for (const Ptr<NodeSurface>& SurfaceNode : InNode->DefaultSurfaces)
				{
					Generate_Generic(SurfaceNode.get());
				}
				for (const Ptr<NodeSurfaceModifier>& ModifierNode : InNode->DefaultModifiers)
				{
					Generate_SurfaceModifier(ModifierNode.get());
				}

				// Restore the NegativeTags to be able to process the non default cases
				CurrentNegativeTags = OldNegativeTags;
			}


			// Process the variations so each node generated from each of them does have as the activation tag the one defined in the variation.
			for (int32 VariationIndex = 0; VariationIndex < InNode->Variations.Num(); ++VariationIndex)
			{
				CurrentPositiveTags.Add(InNode->Variations[VariationIndex].Tag);
				{
					for (const Ptr<NodeSurface>& SurfaceNode : InNode->Variations[VariationIndex].Surfaces)
					{
						Generate_Generic(SurfaceNode.get());
					}

					for (const Ptr<NodeSurfaceModifier>& ModifierNode : InNode->Variations[VariationIndex].Modifiers)
					{
						Generate_SurfaceModifier(ModifierNode.get());
					}
				}
				CurrentPositiveTags.Pop();		// Restore the state prior to processing this variation

				// Tags have an order in a variation node: the current tag should prevent any following
				// variation surface
				CurrentNegativeTags.Add(InNode->Variations[VariationIndex].Tag);
			}

			CurrentNegativeTags = OldNegativeTags;

			break;
		}


		case NodeSurfaceVariation::VariationType::State:
		{
			const int32 StateCount = States.Num();

			// Default
			{
				// Store the states for the default branch here
				StateCondition DefaultStates;
				{
					StateCondition AllTrue;
					AllTrue.Init(true, StateCount);
					DefaultStates = CurrentStateCondition.Last().IsEmpty()
						? AllTrue
						: CurrentStateCondition.Last();

					for (const NodeSurfaceVariation::FVariation& Variation : InNode->Variations)
					{
						for (int32 StateIndex = 0; StateIndex < StateCount; ++StateIndex)
						{
							if (States[StateIndex].Name == Variation.Tag)
							{
								// Remove this state from the default options, since it has its own variation
								DefaultStates[StateIndex] = false;
							}
						}
					}
				}

				CurrentStateCondition.Add(DefaultStates);

				for (const Ptr<NodeSurface>& SurfaceNode : InNode->DefaultSurfaces)
				{
					Generate_Generic(SurfaceNode.get());
				}
				for (const Ptr<NodeSurfaceModifier>& ModifierNode : InNode->DefaultModifiers)
				{
					Generate_Generic(ModifierNode.get());
				}

				CurrentStateCondition.Pop();
			}

			// Variation branches
			for (const auto& v : InNode->Variations)
			{
				// Store the states for this variation here
				StateCondition variationStates;
				variationStates.Init(false, StateCount);

				for (int32 StateIndex = 0; StateIndex < StateCount; ++StateIndex)
				{
					if (States[StateIndex].Name == v.Tag)
					{
						variationStates[StateIndex] = true;
					}
				}

				CurrentStateCondition.Add(variationStates);

				for (const Ptr<NodeSurface>& SurfaceNode : v.Surfaces)
				{
					Generate_Generic(SurfaceNode.get());
				}
				for (const Ptr<NodeSurfaceModifier>& ModifierNode : v.Modifiers)
				{
					Generate_SurfaceModifier(ModifierNode.get());
				}

				CurrentStateCondition.Pop();
			}

			break;
		}

		default:
			// Case not implemented.
			check(false);
			break;
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_SurfaceSwitch(const NodeSurfaceSwitch* InNode)
	{
		if (InNode->Options.Num() == 0)
		{
			// No options in the switch!
			return;
		}

		// Prepare the enumeration parameter
		CodeGenerator::FGenericGenerationOptions Options;
		CodeGenerator::FScalarGenerationResult ScalarResult;
		if (InNode->Parameter)
		{
			Generator->GenerateScalar(ScalarResult, Options, InNode->Parameter);
		}
		else
		{
			// This argument is required
			ScalarResult.op = Generator->GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, InNode->GetMessageContext());
		}

		// Parse the options
		for (int32 t = 0; t < InNode->Options.Num(); ++t)
		{
			if (InNode->Options[t])
			{
				// Create a comparison operation as the boolean parameter for the child
				Ptr<ASTOpBoolEqualIntConst> ParamOp = new ASTOpBoolEqualIntConst;
				ParamOp->Value = ScalarResult.op;
				ParamOp->Constant = t;

				ConditionContext.Push(ParamOp);


				Generate_Generic(InNode->Options[t].get());

				ConditionContext.Pop();
			}
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_ComponentNew(const NodeComponentNew* InNode)
	{
		// Add the data about this surface
		FComponent ThisData;
		ThisData.Component = InNode;

		Ptr<ASTOp> Dummy;
		ConditionContext.GetCurrent(ThisData.ObjectCondition, Dummy);

		ThisData.PositiveTags = CurrentPositiveTags;
		ThisData.NegativeTags = CurrentNegativeTags;
		Components.Add(ThisData);

		CurrentComponent = InNode;
		ConditionContext.Push(nullptr, true);

		bIsSkeletalMeshEntryPoint = true;

		Generate_Generic(InNode->SkeletalMeshObject.get());

		bIsSkeletalMeshEntryPoint = false;

		ConditionContext.Pop();
		CurrentComponent = nullptr;
		
		Generate_Generic(InNode->OverlayMaterial.get());
		
		for (const TTuple<const FName, Ptr<NodeMaterial>>& Material : InNode->OverlayMaterials)
		{
			Generate_Generic(Material.Value.get());
		}
		
		for (const TTuple<const FName, Ptr<NodeMaterial>>& Material : InNode->OverrideMaterials)
		{
			Generate_Generic(Material.Value.get());
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_ModifierSkeletalMeshMerge(const NodeModifierSkeletalMeshMerge* InNode)
	{
		CurrentSkeletalMeshName = InNode->ParentSkeletalMeshName;
		bIsSkeletalMeshEntryPoint = true;

		Generate_Generic(InNode->ToAddSkeletalMesh.get());

		bIsSkeletalMeshEntryPoint = false;
		CurrentSkeletalMeshName = {};
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_ComponentVariation(const NodeComponentVariation* InNode)
	{
		// Any of the tags in the variations would prevent the default surface
		TArray<FString> OldNegativeTags = CurrentNegativeTags;
		for (int32 v = 0; v < InNode->Variations.Num(); ++v)
		{
			CurrentNegativeTags.Add(InNode->Variations[v].Tag);
		}

		Generate_Generic(InNode->DefaultComponent.get());

		CurrentNegativeTags = OldNegativeTags;

		for (int32 v = 0; v < InNode->Variations.Num(); ++v)
		{
			CurrentPositiveTags.Add(InNode->Variations[v].Tag);
			Generate_Generic(InNode->Variations[v].Component.get());

			CurrentPositiveTags.Pop();

			// Tags have an order in a variation node: the current tag should prevent any following variation
			CurrentNegativeTags.Add(InNode->Variations[v].Tag);
		}

		CurrentNegativeTags = OldNegativeTags;
	}


	void FirstPassGenerator::AddSkeletalMeshDataForNode(const NodeSkeletalMesh* InNode)
	{
		FSkeletalMeshData SkeletalMeshData;
		SkeletalMeshData.ParentSkeletalMeshName = CurrentSkeletalMeshName;
		SkeletalMeshData.SkeletalMeshNode = InNode;
		Ptr<ASTOp> Dummy;
		ConditionContext.GetCurrent(SkeletalMeshData.ObjectCondition, Dummy);
		SkeletalMeshData.bIsEntryPoint = bIsSkeletalMeshEntryPoint;
		SkeletalMeshes.Add(SkeletalMeshData);
	}


	void FirstPassGenerator::Generate_SkeletalMeshNew(const NodeSkeletalMeshNew* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		CurrentLOD = 0;
		for (const Ptr<NodeLOD>& c : InNode->LODs)
		{
			if (c)
			{
				Generate_LOD(c.get());
			}
			++CurrentLOD;
		}
		CurrentLOD = -1;
	}

	void FirstPassGenerator::Generate_SkeletalMeshConvert(const NodeSkeletalMeshConvert* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);
	}

	void FirstPassGenerator::Generate_SkeletalMeshMerge(const NodeSkeletalMeshMerge* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);

		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->BaseSkeletalMesh.get());
		Generate_Generic(InNode->ToAddSkeletalMesh.get());

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();
	}
	
	
	void FirstPassGenerator::Generate_SkeletalMeshTransformWithBone(const NodeSkeletalMeshTransformWithBone* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);

		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->Source.get());
		
		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();
	}


	void FirstPassGenerator::Generate_SkeletalMeshSwitch(const NodeSkeletalMeshSwitch* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);
		
		if (InNode->Options.Num() == 0)
		{
			// No options in the switch!
			return;
		}

		// Prepare the enumeration parameter
		CodeGenerator::FGenericGenerationOptions Options;
		CodeGenerator::FScalarGenerationResult ScalarResult;
		if (InNode->Parameter)
		{
			Generator->GenerateScalar(ScalarResult, Options, InNode->Parameter);
		}
		else
		{
			// This argument is required
			ScalarResult.op = Generator->GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, InNode->GetMessageContext());
		}

		// Parse the options
		for (int32 OptionIndex = 0; OptionIndex < InNode->Options.Num(); ++OptionIndex)
		{
			if (InNode->Options[OptionIndex])
			{
				// Create a comparison operation as the boolean parameter for the child
				Ptr<ASTOpBoolEqualIntConst> ParamOp = new ASTOpBoolEqualIntConst();
				ParamOp->Value = ScalarResult.op;
				ParamOp->Constant = OptionIndex;
				
				// Combine the new condition with previous conditions coming from parent objects
				ConditionContext.Push(ParamOp, true);
				
				const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
				bIsSkeletalMeshEntryPoint = false;
				
				Generate_Generic(InNode->Options[OptionIndex].get());
				
				bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;
				
				ConditionContext.Pop();
			}
		}
	}

	
	void FirstPassGenerator::Generate_SkeletalMeshVariation(const NodeSkeletalMeshVariation* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);
		
		// Any of the tags in the variations would prevent the default surface
		TArray<FString> OldNegativeTags = CurrentNegativeTags;
		for (int32 VariationIndex = 0; VariationIndex < InNode->Variations.Num(); ++VariationIndex)
		{
			CurrentNegativeTags.Add(InNode->Variations[VariationIndex].Tag);
		}

		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;
		
		Generate_Generic(InNode->DefaultSkeletalMesh.get());

		CurrentNegativeTags = OldNegativeTags;

		for (int32 VariationIndex = 0; VariationIndex < InNode->Variations.Num(); ++VariationIndex)
		{
			CurrentPositiveTags.Add(InNode->Variations[VariationIndex].Tag);
			Generate_Generic(InNode->Variations[VariationIndex].SkeletalMesh.get());

			CurrentPositiveTags.Pop();

			// Tags have an order in a variation node: the current tag should prevent any following variation
			CurrentNegativeTags.Add(InNode->Variations[VariationIndex].Tag);
		}

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;
		
		CurrentNegativeTags = OldNegativeTags;
	}

	
	void FirstPassGenerator::Generate_SkeletalMeshTransform(const NodeSkeletalMeshTransform* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);

		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->Source.get());

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();
	}

	
	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_ComponentSwitch(const NodeComponentSwitch* InNode)
	{
		if (InNode->Options.Num() == 0)
		{
			// No options in the switch!
			return;
		}

		// Prepare the enumeration parameter
		CodeGenerator::FGenericGenerationOptions Options;
		CodeGenerator::FScalarGenerationResult ScalarResult;
		if (InNode->Parameter)
		{
			Generator->GenerateScalar(ScalarResult, Options, InNode->Parameter);
		}
		else
		{
			// This argument is required
			ScalarResult.op = Generator->GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, InNode->GetMessageContext());
		}

		// Parse the options
		for (int32 OptionIndex = 0; OptionIndex < InNode->Options.Num(); ++OptionIndex)
		{
			// Create a comparison operation as the boolean parameter for the child
			Ptr<ASTOpBoolEqualIntConst> ParamOp = new ASTOpBoolEqualIntConst();
			ParamOp->Value = ScalarResult.op;
			ParamOp->Constant = OptionIndex;
			
			// Combine the new condition with previous conditions coming from parent objects
			ConditionContext.Push(ParamOp);

			if (InNode->Options[OptionIndex])
			{
				Generate_Generic(InNode->Options[OptionIndex].get());
			}

			ConditionContext.Pop();
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_LOD(const NodeLOD* InNode)
	{
		for (const Ptr<NodeSurface>& Surface : InNode->Surfaces)
		{
			if (Surface)
			{
				Generate_Generic(Surface.get());
			}
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_ObjectNew(const NodeObjectNew* InNode)
	{
		// Add the data about this object
		FObject ThisData;
		ThisData.Node = InNode;
		Ptr<ASTOp> Dummy;
		ConditionContext.GetCurrent(ThisData.Condition, Dummy);
		Objects.Add(ThisData);

		// Accumulate the model states
		for (const FObjectState& s : InNode->States)
		{
			States.Add(s);

			if (s.RuntimeParams.Num() > MUTABLE_MAX_RUNTIME_PARAMETERS_PER_STATE)
			{
				FString Msg = FString::Printf(TEXT("State [%s] has more than %d runtime parameters. Their update may fail."),
					*s.Name,
					MUTABLE_MAX_RUNTIME_PARAMETERS_PER_STATE);
				ErrorLog->Add(Msg, ELMT_ERROR, InNode->GetMessageContext());
			}
		}

		// Process the components
		for (const Ptr<NodeComponent>& Component : InNode->Components)
		{
			if (Component)
			{
				Generate_Generic(Component.get());
			}
		}

		// Process the modifiers
		for (const Ptr<NodeModifier>& Child : InNode->Modifiers)
		{
			if (Child)
			{
				Generate_Generic(Child.get());
			}
		}
		
		// Process the children
		for (const Ptr<NodeObject>& Child : InNode->Children)
		{
			if (Child)
			{
				Generate_Generic(Child.get());
			}
		}
	}


	//---------------------------------------------------------------------------------------------
	void FirstPassGenerator::Generate_ObjectGroup(const NodeObjectGroup* Node)
	{
		// Prepare the enumeration parameter if necessary
		Ptr<ASTOpParameter> enumOp;
		if (Node->Type == NodeObjectGroup::CS_ALWAYS_ONE ||
			Node->Type == NodeObjectGroup::CS_ONE_OR_NONE)
		{
			UE::TUniqueLock Lock(ParameterNodes.Mutex);
			Ptr<ASTOpParameter>* Found = ParameterNodes.GenericParametersCache.Find(Node);

			if (!Found)
			{
				Ptr<ASTOpParameter> op = new ASTOpParameter();
				op->Type = EOpType::NU_PARAMETER;
				op->Parameter.DefaultValue.Set<FParamIntType>(Node->DefaultValue);

				op->Parameter.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->Uid, op->Parameter.UID);
				check(bParseOk);
				op->Parameter.Type = EParameterType::Int;
				op->Parameter.DefaultValue.Set<FParamIntType>(Node->DefaultValue);

				if (Node->Type == NodeObjectGroup::CS_ONE_OR_NONE)
				{
					FParameterDesc::FIntValueDesc nullValue;
					nullValue.Value = -1;
					nullValue.Name = "None";
					op->Parameter.PossibleValues.Add(nullValue);
				}

				ParameterNodes.GenericParametersCache.Add(Node, op);

				enumOp = op;
			}
			else
			{
				enumOp = *Found;
			}
		}


		// Parse the child objects
		for (int32 t = 0; t < Node->Children.Num(); ++t)
		{
			if (UE::Mutable::Private::Ptr<const NodeObject> ChildNode = Node->Children[t])
			{
				// Overwrite the implicit condition
				Ptr<ASTOp> paramOp = new ASTOpConstantBool(true);
				switch (Node->Type)
				{
				case NodeObjectGroup::CS_TOGGLE_EACH:
				{
					if (ChildNode->GetType() == NodeObjectGroup::GetStaticType())
					{
						FString Msg = FString::Printf(TEXT("The Group Node [%s] has type Toggle and its direct child is a Group node, which is not allowed. Change the type or add a Child Object node in between them."), *Node->Name);
						ErrorLog->Add(Msg, ELMT_ERROR, Node->GetMessageContext());
					}
					else
					{
						// Create a new boolean parameter			
						UE::TUniqueLock Lock(ParameterNodes.Mutex);
						Ptr<ASTOpParameter>* Found = ParameterNodes.GenericParametersCache.Find(ChildNode);

						if (!Found)
						{
							Ptr<ASTOpParameter> op = new ASTOpParameter();
							op->Type = EOpType::BO_PARAMETER;

							op->Parameter.Name = ChildNode->GetName();
							bool bParseOk = FGuid::Parse(ChildNode->GetUid(), op->Parameter.UID);
							check(bParseOk);
							op->Parameter.Type = EParameterType::Bool;
							op->Parameter.DefaultValue.Set<FParamBoolType>(false);

							ParameterNodes.GenericParametersCache.Add(ChildNode, op);

							paramOp = op;
						}
						else
						{
							paramOp = *Found;
						}
					}

					break;
				}

				case NodeObjectGroup::CS_ALWAYS_ALL:
				{
					// Create a constant true boolean that the optimiser will remove later.
					paramOp = new ASTOpConstantBool(true);
					break;
				}

				case NodeObjectGroup::CS_ONE_OR_NONE:
				case NodeObjectGroup::CS_ALWAYS_ONE:
				{
					// Add the option to the enumeration parameter
					FParameterDesc::FIntValueDesc value;
					value.Value = (int16)t;
					value.Name = ChildNode->GetName();
					enumOp->Parameter.PossibleValues.Add(value);

					check(enumOp);

					// Create a comparison operation as the boolean parameter for the child
					Ptr<ASTOpBoolEqualIntConst> op = new ASTOpBoolEqualIntConst();
					op->Value = enumOp;
					op->Constant = t;

					paramOp = op;
					break;
				}

				default:
					check(false);
				}

				// Combine the new condition with previous conditions coming from parent objects
				ConditionContext.Push(paramOp);

				Generate_Generic(ChildNode.get());

				ConditionContext.Pop();
			}
		}
	}


	void FirstPassGenerator::Generate_SkeletalMeshObjectConvert(const NodeSkeletalMeshObjectConvert* Node)
	{
		Generate_Generic(Node->SkeletalMesh.get());
	}


	void FirstPassGenerator::Generate_SkeletalMeshMorph(const NodeSkeletalMeshMorph* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);
	
		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->Base.get());

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();		
	}


	void FirstPassGenerator::Generate_SkeletalMeshReshape(const NodeSkeletalMeshReshape* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);
	
		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->Base.get());
		Generate_Generic(InNode->BaseShape.get());
		Generate_Generic(InNode->TargetShape.get());

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();		
	}

	
	void FirstPassGenerator::Generate_SkeletalMeshModify(const NodeSkeletalMeshModify* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);
	
		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->SkeletalMesh.get());

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();
		
		for (const TTuple<FName, Ptr<NodeMaterial>>& Pair : InNode->SlotMaterials)
		{
			Generate_Generic(Pair.Value.get());
		}
	}


	void FirstPassGenerator::Generate_SkeletalMeshObjectSwitch(const NodeSkeletalMeshObjectSwitch* InNode)
	{
		if (InNode->Options.Num() == 0)
		{
			// No options in the switch!
			return;
		}

		// Prepare the enumeration parameter
		CodeGenerator::FGenericGenerationOptions Options;
		CodeGenerator::FScalarGenerationResult ScalarResult;
		if (InNode->Parameter)
		{
			Generator->GenerateScalar(ScalarResult, Options, InNode->Parameter);
		}
		else
		{
			// This argument is required
			ScalarResult.op = Generator->GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, InNode->GetMessageContext());
		}

		// Parse the options
		for (int32 OptionIndex = 0; OptionIndex < InNode->Options.Num(); ++OptionIndex)
		{
			// Create a comparison operation as the boolean parameter for the child
			Ptr<ASTOpBoolEqualIntConst> ParamOp = new ASTOpBoolEqualIntConst();
			ParamOp->Value = ScalarResult.op;
			ParamOp->Constant = OptionIndex;

			// Combine the new condition with previous conditions coming from parent objects
			ConditionContext.Push(ParamOp);

			if (InNode->Options[OptionIndex])
			{
				Generate_Generic(InNode->Options[OptionIndex].get());
			}

			ConditionContext.Pop();
		}
	}


	void FirstPassGenerator::Generate_SkeletalMeshClipSkeletalMesh(const NodeSkeletalMeshClipWithSkeletalMesh* InNode)
	{
		AddSkeletalMeshDataForNode(InNode);

		ConditionContext.Push(nullptr, true);

		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;
		bIsSkeletalMeshEntryPoint = false;

		Generate_Generic(InNode->SourceSkeletalMesh.get());
		Generate_Generic(InNode->ClipSkeletalMesh.get());

		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;

		ConditionContext.Pop();
	}


	void FirstPassGenerator::Generate_MaterialModify(const NodeMaterialModify* InNode)
	{
		Generate_Generic(InNode->MaterialSource.get());
	}


	void FirstPassGenerator::Generate_MaterialSwitch(const NodeMaterialSwitch* InNode)
	{
		for (const Ptr<NodeMaterial>& Option : InNode->Options)
		{
			Generate_Generic(Option.get());
		}
	}


	void FirstPassGenerator::Generate_MaterialVariation(const NodeMaterialVariation* InNode)
	{
		Generate_Generic(InNode->DefaultMaterial.get());

		for (const NodeMaterialVariation::FVariation& Variation : InNode->Variations)
		{
			Generate_Generic(Variation.Material.get());
		}
	}


	void FirstPassGenerator::Generate_MaterialSkeletalMeshBreak(const NodeMaterialSkeletalMeshBreak* InNode)
	{
		// The mesh inside the break is only used to look up a material slot; it must not be                                                                                                                                                                                                         
		// registered as a merge entry point for the enclosing skeletal mesh object.                                                                                                                                                                                                                 
		const bool bOldIsSkeletalMeshEntryPoint = bIsSkeletalMeshEntryPoint;                                                                                                                                                                                                                         
		bIsSkeletalMeshEntryPoint = false;                                                                                                                                                                                                                                                           
		
		Generate_Generic(InNode->SkeletalMesh.get());
		
		bIsSkeletalMeshEntryPoint = bOldIsSkeletalMeshEntryPoint;      
	}


	void FirstPassGenerator::FConditionContext::Push(Ptr<ASTOp> InNewCondition, bool bStartLocalPath)
	{
		auto CombineConditions = [](Ptr<ASTOp> ConditionA, Ptr<ASTOp> ConditionB) -> Ptr<ASTOp>
			{
				bool bValue = true;
				if (ConditionA && ASTOpConstantBool::IsConstantBool(ConditionA, bValue) && bValue)
				{
					ConditionA = nullptr;
				}

				if (ConditionB && ASTOpConstantBool::IsConstantBool(ConditionB, bValue) && bValue)
				{
					ConditionB = nullptr;
				}

				if (ConditionA && ConditionB)
				{
					Ptr<ASTOpBoolAnd> Op = new ASTOpBoolAnd();
					Op->A = ConditionA;
					Op->B = ConditionB;
					return Op;
				}

				return ConditionA ? ConditionA : ConditionB;
			};

		Ptr<ASTOp> PathCondition = PathConditions.Num() ? PathConditions.Last() : nullptr;
		PathConditions.Push(CombineConditions(PathCondition, InNewCondition));

		Ptr<ASTOp> LocalPathCondition = LocalPathConditions.Num() ? LocalPathConditions.Last() : nullptr;
		LocalPathConditions.Push(bStartLocalPath ? InNewCondition : CombineConditions(LocalPathCondition, InNewCondition));
	}

	void FirstPassGenerator::FConditionContext::Pop()
	{
		PathConditions.Pop();
		LocalPathConditions.Pop();
	}

	void FirstPassGenerator::FConditionContext::GetCurrent(Ptr<ASTOp>& OutPathCondition, Ptr<ASTOp>& OutLocalPathCondition)
	{
		if (PathConditions.Num())
		{
			OutPathCondition = PathConditions.Last();
		}

		if (LocalPathConditions.Num())
		{
			OutLocalPathCondition = LocalPathConditions.Last();
		}
	}
}
