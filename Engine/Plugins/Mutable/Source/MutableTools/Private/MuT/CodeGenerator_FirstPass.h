// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/AST.h"
#include "MuT/Compiler.h"
#include "MuT/Node.h"
#include "MuT/NodeModifierSkeletalMeshMerge.h"
#include "MuT/NodeComponentNew.h"
#include "MuT/NodeComponentSwitch.h"
#include "MuT/NodeComponentVariation.h"
#include "MuT/NodeLOD.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeSurfaceModifierMeshClipDeform.h"
#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"
#include "MuT/NodeSurfaceModifierMeshClipWithMesh.h"
#include "MuT/NodeSurfaceModifierMeshClipWithUVMask.h"
#include "MuT/NodeSurfaceModifierSurfaceEdit.h"
#include "MuT/NodeObjectGroup.h"
#include "MuT/NodeObjectNew.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeSurfaceVariation.h"
#include "MuT/NodeSurfaceSwitch.h"
#include "MuT/NodeMeshSkeletalMeshObjectBreak.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ErrorLog.h"
#include "Async/Mutex.h"

#include "MuR/ManagedPointer.h"
#include "MuT/NodeSkeletalMeshSwitch.h"
#include "MuT/NodeSkeletalMeshVariation.h"
#include "MuT/NodeSkeletalMeshConvert.h"

namespace UE::Mutable::Private
{
	class NodeMaterialVariation;
	class NodeMaterialSwitch;
	class NodeMaterialSkeletalMeshBreak;
	class NodeMaterialModify;
	class NodeSkeletalMeshModify;
	class NodeSkeletalMeshObjectConvert;
	class NodeSkeletalMeshObjectSwitch;
	class NodeSkeletalMeshTransform;
	class NodeSkeletalMeshTransformWithBone;
	class NodeSkeletalMeshModify;
	class NodeSkeletalMeshClipWithSkeletalMesh;
	class FLayout;
	class NodeSkeletalMeshMerge;
	class NodeSkeletalMeshMorph;
	class NodeSkeletalMeshReshape;
	struct FObjectState;

	/** */
	struct FGeneratedLayout
	{
		TManagedPtr<const FLayout> Layout;
		Ptr<const NodeLayout> Source;

		friend bool operator==(const FGeneratedLayout&, const FGeneratedLayout&) = default;
	};

	/** Store the results of the code generation of a mesh. */
	struct FMeshGenerationResult
	{
		//! Mesh after all code tree is applied
		Ptr<ASTOp> MeshOp;

		//! Original base mesh before removes, morphs, etc.
		Ptr<ASTOp> BaseMeshOp;

		/** Generated node layouts with their own block ids. */
		TArray<FGeneratedLayout> GeneratedLayouts;

		/** TODO: The following members seem related to surface-sharing data and not actual mesh generation result. Maybe they should be moved to a different struct.  */
		TArray<Ptr<ASTOp>> LayoutOps;

		struct FExtraLayouts
		{
			FGuid EditGuid;

			/** Source node layouts to use with these extra mesh. They don't have block ids. */
			TArray<FGeneratedLayout> GeneratedLayouts;
			Ptr<ASTOp> Condition;
			Ptr<ASTOp> MeshFragment;

			friend bool operator==(const FExtraLayouts&, const FExtraLayouts&) = default;
		};
		TArray<FExtraLayouts> ExtraMeshLayouts;

		friend bool operator==(const FMeshGenerationResult&, const FMeshGenerationResult&) = default;
	};

	


	using FMeshTask = UE::Tasks::TTask<FMeshGenerationResult>;

	struct FSurfaceGenerationResult
	{
		Ptr<ASTOp> MaterialOp;
		FGuid SharedSurfaceGuid;
	};

	using FSurfaceTask = UE::Tasks::TTask<FSurfaceGenerationResult>;
	
	struct FLODGenerationResult
	{
		Ptr<ASTOp> LODOp;
		
		struct FMaterialSlot
		{
			Ptr<ASTOp> Material;
			FName SlotName;
			uint32 Id = 0;
			uint32 ExternalId = 0;
		};
		TArray<FMaterialSlot> MaterialSlots;
	};
	
	using FLODTask = UE::Tasks::TTask<FLODGenerationResult>;

	struct FGenericGenerationResult
	{
		Ptr<ASTOp> Op;
	};
	
	using FSkeletalMeshTask = UE::Tasks::TTask<FGenericGenerationResult>;
	using FComponentTask = UE::Tasks::TTask<FGenericGenerationResult>;

	struct FExternalGenerationResult
	{
		Ptr<ASTOp> Op;
    		
		// If set, at least one of the external operation inputs is a mesh
		TOptional<FMeshGenerationResult> MeshGenerationResult;
	};
	
	using FExternalTask = UE::Tasks::TTask<FExternalGenerationResult>;

	
	/** First pass of the code generation process.
	 * It collects data about the object hierarchy, the conditions for each object and the global modifiers.
	 */
	class FirstPassGenerator
	{
	public:

		FirstPassGenerator();

        void Generate(TSharedPtr<FErrorLog>, const Node* Root, bool bIgnoreStates, class CodeGenerator*);

	private:

		void Generate_Generic(const Node*);
		void Generate_SurfaceModifier(const NodeSurfaceModifier*);
		void Generate_SurfaceNew(const NodeSurfaceNew*);
		void Generate_SurfaceSwitch(const NodeSurfaceSwitch*);
		void Generate_SurfaceVariation(const NodeSurfaceVariation*);
		void Generate_ComponentNew(const NodeComponentNew*);
		void Generate_ModifierSkeletalMeshMerge(const NodeModifierSkeletalMeshMerge*);
		void Generate_ComponentSwitch(const NodeComponentSwitch*);
		void Generate_ComponentVariation(const NodeComponentVariation*);
		void Generate_SkeletalMeshNew(const NodeSkeletalMeshNew*);
		void Generate_SkeletalMeshConvert(const NodeSkeletalMeshConvert*);
		void Generate_SkeletalMeshMerge(const NodeSkeletalMeshMerge*);
		void Generate_SkeletalMeshSwitch(const NodeSkeletalMeshSwitch*);
		void Generate_SkeletalMeshVariation(const NodeSkeletalMeshVariation*);
		void Generate_SkeletalMeshTransform(const NodeSkeletalMeshTransform*);
		void Generate_SkeletalMeshMorph(const NodeSkeletalMeshMorph*);
		void Generate_SkeletalMeshReshape(const NodeSkeletalMeshReshape*);
		void Generate_SkeletalMeshModify(const NodeSkeletalMeshModify*);
		void Generate_SkeletalMeshTransformWithBone(const NodeSkeletalMeshTransformWithBone*);
		void Generate_LOD(const NodeLOD*);
		void Generate_ObjectNew(const NodeObjectNew*);
		void Generate_ObjectGroup(const NodeObjectGroup*);
		void Generate_SkeletalMeshObjectConvert(const NodeSkeletalMeshObjectConvert*);
		void Generate_SkeletalMeshObjectSwitch(const NodeSkeletalMeshObjectSwitch*);
		void Generate_SkeletalMeshClipSkeletalMesh(const NodeSkeletalMeshClipWithSkeletalMesh*);
		void Generate_MaterialModify(const NodeMaterialModify*);
		void Generate_MaterialSwitch(const NodeMaterialSwitch*);
		void Generate_MaterialVariation(const NodeMaterialVariation*);
		void Generate_MaterialSkeletalMeshBreak(const NodeMaterialSkeletalMeshBreak*);

		void AddSkeletalMeshDataForNode(const NodeSkeletalMesh*);

	public:

		// Results
		//-------------------------

		//! Store the conditions that will enable or disable every object
		struct FObject
		{
			const NodeObjectNew* Node = nullptr;
            Ptr<ASTOp> Condition;
		};
		TArray<FObject> Objects;

		struct FSkeletalMeshData
		{
			const NodeSkeletalMesh* SkeletalMeshNode = nullptr;
			
			FName ParentSkeletalMeshName;

			bool bIsEntryPoint = true;
			
			Ptr<ASTOp> ObjectCondition;

			TArray<TArray<int32>> SurfacesPerLOD;
		};
		TArray<FSkeletalMeshData> SkeletalMeshes;

		bool bIsSkeletalMeshEntryPoint = true;

        //! Type used to represent the activation conditions regarding states
        //! This is the state mask for the states in which this surface must be added. If it
        //! is empty it means the surface is valid for all states. Otherwise it is only valid
        //! for the states whose index is true.
        using StateCondition = TArray<uint8>;

		/** Store information about every component found. */
		struct FComponent
		{
			/** Main component node. */
			const NodeComponentNew* Component = nullptr;

			// List of tags that are required for the presence of this component
			TArray<FString> PositiveTags;

			// List of tags that block the presence of this component
			TArray<FString> NegativeTags;

			// This conditions is the condition of the object defining this surface which may not
			// be the parent object where this surface will be added.
			Ptr<ASTOp> ObjectCondition;

			// Condition for this component to be added.
			// This is filled in CodeGenerator_SecondPass.
			Ptr<ASTOp> ComponentCondition;
		};
		TArray<FComponent> Components;

		/** Store information about every surface including
		* - the component it may be added to
		* - the conditions that will enable or disable it
		* - all edit operators
        * A surface may have different versions depending on the different parents and conditions it is reached with.
		*/
		struct FSurface
		{
            Ptr<const NodeSurfaceNew> Node;

			/** Parent Component where this surface will be added.It may be different from the
			* Component that defined it (if it was an edit component).
			*/
            const NodeComponentNew* Component = nullptr;

			int32 LOD = 0;

            // List of tags that are required for the presence of this surface
			TArray<FString> PositiveTags;

            // List of tags that block the presence of this surface
			TArray<FString> NegativeTags;

			// This conditions is the condition of the object defining this surface which may not
			// be the parent object where this surface will be added.
            Ptr<ASTOp> ObjectCondition;
            Ptr<ASTOp> MeshCondition;

            // This is filled in the first pass.
            StateCondition StateCondition;

            // Combined condition for the surface and the object conditions.
            // This is filled in CodeGenerator_SecondPass.
            Ptr<ASTOp> FinalCondition;

            // This is filled in the final code generation pass
			FSurfaceTask ResultSurfaceTask;
            Ptr<ASTOp> ResultMeshOp;

			bool operator==(const FSurface&) const = default;
        };
		TArray<FSurface> Surfaces;

		//! Store the conditions that enable every modifier.
		struct FModifier
		{
            const NodeSurfaceModifier* Node = nullptr;

            // List of tags that are required to apply this modifier
			TArray<FString> PositiveTags;

            // List of tags that block the activation of this modifier
			TArray<FString> NegativeTags;

            // This conditions is the condition of the object defining this modifier which may not
            // be the parent object where this surface will be added.
            Ptr<ASTOp> ObjectCondition;

			// Combined condition for the this modifier and the object conditions.
			// This is filled in CodeGenerator_SecondPass.
			Ptr<ASTOp> FinalCondition;

            // This is filled in CodeGenerator_SecondPass.
            StateCondition StateCondition;
						
			bool operator==(const FModifier&) const = default;
        };
		TArray<FModifier> Modifiers;

		/** Info about all found tags. */
		struct FTag
		{
			FString Tag;

            /** Surfaces that activate the tag.These are indices to the FirstPassGenerator::Surfaces array.*/
			TArray<int32> Surfaces;

            /** Modifiers that activate the tag. The index refers to the FirstPassGenerator::Modifiers. */
			TArray<int32> Modifiers;

            /** This conditions is the condition for this tag to be enabled considering no other condition. 
			* This is filled in CodeGenerator_SecondPass. 
			*/
            Ptr<ASTOp> GenericCondition;
        };
        TArray<FTag> Tags;

        /** Accumulate the model states found while generating code. */
        typedef TArray< FObjectState > StateList;
        StateList States;

		/** FParameters added for every node. */
		struct FSafeParameterNodes
		{
			/** Lock this mutex before accessing any data in the struct. */
			UE::FMutex Mutex;

			/** Cache for generic parameters. */
			TMap< Ptr<const Node>, Ptr<ASTOpParameter> > GenericParametersCache;
		};
		FSafeParameterNodes ParameterNodes;

	private:

		struct FConditionContext
		{
			void Push(Ptr<ASTOp> InCurrentCondition, bool bStartLocalPath = false);
			void Pop();

			void GetCurrent(Ptr<ASTOp>& OutPathCondition, Ptr<ASTOp>& OutLocalPathCondition);

		private:

			TArray<Ptr<ASTOp>> PathConditions;
			TArray<Ptr<ASTOp>> LocalPathConditions;

		};
		FConditionContext ConditionContext;

        //!
		TArray< StateCondition > CurrentStateCondition;

		/** When processing surfaces, this is the parent component the surfaces may be added to. */
        const NodeComponentNew* CurrentComponent = nullptr;

		FName CurrentSkeletalMeshName;
		
        //! Current relevant tags so far. Used during traversal.
		TArray<FString> CurrentPositiveTags;
		TArray<FString> CurrentNegativeTags;

		//** Index of the LOD we are processing. */
        int32 CurrentLOD = -1;

		/** Non-owned reference to main code generator. */
		CodeGenerator* Generator = nullptr;

        //!
		TSharedPtr<FErrorLog> ErrorLog;
	};

}
