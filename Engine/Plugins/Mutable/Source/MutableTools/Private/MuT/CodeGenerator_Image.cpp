// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpImageParameterConvert.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Image.h"
#include "MuR/ImagePrivate.h"
#include "MuR/Mesh.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/OpImageProject.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/Serialisation.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpReferenceResource.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImageNormalComposite.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageConform.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageCrop.h"
#include "MuT/ASTOpImageResize.h"
#include "MuT/ASTOpImageResizeRel.h"
#include "MuT/ASTOpImageInterpolate.h"
#include "MuT/ASTOpImageSaturate.h"
#include "MuT/ASTOpImageLuminance.h"
#include "MuT/ASTOpImageColorMap.h"
#include "MuT/ASTOpImageBinarize.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpImageInvert.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpMaterialBreak.h"
#include "MuT/ASTOpMaterialBreakImageParameter.h"
#include "MuT/ASTOpMeshExtractLayoutBlocks.h"
#include "MuT/ASTOpMeshFormat.h"
#include "MuT/ASTOpMeshProject.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/Compiler.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/Node.h"
#include "MuT/NodeBool.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeImageBinarise.h"
#include "MuT/NodeImageColorMap.h"
#include "MuT/NodeImageConditional.h"
#include "MuT/NodeImageConstant.h"
#include "MuT/NodeImageExternal.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeImageConform.h"
#include "MuT/NodeImageFromMaterialParameter.h"
#include "MuT/NodeImageInterpolate.h"
#include "MuT/NodeImageInvert.h"
#include "MuT/NodeImageLayer.h"
#include "MuT/NodeImageLayerColor.h"
#include "MuT/NodeImageLuminance.h"
#include "MuT/NodeImageMaterialBreak.h"
#include "MuT/NodeImageMipmap.h"
#include "MuT/NodeImageMultiLayer.h"
#include "MuT/NodeImageNormalComposite.h"
#include "MuT/NodeImageObjectParameter.h"
#include "MuT/NodeImageConvert.h"
#include "MuT/NodeImagePlainColor.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeImageResize.h"
#include "MuT/NodeImageSaturate.h"
#include "MuT/NodeImageSwitch.h"
#include "MuT/NodeImageSwizzle.h"
#include "MuT/NodeImageTable.h"
#include "MuT/NodeImageTransform.h"
#include "MuT/NodeImageVariation.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/Table.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{

	void CodeGenerator::GenerateImage(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImagePtrConst& Untyped)
	{
		if (!Untyped)
		{
			Result = FImageGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedImageCacheKey key(Options,Untyped);
		{
			UE::TUniqueLock Lock(GeneratedImages.Mutex);
			FGeneratedImagesMap::ValueType* CachedPtr = GeneratedImages.Map.Find(key);
			if (CachedPtr)
			{
				Result = *CachedPtr;
				return;
			}
		}

		const NodeImage* Node = Untyped.get();

		// Generate for each different type of node
		switch (Untyped->GetType()->Type)
		{
		case Node::EType::ImageConstant: GenerateImage_Constant(Options, Result, static_cast<const NodeImageConstant*>(Node)); break;
		case Node::EType::ImageInterpolate: GenerateImage_Interpolate(Options, Result, static_cast<const NodeImageInterpolate*>(Node)); break;
		case Node::EType::ImageSaturate: GenerateImage_Saturate(Options, Result, static_cast<const NodeImageSaturate*>(Node)); break;
		case Node::EType::ImageTable: GenerateImage_Table(Options, Result, static_cast<const NodeImageTable*>(Node)); break;
		case Node::EType::ImageSwizzle: GenerateImage_Swizzle(Options, Result, static_cast<const NodeImageSwizzle*>(Node)); break;
		case Node::EType::ImageColorMap: GenerateImage_ColorMap(Options, Result, static_cast<const NodeImageColorMap*>(Node)); break;
		case Node::EType::ImageBinarise: GenerateImage_Binarise(Options, Result, static_cast<const NodeImageBinarise*>(Node)); break;
		case Node::EType::ImageLuminance: GenerateImage_Luminance(Options, Result, static_cast<const NodeImageLuminance*>(Node)); break;
		case Node::EType::ImageLayer: GenerateImage_Layer(Options, Result, static_cast<const NodeImageLayer*>(Node)); break;
		case Node::EType::ImageLayerColor: GenerateImage_LayerColor(Options, Result, static_cast<const NodeImageLayerColor*>(Node)); break;
		case Node::EType::ImageResize: GenerateImage_Resize(Options, Result, static_cast<const NodeImageResize*>(Node)); break;
		case Node::EType::ImagePlainColor: GenerateImage_PlainColor(Options, Result, static_cast<const NodeImagePlainColor*>(Node)); break;
		case Node::EType::ImageProject: GenerateImage_Project(Options, Result, static_cast<const NodeImageProject*>(Node)); break;
		case Node::EType::ImageMipmap: GenerateImage_Mipmap(Options, Result, static_cast<const NodeImageMipmap*>(Node)); break;
		case Node::EType::ImageSwitch: GenerateImage_Switch(Options, Result, static_cast<const NodeImageSwitch*>(Node)); break;
		case Node::EType::ImageConditional: GenerateImage_Conditional(Options, Result, static_cast<const NodeImageConditional*>(Node)); break;
		case Node::EType::ImageFormat: GenerateImage_Format(Options, Result, static_cast<const NodeImageFormat*>(Node)); break;
		case Node::EType::ImageConform: GenerateImage_Conform(Options, Result, static_cast<const NodeImageConform*>(Node)); break;
		case Node::EType::ImageObjectParameter: GenerateImage_Parameter(Options, Result, static_cast<const NodeImageObjectParameter*>(Node)); break;
		case Node::EType::ImageMultiLayer: GenerateImage_MultiLayer(Options, Result, static_cast<const NodeImageMultiLayer*>(Node)); break;
		case Node::EType::ImageInvert: GenerateImage_Invert(Options, Result, static_cast<const NodeImageInvert*>(Node)); break;
		case Node::EType::ImageVariation: GenerateImage_Variation(Options, Result, static_cast<const NodeImageVariation*>(Node)); break;
		case Node::EType::ImageNormalComposite: GenerateImage_NormalComposite(Options, Result, static_cast<const NodeImageNormalComposite*>(Node)); break;
		case Node::EType::ImageTransform: GenerateImage_Transform(Options, Result, static_cast<const NodeImageTransform*>(Node)); break;
		case Node::EType::ImageMaterialBreak: GenerateImage_MaterialBreak(Options, Result, static_cast<const NodeImageMaterialBreak*>(Node)); break;
		case Node::EType::ImageFromMaterialParameter: GenerateImage_FromMaterialParameter(Options, Result, static_cast<const NodeImageFromMaterialParameter*>(Node)); break;
		case Node::EType::ImageExternal: GenerateImage_External(Options, Result, static_cast<const NodeImageExternal*>(Node)); break;
		case Node::EType::ImageConvert: GenerateImage_ParameterConvert(Options, Result, static_cast<const NodeImageConvert*>(Node)); break;
		default: check(false);
		}

		// Cache the Result
		{
			UE::TUniqueLock Lock(GeneratedImages.Mutex);
			GeneratedImages.Map.Add(key, Result);
		}
	}


	void CodeGenerator::GenerateCrop(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImage& InNode)
	{
		if (Options.ImageLayoutStrategy!= CompilerOptions::TextureLayoutStrategy::None && Options.LayoutToApply)
		{
			// We want to generate only a block from the image.

			const FImageDesc ImageDesc = Result.op->GetImageDesc();
			
			FIntVector2 SourceImageSize(ImageDesc.m_size[0], ImageDesc.m_size[1]);

			int32 BlockIndex = Options.LayoutToApply->FindBlock(Options.LayoutBlockId);
			check( BlockIndex>=0 );

			// Block in layout grid units
			box< FIntVector2 > RectInCells;
			RectInCells.min = Options.LayoutToApply->Blocks[BlockIndex].Min;
			RectInCells.size = Options.LayoutToApply->Blocks[BlockIndex].Size;

			FIntPoint grid = Options.LayoutToApply->GetGridSize();
			grid[0] = FMath::Max(1, grid[0]);
			grid[1] = FMath::Max(1, grid[1]);

			// Transform to pixels
			box< FIntVector2 > RectInPixels;
			RectInPixels.min[0]  = (RectInCells.min[0]  * SourceImageSize[0]) / grid[0];
			RectInPixels.min[1]  = (RectInCells.min[1]  * SourceImageSize[1]) / grid[1];
			RectInPixels.size[0] = (RectInCells.size[0] * SourceImageSize[0]) / grid[0];
			RectInPixels.size[1] = (RectInCells.size[1] * SourceImageSize[1]) / grid[1];

			// Do we need to crop?
			if (RectInPixels.min[0] != 0 || RectInPixels.min[1] != 0 || SourceImageSize[0] != RectInPixels.size[0] || SourceImageSize[1] != RectInPixels.size[1])
			{
				// See if the rect belongs to a single texture tile
				FIntVector2 TileMin(RectInPixels.min[0] / SourceImageSize[0], RectInPixels.min[1] / SourceImageSize[1]);
				FIntVector2 TileMax((RectInPixels.min[0]+ RectInPixels.size[0]-1) / SourceImageSize[0], (RectInPixels.min[1]+ RectInPixels.size[1]-1) / SourceImageSize[1]);

				if (TileMin != TileMax)
				{
					// Blocks spaning multiple texture tiles are not supported.
					// To implement them, assemble a series of instructions to crop and compose the necessary rects from each tile into the final image.
					ensure(false);

					// Log an error message
					ErrorLog->Add("A layout block goes across different texture tiles. This is not supported yet.", ELMT_ERROR, InNode.GetMessageContext());
				}
				else
				{
					Ptr<ASTOpImageCrop> CropOp = new ASTOpImageCrop();
					CropOp->Source = Result.op;
					
					// Bring the crop rect to tile 0,0
					CropOp->Min[0] = RectInPixels.min[0] - TileMin[0] * SourceImageSize[0];
					CropOp->Min[1] = RectInPixels.min[1] - TileMin[1] * SourceImageSize[1];
					CropOp->Size[0] = RectInPixels.size[0];
					CropOp->Size[1] = RectInPixels.size[1];
					Result.op = CropOp;
				}
			}
		}
	}
	

   void CodeGenerator::GenerateImage_Constant(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageConstant* InNode)
    {
		const NodeImageConstant& node = *InNode;
		
		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        // TODO: check duplicates
        TManagedPtr<const FImage> pImage = InNode->Image;

        if (!pImage)
        {
            // This data is required
            pImage = GenerateMissingImage(EImageFormat::RGB_UByte );

            // Log an error message
            ErrorLog->Add( "Constant image not set.", ELMT_WARNING, InNode->GetMessageContext());
        }


		if (pImage->IsReference())
		{
			Ptr<ASTOpReferenceResource> ReferenceOp = new ASTOpReferenceResource();
			ReferenceOp->Type = EOpType::IM_REFERENCE;
			ReferenceOp->ID = pImage->PassthroughObject.GetId();
			ReferenceOp->bForceLoad = pImage->IsForceLoad();
			ReferenceOp->SourceDataDescriptor = InNode->SourceDataDescriptor;

			// Don't store the format. Format can vary between loaded constant image and reference and cause
			// code optimization bugs.
			// As it is now, reference will always have alpha channel but constant resolution can remove the 
			// channel if not used.
			// TODO: review this, probably the reference descriptor generation needs to check for alpha channels 
			// as well.
			ReferenceOp->ImageDesc = FImageDesc(pImage->GetSize(), EImageFormat::None, EImageFormat::None, pImage->GetLODCount());
			Result.op = ReferenceOp;
		}
		else
		{
			Ptr<ASTOpConstantResource> op = new ASTOpConstantResource();
			op->Type = EOpType::IM_CONSTANT;
			op->SetValue(pImage);
			op->SourceDataDescriptor = InNode->SourceDataDescriptor;
			Result.op = op;
		}

		GenerateCrop(Options, Result, *InNode);
    }


	void CodeGenerator::GenerateImage_Parameter(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageObjectParameter* Node )
    {
        Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				op = new ASTOpParameter();
				op->Type = EOpType::IM_PARAMETER;

				op->Parameter.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->UID, op->Parameter.UID);
				check(bParseOk);
				op->Parameter.Type = EParameterType::Texture;
				op->Parameter.DefaultValue.Set<FParamTextureType>(nullptr);

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Node->Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options.GenericOptions, Node->Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		Result.op = op;
		Result.Parameter = true;
		Result.CanNormalLAEncode = EYesNoQuery::No;
    }


	void CodeGenerator::GenerateImage_Layer(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageLayer* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageLayer);

		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        Ptr<ASTOpImageLayer> op = new ASTOpImageLayer();

        op->blendType = Node->Type;

        // Base image
        Ptr<ASTOp> base;
        if (Node->Base )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			base = BaseResult.op;

			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
        }
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Image Layer base"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }

		UE::Math::TIntVector2<int32> TargetSize = Options.RectSize;

		FImageDesc BaseDesc = base->GetImageDesc(true);

		// If no target size was specified in the generation options, get the base size to apply it to the mask and blend.
		if (TargetSize == UE::Math::TIntVector2<int32>(0, 0))
		{
			TargetSize = UE::Math::TIntVector2<int32>(BaseDesc.m_size);
		}

		EImageFormat baseFormat = BaseDesc.m_format;
        base = GenerateImageSize( base, TargetSize);
        op->base = base;

        // Mask of the effect
        Ptr<ASTOp> mask;
        if (Node->Mask )
        {
			FImageGenerationResult MaskResult;
			FImageGenerationOptions MaskOptions(Options);
			MaskOptions.RectSize = TargetSize;
			GenerateImage(MaskOptions, MaskResult, Node->Mask);
			mask = MaskResult.op;

            mask = GenerateImageFormat( mask, EImageFormat::L_UByte );
            mask = GenerateImageSize( mask, TargetSize);
        }
        op->mask = mask;

        // Image to apply
        Ptr<ASTOp> blended = 0;
        if (Node->Blended )
        {
			FImageGenerationResult BlendedResult;
			FImageGenerationOptions BlendOptions(Options);
			BlendOptions.RectSize = TargetSize;
			GenerateImage(BlendOptions, BlendedResult, Node->Blended);
			blended = BlendedResult.op;

			if (BlendedResult.CanNormalLAEncode != Result.CanNormalLAEncode)
			{
				Result.CanNormalLAEncode = EYesNoQuery::Mixed;
			}
        }
        else
        {
            // This argument is required
            blended = GeneratePlainImageCode(FVector4f( 1,1,0,1 ), Options );
        }
        blended = GenerateImageFormat( blended, baseFormat );
        blended = GenerateImageSize( blended, TargetSize);
        op->blend = blended;

        Result.op = op;
    }


	void CodeGenerator::GenerateImage_LayerColor(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageLayerColor* Node)
	{		
		MUTABLE_CPUPROFILER_SCOPE(NodeImageLayerColor);


		bool bCanMulticompose = true;

		Result.CanNormalLAEncode = EYesNoQuery::Yes;
        
		Ptr<ASTOpImageLayerColor> op = new ASTOpImageLayerColor();
		op->blendType = Node->Type;


        // Base image
        Ptr<ASTOp> base;
        if (Node->Base)
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			base = BaseResult.op;
			bCanMulticompose = BaseResult.bCanMulticompose;
			
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
        }
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Layer base image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options );
        }

		UE::Math::TIntVector2<int32> TargetSize = Options.RectSize;

		// If no target size was specified in the generation options, get the base size to apply it to the mask and blend.
		if (TargetSize == UE::Math::TIntVector2<int32>(0, 0))
		{
			FImageDesc BaseDesc = base->GetImageDesc(true);
			TargetSize = UE::Math::TIntVector2<int32>(BaseDesc.m_size);
		}

        base = GenerateImageFormat( base, EImageFormat::RGB_UByte );
        base = GenerateImageSize( base, TargetSize);
        op->base = base;

        // Mask of the effect
        Ptr<ASTOp> mask;
        if (Node->Mask)
        {
			FImageGenerationResult MaskResult;
			GenerateImage(Options, MaskResult, Node->Mask);
			mask = MaskResult.op;
			bCanMulticompose = bCanMulticompose && MaskResult.bCanMulticompose;
			
			mask = GenerateImageFormat( mask, EImageFormat::L_UByte );
            mask = GenerateImageSize( mask, TargetSize);
        }
        op->mask = mask;

        // Color to apply
        Ptr<ASTOp> color = 0;
        if (Node->Color )
        {
			FColorGenerationResult ColorResult;
            GenerateColor(ColorResult, Options.GenericOptions, Node->Color);
			color = ColorResult.op;
        }
        else
        {
            // This argument is required
            color = GenerateMissingColorCode(TEXT("Layer color"), Node->GetMessageContext() );
        }
        op->color = color;

        Result.op = op;
		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_MultiLayer(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageMultiLayer* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageMultiLayer);


		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;

        Ptr<ASTOpImageMultiLayer> op = new ASTOpImageMultiLayer();

		op->blendType = Node->Type;

        // Base image
        Ptr<ASTOp> base;
        if (Node->Base )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			base = BaseResult.op;
			
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
		}
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Image MultiLayer base"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }

		FImageDesc BaseDesc = base->GetImageDesc(true);

		UE::Math::TIntVector2<int32> TargetSize = Options.RectSize;

		// If no target size was specified in the generation options, get the base size to apply it to the mask and blend.
		if (TargetSize == UE::Math::TIntVector2<int32>(0, 0))
		{
			TargetSize = UE::Math::TIntVector2<int32>(BaseDesc.m_size);
		}

		EImageFormat baseFormat = BaseDesc.m_format;
        base = GenerateImageSize( base, TargetSize);
        op->base = base;

        // Mask of the effect
        Ptr<ASTOp> mask;
        if (Node->Mask )
        {
			FImageGenerationResult MaskResult;
			GenerateImage(Options, MaskResult, Node->Mask);
			mask = MaskResult.op;

			mask = GenerateImageFormat( mask, EImageFormat::L_UByte );
            mask = GenerateImageSize( mask, TargetSize);
        }
        op->mask = mask;

        // Image to apply
        Ptr<ASTOp> blended;
        if (Node->Blended)
        {
			FImageGenerationResult BlendedResult;
			GenerateImage(Options, BlendedResult, Node->Blended);
			blended = BlendedResult.op;

			if (BlendedResult.CanNormalLAEncode != Result.CanNormalLAEncode)
			{
				Result.CanNormalLAEncode = EYesNoQuery::Mixed;
			}
        }
        else
        {
            // This argument is required
            blended = GeneratePlainImageCode(FVector4f( 1,1,0,1 ), Options);
        }
        blended = GenerateImageFormat( blended, baseFormat );
        blended = GenerateImageSize( blended, TargetSize);
        op->blend = blended;

        // Range of iteration
        if (Node->Range )
        {
            FRangeGenerationResult rangeResult;
            GenerateRange( rangeResult, Options.GenericOptions, Node->Range );

            op->range.rangeSize = rangeResult.sizeOp;
            op->range.rangeName = rangeResult.rangeName;
            op->range.rangeUID = rangeResult.rangeUID;
        }

        Result.op = op;
    }


	void CodeGenerator::GenerateImage_NormalComposite(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageNormalComposite* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageNormalComposite);


		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        Ptr<ASTOpImageNormalComposite> op = new ASTOpImageNormalComposite();

		op->Mode = Node->Mode;
		op->Power = Node->Power;

        // Base image
        Ptr<ASTOp> base;
        if (Node->Base )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			base = BaseResult.op;
			
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
		}
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Image Composite Base"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }

		FImageDesc BaseDesc = base->GetImageDesc(true);

		UE::Math::TIntVector2<int32> TargetSize = Options.RectSize;

		// If no target size was specified in the generation options, get the base size to apply it to the mask and blend.
		if (TargetSize == UE::Math::TIntVector2<int32>(0, 0))
		{
			TargetSize = UE::Math::TIntVector2<int32>(BaseDesc.m_size);
		}


		EImageFormat baseFormat = BaseDesc.m_format;
        base = GenerateImageSize( base, TargetSize);
        op->Base = base;

        Ptr<ASTOp> normal;
        if (Node->Normal)
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Normal);
			normal = BaseResult.op;

            normal = GenerateImageFormat( normal, EImageFormat::RGB_UByte );
        }
		else
		{
            // This argument is required
            normal = GenerateMissingImageCode(TEXT("Image Composite Normal"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
		}

		normal = GenerateImageSize(normal, TargetSize);

        op->Normal = normal;
        
		Result.op = op;
    }


	void CodeGenerator::GenerateImage_Transform(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageTransform* Node)
    {
        MUTABLE_CPUPROFILER_SCOPE(NodeImageTransform);

		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        Ptr<ASTOpImageTransform> Op = new ASTOpImageTransform();

		Ptr<ASTOp> OffsetX;
		if (Node->OffsetX)
		{
			FScalarGenerationResult ScalarResult;
			GenerateScalar(ScalarResult, Options.GenericOptions, Node->OffsetX);
			OffsetX = ScalarResult.op;
		}

		Ptr<ASTOp> OffsetY;
		if (Node->OffsetY)
		{
			FScalarGenerationResult ScalarResult;
			GenerateScalar(ScalarResult, Options.GenericOptions, Node->OffsetY);
			OffsetY = ScalarResult.op;
		}
	
		Ptr<ASTOp> ScaleX;
		if (Node->ScaleX)
		{
			FScalarGenerationResult ScalarResult;
			GenerateScalar(ScalarResult, Options.GenericOptions, Node->ScaleX);
			ScaleX = ScalarResult.op;
		}
	
		Ptr<ASTOp> ScaleY;
		if (Node->ScaleY)
		{
			FScalarGenerationResult ScalarResult;
			GenerateScalar(ScalarResult, Options.GenericOptions, Node->ScaleY);
			ScaleY = ScalarResult.op;
		}

		Ptr<ASTOp> Rotation;
		if (Node->Rotation)
		{
			FScalarGenerationResult ScalarResult;
			GenerateScalar(ScalarResult, Options.GenericOptions, Node->Rotation);
			Rotation = ScalarResult.op;
		}

		// If one of the inputs (offset or scale) is missing assume uniform translation/scaling 
		Op->OffsetX = OffsetX ? OffsetX : OffsetY;
		Op->OffsetY = OffsetY ? OffsetY : OffsetX;
 		Op->ScaleX = ScaleX ? ScaleX : ScaleY;
		Op->ScaleY = ScaleY ? ScaleY : ScaleX;
		Op->Rotation = Rotation; 
		Op->AddressMode = Node->AddressMode;
		Op->SizeX = Node->SizeX;
		Op->SizeY = Node->SizeY;
		Op->bKeepAspectRatio = Node->bKeepAspectRatio;

		// Base image
        Ptr<ASTOp> Base;
		FImageGenerationOptions NewOptions = Options;
		NewOptions.ImageLayoutStrategy = CompilerOptions::TextureLayoutStrategy::None;
		NewOptions.LayoutToApply = nullptr;
		NewOptions.LayoutBlockId = -1;
		NewOptions.RectSize = {0, 0};


        if (Node->Base)
        {
			FImageGenerationResult BaseResult;
			GenerateImage(NewOptions, BaseResult, Node->Base);
			Base = BaseResult.op;
			
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
		}
        else
        {
            // This argument is required
            Base = GenerateMissingImageCode(TEXT("Image Transform Base"), EImageFormat::RGB_UByte, Node->GetMessageContext(), NewOptions);
        }
		
		FImageDesc BaseDesc = Base->GetImageDesc(true);
        Op->Base = Base;
		Op->SourceSizeX = BaseDesc.m_size.X;
		Op->SourceSizeY = BaseDesc.m_size.Y;
		
        Result.op = Op; 

		// Compute the image crop for the layout block to apply. 
		if (Options.LayoutToApply)
		{
			FIntVector2 TransformImageSize = FIntVector2(Op->SizeX, Op->SizeY);
			if (Op->SizeX == 0 && Op->SizeY == 0)
			{
				TransformImageSize = FIntVector2(BaseDesc.m_size.X, BaseDesc.m_size.Y);	
			}

			int32 BlockIndex = Options.LayoutToApply->FindBlock(Options.LayoutBlockId);
			check(BlockIndex >= 0);

			// Rect in layout grid units
			FIntVector2 RectMinInCells  = Options.LayoutToApply->Blocks[BlockIndex].Min;
			FIntVector2 RectSizeInCells = Options.LayoutToApply->Blocks[BlockIndex].Size;
			
			FIntVector2 Grid = Options.LayoutToApply->GetGridSize();
			Grid = FIntVector2(FMath::Max(1, Grid.X), FMath::Max(1, Grid.Y));

			// Transform to pixels
			FIntVector2 BlockImageMin = FIntVector2(
					(RectMinInCells.X * TransformImageSize.X) / Grid.X,
					(RectMinInCells.Y * TransformImageSize.Y) / Grid.Y);

			FIntVector2 BlockImageSize = FIntVector2(
					(RectSizeInCells.X * TransformImageSize.X) / Grid.X,
					(RectSizeInCells.Y * TransformImageSize.Y) / Grid.Y);
			
			if (BlockImageSize != TransformImageSize)
			{
				Ptr<ASTOpImageCrop> CropOp = new ASTOpImageCrop();
				CropOp->Source = Op;
			
				CropOp->Min.X  = (uint16)BlockImageMin.X;
				CropOp->Min.Y  = (uint16)BlockImageMin.Y;
				CropOp->Size.X = (uint16)BlockImageSize.X;
				CropOp->Size.Y = (uint16)BlockImageSize.Y;

        		Result.op = CropOp;
			}
		}
    }


	void CodeGenerator::GenerateImage_Interpolate(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageInterpolate* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageInterpolate);

		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        Ptr<ASTOpImageInterpolate> op = new ASTOpImageInterpolate();

        // Factor
        if ( NodeScalar* Factor = Node->Factor.get() )
        {
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GenericOptions, Factor);
			op->Factor = ParamResult.op;
        }
        else
        {
            // This argument is required
            op->Factor = GenerateMissingScalarCode(TEXT("Interpolation factor"), 0.5f, Node->GetMessageContext() );
        }

        // Target images
        int32 numTargets = 0;

		UE::Math::TIntVector2<int32> FinalRectSize = Options.RectSize;

		bool bFirstTargetProcessed = false;
		Result.CanNormalLAEncode =  EYesNoQuery::Yes;

        for ( int32 TargetIndex=0
            ; TargetIndex < Node->Targets.Num() && numTargets<MUTABLE_OP_MAX_INTERPOLATE_COUNT
            ; ++TargetIndex)
        {
            if (Node->Targets[TargetIndex] )
            {
				FImageGenerationOptions ChildOptions = Options;
				ChildOptions.RectSize = FinalRectSize;
				FImageGenerationResult BaseResult;
				GenerateImage(ChildOptions, BaseResult, Node->Targets[TargetIndex]);
				Ptr<ASTOp> target = BaseResult.op;

				if (!bFirstTargetProcessed)
				{
					Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
					bFirstTargetProcessed = true;
				}
				else if (Result.CanNormalLAEncode != BaseResult.CanNormalLAEncode)
				{
					Result.CanNormalLAEncode = EYesNoQuery::Mixed;
				}

				if (FinalRectSize[0] == 0)
				{
					FImageDesc ChildDesc = target->GetImageDesc();
					FinalRectSize = UE::Math::TIntVector2<int32>(ChildDesc.m_size);
				}

                // TODO: Support other formats
                target = GenerateImageFormat( target, EImageFormat::RGB_UByte );
                target = GenerateImageSize( target, FinalRectSize);

                op->Targets[numTargets] = target;
                numTargets++;
            }
        }

        // At least one target is required
        if (!op->Targets[0])
        {
            Ptr<ASTOp> target = GenerateMissingImageCode(TEXT("First interpolation image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
            target = GenerateImageSize( target, Options.RectSize);
            op->Targets[0] = target;
        }

        Result.op = op;
    }


	void CodeGenerator::GenerateImage_Swizzle(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageSwizzle* Node)
	{
		bool bCanMulticompose = true;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		if (Node->NewFormat == EImageFormat::None)
		{
			Result.op = GenerateMissingImageCode(TEXT("Make Texture channel."), EImageFormat::L_UByte, Node->GetMessageContext(), Options);
			return;
		}

        // This node always produces a swizzle operation and sometimes it may produce a pixelformat
		// operation to compress the Result
        Ptr<ASTOpImageSwizzle> SwizzleOp = new ASTOpImageSwizzle();

		// Format
		EImageFormat CompressedFormat = EImageFormat::None;

		switch (Node->NewFormat)
		{
        case EImageFormat::BC1:
        case EImageFormat::ASTC_4x4_RGB_LDR:
            CompressedFormat = Node->NewFormat;
            SwizzleOp->Format = (Node->Sources.IsValidIndex(3) && Node->Sources[3]) ? EImageFormat::RGBA_UByte : EImageFormat::RGB_UByte;
			break;

		case EImageFormat::BC2:
		case EImageFormat::BC3:
		case EImageFormat::BC6:
        case EImageFormat::BC7:
        case EImageFormat::ASTC_4x4_RGBA_LDR:
            CompressedFormat = Node->NewFormat;
            SwizzleOp->Format = EImageFormat::RGBA_UByte;
			break;

		case EImageFormat::BC4:
			CompressedFormat = Node->NewFormat;
            SwizzleOp->Format = EImageFormat::L_UByte;
			break;

		case EImageFormat::BC5:
        case EImageFormat::ASTC_4x4_RG_LDR:
            CompressedFormat = Node->NewFormat;
			// TODO: Should be RG
            SwizzleOp->Format = EImageFormat::RGB_UByte;
			break;

		default:
            SwizzleOp->Format = Node->NewFormat;
			break;

		}

		check(Node->NewFormat != EImageFormat::None);

		// Source images and channels
		check(Node->Sources.Num() == Node->SourceChannels.Num());

		// First source, for reference in the size
        Ptr<ASTOp> FirstValid;
		FImageDesc FirstValidDesc;
		int32 FirstValidSourceIndex = -1;


		bool bFirstSourceProcessed = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		check(MUTABLE_OP_MAX_SWIZZLE_CHANNELS >= Node->Sources.Num());
		for (int32 SourceIndex = 0; SourceIndex < Node->Sources.Num(); ++SourceIndex)
		{
			if (Node->Sources[SourceIndex])
			{
				FImageGenerationResult BaseResult;
				GenerateImage(Options, BaseResult, Node->Sources[SourceIndex]);
                Ptr<ASTOp> Source = BaseResult.op;
				bCanMulticompose = bCanMulticompose && BaseResult.bCanMulticompose;

		
				if (!Source)
				{
					// TODO: Warn?
					Source = GenerateMissingImageCode(TEXT("Swizzle channel"), EImageFormat::L_UByte, Node->GetMessageContext(), Options);
				}

				const FImageDesc SourceImageDesc = Source->GetImageDesc();

				const bool bUseFormatWithAlpha = Node->SourceChannels[SourceIndex] == 3 && SourceImageDesc.FormatIfAlpha != EImageFormat::None;
				const EImageFormat SourceFormat = bUseFormatWithAlpha  ? SourceImageDesc.FormatIfAlpha : SourceImageDesc.m_format;
				const EImageFormat TargetFormat = GetUncompressedFormat(SourceFormat);

				if (TargetFormat != SourceFormat)
				{
					// Generate uncompressed format
					Ptr<ASTOpImagePixelFormat> FormatOp = new ASTOpImagePixelFormat();
					FormatOp->Source = Source;
					FormatOp->Format = TargetFormat;
					Source = FormatOp;
				}
				
				if (!bFirstSourceProcessed)
				{
					Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
					bFirstSourceProcessed = true;
				}
				else if (Result.CanNormalLAEncode != BaseResult.CanNormalLAEncode)
				{
					Result.CanNormalLAEncode = EYesNoQuery::Mixed;
				}


                Ptr<ASTOp> SizedSource;
				if (FirstValid && FirstValidDesc.m_size[0])
				{
					SizedSource = GenerateImageSize(Source, FIntVector2(FirstValidDesc.m_size));
				}
				else
				{
					FirstValid = Source;
					SizedSource = Source;
					FirstValidDesc = FirstValid->GetImageDesc();
					FirstValidSourceIndex = SourceIndex;
				}

                SwizzleOp->Sources[SourceIndex] = SizedSource;
                SwizzleOp->SourceChannels[SourceIndex] = (uint8)Node->SourceChannels[SourceIndex];
			}
		}

        if (FirstValidSourceIndex < 0)
		{
            Ptr<ASTOp> Source = GenerateMissingImageCode(TEXT("First swizzle image"), EImageFormat::RGBA_UByte, Node->GetMessageContext(), Options);
            SwizzleOp->Sources[0] = Source;

		}

        Ptr<ASTOp> ResultOp = SwizzleOp;

		if (CompressedFormat != EImageFormat::None)
		{
            Ptr<ASTOpImagePixelFormat> FormatOp = new ASTOpImagePixelFormat();
            FormatOp->Source = ResultOp;
            FormatOp->Format = CompressedFormat;
			ResultOp = FormatOp;
		}

        Result.op = ResultOp;
		Result.bCanMulticompose = bCanMulticompose;
	}


	void CodeGenerator::GenerateImage_Format(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageFormat* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageFormat);


		bool bCanMulticompose = true; 
        
		check(Node->Format != EImageFormat::None);
        

		Ptr<ASTOpImagePixelFormat> FormatOp;

		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		// Source is required
		if (!Node->Source)
		{
			FormatOp = new ASTOpImagePixelFormat();
			FormatOp->Format = Node->Format;
			FormatOp->FormatIfAlpha = Node->FormatIfAlpha;
            FormatOp->Source = GenerateMissingImageCode(TEXT("Source image for format."), EImageFormat::RGBA_UByte, Node->GetMessageContext(), Options);
		}
		else
		{
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Source);
			Result.Parameter = BaseResult.Parameter;
			bCanMulticompose = BaseResult.bCanMulticompose;

			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;	

			FormatOp = GenerateFormatExpression(Node, BaseResult.op, BaseResult.CanNormalLAEncode);	
		}

		Result.op = FormatOp;
		Result.bCanMulticompose = bCanMulticompose;
	}

	void CodeGenerator::GenerateImage_Conform(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageConform* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageConform);
		

		bool bCanMulticompose = true; 
		Result.CanNormalLAEncode = EYesNoQuery::Yes;
         
		Ptr<ASTOpImageConform> ConformOp = new ASTOpImageConform();
		ConformOp->ConformDescriptor = Node->ConformDescriptor;

		// Source is required
		if (!Node->Source)
		{
            ConformOp->Source = GenerateMissingImageCode(TEXT("Source image for conform operation."), EImageFormat::RGBA_UByte, Node->GetMessageContext(), Options);
		}
		else
		{
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Source);
			ConformOp->Source = BaseResult.op;
			Result.Parameter = BaseResult.Parameter;
			bCanMulticompose = BaseResult.bCanMulticompose;
			
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
		}

        Result.op = ConformOp;
		Result.bCanMulticompose = bCanMulticompose;
	}

	void CodeGenerator::GenerateImage_Saturate(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageSaturate* Node)
	{
		

		bool bCanMulticompose = true; 
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		Ptr<ASTOpImageSaturate> op = new ASTOpImageSaturate();
        
		// Source image
        Ptr<ASTOp> Base;
        if (Node->Source )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Source);
			Base = BaseResult.op;
			bCanMulticompose = BaseResult.bCanMulticompose;

			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
		}
        else
        {
            // This argument is required
			Base = GenerateMissingImageCode(TEXT("Saturate image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }
		
		Base = GenerateImageFormat(Base, GetRGBOrRGBAFormat(Base->GetImageDesc().m_format));
		Base = GenerateImageSize(Base, Options.RectSize);
        op->Base = Base;


        // Factor
        if ( NodeScalar* Factor = Node->Factor.get() )
        {
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GenericOptions, Factor);
			op->Factor = ParamResult.op;
		}
        else
        {
            // This argument is required
            op->Factor = GenerateMissingScalarCode(TEXT("Saturation factor"), 0.5f, Node->GetMessageContext() );
        }

        Result.op = op;
		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_Luminance(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageLuminance* Node)
	{
		

		bool bCanMulticompose = true; 
		Result.CanNormalLAEncode = EYesNoQuery::Yes;
		
		Ptr<ASTOpImageLuminance> op = new ASTOpImageLuminance();
        
		// Source image
        Ptr<ASTOp> base;
        if (Node->Source )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Source);
			base = BaseResult.op;
			bCanMulticompose = BaseResult.bCanMulticompose;
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
        }
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Image luminance"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }
        base = GenerateImageFormat( base, EImageFormat::RGB_UByte );
        op->Base = base;

        Result.op = op;
		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_ColorMap(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageColorMap* Node)
	{
		// Image not suitable for multicompose operations
		

		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		Ptr<ASTOpImageColorMap> op = new ASTOpImageColorMap();

        // Base image
        Ptr<ASTOp> base ;
        if (Node->Base )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			base = BaseResult.op;
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
		}
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Colormap base image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }
        base = GenerateImageSize( base, Options.RectSize);
        op->Base = base;

        // Mask of the effect
        Ptr<ASTOp> mask;
        if (Node->Mask )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Mask);
			mask = BaseResult.op;
        }
        else
        {
            // Set the argument default value: affect all pixels.
            // TODO: Special operation code without mask
            mask = GeneratePlainImageCode(FVector4f( 1,1,1,1 ), Options);
        }
        mask = GenerateImageFormat( mask, EImageFormat::L_UByte );
        mask = GenerateImageSize( mask, Options.RectSize);
        op->Mask = mask;

        // Map image
        Ptr<ASTOp> MapImageOp;
        if (Node->Map )
        {
			FImageGenerationOptions MapOptions = Options;
			MapOptions.ImageLayoutStrategy = CompilerOptions::TextureLayoutStrategy::None;
			MapOptions.LayoutToApply = nullptr;
			MapOptions.LayoutBlockId = -1;
			MapOptions.RectSize = {};

			FImageGenerationResult BaseResult;
			GenerateImage(MapOptions, BaseResult, Node->Map);
			MapImageOp = BaseResult.op;
			MapImageOp = GenerateImageFormat(MapImageOp, EImageFormat::RGB_UByte);
		}
        else
        {
            // This argument is required
			MapImageOp = GenerateMissingImageCode(TEXT("Map image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }
        op->Map = MapImageOp;

        Result.op = op;
    }


	void CodeGenerator::GenerateImage_Binarise(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageBinarise* Node)
	{
		

		Ptr<ASTOpImageBinarize> op = new ASTOpImageBinarize();

		bool bCanMulticompose = true; 
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        // A image
        Ptr<ASTOp> BaseOp;
        if (Node->Base )
        {
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			BaseOp = BaseResult.op;
			bCanMulticompose = BaseResult.bCanMulticompose;
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
		}
        else
        {
            // This argument is required
			BaseOp = GenerateMissingImageCode(TEXT("Image Binarise Base"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }
		BaseOp = GenerateImageFormat(BaseOp, EImageFormat::RGB_UByte );
		BaseOp = GenerateImageSize(BaseOp, Options.RectSize);
        op->Base = BaseOp;

        // Threshold
        Ptr<ASTOp> ThresholdOp;
        if ( NodeScalar* ThresholdNode = Node->Threshold.get() )
        {
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GenericOptions, ThresholdNode);
			ThresholdOp = ParamResult.op;
        }
        else
        {
            // This argument is required
			ThresholdOp = GenerateMissingScalarCode(TEXT("Image Binarise Threshold"), 0.5f, Node->GetMessageContext() );
        }
        op->Threshold = ThresholdOp;

        Result.op = op;
		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_Resize(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageResize* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageResize);
		
        // Source image
        Ptr<ASTOp> base;
		bool bParameter = false;
		bool bCanMulticompose = true;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        if (Node->Base)
        {
			FImageGenerationOptions NewOptions = Options;
			
			if (Node->bRelative)
			{
				NewOptions.RectSize[0] = FMath::RoundToInt32(NewOptions.RectSize[0] / Node->SizeX);
				NewOptions.RectSize[1] = FMath::RoundToInt32(NewOptions.RectSize[1] / Node->SizeY);
			}

			FImageGenerationResult BaseResult;
			GenerateImage(NewOptions, BaseResult, Node->Base);
        	base = BaseResult.op;
        	bParameter = BaseResult.Parameter;
			bCanMulticompose = BaseResult.bCanMulticompose;
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
        }
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Image resize base"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }

        // Size
        if (Node->bRelative)
        {
            Ptr<ASTOpImageResizeRel> NewOp = new ASTOpImageResizeRel();
			NewOp->Factor[0] = Node->SizeX;
			NewOp->Factor[1] = Node->SizeY;
			NewOp->Source = base;
	        Result.op = NewOp;
        }
        else
        {
			// Apply the layout block to the rect size
			UE::Math::TIntVector2<int32> FinalImageSize = { int32(Node->SizeX), int32(Node->SizeY) };
			if (Options.LayoutToApply && !bParameter)
			{
				int32 BlockIndex = Options.LayoutToApply->FindBlock(Options.LayoutBlockId);
				check(BlockIndex >= 0);

				// Block in layout grid units
				box< FIntVector2 > RectInCells;
				RectInCells.min = Options.LayoutToApply->Blocks[BlockIndex].Min;
				RectInCells.size = Options.LayoutToApply->Blocks[BlockIndex].Size;

				FIntPoint grid = Options.LayoutToApply->GetGridSize();
				grid[0] = FMath::Max(1, grid[0]);
				grid[1] = FMath::Max(1, grid[1]);

				// Transform to pixels
				FinalImageSize[0] = (RectInCells.size[0] * FinalImageSize[0]) / grid[0];
				FinalImageSize[1] = (RectInCells.size[1] * FinalImageSize[1]) / grid[1];
			}

            Ptr<ASTOpImageResize> ResizeOp = new ASTOpImageResize();
			ResizeOp->Size[0] = (uint16)FinalImageSize[0];
			ResizeOp->Size[1] = (uint16)FinalImageSize[1];
			ResizeOp->Source = base;
        	Result.op = ResizeOp;

        	if (bParameter)
        	{
        		GenerateCrop(Options, Result, *Node);
        	}
        }

		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_PlainColor(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImagePlainColor* Node)
	{
		
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		// Source color
        Ptr<ASTOp> base;
        if (Node->Color )
        {
			FColorGenerationResult ColorResult;
            GenerateColor( ColorResult, Options.GenericOptions, Node->Color.get() );
			base = ColorResult.op;
        }
        else
        {
            // This argument is required
            base = GenerateMissingColorCode(TEXT("Image plain color base"), Node->GetMessageContext() );
        }

		UE::Math::TIntVector2<int32> FinalImageSize = { 0, 0 };

		if (Options.RectSize.X > 0)
		{
			FinalImageSize = Options.RectSize;
		}
		else
		{
			FinalImageSize = { Node->SizeX, Node->SizeY };

			// Apply the layout block to the rect size
			if (Options.LayoutToApply)
			{
				int32 BlockIndex = Options.LayoutToApply->FindBlock(Options.LayoutBlockId);
				check(BlockIndex >= 0);

				// Block in layout grid units
				box< FIntVector2 > RectInCells;
				RectInCells.min = Options.LayoutToApply->Blocks[BlockIndex].Min;
				RectInCells.size = Options.LayoutToApply->Blocks[BlockIndex].Size;

				FIntPoint grid = Options.LayoutToApply->GetGridSize();
				grid[0] = FMath::Max(1, grid[0]);
				grid[1] = FMath::Max(1, grid[1]);

				// Transform to pixels
				FinalImageSize[0] = (RectInCells.size[0] * FinalImageSize[0]) / grid[0];
				FinalImageSize[1] = (RectInCells.size[1] * FinalImageSize[1]) / grid[1];
			}
		}


        Ptr<ASTOpImagePlainColor> op = new ASTOpImagePlainColor();
        op->Color = base;
		op->Format = Node->Format;
		op->Size[0] = FinalImageSize[0];
		op->Size[1] = FinalImageSize[1];
		op->LODs = 1;

        Result.op = op;
    }


    //---------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateImage_Switch(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageSwitch* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageSwitch);

		bool bCanMulticompose = true;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        if (Node->Options.Num() == 0)
		{
			// No options in the switch!
			Result.op = GenerateMissingImageCode(TEXT("Switch option"), EImageFormat::RGBA_UByte, Node->GetMessageContext(), Options);
			return;
		}

        Ptr<ASTOpSwitch> op = new ASTOpSwitch();
        op->Type = EOpType::IM_SWITCH;

		// Variable value
		if (Node->Parameter )
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GenericOptions, Node->Parameter.get());
			op->Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
            op->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Node->GetMessageContext() );
		}

		bool bFirstOptionProcessed = false;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		// Options
        for ( int32 OptionIndex=0; OptionIndex < Node->Options.Num(); ++OptionIndex)
        {
            Ptr<ASTOp> branch;

            if (Node->Options[OptionIndex])
            {
				FImageGenerationResult BaseResult;
				GenerateImage(Options, BaseResult, Node->Options[OptionIndex]);
				branch = BaseResult.op;
				bCanMulticompose = bCanMulticompose && BaseResult.bCanMulticompose;

				if (!bFirstOptionProcessed)
				{
					Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode;
					bFirstOptionProcessed = true;
				}
				else if (BaseResult.CanNormalLAEncode != Result.CanNormalLAEncode)
				{
					Result.CanNormalLAEncode = EYesNoQuery::Mixed;
				}
			}
            else
            {
                // This argument is required
                branch = GenerateMissingImageCode(TEXT("Switch option"), EImageFormat::RGBA_UByte, Node->GetMessageContext(), Options);
            }

            op->Cases.Emplace(int16(OptionIndex),op,branch);
        }

        Ptr<ASTOp> switchAt = op;

        Result.op = switchAt;
		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_Conditional(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageConditional* Node)
	{
		

		Ptr<ASTOpConditional> op = new ASTOpConditional();
        op->type = EOpType::IM_CONDITIONAL;

        // Condition
        if (Node->Parameter)
        {
			FBoolGenerationResult ParamResult;
			GenerateBool(ParamResult, Options.GenericOptions, Node->Parameter.get());
			op->condition = ParamResult.op;
		}
        else
        {
            // This argument is required
            op->condition = GenerateMissingBoolCode(TEXT("Conditional condition"), true, Node->GetMessageContext() );
        }


        // Options
		FImageGenerationResult YesResult;
		GenerateImage(Options, YesResult, Node->True);
		op->yes = YesResult.op;

		Result.CanNormalLAEncode = YesResult.CanNormalLAEncode;


		FImageGenerationResult NoResult;
		GenerateImage(Options, NoResult, Node->False);
		op->no = NoResult.op;
		
		if (Result.CanNormalLAEncode != NoResult.CanNormalLAEncode)
		{
			Result.CanNormalLAEncode = EYesNoQuery::Mixed;
		}
		
        Result.op = op;
		Result.bCanMulticompose = NoResult.bCanMulticompose && YesResult.bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_Project(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageProject* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageProject);

		// Image not suitable for multicompose operations
		Result.bCanMulticompose = false;

        // Mesh project operation
        //------------------------------
        Ptr<ASTOpMeshProject> ProjectOp = new ASTOpMeshProject();

        Ptr<ASTOp> LastMeshOp = ProjectOp;

        // Projector
        FProjectorGenerationResult projectorResult;
        if (Node->Projector )
        {
            GenerateProjector( projectorResult, Options.GenericOptions, Node->Projector );
            //projectorAt = Generate( Node->Projector.get() );
        }
        else
        {
            // This argument is required
            GenerateMissingProjectorCode( projectorResult, Node->GetMessageContext() );
        }

		ProjectOp->Projector = projectorResult.op;
		ProjectOp->LayoutIndex = Node->Layout;

		int32 LayoutBlockIndex = -1;
		if (Options.LayoutToApply)
		{
			LayoutBlockIndex = Options.LayoutToApply->Blocks.IndexOfByPredicate([&](const FLayoutBlock& Block) { return Block.Id == Options.LayoutBlockId; });
		}
		uint64 GeneratedLayoutBlockId = FLayoutBlock::InvalidBlockId;

        // Mesh
        if (Node->Mesh )
        {
			// TODO: This will probably Result in a duplicated mesh subgraph, with the original mesh but new layout block ids.
			// See if it can be optimized and try to reuse the existing layout block ids instead of generating new ones.
			FMeshGenerationStaticOptions MeshStaticOptions(Options.ComponentId, Options.LODIndex);
			MeshStaticOptions.State = Options.GenericOptions.State;
			MeshStaticOptions.ActiveTags = Options.GenericOptions.ActiveTags;
			FMeshGenerationDynamicOptions MeshOptions;
			MeshOptions.bLayouts = true;			// We need the layout that we will use to render
			MeshOptions.bNormalizeUVs = true;		// We need normalized UVs for the projection

			FMeshTask MeshTask = GenerateMesh(MeshStaticOptions, UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>(MeshOptions), Node->Mesh );
			// This forces a wait to sync here. When images are also generated in tasks, it can be turned into a prerequisite instead.
    		WaitTask(MeshTask);
			FMeshGenerationResult MeshResult = MeshTask.GetResult();

			// Match the block id of the block we are generating with the id that resulted in the generated mesh			
			TManagedPtr<const FLayout> Layout = MeshResult.GeneratedLayouts.IsValidIndex(Node->Layout) ? MeshResult.GeneratedLayouts[Node->Layout].Layout : nullptr;
			if (Layout && Layout->Blocks.IsValidIndex(LayoutBlockIndex))
			{
				GeneratedLayoutBlockId = Layout->Blocks[LayoutBlockIndex].Id;
			}
			else if (Layout && Layout->Blocks.Num() == 1)
			{
				// Layout management disabled, use the only block available
				GeneratedLayoutBlockId = Layout->Blocks[0].Id;
			}
			else
			{
				ErrorLog->Add("Layout or block index error.", ELMT_ERROR, Node->GetMessageContext());
			}

			// TODO: 
			// MeshResult.MeshOp has some modifiers applied already: the ones applied before other operations directly in the mesh constant generation. 
			// This is not what was happening before the refactor  so use MeshResult.BaseMeshOp. This is another case of ambiguity of order of modifiers 
			// that whould be fixed with the general ordering design. 
			// Actually use the MeshOp, otherwise the projector will only project to the first option if the mesh operation is a switch. 
			Ptr<ASTOp> CurrentMeshToProjectOp = MeshResult.MeshOp;
			if (!MeshResult.MeshOp)
			{
				Result.op = nullptr;
				return;
			}

			// For wrapping projectors we need the entire mesh (the projection itself removes the faces that are
			// not in the layout block being generated), so the layout block is only extracted for the planar and
			// cylindrical projectors.
            if (projectorResult.type != EProjectorType::Wrapping &&
				GeneratedLayoutBlockId != FLayoutBlock::InvalidBlockId)
            {
                // Extract the mesh layout block
                Ptr<ASTOpMeshExtractLayoutBlocks> eop = new ASTOpMeshExtractLayoutBlocks();
                eop->Source = CurrentMeshToProjectOp;
                eop->LayoutIndex = Node->Layout;

                eop->Blocks.Add(GeneratedLayoutBlockId);

				CurrentMeshToProjectOp = eop;
            }

			ProjectOp->Mesh = CurrentMeshToProjectOp;

        }
        else
        {
            // This argument is required
            TManagedPtr<const FMesh> TempMesh = MakeManaged<FMesh>();
            Ptr<ASTOpConstantResource> cop = new ASTOpConstantResource();
            cop->Type = EOpType::ME_CONSTANT;
            cop->SetValue(TempMesh);
			ProjectOp->Mesh = cop;
            ErrorLog->Add( "Projector mesh not set.", ELMT_ERROR, Node->GetMessageContext() );
        }


        // Image raster operation
        //------------------------------
        Ptr<ASTOpImageRasterMesh> ImageRasterOp = new ASTOpImageRasterMesh();
		ImageRasterOp->mesh = LastMeshOp;
		ImageRasterOp->projector = projectorResult.op;

		// Calculate size of image to raster:
		// The full image is:
		// 0) The hint value in the image options passed down.
		// 1) whatever is specified in the node attributes.
		// 2) if that is 0, the size of the mask
		// 3) if still 0, take the size of the image to project (which is not necessarily related, but often)
		// 4) if still 0, a default value bigger than 0
		// then if we are applying a layout a layout block rect need to be calculated of that size, like in image constants.
		UE::Math::TIntVector2<int32> RasterImageSize = Options.RectSize;
		bool bApplyLayoutToSize = false;

		if (RasterImageSize.X == 0)
		{
			RasterImageSize = UE::Math::TIntVector2<int32>(Node->ImageSize);
			bApplyLayoutToSize = true;
		}

		// Target mask
		if (Node->Mask)
		{
			FImageGenerationResult MaskResult;
			GenerateImage(Options, MaskResult, Node->Mask);
			Ptr<ASTOp> mask = MaskResult.op;

			mask = GenerateImageFormat(mask, EImageFormat::L_UByte);

			ImageRasterOp->mask = GenerateImageSize(mask, RasterImageSize);

			if (RasterImageSize.X == 0)
			{
				FImageDesc MaskDesc = ImageRasterOp->mask->GetImageDesc();
				RasterImageSize = UE::Math::TIntVector2<int32>(MaskDesc.m_size);
				bApplyLayoutToSize = true;
			}
		}

        // Image
        if (Node->Image )
        {
            // Generate
			FImageGenerationOptions NewOptions = Options;
			NewOptions.ImageLayoutStrategy = CompilerOptions::TextureLayoutStrategy::None;
			NewOptions.LayoutToApply = nullptr;
			NewOptions.LayoutBlockId = FLayoutBlock::InvalidBlockId;
			NewOptions.RectSize = { 0,0 };

			FImageGenerationResult ImageResult;
			GenerateImage(NewOptions, ImageResult, Node->Image);
			ImageRasterOp->image = ImageResult.op;

			FImageDesc desc = ImageRasterOp->image->GetImageDesc();
			ImageRasterOp->SourceSizeX = desc.m_size[0];
			ImageRasterOp->SourceSizeY = desc.m_size[1];

			Result.CanNormalLAEncode = ImageResult.CanNormalLAEncode; 

			if (RasterImageSize.X == 0)
			{
				RasterImageSize = UE::Math::TIntVector2<int32>(desc.m_size);
				bApplyLayoutToSize = true;
			}
        }
        else
        {
            // This argument is required
			ImageRasterOp->image = GenerateMissingImageCode(TEXT("Projector image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }

		if (RasterImageSize.X == 0)
		{
			// Last resort
			RasterImageSize = { 256, 256 };
		}

		// Apply the layout block to the rect size
		if (bApplyLayoutToSize && Options.LayoutToApply)
		{
			int32 BlockIndex = Options.LayoutToApply->FindBlock(Options.LayoutBlockId);
			check(BlockIndex >= 0);

			// Block in layout grid units
			box< FIntVector2 > RectInCells;
			RectInCells.min = Options.LayoutToApply->Blocks[BlockIndex].Min;
			RectInCells.size = Options.LayoutToApply->Blocks[BlockIndex].Size;

			FIntPoint grid = Options.LayoutToApply->GetGridSize();
			grid[0] = FMath::Max(1, grid[0]);
			grid[1] = FMath::Max(1, grid[1]);

			// Transform to pixels
			RasterImageSize[0] = (RectInCells.size[0] * RasterImageSize[0]) / grid[0];
			RasterImageSize[1] = (RectInCells.size[1] * RasterImageSize[1]) / grid[1];
		}

        // Image size, from the current block being generated
		ImageRasterOp->SizeX = RasterImageSize[0];
		ImageRasterOp->SizeY = RasterImageSize[1];
		ImageRasterOp->BlockId = GeneratedLayoutBlockId;
		ImageRasterOp->LayoutIndex = Node->Layout;

		ImageRasterOp->bIsRGBFadingEnabled = Node->bIsRGBFadingEnabled;
		ImageRasterOp->bIsAlphaFadingEnabled = Node->bIsAlphaFadingEnabled;
		ImageRasterOp->SamplingMethod = Node->SamplingMethod;
		ImageRasterOp->MinFilterMethod = Node->MinFilterMethod;

		// Fading angles are optional, and stored in a color. If one exists, we generate both.
		if (Node->AngleFadeStart || Node->AngleFadeEnd)
		{
			Ptr<NodeScalarConstant> pDefaultFade = new NodeScalarConstant();
			pDefaultFade->Value = 180.0f;

			Ptr<NodeColorFromScalars> pPropsNode = new NodeColorFromScalars();

			if (Node->AngleFadeStart)
			{
				pPropsNode->X = Node->AngleFadeStart;
			}
			else
			{
				pPropsNode->X = pDefaultFade;
			}

			if (Node->AngleFadeEnd)
			{
				pPropsNode->Y = Node->AngleFadeEnd;
			}
			else
			{
				pPropsNode->Y = pDefaultFade;
			}

			FColorGenerationResult ParamResult;
			GenerateColor(ParamResult, Options.GenericOptions, pPropsNode);
			ImageRasterOp->angleFadeProperties = ParamResult.op;
		}

        // Seam correction operations
        //------------------------------
		if (Node->bEnableTextureSeamCorrection)
		{
			Ptr<ASTOpImageRasterMesh> MaskRasterOp = new ASTOpImageRasterMesh();
			MaskRasterOp->mesh = ImageRasterOp->mesh.child();
			MaskRasterOp->image = 0;
			MaskRasterOp->mask = 0;
			MaskRasterOp->BlockId = ImageRasterOp->BlockId;
			MaskRasterOp->LayoutIndex = ImageRasterOp->LayoutIndex;
			MaskRasterOp->SizeX = ImageRasterOp->SizeX;
			MaskRasterOp->SizeY = ImageRasterOp->SizeY;
			MaskRasterOp->UncroppedSizeX = ImageRasterOp->UncroppedSizeX;
			MaskRasterOp->UncroppedSizeY = ImageRasterOp->UncroppedSizeY;
			MaskRasterOp->CropMinX = ImageRasterOp->CropMinX;
			MaskRasterOp->CropMinY = ImageRasterOp->CropMinY;
			MaskRasterOp->SamplingMethod = ESamplingMethod::Point;
			MaskRasterOp->MinFilterMethod = EMinFilterMethod::None;

			Ptr<ASTOpImageMakeGrowMap> MakeGrowMapOp = new ASTOpImageMakeGrowMap();
			MakeGrowMapOp->Mask = MaskRasterOp;
			MakeGrowMapOp->Border = MUTABLE_GROW_BORDER_VALUE;

			// If we want to be able to generate progressive mips efficiently, we need mipmaps for the "displacement map".
			if (CompilerOptions->OptimisationOptions.bEnableProgressiveImages)
			{
				Ptr<ASTOpImageMipmap> MipMask = new ASTOpImageMipmap;
				MipMask->Source = MakeGrowMapOp->Mask.child();
				MipMask->bPreventSplitTail = true;
				MakeGrowMapOp->Mask = MipMask;
			}

			Ptr<ASTOpImageDisplace> DisplaceOp = new ASTOpImageDisplace;
			DisplaceOp->DisplacementMap = MakeGrowMapOp;
			DisplaceOp->Source = ImageRasterOp;

			Result.op = DisplaceOp;
		}
		else
		{
			Result.op = ImageRasterOp;
		}		
    }


	void CodeGenerator::GenerateImage_Mipmap(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageMipmap* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeImageMipmap);

		bool bCanMulticompose = true;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

        Ptr<ASTOp> res;

        Ptr<ASTOpImageMipmap> op = new ASTOpImageMipmap();

        // At the end of the day, we want all the mipmaps. Maybe the code optimiser will split the process later.
        op->Levels = 0;

        // Source image
        Ptr<ASTOp> base;
        if (Node->Source )
        {
            MUTABLE_CPUPROFILER_SCOPE(Base);
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Source);
			base = BaseResult.op;
			bCanMulticompose = BaseResult.bCanMulticompose;
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
		}
        else
        {
            // This argument is required
            base = GenerateMissingImageCode(TEXT("Mipmap image"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
        }

        op->Source = base;

        // The number of tail mipmaps depends on the cell size. We need to know it for some code
        // optimisation operations. Scan the source image code looking for this info
        int32 blockX = 0;
        int32 blockY = 0;
        if (Options.ImageLayoutStrategy != CompilerOptions::TextureLayoutStrategy::None)
        {
            MUTABLE_CPUPROFILER_SCOPE(GetLayoutBlockSize);
            op->Source->GetLayoutBlockSize( &blockX, &blockY );
        }

        if ( blockX && blockY )
        {
            int32 mipsX = (int32)ceilf( logf( (float)blockX )/logf(2.0f) );
            int32 mipsY = (int32)ceilf( logf( (float)blockY )/logf(2.0f) );
            op->BlockLevels = (uint8)FMath::Max( mipsX, mipsY );
        }
        else
        {
            // No layout. Mipmap all the way down.
            op->BlockLevels = 0;
        }

		op->AddressMode = Node->Settings.AddressMode;
		op->FilterType = Node->Settings.FilterType;

        res = op;

        Result.op = res;
		Result.bCanMulticompose = bCanMulticompose;
    }


	void CodeGenerator::GenerateImage_Invert(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageInvert* Node)
	{

		bool bCanMulticompose = true;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		Ptr<ASTOpImageInvert> op = new ASTOpImageInvert();

		Ptr<ASTOp> a;
		if (Node->Base)
		{
			FImageGenerationResult BaseResult;
			GenerateImage(Options, BaseResult, Node->Base);
			a = BaseResult.op;
			bCanMulticompose = BaseResult.bCanMulticompose;
			Result.CanNormalLAEncode = BaseResult.CanNormalLAEncode; 
		}
		else
		{
			// This argument is required
			a = GenerateMissingImageCode(TEXT("Image Invert Color"), EImageFormat::RGB_UByte, Node->GetMessageContext(), Options);
		}
		a = GenerateImageFormat(a, EImageFormat::RGB_UByte);
		a = GenerateImageSize(a, Options.RectSize);
		op->Base = a;

		Result.op = op;
		Result.bCanMulticompose = bCanMulticompose;
	}


	void CodeGenerator::GenerateImage_Variation(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageVariation* Node)
	{
		

		bool bCanMulticompose = true;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;

		Ptr<ASTOp> currentOp;

		bool bFirstVariationProcessed = false;

        // Default case
        if (Node->DefaultImage )
        {
            FImageGenerationResult BranchResults;
			GenerateImage(Options, BranchResults, Node->DefaultImage);
			currentOp = BranchResults.op;

			Result.CanNormalLAEncode = BranchResults.CanNormalLAEncode;	
			bFirstVariationProcessed = true;
        }
        

        // Process variations in reverse order, since conditionals are built bottom-up.
        for ( int32 t = int32(Node->Variations.Num() ) - 1; t >= 0; --t )
        {
            int32 tagIndex = -1;
            const FString& tag = Node->Variations[t].Tag;
            for ( int32 i = 0; i < int32( FirstPass.Tags.Num() ); ++i )
            {
                if ( FirstPass.Tags[i].Tag == tag )
                {
                    tagIndex = i;
                }
            }

            if ( tagIndex < 0 )
            {
				FString Msg = FString::Printf(TEXT("Unknown tag found in image variation [%s]."), *tag );

                ErrorLog->Add( Msg, ELMT_WARNING, Node->GetMessageContext() );
                continue;
            }

            Ptr<ASTOp> variationOp;
            if (Node->Variations[t].Image )
            {
				FImageGenerationResult VariationResult;
				GenerateImage(Options, VariationResult, Node->Variations[t].Image);
				variationOp = VariationResult.op;
				bCanMulticompose = bCanMulticompose && VariationResult.bCanMulticompose;

				if (!bFirstVariationProcessed)
				{
					Result.CanNormalLAEncode = VariationResult.CanNormalLAEncode;	
					bFirstVariationProcessed = true;
				}
				else if (Result.CanNormalLAEncode != VariationResult.CanNormalLAEncode)
				{
					Result.CanNormalLAEncode = EYesNoQuery::Mixed;	
				}

            }
            else
            {
                // This argument is required
                variationOp = GenerateMissingImageCode(TEXT("Variation option"), EImageFormat::RGBA_UByte, Node->GetMessageContext(), Options);
            }


            Ptr<ASTOpConditional> conditional = new ASTOpConditional;
            conditional->type = EOpType::IM_CONDITIONAL;
            conditional->no = currentOp;
            conditional->yes = variationOp;
            conditional->condition = FirstPass.Tags[tagIndex].GenericCondition;

            currentOp = conditional;
        }

        // Make sure all options are the same format and size
		FImageDesc desc;
		if (currentOp)
		{
			desc = currentOp->GetImageDesc(true);
		}

        if ( desc.m_format == EImageFormat::None )
        {
            // TODO: Look for the most generic of the options?
            // For now force a decently generic one
            desc.m_format = EImageFormat::RGBA_UByte;
        }

        if ( desc.m_size[0] != 0 && desc.m_size[1] != 0 )
        {
            Ptr<ASTOpImageResize> sop = new ASTOpImageResize();
            sop->Size[0] = desc.m_size[0];
            sop->Size[1] = desc.m_size[1];
            sop->Source = currentOp;
            currentOp = sop;
        }

        {
            Ptr<ASTOpImagePixelFormat> fop = new ASTOpImagePixelFormat();
            fop->Format = desc.m_format;
            fop->Source = currentOp;
            currentOp = fop;
        }

        Result.op = currentOp;
		Result.bCanMulticompose = bCanMulticompose;
    }

	
	void CodeGenerator::GenerateImage_Table(const FImageGenerationOptions& Options, FImageGenerationResult& Result, const NodeImageTable* InNode)
	{

		const NodeImageTable& node = *InNode;

		bool bCanMulticompose = true;
		Result.op = GenerateTableSwitch<NodeImageTable, ETableColumnType::Image, EOpType::IM_SWITCH>(node,
			[this, InNode, Options, &bCanMulticompose](const NodeImageTable& node, int32 colIndex, int32 row, FErrorLog* LocalErrorLog)
			{
				const FTableValue& CellData = node.Table->GetPrivate()->Rows[row].Values[colIndex];
				TManagedPtr<FImage> pImage = CellData.Image;

				Ptr<ASTOp> ImageOp;

				if (!pImage)
				{
					if (row != 0) // "None" option (row 0) is always a null image, do not trigger this error.
					{
						FString Msg = FString::Printf(TEXT("Table has a missing image in column %d, row %d."), colIndex, row);
						LocalErrorLog->Add(Msg, ELMT_ERROR, InNode->GetMessageContext());
					}

					return ImageOp;
				}
				else
				{
					Ptr<NodeImageConstant> ImageConst = new NodeImageConstant();
					ImageConst->Image = pImage;

					// TODO: We probably want to get the data tags from the table row.
					ImageConst->SourceDataDescriptor = InNode->SourceDataDescriptor;

					// Combine the SourceId of the node with the RowId to generate one shared between all resources from this row.
					// Hash collisions are allowed, since it is used to group resources, not to differentiate them.
					const uint32 RowId = node.Table->GetPrivate()->Rows[row].Id;
					ImageConst->SourceDataDescriptor.SourceId = HashCombine(InNode->SourceDataDescriptor.SourceId, RowId);

					FImageGenerationResult Result;
					GenerateImage(Options, Result, ImageConst);
					ImageOp = Result.op;
					bCanMulticompose = bCanMulticompose && Result.bCanMulticompose;

					int32 MaxTextureSize = FMath::Max(node.ReferenceImageDesc.m_size[0], node.ReferenceImageDesc.m_size[1]);

					if (MaxTextureSize > 0 && (MaxTextureSize < pImage->GetSizeX() || MaxTextureSize < pImage->GetSizeY()))
					{
						// Use a relative resize, because at this point we may be generating a layout block and not the full image
						float Factor = FMath::Min(MaxTextureSize / (float)(pImage->GetSizeX()), MaxTextureSize / (float)(pImage->GetSizeY()));
						Ptr<ASTOpImageResizeRel> ResizeOp = new ASTOpImageResizeRel();
						ResizeOp->Factor[0] = Factor;
						ResizeOp->Factor[1] = Factor;
						ResizeOp->Source = ImageOp;
						ImageOp = ResizeOp;
					}
				}

				return ImageOp;
			});
		Result.bCanMulticompose = bCanMulticompose;
		Result.CanNormalLAEncode = EYesNoQuery::Yes;
	}


	void CodeGenerator::GenerateImage_MaterialBreak(const FImageGenerationOptions& InOptions, FImageGenerationResult& Result, const NodeImageMaterialBreak* Node)
	{

		Ptr<ASTOpMaterialBreak> MaterialOp = new ASTOpMaterialBreak();

		// Set the parameter key that this break node will generate
		MaterialOp->ParameterKey = Node->ParameterKey;

		// Generate Material Source
		FMaterialGenerationResult MaterialSourceResult;
		FMaterialGenerationOptions MaterialSourceOptions;
		MaterialSourceOptions.ComponentId = InOptions.ComponentId;
		MaterialSourceOptions.LODIndex = InOptions.LODIndex;
		MaterialSourceOptions.GenericOptions.State = InOptions.GenericOptions.State;
		MaterialSourceOptions.ImageLayoutStrategy = InOptions.ImageLayoutStrategy;
		MaterialSourceOptions.RectSize = InOptions.RectSize;
		MaterialSourceOptions.LayoutBlockId = InOptions.LayoutBlockId;
		MaterialSourceOptions.LayoutToApply = InOptions.LayoutToApply;

		GenerateMaterial(MaterialSourceOptions, MaterialSourceResult, Node->MaterialSource);
		MaterialOp->Material = MaterialSourceResult.op;
		MaterialOp->Type = EOpType::IM_MATERIAL_BREAK;

		Result.op = MaterialOp;
		Result.CanNormalLAEncode = EYesNoQuery::No;
	}


	void CodeGenerator::GenerateImage_FromMaterialParameter(const FImageGenerationOptions& InOptions, FImageGenerationResult& Result, const NodeImageFromMaterialParameter* InNode)
	{

		Ptr<ASTOpImageFromMaterialParameter> ImageOp = new ASTOpImageFromMaterialParameter();
		ImageOp->MaterialParameter = InOptions.MaterialParameter;
		ImageOp->ParameterKey = InNode->ImageParameterKey;

		Result.op = ImageOp;
		Result.CanNormalLAEncode = EYesNoQuery::No;
	}


	void CodeGenerator::GenerateImage_External(const FImageGenerationOptions& InOptions, FImageGenerationResult& Result, const NodeImageExternal* InNode)
	{
		

		FExternalGenerationOptions Options;
		Options.ImageOptions = InOptions;

		FExternalTask ExtensionResult = GenerateExternal(Options, InNode->Node);
		WaitTask(ExtensionResult);

		Result.op = ExtensionResult.GetResult().Op;
		Result.Parameter = false;
		Result.bCanMulticompose = false;
		Result.CanNormalLAEncode = EYesNoQuery::No;
	}

	
	void CodeGenerator::GenerateImage_ParameterConvert(const FImageGenerationOptions& InOptions, FImageGenerationResult& Result, const NodeImageConvert* InNode)
	{
		Ptr<ASTOpImageParameterConvert> ReferenceConvert = new ASTOpImageParameterConvert();

		FImageGenerationResult PassthroughImageResult;
		GenerateImage(InOptions, PassthroughImageResult, InNode->ImageParameter);

		ReferenceConvert->ImageParameter = PassthroughImageResult.op;
		ReferenceConvert->ImageDescriptor = InNode->ImageDescriptor;
		
		Result.op = ReferenceConvert;
		Result.Parameter = PassthroughImageResult.Parameter;
		Result.CanNormalLAEncode = PassthroughImageResult.CanNormalLAEncode;
	}


	TManagedPtr<FImage> CodeGenerator::GenerateMissingImage(EImageFormat Format)
	{
		MUTABLE_CPUPROFILER_SCOPE(GenerateMissingImage);

		// Make a checkered debug image
		const FImageSize Size(16, 16);

		TManagedPtr<FImage> GeneratedImage = MakeManaged<FImage>(Size[0], Size[1], 1, Format, EInitializationType::NotInitialized);

		switch (Format)
		{
		case EImageFormat::L_UByte:
		{
			uint8* DataPtr = GeneratedImage->GetLODData(0);
			for (int32 P = 0; P < Size[0]*Size[1]; ++P)
			{
				if ((P + P/Size[0]) % 2)
				{
					DataPtr[0] = 255;
				}
				else
				{
					DataPtr[0] = 64;
				}

				DataPtr++;
			}
			break;
		}
		case EImageFormat::RGB_UByte:
		{
			uint8* DataPtr = GeneratedImage->GetLODData(0);
			for (int32 P = 0; P < Size[0]*Size[1]; ++P)
			{
				if ((P + P/Size[0]) % 2)
				{
					DataPtr[0] = 255;
					DataPtr[1] = 255;
					DataPtr[2] = 64;
				}
				else
				{
					DataPtr[0] = 64;
					DataPtr[1] = 64;
					DataPtr[2] = 255;
				}

				DataPtr += 3;
			}
			break;
		}
		case EImageFormat::BGRA_UByte:
		case EImageFormat::RGBA_UByte:
		{
			uint8* DataPtr = GeneratedImage->GetLODData(0);
			for (int32 P = 0; P < Size[0]*Size[1]; ++P)
			{
				if ((P + P/Size[0]) % 2)
				{
					DataPtr[0] = 255;
					DataPtr[1] = 255;
					DataPtr[2] = 64;
					DataPtr[3] = 255;
				}
				else
				{
					DataPtr[0] = 64;
					DataPtr[1] = 64;
					DataPtr[2] = 255;
					DataPtr[3] = 128;
				}

				DataPtr += 4;
			}
			break;
		}

		default:
			check( false );
			break;

		}

		return GeneratedImage;
	}


	Ptr<ASTOp> CodeGenerator::GenerateMissingImageCode(const TCHAR* strWhere, EImageFormat format, const void* ErrorContext, const FImageGenerationOptions& Options )
	{
		// Log an error message
		FString Msg = FString::Printf(TEXT("Required connection not found: %s"), strWhere );
		ErrorLog->Add( Msg, ELMT_ERROR, ErrorContext);

		// Make a checkered debug image
		TManagedPtr<FImage> GeneratedImage = GenerateMissingImage( format );

		Ptr<NodeImageConstant> pNode = new NodeImageConstant();
		pNode->Image = GeneratedImage;

		FImageGenerationResult Result;
		GenerateImage(Options, Result, pNode);

		return Result.op;
	}


    Ptr<ASTOp> CodeGenerator::GeneratePlainImageCode( const FVector4f& InColor, const FImageGenerationOptions& Options )
    {
		Ptr<NodeColorConstant> ConstantColor = new NodeColorConstant();
		ConstantColor->Value = InColor;

        Ptr<NodeImagePlainColor> PlainNode = new NodeImagePlainColor();
		PlainNode->Color = ConstantColor;

		FImageGenerationResult TempResult;
		GenerateImage(Options, TempResult, PlainNode);
        Ptr<ASTOp> Result = TempResult.op;

        return Result;
    }


    Ptr<ASTOp> CodeGenerator::GenerateImageFormat( Ptr<ASTOp> Op, EImageFormat InFormat)
    {
        Ptr<ASTOp> Result = Op;

        if (InFormat!=EImageFormat::None && Op && Op->GetImageDesc().m_format!=InFormat)
        {
            // Generate the format change code
            Ptr<ASTOpImagePixelFormat> op = new ASTOpImagePixelFormat();
            op->Source = Op;
            op->Format = InFormat;
			Result = op;
        }

        return Result;
    }


    Ptr<ASTOp> CodeGenerator::GenerateImageSize( Ptr<ASTOp> at, UE::Math::TIntVector2<int32> size )
    {
        Ptr<ASTOp> Result = at;

		if (at && size[0] > 0 && size[1] > 0)
		{
			if (UE::Math::TIntVector2<int32>(at->GetImageDesc().m_size) != size)
			{
				Ptr<ASTOpImageResize> op = new ASTOpImageResize();
				op->Source = at;
				op->Size[0] = size[0];
				op->Size[1] = size[1];
				Result = op;
			}
		}

        return Result;
    }


	Ptr<ASTOp> CodeGenerator::GenerateImageMulticompose(
			Ptr<ASTOp> InBaseImageOp, 
			Ptr<ASTOp> InLayoutOp, 
			const FImageGenerationOptions& InBaseImageOptions,
			const NodeImagePtrConst& NodeImage, 
			TManagedPtr<const FLayout> SourceLayout, 
			const CodeGenerator::FLayoutBlockDesc& InLayoutBlockDesc)
	{
		Ptr<ASTOp> Result = InBaseImageOp;

		// Single block layout to generate the entire SourceImage
		TManagedPtr<FLayout> AuxLayout = SourceLayout->Clone();
		AuxLayout->Blocks.SetNum(1);
		AuxLayout->Blocks[0].Min = FIntVector2::ZeroValue;
		AuxLayout->Blocks[0].Size = AuxLayout->Size;

		// Generate the SourceImage
		FImageGenerationOptions ImageOptions = InBaseImageOptions;
		ImageOptions.LayoutToApply = AuxLayout;
		ImageOptions.LayoutBlockId = AuxLayout->Blocks[0].Id;
		FImageGenerationResult ImageResult;
		GenerateImage(ImageOptions, ImageResult, NodeImage);

		Ptr<ASTOp> SourceImage = ImageResult.op;
		if (SourceImage)
		{
			// SourceImage Layout
			Ptr<ASTOpConstantResource> SourceLayoutOp = new ASTOpConstantResource();
			SourceLayoutOp->Type = EOpType::LA_CONSTANT;
			SourceLayoutOp->SetValue(SourceLayout);

			// Compose layout operation
			Ptr<ASTOpImageMultiCompose> MultiComposeOp = new ASTOpImageMultiCompose();
			MultiComposeOp->Layout = InLayoutOp;
			MultiComposeOp->Base = InBaseImageOp;
			MultiComposeOp->SourceLayout = SourceLayoutOp;
			MultiComposeOp->SourceImage = SourceImage;

			MultiComposeOp->LayoutBlockSizeInPixelsX = InLayoutBlockDesc.BlockPixelsX;
			MultiComposeOp->LayoutBlockSizeInPixelsY = InLayoutBlockDesc.BlockPixelsY;

			FImageDesc SourceDesc = SourceImage->GetImageDesc(true);
			MultiComposeOp->SourceSizeX = SourceDesc.m_size.X;
			MultiComposeOp->SourceSizeY = SourceDesc.m_size.Y;
			
			Result = MultiComposeOp;
		}

		return Result;
	}
}
