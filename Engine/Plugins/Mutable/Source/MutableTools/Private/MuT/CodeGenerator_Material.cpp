// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpMaterialSkeletalMeshBreak.h"
#include "ASTOpMaterialSkeletalMeshObjectBreak.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuR/Material.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageBlankLayout.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageResize.h"
#include "MuT/ASTOpMaterialModify.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeMaterialConstant.h"
#include "MuT/NodeMaterialExternal.h"
#include "MuT/NodeMaterialSwitch.h"
#include "MuT/NodeMaterialVariation.h"
#include "MuT/NodeMaterialTable.h"
#include "MuT/NodeMaterialParameter.h"
#include "MuT/NodeMaterialSkeletalMeshObjectBreak.h"
#include "MuT/NodeMaterialModify.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeImageMipmap.h"
#include "MuT/NodeImageSwizzle.h"
#include "MuT/NodeMaterialSkeletalMeshBreak.h"
#include "MuT/Table.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{
	class Node;

	void CodeGenerator::GenerateMaterial(const FMaterialGenerationOptions& InOptions, FMaterialGenerationResult& Result , const Ptr<const NodeMaterial>& InUntypedNode)
	{
		// Early out
		if (!InUntypedNode)
		{
			Result = FMaterialGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedMaterialCacheKey Key;
		Key.Node = InUntypedNode;
		Key.Options = InOptions;
		{
			UE::TUniqueLock Lock(GeneratedMaterials.Mutex);
			FGeneratedMaterialsMap::ValueType* Found = GeneratedMaterials.Map.Find(Key);
			if (Found)
			{
				Result = *Found;
				return;
			}
		}

		// Generate for each different type of node
		const NodeMaterial* Node = InUntypedNode.get();
		switch (Node->GetType()->Type)
		{
		case Node::EType::MaterialConstant:
			GenerateMaterial_Constant(InOptions, Result, static_cast<const NodeMaterialConstant*>(Node)); 
			break;
		
		case Node::EType::MaterialSwitch:
			GenerateMaterial_Switch(InOptions, Result, static_cast<const NodeMaterialSwitch*>(Node));
			break;

		case Node::EType::MaterialVariation:
			GenerateMaterial_Variation(InOptions, Result, static_cast<const NodeMaterialVariation*>(Node));
			break;

		case Node::EType::MaterialTable:
			GenerateMaterial_Table(InOptions, Result, static_cast<const NodeMaterialTable*>(Node));
			break;

		case Node::EType::MaterialParameter:
			GenerateMaterial_Parameter(InOptions, Result, static_cast<const NodeMaterialParameter*>(Node));
			break;

		case Node::EType::MaterialExternal:
			GenerateMaterial_External(InOptions, Result, static_cast<const NodeMaterialExternal*>(Node));
			break;

		case Node::EType::MaterialModify:
			GenerateMaterial_Modify(InOptions, Result, static_cast<const NodeMaterialModify*>(Node));
			break;

		case Node::EType::MaterialSkeletalMeshObjectBreak:
			GenerateMaterial_BreakSkeletalMeshObject(InOptions, Result, static_cast<const NodeMaterialSkeletalMeshObjectBreak*>(Node));
			break;
			
		case Node::EType::MaterialSkeletalMeshBreak:
			GenerateMaterial_BreakSkeletalMesh(InOptions, Result, static_cast<const NodeMaterialSkeletalMeshBreak*>(Node));
			break;

		default: 
			check(false);
		}

		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedMaterials.Mutex);
			GeneratedMaterials.Map.Add(Key, Result);
		}
	}


	void CodeGenerator::GenerateMaterial_Constant(const FMaterialGenerationOptions& Options, FMaterialGenerationResult& Result, const NodeMaterialConstant* Node)
	{
		// Constant Material Resource
		TManagedPtr<FMaterial> MaterialResource = MakeManaged<FMaterial>();
		MaterialResource->PassthroughObject = TPassthroughObjectPtr<UMaterialInterface>(Node->MaterialId);
		MaterialResource->ColorParameters = Node->ColorValues;
		MaterialResource->ScalarParameters = Node->ScalarValues;

		// Constant Material Operation
		Ptr<ASTOpConstantResource> ConstantMaterial = new ASTOpConstantResource();
		ConstantMaterial->Type = EOpType::MI_CONSTANT;

		FImageGenerationOptions ImageOptions(Options.ComponentId, Options.LODIndex);
		ImageOptions.ImageLayoutStrategy = Options.ImageLayoutStrategy;
		ImageOptions.RectSize = Options.RectSize;
		ImageOptions.LayoutBlockId = Options.LayoutBlockId;
		ImageOptions.LayoutToApply = Options.LayoutToApply;

		const TArray<TPair<FParameterKey, Ptr<NodeImage>>>& ImageValuesArray = Node->ImageValues.Array();

		//Store the images into the operation so that we can link them later
		for (int32 ImageIndex = 0; ImageIndex < ImageValuesArray.Num(); ++ImageIndex)
		{
			Ptr<NodeImage> ImageNode = ImageValuesArray[ImageIndex].Value;
			FImageGenerationResult ImageResult;

			GenerateImage(ImageOptions, ImageResult, ImageNode);

			// Store lazy branches
			ConstantMaterial->ImageOperations.Add(ImageValuesArray[ImageIndex].Key, ASTChild(ConstantMaterial, ImageResult.op));
		}

		// Store the constant material
		ConstantMaterial->SetValue(MaterialResource);

		Result.op = ConstantMaterial;
	}


	void CodeGenerator::GenerateMaterial_Switch(const FMaterialGenerationOptions& Options, FMaterialGenerationResult& Result, const NodeMaterialSwitch* Node)
	{
		if (Node->Options.Num() == 0)
		{
			return;
		}

		Ptr<ASTOp> Variable;
		if (Node->Parameter)
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GenericOptions, Node->Parameter.get());
			Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
			Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Node->GetMessageContext());
		}

		Ptr<ASTOpSwitch> SwitchOp = new ASTOpSwitch();
		SwitchOp->Type = EOpType::MI_SWITCH;
		SwitchOp->Variable = Variable;

		// Options
		SwitchOp->Cases.Reserve(Node->Options.Num());

		for (int32 OptionIndex = 0; OptionIndex < Node->Options.Num(); ++OptionIndex)
		{
			FMaterialGenerationResult OptionResult;
			const Ptr<NodeMaterial>& OptionMaterial = Node->Options[OptionIndex];
			if (OptionMaterial)
			{
				GenerateMaterial(Options, OptionResult, OptionMaterial);
			}
			else
			{
				// This argument is required. Empty material
				OptionResult.op = GenerateMissingMaterialCode(TEXT("Material Switch"), Node->GetMessageContext());
			}

			SwitchOp->Cases.Emplace(int16(OptionIndex), SwitchOp, OptionResult.op);
		}

		Result.op = SwitchOp;
	}


	void CodeGenerator::GenerateMaterial_Variation(const FMaterialGenerationOptions& Options, FMaterialGenerationResult& Result, const NodeMaterialVariation* Node)
	{
		Ptr<ASTOp> LastOp;

		// Default case
		if (Node->DefaultMaterial)
		{
			FMaterialGenerationResult DefaultResult;
			GenerateMaterial(Options, DefaultResult, Node->DefaultMaterial);
			LastOp = DefaultResult.op;
		}

		// Process variations in reverse order, since conditionals are built bottom-up.
		for (int32 VariationIndex = int32(Node->Variations.Num()) - 1; VariationIndex >= 0; --VariationIndex)
		{
			int32 TagIndex = INDEX_NONE;
			const FString& tag = Node->Variations[VariationIndex].Tag;
			for (int32 Index = 0; Index < FirstPass.Tags.Num(); ++Index)
			{
				if (FirstPass.Tags[Index].Tag == tag)
				{
					TagIndex = Index;
				}
			}

			if (TagIndex == INDEX_NONE)
			{
				FString Msg = FString::Printf(TEXT("Unknown tag found in material variation [%s]."), *tag);

				ErrorLog->Add(Msg, ELMT_WARNING, Node->GetMessageContext());
				continue;
			}

			Ptr<ASTOp> VariationOp;
			if (Node->Variations[VariationIndex].Material)
			{
				FMaterialGenerationResult VariationResult;
				GenerateMaterial(Options, VariationResult, Node->Variations[VariationIndex].Material);
				VariationOp = VariationResult.op;
			}
			else
			{
				// This argument is required
				VariationOp = GenerateMissingMaterialCode(TEXT("Material Variation"), Node->GetMessageContext());
			}

			Ptr<ASTOpConditional> conditional = new ASTOpConditional;
			conditional->type = EOpType::MI_CONDITIONAL;
			conditional->no = LastOp;
			conditional->yes = VariationOp;
			conditional->condition = FirstPass.Tags[TagIndex].GenericCondition;

			LastOp = conditional;
		}

		Result.op = LastOp;
	}


	void CodeGenerator::GenerateMaterial_Table(const FMaterialGenerationOptions& Options, FMaterialGenerationResult& Result, const NodeMaterialTable* Node)
	{
		Result.op = GenerateTableSwitch<NodeMaterialTable, ETableColumnType::Material, EOpType::MI_SWITCH>(*Node,
			[this](const NodeMaterialTable& Node, int32 ColumnIndex, int32 RowIndex, FErrorLog* pErrorLog)
			{
				Ptr<ASTOp> Op;

				int32 MaterialId = Node.Table->GetPrivate()->Rows[RowIndex].Values[ColumnIndex].Int;
				if (MaterialId != INDEX_NONE)
				{
					// TODO(Max): UE-314401
					// Constant Material Resource
					TManagedPtr<FMaterial> MaterialResource = MakeManaged<FMaterial>();
					MaterialResource->PassthroughObject = TPassthroughObjectPtr<UMaterialInterface>(MaterialId);

					// Constant Material Operation
					Ptr<ASTOpConstantResource> ConstantMaterial = new ASTOpConstantResource();
					ConstantMaterial->Type = EOpType::MI_CONSTANT;

					// Store the constant material
					ConstantMaterial->SetValue(MaterialResource);

					Op = ConstantMaterial;
				}

				return Op;
			});
	}


	void CodeGenerator::GenerateMaterial_Parameter(const FMaterialGenerationOptions& InOptions, FMaterialGenerationResult& Result, const NodeMaterialParameter* InNode)
	{
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			if (Ptr<ASTOpParameter>* Found = FirstPass.ParameterNodes.GenericParametersCache.Find(InNode))
			{
				Result.op = *Found;
				return;
			}
		}
			
		Ptr<ASTOpParameter> MaterialOp = new ASTOpParameter();
		MaterialOp->Type = EOpType::MI_PARAMETER;

		MaterialOp->Parameter.Name = InNode->Name;
		bool bParseOk = FGuid::Parse(InNode->UID, MaterialOp->Parameter.UID);
		check(bParseOk);
		MaterialOp->Parameter.Type = EParameterType::Material;
		MaterialOp->Parameter.DefaultValue.Set<FParamMaterialType>(nullptr);

		// Generate the code for the ranges
		for (const Ptr<NodeRange>& RangeNode : InNode->Ranges)
		{
			FRangeGenerationResult RangeResult;
			GenerateRange(RangeResult, InOptions.GenericOptions, RangeNode);
			MaterialOp->Ranges.Emplace(MaterialOp.get(), RangeResult.sizeOp, RangeResult.rangeName, RangeResult.rangeUID);
		}

		// Generate Image Nodes
		FImageGenerationResult ImageResult;
		FImageGenerationOptions ImageOptions(InOptions.ComponentId, InOptions.LODIndex);
		ImageOptions.ImageLayoutStrategy = InOptions.ImageLayoutStrategy;
		ImageOptions.RectSize = InOptions.RectSize;
		ImageOptions.LayoutBlockId = InOptions.LayoutBlockId;
		ImageOptions.LayoutToApply = InOptions.LayoutToApply;
		ImageOptions.MaterialParameter = MaterialOp->Parameter;

		for (const TPair<FParameterKey, Ptr<NodeImage>>& ImageNode : InNode->ImageParametersToCompile)
		{
			GenerateImage(ImageOptions, ImageResult, ImageNode.Value);

			// Store lazy branches
			MaterialOp->ImageOperations.Add(ImageNode.Key, ASTChild(MaterialOp, ImageResult.op));
		}

		Result.op = MaterialOp;

		FirstPass.ParameterNodes.GenericParametersCache.Add(InNode, MaterialOp);
	}

	
	void CodeGenerator::GenerateMaterial_Modify(const FMaterialGenerationOptions& Options, FMaterialGenerationResult& Result, const NodeMaterialModify* Node)
	{
		Ptr<ASTOpMaterialModify> MaterialModify = new ASTOpMaterialModify();
		
		FMaterialGenerationResult SourceResult;

		if (Node->MaterialSource)
		{
			GenerateMaterial(Options, SourceResult, Node->MaterialSource);
			MaterialModify->Material = SourceResult.op;

			// Add all the parameters that will be modified
			// Textures
			for (const TPair<FParameterKey, FImageParameterData>& ImageParameterData : Node->ImageParameters)
			{
				FImageGenerationResult ImageResult;
				GenerateMaterialImage(ImageParameterData, ImageResult, Options.Modifiers, Options.GenericOptions.State, Options.ComponentId, Options.LODIndex, Options.Tags, Options.MeshResults, Options.LayoutFromExtension, Node->GetMessageContext());
				MaterialModify->ParametersToModify.Add(ImageParameterData.Key, ASTChild(MaterialModify, ImageResult.op));
				MaterialModify->ImagePropertyIndexMap.Add(ImageParameterData.Key, ImageParameterData.Value.ImagePropertyIndex);
			}

			// Colors
			for (const TPair<FParameterKey, Ptr<NodeColor>>& ColorParameterData : Node->ColorParameters)
			{
				FColorGenerationResult ColorResult;
				FGenericGenerationOptions ColorOptions;
				GenerateColor(ColorResult, ColorOptions, ColorParameterData.Value);

				MaterialModify->ParametersToModify.Add(ColorParameterData.Key, ASTChild(MaterialModify, ColorResult.op));
			}

			// Scalars
			for (const TPair<FParameterKey, Ptr<NodeScalar>>& ScalarParameterData : Node->ScalarParameters)
			{
				FScalarGenerationResult ScalarResult;
				FGenericGenerationOptions ScalarOptions;
				GenerateScalar(ScalarResult, ScalarOptions, ScalarParameterData.Value);

				MaterialModify->ParametersToModify.Add(ScalarParameterData.Key, ASTChild(MaterialModify, ScalarResult.op));
			}
		}
		else
		{
			MaterialModify->Material = GenerateMissingMaterialCode(TEXT("Make Material"), Node->GetMessageContext());
		}

		Result.op = MaterialModify;
	}


	void CodeGenerator::GenerateMaterial_External(const FMaterialGenerationOptions& InOptions, FMaterialGenerationResult& Result, const NodeMaterialExternal* InNode)
	{
		FExternalGenerationOptions Options;
		Options.MaterialOptions = InOptions;
					
		FExternalTask ExtensionResult = GenerateExternal(Options, InNode->Node);
		WaitTask(ExtensionResult);

		Result.op = ExtensionResult.GetResult().Op;
	}

	
	void CodeGenerator::GenerateMaterial_BreakSkeletalMeshObject(const FMaterialGenerationOptions& InOptions, FMaterialGenerationResult& OutResult, const NodeMaterialSkeletalMeshObjectBreak* InNode)
	{
		// Early out
		if (!InNode)
		{
			OutResult = FMaterialGenerationResult();
			return;
		}
		
		Ptr<ASTOpMaterialSkeletalMeshObjectBreak> MaterialOp = new ASTOpMaterialSkeletalMeshObjectBreak();
		MaterialOp->SlotName = InNode->SlotName;
		
		FSkeletalMeshGenerationOptions SkeletalMeshOptions{InOptions.GenericOptions, nullptr};
		FSkeletalMeshObjectGenerationResult SkeletalMeshGenerationResult;
		GenerateSkeletalMeshObject(SkeletalMeshGenerationResult, SkeletalMeshOptions, InNode->SkeletalMeshObject);
		MaterialOp->SkeletalMeshObject = SkeletalMeshGenerationResult.SkeletalMeshOp;
		
		OutResult.op = MaterialOp;
	}

	
	void CodeGenerator::GenerateMaterial_BreakSkeletalMesh(const FMaterialGenerationOptions& InOptions, FMaterialGenerationResult& OutResult, const NodeMaterialSkeletalMeshBreak* InNode)
	{
		// Early out
		if (!InNode)
		{
			OutResult = FMaterialGenerationResult();
			return;
		}
		
		Ptr<ASTOpMaterialSkeletalMeshBreak> MaterialOp = new ASTOpMaterialSkeletalMeshBreak();
		MaterialOp->SlotName = InNode->SlotName;
		
		FSkeletalMeshGenerationOptions SkeletalMeshOptions{InOptions.GenericOptions, nullptr};
		SkeletalMeshOptions.NumLODs = UINT8_MAX; // TODO GMT For now are generating all LODs. In reality we are only interested in the Material Slots.
		
		FSkeletalMeshGenerationResult SkeletalMeshGenerationResult;
		GenerateSkeletalMesh(SkeletalMeshGenerationResult, SkeletalMeshOptions, InNode->SkeletalMesh);
		MaterialOp->SkeletalMesh = SkeletalMeshGenerationResult.SkeletalMeshOp;
		
		OutResult.op = MaterialOp;
	}


	Ptr<ASTOp> CodeGenerator::GenerateMissingMaterialCode(const TCHAR* strWhere, const void* ErrorContext)
	{
		// Log an error message
		FString Msg = FString::Printf(TEXT("Required connection not found: %s"), strWhere);
		ErrorLog->Add(Msg, ELMT_ERROR, ErrorContext);

		// Constant Material Operation
		Ptr<ASTOpConstantResource> ConstantMaterial = new ASTOpConstantResource();
		ConstantMaterial->Type = EOpType::MI_CONSTANT;

		TManagedPtr<FMaterial> GenerateMaterial = MakeManaged<FMaterial>();
		GenerateMaterial->PassthroughObject = {};

		// Store the constant material
		ConstantMaterial->SetValue(GenerateMaterial);

		return ConstantMaterial;
	}

	void CodeGenerator::GenerateMaterialImage(
		const TPair<FParameterKey, FImageParameterData>& ImageData,
		FImageGenerationResult& OutImageGenerated,
		const TArray<FirstPassGenerator::FModifier>& Modifiers,
		const int32 State,
		const int32 ComponentId,
		const int32 LODIndex,
		const TArray<FString>& Tags,
		const FMeshGenerationResult& MeshResults,
		const TBitArray<>& LayoutFromExtension,
		const void* ErrorContext)
	{
		MUTABLE_CPUPROFILER_SCOPE(SurfaceTexture);

		// Any image-specific format or mipmapping needs to be applied at the end
		Ptr<NodeImageMipmap> mipmapNode;
		Ptr<NodeImageFormat> formatNode;
		Ptr<NodeImageSwizzle> swizzleNode;

		bool bFound = false;
		Ptr<NodeImage> pImageNode = ImageData.Value.ImageNode;

		while (!bFound && pImageNode)
		{
			if (pImageNode->GetType() == NodeImageMipmap::GetStaticType())
			{
				NodeImageMipmap* tm = static_cast<NodeImageMipmap*>(pImageNode.get());
				if (!mipmapNode) mipmapNode = tm;
				pImageNode = tm->Source;
			}
			else if (pImageNode->GetType() == NodeImageFormat::GetStaticType())
			{
				NodeImageFormat* tf = static_cast<NodeImageFormat*>(pImageNode.get());
				if (!formatNode) formatNode = tf;
				pImageNode = tf->Source;
			}
			else if (pImageNode->GetType() == NodeImageSwizzle::GetStaticType())
			{
				NodeImageSwizzle* ts = static_cast<NodeImageSwizzle*>(pImageNode.get());

				if (!ts->Sources.IsEmpty())
				{
					NodeImage* Source = ts->Sources[0].get();

					bool bAllSourcesAreTheSame = true;
					for (int32 SourceIndex = 1; SourceIndex < ts->Sources.Num(); ++SourceIndex)
					{
						bAllSourcesAreTheSame = bAllSourcesAreTheSame && (Source == ts->Sources[SourceIndex]);
					}

					if (!swizzleNode && bAllSourcesAreTheSame)
					{
						swizzleNode = ts;
						pImageNode = Source;
					}
					else
					{
						bFound = true;
					}
				}
				else
				{
					// break loop if swizzle has no sources.
					bFound = true;
				}
			}
			else
			{
				bFound = true;
			}
		}

		if (bFound)
		{
			const int32 LayoutIndex = ImageData.Value.LayoutIndex;

			// If the layout index has been set to negative, it means we should ignore the layout for this image.
			CompilerOptions::TextureLayoutStrategy ImageLayoutStrategy = (LayoutIndex < 0)
				? CompilerOptions::TextureLayoutStrategy::None
				: CompilerOptions::TextureLayoutStrategy::Pack
				;

			if (ImageLayoutStrategy == CompilerOptions::TextureLayoutStrategy::None)
			{
				// Generate the image
				FImageGenerationOptions ImageOptions(ComponentId, LODIndex);
				ImageOptions.GenericOptions.State = State;
				ImageOptions.ImageLayoutStrategy = ImageLayoutStrategy;
				ImageOptions.GenericOptions.ActiveTags = Tags;
				ImageOptions.RectSize = { 0, 0 };

				// TODO: To tasks
				FImageGenerationResult Result;
				GenerateImage(ImageOptions, Result, pImageNode);
				Ptr<ASTOp> imageAd = Result.op;

				// Placeholder block. Ideally this should be the actual image size
				constexpr int32 FakeLayoutSize = 256;
				FIntPoint GridSize(FakeLayoutSize, FakeLayoutSize);
				FLayoutBlockDesc LayoutBlockDesc;
				LayoutBlockDesc.BlockPixelsX = 1;
				LayoutBlockDesc.BlockPixelsY = 1;
				box< FIntVector2 > RectInCells;
				RectInCells.min = { 0,0 };
				RectInCells.size = { FakeLayoutSize ,FakeLayoutSize };

				imageAd = ApplyImageBlockModifiers(Modifiers, ImageOptions, imageAd, ImageData.Key, GridSize, LayoutBlockDesc, RectInCells, ErrorContext);

				check(imageAd);

				if (swizzleNode)
				{
					Ptr<ASTOpImageSwizzle> fop = new ASTOpImageSwizzle();
					fop->Format = swizzleNode->NewFormat;
					fop->Sources[0] = imageAd;
					fop->Sources[1] = imageAd;
					fop->Sources[2] = imageAd;
					fop->Sources[3] = imageAd;
					fop->SourceChannels[0] = swizzleNode->SourceChannels[0];
					fop->SourceChannels[1] = swizzleNode->SourceChannels[1];
					fop->SourceChannels[2] = swizzleNode->SourceChannels[2];
					fop->SourceChannels[3] = swizzleNode->SourceChannels[3];
					check(fop->Format != EImageFormat::None);
					imageAd = fop;
				}

				if (mipmapNode)
				{
					Ptr<ASTOpImageMipmap> op = new ASTOpImageMipmap();
					op->Levels = 0;
					op->Source = imageAd;
					op->BlockLevels = 0;

					op->AddressMode = mipmapNode->Settings.AddressMode;
					op->FilterType = mipmapNode->Settings.FilterType;
					imageAd = op;
				}

				if (formatNode)
				{
					Ptr<ASTOpImagePixelFormat> FormatOp = GenerateFormatExpression(formatNode.get(), imageAd, Result.CanNormalLAEncode);

					check(FormatOp->Format != EImageFormat::None);
					imageAd = FormatOp;
				}

				// Store the image parameter into the modify operation
				OutImageGenerated.op = imageAd;
			}

			else if (ImageLayoutStrategy == CompilerOptions::TextureLayoutStrategy::Pack) //-V547
			{
				if (LayoutIndex >= MeshResults.GeneratedLayouts.Num() ||
					LayoutIndex >= MeshResults.LayoutOps.Num())
				{
					ErrorLog->Add("Missing layout in object, or its parent.", ELMT_ERROR, ErrorContext);
				}
				else
				{
					TManagedPtr<const FLayout> pLayout = MeshResults.GeneratedLayouts[LayoutIndex].Layout;
					check(pLayout);

					// Image
					//-------------------------------------

					// Size of a layout block in pixels
					FIntPoint GridSize = pLayout->GetGridSize();

					// Try to guess the layout block description from the first valid block that is generated.
					FLayoutBlockDesc LayoutBlockDesc;
					if (formatNode)
					{
						LayoutBlockDesc.FinalFormat = formatNode->FormatIfAlpha;
						if (LayoutBlockDesc.FinalFormat == EImageFormat::None)
						{
							LayoutBlockDesc.FinalFormat = formatNode->Format;
						}
					}

					// Init with the block size added outside core. If BlockSize == 0 the block size will be extracted from the generated image descriptor.
					LayoutBlockDesc.BlockPixelsX = ImageData.Value.BlockSizeX;
					LayoutBlockDesc.BlockPixelsY = ImageData.Value.BlockSizeY;
					
					// Start with a blank image. It will be completed later with the blockSize, format and mips information
					Ptr<ASTOpImageBlankLayout> BlankImageOp;
					Ptr<ASTOp> imageAd;
					{
						BlankImageOp = new ASTOpImageBlankLayout();
						BlankImageOp->Layout = MeshResults.LayoutOps[LayoutIndex];
						// The rest ok the op will be completed below
						BlankImageOp->MipmapCount = 0;
						imageAd = BlankImageOp;
					}

					bool bFirstBlockProcessed = false;

					EYesNoQuery BlocksCanNormalLAEncode = EYesNoQuery::Yes;

					// Skip the block addition for this image if the layout was from a extension.
					if (!LayoutFromExtension[LayoutIndex])
					{
						for (int32 BlockIndex = 0; BlockIndex < pLayout->GetBlockCount(); ++BlockIndex)
						{
							// Generate the image
							FImageGenerationOptions ImageOptions(ComponentId, LODIndex);
							ImageOptions.GenericOptions.State = State;
							ImageOptions.ImageLayoutStrategy = ImageLayoutStrategy;
							ImageOptions.RectSize = { 0,0 };
							ImageOptions.GenericOptions.ActiveTags = Tags;
							ImageOptions.LayoutToApply = pLayout;
							ImageOptions.LayoutBlockId = pLayout->Blocks[BlockIndex].Id;
							FImageGenerationResult ImageResult;
							GenerateImage(ImageOptions, ImageResult, pImageNode);

							if (!bFirstBlockProcessed)
							{
								BlocksCanNormalLAEncode = ImageResult.CanNormalLAEncode;
								bFirstBlockProcessed = true;
							}
							else if (BlocksCanNormalLAEncode != ImageResult.CanNormalLAEncode)
							{
								BlocksCanNormalLAEncode = EYesNoQuery::Mixed;
							}

							Ptr<ASTOp> blockAd = ImageResult.op;

							if (!blockAd)
							{
								// The GenerateImage(...) above has failed, skip this block
								continue;
							}

							// Calculate the desc of the generated block.
							constexpr bool bReturnBestOption = true;
							FImageDesc BlockDesc = blockAd->GetImageDesc(bReturnBestOption, nullptr);
							
							// Block in layout grid units (cells)
							box< FIntVector2 > RectInCells;
							RectInCells.min = pLayout->Blocks[BlockIndex].Min;
							RectInCells.size = pLayout->Blocks[BlockIndex].Size;
							
							// Try to update the layout block desc if we don't know it yet.
							UpdateLayoutBlockDesc(LayoutBlockDesc, BlockDesc, RectInCells.size);

							if (LayoutBlockDesc.BlockPixelsX > 0 && LayoutBlockDesc.BlockPixelsY > 0)
							{
								BlockDesc.m_size[0] = LayoutBlockDesc.BlockPixelsX * RectInCells.size[0];
								BlockDesc.m_size[1] = LayoutBlockDesc.BlockPixelsY * RectInCells.size[1];
							}
							
							// TODO: Allow using multi-compose on images with image block modifiers
							if (CVarAllowMulticompose.GetValueOnAnyThread() && ImageResult.bCanMulticompose && !HasImageBlockModifiers(ImageData.Key, Modifiers, ImageOptions.LODIndex))
							{
								// Replace IM_COMPOSE with a IM_MULTICOMPOSE operation.
								imageAd = GenerateImageMulticompose(imageAd, MeshResults.LayoutOps[LayoutIndex], ImageOptions, pImageNode, pLayout, LayoutBlockDesc);
								break;
							}

							// Even if we force the size afterwards, we need some size hint in some cases, like image projections.
							ImageOptions.RectSize = UE::Math::TIntVector2<int32>(BlockDesc.m_size);

							blockAd = ApplyImageBlockModifiers(Modifiers, ImageOptions, blockAd, ImageData.Key, GridSize, LayoutBlockDesc, RectInCells, ErrorContext);

							// Enforce block size and optimizations
							blockAd = GenerateImageSize(blockAd, FIntVector2(BlockDesc.m_size));

							//EImageFormat baseFormat = imageAd->GetImageDesc().m_format;
							// Actually don't do it, it will be propagated from the top format operation.
							//Ptr<ASTOp> blockAd = GenerateImageFormat(blockAd, baseFormat);

							// Compose layout operation
							Ptr<ASTOpImageCompose> composeOp = new ASTOpImageCompose();
							composeOp->Layout = MeshResults.LayoutOps[LayoutIndex];
							composeOp->Base = imageAd;
							composeOp->BlockImage = blockAd;

							// Set the absolute block index.
							check(pLayout->Blocks[BlockIndex].Id != FLayoutBlock::InvalidBlockId);
							composeOp->BlockId = pLayout->Blocks[BlockIndex].Id;

							imageAd = composeOp;
						}
					}
					check(imageAd);

					FMeshGenerationStaticOptions ModifierOptions(ComponentId, LODIndex);
					ModifierOptions.State = State;
					ModifierOptions.ActiveTags = Tags;
					imageAd = ApplyImageExtendModifiers(Modifiers, ModifierOptions, MeshResults, imageAd, ImageLayoutStrategy,
						LayoutIndex, ImageData.Key, GridSize, LayoutBlockDesc,
						ErrorContext);

					// Complete the base op
					BlankImageOp->BlockSize[0] = uint16(LayoutBlockDesc.BlockPixelsX);
					BlankImageOp->BlockSize[1] = uint16(LayoutBlockDesc.BlockPixelsY);
					BlankImageOp->Format = GetUncompressedFormat(LayoutBlockDesc.FinalFormat);
					BlankImageOp->GenerateMipmaps = LayoutBlockDesc.bBlocksHaveMips;
					BlankImageOp->MipmapCount = 0;

					if (swizzleNode)
					{
						Ptr<ASTOpImageSwizzle> fop = new ASTOpImageSwizzle();
						fop->Format = swizzleNode->NewFormat;

						for (int32 ChannelIndex = 0; ChannelIndex < swizzleNode->SourceChannels.Num(); ++ChannelIndex)
						{
							fop->Sources[ChannelIndex] = imageAd;
							fop->SourceChannels[ChannelIndex] = swizzleNode->SourceChannels[ChannelIndex];
						}
						check(fop->Format != EImageFormat::None);
						imageAd = fop;
					}

					// Apply mipmap and format if necessary, skip if format is None (possibly because a block was skipped above)
					bool bNeedsMips =
						(mipmapNode && LayoutBlockDesc.FinalFormat != EImageFormat::None)
						||
						LayoutBlockDesc.bBlocksHaveMips;

					if (bNeedsMips)
					{
						Ptr<ASTOpImageMipmap> mop = new ASTOpImageMipmap();

						// At the end of the day, we want all the mipmaps. Maybe the code
						// optimiser will split the process later.
						mop->Levels = 0;
						mop->bOnlyTail = false;
						mop->Source = imageAd;

						// We have to avoid mips smaller than the image format block size, so
						// we will devide the layout block by the format block
						const FImageFormatData& PixelFormatInfo = GetImageFormatData(LayoutBlockDesc.FinalFormat);

						int32 mipsX = FMath::CeilLogTwo(LayoutBlockDesc.BlockPixelsX / PixelFormatInfo.PixelsPerBlockX);
						int32 mipsY = FMath::CeilLogTwo(LayoutBlockDesc.BlockPixelsY / PixelFormatInfo.PixelsPerBlockY);
						mop->BlockLevels = (uint8)FMath::Max(mipsX, mipsY);

						if (LayoutBlockDesc.BlockPixelsX < PixelFormatInfo.PixelsPerBlockX || LayoutBlockDesc.BlockPixelsY < PixelFormatInfo.PixelsPerBlockY)
						{
							// In this case, the mipmap will never be useful for blocks, so we indicate that
							// it should make the mips at the root of the expression.
							mop->bOnlyTail = true;
						}

						mop->AddressMode = EAddressMode::ClampToEdge;
						mop->FilterType = EMipmapFilterType::SimpleAverage;

						if (mipmapNode)
						{
							mop->AddressMode = mipmapNode->Settings.AddressMode;
							mop->FilterType = mipmapNode->Settings.FilterType;
						}

						imageAd = mop;
					}

					if (formatNode)
					{
						Ptr<ASTOpImagePixelFormat> FormatOp = GenerateFormatExpression(formatNode.get(), imageAd, BlocksCanNormalLAEncode);
						check(FormatOp->Format != EImageFormat::None);
						imageAd = FormatOp;
					}

					// Add image parameter
					OutImageGenerated.op = imageAd;
				}
			}

			else
			{
				// Unimplemented texture layout strategy
				check(false);
			}
		}
	}
}
