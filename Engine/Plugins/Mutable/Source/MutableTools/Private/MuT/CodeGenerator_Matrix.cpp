// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/CodeGenerator.h"

#include "MuT/ASTOpConstantMatrix.h"
#include "MuT/NodeMatrixConstant.h"
#include "MuT/NodeMatrixParameter.h"

namespace UE::Mutable::Private
{

	void CodeGenerator::GenerateMatrix(FMatrixGenerationResult& Result, const FGenericGenerationOptions& Options, const Ptr<const NodeMatrix>& Untyped)
	{
		if (!Untyped)
		{
			Result = FMatrixGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;
		{
			UE::TUniqueLock Lock(GeneratedMatrices.Mutex);
			FGeneratedMatrixMap::ValueType* Found = GeneratedMatrices.Map.Find(Key);
			if (Found)
			{
				Result = *Found;
				return;
			}
		}

		if (Untyped->GetType() == NodeMatrixConstant::GetStaticType())
		{
			const NodeMatrixConstant* Constant = static_cast<const NodeMatrixConstant*>(Untyped.get());
			GenerateMatrix_Constant(Result, Options, Constant);
		}
		else if (Untyped->GetType() == NodeMatrixParameter::GetStaticType())
		{
			const NodeMatrixParameter* Parameter = static_cast<const NodeMatrixParameter*>(Untyped.get());
			GenerateMatrix_Parameter(Result, Options, Parameter);
		}  

		{
			UE::TUniqueLock Lock(GeneratedMatrices.Mutex);
			GeneratedMatrices.Map.Add(Key,Result);
		}
	}

	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateMatrix_Constant(FMatrixGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeMatrixConstant>& Typed)
	{
		Ptr<ASTOpConstantMatrix> op = new ASTOpConstantMatrix();
		op->value = Typed->Value;
		result.op = op;
	}

	void CodeGenerator::GenerateMatrix_Parameter(FMatrixGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeMatrixParameter>& Typed)
	{
		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Typed.get());
			if (!Found)
			{
				FParameterDesc param;
				param.Name = Typed->Name;
				bool bParseOk = FGuid::Parse(Typed->Uid, param.UID);
				check(bParseOk);
				param.Type = EParameterType::Matrix;
				param.DefaultValue.Set<FParamMatrixType>(Typed->DefaultValue);

				op = new ASTOpParameter();
				op->Type = EOpType::MA_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Typed.get(), op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Typed->Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options, Typed->Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		result.op = op;
	}

}
