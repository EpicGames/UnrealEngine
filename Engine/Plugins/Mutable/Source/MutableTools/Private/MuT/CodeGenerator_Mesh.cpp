// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpConstantBool.h"
#include "ASTOpExternal.h"
#include "ASTOpMeshMaskClipMesh.h"
#include "ASTOpMeshMaskDiff.h"
#include "ASTOpMeshRemoveMask.h"
#include "ASTOpMeshSkeletalMeshBreak.h"
#include "ASTOpMeshTransformWithBone.h"
#include "ASTOpMeshTransformWithBoundingMesh.h"
#include "Containers/Array.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Layout.h"
#include "MuR/Mesh.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/OpMeshPrepareLayout.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/System.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpMeshApplyPose.h"
#include "MuT/ASTOpMeshApplyShape.h"
#include "MuT/ASTOpMeshBindShape.h"
#include "MuT/ASTOpMeshClipDeform.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshClipWithMesh.h"
#include "MuT/ASTOpMeshExtractLayoutBlocks.h"
#include "MuT/ASTOpMeshFormat.h"
#include "MuT/ASTOpMeshTransform.h"
#include "MuT/ASTOpMeshDifference.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeLayout.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeMeshApplyPose.h"
#include "MuT/NodeMeshClipDeform.h"
#include "MuT/NodeMeshClipMorphPlane.h"
#include "MuT/NodeMeshClipWithMesh.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeMeshExternal.h"
#include "MuT/NodeMeshFormat.h"
#include "MuT/NodeMeshFragment.h"
#include "MuT/NodeMeshMakeMorph.h"
#include "MuT/NodeMeshMorph.h"
#include "MuT/NodeMeshRemoveMesh.h"
#include "MuT/NodeMeshReshape.h"
#include "MuT/NodeMeshSwitch.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeMeshTransform.h"
#include "MuT/NodeMeshVariation.h"
#include "MuT/NodeMeshSkeletalMeshObjectBreak.h"
#include "MuT/NodeMeshTransformInMesh.h"
#include "MuT/NodeMeshTransformWithBone.h"
#include "MuT/Table.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{
	class Node;

	static TAutoConsoleVariable<bool> CVarUseASTOpMaskCliMeshForClipMeshWithMesh(
		TEXT("mutable.UseASTOpMaskToPerformClipMeshWithMesh"),
		true,
		TEXT("Defines what ASTOps does the 'GenerateMesh_ClipWithMesh' use to perform the clipping of the mesh.\n")
		TEXT("  false : Use the 'ASTOpMeshClipWithMesh' to perform the mesh clipping (not tested in depth so it may have bugs).\n")
		TEXT("  true :  Use instead an 'ASTOpMeshMaskClipMesh' op alongside an 'ASTOpMeshRemoveMask (used by some modifier nodes already).'\n"),
		ECVF_Scalability | ECVF_RenderThreadSafe
	);

	FMeshTask CodeGenerator::GenerateMesh(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const Ptr<const NodeMesh>& InUntypedNode)
    {
        if (!InUntypedNode)
        {
            return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>(FMeshGenerationResult());
        }

		const NodeMesh* Node = InUntypedNode.get();

        // Generate for each different type of node
		FMeshTask Result;
		switch (Node->GetType()->Type)
		{
		case Node::EType::MeshConstant: Result = GenerateMesh_Constant(StaticOptions, InOptions, static_cast<const NodeMeshConstant*>(Node)); break;
		case Node::EType::MeshFormat: Result = GenerateMesh_Format(StaticOptions, InOptions, static_cast<const NodeMeshFormat*>(Node)); break;
		case Node::EType::MeshMorph: Result = GenerateMesh_Morph(StaticOptions, InOptions, static_cast<const NodeMeshMorph*>(Node)); break;
		case Node::EType::MeshMakeMorph: Result = GenerateMesh_MakeMorph(StaticOptions, InOptions, static_cast<const NodeMeshMakeMorph*>(Node)); break;
		case Node::EType::MeshFragment: Result = GenerateMesh_Fragment(StaticOptions, InOptions, static_cast<const NodeMeshFragment*>(Node)); break;
		case Node::EType::MeshSwitch: Result = GenerateMesh_Switch(StaticOptions, InOptions, static_cast<const NodeMeshSwitch*>(Node)); break;
		case Node::EType::MeshTransform: Result = GenerateMesh_Transform(StaticOptions, InOptions, static_cast<const NodeMeshTransform*>(Node)); break;
		case Node::EType::MeshClipMorphPlane: Result = GenerateMesh_ClipMorphPlane(StaticOptions, InOptions, static_cast<const NodeMeshClipMorphPlane*>(Node)); break;
		case Node::EType::MeshClipWithMesh: Result = GenerateMesh_ClipWithMesh(StaticOptions, InOptions, static_cast<const NodeMeshClipWithMesh*>(Node)); break;
		case Node::EType::MeshRemoveMesh: Result = GenerateMesh_RemoveMesh(StaticOptions, InOptions, static_cast<const NodeMeshRemoveMesh*>(Node)); break;
		case Node::EType::MeshTransformWithBone: Result = GenerateMesh_TransformWithBone(StaticOptions, InOptions, static_cast<const NodeMeshTransformWithBone*>(Node)); break;
		case Node::EType::MeshTransformInMesh: Result = GenerateMesh_TransformInMesh(StaticOptions, InOptions, static_cast<const NodeMeshTransformInMesh*>(Node)); break;
		case Node::EType::MeshApplyPose: Result = GenerateMesh_ApplyPose(StaticOptions, InOptions, static_cast<const NodeMeshApplyPose*>(Node)); break;
		case Node::EType::MeshVariation: Result = GenerateMesh_Variation(StaticOptions, InOptions, static_cast<const NodeMeshVariation*>(Node)); break;
		case Node::EType::MeshTable: Result = GenerateMesh_Table(StaticOptions, InOptions, static_cast<const NodeMeshTable*>(Node)); break;
		case Node::EType::MeshReshape: Result = GenerateMesh_Reshape(StaticOptions, InOptions, static_cast<const NodeMeshReshape*>(Node)); break;
		case Node::EType::MeshClipDeform: Result = GenerateMesh_ClipDeform(StaticOptions, InOptions, static_cast<const NodeMeshClipDeform*>(Node)); break;
		case Node::EType::MeshSkeletalMeshObjectBreak: Result = GenerateMesh_SkeletalMeshBreak(StaticOptions, InOptions, static_cast<const NodeMeshSkeletalMeshObjectBreak*>(Node)); break;
		case Node::EType::MeshExternal: Result = GenerateMesh_External(StaticOptions, InOptions, static_cast<const NodeMeshExternal*>(Node)); break;
		default: check(false);
		}

		return Result;
    }


	FMeshTask CodeGenerator::GenerateMesh_Morph( const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshMorph* Node )
    {
		Ptr<ASTOp> FactorOp;

        // Factor
        if (Node->Factor )
        {
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, StaticOptions, Node->Factor.get());
			FactorOp = ParamResult.op;
        }
        else
        {
            // This argument is required
			FactorOp = GenerateMissingScalarCode(TEXT("Morph factor"), 0.5f, Node->GetMessageContext());
        }

		TArray<UE::Tasks::FTask, TInlineAllocator<4>> Requisites;

        // Base
		FMeshTask BaseTask;
        if (Node->Base )
        {
			BaseTask = GenerateMesh(StaticOptions, InOptions, Node->Base );
			Requisites.Add(BaseTask);
        }
        else
        {
            // This argument is required
            ErrorLog->Add( "Mesh morph base node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
        }        

		if (Node->Name.IsNone())
		{
			return BaseTask;
		}

		return UE::Tasks::Launch(TEXT("MutableMorph"),
			[BaseTask, Node, FactorOp]() mutable
			{
				FMeshGenerationResult BaseResult = BaseTask.GetResult();

				Ptr<ASTOpMeshMorph> OpMorph = new ASTOpMeshMorph();
				OpMorph->Name = Node->Name;
				OpMorph->Factor = FactorOp;
				OpMorph->Base = BaseTask.GetResult().MeshOp;

				FMeshGenerationResult Result;
				Result.MeshOp = OpMorph;
				Result.BaseMeshOp = BaseResult.BaseMeshOp;
				Result.GeneratedLayouts = BaseResult.GeneratedLayouts;

				return Result;
			},
			UE::Tasks::Prerequisites(BaseTask)
		);
	}


	FMeshTask CodeGenerator::GenerateMesh_MakeMorph(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshMakeMorph* Node )
    {
		if (!Node->Base)
		{
			// This argument is required
			ErrorLog->Add("Mesh make morph base node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		if (!Node->Target)
		{
			// This argument is required
			ErrorLog->Add("Mesh make morph target node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

        // Base
		FMeshTask BaseTask;
        {
			FMeshOptionsTask BaseOptionsTask = UE::Tasks::Launch(TEXT("MutableMorphBaseOptions"),
				[InOptions]() mutable
				{
					FMeshGenerationDynamicOptions Result = InOptions.GetResult();
					Result.bLayouts = false;
					return Result;
				},
				InOptions
			);

			FMeshGenerationStaticOptions BaseStaticOptions = StaticOptions;
			BaseTask = GenerateMesh(BaseStaticOptions, BaseOptionsTask, Node->Base );
        }

        // Target
		FMeshTask TargetTask;
		{
			FMeshGenerationStaticOptions TargetStaticOptions = StaticOptions;
			TargetStaticOptions.ActiveTags.Empty();

			FMeshOptionsTask TargetOptionsTask = UE::Tasks::Launch(TEXT("MutableMorphTargetOptions"),
				[InOptions]() mutable
				{
					FMeshGenerationDynamicOptions Result = InOptions.GetResult();
					Result.bLayouts = false;
					Result.OverrideLayouts.Empty();
					return Result;
				},
				InOptions
			);

			TargetTask = GenerateMesh(TargetStaticOptions, TargetOptionsTask, Node->Target );
        }

		return UE::Tasks::Launch(TEXT("MutableMakeMorph"),
			[BaseTask, TargetTask, Node]() mutable
			{
				Ptr<ASTOpMeshDifference> op = new ASTOpMeshDifference();

				// \todo Texcoords are broken?
				op->bIgnoreTextureCoords = true;

				// UE only has position and normal morph data, optimize for this case if indicated. 
				if (Node->bOnlyPositionAndNormal)
				{
					op->Channels = { {static_cast<uint8>(EMeshBufferSemantic::Position), 0}, {static_cast<uint8>(EMeshBufferSemantic::Normal), 0} };
				}

				FMeshGenerationResult BaseResult = BaseTask.GetResult();
				op->Base = BaseResult.MeshOp;

				FMeshGenerationResult TargetResult = TargetTask.GetResult();
				op->Target = TargetResult.MeshOp;

				FMeshGenerationResult Result;
				Result.MeshOp = op;
				Result.BaseMeshOp = BaseResult.BaseMeshOp;
				Result.GeneratedLayouts = BaseResult.GeneratedLayouts;
				return Result;
			},
			UE::Tasks::Prerequisites(BaseTask, TargetTask)
			);
	}


	FMeshTask CodeGenerator::GenerateMesh_Fragment(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshFragment* Node )
    {
		if (!Node->SourceMesh)
		{
			// This argument is required
			ErrorLog->Add("Mesh fragment source is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		TManagedPtr<const FLayout> Layout;
		if (Node->Layout)
		{
			// Generate the layout with blocks to extract
			Layout = GenerateLayout(Node->Layout, 0);
		}

		FMeshTask SourceTask;
		{
			FMeshOptionsTask SourceOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshFragmentOptions"),
				[InOptions, Layout, Node]() mutable
				{
					FMeshGenerationDynamicOptions Result = InOptions.GetResult();
					Result.bLayouts = true;
					Result.bEnsureAllVerticesHaveLayoutBlock = false;
					if (Layout)
					{
						Result.OverrideLayouts.Empty();
						Result.OverrideLayouts.Add({ Layout, Node->Layout });
					}
					return Result;
				},
				InOptions
			);

			SourceTask = GenerateMesh(StaticOptions, SourceOptionsTask, Node->SourceMesh);
		}

		return UE::Tasks::Launch(TEXT("MutableMeshFragment"),
			[SourceTask, Node]() mutable
			{
				FMeshGenerationResult SourceResult = SourceTask.GetResult();

				Ptr<ASTOpMeshExtractLayoutBlocks> op = new ASTOpMeshExtractLayoutBlocks();
				op->LayoutIndex = (uint16)Node->LayoutIndex;
				op->Source = SourceResult.MeshOp;

				FMeshGenerationResult Result;
				Result.MeshOp = op;
				Result.BaseMeshOp = SourceResult.BaseMeshOp;
				Result.GeneratedLayouts = SourceResult.GeneratedLayouts;
				return Result;
			},
			UE::Tasks::Prerequisites(SourceTask)
		);
	}


	FMeshTask CodeGenerator::GenerateMesh_Switch(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshSwitch* Node )
    {
        if (Node->Options.Num() == 0)
        {
            // No options in the switch!
            // TODO
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		Ptr<ASTOp> Variable;
        if (Node->Parameter )
        {
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, StaticOptions, Node->Parameter.get());
			Variable = ParamResult.op;
		}
        else
        {
            // This argument is required
			Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Node->GetMessageContext());
        }

		Ptr<ASTOpSwitch> SwitchOp = new ASTOpSwitch();
		SwitchOp->Type = EOpType::ME_SWITCH;
		SwitchOp->Variable = Variable;

        // Options
		bool bFirstValidConnectionFound = false;
		FMeshOptionsTask CurrentTargetOptions = InOptions;
		TArray<FMeshTask> OptionTasks;
		TArray<int16> OptionIndices;
		OptionTasks.Reserve(Node->Options.Num());
		OptionIndices.Reserve(Node->Options.Num());
		for ( int32 OptionIndex=0; OptionIndex< Node->Options.Num(); ++OptionIndex)
        {
            if (Node->Options[OptionIndex] )
            {								
				FMeshTask BranchTask = GenerateMesh(StaticOptions, CurrentTargetOptions, Node->Options[OptionIndex] );
				OptionTasks.Emplace(BranchTask);
				OptionIndices.Emplace(int16(OptionIndex));

				// Take the layouts from the first non-null connection.
				if (!bFirstValidConnectionFound)
				{
					bFirstValidConnectionFound = true;
				}

				// Separated in case the logic for bFirstValidConnectionFound gets more complex.
				if (bFirstValidConnectionFound)
				{
					CurrentTargetOptions = UE::Tasks::Launch(TEXT("MutableMeshSwitchOptions"),
						[InOptions, BranchTask]() mutable
						{
							FMeshGenerationDynamicOptions Result = InOptions.GetResult();
							Result.OverrideLayouts = BranchTask.GetResult().GeneratedLayouts;
							return Result;
						},
						UE::Tasks::Prerequisites(InOptions, BranchTask)
					);
				}
			}
        }

		return UE::Tasks::Launch(TEXT("MutableMeshSwitch"),
			[SwitchOp, OptionTasks, OptionIndices]() mutable
			{
				FMeshGenerationResult Result;

				bool bFirstValidConnectionFound = false;
				for (int32 ValidOptionIndex = 0; ValidOptionIndex < OptionIndices.Num(); ++ValidOptionIndex)
				{
					FMeshGenerationResult BranchResult = OptionTasks[ValidOptionIndex].GetResult();
					if (!bFirstValidConnectionFound)
					{
						bFirstValidConnectionFound = true;
						Result = BranchResult;
					}

					Ptr<ASTOp> Branch = BranchResult.MeshOp;
					SwitchOp->Cases.Emplace(OptionIndices[ValidOptionIndex], SwitchOp, Branch);
				}

				Result.MeshOp = SwitchOp;
				return Result;
			},
			OptionTasks
		);
    }


	FMeshTask CodeGenerator::GenerateMesh_Table(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshTable* TableNode)
	{
		Ptr<NodeMeshSwitch> SwitchNode;
		GenerateTableSwitchNode<NodeMeshTable, ETableColumnType::Mesh, NodeMesh, NodeMeshSwitch>(*TableNode, SwitchNode,
			[this] (const NodeMeshTable& Node, int32 ColIndex, int32 Row, FErrorLog* pErrorLog) -> Ptr<NodeMesh>
			{
				const FTableValue& Cell = Node.Table->GetPrivate()->Rows[Row].Values[ColIndex];
				TManagedPtr<UE::Mutable::Private::FMesh> MeshValue = Cell.Mesh;
				
				if (!MeshValue)
				{
					return nullptr;
				}

				Ptr<NodeMeshConstant> CellNode = new NodeMeshConstant();
				CellNode->Value = MeshValue;

				// TODO Take into account layout strategy
				CellNode->Layouts = Node.Layouts;
				CellNode->RealTimeMorphNames = Node.RealTimeMorphNames;

				CellNode->SetMessageContext(Cell.ErrorContext);
				CellNode->SourceDataDescriptor = Node.SourceDataDescriptor;

				// Combine the SourceId of the node with the RowId to generate one shared between all resources from this row.
				// Hash collisions are allowed, since it is used to group resources, not to differentiate them.
				const uint32 RowId = Node.Table->GetPrivate()->Rows[Row].Id;
				CellNode->SourceDataDescriptor.SourceId = HashCombine(Node.SourceDataDescriptor.SourceId, RowId);

				return CellNode;
			});

		return GenerateMesh(StaticOptions, InOptions, SwitchNode);
	}


	FMeshTask CodeGenerator::GenerateMesh_Variation(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshVariation* Node )
    {
		bool bFirstValidConnectionFound = false;
		FMeshOptionsTask VariationOptionsTask = InOptions;

		FMeshTask DefaultTask = UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();

        // Default case
        if (Node->DefaultMesh )
        {
			DefaultTask = GenerateMesh(StaticOptions, InOptions, Node->DefaultMesh );
			bFirstValidConnectionFound = true;

			VariationOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshVariationDefaultOptions"),
				[InOptions, DefaultTask]() mutable
				{
					FMeshGenerationDynamicOptions Result = InOptions.GetResult();
					Result.OverrideLayouts = DefaultTask.GetResult().GeneratedLayouts;
					return Result;
				},
				UE::Tasks::Prerequisites(InOptions, DefaultTask)
			);
        }

        // Process variations in reverse order, since conditionals are built bottom-up.
		TArray<FMeshTask> ReverseVariations;
		TArray<int32> ReverseVariationsIndices;
		ReverseVariations.Reserve(Node->Variations.Num());
		ReverseVariationsIndices.Reserve(Node->Variations.Num());

        for ( int32 VariationIndex = Node->Variations.Num()-1; VariationIndex >= 0; --VariationIndex)
        {
            int32 tagIndex = -1;
            const FString& tag = Node->Variations[VariationIndex].Tag;
            for ( int32 i = 0; i < FirstPass.Tags.Num(); ++i )
            {
                if ( FirstPass.Tags[i].Tag==tag)
                {
                    tagIndex = i;
                }
            }

            if ( tagIndex < 0 )
            {
                ErrorLog->Add( 
					FString::Printf(TEXT("Unknown tag found in mesh variation [%s]."), *tag),
					ELMT_WARNING,
					Node->GetMessageContext(),
					ELMSB_UNKNOWN_TAG
				);
                continue;
            }

            Ptr<ASTOp> variationMeshOp;
            if ( Node->Variations[VariationIndex].Mesh )
            {         
				FMeshTask BranchTask = GenerateMesh(StaticOptions, VariationOptionsTask, Node->Variations[VariationIndex].Mesh );

				ReverseVariations.Add(BranchTask);

                if ( !bFirstValidConnectionFound)
                {
					bFirstValidConnectionFound = true;

					VariationOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshVariationOptions"),
						[InOptions, BranchTask]() mutable
						{
							FMeshGenerationDynamicOptions Result = InOptions.GetResult();
							Result.OverrideLayouts = BranchTask.GetResult().GeneratedLayouts;
							return Result;
						},
						UE::Tasks::Prerequisites(VariationOptionsTask, BranchTask)
					);
				}
            }
			else
			{

				ReverseVariations.Add(UE::Tasks::MakeCompletedTask<FMeshGenerationResult>());
			}
		
			ReverseVariationsIndices.Add(tagIndex);
		}
		
		TArray<UE::Tasks::FTask> Requisites;
		Requisites.Reserve(ReverseVariations.Num()+2);
		Requisites.Add(DefaultTask);
		Requisites.Append(ReverseVariations);

		return UE::Tasks::Launch(TEXT("MutableMeshVariation"),
			[this, Node, DefaultTask, ReverseVariations, ReverseVariationsIndices]() mutable
			{
				FMeshGenerationResult Result;
				Ptr<ASTOp> CurrentMeshOp;

				bool bFirstValidConnectionFound = false;

				// Default case
				if (Node->DefaultMesh)
				{
					Result = DefaultTask.GetResult();
					CurrentMeshOp = Result.MeshOp;
					bFirstValidConnectionFound = true;
				}

				for (int32 ReverseVariationIndex = 0; ReverseVariationIndex<ReverseVariations.Num(); ++ReverseVariationIndex)
				{
					int32 tagIndex = ReverseVariationsIndices[ReverseVariationIndex];

					FMeshGenerationResult VariationResult = ReverseVariations[ReverseVariationIndex].GetResult();

					Ptr<ASTOpConditional> Conditional = new ASTOpConditional;
					Conditional->type = EOpType::ME_CONDITIONAL;
					Conditional->no = CurrentMeshOp;
					Conditional->yes = VariationResult.MeshOp;
					Conditional->condition = FirstPass.Tags[tagIndex].GenericCondition;

					CurrentMeshOp = Conditional;
				}

				Result.MeshOp = CurrentMeshOp;
				return Result;
			},
			Requisites
		);

	}


	void GenerateLayoutsForMesh(const TManagedPtr<FMesh>& Mesh, const TArray<Ptr<NodeLayout>>& Layouts)
	{
		if (!Mesh)
		{
			return;
		}
		
		for (const Ptr<NodeLayout>& Layout : Layouts)
		{
			if (!Layout)
			{
				continue;
			}

			// Legacy behavior
			if ((Layout->AutoBlockStrategy == EAutoBlocksStrategy::Ignore || Layout->Strategy == EPackStrategy::Overlay) &&
				Layout->Blocks.IsEmpty())
			{
				Layout->Blocks.SetNum(1);
				Layout->Blocks[0].Min = { 0,0 };
				Layout->Blocks[0].Size = Layout->Size;
				Layout->Blocks[0].Priority = 0;
				Layout->Blocks[0].bReduceBothAxes = false;
				Layout->Blocks[0].bReduceByTwo = false;
				continue;
			}

			UntypedMeshBufferIteratorConst TexIt(Mesh->GetVertexBuffers(), EMeshBufferSemantic::TexCoords, Layout->TexCoordsIndex);
			if (TexIt.ptr() == nullptr)
			{
				continue;
			}

			if (Layout->AutoBlockStrategy == EAutoBlocksStrategy::Rectangles)
			{
				Layout->GenerateLayoutBlocks(Mesh, Layout->TexCoordsIndex);
			}
			else if (Layout->AutoBlockStrategy == EAutoBlocksStrategy::UVIslands)
			{
				Layout->GenerateLayoutBlocksFromUVIslands(Mesh, Layout->TexCoordsIndex);
			}
		}
	}

	Ptr<ASTOp> CodeGenerator::GenerateLayoutOpsAndResult(
		const FMeshGenerationDynamicOptions& Options, 
		Ptr<ASTOp> LastMeshOp, 
		const TArray<Ptr<NodeLayout>>& OriginalLayouts,
		uint32 MeshPrefix,
		FMeshGenerationResult& OutResult)
	{
		if (!Options.bLayouts)
		{
			return LastMeshOp;
		}

		// Always absolute to ease mesh reusal.
		bool bUseAbsoluteBlockIds = true;

		bool bIsOverridingLayouts = !Options.OverrideLayouts.IsEmpty();
		if (!bIsOverridingLayouts)
		{
			OutResult.GeneratedLayouts.Reserve(OriginalLayouts.Num());
			for (int32 LayoutIndex = 0; LayoutIndex < OriginalLayouts.Num(); ++LayoutIndex)
			{
				FGeneratedLayout GeneratedData;
				Ptr<const NodeLayout> LayoutNode = OriginalLayouts[LayoutIndex];
				if (LayoutNode)
				{
					GeneratedData.Source = LayoutNode;
					GeneratedData.Layout = this->GenerateLayout(LayoutNode, MeshPrefix);
				}
				OutResult.GeneratedLayouts.Add(GeneratedData);
			}
		}
		else
		{
			// In this case we need the layout block ids to use the ids in the parent layout, and not be prefixed with
			// the current mesh id prefix. For this reason we need them to be absolute.
			bUseAbsoluteBlockIds = true;

			// We need to apply the transform of the layouts used to override
			OutResult.GeneratedLayouts.Reserve(Options.OverrideLayouts.Num());
			for (int32 LayoutIndex = 0; LayoutIndex < Options.OverrideLayouts.Num(); ++LayoutIndex)
			{
				const FGeneratedLayout& OverrideData = Options.OverrideLayouts[LayoutIndex];
				OutResult.GeneratedLayouts.Add(OverrideData);
			}
		}

		// Generate the chain of ops preparing the mesh for the layouts
		for (int32 LayoutIndex = 0; LayoutIndex < OutResult.GeneratedLayouts.Num(); ++LayoutIndex)
		{
			TManagedPtr<const UE::Mutable::Private::FLayout> Layout = OutResult.GeneratedLayouts[LayoutIndex].Layout;

			Ptr<ASTOpConstantResource> LayoutOp = new ASTOpConstantResource;
			LayoutOp->Type = EOpType::LA_CONSTANT;
			LayoutOp->SetValue(Layout);

			Ptr<ASTOpMeshPrepareLayout> PrepareOp = new ASTOpMeshPrepareLayout;
			PrepareOp->Mesh = LastMeshOp;
			PrepareOp->Layout = LayoutOp;
			PrepareOp->LayoutChannel = LayoutIndex;
			PrepareOp->bUseAbsoluteBlockIds = bUseAbsoluteBlockIds;
			PrepareOp->bNormalizeUVs = Options.bNormalizeUVs;
			PrepareOp->bClampUVIslands = Options.bClampUVIslands;
			PrepareOp->bEnsureAllVerticesHaveLayoutBlock = Options.bEnsureAllVerticesHaveLayoutBlock;

			LastMeshOp = PrepareOp;
		}

		return LastMeshOp;
	}


	FMeshTask CodeGenerator::GenerateMesh_Constant(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshConstant* Node )
    {
		MUTABLE_CPUPROFILER_SCOPE(GenerateMesh_Constant);

		TManagedPtr<FMesh> Mesh = Node->Value;
		
		if (!Mesh)
		{
			Ptr<ASTOpConstantResource> ConstantOp = new ASTOpConstantResource();
			ConstantOp->Type = EOpType::ME_CONSTANT;
			ConstantOp->SourceDataDescriptor = Node->SourceDataDescriptor;

			FMeshGenerationResult Result;
			Result.BaseMeshOp = ConstantOp;
			Result.MeshOp = ConstantOp;

			// This data is required
			TManagedPtr<FMesh> EmptyMesh = MakeManaged<FMesh>();
			ConstantOp->SetValue(EmptyMesh);
			EmptyMesh->MeshIDPrefix = UniqueMeshIds.EnsureUnique(ConstantOp->GetValueHash());

			// Log an error message
			ErrorLog->Add("Constant mesh not set.", ELMT_WARNING, Node->GetMessageContext());

			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>(Result);
		}

		TArray<UE::Tasks::FTask, TInlineAllocator<4>> Requisites;
		Requisites.Emplace(InOptions);

		TSharedPtr<TManagedPtr<FMesh>> ResolveMesh = MakeShared<TManagedPtr<FMesh>>();

		// Compile-time reference? Add task to resolve as requisite.
		if (Mesh->IsReference())
		{
			PASSTHROUGH_ID MeshID = Mesh->PassthroughObject.GetId();

			bool bRunImmediatlyIfPossible = IsInGameThread();
			FString Morph = Mesh->GetReferencedMorph();
			UE::Tasks::FTask ReferenceCompletion = CompilerOptions->OptimisationOptions.ReferencedMeshResourceProvider(MeshID, Morph, ResolveMesh, bRunImmediatlyIfPossible);
			Requisites.Emplace(ReferenceCompletion);
		}
		else
		{
			*ResolveMesh = Mesh;
		}

		// This task does the necessary processing of the mesh when it is available
		Ptr<const NodeMeshConstant> NodeCopy = Node;
		FMeshTask MeshProcessingTask = GenerateMeshConstantPipe.Launch( TEXT("MutableGenerateMeshConstant"),
			[ReferenceMesh=Mesh,ResolveMesh,StaticOptions, InOptions, NodeCopy,
			// This is read-only
			//CompilerOptions=this->CompilerOptions,
			this
			// This is read-write so it is protected with a mutex.
			//&GenerateMeshConstantState = this->GenerateMeshConstantState
			]() mutable
			{
				TManagedPtr<FMesh> Mesh = *ResolveMesh;

				if (!Mesh)
				{
					return FMeshGenerationResult();
				}

				// Separate metadata from the mesh
				TArray<FName> GameplayTags;
				TArray<PASSTHROUGH_ID> AssetUserDataIds;
				TArray<TPair<FName, PASSTHROUGH_ID>> AnimationSlots;
				PASSTHROUGH_ID SkeletonID = PASSTHROUGH_ID_INVALID;
				const int8 BonePosePriority = NodeCopy->BonePosePriority;
				TArray<FMeshSocket> Sockets;
				const int8 SocketPriority = NodeCopy->SocketPriority;
				PASSTHROUGH_ID PhysicsAssetId = PASSTHROUGH_ID_INVALID;
				TArray<PASSTHROUGH_ID> AdditionalPhysicsAssetId;

				const bool bMeshHasMetadata = 
					Mesh->SkeletonObjects.Num() > 0 ||
					Mesh->GameplayTags.Num() ||
					Mesh->AssetUserData.Num() ||
					Mesh->AnimationSlots.Num() ||
					BonePosePriority != 0 ||
					Mesh->Sockets.Num() != 0 ||
					(Mesh->Sockets.Num() && SocketPriority != 0) ||
					Mesh->PhysicsAssets.Num() ||
					Mesh->AdditionalPhysicsAssets.Num();

				if (bMeshHasMetadata)
				{
					GameplayTags = Mesh->GameplayTags;

					for (const TPassthroughObjectPtr<UAssetUserData>& AssetUserData : Mesh->AssetUserData)
					{
						AssetUserDataIds.AddUnique(AssetUserData.GetId());
					}
					
					for (const TPair<FName, TPassthroughObjectPtr<UClass>>& AnimationSlot : Mesh->AnimationSlots)
					{
						AnimationSlots.Add({ AnimationSlot.Key, AnimationSlot.Value.GetId() });
					}
					
					SkeletonID = Mesh->SkeletonObjects.IsEmpty() ? PASSTHROUGH_ID_INVALID : Mesh->SkeletonObjects[0].GetId();

					if (Mesh->PhysicsAssets.Num())
					{
						check(Mesh->PhysicsAssets.Num() == 1);
						PhysicsAssetId =  Mesh->PhysicsAssets[0].GetId();
					}

					for (const TPassthroughObjectPtr<UPhysicsAsset>& AdditionalPhysicsAsset : Mesh->AdditionalPhysicsAssets)
					{
						AdditionalPhysicsAssetId.Add(AdditionalPhysicsAsset.GetId());
					}

					TManagedPtr<FMesh> MetadatalessMesh = CloneOrTakeOver(MoveTemp(Mesh));

					MetadatalessMesh->GameplayTags.SetNum(0, EAllowShrinking::No);
					MetadatalessMesh->AssetUserData.SetNum(0, EAllowShrinking::No);
					MetadatalessMesh->AnimationSlots.SetNum(0, EAllowShrinking::No);
					MetadatalessMesh->SkeletonObjects.SetNum(0, EAllowShrinking::No);
					MetadatalessMesh->PhysicsAssets.SetNum(0, EAllowShrinking::No);
					MetadatalessMesh->AdditionalPhysicsAssets.SetNum(0, EAllowShrinking::No);

					Exchange(Sockets, MetadatalessMesh->Sockets);

					Mesh = MetadatalessMesh;
				}

				FMeshGenerationDynamicOptions Options = InOptions.GetResult();
				bool bIsOverridingLayouts = !Options.OverrideLayouts.IsEmpty();

				// Find out if we can (or have to) reuse a mesh that we have already generated.
				FGenerateMeshConstantState::FGeneratedConstantMesh DuplicateOf;
				uint32 ThisMeshHash = HashCombineFast(GetTypeHash(Mesh->GetVertexCount()), GetTypeHash(Mesh->GetIndexCount()));

				TArray<FGenerateMeshConstantState::FGeneratedConstantMesh>& CachedCandidates = GenerateMeshConstantState.GeneratedConstantMeshes.FindOrAdd(ThisMeshHash, {});
				for (const FGenerateMeshConstantState::FGeneratedConstantMesh& Candidate : CachedCandidates)
				{
					if (Candidate.Mesh->IsSimilar(*Mesh))
					{
						DuplicateOf = Candidate;
						break;
					}
				}

				Ptr<ASTOp> LastMeshOp;
				uint32 MeshIDPrefix = 0;

				if (!DuplicateOf.Mesh)
				{
					Ptr<ASTOpConstantResource> ConstantOp = new ASTOpConstantResource();
					ConstantOp->Type = EOpType::ME_CONSTANT;
					ConstantOp->SourceDataDescriptor = NodeCopy->SourceDataDescriptor;

					LastMeshOp = ConstantOp;

					// We need to clone the mesh in the node because we will modify it.
					TManagedPtr<FMesh> ClonedMesh = Mesh->Clone();
					ClonedMesh->EnsureSurfaceData();

					ConstantOp->SetValue(ClonedMesh);

					// Add the unique vertex ID prefix in all cases, since it is free memory-wise
					MeshIDPrefix = uint32(ConstantOp->GetValueHash());
					MeshIDPrefix = UniqueMeshIds.EnsureUnique(MeshIDPrefix);
					ClonedMesh->MeshIDPrefix = MeshIDPrefix;

					// Add the constant data
					FGenerateMeshConstantState::FGeneratedConstantMesh MeshEntry;
					MeshEntry.Mesh = ClonedMesh;
					MeshEntry.LastMeshOp = LastMeshOp;
					CachedCandidates.Add(MeshEntry);
				}

				if (DuplicateOf.Mesh)
				{
					LastMeshOp = DuplicateOf.LastMeshOp;
					MeshIDPrefix = DuplicateOf.Mesh->MeshIDPrefix;
				}

				FMeshGenerationResult Result;

				if (Options.bLayouts)
				{
					if (Options.OverrideLayouts.IsEmpty())
					{
						GenerateLayoutsForMesh(Mesh, NodeCopy->Layouts);
					}

					LastMeshOp = GenerateLayoutOpsAndResult(Options, LastMeshOp, NodeCopy->Layouts, MeshIDPrefix, Result);
				}

				Result.BaseMeshOp = LastMeshOp;

	
				TArray<FString> RealTimeMorphNames;
				if (NodeCopy->RealTimeMorphNames.Num())
				{
					RealTimeMorphNames.Reserve(NodeCopy->RealTimeMorphNames.Num());
					Algo::Transform(NodeCopy->RealTimeMorphNames, RealTimeMorphNames, [](const FName& Name) {return Name.ToString(); });
				}

				// Add the metadata operation
				if (bMeshHasMetadata || RealTimeMorphNames.Num() > 0)
				{
					Ptr<ASTOpMeshAddMetadata> AddMetadataOp = new ASTOpMeshAddMetadata;
					AddMetadataOp->Source = LastMeshOp;
					
					AddMetadataOp->GameplayTags = MoveTemp(GameplayTags);
					
					AddMetadataOp->AssetUserDataIds = MoveTemp(AssetUserDataIds);
					
					AddMetadataOp->AnimationSlots = MoveTemp(AnimationSlots);
					
					AddMetadataOp->SkeletonId = SkeletonID;

					AddMetadataOp->BonePosePriority = BonePosePriority;

					AddMetadataOp->Sockets = MoveTemp(Sockets);
					AddMetadataOp->SocketPriority = SocketPriority;

					AddMetadataOp->PhysicsAssetId = PhysicsAssetId;
					AddMetadataOp->AdditionalPhysicsAssetIds = MoveTemp(AdditionalPhysicsAssetId);

					AddMetadataOp->RealTimeMorphNames = MoveTemp(RealTimeMorphNames);

					LastMeshOp = AddMetadataOp;
				}

				Result.MeshOp = LastMeshOp;

				return Result;
			},
			Requisites);


		// Apply the modifier for the pre-normal operations stage.
		TArray<FirstPassGenerator::FModifier> Modifiers;
		constexpr bool bModifiersForBeforeOperations = true;
		this->GetModifiersFor(StaticOptions.ComponentId, StaticOptions.ActiveTags, bModifiersForBeforeOperations, Modifiers);

		// This task does the necessary processing of the mesh when it is available
		FMeshTask ApplyModifiersTask = ApplyMeshModifiers(Modifiers, StaticOptions, InOptions, MeshProcessingTask, FGuid(), Node->GetMessageContext(), Node);

		return ApplyModifiersTask;
    }


	FMeshTask CodeGenerator::GenerateMesh_Format(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshFormat* Node )
    {
		if (!Node->Source)
		{
			// Put something there
			return GenerateMesh(StaticOptions, InOptions, new NodeMeshConstant());
		}

 		FMeshTask BaseTask = GenerateMesh(StaticOptions, InOptions, Node->Source);

		return UE::Tasks::Launch(TEXT("MutableMeshFormat"),
			[Node, BaseTask, CompilerOptions=CompilerOptions]() mutable
			{
				Ptr<ASTOpMeshFormat> op = new ASTOpMeshFormat();

				FMeshGenerationResult baseResult = BaseTask.GetResult();
				op->Source = baseResult.MeshOp;
				op->Flags = 0;

				TManagedPtr<FMesh> FormatMesh = MakeManaged<FMesh>();

				if (Node->VertexBuffers.GetBufferCount())
				{
					op->Flags |= FOperation::MeshFormatArgs::Vertex;
					FormatMesh->VertexBuffers = Node->VertexBuffers;
				}

				if (Node->IndexBuffers.GetBufferCount())
				{
					op->Flags |= FOperation::MeshFormatArgs::Index;
					FormatMesh->IndexBuffers = Node->IndexBuffers;
				}

				if (Node->bOptimizeBuffers)
				{
					op->Flags |= FOperation::MeshFormatArgs::OptimizeBuffers;
				}

				Ptr<ASTOpConstantResource> cop = new ASTOpConstantResource();
				cop->Type = EOpType::ME_CONSTANT;
				cop->SetValue(FormatMesh);
				if (baseResult.BaseMeshOp)
				{
					cop->SourceDataDescriptor = baseResult.BaseMeshOp->GetSourceDataDescriptor();
				}
				op->Format = cop;

				FMeshGenerationResult Result;
				Result.MeshOp = op;
				Result.BaseMeshOp = baseResult.BaseMeshOp;
				Result.GeneratedLayouts = baseResult.GeneratedLayouts;

				return Result;
			},
			BaseTask
		);
    }


	FMeshTask CodeGenerator::GenerateMesh_Transform(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshTransform* Node )
    {
		if (!Node->Source)
		{
			// This argument is required
			ErrorLog->Add("Mesh transform base node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		FMeshTask SourceTask = GenerateMesh(StaticOptions, InOptions, Node->Source);

		Ptr<const NodeMeshTransform> NodeCopy = Node;
		return UE::Tasks::Launch(TEXT("MutableMeshTransform"),
			[NodeCopy, SourceTask, CompilerOptions = CompilerOptions]() mutable
			{
				FMeshGenerationResult SourceResult = SourceTask.GetResult();

				Ptr<ASTOpMeshTransform> TransformOp = new ASTOpMeshTransform();
				TransformOp->source = SourceResult.MeshOp;
				TransformOp->matrix = NodeCopy->Transform;

				FMeshGenerationResult Result = SourceResult;
				Result.MeshOp = TransformOp;
				return Result;
			},
			SourceTask
		);
     }


	FMeshTask CodeGenerator::GenerateMesh_ClipMorphPlane(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshClipMorphPlane* ClipNode )
    {
		if (!ClipNode->Source)
		{
			// This argument is required
			ErrorLog->Add("Mesh transform base node is not set.", ELMT_ERROR, ClipNode->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		FMeshTask SourceTask = GenerateMesh(StaticOptions, InOptions, ClipNode->Source);

		return UE::Tasks::Launch(TEXT("MutableMeshClipMorphPlane"),
			[ClipNode, SourceTask, CompilerOptions = CompilerOptions]() mutable
			{
				FMeshGenerationResult SourceResult = SourceTask.GetResult();

				Ptr<ASTOpMeshClipMorphPlane> op = new ASTOpMeshClipMorphPlane();

				op->FaceCullStrategy = ClipNode->Parameters.FaceCullStrategy;
				op->Source = SourceResult.MeshOp;

				// Morph to an ellipse
				{
					op->MorphShape.type = uint8(FShape::Type::Ellipse);
					op->MorphShape.position = ClipNode->Parameters.Origin;
					op->MorphShape.up = ClipNode->Parameters.Normal;
					op->MorphShape.size = FVector3f(ClipNode->Parameters.Radius1, ClipNode->Parameters.Radius2, ClipNode->Parameters.Rotation); // TODO: Move rotation to ellipse rotation reference base instead of passing it directly

					// Generate a "side" vector.
					// \todo: make generic and move to the vector class
					{
						// Generate vector perpendicular to normal for ellipse rotation reference base
						FVector3f aux_base(0.f, 1.f, 0.f);

						if (fabs(FVector3f::DotProduct(ClipNode->Parameters.Normal, aux_base)) > 0.95f)
						{
							aux_base = FVector3f(0.f, 0.f, 1.f);
						}

						op->MorphShape.side = FVector3f::CrossProduct(ClipNode->Parameters.Normal, aux_base);
					}
				}

				// Selection by shape
				op->VertexSelectionType = ClipNode->Parameters.VertexSelectionType;
				if (op->VertexSelectionType == EClipVertexSelectionType::Shape)
				{
					op->SelectionShape.type = uint8(FShape::Type::AABox);
					op->SelectionShape.position = ClipNode->Parameters.SelectionBoxOrigin;
					op->SelectionShape.size = ClipNode->Parameters.SelectionBoxRadius;
				}
				else if (op->VertexSelectionType == EClipVertexSelectionType::BoneHierarchy)
				{
					// Selection by bone hierarchy?
					op->VertexSelectionBone = ClipNode->Parameters.VertexSelectionBone;
					op->VertexSelectionBoneMaxRadius = ClipNode->Parameters.MaxEffectRadius;
				}

				// FParameters
				op->Dist = ClipNode->Parameters.DistanceToPlane;
				op->Factor = ClipNode->Parameters.LinearityFactor;

				FMeshGenerationResult Result = SourceResult;
				Result.MeshOp = op;

				return Result;
			},
			SourceTask
		);
    }


	FMeshTask CodeGenerator::GenerateMesh_ClipWithMesh(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshClipWithMesh* ClipNode)
    {
		if (!ClipNode->Source)
		{
			// This argument is required
			ErrorLog->Add("Mesh clip-with-mesh source node is not set.", ELMT_ERROR, ClipNode->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		FMeshTask SourceTask = GenerateMesh(StaticOptions, InOptions, ClipNode->Source);

		if (!ClipNode->ClipMesh)
		{
			// This argument is required
			ErrorLog->Add("Mesh clip-with-mesh clipping mesh node is not set.", ELMT_ERROR, ClipNode->GetMessageContext());
			return SourceTask;
		}

		FMeshGenerationStaticOptions StaticClipOptions = StaticOptions;
		StaticClipOptions.ActiveTags.Empty();
		FMeshOptionsTask ClipOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshClipWithMeshClipOptions"),
			[InOptions]() mutable
			{
				FMeshGenerationDynamicOptions Result = InOptions.GetResult();
				Result.bLayouts = false;
				Result.OverrideLayouts.Empty();
				return Result;
			},
			InOptions
		);
		FMeshTask ClipTask = GenerateMesh(StaticClipOptions, ClipOptionsTask, ClipNode->ClipMesh);

		return UE::Tasks::Launch(TEXT("MutableMeshClipWithMesh"),
			[ClipNode, SourceTask, ClipTask]() mutable
			{
				FMeshGenerationResult SourceResult = SourceTask.GetResult();
				FMeshGenerationResult ClipResult = ClipTask.GetResult();
					
				FMeshGenerationResult Result;
					
				if (CVarUseASTOpMaskCliMeshForClipMeshWithMesh.GetValueOnAnyThread())
				{
					Ptr<UE::Mutable::Private::ASTOpMeshMaskClipMesh> MaskOp = new UE::Mutable::Private::ASTOpMeshMaskClipMesh();
					MaskOp->source = SourceResult.MeshOp;
					MaskOp->clip = ClipResult.MeshOp;

					Ptr<UE::Mutable::Private::ASTOpConstantBool> ConditionOp = new UE::Mutable::Private::ASTOpConstantBool(true);
										
					Ptr<UE::Mutable::Private::ASTOpMeshRemoveMask> RemoveOp = new UE::Mutable::Private::ASTOpMeshRemoveMask();
					RemoveOp->source = SourceResult.MeshOp;
					RemoveOp->FaceCullStrategy = ClipNode->FaceCullStrategy;
					RemoveOp->AddRemove(ConditionOp, MaskOp);

					Result = SourceResult;
					Result.MeshOp = RemoveOp;
				}
				else
				{
					Ptr<ASTOpMeshClipWithMesh> ClipOp = new ASTOpMeshClipWithMesh();

					// Base
					ClipOp->Source = SourceResult.MeshOp;
					ClipOp->ClipMesh = ClipResult.MeshOp;

					Result = SourceResult;
					Result.MeshOp = ClipOp;
				}
					
				return Result;
			},
			UE::Tasks::Prerequisites( SourceTask, ClipTask )
		);
    }

	
	FMeshTask CodeGenerator::GenerateMesh_TransformWithBone(const FMeshGenerationStaticOptions& InStaticOptions, FMeshOptionsTask InOptions, const NodeMeshTransformWithBone* InNode)
	{
		const void* MessageContext = InNode->GetMessageContext();
		
		if (!InNode->Source)
		{
			// This argument is required
			ErrorLog->Add("Mesh Transform with Bone source node is not set.", ELMT_ERROR, MessageContext);
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}
		
		// Generate the source mesh
		FMeshTask MeshGenerationTask = GenerateMesh(InStaticOptions, InOptions, InNode->Source);
		
		if (!InNode->MatrixNode)
		{
			// This argument is required
			ErrorLog->Add("Mesh Transform with Bone transform matrix node is not set.", ELMT_ERROR, MessageContext);
			return MeshGenerationTask;
		}
		
		// Transform matrix.
		FMatrixGenerationResult ChildResult;
		GenerateMatrix(ChildResult, InStaticOptions, InNode->MatrixNode);
		Ptr<ASTOp> MatrixOp = ChildResult.op;
		
		return UE::Tasks::Launch(TEXT("MutableMeshTransformWithBone"),
			[MeshGenerationTask, MatrixOp, BoneName = InNode->BoneName, ThresholdFactor = InNode->ThresholdFactor]() mutable
			{
				FMeshGenerationResult SourceMeshGenerationResults = MeshGenerationTask.GetResult();

				Ptr<ASTOpMeshTransformWithBone> TransformOp = new ASTOpMeshTransformWithBone();
				TransformOp->SourceMesh = SourceMeshGenerationResults.MeshOp;
				TransformOp->Matrix = MatrixOp;
				TransformOp->BoneName = BoneName;
				TransformOp->ThresholdFactor = ThresholdFactor;

				SourceMeshGenerationResults.MeshOp = TransformOp;

				return SourceMeshGenerationResults;
			},
			UE::Tasks::Prerequisites(MeshGenerationTask)
		);
	}

	
	FMeshTask CodeGenerator::GenerateMesh_TransformInMesh(const FMeshGenerationStaticOptions& InStaticOptions, const FMeshOptionsTask InOptions, const NodeMeshTransformInMesh* InNode)
	{
		const void* MessageContext = InNode->GetMessageContext();
		
		if (!InNode->SourceMesh)
		{
			ErrorLog->Add("Mesh Transform In Mesh source node is not set.", ELMT_ERROR, MessageContext);
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}
		
		// Generate the source mesh
		FMeshTask SourceMeshTask = GenerateMesh(InStaticOptions, InOptions, InNode->SourceMesh);
		
		if (!InNode->MatrixNode)
		{
			ErrorLog->Add("Mesh Transform In Mesh transform matrix node is not set.", ELMT_ERROR, MessageContext);
			return SourceMeshTask;
		}
		
		// Transform matrix.
		FMatrixGenerationResult ChildResult;
		GenerateMatrix(ChildResult, InStaticOptions, InNode->MatrixNode);
		Ptr<ASTOp> MatrixOp = ChildResult.op;
		
		if (!InNode->BoundingMesh)
		{
			ErrorLog->Add("Mesh Transform In Mesh bounding mesh node is not set.", ELMT_ERROR, MessageContext);
			return SourceMeshTask;
		}
		
		// Bounding mesh
		FMeshGenerationStaticOptions BoundingMeshStaticOptions(InStaticOptions);
		BoundingMeshStaticOptions.ActiveTags.Empty();
		FMeshGenerationDynamicOptions BoundingMeshDynamicOptions;
		BoundingMeshDynamicOptions.bLayouts = false;
		FMeshTask BoundingMeshTask = GenerateMesh(BoundingMeshStaticOptions, UE::Tasks::MakeCompletedTask<FMeshGenerationDynamicOptions>(BoundingMeshDynamicOptions), InNode->BoundingMesh);
		
		return UE::Tasks::Launch(TEXT("MutableMeshTransformWithBoundingMeshModifier"),
			[BoundingMeshTask, SourceMeshTask, MatrixOp, ErrorLog = ErrorLog, MessageContext]() mutable
			{
				FMeshGenerationResult BoundingResults = BoundingMeshTask.GetResult();
				FMeshGenerationResult SourceMeshResults = SourceMeshTask.GetResult();

				Ptr<ASTOpMeshTransformWithBoundingMesh> TransformOp = new ASTOpMeshTransformWithBoundingMesh();
				TransformOp->source = SourceMeshResults.MeshOp;
				TransformOp->matrix = MatrixOp;
				TransformOp->boundingMesh = BoundingResults.MeshOp;

				if (TransformOp->boundingMesh)
				{
					ASTOp::EClosedMeshTest IsClosed = TransformOp->boundingMesh->IsClosedMesh();
					if (IsClosed == ASTOp::EClosedMeshTest::No)
					{
						ErrorLog->Add("Bounding mesh used is not closed.", ELMT_WARNING, MessageContext);
					}
				}

				FMeshGenerationResult Result = SourceMeshResults;
				Result.MeshOp = TransformOp;
						
				return Result;
			},
			UE::Tasks::Prerequisites(SourceMeshTask, BoundingMeshTask)
		);
	}


	FMeshTask CodeGenerator::GenerateMesh_RemoveMesh(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshRemoveMesh* RemoveMeshNode)
	{
		if (!RemoveMeshNode->Source)
		{
			// This argument is required
			ErrorLog->Add("RemoveMesh base mesh node is not set.", ELMT_ERROR, RemoveMeshNode->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}
		
		if (!RemoveMeshNode->RemoveMesh)
		{
			// This argument is required
			ErrorLog->Add("RemoveMesh mesh to remove node is not set.", ELMT_ERROR, RemoveMeshNode->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}
		
		FMeshTask BaseMeshTask = GenerateMesh(StaticOptions, InOptions, RemoveMeshNode->Source);
		
		FMeshGenerationStaticOptions StaticRemoveOptions = StaticOptions;
		StaticRemoveOptions.ActiveTags.Empty();
		FMeshOptionsTask RemoveMeshOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshRemoveMeshOptions"),
			[InOptions]() mutable
			{
				FMeshGenerationDynamicOptions Result = InOptions.GetResult();
				Result.bLayouts = false;
				Result.OverrideLayouts.Empty();
				return Result;
			},
			InOptions
		);
		
		FMeshTask RemoveMeshTask = GenerateMesh(StaticRemoveOptions, RemoveMeshOptionsTask, RemoveMeshNode->RemoveMesh);
		
		return UE::Tasks::Launch(TEXT("MutableRemoveMesh"),
			[RemoveMeshTask, BaseMeshTask, RemoveMeshNode]() mutable
			{
				FMeshGenerationResult RemoveResults = RemoveMeshTask.GetResult();
				FMeshGenerationResult BaseResults = BaseMeshTask.GetResult();
					
				Ptr<ASTOpMeshMaskDiff> MaskOp = new ASTOpMeshMaskDiff;
				MaskOp->Source = BaseResults.BaseMeshOp;
				MaskOp->Fragment = RemoveResults.MeshOp;
					
				Ptr<UE::Mutable::Private::ASTOpConstantBool> ConditionOp = new UE::Mutable::Private::ASTOpConstantBool(true);
					
				Ptr<ASTOpMeshRemoveMask> RemoveOp = new ASTOpMeshRemoveMask();
				RemoveOp->source = BaseResults.MeshOp;
				RemoveOp->FaceCullStrategy = RemoveMeshNode->FaceCullStrategy;
				RemoveOp->AddRemove(ConditionOp, MaskOp);

				FMeshGenerationResult Result = BaseResults;
				Result.MeshOp = RemoveOp;
					
				return Result;
			},
			UE::Tasks::Prerequisites( BaseMeshTask, RemoveMeshTask)
		);
	}


	FMeshTask CodeGenerator::GenerateMesh_ClipDeform(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshClipDeform* ClipDeform)
	{
		if (!ClipDeform->BaseMesh)
		{
			// This argument is required
			ErrorLog->Add("Mesh Clip Deform base mesh node is not set.", ELMT_ERROR, ClipDeform->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		// Base Mesh
		FMeshTask BaseTask = GenerateMesh(StaticOptions, InOptions, ClipDeform->BaseMesh);

		if (!ClipDeform->ClipShape)
		{
			return BaseTask;
		}

		FMeshGenerationStaticOptions StaticClipOptions = StaticOptions;
		StaticClipOptions.ActiveTags.Empty();
		FMeshOptionsTask ClipOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshClipDeformOptions"),
			[InOptions]() mutable
			{
				FMeshGenerationDynamicOptions Result = InOptions.GetResult();
				Result.bLayouts = false;
				Result.OverrideLayouts.Empty();
				return Result;
			},
			InOptions
		);
		FMeshTask ShapeTask = GenerateMesh(StaticClipOptions, ClipOptionsTask, ClipDeform->ClipShape);

		return UE::Tasks::Launch(TEXT("MutableMeshClipDeform"),
			[BaseTask, ShapeTask]() mutable
			{
				FMeshGenerationResult BaseResult = BaseTask.GetResult();
				FMeshGenerationResult ShapeResult = ShapeTask.GetResult();

				const Ptr<ASTOpMeshBindShape> OpBind = new ASTOpMeshBindShape();
				const Ptr<ASTOpMeshClipDeform> OpClipDeform = new ASTOpMeshClipDeform();
				OpBind->Mesh = BaseResult.MeshOp;
				OpBind->Shape = ShapeResult.MeshOp;
				OpClipDeform->ClipShape = ShapeResult.MeshOp;
				OpClipDeform->Mesh = OpBind;

				FMeshGenerationResult Result = BaseResult;
				Result.MeshOp = OpClipDeform;

				return Result;
			},
			UE::Tasks::Prerequisites(BaseTask, ShapeTask)
		);
	}

	
	FMeshTask CodeGenerator::GenerateMesh_ApplyPose(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshApplyPose* Node )
    {
		if (!Node->Base)
		{
			// This argument is required
			ErrorLog->Add("Mesh apply-pose base node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		// Base Mesh
		FMeshTask BaseTask = GenerateMesh(StaticOptions, InOptions, Node->Base);

		if (!Node->Pose)
		{
			ErrorLog->Add("Mesh apply-pose pose node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return BaseTask;
		}

		FMeshGenerationStaticOptions StaticPoseOptions = StaticOptions;
		StaticPoseOptions.ActiveTags.Empty();
		FMeshOptionsTask PoseOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshApplyPoseOptions"),
			[InOptions]() mutable
			{
				FMeshGenerationDynamicOptions Result = InOptions.GetResult();
				Result.bLayouts = false;
				Result.OverrideLayouts.Empty();
				return Result;
			},
			InOptions
		);
		FMeshTask PoseTask = GenerateMesh(StaticPoseOptions, PoseOptionsTask, Node->Pose);

		return UE::Tasks::Launch(TEXT("MutableMeshApplyPose"),
			[BaseTask, PoseTask]() mutable
			{
				FMeshGenerationResult BaseResult = BaseTask.GetResult();
				FMeshGenerationResult PoseResult = PoseTask.GetResult();

				Ptr<ASTOpMeshApplyPose> op = new ASTOpMeshApplyPose();
				op->Base = BaseResult.MeshOp;
				op->Pose = PoseResult.MeshOp;

				FMeshGenerationResult Result = BaseResult;
				Result.MeshOp = op;

				return Result;
			},
			UE::Tasks::Prerequisites(BaseTask, PoseTask)
		);
    }


	FMeshTask CodeGenerator::GenerateMesh_Reshape(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshReshape* Node)
	{
		if (!Node->BaseMesh)
		{
			// This argument is required
			ErrorLog->Add("Mesh reshape base node is not set.", ELMT_ERROR, Node->GetMessageContext());
			return UE::Tasks::MakeCompletedTask<FMeshGenerationResult>();
		}

		// Base Mesh
		FMeshTask BaseTask = GenerateMesh(StaticOptions, InOptions, Node->BaseMesh);

		if (!Node->BaseShape || !Node->TargetShape)
		{
			return BaseTask;
		}


		FMeshGenerationStaticOptions StaticShapeOptions = StaticOptions;
		StaticShapeOptions.ActiveTags.Empty();
		FMeshOptionsTask ShapeOptionsTask = UE::Tasks::Launch(TEXT("MutableMeshApplyPoseOptions"),
			[InOptions]() mutable
			{
				FMeshGenerationDynamicOptions Result = InOptions.GetResult();
				Result.bLayouts = false;
				Result.OverrideLayouts.Empty();
				return Result;
			},
			InOptions
		);
		FMeshTask BaseShapeTask = GenerateMesh(StaticShapeOptions, ShapeOptionsTask, Node->BaseShape);
		FMeshTask TargetShapeTask = GenerateMesh(StaticShapeOptions, ShapeOptionsTask, Node->TargetShape);

		return UE::Tasks::Launch(TEXT("MutableMeshApplyPose"),
			[BaseTask, BaseShapeTask, TargetShapeTask, Node]() mutable
			{
				FMeshGenerationResult BaseResult = BaseTask.GetResult();
				FMeshGenerationResult BaseShapeResult = BaseShapeTask.GetResult();
				FMeshGenerationResult TargetShapeResult = TargetShapeTask.GetResult();

				Ptr<ASTOpMeshBindShape> OpBind = new ASTOpMeshBindShape();
				Ptr<ASTOpMeshApplyShape> OpApply = new ASTOpMeshApplyShape();

				OpBind->bReshapeSkeleton = Node->bReshapeSkeleton;
				OpBind->bReshapeSkeletonInvertSelection = Node->bReshapeSkeletonInvertSelection;
				OpBind->BonesToDeform = Node->BonesToDeform;
				OpBind->bReshapePhysicsVolumes = Node->bReshapePhysicsVolumes;
				OpBind->bReshapePhysicsVolumesInvertSelection = Node->bReshapePhysicsVolumesInvertSelection;
				OpBind->PhysicsToDeform = Node->PhysicsToDeform;
				OpBind->bReshapeVertices = Node->bReshapeVertices;
				OpBind->bRecomputeNormals = Node->bRecomputeNormals;
				OpBind->bApplyLaplacian = Node->bApplyLaplacian;
				OpBind->BindingMethod = static_cast<uint32>(EShapeBindingMethod::ReshapeClosestProject);

				OpBind->RChannelUsage = Node->ColorRChannelUsage;
				OpBind->GChannelUsage = Node->ColorGChannelUsage;
				OpBind->BChannelUsage = Node->ColorBChannelUsage;
				OpBind->AChannelUsage = Node->ColorAChannelUsage;

				OpApply->bReshapeVertices = OpBind->bReshapeVertices;
				OpApply->bRecomputeNormals = OpBind->bRecomputeNormals;
				OpApply->bReshapeSkeleton = OpBind->bReshapeSkeleton;
				OpApply->bApplyLaplacian = OpBind->bApplyLaplacian;
				OpApply->bReshapePhysicsVolumes = OpBind->bReshapePhysicsVolumes;

				// Base Mesh
				OpBind->Mesh = BaseResult.MeshOp;
				OpBind->Shape = BaseShapeResult.MeshOp;

				OpApply->Mesh = OpBind;
				OpApply->Shape = TargetShapeResult.MeshOp;

				FMeshGenerationResult Result = BaseResult;
				Result.MeshOp = OpApply;

				return Result;
			},
			UE::Tasks::Prerequisites(BaseTask, BaseShapeTask, TargetShapeTask)
		);
	}


	uint32 CodeGenerator::FUniqueMeshIds::EnsureUnique(uint32 Id)
	{
		UE::TUniqueLock Lock(Mutex);

		while (true)
		{
			bool bAlreadyPresent = false;
			Map.FindOrAdd(Id, &bAlreadyPresent);

			const bool bValid = !bAlreadyPresent && Id != 0 && Id != UINT32_MAX;
			if (bValid)
			{
				break;
			}

			++Id;
		}

		return Id;
	}


	FMeshTask CodeGenerator::GenerateMesh_SkeletalMeshBreak(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshSkeletalMeshObjectBreak* Node)
	{
		TArray<UE::Tasks::FTask, TInlineAllocator<2>> Requisites;
		Requisites.Emplace(InOptions);

		// Reference Mesh used to generate the UV layouts.
		TSharedPtr< TManagedPtr<FMesh> > ReferenceMesh = MakeShared<TManagedPtr<FMesh>>();

		// Compile-time reference? Add task to resolve as requisite.
		TManagedPtr<FMesh> Mesh = Node->ReferenceMesh;
		if (Mesh)
		{
			if (Mesh->IsReference())
			{
				PASSTHROUGH_ID MeshID = Mesh->PassthroughObject.GetId();

				bool bRunImmediatlyIfPossible = IsInGameThread();
				FString Morph = Mesh->GetReferencedMorph();
				UE::Tasks::FTask ReferenceCompletion = CompilerOptions->OptimisationOptions.ReferencedMeshResourceProvider(MeshID, Morph, ReferenceMesh, bRunImmediatlyIfPossible);
				Requisites.Emplace(ReferenceCompletion);
			}
			else
			{
				*ReferenceMesh = Mesh;
			}
		}

		FSkeletalMeshObjectGenerationResult SkeletalMeshResult;
		FSkeletalMeshGenerationOptions SkeletalMeshOptions(StaticOptions, nullptr);
		GenerateSkeletalMeshObject(SkeletalMeshResult, SkeletalMeshOptions, Node->SkeletalMeshObject);

		Ptr<ASTOp> SkeletalMeshParameter = SkeletalMeshResult.SkeletalMeshOp;
		
		FMeshTask MeshParamTask = UE::Tasks::Launch(TEXT("MutableMeshParameter"),
			[InOptions, Node, ReferenceMesh, StaticOptions, SkeletalMeshParameter, this]() mutable
			{
				FMeshGenerationDynamicOptions Options = InOptions.GetResult();

				FMeshGenerationResult Result;
				Ptr<ASTOpMeshSkeletalMeshBreak> Op;
					
				Op = new ASTOpMeshSkeletalMeshBreak();
				Op->SkeletalMeshParameter = SkeletalMeshParameter;
				Op->BaseLODIndex = Node->BaseLODIndex;
				Op->BaseSectionIndex = Node->BaseSectionIndex;
				Op->LODBias = Node->LODBias;
				Op->ConversionFlags = Node->ConversionFlags;

				// Assign an ID prefix to the mesh. 
				uint32 MeshID = Op->Hash();
				MeshID = HashCombine(MeshID, GetTypeHash(Op->BaseSectionIndex));
				MeshID = UniqueMeshIds.EnsureUnique(MeshID);
				Op->MeshID = MeshID;

				if (Options.OverrideLayouts.IsEmpty())
				{
					GenerateLayoutsForMesh(*ReferenceMesh, Node->Layouts);
				}

				
				// Fill the result structure. Some ops will be completed outside the lock.
				Ptr<ASTOp> LastMeshOp = GenerateLayoutOpsAndResult(Options, Op, Node->Layouts, Op->MeshID, Result);
				Result.MeshOp = LastMeshOp;
				Result.BaseMeshOp = LastMeshOp;
			
				TArray<FString> RealTimeMorphNames;
				RealTimeMorphNames.Reserve(Node->RealTimeMorphNames.Num());
				Algo::Transform(Node->RealTimeMorphNames, RealTimeMorphNames, [](const FName& Name) { return Name.ToString(); });

				const bool bAddMetadata = RealTimeMorphNames.Num() || Node->BonePosePriority != 0 || Node->SocketPriority != 0;
				if (bAddMetadata)
				{
					Ptr<ASTOpMeshAddMetadata> AddMetadataOp = new ASTOpMeshAddMetadata;

					AddMetadataOp->RealTimeMorphNames = MoveTemp(RealTimeMorphNames);
					AddMetadataOp->BonePosePriority = Node->BonePosePriority;
					AddMetadataOp->SocketPriority = Node->SocketPriority;

					AddMetadataOp->Source = Result.MeshOp;
					Result.MeshOp = AddMetadataOp;
				}

				return Result;
			},
			Requisites
		);

		
		if (UE::Tasks::Private::GetCurrentTask())
		{
			UE::Tasks::AddNested(MeshParamTask);
		}
		
		// Apply the modifier for the pre-normal operations stage.
		TArray<FirstPassGenerator::FModifier> Modifiers;
		constexpr bool bModifiersForBeforeOperations = true;
		this->GetModifiersFor(StaticOptions.ComponentId, StaticOptions.ActiveTags, bModifiersForBeforeOperations, Modifiers);

		// This task does the necessary processing of the mesh when it is available
		FMeshTask ApplyModifiersTask = ApplyMeshModifiers(Modifiers, StaticOptions, InOptions, MeshParamTask, FGuid(), Node->GetMessageContext(), Node);

		return ApplyModifiersTask;
	}


	FMeshTask CodeGenerator::GenerateMesh_External(const FMeshGenerationStaticOptions& StaticOptions, FMeshOptionsTask InOptions, const NodeMeshExternal* Node)
	{
		FExternalGenerationOptions Options;
		Options.MeshStaticOptions = StaticOptions;
		Options.MeshDynamicOptions = InOptions;

		FExternalTask ExternalTask = GenerateExternal(Options, Node->Node);

		return UE::Tasks::Launch(UE_SOURCE_LOCATION,[ExternalTask, this]() mutable
		{
			const FExternalGenerationResult& ExtensionResult = ExternalTask.GetResult();
				
			FMeshGenerationResult MeshResult;

			if (ExtensionResult.MeshGenerationResult.IsSet())
			{
				MeshResult = ExtensionResult.MeshGenerationResult.GetValue();
			}
				
			MeshResult.MeshOp = ExtensionResult.Op;
				
			return MeshResult;
		},
		ExternalTask);
	}
}
