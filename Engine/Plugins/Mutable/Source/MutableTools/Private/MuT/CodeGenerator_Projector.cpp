// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConstantProjector.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/CodeGenerator.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeProjector.h"
#include "MuT/NodeRange.h"


namespace UE::Mutable::Private
{


	void CodeGenerator::GenerateProjector(FProjectorGenerationResult& OutResult, const FGenericGenerationOptions& Options, const Ptr<const NodeProjector>& Untyped)
	{
		if (!Untyped)
		{
			OutResult = FProjectorGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;
		{
			UE::TUniqueLock Lock(GeneratedProjectors.Mutex);
			FGeneratedProjectorsMap::ValueType* Found = GeneratedProjectors.Map.Find(Key);
			if (Found)
			{
				OutResult = *Found;
				return;
			}
		}

		// Generate for each different type of node
		if ( Untyped->GetType()== NodeProjectorConstant::GetStaticType() )
		{
			GenerateProjector_Constant(OutResult, Options, static_cast<const NodeProjectorConstant*>(Untyped.get()));
		}
		else if (Untyped->GetType() == NodeProjectorParameter::GetStaticType() )
		{
			GenerateProjector_Parameter(OutResult, Options, static_cast<const NodeProjectorParameter*>(Untyped.get()));
		}
		else
		{
			check(false);
		}


		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedProjectors.Mutex);
			GeneratedProjectors.Map.Add(Key, OutResult);
		}
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateProjector_Constant(FProjectorGenerationResult& result, const FGenericGenerationOptions& Options,
		const Ptr<const NodeProjectorConstant>& Node)
	{
		Ptr<ASTOpConstantProjector> op = new ASTOpConstantProjector();
		FProjector p;
		p.type = Node->Type;
		p.position = Node->Position;
		p.direction = Node->Direction;
		p.up = Node->Up;
		p.scale = Node->Scale;
		p.projectionAngle = Node->ProjectionAngle;
		op->value = p;

		result.op = op;
		result.type = op->value.type;
	}


	void CodeGenerator::GenerateProjector_Parameter(FProjectorGenerationResult& OutResult, const FGenericGenerationOptions& Options,
		const Ptr<const NodeProjectorParameter>& Node)
	{
		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				FParameterDesc param;
				param.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->UID, param.UID);
				check(bParseOk);
				param.Type = EParameterType::Projector;

				FProjector p;
				p.type = Node->Type;
				p.position = Node->Position;
				p.direction = Node->Direction;
				p.up = Node->Up;
				p.scale = Node->Scale;
				p.projectionAngle = Node->ProjectionAngle;

				param.DefaultValue.Set<FParamProjectorType>(p);

				op = new ASTOpParameter();
				op->Type = EOpType::PR_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Node->Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options, Node->Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		const FProjector& Projector = op->Parameter.DefaultValue.Get<FParamProjectorType>();
		OutResult.type = Projector.type;
		OutResult.op = op;
	}


	//-------------------------------------------------------------------------------------------------
	void CodeGenerator::GenerateMissingProjectorCode(FProjectorGenerationResult& result,
		const void* errorContext)
	{
		// Log an error message
		ErrorLog->Add("Required projector connection not found.", ELMT_ERROR, errorContext);

		FProjector p;
		p.type = EProjectorType::Planar;

		p.direction = { 1, 0 ,0 };
		p.up = { 0, 0, 1 };
		p.position = { 0.0f, 0.0f, 0.0f };
		p.scale = { 1.0f, 1.0f, 1.0f };

		Ptr<ASTOpConstantProjector> op = new ASTOpConstantProjector();
		op->value = p;

		result.op = op;
		result.type = p.type;
	}

}
