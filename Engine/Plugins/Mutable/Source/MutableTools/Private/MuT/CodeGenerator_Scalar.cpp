// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Image.h"
#include "MuR/MutableMath.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Parameters.h"
#include "MuR/Platform.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ASTOpScalarCurve.h"
#include "MuT/ASTOpScalarArithmetic.h"
#include "MuT/ASTOpConstantScalar.h"
#include "MuT/ASTOpMaterialBreak.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/ErrorLog.h"
#include "MuT/NodeImage.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeScalarArithmeticOperation.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/NodeScalarCurve.h"
#include "MuT/NodeScalarEnumParameter.h"
#include "MuT/NodeScalarExternal.h"
#include "MuT/NodeScalarMaterialBreak.h"
#include "MuT/NodeScalarParameter.h"
#include "MuT/NodeScalarSwitch.h"
#include "MuT/NodeScalarTable.h"
#include "MuT/NodeScalarVariation.h"
#include "MuT/Table.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{

	void CodeGenerator::GenerateScalar(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalar>& Untyped)
	{
		if (!Untyped)
		{
			result = FScalarGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;
		{
			UE::TUniqueLock Lock(GeneratedScalars.Mutex);
			FGeneratedScalarsMap::ValueType* Found = GeneratedScalars.Map.Find(Key);
			if (Found)
			{
				result = *Found;
				return;
			}
		}

		// Generate for each different type of node
		if (Untyped->GetType() == NodeScalarConstant::GetStaticType())
		{
			const NodeScalarConstant* Constant = static_cast<const NodeScalarConstant*>(Untyped.get());
			GenerateScalar_Constant(result, Options, Constant);
		}
		else if (Untyped->GetType() == NodeScalarParameter::GetStaticType())
		{
			const NodeScalarParameter* Param = static_cast<const NodeScalarParameter*>(Untyped.get());
			GenerateScalar_Parameter(result, Options, Param);
		}
		else if (Untyped->GetType() == NodeScalarSwitch::GetStaticType())
		{
			const NodeScalarSwitch* Switch = static_cast<const NodeScalarSwitch*>(Untyped.get());
			GenerateScalar_Switch(result, Options, Switch);
		}
		else if (Untyped->GetType() == NodeScalarEnumParameter::GetStaticType())
		{
			const NodeScalarEnumParameter* EnumParam = static_cast<const NodeScalarEnumParameter*>(Untyped.get());
			GenerateScalar_EnumParameter(result, Options, EnumParam);
		}
		else if (Untyped->GetType() == NodeScalarCurve::GetStaticType())
		{
			const NodeScalarCurve* Curve = static_cast<const NodeScalarCurve*>(Untyped.get());
			GenerateScalar_Curve(result, Options, Curve);
		}
		else if (Untyped->GetType() == NodeScalarArithmeticOperation::GetStaticType())
		{
			const NodeScalarArithmeticOperation* Arithmetic = static_cast<const NodeScalarArithmeticOperation*>(Untyped.get());
			GenerateScalar_Arithmetic(result, Options, Arithmetic);
		}
		else if (Untyped->GetType() == NodeScalarVariation::GetStaticType())
		{
			const NodeScalarVariation* Variation = static_cast<const NodeScalarVariation*>(Untyped.get());
			GenerateScalar_Variation(result, Options, Variation);
		}
		else if (Untyped->GetType() == NodeScalarTable::GetStaticType())
		{
			const NodeScalarTable* Table = static_cast<const NodeScalarTable*>(Untyped.get());
			GenerateScalar_Table(result, Options, Table);
		}
		else if (Untyped->GetType() == NodeScalarMaterialBreak::GetStaticType())
		{
			const NodeScalarMaterialBreak* Table = static_cast<const NodeScalarMaterialBreak*>(Untyped.get());
			GenerateScalar_MaterialBreak(result, Options, Table);
		}
		else if (Untyped->GetType() == NodeScalarExternal::GetStaticType())
		{
			const NodeScalarExternal* Node = static_cast<const NodeScalarExternal*>(Untyped.get());
			GenerateScalar_External(result, Options, Node);
		}
		else
		{
			check(false);
			return;
		}

		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedScalars.Mutex);
			GeneratedScalars.Map.Add(Key, result);
		}
	}


	void CodeGenerator::GenerateScalar_Constant(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarConstant>& Node)
	{
		Ptr<ASTOpConstantScalar> op = new ASTOpConstantScalar;
		op->Value = Node->Value;
		result.op = op;
	}


	void CodeGenerator::GenerateScalar_Parameter(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarParameter>& Node)
	{
		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;		
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				FParameterDesc param;
				param.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->UID, param.UID);
				check(bParseOk);
				param.Type = EParameterType::Float;
				param.DefaultValue.Set<FParamFloatType>(Node->DefaultValue);

				op = new ASTOpParameter();
				op->Type = EOpType::SC_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Node->Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options, Node->Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		result.op = op;
	}


	void CodeGenerator::GenerateScalar_EnumParameter(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarEnumParameter>& Node)
	{
		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				FParameterDesc param;
				param.Name = Node->Name;
				if (!Node->UID.IsEmpty())
				{
					bool bParseOk = FGuid::Parse(Node->UID, param.UID);
					check(bParseOk);
				}
				param.Type = EParameterType::Int;
				param.DefaultValue.Set<FParamIntType>(Node->DefaultValue);

				param.PossibleValues.SetNum(Node->Options.Num());
				for (int32 i = 0; i < Node->Options.Num(); ++i)
				{
					param.PossibleValues[i].Value = (int16)Node->Options[i].Value;
					param.PossibleValues[i].Name = Node->Options[i].Name;
				}

				op = new ASTOpParameter();
				op->Type = EOpType::NU_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 RangeIndex = 0; RangeIndex < Node->Ranges.Num(); ++RangeIndex)
			{
				FRangeGenerationResult RangeResult;
				GenerateRange(RangeResult, Options, Node->Ranges[RangeIndex]);
				op->Ranges.Emplace(op.get(), RangeResult.sizeOp, RangeResult.rangeName, RangeResult.rangeUID);
			}
		}

		result.op = op;
	}


	void CodeGenerator::GenerateScalar_Switch(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarSwitch>& Typed)
	{
		const NodeScalarSwitch& node = *Typed;

		if (node.Options.Num() == 0)
		{
			// No options in the switch!
			Ptr<ASTOp> missingOp = GenerateMissingScalarCode(TEXT("Switch option"),
				1.0f,
				Typed->GetMessageContext());
			result.op = missingOp;
			return;
		}

		Ptr<ASTOpSwitch> op = new ASTOpSwitch();
		op->Type = EOpType::SC_SWITCH;

		// Variable value
		if (node.Parameter)
		{
			FScalarGenerationResult ChildResult;
			GenerateScalar(ChildResult, Options, node.Parameter.get());
			op->Variable = ChildResult.op;
		}
		else
		{
			// This argument is required
			op->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Typed->GetMessageContext());
		}

		// Options
		for (int32 t = 0; t < node.Options.Num(); ++t)
		{
			Ptr<ASTOp> branch;
			if (node.Options[t])
			{
				FScalarGenerationResult ChildResult;
				GenerateScalar(ChildResult, Options, node.Options[t].get());
				branch = ChildResult.op;
			}
			else
			{
				// This argument is required
				branch = GenerateMissingScalarCode(TEXT("Switch option"), 1.0f, Typed->GetMessageContext());
			}
			op->Cases.Emplace((int16_t)t, op, branch);
		}

		result.op = op;
	}


	void CodeGenerator::GenerateScalar_Variation(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarVariation>& Node)
	{
		Ptr<ASTOp> op;

		// Default case
		if (Node->DefaultScalar)
		{
			FMeshGenerationResult branchResults;

			FScalarGenerationResult ChildResult;
			GenerateScalar(ChildResult, Options, Node->DefaultScalar);
			op = ChildResult.op;
		}

		// Process variations in reverse order, since conditionals are built bottom-up.
		for (int32 t = Node->Variations.Num() - 1; t >= 0; --t)
		{
			int32 tagIndex = -1;
			const FString& tag = Node->Variations[t].Tag;
			for (int32 i = 0; i < FirstPass.Tags.Num(); ++i)
			{
				if (FirstPass.Tags[i].Tag == tag)
				{
					tagIndex = i;
				}
			}

			if (tagIndex < 0)
			{
				FString Msg = FString::Printf(TEXT("Unknown tag found in image variation [%s]."), *tag);

				ErrorLog->Add(Msg, ELMT_WARNING, Node->GetMessageContext());
				continue;
			}

			Ptr<ASTOp> variationOp;
			if (Node->Variations[t].Scalar)
			{
				FScalarGenerationResult ChildResult;
				GenerateScalar(ChildResult, Options, Node->Variations[t].Scalar);
				variationOp = ChildResult.op;
			}
			else
			{
				// This argument is required
				variationOp = GenerateMissingScalarCode(TEXT("Variation option"), 0.0f, Node->GetMessageContext());
			}


			Ptr<ASTOpConditional> conditional = new ASTOpConditional;
			conditional->type = EOpType::SC_CONDITIONAL;
			conditional->no = op;
			conditional->yes = variationOp;
			conditional->condition = FirstPass.Tags[tagIndex].GenericCondition;

			op = conditional;
		}

		result.op = op;
	}


	void CodeGenerator::GenerateScalar_Curve(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarCurve>& Typed)
	{
		Ptr<ASTOpScalarCurve> op = new ASTOpScalarCurve();

		if (NodeScalar* pA = Typed->CurveSampleValue.get())
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options, pA);
			op->time = ParamResult.op;
		}
		else
		{
			op->time = CodeGenerator::GenerateMissingScalarCode(TEXT("Curve T"), 0.5f, Typed->GetMessageContext());
		}

		op->Curve = Typed->Curve;

		result.op = op;
	}


	void CodeGenerator::GenerateScalar_Arithmetic(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarArithmeticOperation>& Node)
	{
		Ptr<ASTOpScalarArithmetic> op = new ASTOpScalarArithmetic;

		switch (Node->Operation)
		{
		case NodeScalarArithmeticOperation::AO_ADD: op->Operation = FOperation::ArithmeticArgs::ADD; break;
		case NodeScalarArithmeticOperation::AO_SUBTRACT: op->Operation = FOperation::ArithmeticArgs::SUBTRACT; break;
		case NodeScalarArithmeticOperation::AO_MULTIPLY: op->Operation = FOperation::ArithmeticArgs::MULTIPLY; break;
		case NodeScalarArithmeticOperation::AO_DIVIDE: op->Operation = FOperation::ArithmeticArgs::DIVIDE; break;
		default:
			checkf(false, TEXT("Unknown arithmetic operation."));
			op->Operation = FOperation::ArithmeticArgs::NONE;
			break;
		}

		// A
		if (NodeScalar* pA = Node->A.get())
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options, pA);

			op->A = ParamResult.op;
		}
		else
		{
			op->A = GenerateMissingScalarCode( TEXT("ScalarArithmetic A"), 1.0f, Node->GetMessageContext() );
		}

		// B
		if (NodeScalar* pB = Node->B.get())
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options, pB);

			op->B = ParamResult.op;
		}
		else
		{
			op->B =  GenerateMissingScalarCode( TEXT("ScalarArithmetic B"), 1.0f, Node->GetMessageContext() );
		}

		result.op = op;
	}


	void CodeGenerator::GenerateScalar_Table(FScalarGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarTable>& Node)
	{
		Ptr<ASTOp> Op = GenerateTableSwitch<NodeScalarTable, ETableColumnType::Scalar, EOpType::SC_SWITCH>(*Node,
			[this,&Options](const NodeScalarTable& Node, int32 colIndex, int32 row, FErrorLog*)
			{
				Ptr<NodeScalarConstant> Cell = new NodeScalarConstant();
				Cell->Value = Node.Table->GetPrivate()->Rows[row].Values[colIndex].Scalar;

				FScalarGenerationResult ParamResult;
				GenerateScalar(ParamResult, Options, Cell);

				return ParamResult.op;
			});

		result.op = Op;
	}


	void CodeGenerator::GenerateScalar_MaterialBreak(FScalarGenerationResult& Result, const FGenericGenerationOptions& Options, const Ptr<const NodeScalarMaterialBreak>& Node)
	{
		Ptr<ASTOpMaterialBreak> MaterialOp = new ASTOpMaterialBreak();

		// Set the key that identifies the break node parameter
		MaterialOp->ParameterKey = Node->ParameterKey;

		// Generate Material Source
		FMaterialGenerationOptions MaterialSourceOptions;
		FMaterialGenerationResult MaterialSourceResult;

		GenerateMaterial(MaterialSourceOptions, MaterialSourceResult, Node->MaterialSource);
		MaterialOp->Material = MaterialSourceResult.op;
		MaterialOp->Type = EOpType::SC_MATERIAL_BREAK;

		Result.op = MaterialOp;
	}


	void CodeGenerator::GenerateScalar_External(FScalarGenerationResult& Result, const FGenericGenerationOptions& InOptions, const Ptr<const NodeScalarExternal>& InNode)
	{
		FExternalGenerationOptions Options;
		Options.ScalarOptions = InOptions;
					
		FExternalTask ExtensionResult = GenerateExternal(Options, InNode->Node);
		WaitTask(ExtensionResult);

		Result.op = ExtensionResult.GetResult().Op;
	}


	UE::Mutable::Private::Ptr<ASTOp> CodeGenerator::GenerateMissingScalarCode(const TCHAR* strWhere, float value, const void* errorContext)
	{
		// Log a warning
		FString Msg = FString::Printf(TEXT("Required connection not found: %s"), strWhere );
		ErrorLog->Add(Msg, ELMT_ERROR, errorContext);

		// Create a constant node
		Ptr<NodeScalarConstant> Node = new NodeScalarConstant();
		Node->Value = value;

		FGenericGenerationOptions Options;
		FScalarGenerationResult ParamResult;
		GenerateScalar(ParamResult, Options, Node.get());

		return ParamResult.op;
	}

}
