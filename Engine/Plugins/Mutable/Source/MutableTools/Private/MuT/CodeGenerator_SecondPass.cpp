// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/CodeGenerator_SecondPass.h"

#include "ASTOpBoolOr.h"
#include "Misc/AssertionMacros.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Types.h"
#include "MuT/ASTOpConstantBool.h"
#include "MuT/ASTOpBoolAnd.h"
#include "MuT/ASTOpBoolNot.h"
#include "MuT/ErrorLog.h"
#include "MuT/CodeGenerator_FirstPass.h"

namespace UE::Mutable::Private
{


	//---------------------------------------------------------------------------------------------
	SecondPassGenerator::SecondPassGenerator(
		FirstPassGenerator* firstPass,
		const CompilerOptions::Private* options)
	{
		check(firstPass);
		check(options);
		FirstPass = firstPass;
		CompilerOptions = options;
	}


	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> SecondPassGenerator::GenerateTagCondition(int32 tagIndex,
		const TSet<int32>& posSurf,
		const TSet<int32>& negSurf,
		const TSet<int32>& posTag,
		const TSet<int32>& negTag)
	{
		FirstPassGenerator::FTag& t = FirstPass->Tags[tagIndex];

		// If this tag is already in the list of positive tags, return true as condition
		if (posTag.Contains(tagIndex))
		{
			return OpPool.Add(new ASTOpConstantBool(true));
		}

		// If this tag is already in the list of negative tags, return false as condition
		if (negTag.Contains(tagIndex))
		{
			return OpPool.Add(new ASTOpConstantBool(false));
		}

		// Cached?
		FConditionGenerationKey key;
		key.tagOrSurfIndex = tagIndex;
		//    key.negSurf = negSurf;
		//    key.posSurf = posSurf;
		//    key.negTag = negTag;
		//    key.posTag = posTag;
		for (auto s : negTag) { if (TagsPerTag[tagIndex].Contains(s)) { key.negTag.Add(s); } }
		for (auto s : posTag) { if (TagsPerTag[tagIndex].Contains(s)) { key.posTag.Add(s); } }
		for (auto s : negSurf) { if (SurfacesPerTag[tagIndex].Contains(s)) { key.negSurf.Add(s); } }
		for (auto s : posSurf) { if (SurfacesPerTag[tagIndex].Contains(s)) { key.posSurf.Add(s); } }
		
		if (Ptr<ASTOp>* Found = TagConditionGenerationCache.Find(key))
		{
			return *Found;
		}
		
		Ptr<ASTOp> c = new ASTOpConstantBool(false);

		// Condition expression for all the surfaces that activate the tag
		for (int32 surfIndex : t.Surfaces)
		{
			if (posSurf.Contains(surfIndex))
			{
				// This surface is already a positive requirement higher up in the condition so
				// we can ignore it here.
				continue;
			}

			if (negSurf.Contains(surfIndex))
			{
				// This surface is a negative requirement higher up in the condition so
				// this branch never be true.
				continue;
			}

			const FirstPassGenerator::FSurface& surface = FirstPass->Surfaces[surfIndex];

			auto PositiveTags = posTag;
			PositiveTags.Add(tagIndex);

			Ptr<ASTOp> SurfCondition = GenerateDataCodition(surfIndex,
				FirstPass->Surfaces[surfIndex].PositiveTags,
				FirstPass->Surfaces[surfIndex].NegativeTags,
				posSurf,
				negSurf,
				PositiveTags,
				negTag);

			bool bSurfaceConstantValue = false;
			bool bIsSurfaceConditionConstant = ASTOpConstantBool::IsConstantBool(SurfCondition, bSurfaceConstantValue);

			if (bIsSurfaceConditionConstant)
			{
				// If the surface is a constant false, we can skip adding it
				if (!bSurfaceConstantValue)
				{
					continue;
				}
				
				SurfCondition.reset();
			}

			Ptr<ASTOp> ObjectCondition = surface.ObjectCondition;

			bool bObjectConstantValue = false;
			bool bIsObjectConditionConstant = ASTOpConstantBool::IsConstantBool(ObjectCondition, bObjectConstantValue);

			if (bIsObjectConditionConstant)
			{
				// If the surface is a constant false, we can skip adding it
				if (!bObjectConstantValue)
				{
					continue;
				}

				ObjectCondition.reset();
			}

			Ptr<ASTOp> FullCondition;
			if (SurfCondition && ObjectCondition)
			{
				Ptr<ASTOpBoolAnd> f = new ASTOpBoolAnd;
				f->A = ObjectCondition;
				f->B = SurfCondition;
				FullCondition = OpPool.Add(f);
			}
			else
			{
				FullCondition = SurfCondition ? SurfCondition : ObjectCondition;
			}
			
			if (FullCondition)
			{
				Ptr<ASTOpBoolOr> o = new ASTOpBoolOr();
				o->A = FullCondition;
				o->B = c;
				c = OpPool.Add(o);
			}
			else
			{
				// No condition means always true.
				return OpPool.Add(new ASTOpConstantBool(true));
			}

			// Optimise the condition now.
			//PartialOptimise( c, CompilerOptions->OptimisationOptions );
		}

		TagConditionGenerationCache.Add(key, c);

		return c;
	}

	//---------------------------------------------------------------------------------------------
	UE::Mutable::Private::Ptr<ASTOp> SecondPassGenerator::GenerateDataCodition(int32 Index,
		const TArray<FString>& PositiveTags,
		const TArray<FString>& NegativeTags,
		const TSet<int32>& posSurf,
		const TSet<int32>& negSurf,
		const TSet<int32>& posTag,
		const TSet<int32>& negTag)
	{
		// If this surface is already in the list of positive surfaces, return true as condition
		if (posSurf.Contains(Index))
		{
			return OpPool.Add(new ASTOpConstantBool(true));
		}

		// If this surface is already in the list of negative surfaces, return false as condition
		if (negSurf.Contains(Index))
		{
			return OpPool.Add(new ASTOpConstantBool(false));
		}

		Ptr<ASTOp> c = new ASTOpConstantBool(true);

		for (const FString& PositiveTag : PositiveTags)
		{
			const int32 PositiveTagIndex = FirstPass->Tags.IndexOfByPredicate([&](const FirstPassGenerator::FTag& e) { return e.Tag == PositiveTag; });
			if (PositiveTagIndex == INDEX_NONE)
			{
				// This could happen if a tag is in a variation but no one defines it.
				// This surface depends on a tag that will never be active, so it will never be used.
				return OpPool.Add(new ASTOpConstantBool(false));
			}
			
			TSet<int32> positiveSurfacesVisited = posSurf;
			positiveSurfacesVisited.Add(Index);

			Ptr<ASTOp> TagCondition = GenerateTagCondition(PositiveTagIndex,
				positiveSurfacesVisited,
				negSurf,
				posTag,
				negTag);

			if (!TagCondition)
			{
				// This tag is unconditionally activated, so there's no condition logic to add
				continue;
			}

			// TODO: Optimise the tag condition here

			// If the tag is a constant ...
			bool bConstantValue = false;
			bool bIsConstantBool = ASTOpConstantBool::IsConstantBool(TagCondition, bConstantValue);

			if (!bIsConstantBool)
			{
				Ptr<ASTOpBoolAnd> o = new ASTOpBoolAnd;
				o->A = TagCondition;
				o->B = c;
				c = OpPool.Add(o);
			}
			else if(bConstantValue == false)
			{
				// Entire expression will be false. No need to evaluate anything else.
				return OpPool.Add(new ASTOpConstantBool(false));
			}
		}


		for (const FString& NegativeTag : NegativeTags)
		{
			const int32 NegativeTagIndex = FirstPass->Tags.IndexOfByPredicate([&](const FirstPassGenerator::FTag& e) { return e.Tag == NegativeTag; });
			if (NegativeTagIndex == INDEX_NONE)
			{
				// This could happen if a tag is in a variation but none defines it.
				continue;
			}
			
			TSet<int32> PositiveSurfacesVisited = negSurf;
			TSet<int32> NegativeSurfacesVisited = posSurf;
			NegativeSurfacesVisited.Add(Index);
			TSet<int32> PositiveTagsVisited = negTag;
			TSet<int32> NegativeTagsVisited = posTag;
			Ptr<ASTOp> TagCondition = GenerateTagCondition(NegativeTagIndex,
				PositiveSurfacesVisited,
				NegativeSurfacesVisited,
				PositiveTagsVisited,
				NegativeTagsVisited);

			// No condition is equal to a conditional with a true constant
			if (!TagCondition)
			{
				TagCondition = new ASTOpConstantBool(true);
			}

			// If the tag is a constant ...
			bool bConstantValue = false;
			if (!ASTOpConstantBool::IsConstantBool(TagCondition, bConstantValue))
			{
				Ptr<ASTOpBoolNot> n = new ASTOpBoolNot;
				n->A = TagCondition;

				Ptr<ASTOpBoolAnd> o = new ASTOpBoolAnd;
				o->A = n;
				o->B = c;
				c = OpPool.Add(o);
			}
			else if(bConstantValue == true)
			{
				// Always true which becomes always false. No need to evaluate anything else.
				return OpPool.Add(new ASTOpConstantBool(false));
			}
		}

		return c;
	}


	//---------------------------------------------------------------------------------------------
	bool SecondPassGenerator::Generate(TSharedPtr<FErrorLog> InErrorLog, const Node* root)
	{
		MUTABLE_CPUPROFILER_SCOPE(SecondPassGenerate);

		check(root);
		ErrorLog = InErrorLog;

		// Find the list of surfaces every tag depends on
		SurfacesPerTag.Empty();
		SurfacesPerTag.SetNum(FirstPass->Tags.Num());
		TagsPerTag.Empty();
		TagsPerTag.SetNum(FirstPass->Tags.Num());
		for (int32 TagIndex = 0; TagIndex < FirstPass->Tags.Num(); ++TagIndex)
		{
			TSet<int32> PendingSurfaces;
			for (int32 SurfaceIndex : FirstPass->Tags[TagIndex].Surfaces)
			{
				PendingSurfaces.Add(SurfaceIndex);
			}

			TSet<int32> ProcessedSurfaces;
			while (!PendingSurfaces.IsEmpty())
			{
				const int32 PendingSurfaceIndex = *PendingSurfaces.begin();
				PendingSurfaces.Remove(PendingSurfaceIndex);

				if (ProcessedSurfaces.Contains(PendingSurfaceIndex))
				{
					continue;
				}

				ProcessedSurfaces.Add(PendingSurfaceIndex);
				SurfacesPerTag[TagIndex].Add(PendingSurfaceIndex);
				
				FirstPassGenerator::FSurface& CurrentSurface = FirstPass->Surfaces[PendingSurfaceIndex];
				for (const FString& PositiveTag : CurrentSurface.PositiveTags) 
				{
					const int32 PositiveTagIndex = FirstPass->Tags.IndexOfByPredicate([&](const FirstPassGenerator::FTag& e) { return e.Tag == PositiveTag; });
					if (PositiveTagIndex == INDEX_NONE)
					{
						// This could happen if a tag is in a variation but no one defines it.
						continue;
					}
					
					TagsPerTag[TagIndex].Add(PositiveTagIndex);

					for (int32 Surface : FirstPass->Tags[PositiveTagIndex].Surfaces)
					{
						if (SurfacesPerTag[TagIndex].Contains(Surface))
						{
							PendingSurfaces.Add(Surface);
						}
					}
				}
				for (const FString& NegativeTag : CurrentSurface.NegativeTags)
				{
					// Look for the NegativeTag being processed and store it.
					const int32 NegativeTagIndex = FirstPass->Tags.IndexOfByPredicate([&](const FirstPassGenerator::FTag& e) { return e.Tag == NegativeTag; });
					if (NegativeTagIndex == INDEX_NONE)
					{
						// This could happen if a tag is in a variation but no one defines it.
						continue;
					}
					
					TagsPerTag[TagIndex].Add(NegativeTagIndex);

					for (size_t Surface : FirstPass->Tags[NegativeTagIndex].Surfaces)
					{
						if (SurfacesPerTag[TagIndex].Contains(Surface))
						{
							PendingSurfaces.Add(Surface);
						}
					}
				}
			}
		}

		// Create the conditions for every surface, modifier, component and individual tag.
		TagConditionGenerationCache.Reset();

		TSet<int32> Empty;

		for (int32 SurfaceIndex = 0; SurfaceIndex < FirstPass->Surfaces.Num(); ++SurfaceIndex)
		{
			FirstPassGenerator::FSurface& Surface = FirstPass->Surfaces[SurfaceIndex];

			Ptr<ASTOp> c = GenerateDataCodition(
				SurfaceIndex, 
				Surface.PositiveTags,
				Surface.NegativeTags,
				Empty, Empty, Empty, Empty);

			bool bDataConditionValue = false;
			if (ASTOpConstantBool::IsConstantBool(c, bDataConditionValue))
			{
				if (!bDataConditionValue)
				{
					Surface.FinalCondition = c;
					continue;
				}

				c = nullptr;
			}

			Ptr<ASTOp> MeshCondition = Surface.MeshCondition;

			bool bMeshConditionValue = false;
			if (ASTOpConstantBool::IsConstantBool(MeshCondition, bMeshConditionValue))
			{
				if (!bMeshConditionValue)
				{
					Surface.FinalCondition = Surface.MeshCondition;
					continue;
				}

				MeshCondition = nullptr;
			}

			if (c && MeshCondition)
			{
				Ptr<ASTOpBoolAnd> ConditionOp = new ASTOpBoolAnd();
				ConditionOp->A = Surface.MeshCondition ? Surface.MeshCondition : new ASTOpConstantBool(true);
				ConditionOp->B = c;
				Surface.FinalCondition = ConditionOp;
			}
			else
			{
				Surface.FinalCondition = c ? c : MeshCondition;
			}
			
			// TODO: Convert to modifiers that enable tags?
			//for (int32 EditIndex = 0; EditIndex < Surface.Edits.Num(); ++EditIndex)
			//{
			//	FirstPassGenerator::FSurface::FEdit& Edit = Surface.Edits[EditIndex];
			//	
			//	Ptr<ASTOp> c = GenerateDataCodition(
			//		EditIndex, 
			//		Surface.Edits[EditIndex].PositiveTags,
			//		Surface.Edits[EditIndex].NegativeTags,
			//		Empty, Empty, Empty, Empty);

			//	Ptr<ASTOpBoolAnd> OpAnd = new ASTOpBoolAnd;
			//	OpAnd->a = Edit.Condition;
			//	OpAnd->b = c;
			//	c = OpPool.Add(OpAnd);
			//	
			//	Edit.Condition = OpAnd;
			//}
		}

		for (int32 ModifierIndex = 0; ModifierIndex < FirstPass->Modifiers.Num(); ++ModifierIndex)
		{
			Ptr<ASTOp> c = GenerateDataCodition(
				ModifierIndex, 
				FirstPass->Modifiers[ModifierIndex].PositiveTags,
				FirstPass->Modifiers[ModifierIndex].NegativeTags,
				Empty, Empty, Empty, Empty);

			Ptr<ASTOpBoolAnd> ConditionOp = new ASTOpBoolAnd();
			ConditionOp->A = FirstPass->Modifiers[ModifierIndex].ObjectCondition ? FirstPass->Modifiers[ModifierIndex].ObjectCondition : new ASTOpConstantBool(true);
			ConditionOp->B = c;

			FirstPass->Modifiers[ModifierIndex].FinalCondition = ConditionOp;
		}

		for (int32 ComponentIndex = 0; ComponentIndex < FirstPass->Components.Num(); ++ComponentIndex)
		{
			Ptr<ASTOp> c = GenerateDataCodition(
				ComponentIndex, 
				FirstPass->Components[ComponentIndex].PositiveTags,
				FirstPass->Components[ComponentIndex].NegativeTags,
				Empty, Empty, Empty, Empty);

			FirstPass->Components[ComponentIndex].ComponentCondition = c;
		}

		// TODO: Do we really need the tag conditions from here on?
		for (int32 s = 0; s < FirstPass->Tags.Num(); ++s)
		{
			Ptr<ASTOp> c = GenerateTagCondition(s, Empty, Empty, Empty, Empty);
			FirstPass->Tags[s].GenericCondition = c;
		}

		FirstPass = nullptr;

		return true;
	}


}
