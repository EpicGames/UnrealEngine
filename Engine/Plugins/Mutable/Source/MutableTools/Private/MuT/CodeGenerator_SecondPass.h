// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformCrt.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/AST.h"
#include "MuT/CodeGenerator_FirstPass.h"
#include "MuT/Compiler.h"
#include "MuT/Node.h"
#include "Containers/Set.h"
#include "Containers/Map.h"

namespace UE::Mutable::Private
{
	inline bool SetsEquals(const TSet<int32>& Left, const TSet<int32>& Right)
	{
		return Left.Num() == Right.Num() && Left.Includes(Right);
	}


	/** Second pass of the code generation process.
     * It solves surface and modifier conditions from tags and variations
	 */
    class SecondPassGenerator
	{
	public:

		SecondPassGenerator( FirstPassGenerator*, const CompilerOptions::Private*  );

		// Return true on success.
        bool Generate(TSharedPtr<FErrorLog>, const Node* Root );

	private:

        FirstPassGenerator* FirstPass = nullptr;
        const CompilerOptions::Private *CompilerOptions = nullptr;


        //!
		TSharedPtr<FErrorLog> ErrorLog;

        //!
        struct FConditionGenerationKey
        {
            int32 tagOrSurfIndex = 0;
			TSet<int32> posSurf;
            TSet<int32> negSurf;
            TSet<int32> posTag;
            TSet<int32> negTag;

			friend FORCEINLINE uint32 GetTypeHash(const FConditionGenerationKey& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.tagOrSurfIndex));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.posSurf.Num()));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.negSurf.Num()));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.posTag.Num()));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.negTag.Num()));
				return KeyHash;
			}

			FORCEINLINE bool operator==(const FConditionGenerationKey& Other) const
			{
				if (tagOrSurfIndex != Other.tagOrSurfIndex) return false;
				if (!SetsEquals(posSurf, Other.posSurf)) return false;
				if (!SetsEquals(negSurf, Other.negSurf)) return false;
				if (!SetsEquals(posTag, Other.posTag)) return false;
				if (!SetsEquals(negTag, Other.negTag)) return false;
				return true;
			}
			
		};

        // List of surfaces that activate or deactivate every tag, or another surface that activates a tag in this set.
		TArray< TSet<int32> > SurfacesPerTag;
		TArray< TSet<int32> > TagsPerTag;

        TMap<FConditionGenerationKey,Ptr<ASTOp>> TagConditionGenerationCache;

        FUniqueOpPool OpPool;

        //!
        Ptr<ASTOp> GenerateTagCondition( int32 tagIndex,
                                         const TSet<int32>& posSurf,
                                         const TSet<int32>& negSurf,
                                         const TSet<int32>& posTag,
                                         const TSet<int32>& negTag );

        /** Generate Surface, Edit or Modifier condition.
    	 * @param Index Surface, Edit, Component or Modifier index.
    	 * @param PositiveTags function that given the Surface, Edit or Modifier index, returns its positive tags.
    	 * @param NegativeTags function that given the Surface, Edit or Modifier, returns its negative tags.
    	 * @param posSurf already visited Surfaces, Edits, or Modifiers that participate positively in the condition.
    	 * @param negSurf already visited Surfaces, Edits, or Modifiers that participate negatively in the condition.
      	 * @param posSurf Tags that already belong to the condition (positively).
		 * @param negSurf Tags that already belong to the condition (negatively). */
        Ptr<ASTOp> GenerateDataCodition(int32 Index,
									        const TArray<FString>& PositiveTags,
											const TArray<FString>& NegativeTags,
                                            const TSet<int32>& posSurf,
                                            const TSet<int32>& negSurf,
                                            const TSet<int32>& posTag,
                                            const TSet<int32>& negTag);
    };

}

