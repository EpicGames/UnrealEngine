// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeGenerator.h"

#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSkeletalMeshNew.h"
#include "MuT/ASTOpSkeletalMeshMaterialModify.h"
#include "MuT/ASTOpSkeletalMeshMorph.h"
#include "MuT/ASTOpSkeletalMeshReshape.h"
#include "MuT/ASTOpSkeletalMeshMerge.h"
#include "MuT/ASTOpSkeletalMeshClipWithMesh.h"
#include "MuT/ASTOpSkeletalMeshConvert.h"
#include "MuT/ASTOpSkeletalMeshTransform.h"
#include "MuT/ASTOpSkeletalMeshTransformWithBone.h"
#include "MuT/NodeSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshClipWithSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshConvert.h"
#include "MuT/NodeSkeletalMeshMorph.h"
#include "MuT/NodeSkeletalMeshReshape.h"
#include "MuT/NodeSkeletalMeshModify.h"
#include "MuT/NodeSkeletalMeshSwitch.h"
#include "MuT/NodeSkeletalMeshVariation.h"
#include "MuT/NodeSkeletalMeshMerge.h"
#include "MuT/NodeSkeletalMeshTransform.h"
#include "MuT/NodeSkeletalMeshTransformWithBone.h"


namespace UE::Mutable::Private
{
    void CodeGenerator::GenerateSkeletalMesh(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const Ptr<const NodeSkeletalMesh>& Node)
	{
    	if (!Node)
    	{
    		Result = FSkeletalMeshGenerationResult();
    		return;
    	}

    	// See if it was already generated
    	FGeneratedCacheKey Key;
    	Key.Node = Node;
    	Key.Options = Options.GenericOptions;
	    {
    		UE::TUniqueLock Lock(GeneratedSkeletalMeshes.Mutex);
    		FGeneratedSkeletalMeshesMap::ValueType* Found = GeneratedSkeletalMeshes.Map.Find(Key);
    		if (Found)
    		{
    			Result = *Found;
    			return;
    		}
	    }

	    // Generate for each different type of node
		switch (Node->GetType()->Type)
		{
	    case Node::EType::SkeletalMeshNew:
			GenerateSkeletalMesh_New(Result, Options, static_cast<const NodeSkeletalMeshNew*>(Node.get()));
			break;

		case Node::EType::SkeletalMeshConvert:
			GenerateSkeletalMesh_Convert(Result, Options, static_cast<const NodeSkeletalMeshConvert*>(Node.get()));
			break;
			
		case Node::EType::SkeletalMeshMorph:
			GenerateSkeletalMesh_Morph(Result, Options, static_cast<const NodeSkeletalMeshMorph*>(Node.get()));
			break;

		case Node::EType::SkeletalMeshModify:
			GenerateSkeletalMesh_Modify(Result, Options, static_cast<const NodeSkeletalMeshModify*>(Node.get()));
			break;

	    case Node::EType::SkeletalMeshSwitch:
			GenerateSkeletalMesh_Switch(Result, Options, static_cast<const NodeSkeletalMeshSwitch*>(Node.get()));
			break;

	    case Node::EType::SkeletalMeshVariation:
			GenerateSkeletalMesh_Variation(Result, Options, static_cast<const NodeSkeletalMeshVariation*>(Node.get()));
			break;

		case Node::EType::SkeletalMeshMerge:
			GenerateSkeletalMesh_Merge(Result, Options, static_cast<const NodeSkeletalMeshMerge*>(Node.get()));
			break;

	    case Node::EType::SkeletalMeshTransform:
			GenerateSkeletalMesh_Transform(Result, Options, static_cast<const NodeSkeletalMeshTransform*>(Node.get()));
			break;
			
	    case Node::EType::SkeletalMeshTransformWithBone:
			GenerateSkeletalMesh_TransformWithBone(Result, Options, static_cast<const NodeSkeletalMeshTransformWithBone*>(Node.get()));
			break;

		case Node::EType::SkeletalMeshClipWithSkeletalMesh:
			GenerateSkeletalMesh_ClipWithSkeletalMesh(Result, Options, static_cast<const NodeSkeletalMeshClipWithSkeletalMesh*>(Node.get()));
			break;
			
		case Node::EType::SkeletalMeshReshape:
			GenerateSkeletalMesh_Reshape(Result, Options, static_cast<const NodeSkeletalMeshReshape*>(Node.get()));
			break;

		default:
			unimplemented();
		}
    	
    	// Cache the result
	    {
    		UE::TUniqueLock Lock(GeneratedSkeletalMeshes.Mutex);
    		GeneratedSkeletalMeshes.Map.Add(Key, Result);
	    }
	}


    void CodeGenerator::GenerateSkeletalMesh_New(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshNew* Node)
    {
    	const uint8 NumLODs = FMath::Min(Options.NumLODs, static_cast<uint8>(Node->LODs.Num()));
    	
    	// Merge the surfaces that do share LOD and Node.
    	TArray<TArray<int32>> MergedSurfacesPerLOD;
    	MergedSurfacesPerLOD.SetNum(NumLODs);
    	
    	bool bHasSurfaces = false;
    	for (FirstPassGenerator::FSkeletalMeshData& SkeletalMeshData : FirstPass.SkeletalMeshes)
    	{
    		if (SkeletalMeshData.SkeletalMeshNode == Node)
    		{
				check(SkeletalMeshData.SurfacesPerLOD.Num() <= MAX_uint8);
				const uint8 MinNumLODs = FMath::Min(NumLODs, static_cast<uint8>(SkeletalMeshData.SurfacesPerLOD.Num()));
    			for (uint8 LODIndex = Options.FirstLODAvailable; LODIndex < MinNumLODs; ++LODIndex)
    			{
    				TArray<int32>& MeshDataSurfacesPerLOD = SkeletalMeshData.SurfacesPerLOD[LODIndex];
    				bHasSurfaces |= !MeshDataSurfacesPerLOD.IsEmpty();
    				
    				MergedSurfacesPerLOD[LODIndex].Append(MeshDataSurfacesPerLOD);
    			}
    		}
    	}
    	
    	if (!bHasSurfaces)
    	{
    		return;
    	}
    	
    	TArray<FLODTask> LODTasks;
    	LODTasks.SetNum(NumLODs);
    	
    	// Launch tasks for each lod, making sure each LOD depends on the previous one.
    	FLODTask LastLODTask;
    	for (int32 LODIndex = Options.FirstLODAvailable; LODIndex < NumLODs; ++LODIndex)
    	{
			const NodeLOD* LODNode = Node->LODs[LODIndex].get();
			if (LODNode && MergedSurfacesPerLOD.IsValidIndex(LODIndex))
    		{
				FLODGenerationOptions LODOptions(Options.GenericOptions, LODIndex, Options.Component, MergedSurfacesPerLOD[LODIndex]);
				
    			LastLODTask = GenerateLOD(LODOptions, LODNode, LastLODTask);
    			LODTasks[LODIndex] = LastLODTask;
    		}
    		else
    		{
    			LODTasks[LODIndex] = Tasks::MakeCompletedTask<FLODGenerationResult>();
    		}
    	}

		Ptr<ASTOpSkeletalMeshNew> SkeletalMeshNewOp = new ASTOpSkeletalMeshNew();
    	
    	for (int32 LODIndex = 0; LODIndex < Options.FirstLODAvailable; ++LODIndex)
    	{
    		SkeletalMeshNewOp->LODs.Emplace(SkeletalMeshNewOp, nullptr);
    	}
    	
    	for (int32 LODIndex = Options.FirstLODAvailable; LODIndex < NumLODs; ++LODIndex)
    	{
    		FLODTask& LODTask = LODTasks[LODIndex];

    		WaitTask(LODTask);
			
    		FLODGenerationResult LODResult = LODTask.GetResult();
			SkeletalMeshNewOp->LODs.Emplace(SkeletalMeshNewOp, LODResult.LODOp);

    		for (const FLODGenerationResult::FMaterialSlot& MaterialSlot : LODResult.MaterialSlots)
    		{
				const int32 MaterialSlotIndex = SkeletalMeshNewOp->MaterialSlotIds.Find(MaterialSlot.Id);
				if (MaterialSlotIndex == INDEX_NONE)
    			{
					SkeletalMeshNewOp->MaterialSlotIds.Add(MaterialSlot.Id);
					SkeletalMeshNewOp->MaterialSlotMaterials.Add(ASTChild(SkeletalMeshNewOp, MaterialSlot.Material));
					SkeletalMeshNewOp->MaterialSlotNames.Add(MaterialSlot.SlotName);
				}

				// Hack. Due to current surface generation flow, surface switch cases are split in multiple entries with the same
				// MaterialSlot.Id. Reconstruct the op chain to avoid missing materials.
				else if (SkeletalMeshNewOp->MaterialSlotMaterials.IsValidIndex(MaterialSlotIndex) &&
					SkeletalMeshNewOp->MaterialSlotMaterials[MaterialSlotIndex].child() != MaterialSlot.Material)
				{
					if (MaterialSlot.Material)
					{
						if (MaterialSlot.Material->GetOpType() == EOpType::MI_CONDITIONAL)
						{
							ASTOp::MapChildFunc Identity = [](const Ptr<ASTOp>& o) {return o; };

							Ptr<ASTOpConditional> NewConditional = static_cast<ASTOpConditional*>(MaterialSlot.Material->Clone(Identity).get());
							NewConditional->no = SkeletalMeshNewOp->MaterialSlotMaterials[MaterialSlotIndex].child();

							SkeletalMeshNewOp->MaterialSlotMaterials[MaterialSlotIndex] = NewConditional;
						}
						else
						{
							SkeletalMeshNewOp->MaterialSlotMaterials[MaterialSlotIndex] = MaterialSlot.Material;
						}
					}
    			}
    		}
    	}

		Result.SkeletalMeshOp = SkeletalMeshNewOp;
    }


    void CodeGenerator::GenerateSkeletalMesh_Convert(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshConvert* Node)
    {
		Ptr<ASTOpSkeletalMeshConvert> SkeletalMeshConvertOp = new ASTOpSkeletalMeshConvert();
		
		FSkeletalMeshObjectGenerationResult SkeletalMeshObjectResult;
		GenerateSkeletalMeshObject(SkeletalMeshObjectResult, Options, Node->SkeletalMesh);
		SkeletalMeshConvertOp->SkeletalMeshObject = SkeletalMeshObjectResult.SkeletalMeshOp;
		SkeletalMeshConvertOp->ConversionFlags = Node->ConversionFlags;

		OutResult.SkeletalMeshOp = SkeletalMeshConvertOp;
    }

	
	void CodeGenerator::GenerateSkeletalMesh_Morph(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshMorph* Node)
	{
		Ptr<ASTOpSkeletalMeshMorph> SkeletalMeshMorphOp = new ASTOpSkeletalMeshMorph();
		
		FSkeletalMeshGenerationResult SkeletalMeshGenerationResult;
		GenerateSkeletalMesh(SkeletalMeshGenerationResult, Options, Node->Base);
		SkeletalMeshMorphOp->Base = SkeletalMeshGenerationResult.SkeletalMeshOp;
		
		FScalarGenerationResult ScalarGenerationResult;
		GenerateScalar(ScalarGenerationResult, Options.GenericOptions, Node->Factor);
		SkeletalMeshMorphOp->Factor = ScalarGenerationResult.op;
	
		SkeletalMeshMorphOp->MorphName = Node->Name;
	
		Result.SkeletalMeshOp = SkeletalMeshMorphOp;
	}

	void CodeGenerator::GenerateSkeletalMesh_Reshape(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshReshape* Node)
	{
		Ptr<ASTOpSkeletalMeshReshape> SkeletalMeshReshapeOp = new ASTOpSkeletalMeshReshape();

		FSkeletalMeshGenerationResult SkeletalMeshGenerationResultBase;
		GenerateSkeletalMesh(SkeletalMeshGenerationResultBase, Options, Node->Base);
		SkeletalMeshReshapeOp->Base = SkeletalMeshGenerationResultBase.SkeletalMeshOp;

		FSkeletalMeshGenerationResult SkeletalMeshGenerationResultBaseShape;
		GenerateSkeletalMesh(SkeletalMeshGenerationResultBaseShape, Options, Node->BaseShape);
		SkeletalMeshReshapeOp->BaseShape = SkeletalMeshGenerationResultBaseShape.SkeletalMeshOp;

		FSkeletalMeshGenerationResult SkeletalMeshGenerationResultTargetShape;
		GenerateSkeletalMesh(SkeletalMeshGenerationResultTargetShape, Options, Node->TargetShape);
		SkeletalMeshReshapeOp->TargetShape = SkeletalMeshGenerationResultTargetShape.SkeletalMeshOp;

		SkeletalMeshReshapeOp->bReshapeVertices = Node->bReshapeVertices;
		SkeletalMeshReshapeOp->bRecomputeNormals = Node->bRecomputeNormals;
		SkeletalMeshReshapeOp->bApplyLaplacian = Node->bApplyLaplacian;
		SkeletalMeshReshapeOp->bReshapeSkeleton = Node->bReshapeSkeleton;
		SkeletalMeshReshapeOp->bReshapePhysicsVolumes = Node->bReshapePhysicsVolumes;
		SkeletalMeshReshapeOp->bReshapeSkeletonInvertSelection = Node->bReshapeSkeletonInvertSelection;
		SkeletalMeshReshapeOp->bReshapePhysicsVolumesInvertSelection = Node->bReshapePhysicsVolumesInvertSelection;

		SkeletalMeshReshapeOp->RChannelUsage = Node->ColorRChannelUsage;
		SkeletalMeshReshapeOp->GChannelUsage = Node->ColorGChannelUsage;
		SkeletalMeshReshapeOp->BChannelUsage = Node->ColorBChannelUsage;
		SkeletalMeshReshapeOp->AChannelUsage = Node->ColorAChannelUsage;

		SkeletalMeshReshapeOp->BonesToDeform = Node->BonesToDeform;
		SkeletalMeshReshapeOp->PhysicsToDeform = Node->PhysicsToDeform;

		Result.SkeletalMeshOp = SkeletalMeshReshapeOp;
	}

	void CodeGenerator::GenerateSkeletalMesh_Modify(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshModify* Node)
	{
		GenerateSkeletalMesh(Result, Options, Node->SkeletalMesh);
		
		for (const TPair<FName, Ptr<NodeMaterial>>& MaterialModify : Node->SlotMaterials)
		{
			Ptr<ASTOpSkeletalMeshMaterialModify> SkeletalMeshMaterialModifyOp = new ASTOpSkeletalMeshMaterialModify;
			
			SkeletalMeshMaterialModifyOp->MaterialSlotNameToModify = MaterialModify.Key;
			SkeletalMeshMaterialModifyOp->SkeletalMesh = Result.SkeletalMeshOp;	
			FMaterialGenerationResult MaterialResult;
			FMaterialGenerationOptions MaterialGenerationOptions
			{
				.GenericOptions = Options.GenericOptions
			};	
			GenerateMaterial(MaterialGenerationOptions, MaterialResult, MaterialModify.Value);
			SkeletalMeshMaterialModifyOp->NewMaterial = MaterialResult.op;
		
			Result.SkeletalMeshOp = SkeletalMeshMaterialModifyOp;
		}
	}


	void CodeGenerator::GenerateSkeletalMesh_Transform(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshTransform* Node)
	{
    	if (!Node->Source)
    	{
    		// This argument is required
    		ErrorLog->Add("Mesh transform base node is not set.", ELMT_ERROR, Node->GetMessageContext());
    		return;
    	}

    	FSkeletalMeshGenerationResult SkeletalMeshGenerationResult;
    	GenerateSkeletalMesh(SkeletalMeshGenerationResult, Options, Node->Source);
    	
    	Ptr<ASTOpSkeletalMeshTransform> TransformOp = new ASTOpSkeletalMeshTransform();
    	TransformOp->Source = SkeletalMeshGenerationResult.SkeletalMeshOp;
    	TransformOp->Matrix = Node->Transform;

    	OutResult.SkeletalMeshOp = TransformOp;
	}


	void CodeGenerator::GenerateSkeletalMesh_TransformWithBone(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshTransformWithBone* Node)
	{
    	Ptr<ASTOpSkeletalMeshTransformWithBone> Op = new ASTOpSkeletalMeshTransformWithBone;

    	Op->BoneName = Node->BoneName;
    	Op->ThresholdFactor = Node->ThresholdFactor;
    	
    	FSkeletalMeshGenerationResult SkeletalMeshResult;
    	GenerateSkeletalMesh(SkeletalMeshResult, Options, Node->Source);
    	Op->SourceSkeletalMesh = SkeletalMeshResult.SkeletalMeshOp;

    	FMatrixGenerationResult MatrixResult;
    	GenerateMatrix(MatrixResult, Options.GenericOptions, Node->MatrixNode);
    	Op->Matrix = MatrixResult.op;
    	
    	Result.SkeletalMeshOp = Op;
	}


	void CodeGenerator::GenerateSkeletalMesh_ClipWithSkeletalMesh(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions& Options,
		const NodeSkeletalMeshClipWithSkeletalMesh* Node)
	{
		Ptr<ASTOpSkeletalMeshClipWithSkeletalMesh> SkeletalMeshClipOp = new ASTOpSkeletalMeshClipWithSkeletalMesh();

		FSkeletalMeshGenerationResult SourceGenerationResult;
		GenerateSkeletalMesh(SourceGenerationResult, Options, Node->SourceSkeletalMesh);
		SkeletalMeshClipOp->Source = SourceGenerationResult.SkeletalMeshOp;
    	
		FSkeletalMeshGenerationResult ClipGenerationResult;
		GenerateSkeletalMesh(ClipGenerationResult, Options, Node->ClipSkeletalMesh);
		SkeletalMeshClipOp->Clip = ClipGenerationResult.SkeletalMeshOp;

		SkeletalMeshClipOp->FaceCullStrategy = Node->FaceCullStrategy;

		OutResult.SkeletalMeshOp = SkeletalMeshClipOp;
	}


	void CodeGenerator::GenerateSkeletalMesh_Switch(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshSwitch* Node)
    {
    	MUTABLE_CPUPROFILER_SCOPE(NodeSkeletalmeshSwitch);
    	
    	if (Node->Options.Num() == 0)
    	{
    		OutResult.SkeletalMeshOp = nullptr;
    		return;
    	}

    	Ptr<ASTOpSwitch> SwitchOp = new ASTOpSwitch();
    	SwitchOp->Type = EOpType::SK_SWITCH;

    	// Variable value
    	if (Node->Parameter)
    	{
    		FScalarGenerationResult ParamResult;
    		GenerateScalar(ParamResult, Options.GenericOptions, Node->Parameter.get());
    		SwitchOp->Variable = ParamResult.op;
    	}
    	else
    	{
    		// This argument is required
    		SwitchOp->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Node->GetMessageContext() );
    	}

    	// Options
    	for ( int32 OptionIndex = 0; OptionIndex < Node->Options.Num(); ++OptionIndex)
    	{
    		Ptr<ASTOp> Branch = nullptr;

    		if (Node->Options[OptionIndex])
    		{
    			FSkeletalMeshGenerationResult BaseResult;
    			GenerateSkeletalMesh(BaseResult, Options, Node->Options[OptionIndex]);
    			Branch = BaseResult.SkeletalMeshOp;
    		}

    		SwitchOp->Cases.Emplace(OptionIndex, SwitchOp, Branch);
    	}

    	OutResult.SkeletalMeshOp = SwitchOp;
    }


 void CodeGenerator::GenerateSkeletalMesh_Variation(FSkeletalMeshGenerationResult& OutResult, const FSkeletalMeshGenerationOptions& Options,
	    const NodeSkeletalMeshVariation* Node)
    {
    	MUTABLE_CPUPROFILER_SCOPE(NodeSkeletalmeshVariation);
    	
    	bool bFirstValidConnectionFound = false;

        // Default case
    	FSkeletalMeshGenerationResult DefaultSkeletalMeshResult;
        if (Node->DefaultSkeletalMesh)
        {
        	GenerateSkeletalMesh(DefaultSkeletalMeshResult, Options, Node->DefaultSkeletalMesh);
			bFirstValidConnectionFound = true;
        }

        // Process variations in reverse order, since conditionals are built bottom-up.
		TArray<FSkeletalMeshGenerationResult> ReverseVariations;
		TArray<int32> ReverseVariationsIndices;
		ReverseVariations.Reserve(Node->Variations.Num());
		ReverseVariationsIndices.Reserve(Node->Variations.Num());

        for ( int32 VariationIndex = Node->Variations.Num()-1; VariationIndex >= 0; --VariationIndex)
        {
            int32 TagIndex = -1;
            const FString& Tag = Node->Variations[VariationIndex].Tag;
            for ( int32 i = 0; i < FirstPass.Tags.Num(); ++i )
            {
                if ( FirstPass.Tags[i].Tag==Tag)
                {
                    TagIndex = i;
                }
            }

            if ( TagIndex < 0 )
            {
                ErrorLog->Add( 
					FString::Printf(TEXT("Unknown tag found in skeletal mesh variation [%s]."), *Tag),
					ELMT_WARNING,
					Node->GetMessageContext(),
					ELMSB_UNKNOWN_TAG
				);
                continue;
            }

            Ptr<ASTOp> VariationMeshOp;
        	FSkeletalMeshGenerationResult VariationSkeletalMeshResult;
            if (const Ptr<NodeSkeletalMesh> VariationSkeletalMesh = Node->Variations[VariationIndex].SkeletalMesh)
            {         
            	GenerateSkeletalMesh(VariationSkeletalMeshResult, Options, VariationSkeletalMesh );
            	
                if (!bFirstValidConnectionFound)
                {
					bFirstValidConnectionFound = true;
				}
            }
		
        	ReverseVariations.Add(VariationSkeletalMeshResult);
			ReverseVariationsIndices.Add(TagIndex);
		}
    	

	    {
    		Ptr<ASTOp> CurrentMeshOp;
    		
    		// Default case
    		if (Node->DefaultSkeletalMesh)
    		{
    			OutResult = DefaultSkeletalMeshResult;
    			CurrentMeshOp = OutResult.SkeletalMeshOp;
    		}

    		// Variations
    		for (int32 ReverseVariationIndex = 0; ReverseVariationIndex<ReverseVariations.Num(); ++ReverseVariationIndex)
    		{
    			const int32 TagIndex = ReverseVariationsIndices[ReverseVariationIndex];

			    const FSkeletalMeshGenerationResult& VariationResult = ReverseVariations[ReverseVariationIndex];

    			Ptr<ASTOpConditional> Conditional = new ASTOpConditional;
    			Conditional->type = EOpType::SK_CONDITIONAL;
    			Conditional->no = CurrentMeshOp;
    			Conditional->yes = VariationResult.SkeletalMeshOp;
    			Conditional->condition = FirstPass.Tags[TagIndex].GenericCondition;

    			CurrentMeshOp = Conditional;
    		}

    		OutResult.SkeletalMeshOp = CurrentMeshOp;
	    }
    	
    }

    

	void CodeGenerator::GenerateSkeletalMesh_Merge(FSkeletalMeshGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshMerge* Node)
	{
		Ptr<ASTOp> Base;
		Ptr<ASTOp> Added;

		if (Node->BaseSkeletalMesh)
		{
			FSkeletalMeshGenerationResult BaseResult;
			GenerateSkeletalMesh(BaseResult, Options, Node->BaseSkeletalMesh);
			Base = BaseResult.SkeletalMeshOp;
		}

		if (Node->ToAddSkeletalMesh)
		{
			FSkeletalMeshGenerationResult AddedResult;
			GenerateSkeletalMesh(AddedResult, Options, Node->ToAddSkeletalMesh);
			Added = AddedResult.SkeletalMeshOp;
		}

		if (Base && Added)
		{
			Ptr<ASTOpSkeletalMeshMerge> SKMergeOp = new ASTOpSkeletalMeshMerge();
			SKMergeOp->BaseMesh = Base;
			SKMergeOp->AddedMesh = Added;
			Result.SkeletalMeshOp = SKMergeOp;
		}
		else
		{
			Result.SkeletalMeshOp = Base ? Base : Added;
		}
	}

}
