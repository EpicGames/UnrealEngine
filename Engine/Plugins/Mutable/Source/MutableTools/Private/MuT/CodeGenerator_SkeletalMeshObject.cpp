// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpConditional.h"
#include "ASTOpSkeletalMeshMerge.h"
#include "ASTOpSkeletalMeshObjectConvert.h"
#include "CodeGenerator.h"

#include "ASTOpSkeletalMeshNew.h"
#include "MuT/NodeSkeletalMeshConvert.h"
#include "MuT/NodeSkeletalMeshObjectConvert.h"
#include "MuT/NodeSkeletalMeshObjectParameter.h"
#include "MuT/NodeSkeletalMeshObjectSwitch.h"


namespace UE::Mutable::Private
{
    void CodeGenerator::GenerateSkeletalMeshObject(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const Ptr<const NodeSkeletalMeshObject>& Node)
    {
    	if (!Node)
    	{
    		Result = FSkeletalMeshObjectGenerationResult();
    		return;
    	}

    	// See if it was already generated
    	FGeneratedCacheKey Key;
    	Key.Node = Node;
    	Key.Options = Options.GenericOptions;
	    {
    		UE::TUniqueLock Lock(GeneratedSkeletalMeshObjects.Mutex);
    		FGeneratedSkeletalMeshObjectsMap::ValueType* Found = GeneratedSkeletalMeshObjects.Map.Find(Key);
    		if (Found)
    		{
    			Result = *Found;
    			return;
    		}
	    }

    	// Generate for each different type of node
    	switch (Node->GetType()->Type)
    	{
	    case Node::EType::SkeletalMeshObjectConvert:
    		GenerateSkeletalMeshObject_Convert(Result, Options, static_cast<const NodeSkeletalMeshObjectConvert*>(Node.get()));
    		break;

	    case Node::EType::SkeletalMeshObjectParameter:
    		GenerateSkeletalMeshObject_Parameter(Result, Options, static_cast<const NodeSkeletalMeshObjectParameter*>(Node.get()));
    		break;

		case Node::EType::SkeletalMeshObjectSwitch:
			GenerateSkeletalMeshObject_Switch(Result, Options, static_cast<const NodeSkeletalMeshObjectSwitch*>(Node.get()));
			break;
			
	    default:
    		unimplemented();
    	}
    	
    	// Cache the result
	    {
    		UE::TUniqueLock Lock(GeneratedSkeletalMeshObjects.Mutex);
    		GeneratedSkeletalMeshObjects.Map.Add(Key, Result);
	    }
    }


    void CodeGenerator::GenerateSkeletalMeshObject_Convert(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshObjectConvert* Node)
    {
    	Ptr<ASTOpSkeletalMeshObjectConvert> Op = new ASTOpSkeletalMeshObjectConvert();
    	
    	FSkeletalMeshGenerationOptions SkeletalMeshOptions(Options);
    	SkeletalMeshOptions.FirstLODAvailable = Node->FirstLODAvailable;
    	SkeletalMeshOptions.NumLODs = Node->NumLODs;

    	Ptr<ASTOp> LastSkeletalMeshOp;

    	{
    		FSkeletalMeshGenerationResult SkeletalMeshResult;
    		GenerateSkeletalMesh(SkeletalMeshResult, SkeletalMeshOptions, Node->SkeletalMesh);
    		
    		LastSkeletalMeshOp = SkeletalMeshResult.SkeletalMeshOp;
    	}
    	
    	for (const FirstPassGenerator::FSkeletalMeshData& SkeletalMesh : FirstPass.SkeletalMeshes)
    	{
    		if (SkeletalMesh.SkeletalMeshNode == Node->SkeletalMesh.get())
    		{
    			continue;
    		}
    		
    		if (SkeletalMesh.ParentSkeletalMeshName == Node->Name && SkeletalMesh.bIsEntryPoint)
    		{
    			const Ptr<ASTOp> BaseSkeletalMesh = LastSkeletalMeshOp;

    			FSkeletalMeshGenerationResult SkeletalMeshResult;
    			GenerateSkeletalMesh(SkeletalMeshResult, SkeletalMeshOptions, SkeletalMesh.SkeletalMeshNode);
    			
    			if (SkeletalMeshResult.SkeletalMeshOp)
    			{
    				Ptr<ASTOpSkeletalMeshMerge> MergeOp = new ASTOpSkeletalMeshMerge();
    				MergeOp->BaseMesh = BaseSkeletalMesh;
    				MergeOp->AddedMesh = SkeletalMeshResult.SkeletalMeshOp;
    				
    				if (SkeletalMesh.ObjectCondition)
    				{
    					Ptr<ASTOpConditional> Conditional = new ASTOpConditional();
    					Conditional->type = EOpType::SK_CONDITIONAL;
    					Conditional->condition = SkeletalMesh.ObjectCondition;
    					Conditional->no = BaseSkeletalMesh;
    					Conditional->yes = MergeOp;
    					
    					LastSkeletalMeshOp = Conditional;
    				}
    				else
    				{
    					LastSkeletalMeshOp = MergeOp;
    				}
    			}
    		}
    	}
    	
    	Op->SkeletalMesh = LastSkeletalMeshOp;
    	
    	Op->Name = Node->Name;
    	Op->NumLODs = Node->NumLODs;
    	Op->FirstLODAvailable = Node->FirstLODAvailable;
    	Op->FirstLODResident = Node->FirstLODResident;
    	
    	Op->MinLODs = Node->MinLODs;
    	Op->MinQualityLevelLODs = Node->MinQualityLevelLODs;
    	
    	Op->LODInfos = Node->LODInfos;
    	
    	Result.SkeletalMeshOp = Op;
    }


    void CodeGenerator::GenerateSkeletalMeshObject_Parameter(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshObjectParameter* Node)
	{
		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				op = new ASTOpParameter();
				op->Type = EOpType::SK_PARAMETER;

				op->Parameter.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->UID, op->Parameter.UID);
				check(bParseOk);
				op->Parameter.Type = EParameterType::SkeletalMesh;
				op->Parameter.DefaultValue.Set<FParamSkeletalMeshType>(nullptr);

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Node->Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options.GenericOptions, Node->Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		Result.SkeletalMeshOp = op;
	}

	void CodeGenerator::GenerateSkeletalMeshObject_Switch(FSkeletalMeshObjectGenerationResult& Result, const FSkeletalMeshGenerationOptions& Options, const NodeSkeletalMeshObjectSwitch* Node)
	{
		MUTABLE_CPUPROFILER_SCOPE(NodeSkeletalmeshSwitch);

		if (Node->Options.Num() == 0)
		{
			Result.SkeletalMeshOp = nullptr;
			return;
		}

		Ptr<ASTOpSwitch> SwitchOp = new ASTOpSwitch();
		SwitchOp->Type = EOpType::SK_SWITCH;

		// Variable value
		if (Node->Parameter)
		{
			FScalarGenerationResult ParamResult;
			GenerateScalar(ParamResult, Options.GenericOptions, Node->Parameter.get());
			SwitchOp->Variable = ParamResult.op;
		}
		else
		{
			// This argument is required
			SwitchOp->Variable = GenerateMissingScalarCode(TEXT("Switch variable"), 0.0f, Node->GetMessageContext());
		}

		// Options
		for (int32 OptionIndex = 0; OptionIndex < Node->Options.Num(); ++OptionIndex)
		{
			Ptr<ASTOp> Branch = nullptr;

			if (Node->Options[OptionIndex])
			{
				FSkeletalMeshObjectGenerationResult BaseResult;
				GenerateSkeletalMeshObject(BaseResult, Options, Node->Options[OptionIndex]);
				Branch = BaseResult.SkeletalMeshOp;
			}

			SwitchOp->Cases.Emplace(OptionIndex, SwitchOp, Branch);
		}

		Result.SkeletalMeshOp = SwitchOp;
	}
}
