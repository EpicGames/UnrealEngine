// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuR/Types.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConstantString.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/CodeGenerator.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeString.h"
#include "MuT/NodeStringConstant.h"
#include "MuT/NodeStringParameter.h"

namespace UE::Mutable::Private
{

	void CodeGenerator::GenerateString(FStringGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeString>& Untyped)
	{
		if (!Untyped)
		{
			result = FStringGenerationResult();
			return;
		}

		// See if it was already generated
		FGeneratedCacheKey Key;
		Key.Node = Untyped;
		Key.Options = Options;
		{
			UE::TUniqueLock Lock(GeneratedStrings.Mutex);
			FGeneratedStringsMap::ValueType* it = GeneratedStrings.Map.Find(Key);
			if (it)
			{
				result = *it;
				return;
			}
		}

		// Generate for each different type of node
		if (Untyped->GetType()==NodeStringConstant::GetStaticType())
		{
			GenerateString_Constant(result, Options, static_cast<const NodeStringConstant*>(Untyped.get()));
		}
		else if (Untyped->GetType() == NodeStringParameter::GetStaticType())
		{
			GenerateString_Parameter(result, Options, static_cast<const NodeStringParameter*>(Untyped.get()));
		}
		else
		{
			check(false);
		}

		// Cache the result
		{
			UE::TUniqueLock Lock(GeneratedStrings.Mutex);
			GeneratedStrings.Map.Add(Key, result);
		}
	}


	void CodeGenerator::GenerateString_Constant(FStringGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeStringConstant>& Node)
	{
		Ptr<ASTOpConstantString> op = new ASTOpConstantString();
		op->value = Node->Value;

		result.op = op;
	}


	void CodeGenerator::GenerateString_Parameter(FStringGenerationResult& result, const FGenericGenerationOptions& Options, const Ptr<const NodeStringParameter>& Node)
	{
		Ptr<ASTOpParameter> op;

		Ptr<ASTOpParameter>* Found = nullptr;
		{
			UE::TUniqueLock Lock(FirstPass.ParameterNodes.Mutex);
			Found = FirstPass.ParameterNodes.GenericParametersCache.Find(Node);
			if (!Found)
			{
				FParameterDesc param;
				param.Name = Node->Name;
				bool bParseOk = FGuid::Parse(Node->UID, param.UID);
				check(bParseOk);
				param.Type = EParameterType::Float;
				param.DefaultValue.Set<FParamStringType>(Node->DefaultValue);

				op = new ASTOpParameter();
				op->Type = EOpType::ST_PARAMETER;
				op->Parameter = param;

				FirstPass.ParameterNodes.GenericParametersCache.Add(Node, op);
			}
			else
			{
				op = *Found;
			}
		}

		if (!Found)
		{
			// Generate the code for the ranges
			for (int32 a = 0; a < Node->Ranges.Num(); ++a)
			{
				FRangeGenerationResult rangeResult;
				GenerateRange(rangeResult, Options, Node->Ranges[a]);
				op->Ranges.Emplace(op.get(), rangeResult.sizeOp, rangeResult.rangeName, rangeResult.rangeUID);
			}
		}

		result.op = op;
	}


}
