// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/CodeOptimiser.h"

#include "MuT/ASTOpAddSkeletalMesh.h"
#include "MuT/ASTOpLODNew.h"
#include "MuT/CodeOptimiserPrivate.h"
#include "MuT/ErrorLog.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpConstantBool.h"
#include "MuT/ASTOpConstantColor.h"
#include "MuT/ASTOpMeshApplyShape.h"
#include "MuT/ASTOpMeshBindShape.h"
#include "MuT/ASTOpMeshClipMorphPlane.h"
#include "MuT/ASTOpMeshApplyPose.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpReferenceResource.h"

#include "MuR/Model.h"
#include "MuR/System.h"
#include "MuR/Operations.h"
#include "MuR/OpMeshMerge.h"
#include "MuR/MutableRuntimeModule.h"

#include "Tasks/Task.h"
#include "Math/NumericLimits.h"

namespace UE::Mutable::Private
{
	bool DuplicatedDataRemoverAST(ASTOpList& Roots)
	{
		MUTABLE_CPUPROFILER_SCOPE(DuplicatedDataRemoverAST);

		bool bModified = false;

		TMap<FResourceEntry<FMesh>, Ptr<ASTOpConstantResource>> MeshOps;
		TMap<FResourceEntry<FImage>, Ptr<ASTOpConstantResource>> ImageOps;
		TMap<FResourceEntry<FLayout>, Ptr<ASTOpConstantResource>> LayoutOps;
		TMap<TPassthroughObjectPtr<UMaterialInterface>, Ptr<ASTOpConstantResource>> MaterialOps;

		ASTOp::Traverse_TopRandom_Unique_NonReentrant(Roots, [&](Ptr<ASTOp> Op)
		{
			const EOpType OperationType = Op->GetOpType();
			switch (OperationType)
			{
			case EOpType::ME_CONSTANT:
				{
					const Ptr<ASTOpConstantResource> OpResource = static_pointer_cast<ASTOpConstantResource>(Op);
    				bModified |= Deduplicate<FMesh>(MeshOps, OpResource);
    				break;
				}

			case EOpType::IM_CONSTANT:
				{
					const Ptr<ASTOpConstantResource> OpResource = static_pointer_cast<ASTOpConstantResource>(Op);
					bModified |= Deduplicate<FImage>(ImageOps, OpResource);
					break;
				}

			case EOpType::LA_CONSTANT:
				{
					const Ptr<ASTOpConstantResource> OpResource = static_pointer_cast<ASTOpConstantResource>(Op);
					bModified |= Deduplicate<FLayout>(LayoutOps, OpResource);
					break;
				}

			case EOpType::MI_CONSTANT:
				{
					const Ptr<ASTOpConstantResource> OpResource = static_pointer_cast<ASTOpConstantResource>(Op);
					bModified |= DeduplicateFMaterial(MaterialOps, OpResource);
					break;
				}
			}

			return true;
		});
		
		return bModified;
	}


	bool DuplicatedCodeRemoverAST( ASTOpList& roots )
	{
		MUTABLE_CPUPROFILER_SCOPE(DuplicatedCodeRemoverAST);

		bool bModified = false;

		struct FKeyFuncs : BaseKeyFuncs<Ptr<ASTOp>, Ptr<ASTOp>, false>
		{
			static KeyInitType GetSetKey(ElementInitType Element) { return Element; }
			static bool Matches(const Ptr<ASTOp>& lhs, const Ptr<ASTOp>& rhs) { return lhs == rhs || *lhs == *rhs; }
			static uint32 GetKeyHash(const Ptr<ASTOp>& Key) { return Key->Hash(); }
		};

		// Visited nodes, per type
		TSet<Ptr<ASTOp>, FKeyFuncs, TInlineSetAllocator<32>> Visited[int32(EOpType::COUNT)];

		ASTOp::Traverse_BottomUp_Unique_NonReentrant( roots, 
			[&bModified,&Visited, &roots]
			(Ptr<ASTOp>& n)
			{
				TSet<Ptr<ASTOp>, FKeyFuncs, TInlineSetAllocator<32>>& Container = Visited[(int32)n->GetOpType()];

				// Insert will tell us if it was already there
				bool bIsAlreadyInSet = false;
				Ptr<ASTOp>& Found = Container.FindOrAdd(n, &bIsAlreadyInSet);
				if( bIsAlreadyInSet)
				{
					// It wasn't inserted, so it was already there
					ASTOp::Replace(n, Found);

					// Is it one of the roots? Then we also need to update it
					for (Ptr<ASTOp>& Root : roots)
					{
						if (Root == n)
						{
							Root = Found;
						}
					}

					bModified = true;
				}
			});

		return bModified;
	}


	class FConstantTask 
	{
	public:

		// input
		Ptr<ASTOp> Source;
		int32 ImageCompressionQuality = 0;
		int32 OptimizationPass = 0;
		FReferencedMeshResourceFunc ReferencedMeshResourceProvider;
		FReferencedImageResourceFunc ReferencedImageResourceProvider;

		// Intermediate
		Ptr<ASTOp> SourceCloned;

		// Result
		Ptr<ASTOp> Result;

	public:

		FConstantTask( const Ptr<ASTOp>& InSource, const CompilerOptions::Private* InOptions, int32 InOptimizationPass )
		{
			OptimizationPass = InOptimizationPass;
			Source = InSource;
			ImageCompressionQuality = InOptions->ImageCompressionQuality;
			ReferencedMeshResourceProvider = InOptions->OptimisationOptions.ReferencedMeshResourceProvider;
			ReferencedImageResourceProvider = InOptions->OptimisationOptions.ReferencedImageResourceProvider;
		}

		void Run(FImageOperator ImOp)
		{
			MUTABLE_CPUPROFILER_SCOPE(ConstantTask_Run);

			// This runs in a worker thread

			EOpType type = SourceCloned->GetOpType();
			EDataType DataType = GetOpDataType(type);

			FSettings Settings;
			Settings.SetProfile( false );
			Settings.SetImageCompressionQuality( ImageCompressionQuality );
			TSharedPtr<FSystem> System = MakeShared<FSystem>(Settings);
			
			FSourceDataDescriptor SourceDataDescriptor;
			if (DataType == EDataType::Image || DataType == EDataType::Mesh)
			{
				SourceDataDescriptor = SourceCloned->GetSourceDataDescriptor();
				check(!SourceDataDescriptor.IsInvalid());
			}

			// Don't generate mips during linking here.
			FLinkerOptions LinkerOptions(ImOp);
			LinkerOptions.MinTextureResidentMipCount = 255;
			LinkerOptions.bSeparateImageMips = false;

			TSharedPtr<FModel> Model = MakeShared<FModel>();
			FOperation::ADDRESS at = ASTOp::FullLink(SourceCloned, Model->GetProgram(), &LinkerOptions);

			FProgram::FState state;
			state.Root = at;
			Model->GetProgram().States.Add(state);

			TSharedPtr<FParameters> LocalParams = FModel::NewParameters(Model);

			TSharedRef<FLiveInstance> TempLiveInstance = System->NewBuildInstance(Model, LocalParams,  nullptr);
			TempLiveInstance->PixelFormatOverride = ImOp.FormatImageOverride;

			// Calculate the value and replace this op by a constant
			switch( DataType )
			{
			case EDataType::Mesh:
			{
				MUTABLE_CPUPROFILER_SCOPE(ConstantMesh);

				TManagedPtr<const FMesh> MeshBuild = System->BuildMesh(TempLiveInstance, at, EMeshContentFlags::AllFlags);

				if (MeshBuild)
				{
					UE::Mutable::Private::Ptr<ASTOpConstantResource> ConstantOp = new ASTOpConstantResource();
					ConstantOp->SourceDataDescriptor = SourceDataDescriptor;
					ConstantOp->Type = EOpType::ME_CONSTANT;
					ConstantOp->SetValue(MeshBuild);
					Result = ConstantOp;
				}
				break;
			}

			case EDataType::Image:
			{
				MUTABLE_CPUPROFILER_SCOPE(ConstantImage);

				TManagedPtr<const FImage> ImageResult = System->BuildImage(TempLiveInstance, at, 0);

				if (ImageResult)
				{
					UE::Mutable::Private::Ptr<ASTOpConstantResource> ConstantOp = new ASTOpConstantResource();
					ConstantOp->SourceDataDescriptor = SourceDataDescriptor;
					ConstantOp->Type = EOpType::IM_CONSTANT;
					ConstantOp->SetValue(ImageResult);
					Result = ConstantOp;
				}
				break;
			}

			case EDataType::Layout:
			{
				MUTABLE_CPUPROFILER_SCOPE(ConstantLayout);

				TManagedPtr<const FLayout> LayoutResult = System->BuildLayout(TempLiveInstance, at);

				if (LayoutResult)
				{
					UE::Mutable::Private::Ptr<ASTOpConstantResource> ConstantOp = new ASTOpConstantResource();
					ConstantOp->Type = EOpType::LA_CONSTANT;
					ConstantOp->SetValue(LayoutResult);
					Result = ConstantOp;
				}
				break;
			}

			case EDataType::Bool:
			{
				MUTABLE_CPUPROFILER_SCOPE(ConstantBool);

				bool value = System->BuildBool(TempLiveInstance, at);
				Result = new ASTOpConstantBool(value);
				break;
			}

			case EDataType::Color:
			{
				MUTABLE_CPUPROFILER_SCOPE(ConstantBool);

				FVector4f ResultColor(0, 0, 0, 0);
				ResultColor = System->BuildColor(TempLiveInstance, at);

				{
					UE::Mutable::Private::Ptr<ASTOpConstantColor> ConstantOp = new ASTOpConstantColor();
					ConstantOp->Value = ResultColor;
					Result = ConstantOp;
				}
				break;
			}

			case EDataType::Int:
			case EDataType::Scalar:
			case EDataType::String:
			case EDataType::Projector:
				// TODO
				break;
				
			case EDataType::LOD:
			case EDataType::SkeletalMesh:
				// TODO Constants not implemented.
				break;

			default:
				break;
			}
		}
	};


	bool ConstantGenerator( const CompilerOptions::Private* InOptions, Ptr<ASTOp>& Root, int32 Pass )
	{
		MUTABLE_CPUPROFILER_SCOPE(ConstantGenerator);

		// don't do this if constant optimization has been disabled, usually for debugging.
		if (!InOptions->OptimisationOptions.bConstReduction)
		{
			return false;
		}

		// Gather the roots of all constant operations
		struct FConstantSubgraph
		{
			Ptr<ASTOp> Root;
			UE::Tasks::FTaskEvent CompletedEvent;
		};
		TArray< FConstantSubgraph > ConstantSubgraphs;
		ConstantSubgraphs.Reserve(256);
		{
			MUTABLE_CPUPROFILER_SCOPE(ConstantGenerator_GenerateTasks);

			ASTOp::Traverse_BottomUp_Unique(Root,
				[&ConstantSubgraphs, Pass]
				(Ptr<ASTOp>& SubgraphRoot)
				{
					EOpType SubgraphType = SubgraphRoot->GetOpType();

					bool bGetFromChildren = false;

					bool bIsConstantSubgraph = true;
					switch (SubgraphType)
					{
					case EOpType::IM_REFERENCE:
					case EOpType::ME_REFERENCE:
						if (Pass < 2)
						{
							bIsConstantSubgraph = false;
						}
						else
						{
							const ASTOpReferenceResource* Typed = static_cast<const ASTOpReferenceResource*>(SubgraphRoot.get());
							bIsConstantSubgraph = Typed->bForceLoad;
						}
						break;
					case EOpType::BO_PARAMETER:
					case EOpType::NU_PARAMETER:
					case EOpType::SC_PARAMETER:
					case EOpType::CO_PARAMETER:
					case EOpType::PR_PARAMETER:
					case EOpType::IM_PARAMETER:
					case EOpType::SK_PARAMETER:
					case EOpType::MA_PARAMETER:
					case EOpType::MI_PARAMETER:
					case EOpType::IM_PARAMETER_FROM_MATERIAL:
					case EOpType::IS_PARAMETER:
						bIsConstantSubgraph = false;
						break;
					default:
						// Propagate from children
						SubgraphRoot->ForEachChild([&bIsConstantSubgraph](ASTChild& c)
							{
								if (c)
								{
									bIsConstantSubgraph = bIsConstantSubgraph && c->bIsConstantSubgraph;
								}
							});
						break;
					}
					SubgraphRoot->bIsConstantSubgraph = bIsConstantSubgraph;

					// We avoid generating constants for these operations, to avoid the memory explosion.
					// TODO: Make compiler options for some of them
					// TODO: Some of them are worth if the code below them is unique.
					bool bHasSpecialOpInSubgraph = false;
					switch (SubgraphType)
					{
					case EOpType::IM_BLANKLAYOUT:
					case EOpType::IM_COMPOSE:
					case EOpType::IM_MULTICOMPOSE:
					case EOpType::ME_MERGE:
					case EOpType::ME_CLIPWITHMESH:
					case EOpType::ME_CLIPMORPHPLANE:
					case EOpType::ME_APPLYPOSE:
					case EOpType::ME_REMOVEMASK:
					case EOpType::ME_PREPARELAYOUT:
					case EOpType::ME_ADDMETADATA:
					case EOpType::IM_PLAINCOLOR:
						bHasSpecialOpInSubgraph = true;
						break;

					case EOpType::IM_RASTERMESH:
					{
						const ASTOpImageRasterMesh* Raster = static_cast<const ASTOpImageRasterMesh*>(SubgraphRoot.get());
						// If this operation is only rastering the mesh UVs, reduce it to constant. Otherwise avoid reducing it
						// for the case of a constant projector of a large set of possible images. We don't want to generate all the
						// projected version of the images beforehand. TODO: Make it a compile-time option?
						bHasSpecialOpInSubgraph = Raster->image.child().get() != nullptr;
						break;
					}

					case EOpType::LA_FROMMESH:
					case EOpType::ME_EXTRACTLAYOUTBLOCK:
					case EOpType::ME_APPLYLAYOUT:
					{
						// We want to reduce this type of operation regardless of it having special ops below.
						bHasSpecialOpInSubgraph = false;
						break;
					}

					case EOpType::IM_REFERENCE:
						// If we are in a reference-resolution optimization phase, then the ops are not special.
						if (Pass < 2)
						{
							bHasSpecialOpInSubgraph = true;
						}
						else
						{
							const ASTOpReferenceResource* Typed = static_cast<const ASTOpReferenceResource*>(SubgraphRoot.get());
							bHasSpecialOpInSubgraph = !Typed->bForceLoad;
						}
						break;

					default:
						bGetFromChildren = true;
						break;
					}

					if (bGetFromChildren)
					{
						// Propagate from children
						SubgraphRoot->ForEachChild([&](ASTChild& c)
							{
								if (c)
								{
									bHasSpecialOpInSubgraph = bHasSpecialOpInSubgraph || c->bHasSpecialOpInSubgraph;
								}
							});
					}

					SubgraphRoot->bHasSpecialOpInSubgraph = bHasSpecialOpInSubgraph;

					bool bIsDataTypeThanCanTurnIntoConst = false;
					EDataType DataType = GetOpDataType(SubgraphType);
					switch (DataType)
					{
					case EDataType::Mesh:
					case EDataType::Image:
					case EDataType::Layout:
					case EDataType::Bool:
					case EDataType::Color:
						bIsDataTypeThanCanTurnIntoConst = true;
						break;
					default:
						break;
					}


					// See if it is worth generating this as constant
					// ---------------------------------------------
					bool bWorthGenerating = SubgraphRoot->bIsConstantSubgraph
						&& !SubgraphRoot->bHasSpecialOpInSubgraph
						&& !SubgraphRoot->IsConstantOp()
						&& bIsDataTypeThanCanTurnIntoConst;

					if (bWorthGenerating)						 
					{
						bool bCanBeGenerated = true;

						// Check source data incompatiblities: when generating constants don't mix data that has different source descriptors (tags and other properties).
						if (DataType == EDataType::Image || DataType == EDataType::Mesh)
						{
							FSourceDataDescriptor SourceDescriptor = SubgraphRoot->GetSourceDataDescriptor();
							if (SourceDescriptor.IsInvalid())
							{
								bCanBeGenerated = false;
							}
						}

						if (bCanBeGenerated)
						{
							ConstantSubgraphs.Add({ SubgraphRoot, UE::Tasks::FTaskEvent(TEXT("MutableConstantSubgraph")) });
						}
					}
				});
		}

		auto GetRequisites = [&ConstantSubgraphs](const Ptr<ASTOp>& SubgraphRoot, TArray< UE::Tasks::FTask, TInlineAllocator<8> >& OutRequisites)
		{
			MUTABLE_CPUPROFILER_SCOPE(ConstantGenerator_GetRequisites);

			TArray< Ptr<ASTOp> > ScanRoots;
			ScanRoots.Add(SubgraphRoot);
			ASTOp::Traverse_TopDown_Unique_Imprecise(ScanRoots, [&SubgraphRoot, &OutRequisites, &ConstantSubgraphs](Ptr<ASTOp>& ChildNode)
				{
					bool bRecurse = true;

					// Subgraph root?
					if (SubgraphRoot == ChildNode)
					{
						return bRecurse;
					}

					FConstantSubgraph* DependencyFound = ConstantSubgraphs.FindByPredicate([&ChildNode](const FConstantSubgraph& Candidate) { return Candidate.Root == ChildNode; });
					if (DependencyFound)
					{
						bRecurse = false;
						OutRequisites.Add(DependencyFound->CompletedEvent);
					}

					return bRecurse;
				});
		};


		// Launch the tasks.
		UE::Tasks::FTask LaunchTask = UE::Tasks::Launch(TEXT("ConstantGeneratorLaunchTasks"),
			[&ConstantSubgraphs, &GetRequisites, Pass, InOptions]()
			{
				MUTABLE_CPUPROFILER_SCOPE(ConstantGenerator_LaunchTasks);

				FImageOperator ImOp = FImageOperator::GetDefault(InOptions->ImageFormatFunc);

				// Traverse list of constants to generate. It is ordered in a bottom-up way.
				int32 SubgraphCount = ConstantSubgraphs.Num();
				for (int32 OrderIndex = 0; OrderIndex < SubgraphCount; ++OrderIndex)
				{
					int32 Index = SubgraphCount - 1 - OrderIndex;

					Ptr<ASTOp> SubgraphRoot = ConstantSubgraphs[Index].Root;
					UE::Tasks::FTaskEvent& SubgraphCompletionEvent = ConstantSubgraphs[Index].CompletedEvent;

					bool bIsReference = false;
					EOpType SubgraphType = SubgraphRoot->GetOpType();

					if (SubgraphType == EOpType::IM_REFERENCE
						||
						SubgraphType == EOpType::IM_CONSTANT)
					{
						PASSTHROUGH_ID ImageID = PASSTHROUGH_ID_INVALID;
						if (SubgraphType == EOpType::IM_REFERENCE)
						{
							bIsReference = true;
							const ASTOpReferenceResource* Typed = static_cast<const ASTOpReferenceResource*>(SubgraphRoot.get());
							ImageID = Typed->ID;
						}
						else if (SubgraphType == EOpType::IM_CONSTANT)
						{
							const ASTOpConstantResource* Typed = static_cast<const ASTOpConstantResource*>(SubgraphRoot.get());
							const FImage* Value = static_cast<const FImage*>(Typed->GetValue().Get());
							bIsReference = Value->IsReference();
							if (bIsReference)
							{
								ImageID = Value->PassthroughObject.GetId();
							}
						}

						// Instead of generating the constant we resolve the reference, which also replaces the ASTOp.
						if (bIsReference)
						{
							TSharedPtr< TManagedPtr<FImage> > ResolveImage = MakeShared<TManagedPtr<FImage>>();

							constexpr bool bRunImmediatlyIfPossible = false;
							UE::Tasks::FTask ReferenceCompletion = InOptions->OptimisationOptions.ReferencedImageResourceProvider(ImageID, ResolveImage, bRunImmediatlyIfPossible);

							UE::Tasks::FTask CompleteTask = UE::Tasks::Launch(TEXT("MutableResolveComplete"),
								[SubgraphRoot, InOptions, ResolveImage]()
								{
									Ptr<ASTOpConstantResource> ConstantOp;
									{
										MUTABLE_CPUPROFILER_SCOPE(MutableResolveComplete_CreateConstant);
										ConstantOp = new ASTOpConstantResource;
										ConstantOp->Type = EOpType::IM_CONSTANT;
										{
											MUTABLE_CPUPROFILER_SCOPE(GetSourceDataDescriptor);
											ConstantOp->SourceDataDescriptor = SubgraphRoot->GetSourceDataDescriptor();
										}
										ConstantOp->SetValue(*ResolveImage);
									}
									{
										MUTABLE_CPUPROFILER_SCOPE(MutableResolveComplete_Replace);
										ASTOp::Replace(SubgraphRoot, ConstantOp);
									}
								},
								ReferenceCompletion,
								LowLevelTasks::ETaskPriority::BackgroundNormal);

							SubgraphCompletionEvent.AddPrerequisites(CompleteTask);
						}

					}

					if (SubgraphType == EOpType::ME_CONSTANT)
					{
						PASSTHROUGH_ID MeshID = PASSTHROUGH_ID_INVALID;
						FString MeshMorph;
						
						const ASTOpConstantResource* Typed = static_cast<const ASTOpConstantResource*>(SubgraphRoot.get());
						const FMesh* Value = static_cast<const FMesh*>(Typed->GetValue().Get());
						bIsReference = Value->IsReference();
						if (bIsReference)
						{
							MeshID = Value->PassthroughObject.GetId();
							MeshMorph = Value->GetReferencedMorph();
						}

						// Instead of generating the constant we resolve the reference, which also replaces the ASTOp.
						if (bIsReference)
						{
							TSharedPtr< TManagedPtr<FMesh> > ResolveMesh = MakeShared<TManagedPtr<FMesh>>();

							constexpr bool bRunImmediatlyIfPossible = false;
							UE::Tasks::FTask ReferenceCompletion = InOptions->OptimisationOptions.ReferencedMeshResourceProvider(MeshID, MeshMorph, ResolveMesh, bRunImmediatlyIfPossible);

							UE::Tasks::FTask CompleteTask = UE::Tasks::Launch(TEXT("MutableResolveComplete"),
								[SubgraphRoot, InOptions, ResolveMesh]()
								{
									Ptr<ASTOpConstantResource> ConstantOp;
									{
										MUTABLE_CPUPROFILER_SCOPE(MutableResolveComplete_CreateConstant);
										ConstantOp = new ASTOpConstantResource;
										ConstantOp->Type = EOpType::ME_CONSTANT;
										{
											MUTABLE_CPUPROFILER_SCOPE(GetSourceDataDescriptor);
											ConstantOp->SourceDataDescriptor = SubgraphRoot->GetSourceDataDescriptor();
										}
										ConstantOp->SetValue(*ResolveMesh);
									}
									{
										MUTABLE_CPUPROFILER_SCOPE(MutableResolveComplete_Replace);
										ASTOp::Replace(SubgraphRoot, ConstantOp);
									}
								},
								ReferenceCompletion,
								LowLevelTasks::ETaskPriority::BackgroundNormal);

							SubgraphCompletionEvent.AddPrerequisites(CompleteTask);
						}
					}

					if (!bIsReference)
					{
						// Scan for requisites
						TArray< UE::Tasks::FTask, TInlineAllocator<8> > Requisites;
						GetRequisites(SubgraphRoot, Requisites);

						TUniquePtr<FConstantTask> Task(new FConstantTask(SubgraphRoot, InOptions, Pass));

						// Launch the preparation on the AST-modification pipe
						UE::Tasks::FTask CompleteTask = UE::Tasks::Launch(TEXT("MutableConstant"), [TaskPtr = MoveTemp(Task), ImOp]()
							{
								MUTABLE_CPUPROFILER_SCOPE(MutableConstantPrepare);

								// We need the clone because linking modifies ASTOp state and also to be safe for concurrency.
								TaskPtr->SourceCloned = ASTOp::DeepClone(TaskPtr->Source);

								TaskPtr->Run(ImOp);

								ASTOp::Replace(TaskPtr->Source, TaskPtr->Result);
								TaskPtr->Result = nullptr;
								TaskPtr->Source = nullptr;
							},
							Requisites,
							LowLevelTasks::ETaskPriority::BackgroundHigh);

						SubgraphCompletionEvent.AddPrerequisites(CompleteTask);
					}

					ConstantSubgraphs[Index].Root = nullptr;
					SubgraphCompletionEvent.Trigger();

					UE::Tasks::AddNested(SubgraphCompletionEvent);
				}

			});

		// Wait for pending tasks
		{
			MUTABLE_CPUPROFILER_SCOPE(Waiting);
			LaunchTask.Wait();
		}


		bool bSomethingModified = ConstantSubgraphs.Num() > 0;
		return bSomethingModified;
	}


	CodeOptimiser::CodeOptimiser(Ptr<CompilerOptions> InOptions, TArray<FStateCompilationData>& InStates )
		: States( InStates )
	{
		Options = InOptions;
	}


	void CodeOptimiser::FullOptimiseAST( ASTOpList& roots, int32 Pass )
	{
		bool bModified = true;
		int32 NumIterations = 0;
		while (bModified && (OptimizeIterationsLeft>0 || !NumIterations))
		{
			bool bModifiedInInnerLoop = true;
			while (bModifiedInInnerLoop && (OptimizeIterationsLeft>0 || !NumIterations))
			{
				--OptimizeIterationsLeft;
				++NumIterations;
				UE_LOGF(LogMutableCore, Verbose, "Main optimise iteration %d, left %d", NumIterations, OptimizeIterationsLeft);

				bModifiedInInnerLoop = false;

				// All kind of optimisations that depend on the meaning of each operation
				// \TODO: We are doing it for all states.
				UE_LOGF(LogMutableCore, Verbose, " - semantic optimiser");
				bModifiedInInnerLoop |= SemanticOptimiserAST(roots, Options->GetPrivate()->OptimisationOptions, Pass);
				ASTOp::LogHistogram(roots);

				UE_LOGF(LogMutableCore, Verbose, " - sink optimiser");
				bModifiedInInnerLoop |= SinkOptimiserAST(roots, Options->GetPrivate()->OptimisationOptions);
				ASTOp::LogHistogram(roots);

				// Image size operations are treated separately
				UE_LOGF(LogMutableCore, Verbose, " - size optimiser");
				bModifiedInInnerLoop |= SizeOptimiserAST(roots);
			}

			bModified = bModifiedInInnerLoop;

			UE_LOGF(LogMutableCore, Verbose, " - duplicated code remover");
			bModified |= DuplicatedCodeRemoverAST(roots);
			//UE_LOGF(LogMutableCore, Verbose, TEXT("(int) %ls : %ld"), "ast size", int64(ASTOp::CountNodes(roots)));

			ASTOp::LogHistogram(roots);

			UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
			bModified |= DuplicatedDataRemoverAST(roots);
			//UE_LOGF(LogMutableCore, Verbose, TEXT("(int) %ls : %ld"), "ast size", int64(ASTOp::CountNodes(roots)));

			ASTOp::LogHistogram(roots);

			// Generate constants
			bool bModifiedInConstants = false;
			for (Ptr<ASTOp>& Root : roots)
			{
				//UE_LOGF(LogMutableCore, Verbose, TEXT("(int) %ls : %ld"), "ast size", int64(ASTOp::CountNodes(roots)));
				UE_LOGF(LogMutableCore, Verbose, " - constant generator");

				// Constant subtree generation
				bModifiedInConstants |= ConstantGenerator(Options->GetPrivate(), Root, Pass);
			}

			ASTOp::LogHistogram(roots);

			if (bModifiedInConstants)
			{
				bModified = true;

				UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
				DuplicatedDataRemoverAST(roots);
			}

			//if (!bModified)
			{
				UE_LOGF(LogMutableCore, Verbose, " - logic optimiser");
				bModified |= LocalLogicOptimiserAST(roots);
			}

			ASTOp::LogHistogram(roots);
		}
	}


	// The state represents if there is a parent operation requiring skeleton for current mesh subtree.
	class CollectAllMeshesForSkeletonVisitorAST : public Visitor_TopDown_Unique_Const<uint8>
	{
	public:

		CollectAllMeshesForSkeletonVisitorAST( const ASTOpList& roots  )
		{
			Traverse( roots, false );
		}

		// List of meshes that require a skeleton
		TArray<UE::Mutable::Private::Ptr<ASTOpConstantResource>> MeshesRequiringSkeleton;

	private:

		// Visitor_TopDown_Unique_Const<uint8_t> interface
		bool Visit( const UE::Mutable::Private::Ptr<ASTOp>& node ) override
		{
			// \todo: refine to avoid instruction branches with irrelevant skeletons.

			uint8_t currentProtected = GetCurrentState();

			switch (node->GetOpType())
			{

			case EOpType::ME_CONSTANT:
			{
				UE::Mutable::Private::Ptr<ASTOpConstantResource> typedOp = static_cast<ASTOpConstantResource*>(node.get());

				if (currentProtected)
				{
					MeshesRequiringSkeleton.AddUnique(typedOp);
				}

				return false;
			}

			case EOpType::ME_CLIPMORPHPLANE:
			{
				ASTOpMeshClipMorphPlane* typedOp = static_cast<ASTOpMeshClipMorphPlane*>(node.get());
				if (typedOp->VertexSelectionType == EClipVertexSelectionType::BoneHierarchy)
				{
					// We need the skeleton for the source mesh
					RecurseWithState( typedOp->Source.child(), true );
					return false;
				}

				return true;
			}

			case EOpType::ME_APPLYPOSE:
			{
				ASTOpMeshApplyPose* typedOp = static_cast<ASTOpMeshApplyPose*>(node.get());

				// We need the skeleton for both meshes
				RecurseWithState(typedOp->Base.child(), true);
				RecurseWithState(typedOp->Pose.child(), true);
				return false;
			}

			case EOpType::ME_BINDSHAPE:
			{
				ASTOpMeshBindShape* typedOp = static_cast<ASTOpMeshBindShape*>(node.get());
				if (typedOp->bReshapeSkeleton)
				{
					RecurseWithState(typedOp->Mesh.child(), true);
					return false;
				}

				break;
			}

			case EOpType::ME_APPLYSHAPE:
			{
				ASTOpMeshApplyShape* typedOp = static_cast<ASTOpMeshApplyShape*>(node.get());
				if (typedOp->bReshapeSkeleton)
				{
					RecurseWithState(typedOp->Mesh.child(), true);
					return false;
				}

				break;
			}

			default:
				break;
			}

			return true;
		}

	};


	// This stores an ADD_MESH op with the child meshes collected and the final skeleton to use
	// for this op.
	struct FAddMeshSkeleton
	{
		UE::Mutable::Private::Ptr<ASTOp> AddMeshOp;
		TArray<UE::Mutable::Private::Ptr<ASTOpConstantResource>> ContributingMeshes;
		TManagedPtr<FSkeleton> FinalSkeleton;

		FAddMeshSkeleton( const UE::Mutable::Private::Ptr<ASTOp>& InAddMeshOp,
						  TArray<UE::Mutable::Private::Ptr<ASTOpConstantResource>>& InContributingMeshes,
						  const TManagedPtr<FSkeleton>& InFinalSkeleton )
		{
			AddMeshOp = InAddMeshOp;
			ContributingMeshes = MoveTemp(InContributingMeshes);
			FinalSkeleton = InFinalSkeleton;
		}
	};


	void SkeletonCleanerAST( TArray<UE::Mutable::Private::Ptr<ASTOp>>& roots, const FModelOptimizationOptions& options )
	{
		// This collects all the meshes that require a skeleton because they are used in operations
		// that require it.
		CollectAllMeshesForSkeletonVisitorAST requireSkeletonCollector( roots );

		ASTOp::Traverse_TopDown_Unique_Imprecise( roots, [&](UE::Mutable::Private::Ptr<ASTOp>& at )
		{
			// Only recurse instance construction ops.
			bool processChildren;

			const EDataType DataType = GetOpDataType(at->GetOpType());
			switch (DataType)
			{
			case EDataType::None:
			case EDataType::Instance:
			case EDataType::InstancedStruct:
			case EDataType::SkeletalMesh:
			case EDataType::LOD:
				processChildren = true;
				break;
    		
			default: 
				processChildren = false;
			}

			if ( at->GetOpType() == EOpType::LD_NEW )
			{
				ASTOpLODNew* Op = static_cast<ASTOpLODNew*>(at.get());
				
				for (const ASTChild& MeshOp : Op->Meshes)
				{
					UE::Mutable::Private::Ptr<ASTOp> MeshRoot = MeshOp.child();
					if (!MeshRoot)
					{
						continue;
					}
					
					// Gather constant meshes contributing to the final mesh
					TArray<UE::Mutable::Private::Ptr<ASTOpConstantResource>> subtreeMeshes;
					TArray<UE::Mutable::Private::Ptr<ASTOp>> tempRoots;
					tempRoots.Add(MeshRoot);
					ASTOp::Traverse_TopDown_Unique_Imprecise( tempRoots, [&](UE::Mutable::Private::Ptr<ASTOp>& lat )
					{
						// \todo: refine to avoid instruction branches with irrelevant skeletons.
						if ( lat->GetOpType() == EOpType::ME_CONSTANT )
						{
							UE::Mutable::Private::Ptr<ASTOpConstantResource> typedOp = static_cast<ASTOpConstantResource*>(lat.get());
							if ( subtreeMeshes.Find(typedOp)
								 ==
								 INDEX_NONE )
							{
								subtreeMeshes.Add(typedOp);
							}
						}
						return true;
					});

					// Create a mesh with the unified skeleton
					TManagedPtr<FSkeleton> FinalSkeleton = MakeManaged<FSkeleton>();
					for (const auto& MeshAt: subtreeMeshes)
					{
						TManagedPtr<FMesh> Mesh = ConstCastManagedPtr<FMesh>(StaticCastManagedPtr<const FMesh>(MeshAt->GetValue()));
						TManagedPtr<const FSkeleton> SourceSkeleton = Mesh ? Mesh->GetSkeleton() : nullptr;
						if (SourceSkeleton)
						{
							TArray<int32> RemappedSourceBoneIndices;
							MergeSkeletons(*FinalSkeleton.Get(), *SourceSkeleton.Get(), RemappedSourceBoneIndices);

							// Update the meshs' skeleton and remap bone indices
							Mesh->SetSkeleton(FinalSkeleton);

							const int32 NumBonesBoneMap = Mesh->BoneMap.Num();
							for (int32 Index = 0; Index < NumBonesBoneMap; ++Index)
							{
								Mesh->BoneMap[Index].Index = RemappedSourceBoneIndices[Mesh->BoneMap[Index].Index];
							}

							const int32 NumBonePoses = Mesh->BonePoses.Num();
							for (int32 Index = 0; Index < NumBonePoses; ++Index)
							{
								Mesh->BonePoses[Index].BoneId.Index = RemappedSourceBoneIndices[Mesh->BonePoses[Index].BoneId.Index];
							}
							
							Mesh->BonePoses.Sort([](const FMesh::FBonePose& A, const FMesh::FBonePose& B)
							{
								return A.BoneId.Index < B.BoneId.Index;
							});
						}
					}
				}
			}

			return processChildren;
		});
	}


	void CodeOptimiser::Optimise()
	{
		MUTABLE_CPUPROFILER_SCOPE(Optimise);

		// Gather all the roots (one for each state)
		TArray<Ptr<ASTOp>> roots;
		for(const FStateCompilationData& s:States)
		{
			roots.Add(s.root);
		}

		//UE_LOGF(LogMutableCore, Verbose, TEXT("(int) %ls : %ld"), "ast size", int64(ASTOp::CountNodes(roots)));

		if ( Options->GetPrivate()->OptimisationOptions.bEnabled )
		{
			// We use 4 times the count because at the time we moved to sharing this count it
			// was being used 4 times, and we want to keep the tests consistent.
			int32 MaxIterations = Options->GetPrivate()->OptimisationOptions.MaxOptimisationLoopCount;
			OptimizeIterationsLeft = MaxIterations ? MaxIterations * 4 : TNumericLimits<int32>::Max();

			// The first duplicated data remover has the special mission of removing
			// duplicated data (meshes) that may have been specified in the source
			// data, before we make it diverge because of different uses, like layout
			// creation
			UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
			DuplicatedDataRemoverAST( roots );

			ASTOp::LogHistogram(roots);

			UE_LOGF(LogMutableCore, Verbose, " - duplicated code remover");
			DuplicatedCodeRemoverAST( roots );

			// Special optimization stages
			if ( Options->GetPrivate()->OptimisationOptions.bUniformizeSkeleton )
			{
				UE_LOGF(LogMutableCore, Verbose, " - skeleton cleaner");
				ASTOp::LogHistogram(roots);

				SkeletonCleanerAST( roots, Options->GetPrivate()->OptimisationOptions );
				ASTOp::LogHistogram(roots);
			}

			// First optimisation stage. It tries to resolve all the image sizes. This is necessary
			// because some operations cannot be applied correctly until the image size is known
			// like the grow-map generation.
			bool bModified = true;
			int32 NumIterations = 0;
			while (bModified)
			{
				MUTABLE_CPUPROFILER_SCOPE(FirstStage);

				--OptimizeIterationsLeft;
				++NumIterations;
				UE_LOGF(LogMutableCore, Verbose, "First optimise iteration %d, left %d", NumIterations, OptimizeIterationsLeft);

				bModified = false;

				UE_LOGF(LogMutableCore, Verbose, " - size optimiser");
				bModified |= SizeOptimiserAST( roots );
			}

			// Main optimisation stage
			{
				MUTABLE_CPUPROFILER_SCOPE(MainStage);
				FullOptimiseAST( roots, 0 );

				FullOptimiseAST( roots, 1 );
			}

			// Constant resolution stage: resolve referenced assets.
			{
				MUTABLE_CPUPROFILER_SCOPE(ReferenceResolution);
				
				constexpr int32 Pass = 2;

				//FullOptimiseAST(roots, 2);

				// Generate constants
				for (Ptr<ASTOp>& Root : roots)
				{
					// Constant subtree generation
					bModified = ConstantGenerator(Options->GetPrivate(), Root, Pass);
				}

				DuplicatedDataRemoverAST(roots);
			}

			// Main optimisation stage again for data-aware optimizations
			{
				MUTABLE_CPUPROFILER_SCOPE(FinalStage);
				FullOptimiseAST(roots, 0);
				ASTOp::LogHistogram(roots);

				FullOptimiseAST(roots, 1);
				ASTOp::LogHistogram(roots);
			}

			// Analyse mesh constants to see which of them are in optimised mesh formats, and set the flags.
			ASTOp::Traverse_BottomUp_Unique_NonReentrant( roots, [&](Ptr<ASTOp>& n)
			{
				if (n->GetOpType()==EOpType::ME_CONSTANT)
				{
					ASTOpConstantResource* typed = static_cast<ASTOpConstantResource*>(n.get());
					TManagedPtr<const FMesh> pMesh = StaticCastManagedPtr<const FMesh>(typed->GetValue());
					pMesh->ResetStaticFormatFlags();
					typed->SetValue(pMesh);
				}
			});

			ASTOp::LogHistogram(roots);

			// Reset the state root operations in case they have changed due to optimization
			for (int32 RootIndex = 0; RootIndex < States.Num(); ++RootIndex)
			{
				States[RootIndex].root = roots[RootIndex];
			}

			{
				MUTABLE_CPUPROFILER_SCOPE(StatesStage);

				// Optimise for every state
				OptimiseStatesAST( );

				// Optimise the data formats (TODO)
				//OperationFlagGenerator flagGen( pResult.get() );
			}

			ASTOp::LogHistogram(roots);
		}

		// Minimal optimization of constant subtrees
		else if ( Options->GetPrivate()->OptimisationOptions.bConstReduction )
		{
			// The first duplicated data remover has the special mission of removing
			// duplicated data (meshes) that may have been specified in the source
			// data, before we make it diverge because of different uses, like layout
			// creation
			UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
			DuplicatedDataRemoverAST( roots );

			UE_LOGF(LogMutableCore, Verbose, " - duplicated code remover");
			DuplicatedCodeRemoverAST( roots );

			// Constant resolution stage: resolve referenced assets.
			{
				MUTABLE_CPUPROFILER_SCOPE(ReferenceResolution);
				FullOptimiseAST(roots, 2);
			}

			for ( int32 StateIndex=0; StateIndex <States.Num(); ++StateIndex)
			{
				constexpr int32 Pass = 1;

				UE_LOGF(LogMutableCore, Verbose, " - constant generator");
				ConstantGenerator( Options->GetPrivate(), roots[StateIndex], Pass);
			}

			UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
			DuplicatedDataRemoverAST( roots );

			UE_LOGF(LogMutableCore, Verbose, " - duplicated code remover");
			DuplicatedCodeRemoverAST( roots );

			// Reset the state root operations in case they have changed due to optimization
			for (int32 RootIndex = 0; RootIndex < States.Num(); ++RootIndex)
			{
				States[RootIndex].root = roots[RootIndex];
			}
		}

		ASTOp::LogHistogram(roots);

	}

}
