// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Operations.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/AST.h"

namespace UE::Mutable::Private
{

    /** Code optimiser: Rebuild an ASTOp graph so that it is more efficient. */
    class CodeOptimiser
    {
    public:
        CodeOptimiser( Ptr<CompilerOptions>, TArray<FStateCompilationData>& States );

        void Optimise();

    private:

		Ptr<CompilerOptions> Options;
		TArray<FStateCompilationData>& States;

        /** The max number of optimize iterations is shared across several stages now. This is how many are left. */
        int32 OptimizeIterationsLeft=0;

        //! Full optimisation pass
        void FullOptimiseAST( ASTOpList& roots, int32 Pass);

        /** Optimise the code for each specific state, generating new instructions and state information. */
        void OptimiseStatesAST();

    };


    /** ConstantGenerator replaces constant subtrees of operations with an equivalent single */
    extern bool ConstantGenerator( const CompilerOptions::Private*, Ptr<ASTOp>& Root, int32 Pass );

    /** \TODO: shapes, projectors, others? but not switches (they must be unique) */
    extern bool DuplicatedDataRemoverAST( ASTOpList& Roots );

    /** Mark all the duplicated code instructions to point at the same operation, leaving the copies unreachable. */
    extern bool DuplicatedCodeRemoverAST( ASTOpList& Roots );

    /** All kinds of optimisations that depend on the meaning of each operation. */
    extern bool SemanticOptimiserAST( ASTOpList& Roots, const FModelOptimizationOptions&, int32 Pass);

    /** Semantic operator that reorders instructions moving expensive ones down to the
    * leaves of the expressions trying to turn them into constants.
    */
    extern bool SinkOptimiserAST( ASTOpList& Roots, const FModelOptimizationOptions& );

    /** */
    extern bool SizeOptimiserAST( ASTOpList& Roots );

    /** */
    extern bool LocalLogicOptimiserAST(ASTOpList& roots);

    /** Discard all LODs beyond the given lod count. */
    class LODCountReducerAST : public Visitor_TopDown_Unique_Cloning
    {
    public:

        LODCountReducerAST( Ptr<ASTOp>& Root, uint8 NumExtraLODsToBuildAfterFirstLOD );

    protected:

        Ptr<ASTOp> Visit( Ptr<ASTOp>, bool& bOutProcessChildren ) override;

        uint8 NumExtraLODs = 0;
    };


    /** Scan the code in the given subtree and return true if a state runtime parameters is found.
    * Intermediate data is used betwen calls to apply, so don't remove program code or directly
    * change the instructions. Adding new instructions is ok.
	*/
    class RuntimeParameterVisitorAST
    {
    public:

        RuntimeParameterVisitorAST(const FStateCompilationData* );

        bool HasAny( const Ptr<ASTOp>& Root );

    private:

        const FStateCompilationData* State;

        //!
        struct FPendingItem
        {
            //! 0: indicate subtree pending
            //! 1: indicate children finished
            uint8 ItemType;

            //! 0: everything is relevant
            //! 1: only layouts are relevant
            uint8 OnlyLayoutsRelevant;

            //! Operation to visit
            Ptr<ASTOp> Op;
        };

        //!
        TArray< FPendingItem > Pending;

        //! Possible op state
        enum class EOpState : uint8
        {
            NotVisited = 0,
            ChildrenPendingFull,
            ChildrenPendingPartial,
            VisitedHasRuntime,
            VisitedFullDoesntHaveRuntime,
            VisitedPartialDoesntHaveRuntime
        };

        TMap<Ptr<ASTOp>, EOpState> Visited;

        //!
        void AddIfNeeded( const FPendingItem& );

    };


    /** Remove all texture compression operations that would happen for runtime parameter changes. */
    class RuntimeTextureCompressionRemoverAST : public Visitor_TopDown_Unique_Cloning
    {
    public:

        RuntimeTextureCompressionRemoverAST(FStateCompilationData*, bool bInAlwaysUncompress );

    protected:

        Ptr<ASTOp> Visit( Ptr<ASTOp>, bool& bOutProcessChildren ) override;

    private:

        RuntimeParameterVisitorAST HasRuntimeParamVisitor;
		bool bAlwaysUncompress = false;

    };

    /** Restructure the code to move operations involving runtime parameters as high as possible. */
    class ParameterOptimiserAST : public Visitor_TopDown_Unique_Cloning
    {
    private:

		FStateCompilationData& StateProps;

    public:

        ParameterOptimiserAST(FStateCompilationData&, const FModelOptimizationOptions& );

        bool Apply();

    private:

        Ptr<ASTOp> Visit( Ptr<ASTOp>, bool& bOutProcessChildren ) override;

        bool bModified;

        FModelOptimizationOptions OptimisationOptions;

        RuntimeParameterVisitorAST HasRuntimeParamVisitor;

    };


    /** Some masks are optional.If they are null, replace them by a white plain image of the right size. */
    extern Ptr<ASTOp> EnsureValidMask( Ptr<ASTOp> mask, Ptr<ASTOp> base );

    /** Calculate all the parameters found relevant under a particular operation.This may not
    * incldue all the parameters in the subtree (if because of the operations they are not
    * relevant)
    * It has an internal cache, so don't reuse if the program changes.
	*/
    class SubtreeRelevantParametersVisitorAST : public Visitor_TopDown_Unique_Const<bool>
    {
    public:

        void Run( Ptr<ASTOp> Root );

        virtual bool Visit(const Ptr<ASTOp>& node) override;

        //! After Run, list of relevant parameters.
        TSet< FString > Parameters;

    private:

        struct FState
        {
            Ptr<ASTOp> Op;
            bool bOnlyLayoutIsRelevant=false;

			FState( Ptr<ASTOp> o=nullptr, bool l=false) : Op(o), bOnlyLayoutIsRelevant(l) {}

            bool operator==(const FState& o) const
            {
                return Op == o.Op &&
					bOnlyLayoutIsRelevant == o.bOnlyLayoutIsRelevant;
            }

			friend FORCEINLINE uint32 GetTypeHash(const FState& InKey)
			{
				uint32 KeyHash = 0;
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.Op.get()));
				KeyHash = HashCombineFast(KeyHash, ::GetTypeHash(InKey.bOnlyLayoutIsRelevant));
				return KeyHash;
			}
		};


        // Result cache
        // \todo optimise by storing unique lists separately and an index here.
        TMap< FState, TSet<FString> > ResultCache;
    };
	
	
	class SubtreeResourceRelevantParametersVisitorAST : public SubtreeRelevantParametersVisitorAST
	{
	public:
		virtual bool Visit(const Ptr<ASTOp>& node) override;
	};
}
