// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodeOptimiserPrivate.h"

namespace UE::Mutable::Private
{
	bool DeduplicateFMaterial(TMap<TPassthroughObjectPtr<UMaterialInterface>, Ptr<ASTOpConstantResource>>& Ops, const Ptr<ASTOpConstantResource>& OpResource)
	{
		TManagedPtr<const FMaterial> CurrentValue = StaticCastManagedPtr<const FMaterial>(OpResource->GetValue());
		
		if (Ptr<ASTOpConstantResource>* CachedOpResource = Ops.Find(CurrentValue->PassthroughObject))
		{
			TManagedPtr<const FMaterial> CachedOpResourceValue = StaticCastManagedPtr<const FMaterial>(CachedOpResource->get()->GetValue());
			TManagedPtr<FMaterial> ClonedCachedOpResourceValue = CachedOpResourceValue->Clone();
			
			// Append onto the already processed ASTOp the parameters found in the ASTOp being processed
			ClonedCachedOpResourceValue->ImageParameters.Append(CurrentValue->ImageParameters);
			ClonedCachedOpResourceValue->ColorParameters.Append(CurrentValue->ColorParameters);
			ClonedCachedOpResourceValue->ScalarParameters.Append(CurrentValue->ScalarParameters);
			
			CachedOpResource->get()->SetValue(ClonedCachedOpResourceValue);

			// Add the Child ASTOps of the OpResource being processed as children of already processed ASTOp.
			for (TTuple<FParameterKey, ASTChild>& NewImageOperation : OpResource->ImageOperations)
			{
				// The children will have as parent the cached ASTOp resource as now they will hang from it
				(*CachedOpResource)->ImageOperations.Add(NewImageOperation.Key, ASTChild(*CachedOpResource, NewImageOperation.Value.Child));
			}
			
			ASTOp::Replace(OpResource, *CachedOpResource);
			return true;
		}
		else
		{
			Ops.Add(CurrentValue->PassthroughObject, OpResource);
			return false;
		}
	}	
}

