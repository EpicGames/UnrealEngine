// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ASTOpConstantResource.h"
#include "MuR/Ptr.h"
#include "Templates/SharedPointer.h"
#include "MuT/AST.h"


namespace UE::Mutable::Private
{
	template<typename T>
	struct FResourceEntry
	{
		uint32 Hash = 0;
		TManagedPtr<const T> Value;

		bool operator==(const FResourceEntry& Entry) const
		{
			return Value == Entry.Value || *Value == *Entry.Value;
		}

		friend uint32 GetTypeHash(const FResourceEntry& Entry)
		{
			return Entry.Hash;
		}
	};


	template<typename T>
	bool Deduplicate(TMap<FResourceEntry<T>, Ptr<ASTOpConstantResource>>& Ops, const Ptr<ASTOpConstantResource>& OpResource)
	{
		uint32 Hash = OpResource->GetValueHash();
		TManagedPtr<const T> Value = StaticCastManagedPtr<const T>(OpResource->GetValue());

		FResourceEntry<T> Entry(Hash, Value);
		
		if (Ptr<ASTOpConstantResource>* Found = Ops.Find(Entry))
		{
			ASTOp::Replace(OpResource, *Found);
			return true;
		}
		else
		{
			Ops.Add(Entry, OpResource);
			return false;
		}
	}
	
	
	bool DeduplicateFMaterial(TMap<TPassthroughObjectPtr<UMaterialInterface>, Ptr<ASTOpConstantResource>>& Ops, const Ptr<ASTOpConstantResource>& OpResource);
}
