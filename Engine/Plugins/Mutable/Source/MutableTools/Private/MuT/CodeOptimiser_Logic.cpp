// Copyright Epic Games, Inc. All Rights Reserved.

#include "ASTOpBoolAnd.h"
#include "Misc/AssertionMacros.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/ASTOpBoolEqualIntConst.h"
#include "MuT/CodeOptimiser.h"


namespace UE::Mutable::Private
{
	/**
	 * Enumeration representing the branches that an ASTOpBoolAnd operation has.
	 */
	enum class EASTOpBoolAndBranch : int32
	{
		A = 0,
		B,
	};
	
	
	Ptr<ASTOpSwitch> OptimizeConditionalChainBranch(Ptr<ASTOp>& n, const EASTOpBoolAndBranch TargetBoolAndOpBranch)
	{
		ASTOpConditional* RootConditional = nullptr;
		if (n && n->IsConditional())
		{
			RootConditional = static_cast<ASTOpConditional*>(n.get());
		}

		Ptr<ASTOpSwitch> SwitchOP = nullptr;
		
		// Crete the switch object first and set up the type and the variable to later be used to evaluate what case is the active one
		if ( RootConditional && RootConditional->condition &&  
			 (RootConditional->condition->GetOpType()==EOpType::BO_EQUAL_INT_CONST || RootConditional->condition->GetOpType() == EOpType::BO_AND)
			 &&
			 RootConditional->no
			 &&
			 RootConditional->no->GetOpType()==RootConditional->GetOpType()
			)
		{
			SwitchOP = new ASTOpSwitch();
			SwitchOP->Type = GetSwitchForType(GetOpDataType(RootConditional->GetOpType()));
			
			// Look for the conditional node of the topConditional and use the value set on it as the variable for the switch. 
			{
				// Simple Case, BO_EQUAL_INT_CONST
				if (RootConditional->condition->GetOpType()==EOpType::BO_EQUAL_INT_CONST )
				{
					ASTOpBoolEqualIntConst* TopConditionalCondition = static_cast<ASTOpBoolEqualIntConst*>(RootConditional->condition.child().get());
					SwitchOP->Variable = TopConditionalCondition->Value.child();
				}
				
				// The condition is an AND operation
				else if (RootConditional->condition->GetOpType() == EOpType::BO_AND)
				{
					ASTOpBoolAnd* firstCompare = static_cast<ASTOpBoolAnd*>(RootConditional->condition.child().get());
				
					// Locate the Equal_cost of the And  use it as the variable of the switch
					if (TargetBoolAndOpBranch == EASTOpBoolAndBranch::A && 
						firstCompare->A.child()->GetOpType() == EOpType::BO_EQUAL_INT_CONST)
					{
						ASTOpBoolEqualIntConst* EqualIntConst = static_cast<ASTOpBoolEqualIntConst*>(firstCompare->A.child().get());
						check(EqualIntConst);
						
						// Now the variable of the switch will use the "parameter" set in the value branch of the const
						SwitchOP->Variable = EqualIntConst->Value.child();
					}
					else if (TargetBoolAndOpBranch == EASTOpBoolAndBranch::B && 
						firstCompare->B.Child->GetOpType()==EOpType::BO_EQUAL_INT_CONST)
					{
						ASTOpBoolEqualIntConst* EqualIntConst = static_cast<ASTOpBoolEqualIntConst*>(firstCompare->B.child().get());
						check(EqualIntConst);
						
						SwitchOP->Variable = EqualIntConst->Value.child();
					}
				}
			}
			
			// node not compatible with this optimization as no variable for the switch could be found
			if (!SwitchOP->Variable.child().get())
			{
				return nullptr;
			}

			TArray<Ptr<ASTOpConditional>> NewConditionals;
			
	        // Iterate over the child objects and add the ones that use the same variable as the switch as cases for it
			Ptr<ASTOp> current = n;
	        while(current)
			{
				bool bValid = false;

				ASTOpConditional* CurrentConditional = nullptr;
				if (current && current->IsConditional())
				{
					CurrentConditional = static_cast<ASTOpConditional*>(current.get());
				}
	        	
				// Simple case: the conditional is an BO_EQUAL_INT_CONST
				if ( CurrentConditional
					 && 
					 CurrentConditional->GetOpType()==RootConditional->GetOpType()
					 &&
					 CurrentConditional->condition
					 &&
					 CurrentConditional->condition->GetOpType()==EOpType::BO_EQUAL_INT_CONST )
				{
					ASTOpBoolEqualIntConst* Compare = static_cast<ASTOpBoolEqualIntConst*>(CurrentConditional->condition.child().get());
					check(Compare);

					if ( Compare->Value.child() == SwitchOP->Variable.child() )
					{
						const auto CaseSearchLambda = [NewCaseIndex = Compare->Constant](const ASTOpSwitch::FCase& Other)->bool
						{
							return Other.Condition == NewCaseIndex;
						};           		
					
						if (!SwitchOP->Cases.ContainsByPredicate(CaseSearchLambda))
						{
							SwitchOP->Cases.Emplace( Compare->Constant, SwitchOP, CurrentConditional->yes.child() );

							current = CurrentConditional->no.child();
							bValid = true;
						}
						else
						{
							// If we added this into the cases it would collide with another case's condition. Abort optimization.
							return nullptr;
						}
					}
				}
				// A bit more complex: 
				else if ( CurrentConditional
				 && 
				 CurrentConditional->GetOpType()==RootConditional->GetOpType()
				 &&
				 CurrentConditional->condition
				 &&
				 CurrentConditional->condition->GetOpType()==EOpType::BO_AND )
				{
					ASTOpBoolAnd* AndOp = static_cast<ASTOpBoolAnd*>(CurrentConditional->condition.child().get());
					check(AndOp);
					
					int32 SwitchConditionValue = -1;
					Ptr<ASTOp> CaseChildCondition = nullptr;
					bool bSwitchConditionValueFound = false;
					
					if (TargetBoolAndOpBranch == EASTOpBoolAndBranch::A && AndOp->A.child()->GetOpType() == EOpType::BO_EQUAL_INT_CONST)
					{
						ASTOpBoolEqualIntConst* Compare = static_cast<ASTOpBoolEqualIntConst*>(AndOp->A.child().get());
						check(Compare);
						
						// Ensure the value of this const also points to the same variable of the switch. If not just ignore it
						if (Compare->Value.child() == SwitchOP->Variable.child())
						{
							const int32 ACaseConditionValue = Compare->Constant;
							
							const auto CaseSearchLambdaA = [ACaseConditionValue](const ASTOpSwitch::FCase& Other)->bool
							{
								return Other.Condition == ACaseConditionValue;
							};
							
							if (!SwitchOP->Cases.ContainsByPredicate(CaseSearchLambdaA))
							{
								SwitchConditionValue = ACaseConditionValue;
								bSwitchConditionValueFound = true;
							}
							else
							{
								// If we added this into the cases it would collide with another case's condition. Abort optimization.
								return nullptr;
							}
							
							// since A is the equal then B is the condition we want now t use with this node
							CaseChildCondition = AndOp->B.child();
						}
					}
					
					// A is not a valid case, try B branch
					else if (TargetBoolAndOpBranch == EASTOpBoolAndBranch::B && AndOp->B.child()->GetOpType()==EOpType::BO_EQUAL_INT_CONST)
					{
						ASTOpBoolEqualIntConst* Compare = static_cast<ASTOpBoolEqualIntConst*>(AndOp->B.child().get());
						check(Compare);
						
						if (Compare->Value.child() == SwitchOP->Variable.child())
						{	
							const int32 BCaseConditionValue = Compare->Constant;
															
							const auto CaseSearchLambdaB = [BCaseConditionValue](const ASTOpSwitch::FCase& Other)->bool
							{
								return Other.Condition == BCaseConditionValue;
							};
								
							if (!SwitchOP->Cases.ContainsByPredicate(CaseSearchLambdaB))
							{
								SwitchConditionValue = BCaseConditionValue;
								bSwitchConditionValueFound = true;
							}
							else
							{
								// If we added this into the cases it would collide with another case's condition. Abort optimization.
								return nullptr;
							}
							
							// since B is the equal then A is the condition we want now t use with this node
							CaseChildCondition = AndOp->A.child();
						}
					}
					
					// Ony add a new case with condition index if the index has not already been added
					if (bSwitchConditionValueFound)
					{
						// Make a new conditional to use as the new case where only the Yes section is there with the condition that did 
						// coexist with the EQ_Const of the BO_And. Since the switch already hadles the EQ_Const then the only condition we need is the other one
						Ptr<ASTOpConditional> NewConditional = new ASTOpConditional();
						NewConditional->type = CurrentConditional->type;
						NewConditional->yes = ASTChild( NewConditional, CurrentConditional->yes.child().get());
						NewConditional->no = ASTChild( NewConditional, nullptr);
						NewConditional->condition = CaseChildCondition;
						
						NewConditionals.Add( NewConditional);
						
						// Add the NewConditional node as a case for the switch.
						// This case has:
						//	- condition : the constant value of an ASTOpBoolEqualIntConst extracted from the Branch A or Branch B of the BO_AND op
						//	- parent : the switch node it will be part of
						//	- branch : a branch that has as root a new conditional OP NewConditional that has as condition the complementary branch from the one 
						//				used to get the "condition" integer value.
						//				it will look something like this :
						//				IF_CONDITIONAL == "NewConditional"
						//					->	Condition : A if case condition is B, B otherwise.
						//					->	YES : The "yes" from the conditional node we are processing now. 
						//					->	NO : Default case of the switch. (added later when iterating the NewConditionals array)NewConditional
						
						SwitchOP->Cases.Emplace( SwitchConditionValue, SwitchOP, NewConditional);
						
						// make the next branch to be processed the other side of the current conditional as we have added to the switch a path including the YES side
						current = CurrentConditional->no.child();			
						bValid = true;
					}
				}

				if (!bValid)
				{
					SwitchOP->Default = current;

					// Make the no branch have the current node branch once the switch setup is done
					for (Ptr<ASTOpConditional>& NewConditional : NewConditionals)
					{
						NewConditional->no = SwitchOP->Default.child();
					}
					
					current = nullptr;
				}
			}
		}
		
		return SwitchOP;
	}
	
	
	bool LocalLogicOptimiserAST(ASTOpList& roots)
    {
        MUTABLE_CPUPROFILER_SCOPE(LocalLogicOptimiserAST);

        bool bModified = false;

        // The separate steps should not be combined into one traversal

		// Unwrap some typical code daisy-chains
        //-----------------------------------------------------------------------------------------
        {
            MUTABLE_CPUPROFILER_SCOPE(ContionalExclusive);
            ASTOp::Traverse_TopDown_Unique_Imprecise(roots, [&](Ptr<ASTOp>& Op)
            {
            	if (Op->IsConditional())
                {
					ASTOpConditional* TopConditional = static_cast<ASTOpConditional*>(Op.get());
                    Ptr<ASTOp> Base = TopConditional->yes.child();
                    if (Base)
                    {
                        bool bEnd = false;
                        while (!bEnd)
                        {
                            bEnd = true;
                        	Base->ForEachChild([&](ASTChild& Child)
                        	{
                        		if (!Child || !Child->IsConditional())
	                            {
		                            return;
	                            }
                        			
		                        ASTOpConditional* BottomConditional = static_cast<ASTOpConditional*>(Child.child().get());

		                        ASTOpList Facts;
		                        Facts.Add(TopConditional->condition.child());
									
		                        Ptr<ASTOp> BottomCondition = BottomConditional->condition.child();
		                        ASTOp::FBoolEvalResult Result = BottomCondition->EvaluateBool( Facts );
                        		
		                        const bool bIsConditionalExclusive = Result == ASTOp::BET_FALSE;
		                        if (bIsConditionalExclusive)
		                        {
			                        if (Base->GetParentCount() == 1)
			                        {
				                        // Directly modify the instruction to skip the impossible child option
				                        Child = BottomConditional->no.child();
			                        }
			                        else
			                        {
				                        // Other parents may not impose the same condition that allows the optimisation.
				                        Ptr<ASTOp> NewBase = UE::Mutable::Private::Clone<ASTOp>(Base);
			                        	ASTOp::Replace(Child.child(), BottomConditional->no.child());
				                        TopConditional->yes = NewBase;
				                        Base = NewBase.get();
			                        }

			                        bModified = true;
			                        bEnd = false;
		                        }
                        	});
                        }
                    }
                }

                return true;
            });
        }


        // See if we can turn conditional chains into switches: all conditions must be integer
        // comparison with the same variable.
        //-----------------------------------------------------------------------------------------
        {
            MUTABLE_CPUPROFILER_SCOPE(ConditionalToSwitch);
        	
        	constexpr int32 MIN_CONDITIONS_TO_CREATE_SWITCH = 2;
        	
        	// Force optimization of the A branch of the conditional BO_AND operation
            ASTOp::Traverse_TopDown_Unique_Imprecise( roots, [&](Ptr<ASTOp>& CurrentOp)
            {
				// Try processing first the A branch of the BO_AND (if found)
				const Ptr<ASTOpSwitch> SwitchOp = OptimizeConditionalChainBranch(CurrentOp, EASTOpBoolAndBranch::A);
								
				if (SwitchOp && SwitchOp->Cases.Num() >= MIN_CONDITIONS_TO_CREATE_SWITCH)
				{
					ASTOp::Replace(CurrentOp,SwitchOp);
					CurrentOp = SwitchOp;
					bModified = true;
				}
            		
            	constexpr bool bRecurse = true;
                return bRecurse;
            });
        	
        	// Force optimization of the B branch of the conditional BO_AND operation
        	ASTOp::Traverse_TopDown_Unique_Imprecise( roots, [&](Ptr<ASTOp>& CurrentOp)
			{			
				// Then do the same for the B branch of the BO_AND (also if found)
				const Ptr<ASTOpSwitch> SwitchOp = OptimizeConditionalChainBranch(CurrentOp, EASTOpBoolAndBranch::B);
			
				if (SwitchOp && SwitchOp->Cases.Num() >= MIN_CONDITIONS_TO_CREATE_SWITCH)
				{
					ASTOp::Replace(CurrentOp,SwitchOp);
					CurrentOp = SwitchOp;
					bModified = true;
				}
            		
				constexpr bool bRecurse = true;
				return bRecurse;
			});
        }

		
		// TODO: This code needs to be revised as currently causes some instances to crash during the setup of the render sections (indirect consequence).
		// ASTOp::Traverse_BottomUp_Unique( roots, [&](Ptr<ASTOp>& CurrentOp)
		// {	
		// 	if (!CurrentOp->IsSwitch())
		// 	{
		// 		return;
		//   }
		// 		
		// 	ASTOpSwitch* Switch = static_cast<ASTOpSwitch*>(CurrentOp.get());
		// 
		// 	Ptr<ASTOpSwitch> NewSwitch = Clone<ASTOpSwitch>(Switch);
		// 	NewSwitch->Cases.Reset();
		// 		
		// 	ASTOp* FirstCaseCondition = nullptr;
		// 		
		// 	for (int32 Index = 0; Index < Switch->Cases.Num(); ++Index)
		// 	{
		// 		const ASTOpSwitch::FCase& Case = Switch->Cases[Index];
		// 		
		// 		if (!Case.Branch || !Case.Branch->IsConditional())
		// 		{
		// 			return;
		// 		}
		// 		
		// 		ASTOpConditional* CaseConditional = static_cast<ASTOpConditional*>(Case.Branch.child().get());
		// 		
		// 		if (!FirstCaseCondition)
		// 		{
		// 			FirstCaseCondition = CaseConditional->condition.child().get();
		// 		}
		// 		
		// 		if (FirstCaseCondition != CaseConditional->condition.child())
		// 		{
		// 			return;
		// 		}
		// 		
		// 		if (CaseConditional->no != Switch->Default)
		// 		{
		// 			return;
		// 		}
		// 		
		// 		if (!CaseConditional->yes)
		// 		{
		// 			return;
		// 		}
		// 		
		// 		NewSwitch->Cases.Emplace(Case.Condition, NewSwitch, CaseConditional->yes.child());
		// 	}
		// 		
		// 	// All conditionals should have a condition.
		// 	check(FirstCaseCondition);
		// 		
		// 	Ptr<ASTOpConditional> NewConditional = new ASTOpConditional();
		// 	NewConditional->type = GetConditionalForType(GetOpDataType(NewSwitch->GetOpType()));
		// 	NewConditional->condition = FirstCaseCondition;
		// 	NewConditional->yes = NewSwitch;
		// 	NewConditional->no = NewSwitch->Default.child();
		// 		
		// 	ASTOp::Replace(CurrentOp, NewConditional);
		// });
		

        // Float operations up switches, to tidy up the code and reduce its size.
		// TODO?
		/*
        {
            MUTABLE_CPUPROFILER_SCOPE(FloatSwitches);

            ASTOp::Traverse_TopDown_Unique_Imprecise( roots, [&](Ptr<ASTOp>& n)
            {
                bool bRecurse = true;

				ASTOpSwitch* TopSwitch = nullptr;
				if (n && n->IsSwitch())
				{
					TopSwitch = static_cast<ASTOpSwitch*>(n.get());
				}

                if (TopSwitch)
                {
                    bool bFirst = true;
					EOpType CaseType = EOpType::NONE;
                    for (const ASTOpSwitch::FCase& c: TopSwitch->Cases)
                    {
                        if ( c.Branch )
                        {
                            if (bFirst)
                            {
								bFirst = false;
								CaseType = c.Branch->GetOpType();
                            }
                            else
                            {
                                if (CaseType !=c.Branch->GetOpType())
                                {
									CaseType = EOpType::NONE;
                                    break;
                                }
                            }
                        }
                    }
                }

                return bRecurse;
            });
        }
		*/

        return bModified;
    }

}
