// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageResizeLike.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpMaterialModify.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/ASTOpConstantColor.h"
#include "MuT/CodeOptimiser.h"
#include "MuT/Compiler.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/DataPacker.h"
#include "MuT/ASTOpLODNew.h"
#include "MuT/ASTOpSkeletalMeshNew.h"
#include "MuR/Image.h"
#include "MuR/ImagePrivate.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/MutableRuntimeModule.h"

#include "HAL/PlatformCrt.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"

#include <array>

#include "ASTOpSkeletalMeshMaterialModify.h"
#include "ASTOpSkeletalMeshObjectConvert.h"


namespace UE::Mutable::Private
{

    RuntimeParameterVisitorAST::RuntimeParameterVisitorAST(const FStateCompilationData* InState)
        : State(InState)
    {
    }


    bool RuntimeParameterVisitorAST::HasAny( const Ptr<ASTOp>& Root )
    {
        if (!State->nodeState.RuntimeParams.Num())
        {
            return false;
        }

        // Shortcut flag: if true we already found a runtime parameter, don't process new ops,
        // but still store the results of processed ops.
        bool bRuntimeFound = false;

        Pending.Reset();

        FPendingItem Start;
		Start.Op = Root;
		Start.ItemType = 0;
		Start.OnlyLayoutsRelevant = 0;
		Pending.Add(Start);

        // Don't early out to be able to complete parent op cached flags
        while (!Pending.IsEmpty())
        {
			FPendingItem Item = Pending.Pop(EAllowShrinking::No);
            Ptr<ASTOp> Op = Item.Op;

			if (!Op)
			{
				continue;
			}

            // Not cached?
			EOpState* FoundCached = Visited.Find(Op);
            if ( 
				!FoundCached
				||
				(
					*FoundCached !=EOpState::VisitedHasRuntime
					&&
					*FoundCached != EOpState::VisitedFullDoesntHaveRuntime
					) 
				)
            {
                if (Item.ItemType)
                {
                    // Item indicating we finished with all the children of a parent
                    check(*FoundCached ==EOpState::ChildrenPendingFull
                        ||
						*FoundCached ==EOpState::ChildrenPendingPartial
                        ||
						*FoundCached ==EOpState::VisitedPartialDoesntHaveRuntime );

                    bool bSubtreeFound = false;
					Op->ForEachChild( [&](ASTChild& ref)
                    {
						EOpState* FoundChild = Visited.Find(ref.child());
						if (FoundChild && *FoundChild == EOpState::VisitedHasRuntime)
						{
							bSubtreeFound = true;
						}
                    });

                    if (bSubtreeFound)
                    {
						*FoundCached = EOpState::VisitedHasRuntime;
                    }
                    else
                    {
						*FoundCached = Item.OnlyLayoutsRelevant
                                ? EOpState::VisitedPartialDoesntHaveRuntime
                                : EOpState::VisitedFullDoesntHaveRuntime;
                    }
                }

                else if (!bRuntimeFound)
                {
                    // We need to process the subtree
                    check(!FoundCached 
						||
						*FoundCached ==EOpState::NotVisited
						||
						(*FoundCached == EOpState::VisitedPartialDoesntHaveRuntime
							&&
							Item.OnlyLayoutsRelevant==0 )
					);

                    // Request the processing of the end of this instruction
					FPendingItem endItem = Item;
                    endItem.ItemType = 1;
                    Pending.Add( endItem );
					Visited.Add(Op, Item.OnlyLayoutsRelevant
						? EOpState::ChildrenPendingPartial
						: EOpState::ChildrenPendingFull);

                    // Is it a special op type?
                    switch ( Op->GetOpType() )
                    {

                    case EOpType::BO_PARAMETER:
                    case EOpType::NU_PARAMETER:
                    case EOpType::SC_PARAMETER:
                    case EOpType::CO_PARAMETER:
                    case EOpType::PR_PARAMETER:
					case EOpType::IM_PARAMETER:
					case EOpType::SK_PARAMETER:
					case EOpType::MA_PARAMETER:
					{
						const ASTOpParameter* Typed = static_cast<const ASTOpParameter*>(Op.get());
                        const TArray<FString>& Params = State->nodeState.RuntimeParams;
                        if (Params.Find(Typed->Parameter.Name)
                             !=
                             INDEX_NONE )
                        {
							bRuntimeFound = true;
                            Visited.Add(Op,EOpState::VisitedHasRuntime);
                        }
                        break;
                    }

                    default:
                    {
                        Op->ForEachChild([&](ASTChild& ref)
                        {
							FPendingItem ChildItem;
                            ChildItem.ItemType = 0;
                            ChildItem.Op = ref.child();
                            ChildItem.OnlyLayoutsRelevant = Item.OnlyLayoutsRelevant;
                            AddIfNeeded(ChildItem);
                        });
                        break;
                    }

                    }

                }

                else
                {
                    // We won't process it.
					Visited.Add(Op,EOpState::NotVisited);
                }
            }
        }

        return Visited[Root]==EOpState::VisitedHasRuntime;
    }


    void RuntimeParameterVisitorAST::AddIfNeeded( const FPendingItem& Item )
    {
        if (Item.Op)
        {
			const EOpState* Found = Visited.Find(Item.Op);
            if (!Found || *Found==EOpState::NotVisited)
            {
                Pending.Add(Item);
            }
            else if (*Found ==EOpState::VisitedPartialDoesntHaveRuntime
				&&
				Item.OnlyLayoutsRelevant==0)
            {
				Pending.Add(Item);
            }
            else if (*Found ==EOpState::ChildrenPendingPartial
				&&
				Item.OnlyLayoutsRelevant==0)
            {
				Pending.Add(Item);
            }
        }
    }


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    Ptr<ASTOp> EnsureValidMask( Ptr<ASTOp> Mask, Ptr<ASTOp> Base )
    {
        if ( !Mask)
        {
            Ptr<ASTOpConstantColor> whiteOp = new ASTOpConstantColor;
            whiteOp->Value = FVector4f(1,1,1,1);

            Ptr<ASTOpImagePlainColor> wplainOp = new ASTOpImagePlainColor;
            wplainOp->Color = whiteOp;
            wplainOp->Format = EImageFormat::L_UByte;
            wplainOp->Size[0] = 4;
			wplainOp->Size[1] = 4;
			wplainOp->LODs = 1;

            Ptr<ASTOpImageResizeLike> ResizeOp = new ASTOpImageResizeLike;
			ResizeOp->Source = wplainOp;
			ResizeOp->SizeSource = Base;

			Mask = ResizeOp;
        }

        return Mask;
    }


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    ParameterOptimiserAST::ParameterOptimiserAST(
		FStateCompilationData& s,
            const FModelOptimizationOptions& optimisationOptions
            )
            : StateProps(s)
            , bModified(false)
            , OptimisationOptions(optimisationOptions)
            , HasRuntimeParamVisitor(&s)
    {
    }


    bool ParameterOptimiserAST::Apply()
    {
        MUTABLE_CPUPROFILER_SCOPE(ParameterOptimiserAST);

        bModified = false;

        // Optimise the cloned tree
        Traverse( StateProps.root );

        return bModified;
    }


    Ptr<ASTOp> ParameterOptimiserAST::Visit( Ptr<ASTOp> At, bool& processChildren )
    {
        // Only process children if there are runtime parameters in the subtree
        processChildren = HasRuntimeParamVisitor.HasAny(At);

		// TODO IM_MULTICOMPOSE

        EOpType type = At->GetOpType();
        switch ( type )
        {
        //-------------------------------------------------------------------------------------
        // Be careful with changing merge options and "mergesurfaces" flags
//        case EOpType::ME_MERGE:
//        {
//            OP::MeshMergeArgs mergeArgs = program.m_code[At].args.MeshMerge;

//            RuntimeParameterVisitor paramVis;

//            switch ( program.m_code[ mergeArgs.base ].type )
//            {
//            case EOpType::ME_CONDITIONAL:
//            {
//                OP::ADDRESS conditionAt =
//                        program.m_code[ mergeArgs.base ].args.Conditional.condition;
//                bool conditionConst = !paramVis.HasAny( Model.get(),
//                                                        m_state,
//                                                        conditionAt );
//                if ( conditionConst )
//                {
//                    // TODO: this may unfold mesh combinations of some models increasing the size of
//                    // the model data. Make this optimisation optional.
//                    bModified = true;

//                    OP yesOp = program.m_code[At];
//                    yesOp.args.MeshMerge.base = program.m_code[ mergeArgs.base ].args.Conditional.yes;

//                    OP noOp = program.m_code[At];
//                    noOp.args.MeshMerge.base = program.m_code[ mergeArgs.base ].args.Conditional.no;

//                    OP op = program.m_code[ mergeArgs.base ];
//                    op.args.Conditional.yes = program.AddOp( yesOp );
//                    op.args.Conditional.no = program.AddOp( noOp );
//                    At = program.AddOp( op );
//                }

//                break;
//            }

//            case EOpType::ME_MERGE:
//            {
//                OP::ADDRESS childBaseAt = program.m_code[ mergeArgs.base ].args.MeshMerge.base;
//                bool childBaseConst = !paramVis.HasAny( Model.get(),
//                                                        m_state,
//                                                        childBaseAt );

//                OP::ADDRESS childAddAt = program.m_code[ mergeArgs.base ].args.MeshMerge.added;
//                bool childAddConst = !paramVis.HasAny( Model.get(),
//                                                       m_state,
//                                                       childAddAt );

//                bool addConst = !paramVis.HasAny( Model.get(),
//                                                  m_state,
//                                                  mergeArgs.added );

//                if ( !childBaseConst && childAddConst && addConst )
//                {
//                    bModified = true;

//                    OP bottom = program.m_code[At];
//                    bottom.args.MeshMerge.base = childAddAt;

//                    OP top = program.m_code[ mergeArgs.base ];
//                    top.args.MeshMerge.added = program.AddOp( bottom );
//                    At = program.AddOp( top );
//                }
//                break;
//            }

//            default:
//                break;
//            }

//            break;
//        }

        //-------------------------------------------------------------------------------------
        //-------------------------------------------------------------------------------------
        //-------------------------------------------------------------------------------------
        case EOpType::IM_CONDITIONAL:
        {
			const ASTOpConditional* TypedOp = static_cast<const ASTOpConditional*>(At.get());

            // If the condition is not runtime, but the branches are, try to move the
            // conditional down
            bool optimised = false;

            if ( !HasRuntimeParamVisitor.HasAny( TypedOp->condition.child() ) ) 
            {
                EOpType yesType = TypedOp->yes.child().get() ? TypedOp->yes->GetOpType() : EOpType::NONE;
                EOpType noType = TypedOp->no.child().get() ? TypedOp->no->GetOpType() : EOpType::NONE;

                bool yesHasAny = yesType != EOpType::NONE && HasRuntimeParamVisitor.HasAny( TypedOp->yes.child() );
                bool noHasAny = noType != EOpType::NONE && HasRuntimeParamVisitor.HasAny( TypedOp->no.child() );

                if ( !optimised && yesHasAny && noHasAny && yesType==noType)
                {
                    switch (yesType)
                    {
                        case EOpType::IM_COMPOSE:
                        {
						const ASTOpImageCompose* typedYes = static_cast<const ASTOpImageCompose*>(TypedOp->yes.child().get());
						const ASTOpImageCompose* typedNo = static_cast<const ASTOpImageCompose*>(TypedOp->no.child().get());
                        if ( typedYes->BlockId == typedNo->BlockId
                             &&
                             (
                                 (typedYes->Mask.child().get() != nullptr)
                                  ==
                                 (typedNo->Mask.child().get() != nullptr)
                                )
                             )
                            {
                                // Move the conditional down
                                Ptr<ASTOpImageCompose> compOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(typedYes);

                                Ptr<ASTOpConditional> baseCond = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                                baseCond->yes = typedYes->Base.child();
                                baseCond->no = typedNo->Base.child();
                                compOp->Base = baseCond;

                                Ptr<ASTOpConditional> blockCond = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                                blockCond->yes = typedYes->BlockImage.child();
                                blockCond->no = typedNo->BlockImage.child();
                                compOp->BlockImage = blockCond;

                                if (typedYes->Mask)
                                {
                                    Ptr<ASTOpConditional> maskCond = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                                    maskCond->yes = typedYes->Mask.child();
                                    maskCond->no = typedNo->Mask.child();
                                    compOp->Mask = maskCond;
                                }

                                Ptr<ASTOpConditional> layCond = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                                layCond->type = EOpType::LA_CONDITIONAL;
                                layCond->yes = typedYes->Layout.child();
                                layCond->no = typedNo->Layout.child();
                                compOp->Layout = layCond;


                                At = compOp;
                                optimised = true;
                            }
                            break;
                        }

                    default:
                        break;

                    }
                }

                if ( !optimised && yesHasAny )
                {
                    switch (yesType)
                    {
                        case EOpType::IM_LAYERCOLOR:
                        {
                            optimised = true;

							const ASTOpImageLayerColor* typedYes = static_cast<const ASTOpImageLayerColor*>(TypedOp->yes.child().get());

                            Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
                            blackOp->Value = FVector4f(0,0,0,1);

                            Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
                            plainOp->Color = blackOp;
                            plainOp->Format = EImageFormat::L_UByte;
                            plainOp->Size[0] = 4;
							plainOp->Size[1] = 4;
							plainOp->LODs = 1;

                            Ptr<ASTOpImageResizeLike> ResizeOp = new ASTOpImageResizeLike;
							ResizeOp->Source = plainOp;
							ResizeOp->SizeSource = typedYes->base.child();

                            Ptr<ASTOpConditional> maskOp = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                            maskOp->no = ResizeOp;

                            // If there is no mask (because it is optional), we need to make a
                            // white plain image
                            maskOp->yes = EnsureValidMask( typedYes->mask.child(), typedYes->base.child() );

							Ptr<ASTOpConditional> baseOp = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                            baseOp->yes = typedYes->base.child();

                            Ptr<ASTOpImageLayerColor> softOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(typedYes);
                            softOp->base = baseOp;
                            softOp->mask = maskOp;

                            At = softOp;
                            break;
                        }

                        // TODO
                        // It seems this is not worth since it replaces a conditional by a compose
                        // (but only At build time, not update?) and it introduces the use of masks
                        // and resize likes... plus masks can't always be used if BC formats.
//						case EOpType::IM_COMPOSE:
//						{
//							optimised = true;

//							OP blackOp;
//							blackOp.type = EOpType::CO_CONSTANT;
//							blackOp.args.ColorConstant.value[0] = 0;
//							blackOp.args.ColorConstant.value[1] = 0;
//							blackOp.args.ColorConstant.value[2] = 0;
//							blackOp.args.ColorConstant.value[3] = 1;

//							OP plainOp;
//							plainOp.type = EOpType::IM_PLAINCOLOR;
//							plainOp.args.ImagePlainColor.color = program.AddOp( blackOp );
//							plainOp.args.ImagePlainColor.format = L_UByte;
//							plainOp.args.ImagePlainColor.size = ;
//							plainOp.args.ImagePlainColor.LODs = ;

//							OP resizeOp;
//							resizeOp.type = EOpType::IM_RESIZELIKE;
//							resizeOp.args.ImageResizeLike.source = program.AddOp( plainOp );
//							resizeOp.args.ImageResizeLike.sizeSource =
//									program.m_code[args.yes].args.ImageCompose.blockImage;

//							OP maskOp = program.m_code[At];
//							maskOp.args.Conditional.no = program.AddOp( resizeOp );

//							// If there is no mask (because it is optional), we need to make a
//							// white plain image
//							maskOp.args.Conditional.yes = EnsureValidMask
//									( program.m_code[args.yes].args.ImageCompose.mask,
//									  program.m_code[args.yes].args.ImageCompose.base,
//									  program );

//							OP baseOp = program.m_code[At];
//							baseOp.args.Conditional.yes =
//									program.m_code[args.yes].args.ImageCompose.base;

//							OP composeOp = program.m_code[args.yes];
//							composeOp.args.ImageCompose.base = program.AddOp( baseOp );
//							composeOp.args.ImageCompose.mask = program.AddOp( maskOp );

//							At = program.AddOp( composeOp );

//							// Process the new children
//							At = Recurse( At, program );
//							break;
//						}

                        default:
                            break;

                    }
                }

                else if ( !optimised && noHasAny )
                {
                    switch (noType)
                    {
                        case EOpType::IM_LAYERCOLOR:
                        {
                            optimised = true;

                            const ASTOpImageLayerColor* typedNo = static_cast<const ASTOpImageLayerColor*>(TypedOp->no.child().get());

                            Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
                            blackOp->Value = FVector4f(0,0,0,1);

                            Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
                            plainOp->Color = blackOp;
                            plainOp->Format = EImageFormat::L_UByte;
                            plainOp->Size[0] = 4;
							plainOp->Size[1] = 4;
							plainOp->LODs = 1;

                            Ptr<ASTOpImageResizeLike> ResizeOp = new ASTOpImageResizeLike;
							ResizeOp->Source = plainOp;
							ResizeOp->SizeSource = typedNo->base.child();

                            Ptr<ASTOpConditional> maskOp = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                            maskOp->no = ResizeOp;

                            // If there is no mask (because it is optional), we need to make a
                            // white plain image
                            maskOp->no = EnsureValidMask( typedNo->mask.child(), typedNo->base.child() );

                            Ptr<ASTOpConditional> baseOp = UE::Mutable::Private::Clone<ASTOpConditional>(TypedOp);
                            baseOp->no = typedNo->base.child();

                            Ptr<ASTOpImageLayerColor> softOp = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(typedNo);
                            softOp->base = baseOp;
                            softOp->mask = maskOp;

                            At = softOp;
                            break;
                        }

                        default:
                            break;

                    }
                }
            }

            bModified |= optimised;

            break;
        }


        //-------------------------------------------------------------------------------------
        case EOpType::IM_SWITCH:
        {
            // If the switch is not runtime, but the branches are, try to move the
            // switch down
//            bool optimised = false;

//            OP::ADDRESS variable = program.m_code[At].args.Switch.variable;

//            if ( !HasRuntimeParamVisitor.HasAny( variable, program ) )
//            {
//                SWITCH_CHAIN chain = GetSwitchChain( program, At );

//                bool branchHasAny = false;
//                EOpType branchType = (EOpType)program.m_code[program.m_code[At].args.Switch.values[0]].type;
//                for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                      it != chain.cases.end();
//                      ++it )
//                {
//                    if ( program.m_code[it->second].type != branchType )
//                    {
//                        branchType = EOpType::NONE;
//                    }
//                    else
//                    {
//                        if (!branchHasAny)
//                        {
//                            branchHasAny = HasRuntimeParamVisitor.HasAny( it->second, program );
//                        }
//                    }
//                }

//                if ( chain.def && program.m_code[chain.def].type != branchType )
//                {
//                    branchType = EOpType::NONE;
//                }

//                // Some branch in runtime
//                if ( branchHasAny )
//                {
//                    switch ( branchType )
//                    {

//                    // TODO: Other operations
//                    case EOpType::IM_BLEND:
//                    case EOpType::IM_MULTIPLY:
//                    {
//                        // Move the switch down the base
//                        OP::ADDRESS baseAt = 0;
//                        for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                              it != chain.cases.end();
//                              )
//                        {
//                            OP bsw;
//                            bsw.type = EOpType::IM_SWITCH;
//                            bsw.args.Switch.variable = variable;

//                            for ( int b=0;
//                                  it != chain.cases.end() && b<MUTABLE_OP_MAX_SWITCH_OPTIONS;
//                                  ++b )
//                            {
//                                bsw.args.Switch.conditions[b] = it->first;
//                                bsw.args.Switch.values[b] =
//                                        program.m_code[it->second].args.ImageLayer.base;
//                                ++it;
//                            }

//                            bsw.args.Switch.def = baseAt;
//                            baseAt = program.AddOp( bsw );
//                        }

//                        // Move the switch down the mask
//                        OP::ADDRESS maskAt = 0;
//                        for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                              it != chain.cases.end();
//                              )
//                        {
//                            OP bsw;
//                            bsw.type = EOpType::IM_SWITCH;
//                            bsw.args.Switch.variable = variable;

//                            for ( int b=0;
//                                  it != chain.cases.end() && b<MUTABLE_OP_MAX_SWITCH_OPTIONS;
//                                  ++b )
//                            {
//                                bsw.args.Switch.conditions[b] = it->first;
//                                bsw.args.Switch.values[b] =
//                                        program.m_code[it->second].args.ImageLayer.mask;
//                                ++it;
//                            }

//                            bsw.args.Switch.def = maskAt;
//                            maskAt = program.AddOp( bsw );
//                        }

//                        // Move the switch down the blended
//                        OP::ADDRESS blendedAt = 0;
//                        for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                              it != chain.cases.end();
//                              )
//                        {
//                            OP bsw;
//                            bsw.type = EOpType::IM_SWITCH;
//                            bsw.args.Switch.variable = variable;

//                            for ( int b=0;
//                                  it != chain.cases.end() && b<MUTABLE_OP_MAX_SWITCH_OPTIONS;
//                                  ++b )
//                            {
//                                bsw.args.Switch.conditions[b] = it->first;
//                                bsw.args.Switch.values[b] =
//                                        program.m_code[it->second].args.ImageLayer.blended;
//                                ++it;
//                            }

//                            bsw.args.Switch.def = blendedAt;
//                            blendedAt = program.AddOp( bsw );
//                        }

//                        OP top;
//                        top.type = branchType;
//                        top.args.ImageLayer.base = baseAt;
//                        top.args.ImageLayer.mask = maskAt;
//                        top.args.ImageLayer.blended = blendedAt;
//                        At = program.AddOp( top );
//                        optimised = true;
//                        break;
//                    }


//                    // TODO: Other operations
//                    case EOpType::IM_SOFTLIGHTCOLOR:
//                    {
//                        // Move the switch down the base
//                        OP::ADDRESS baseAt = 0;
//                        for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                              it != chain.cases.end();
//                              )
//                        {
//                            OP bsw;
//                            bsw.type = EOpType::IM_SWITCH;
//                            bsw.args.Switch.variable = variable;

//                            for ( int b=0;
//                                  it != chain.cases.end() && b<MUTABLE_OP_MAX_SWITCH_OPTIONS;
//                                  ++b )
//                            {
//                                bsw.args.Switch.conditions[b] = it->first;
//                                bsw.args.Switch.values[b] =
//                                        program.m_code[it->second].args.ImageLayerColor.base;
//                                ++it;
//                            }

//                            bsw.args.Switch.def = baseAt;
//                            baseAt = program.AddOp( bsw );
//                        }

//                        // Move the switch down the mask
//                        OP::ADDRESS maskAt = 0;
//                        for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                              it != chain.cases.end();
//                              )
//                        {
//                            OP bsw;
//                            bsw.type = EOpType::IM_SWITCH;
//                            bsw.args.Switch.variable = variable;

//                            for ( int b=0;
//                                  it != chain.cases.end() && b<MUTABLE_OP_MAX_SWITCH_OPTIONS;
//                                  ++b )
//                            {
//                                bsw.args.Switch.conditions[b] = it->first;
//                                bsw.args.Switch.values[b] =
//                                        program.m_code[it->second].args.ImageLayerColor.mask;
//                                ++it;
//                            }

//                            bsw.args.Switch.def = maskAt;
//                            maskAt = program.AddOp( bsw );
//                        }

//                        // Move the switch down the color
//                        OP::ADDRESS colorAt = 0;
//                        for ( map<uint16,OP::ADDRESS>::const_iterator it=chain.cases.begin();
//                              it != chain.cases.end();
//                              )
//                        {
//                            OP bsw;
//                            bsw.type = EOpType::CO_SWITCH;
//                            bsw.args.Switch.variable = variable;

//                            for ( int b=0;
//                                  it != chain.cases.end() && b<MUTABLE_OP_MAX_SWITCH_OPTIONS;
//                                  ++b )
//                            {
//                                bsw.args.Switch.conditions[b] = it->first;
//                                bsw.args.Switch.values[b] =
//                                        program.m_code[it->second].args.ImageLayerColor.color;
//                                ++it;
//                            }

//                            bsw.args.Switch.def = colorAt;
//                            colorAt = program.AddOp( bsw );
//                        }

//                        OP top;
//                        top.type = branchType;
//                        top.args.ImageLayerColor.base = baseAt;
//                        top.args.ImageLayerColor.mask = maskAt;
//                        top.args.ImageLayerColor.color = colorAt;
//                        At = program.AddOp( top );
//                        optimised = true;
//                        break;
//                    }


//                    default:
//                        break;

//                    }
//                }

//            }

//            bModified |= optimised;

            break;
        }


        //-----------------------------------------------------------------------------------------
        case EOpType::IM_COMPOSE:
        {
			const ASTOpImageCompose* TypedOp = static_cast<const ASTOpImageCompose*>(At.get());

            Ptr<ASTOp> blockAt = TypedOp->BlockImage.child();
			Ptr<ASTOp> baseAt = TypedOp->Base.child();
			Ptr<ASTOp> layoutAt = TypedOp->Layout.child();

			if (!blockAt)
			{
				At = baseAt;
				break;
			}

            EOpType blockType = blockAt->GetOpType();
            EOpType baseType = baseAt->GetOpType();

            bool baseHasRuntime = HasRuntimeParamVisitor.HasAny( baseAt );
            bool blockHasRuntime = HasRuntimeParamVisitor.HasAny( blockAt );
            bool layoutHasRuntime = HasRuntimeParamVisitor.HasAny( layoutAt );

            bool optimised = false;

            // Try to optimise base and block together, if possible
            if ( blockHasRuntime && baseHasRuntime && !layoutHasRuntime )
            {
                if ( baseType == blockType )
                {
                    switch ( blockType )
                    {
                    case EOpType::IM_LAYERCOLOR:
                    {
                        optimised = true;

						const ASTOpImageLayerColor* typedBaseAt = static_cast<const ASTOpImageLayerColor*>(baseAt.get());
						const ASTOpImageLayerColor* typedBlockAt = static_cast<const ASTOpImageLayerColor*>(blockAt.get());

                        // The mask is a compose of the block mask on the base mask, but if none has
                        // a mask we don't need to make one.
						Ptr<ASTOp> baseImage = typedBaseAt->base.child();
						Ptr<ASTOp> baseMask = typedBaseAt->mask.child();
						Ptr<ASTOp> blockImage = typedBlockAt->base.child();
						Ptr<ASTOp> blockMask = typedBlockAt->mask.child();

                        Ptr<ASTOpImageCompose> maskOp;
                        if (baseMask || blockMask)
                        {
                            // This may create a discrepancy of number of mips between the base image and the mask This is for now solved with emergy fix
							Ptr<ASTOp> newBaseMask = EnsureValidMask(baseMask, baseImage);
							Ptr<ASTOp> newBlockMask = EnsureValidMask(blockMask, blockImage);

                            maskOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(TypedOp);
                            maskOp->Base = newBaseMask;
                            maskOp->BlockImage = newBlockMask;
                        }

                        // The base is composition of the bases of both layer effect
						Ptr<ASTOpImageCompose> baseOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(TypedOp);
                        baseOp->Base = baseImage;
                        baseOp->BlockImage = blockImage;

                        Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(blockAt);
                        nop->mask = maskOp;
                        nop->base = baseOp;

                        // Done
                        At = nop;
                        break;
                    }

                    case EOpType::IM_LAYER:
                    {
                        optimised = true;

						const ASTOpImageLayer* typedBaseAt = static_cast<const ASTOpImageLayer*>(baseAt.get());
						const ASTOpImageLayer* typedBlockAt = static_cast<const ASTOpImageLayer*>(blockAt.get());

                        // The mask is a compose of the block mask on the base mask, but if none has
                        // a mask we don't need to make one.
						Ptr<ASTOp> baseImage = typedBaseAt->base.child();
						Ptr<ASTOp> baseBlended = typedBaseAt->blend.child();
						Ptr<ASTOp> baseMask = typedBaseAt->mask.child();
						Ptr<ASTOp> blockImage = typedBlockAt->base.child();
						Ptr<ASTOp> blockBlended = typedBlockAt->blend.child();
						Ptr<ASTOp> blockMask = typedBlockAt->mask.child();

                        Ptr<ASTOpImageCompose> maskOp;
                        if (baseMask || blockMask)
                        {
                            // This may create a discrepancy of number of mips between the base image and the mask This is for now solved with emergy fix
							Ptr<ASTOp> newBaseMask = EnsureValidMask(baseMask, baseImage);
							Ptr<ASTOp> newBlockMask = EnsureValidMask(blockMask, blockImage);

                            maskOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(TypedOp);
                            maskOp->Base = newBaseMask;
                            maskOp->BlockImage = newBlockMask;
                        }

                        // The base is composition of the bases of both layer effect
						Ptr<ASTOpImageCompose> baseOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(TypedOp);
                        baseOp->Base = baseImage;
                        baseOp->BlockImage = blockImage;

                        // The base is composition of the bases of both layer effect
						Ptr<ASTOpImageCompose> blendedOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(TypedOp);
                        blendedOp->Base = baseBlended;
                        blendedOp->BlockImage = blockBlended;

                        Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(blockAt);
                        nop->mask = maskOp;
                        nop->base = baseOp;
                        nop->blend = blendedOp;

                        // Done
                        At = nop;
                        break;
                    }

                    default:
                        break;
                    }
                }
            }


            // Swap two composes
            if ( !optimised && baseHasRuntime && !blockHasRuntime
                 &&
                 baseType == EOpType::IM_COMPOSE )
            {
				const ASTOpImageCompose* typedBaseAt = static_cast<const ASTOpImageCompose*>(baseAt.get());

				Ptr<ASTOp> baseBlockAt = typedBaseAt->BlockImage.child();
                bool baseBlockHasAny = HasRuntimeParamVisitor.HasAny( baseBlockAt );
                if ( baseBlockHasAny )
                {
                    optimised = true;

                    // Swap
					Ptr<ASTOpImageCompose> childCompose = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    childCompose->Base = typedBaseAt->Base.child();

					Ptr<ASTOpImageCompose> parentCompose = UE::Mutable::Private::Clone<ASTOpImageCompose>(baseAt);
                    parentCompose->Base = childCompose;

                    At = parentCompose;
                }
            }


            // Try to optimise the block
            // This optimisation requires a lot of memory for every target. Use only if
            // we are optimising for GPU processing.
            if ( !optimised && blockHasRuntime && !baseHasRuntime
                 //&& StateProps.m_gpu.m_external
                 // TODO BLEH
                 // Only worth in case of more than one block using the same operation. Move this
                 // optimisation to that test.
                 //&& false
                 )
            {
                switch ( blockType )
                {
                case EOpType::IM_LAYERCOLOR:
                {
                    optimised = true;

					const ASTOpImageLayerColor* typedBlockAt = static_cast<const ASTOpImageLayerColor*>(blockAt.get());

					Ptr<ASTOp> blockImage = typedBlockAt->base.child();
					Ptr<ASTOp> blockMask = typedBlockAt->mask.child();

                    // The mask is a compose of the layer mask on a black image, however if there is
                    // no mask and the base of the layer opertation is a blanklayout, we can skip
                    // generating a mask.
                    Ptr<ASTOpImageCompose> maskOp;
                    if (blockMask || baseType!=EOpType::IM_BLANKLAYOUT)
                    {
                        maskOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
						Ptr<ASTOp> newMaskBlock = EnsureValidMask(blockMask, blockImage);
                        maskOp->BlockImage = newMaskBlock;

                        Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
                        blackOp->Value = FVector4f(0,0,0,1);

                        Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
                        plainOp->Color = blackOp;
                        plainOp->Format = EImageFormat::L_UByte;
                        plainOp->Size[0] = 4;
						plainOp->Size[1] = 4;
						plainOp->LODs = 1;

                        Ptr<ASTOpImageResizeLike> BaseResizeOp = new ASTOpImageResizeLike;
						BaseResizeOp->SizeSource = baseAt;
						BaseResizeOp->Source = plainOp;

                        maskOp->Base = BaseResizeOp;
                    }

                    // The base is composition of the layer base on the compose base
                    Ptr<ASTOpImageCompose> baseOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    baseOp->BlockImage = typedBlockAt->base.child();

                    Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(blockAt);
                    nop->mask = maskOp;
                    nop->base = baseOp;

                    // Done
                    At = nop;
                    break;
                }

                case EOpType::IM_LAYER:
                {
                    optimised = true;

                    const ASTOpImageLayer* typedBlockAt = static_cast<const ASTOpImageLayer*>(blockAt.get());

					Ptr<ASTOp> blockImage = typedBlockAt->base.child();
					Ptr<ASTOp> blockBlended = typedBlockAt->blend.child();
					Ptr<ASTOp> blockMask = typedBlockAt->mask.child();

                    // The mask is a compose of the layer mask on a black image, however if there is
                    // no mask and the base of the layer opertation is a blanklayout, we can skip
                    // generating a mask.
                    Ptr<ASTOpImageCompose> maskOp;
                    if (blockMask || baseType != EOpType::IM_BLANKLAYOUT)
                    {
                        maskOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
						Ptr<ASTOp> newMaskBlock = EnsureValidMask(blockMask, blockImage);
                        maskOp->BlockImage = newMaskBlock;


                        Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
                        blackOp->Value = FVector4f(0,0,0,1);

                        Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
                        plainOp->Color = blackOp;
                        plainOp->Format = EImageFormat::L_UByte;
                        plainOp->Size[0] = 4;
						plainOp->Size[1] = 4;
						plainOp->LODs = 1;

                        Ptr<ASTOpImageResizeLike> BaseResizeOp = new ASTOpImageResizeLike;
						BaseResizeOp->SizeSource = baseAt;
						BaseResizeOp->Source = plainOp;

                        maskOp->Base = BaseResizeOp;
                    }

                    // The blended is a compose of the blended image on a blank image
					Ptr<ASTOpImageCompose> blendedOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    {
                        blendedOp->BlockImage = blockBlended;

                        Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
						blackOp->Value = FVector4f(0, 0, 0, 1);

                        Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
                        plainOp->Color = blackOp;
                        FImageDesc blendedDesc = baseAt->GetImageDesc();
                        plainOp->Format = blendedDesc.m_format;
                        plainOp->Size[0] = 4;
						plainOp->Size[1] = 4;
						plainOp->LODs = 1;

                        Ptr<ASTOpImageResizeLike> ResizeOp = new ASTOpImageResizeLike;
						ResizeOp->SizeSource = baseAt;
						ResizeOp->Source = plainOp;

                        blendedOp->Base = ResizeOp;
                    }

                    // The base is composition of the softlight base on the compose base
                    Ptr<ASTOpImageCompose> baseOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    baseOp->BlockImage = typedBlockAt->base.child();

                    Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(blockAt);
                    nop->base = baseOp;
                    nop->mask = maskOp;
                    nop->blend = blendedOp;

                    // Done
                    At = nop;
                    break;
                }


//                case EOpType::IM_INTERPOLATE:
//                {
//                    optimised = true;
//                    OP op = program.m_code[blockAt];

//                    // The targets are composition of the block targets on the compose base
//                    for ( int t=0; t<MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++t )
//                    {
//                        if ( op.args.ImageInterpolate.targets[t] )
//                        {
//                            OP targetOp = program.m_code[At];
//                            targetOp.args.ImageCompose.blockImage =
//                                    op.args.ImageInterpolate.targets[t];
//                            op.args.ImageInterpolate.targets[t] = program.AddOp( targetOp );
//                        }
//                    }

//                    // Done
//                    At = program.AddOp( op );

//                    // Reprocess the new children
//                    At = Recurse( At, program );

//                    break;
//                }

                default:
                    break;

                }
            }


            // Try to optimise the base
            if ( !optimised && baseHasRuntime /*&& StateProps.nodeState.m_optimisation.m_gpu.m_external*/ )
            {
                switch ( baseType )
                {
                case EOpType::IM_LAYERCOLOR:
                {
                    optimised = true;

                    const ASTOpImageLayerColor* typedBaseAt = static_cast<const ASTOpImageLayerColor*>(baseAt.get());

					Ptr<ASTOpImageCompose> maskOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    {
                        Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
                        blackOp->Value = FVector4f(0,0,0,1);

                        Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
						plainOp->Color = blackOp;
                        plainOp->Format = EImageFormat::L_UByte; //TODO: FORMAT_LIKE
                        plainOp->Size[0] = 4;
						plainOp->Size[1] = 4;
						plainOp->LODs = 1;

                        Ptr<ASTOpImageResizeLike> BlockResizeOp = new ASTOpImageResizeLike;
						BlockResizeOp->SizeSource = blockAt;
						BlockResizeOp->Source = plainOp;

                        // Blank out the block from the mask
						Ptr<ASTOp> newMaskBase = EnsureValidMask( typedBaseAt->mask.child(), baseAt );
                        maskOp->Base = newMaskBase;
                        maskOp->BlockImage = BlockResizeOp;
                    }

                    // The base is composition of the softlight base on the compose base
					Ptr<ASTOpImageCompose> baseOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    baseOp->Base = typedBaseAt->base.child();

                    Ptr<ASTOpImageLayerColor> nop = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(baseAt);
                    nop->base = baseOp;
                    nop->mask = maskOp;

                    // Done
                    At = nop;
                    break;
                }

                case EOpType::IM_LAYER:
                {
                    optimised = true;

                    const ASTOpImageLayer* typedBaseAt = static_cast<const ASTOpImageLayer*>(baseAt.get());

					Ptr<ASTOpImageCompose> maskOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    {
                        Ptr<ASTOpConstantColor> blackOp = new ASTOpConstantColor;
						blackOp->Value = FVector4f(0, 0, 0, 1);

                        Ptr<ASTOpImagePlainColor> plainOp = new ASTOpImagePlainColor;
                        plainOp->Color = blackOp;
                        plainOp->Format = EImageFormat::L_UByte; //TODO: FORMAT_LIKE
                        plainOp->Size[0] = 4;
						plainOp->Size[1] = 4;
						plainOp->LODs = 1;

                        Ptr<ASTOpImageResizeLike> BlockResizeOp = new ASTOpImageResizeLike;
						BlockResizeOp->SizeSource = blockAt;
						BlockResizeOp->Source = plainOp;

                        // Blank out the block from the mask
                        Ptr<ASTOp> newMaskBase = EnsureValidMask( typedBaseAt->mask.child(), baseAt );
                        maskOp->Base = newMaskBase;
                        maskOp->BlockImage = BlockResizeOp;
                    }

                    // The base is composition of the effect base on the compose base
                    Ptr<ASTOpImageCompose> baseOp = UE::Mutable::Private::Clone<ASTOpImageCompose>(At);
                    baseOp->Base = typedBaseAt->base.child();

                    Ptr<ASTOpImageLayer> nop = UE::Mutable::Private::Clone<ASTOpImageLayer>(baseAt);
                    nop->base = baseOp;
                    nop->mask = maskOp;

                    // Done
                    At = nop;
                    break;
                }


//                case EOpType::IM_INTERPOLATE:
//                {
//                    optimised = true;
//                    OP op = program.m_code[baseAt];

//                    // The targets are composition of the blocks on the compose base targets
//                    for ( int t=0; t<MUTABLE_OP_MAX_INTERPOLATE_COUNT; ++t )
//                    {
//                        if ( op.args.ImageInterpolate.targets[t] )
//                        {
//                            OP targetOp = program.m_code[At];
//                            targetOp.args.ImageCompose.base =
//                                    op.args.ImageInterpolate.targets[t];
//                            op.args.ImageInterpolate.targets[t] = program.AddOp( targetOp );
//                        }
//                    }

//                    // Done
//                    At = program.AddOp( op );

//                    // Reprocess the new children
//                    At = Recurse( At, program );
//                    break;
//                }

                default:
                    break;

                }
            }


            bModified = bModified || optimised;
            break;
        }

/*
        //-----------------------------------------------------------------------------------------
        // TODO: Other ops?
        case EOpType::IM_BLEND:
        {
            OP op = program.m_code[At];

            if ( !HasRuntimeParamVisitor.HasAny( op.args.ImageLayer.mask, program ) )
            {
                // If both the base and the blended have the same image layer operation with
                // similar parameters, we can move that operation up.
                EOpType baseType = program.m_code[ op.args.ImageLayer.base ].type;
                EOpType blendedType = program.m_code[ op.args.ImageLayer.blended ].type;
                if ( baseType == blendedType )
                {
                    switch ( baseType )
                    {
                    case EOpType::IM_BLENDCOLOR:
                    case EOpType::IM_SOFTLIGHTCOLOR:
                    case EOpType::IM_HARDLIGHTCOLOR:
                    case EOpType::IM_BURNCOLOR:
                    case EOpType::IM_SCREENCOLOR:
                    case EOpType::IM_OVERLAYCOLOR:
                    case EOpType::IM_DODGECOLOR:
                    case EOpType::IM_MULTIPLYCOLOR:
                    {
                        if ( OptimisationOptions.m_optimiseOverlappedMasks )
                        {
                            OP::ADDRESS maskAt = op.args.ImageLayer.mask;
                            OP::ADDRESS baseMaskAt =
                                    program.m_code[ op.args.ImageLayer.base ].args.ImageLayerColor.mask;
                            OP::ADDRESS blendedMaskAt =
                                    program.m_code[ op.args.ImageLayer.blended ].args.ImageLayerColor.mask;
                            OP::ADDRESS baseColorAt =
                                    program.m_code[ op.args.ImageLayer.base ].args.ImageLayerColor.color;
                            OP::ADDRESS blendedColorAt =
                                    program.m_code[ op.args.ImageLayer.blended ].args.ImageLayerColor.color;

                            // Check extra conditions on the masks
                            if ( maskAt && baseMaskAt && blendedMaskAt
                                 && (baseColorAt==blendedColorAt)
                                 && !AreMasksOverlapping( program, baseMaskAt, blendedMaskAt )
                                 && !AreMasksOverlapping( program, baseMaskAt, maskAt ) )
                            {
                                // We can apply the transform
                                bModified = true;

                                OP baseOp;
                                baseOp.type = EOpType::IM_BLEND;
                                baseOp.args.ImageLayer.base =
                                        program.m_code[ op.args.ImageLayer.base ].args.ImageLayerColor.base;
                                baseOp.args.ImageLayer.blended =
                                        program.m_code[ op.args.ImageLayer.blended ].args.ImageLayerColor.base;
                                baseOp.args.ImageLayer.mask = maskAt;

                                // TODO: Find out why these are not equivalent
    //							OP maskOp;
    //							maskOp.type = EOpType::IM_SCREEN;
    //							maskOp.args.ImageLayer.base = baseMaskAt;
    //							maskOp.args.ImageLayer.blended = blendedMaskAt;
                                OP maskOp;
                                maskOp.type = EOpType::IM_BLEND;
                                maskOp.args.ImageLayer.base = baseMaskAt;
                                maskOp.args.ImageLayer.blended = blendedMaskAt;
                                maskOp.args.ImageLayer.mask = blendedMaskAt;
                                maskOp.args.ImageLayer.flags = OP::ImageLayerArgs::F_BINARY_MASK;

                                OP top;
                                top.type = baseType;
                                top.args.ImageLayerColor.base = program.AddOp( baseOp );
                                top.args.ImageLayerColor.mask = program.AddOp( maskOp );
                                top.args.ImageLayerColor.color = baseColorAt;
                                At = program.AddOp( top );
                            }
                        }

                        break;
                    }

                    default:
                        break;

                    }
                }
            }
            break;
        }

        */

        //-----------------------------------------------------------------------------------------
        // Sink the mipmap if worth it.
        case EOpType::IM_MIPMAP:
        {
			const ASTOpImageMipmap* TypedOp = static_cast<const ASTOpImageMipmap*>(At.get());

			Ptr<ASTOp> sourceOp = TypedOp->Source.child();

            switch ( sourceOp->GetOpType() )
            {
            case EOpType::IM_LAYERCOLOR:
            {
                const ASTOpImageLayerColor* typedSource = static_cast<const ASTOpImageLayerColor*>(sourceOp.get());

                bool colorHasRuntime = HasRuntimeParamVisitor.HasAny( typedSource->color.child() );

                if (colorHasRuntime)
                {
                    bModified = true;

					Ptr<ASTOpImageLayerColor> top = UE::Mutable::Private::Clone<ASTOpImageLayerColor>(sourceOp);

					Ptr<ASTOpImageMipmap> baseOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(At);
                    baseOp->Source = typedSource->base.child();
                    top->base = baseOp;

					Ptr<ASTOp> sourceMaskOp = typedSource->mask.child();
                    if (sourceMaskOp)
                    {
						Ptr<ASTOpImageMipmap> maskOp = UE::Mutable::Private::Clone<ASTOpImageMipmap>(At);
                        maskOp->Source = sourceMaskOp;
                        top->mask = maskOp;
                    }

                    At = top;
                }

                break;
            }

            default:
                break;
            }

            break;
        }

        default:
            break;

        }

        return At;
    }


    class AccumulateAllImageFormatsOpAST
            : public Visitor_TopDown_Unique_Const< std::array<uint8_t, size_t(EImageFormat::Count)> >
    {
    public:

        void Run( const ASTOpList& roots )
        {
            MUTABLE_CPUPROFILER_SCOPE(AccumulateAllImageFormatsOpAST);

            // Initially, all formats are supported
            AllSupported.fill(1);

            // The initial traversal state is no format supported
            InitialState.fill(0);
            Traverse( roots, InitialState );
        }


        bool Visit( const Ptr<ASTOp>& At ) override
        {
            bool recurse = false;

            const std::array<uint8, SIZE_T(EImageFormat::Count)>& CurrentFormats = GetCurrentState();

            // Remove unsupported formats
            if (GetOpDataType( At->GetOpType() )==EDataType::Image)
            {
				std::array<uint8, SIZE_T(EImageFormat::Count)>* it = SupportedFormats.Find(At);
                if (!it)
                {
                    // Default to all supported
                    SupportedFormats.Add(At, AllSupported);
                    it = SupportedFormats.Find(At);
                }

                for ( int32 f=0; f< int32(EImageFormat::Count); ++f )
                {
                    if ( !CurrentFormats[f] )
                    {
                        (*it)[f] = 0;
                    }
                }
            }

            switch ( At->GetOpType() )
            {
            // TODO: Code shared with the constant data format optimisation visitor
            case EOpType::IM_LAYERCOLOR:
            {
                const ASTOpImageLayerColor* TypedOp = static_cast<const ASTOpImageLayerColor*>(At.get());

                RecurseWithCurrentState( TypedOp->base.child() );
                RecurseWithCurrentState( TypedOp->color.child() );

                if ( TypedOp->mask )
                {
                    std::array<uint8, SIZE_T(EImageFormat::Count)> NewState;
                    NewState.fill(0);
                    NewState[SIZE_T(EImageFormat::L_UByte) ] = 1;
                    NewState[SIZE_T(EImageFormat::L_UByteRLE) ] = 1;

                    RecurseWithState( TypedOp->mask.child(), NewState );
                }
                break;
            }

            case EOpType::IM_LAYER:
            {
                const ASTOpImageLayer* TypedOp = static_cast<const ASTOpImageLayer*>(At.get());

                RecurseWithCurrentState( TypedOp->base.child() );
                RecurseWithCurrentState( TypedOp->blend.child() );

                std::array<uint8, SIZE_T(EImageFormat::Count)> NewState;
                NewState.fill(0);
                // TODO
                //NewState[ L_UByte ] = 1;
                //NewState[ L_UByteRLE ] = 1;

                if ( TypedOp->mask )
                {
                    RecurseWithState( TypedOp->mask.child(), NewState );
                }
                break;
            }

            case EOpType::IM_DISPLACE:
            {
				const ASTOpImageDisplace* TypedOp = static_cast<const ASTOpImageDisplace*>(At.get());

                RecurseWithCurrentState( TypedOp->Source.child() );

                std::array<uint8, SIZE_T(EImageFormat::Count)> NewState;
                NewState.fill(0);
                NewState[SIZE_T(EImageFormat::L_UByte) ] = 1;
                NewState[SIZE_T(EImageFormat::L_UByteRLE) ] = 1;
                RecurseWithState( TypedOp->DisplacementMap.child(), NewState );
                break;
            }

            default:
                SetCurrentState( InitialState );
                recurse = true;
                break;
            }

            return recurse;
        }


        bool IsSupportedFormat( Ptr<ASTOp> Op, EImageFormat Format ) const
        {
            const std::array<uint8, SIZE_T(EImageFormat::Count)>* it = SupportedFormats.Find(Op);
            if (!it)
            {
                return false;
            }

            return (*it)[SIZE_T(Format)]!=0;
        }


    private:

        //! Formats known to be supported for every instruction.
        //! Count*code.size() entries
        TMap< Ptr<ASTOp>, std::array<uint8, SIZE_T(EImageFormat::Count)> > SupportedFormats;

        //! Constant convenience initial value
        std::array<uint8, SIZE_T(EImageFormat::Count)> InitialState;

        //! Constant convenience initial value
        std::array<uint8, SIZE_T(EImageFormat::Count)> AllSupported;

    };


    //---------------------------------------------------------------------------------------------
    void SubtreeRelevantParametersVisitorAST::Run( Ptr<ASTOp> Root )
    {
    	MUTABLE_CPUPROFILER_SCOPE(SubtreeRelevantParametersVisitorAST);

        // Cached?
		TSet<FString>* it = ResultCache.Find( FState(Root,false) );
        if (it)
        {
            Parameters = *it;
            return;
        }

        // Not cached
        {

            Parameters.Empty();
			
			ASTOpList Roots;
			Roots.Add(Root);
    		
    		const bool bInitialState = false;
			Traverse(Roots, bInitialState);

            ResultCache.Add( FState(Root,false), Parameters );
        }
    }


    bool SubtreeRelevantParametersVisitorAST::Visit(const Ptr<ASTOp>& Op)
    {
    	switch (Op->GetOpType())
    	{
    	case EOpType::NU_PARAMETER:
    	case EOpType::SC_PARAMETER:
    	case EOpType::BO_PARAMETER:
    	case EOpType::CO_PARAMETER:
    	case EOpType::PR_PARAMETER:
    	case EOpType::IM_PARAMETER:
    	case EOpType::SK_PARAMETER:
    	case EOpType::MA_PARAMETER:
    	case EOpType::IS_PARAMETER:
    		{
    			const ASTOpParameter* TypedOp = static_cast<const ASTOpParameter*>(Op.get());
    			Parameters.Add(TypedOp->Parameter.Name);

    			// Not interested in the parameters from the parameters decorators.
    			return false;
    		}
    	
    	default:
    		break;
    	}

    	return true;
    }


    bool SubtreeResourceRelevantParametersVisitorAST::Visit(const Ptr<ASTOp>& Op)
    {
    	switch (Op->GetOpType())
    	{
    	case EOpType::SK_NEW:
    		{
    			const ASTOpSkeletalMeshNew* TypedOp = static_cast<const ASTOpSkeletalMeshNew*>(Op.get());
    		
    			for (const ASTChild& LOD : TypedOp->LODs)
    			{
    				RecurseWithCurrentState(LOD.child());
    			}

    			return false;
    		}
    		
    	case EOpType::SK_MATERIALMODIFY:
    		{
    			const ASTOpSkeletalMeshMaterialModify* TypedOp = static_cast<const ASTOpSkeletalMeshMaterialModify*>(Op.get());
    		
    			RecurseWithCurrentState(TypedOp->SkeletalMesh.child());

    			return false;
    		}
    		
    	default:
    		return SubtreeRelevantParametersVisitorAST::Visit(Op);
    	}
    }


    //---------------------------------------------------------------------------------------------
    //! Mark all the instructions that don't depend on runtime parameters but are below
    //! instructions that do.
    //! Also detect which instructions are the root of a resource that is dynamic in this state.
    //! Visitor state is:
    //!   .first IsResourceRoot
    //!   .second ParentIsRuntime
    //---------------------------------------------------------------------------------------------
    class StateCacheDetectorAST : public Visitor_TopDown_Unique_Const<bool>
    {
    public:

        StateCacheDetectorAST(FStateCompilationData* State )
            : HasRuntimeParamVisitor( State )
        {
            ASTOpList roots;
            roots.Add(State->root);
        	
        	const bool bInitialState = false;
			Traverse(roots, bInitialState);

            State->m_updateCache.Empty();
            State->m_dynamicResources.Empty();

            for( const TPair<Ptr<ASTOp>, bool>& i : Cache )
            {
                if ( i.Value )
                {
                    State->m_updateCache.Add( i.Key );
                }
            }

            for(const TPair<Ptr<ASTOp>, bool>& i : DynamicResourceRoot )
            {
                if ( i.Value )
                {
                    // Generate the list of relevant parameters
                    SubtreeRelevantParametersVisitorAST subtreeParams;
                    subtreeParams.Run( i.Key );

					// Temp copy
					TArray<FString> ParamCopy;
					for ( const FString& e: subtreeParams.Parameters )
					{
						ParamCopy.Add(e);
					}

                    State->m_dynamicResources.Emplace( i.Key, MoveTemp(ParamCopy) );
                }
            }
        }


        bool Visit( const Ptr<ASTOp>& At ) override
        {
            bool bThisIsRuntime = HasRuntimeParamVisitor.HasAny( At );

            bool bParentIsRuntime = GetCurrentState();

			Cache.FindOrAdd(At, false);

            EOpType Type = At->GetOpType();
            if ( GetOpToolsDesc( Type ).bCached )
            {
                // If parent is runtime, but we are not
                if (!bThisIsRuntime && bParentIsRuntime)
                {
                    // We want to cache this result to update the instances.
                    // Mark this as update cache
                    Cache.Add(At, true);
                }
            }

            if ( !Cache[At] && bThisIsRuntime )
            {
                DynamicResourceRoot.Add(At, true);
            }

            if ( !Cache[At] && bThisIsRuntime )
            {
            	SetCurrentState(bThisIsRuntime);
            	return true;
            }

            return false;
        }

    private:

        //!
        TMap<Ptr<ASTOp>,bool> Cache;
		TMap<Ptr<ASTOp>,bool> DynamicResourceRoot;

        RuntimeParameterVisitorAST HasRuntimeParamVisitor;
    };


    //---------------------------------------------------------------------------------------------
    //! Find out what images can be compressed during build phase of an instance so that the update
    //! cache can be smaller (and some update operations faster)
    //---------------------------------------------------------------------------------------------
    class StateCacheFormatOptimiserAST : public Visitor_TopDown_Unique_Cloning
    {
    public:

        StateCacheFormatOptimiserAST(FStateCompilationData& state,
                                   const AccumulateAllImageFormatsOpAST& opFormats )
            : m_state(state)
            , m_opFormats(opFormats)
        {
            Traverse( state.root );
        }


    protected:

        Ptr<ASTOp> Visit( Ptr<ASTOp> At, bool& processChildren ) override
        {
            processChildren = true;

            bool isUpdateCache = m_state.m_updateCache.Contains(At);

            if ( isUpdateCache )
            {
                // Its children cannot be update-cache, so no need to process them.
                processChildren = false;

                // See if we can convert it to a more efficient format
                if ( GetOpDataType( At->GetOpType() )== EDataType::Image )
                {
                    FImageDesc desc = At->GetImageDesc();

                    if ( desc.m_format!=EImageFormat::L_UByteRLE
                         &&
                         m_opFormats.IsSupportedFormat(At, EImageFormat::L_UByteRLE) )
                    {
                        Ptr<ASTOpImagePixelFormat> op = new ASTOpImagePixelFormat;
                        op->Format = EImageFormat::L_UByteRLE;
                        // Note: we have to clone here, to avoid a loop with the visitor system
                        // that updates visited children before processing a node.
						ASTOp::MapChildFunc Identity = [](const Ptr<ASTOp>& o) {return o;};
						op->Source = At->Clone(Identity);

                        At = op;
                    }
                }

            }

            return At;
        }

    private:

		FStateCompilationData& m_state;

        const AccumulateAllImageFormatsOpAST& m_opFormats;
    };


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    RuntimeTextureCompressionRemoverAST::RuntimeTextureCompressionRemoverAST(
		FStateCompilationData* State,
		bool bInAlwaysUncompress
	)
        : HasRuntimeParamVisitor(State)
		, bAlwaysUncompress(bInAlwaysUncompress)
    {
        Traverse( State->root );
    }


    Ptr<ASTOp> RuntimeTextureCompressionRemoverAST::Visit( Ptr<ASTOp> At, bool& processChildren )
    {
        EOpType type = At->GetOpType();
        
    	const EDataType DataType = GetOpDataType(type);
    	switch (DataType)
    	{
	    case EDataType::None:
	    case EDataType::Material:
	    case EDataType::Instance:
	    case EDataType::InstancedStruct:
		case EDataType::SkeletalMesh:
	    case EDataType::LOD:
    		processChildren = true;
		    break;
    		
	    default: 
    		processChildren = false;
    	}

        // TODO: Finer grained: what if the runtime parameter just selects between compressed
        // textures? We don't want them uncompressed.

        if (type == EOpType::MI_MODIFY)
        {
			ASTOpMaterialModify* TypedOp = static_cast<ASTOpMaterialModify*>(At.get());

			if (TypedOp)
			{
				bool bHasChanged = false;

				// Check if we need to add a compress pixel format to uncompress any texture of the material modify
				for (TPair<FParameterKey, ASTChild>& Parameter : TypedOp->ParametersToModify)
				{
					if (!Parameter.Value.Child || !Parameter.Value.Child.get() || GetOpDataType(Parameter.Value.Child->GetOpType()) != EDataType::Image)
					{
						continue;
					}

					Ptr<ASTOp> ImageOp = Parameter.Value.child();

					// Does it have a runtime parameter in its subtree?
					bool hasRuntimeParameter = HasRuntimeParamVisitor.HasAny(ImageOp);

					if (bAlwaysUncompress || hasRuntimeParameter)
					{
						FImageDesc imageDesc = ImageOp->GetImageDesc(true);

						// Is it a compressed format?
						EImageFormat format = imageDesc.m_format;
						EImageFormat uncompressedFormat = GetUncompressedFormat(format);
						bool isCompressedFormat = (uncompressedFormat != format);

						if (isCompressedFormat)
						{
							bHasChanged = true;
							break;
						}
					}
				}

				// Clone the operation and add a compressed pixel format operation if needed
				if (bHasChanged)
				{
					Ptr<ASTOpMaterialModify> NewOp = UE::Mutable::Private::Clone<ASTOpMaterialModify>(TypedOp);

					// If needed, add a a compressed pixel format operation for all images 
					for (TPair<FParameterKey, ASTChild>& Parameter : NewOp->ParametersToModify)
					{
						if (GetOpDataType(Parameter.Value.Child->GetOpType()) != EDataType::Image)
						{
							continue;
						}

						Ptr<ASTOp> ImageOp = Parameter.Value.child();

						// Does it have a runtime parameter in its subtree?
						bool hasRuntimeParameter = HasRuntimeParamVisitor.HasAny(ImageOp);

						if (bAlwaysUncompress || hasRuntimeParameter)
						{
							FImageDesc imageDesc = ImageOp->GetImageDesc(true);

							// Is it a compressed format?
							EImageFormat format = imageDesc.m_format;
							EImageFormat uncompressedFormat = GetUncompressedFormat(format);
							bool isCompressedFormat = (uncompressedFormat != format);

							if (isCompressedFormat)
							{
								// Add a new format operation to uncompress the image
								Ptr<ASTOpImagePixelFormat> fop = new ASTOpImagePixelFormat;
								fop->Format = uncompressedFormat;
								fop->FormatIfAlpha = uncompressedFormat;
								fop->Source = ImageOp;

								Parameter.Value = fop;
							}
						}
					}

					At = NewOp;
				}
			}
        }

        return At;
    }


	LODCountReducerAST::LODCountReducerAST( Ptr<ASTOp>& root, uint8 NumExtraLODsToBuildAfterFirstLOD )
    {
        NumExtraLODs = NumExtraLODsToBuildAfterFirstLOD;
        Traverse( root );
    }


    Ptr<ASTOp> LODCountReducerAST::Visit( Ptr<ASTOp> At, bool& processChildren )
    {
        processChildren = true;

    	if (At->GetOpType() == EOpType::SKO_CONVERT)
    	{
    		ASTOpSkeletalMeshObjectConvert* TypedOp = static_cast<ASTOpSkeletalMeshObjectConvert*>(At.get());

    		const int32 NumLODs = TypedOp->FirstLODAvailable + NumExtraLODs + 1;
    		
    		if (TypedOp->NumLODs != NumLODs)
    		{
    			Ptr<ASTOpSkeletalMeshObjectConvert> NewOp = UE::Mutable::Private::Clone<ASTOpSkeletalMeshObjectConvert>(TypedOp);

    			NewOp->NumLODs = NumLODs;
    			NewOp->FirstLODResident = FMath::Clamp(NewOp->FirstLODResident, NewOp->FirstLODAvailable, NewOp->NumLODs - 1);
    		
    			At = NewOp;
    		}
    	}
    	else if (At->GetOpType() == EOpType::SK_NEW)
        {
			ASTOpSkeletalMeshNew* TypedOp = static_cast<ASTOpSkeletalMeshNew*>(At.get());

        	// Search for the first LOD that has a valid mesh.
        	const int32 FirstLOD = TypedOp->LODs.IndexOfByPredicate([](const ASTChild& Element)
        	{
       			return Element.child();
        	});

        	const int32 NumLODs = FirstLOD + NumExtraLODs + 1;
        	
            if (TypedOp->LODs.Num() > NumLODs)
            {
                Ptr<ASTOpSkeletalMeshNew> NewOp = UE::Mutable::Private::Clone<ASTOpSkeletalMeshNew>(At);
                while (NewOp->LODs.Num() > NumLODs)
                {
                    NewOp->LODs.Pop();
                }
                At = NewOp;
            }

            processChildren = false;
        }

        return At;
    }


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    void CodeOptimiser::OptimiseStatesAST()
    {
		MUTABLE_CPUPROFILER_SCOPE(OptimiseStatesAST);

		for ( int32 s=0; s<States.Num(); ++s )
		{
			// Remove the unnecessary lods
			FStateOptimizationOptions StateOptimization = States[s].nodeState.Optimisation;
			if (StateOptimization.bOnlyFirstLOD)
			{
				LODCountReducerAST(States[s].root, StateOptimization.NumExtraLODsToBuildAfterFirstLOD);
			}

			// Apply texture compression strategy
			bool bModified = false;
			switch (StateOptimization.TextureCompressionStrategy)
			{
			case ETextureCompressionStrategy::DontCompressRuntime:
			{
				MUTABLE_CPUPROFILER_SCOPE(RuntimeTextureCompressionRemover);
				RuntimeTextureCompressionRemoverAST r(&States[s], false);
				bModified = true;
				break;
			}

			case ETextureCompressionStrategy::NeverCompress:
			{
				MUTABLE_CPUPROFILER_SCOPE(RuntimeTextureCompressionRemover);
				RuntimeTextureCompressionRemoverAST r(&States[s], true);
				bModified = true;
				break;
			}

			default:
				break;
			}

            // If a state has no runtime parameters, skip its optimisation alltogether
            if (bModified || States[s].nodeState.RuntimeParams.Num())
            {
                // Promote the intructions that depend on runtime parameters, and sink new
                // format instructions.
                bModified = true;
                int32 NumIterations = 0;
                while (bModified && ( OptimizeIterationsLeft>0 || !NumIterations))
                {
					bModified = false;

                    ++NumIterations;
                    --OptimizeIterationsLeft;
                    UE_LOGF(LogMutableCore, Verbose, "State optimise iteration %d, left %d", NumIterations, OptimizeIterationsLeft);

                    UE_LOGF(LogMutableCore, Verbose, " - before parameter optimiser");

                    ParameterOptimiserAST param( States[s], Options->GetPrivate()->OptimisationOptions );
					bModified = param.Apply();

                    TArray<Ptr<ASTOp>> Roots;
                    Roots.Add(States[s].root);

                    UE_LOGF(LogMutableCore, Verbose, " - after parameter optimiser");

                    // All kind of optimisations that depend on the meaning of each operation
                    UE_LOGF(LogMutableCore, Verbose, " - semantic optimiser");
					bModified |= SemanticOptimiserAST(Roots, Options->GetPrivate()->OptimisationOptions, 1 );

                    UE_LOGF(LogMutableCore, Verbose, " - sink optimiser");
					bModified |= SinkOptimiserAST(Roots, Options->GetPrivate()->OptimisationOptions );

                    // Image size operations are treated separately
                    UE_LOGF(LogMutableCore, Verbose, " - size optimiser");
					bModified |= SizeOptimiserAST(Roots);

					// Some sink optimizations can only be applied after some constant reductions
					for (Ptr<ASTOp>& Root : Roots)
					{
						bModified |= ConstantGenerator(Options->GetPrivate(), Root, 1);
					}

                }

                TArray<Ptr<ASTOp>> Roots;
                Roots.Add(States[s].root);

                UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
				bModified |= DuplicatedDataRemoverAST(Roots);

                UE_LOGF(LogMutableCore, Verbose, " - duplicated code remover");
				bModified |= DuplicatedCodeRemoverAST(Roots);

                States[s].root = Roots[0];
            }
        }


		TArray<Ptr<ASTOp>> Roots;
		for (const FStateCompilationData& s : States)
		{
			Roots.Add(s.root);
		}

        // Mark the instructions that don't depend on runtime parameters to be cached. This is
        // necessary At this stage before GPU optimisation.
        {
            AccumulateAllImageFormatsOpAST opFormats;
            opFormats.Run(Roots);

			// Reset the state root operations in case they have changed due to optimization
			for (int32 RootIndex = 0; RootIndex < States.Num(); ++RootIndex)
			{
				States[RootIndex].root = Roots[RootIndex];
			}

            for (FStateCompilationData& s: States )
            {
                {
                    UE_LOGF(LogMutableCore, Verbose, " - state cache");
                    MUTABLE_CPUPROFILER_SCOPE(StateCache);
                    StateCacheDetectorAST c( &s );
                }

                {
                    UE_LOGF(LogMutableCore, Verbose, " - state cache format");
                    MUTABLE_CPUPROFILER_SCOPE(StateCacheFormat);
                    StateCacheFormatOptimiserAST f( s, opFormats );
                }
            }
        }

        // Reoptimise because of state cache reformats
        {
            MUTABLE_CPUPROFILER_SCOPE(Reoptimise);
            bool bModified = true;
            int32 NumIterations = 0;
			int32 Pass = 1;
            while (bModified && (OptimizeIterationsLeft>0 || !NumIterations))
            {
                ++NumIterations;
                --OptimizeIterationsLeft;
                UE_LOGF(LogMutableCore, Verbose, "State reoptimise iteration %d, left %d", NumIterations, OptimizeIterationsLeft);

				bModified = false;

                UE_LOGF(LogMutableCore, Verbose, " - semantic optimiser");
				bModified |= SemanticOptimiserAST( Roots, Options->GetPrivate()->OptimisationOptions, Pass );

                // Image size operations are treated separately
                UE_LOGF(LogMutableCore, Verbose, " - size optimiser");
				bModified |= SizeOptimiserAST( Roots );
			}

            for(Ptr<ASTOp>& Root : Roots)
            {
                UE_LOGF(LogMutableCore, Verbose, " - constant optimiser");
				bModified = ConstantGenerator( Options->GetPrivate(), Root, Pass );
			}

            UE_LOGF(LogMutableCore, Verbose, " - duplicated data remover");
            DuplicatedDataRemoverAST( Roots );

            UE_LOGF(LogMutableCore, Verbose, " - duplicated code remover");
            DuplicatedCodeRemoverAST( Roots );
		}

		// Reset the state root operations in case they have changed due to optimization
		for (int32 RootIndex = 0; RootIndex < States.Num(); ++RootIndex)
		{
			States[RootIndex].root = Roots[RootIndex];
		}

        // Optimise the data formats
        {
            MUTABLE_CPUPROFILER_SCOPE(DataFormats);

            DataOptimise( Options.get(), Roots);

            // After optimising the data formats, we may remove more constants
            DuplicatedDataRemoverAST( Roots );
            DuplicatedCodeRemoverAST( Roots );

            // Update the marks for the instructions that don't depend on runtime parameters to be cached.
            for (FStateCompilationData& s:States)
            {
                StateCacheDetectorAST c( &s );
            }
        }

    }


}

