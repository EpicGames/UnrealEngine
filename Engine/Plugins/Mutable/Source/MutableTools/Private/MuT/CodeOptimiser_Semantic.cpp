// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeImageFormat.h"
#include "MuT/AST.h"
#include "MuT/ASTOpConditional.h"
#include "MuT/ASTOpConstantBool.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMipmap.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImagePatch.h"
#include "MuT/ASTOpImagePixelFormat.h"
#include "MuT/ASTOpImageMakeGrowMap.h"
#include "MuT/ASTOpImageSwizzle.h"
#include "MuT/ASTOpImageTransform.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/ASTOpImageResize.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshAddMetadata.h"
#include "MuT/ASTOpSwitch.h"
#include "MuT/CodeOptimiser.h"
#include "MuR/Image.h"
#include "MuR/MutableMath.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Table.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"


namespace UE::Mutable::Private
{

    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    bool SemanticOptimiserAST(
		ASTOpList& Roots,
		const FModelOptimizationOptions& OptimisationOptions,
		int32 Pass
	)
    {
        MUTABLE_CPUPROFILER_SCOPE(SemanticOptimiserAST);

        bool bModified = false;

        // TODO: isn't top down better suited?
        ASTOp::Traverse_BottomUp_Unique( Roots, [&](Ptr<ASTOp>& CurrentOp)
        {
            Ptr<ASTOp> OptimizedOp = CurrentOp->OptimiseSemantic(OptimisationOptions, Pass);

            // If the returned value is null it means no change.
            if (OptimizedOp && OptimizedOp !=CurrentOp)
            {
				bModified = true;
                ASTOp::Replace(CurrentOp, OptimizedOp);

				// Check if we are replacing one of the root operations and update the parameter array.
				for (int32 RootIndex=0; RootIndex<Roots.Num(); ++RootIndex)
				{
					if (Roots[RootIndex]== CurrentOp)
					{
						Roots[RootIndex] = OptimizedOp;
					}
				}
            }
        });

        return bModified;
    }


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    bool SinkOptimiserAST
    (
            ASTOpList& InRoots,
            const FModelOptimizationOptions& InOptimisationOptions
    )
    {
        MUTABLE_CPUPROFILER_SCOPE(SinkOptimiserAST);

        bool bModified = false;

		FOptimizeSinkContext Context;

        ASTOp::Traverse_TopDown_Unique_Imprecise(InRoots, [&](Ptr<ASTOp>& n)
        {
            auto o = n->OptimiseSink(InOptimisationOptions, Context);
            if (o && n!=o)
            {
				bModified = true;
                ASTOp::Replace(n,o);
            }

            return true;
        });

        return bModified;
    }

    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    bool SizeOptimiserAST( ASTOpList& roots )
    {
        MUTABLE_CPUPROFILER_SCOPE(SizeOptimiser);

        bool modified = false;

        // TODO: isn't top down better suited?
        ASTOp::Traverse_BottomUp_Unique( roots, [&](Ptr<ASTOp>& n)
        {
            auto o = n->OptimiseSize();
            if (o && o!=n)
            {
                modified = true;
                ASTOp::Replace(n,o);
            }
        });

        return modified;
    }


}
