// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/Compiler.h"

#include "MuT/CompilerPrivate.h"
#include "MuT/AST.h"
#include "MuT/ASTOpParameter.h"
#include "MuT/CodeGenerator.h"
#include "MuT/CodeOptimiser.h"
#include "MuT/ErrorLog.h"
#include "MuT/Node.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Parameters.h"
#include "MuR/Serialisation.h"
#include "MuR/MutableRuntimeModule.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/AssertionMacros.h"
#include "Async/ParallelFor.h"

#include <inttypes.h>	// Required for 64-bit printf macros

namespace UE::Mutable::Private
{

    const char* CompilerOptions::GetTextureLayoutStrategyName( CompilerOptions::TextureLayoutStrategy s )
    {
        static const char* s_textureLayoutStrategyName[ 2 ] =
        {
            "Unscaled Pack",
            "No Packing",
        };

        return s_textureLayoutStrategyName[(int)s];
    }


	// clang-format off
	static const FOpToolsDesc SOpToolsDescs[] =
	{
		// cached	supported base image formats
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// NONE

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// BO_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// BO_PARAMETER
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// BO_EQUAL_SC_CONST
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// BO_AND
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// BO_OR
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// BO_NOT

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// NU_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// NU_PARAMETER
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// NU_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// NU_SWITCH

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_PARAMETER
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_SWITCH
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_MATERIAL_BREAK
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_ARITHMETIC
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_CURVE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SC_EXTERNAL

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_PARAMETER
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_SWITCH
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_MATERIAL_BREAK
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_SAMPLEIMAGE
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_SWIZZLE
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_FROMSCALARS
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_ARITHMETIC
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_LINEARTOSRGB
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// CO_EXTERNAL

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ST_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ST_PARAMETER

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// PR_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// PR_PARAMETER

		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_PARAMETER
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_REFERENCE
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 1, 1, 0, 0, 0 }		},	// IM_CONDITIONAL
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 1, 1, 0, 0, 0 }		},	// IM_SWITCH
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_MATERIAL_BREAK
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_PARAMETER_FROM_MATERIAL
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_LAYER
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_LAYERCOLOR
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_PIXELFORMAT
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_CONFORM
		{ true,		{ 1, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_MIPMAP
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_RESIZE
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_RESIZELIKE
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_RESIZEREL
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_BLANKLAYOUT
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 0, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 1, 1, 0, 0, 0 }		},	// IM_COMPOSE
		{ true,		{ 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 0, 1, 1, 1, 1, 1, 1, 1, 1, 0, 0, 1, 1, 0, 0, 0 }		},	// IM_MULTICOMPOSE
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_INTERPOLATE
		{ true,		{ 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_SATURATE
		{ true,		{ 0, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_LUMINANCE
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_SWIZZLE
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_COLORMAP
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_BINARISE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_PLAINCOLOR
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_CROP
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_PATCH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_RASTERMESH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_MAKEGROWMAP
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_DISPLACE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_MULTILAYER
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_INVERT
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_NORMALCOMPOSITE
		{ true,		{ 0, 1, 1, 1, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_TRANSFORM
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_PARAMETER_CONVERT
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IM_EXTERNAL

		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_PARAMETER
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_REFERENCE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_CONDITIONAL
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_SWITCH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_APPLYLAYOUT
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_PREPARELAYOUT
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_DIFFERENCE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_MORPH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_MERGE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_MASKCLIPMESH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_MASKCLIPUVMASK
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_MASKDIFF
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_REMOVEMASK
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_FORMAT
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_EXTRACTLAYOUTBLOCK
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_TRANSFORM
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_CLIPMORPHPLANE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_CLIPWITHMESH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_PROJECT
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_APPLYPOSE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_BINDSHAPE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_APPLYSHAPE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_CLIPDEFORM
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_ADDMETADATA
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_SETMATERIALSLOTID
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_TRANSFORMWITHMESH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_TRANSFORMWITHBONE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_EXTERNAL
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ME_SKELETALMESH_BREAK

		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LD_CONDITIONAL,
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LD_SWITCH
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LD_NEW

		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_CONDITIONAL
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_SWITCH
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_NEW
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_PARAMETER
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_MERGE
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_CONVERT
    	{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_MORPH
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_RESHAPE
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_MATERIALMODIFY
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_CLIPMESHWITHMESH
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_TRANSFORM
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SK_TRANSFORMWITHBONE
    	
		{ true,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// SKO_CONVERT
    	
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_SWITCH
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_ADDCOMPONENT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_ADDSKELETALMESH
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_ADDEXTENSIONDATA
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_ADDOVERLAYMATERIAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IN_ADDOVERRIDEMATERIAL

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_SWITCH
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_PACK
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_MERGE
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_REMOVEBLOCKS
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// LA_FROMMESH

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_PARAMETER
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_SWITCH
    	{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_SKELETALMESHOBJECT_BREAK
    	{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_SKELETALMESH_BREAK
    	{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_FROM_SKELETALMESH_SLOT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_MODIFY
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MI_EXTERNAL

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MA_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// MA_PARAMETER
		
		{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ED_CONSTANT
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ED_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// ED_SWITCH

		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IS_PARAMETER
    	{ true,		{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IS_EXTERNAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IS_CONDITIONAL
		{ false,	{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0 }		},	// IS_SWITCH
	};

	// clang-format on

	static_assert(UE_ARRAY_COUNT(SOpToolsDescs) == (int32)EOpType::COUNT, "OperationDescMismatch");

	const FOpToolsDesc& GetOpToolsDesc(EOpType type)
	{
		return SOpToolsDescs[(int32)type];
	}


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    CompilerOptions::CompilerOptions()
    {
        m_pD = new Private();
    }


    //---------------------------------------------------------------------------------------------
    CompilerOptions::~CompilerOptions()
    {
        check( m_pD );
        delete m_pD;
        m_pD = 0;
    }


    //---------------------------------------------------------------------------------------------
    CompilerOptions::Private* CompilerOptions::GetPrivate() const
    {
        return m_pD;
    }


    //---------------------------------------------------------------------------------------------
    void CompilerOptions::SetLogEnabled( bool bEnabled)
    {
        m_pD->bLog = bEnabled;
    }


    //---------------------------------------------------------------------------------------------
    void CompilerOptions::SetOptimisationEnabled( bool bEnabled)
    {
        m_pD->OptimisationOptions.bEnabled = bEnabled;
        if (bEnabled)
        {
            m_pD->OptimisationOptions.bConstReduction = true;
        }
    }


    //---------------------------------------------------------------------------------------------
    void CompilerOptions::SetConstReductionEnabled( bool bConstReductionEnabled )
    {
        m_pD->OptimisationOptions.bConstReduction = bConstReductionEnabled;
    }


    //---------------------------------------------------------------------------------------------
    void CompilerOptions::SetOptimisationMaxIteration( int32 MaxIterations )
    {
        m_pD->OptimisationOptions.MaxOptimisationLoopCount = MaxIterations;
    }


    //---------------------------------------------------------------------------------------------
    void CompilerOptions::SetIgnoreStates( bool bIgnore )
    {
        m_pD->bIgnoreStates = bIgnore;
    }


	//---------------------------------------------------------------------------------------------
	void CompilerOptions::SetImageCompressionQuality(int32 Quality)
	{
		m_pD->ImageCompressionQuality = Quality;
	}


	//---------------------------------------------------------------------------------------------
	void CompilerOptions::SetDataPackingStrategy(int32 MinTextureResidentMipCount, uint64 EmbeddedDataBytesLimit)
	{
		m_pD->EmbeddedDataBytesLimit = EmbeddedDataBytesLimit;
		m_pD->MinTextureResidentMipCount = MinTextureResidentMipCount;
	}


	//---------------------------------------------------------------------------------------------
	void CompilerOptions::SetEnableProgressiveImages(bool bEnabled)
	{
		m_pD->OptimisationOptions.bEnableProgressiveImages = bEnabled;
	}


	//---------------------------------------------------------------------------------------------
	void CompilerOptions::SetImagePixelFormatOverride(const FImageOperator::FImagePixelFormatFunc& InFunc)
	{
		m_pD->ImageFormatFunc = InFunc;
	}


	//---------------------------------------------------------------------------------------------
	void CompilerOptions::SetReferencedResourceCallback(const FReferencedImageResourceFunc& Image, const FReferencedMeshResourceFunc& Mesh)
	{
		m_pD->OptimisationOptions.ReferencedImageResourceProvider = Image;
		m_pD->OptimisationOptions.ReferencedMeshResourceProvider = Mesh;
	}
	
	void CompilerOptions::SetDisableMaterialGeneration(bool bDisabled)
	{
		m_pD->OptimisationOptions.bDisableMaterialGeneration = bDisabled;
	}
	
	//---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    Compiler::Compiler( Ptr<CompilerOptions> options, TFunction<void()>& InWaitCallback)
    {
        m_pD = new Private();
		m_pD->Options = options;
		m_pD->WaitCallback = InWaitCallback;
		if (!m_pD->Options)
        {
            m_pD->Options = new CompilerOptions;
        }
    }


    //---------------------------------------------------------------------------------------------
    Compiler::~Compiler()
    {
        check( m_pD );
        delete m_pD;
        m_pD = 0;
    }

    namespace Private
    {
        void RemoveMaterialOps(const TArray<FStateCompilationData>& States)
        {
            MUTABLE_CPUPROFILER_SCOPE(RemoveDisabledAddInstanceOpsValues);
            
            TArray<Ptr<ASTOp>> Roots;
            for(const FStateCompilationData& State : States)
            {
                Roots.Add(State.root);
            }

            TArray<Ptr<ASTOp>> OpsToRemove;
        	
        	ASTOp::Traverse_TopDown_Unique_Imprecise(Roots, [&](Ptr<ASTOp> Node)
			{
				if (GetOpDataType(Node->GetOpType()) == EDataType::Material)
				{
					OpsToRemove.Add(Node);
				}
        			
				return true;
			});

            for (Ptr<ASTOp>& Op : OpsToRemove)
            {
            	ASTOp::Replace(Op, nullptr);
           }
        }
    }

    //---------------------------------------------------------------------------------------------
	TSharedPtr<FModel> Compiler::Compile( const Ptr<Node>& pNode, TMap<FOperation::ADDRESS, FInstancedStruct>& ExternalOperations)
    {
        MUTABLE_CPUPROFILER_SCOPE(Compile);

		double StartTimeSeconds = FPlatformTime::Seconds();

        TArray< FStateCompilationData > States;
        TSharedPtr<FErrorLog> GenErrorLog;
		TArray<FParameterDesc> Parameters;
        {
            CodeGenerator Generator( m_pD->Options->GetPrivate(), m_pD->WaitCallback );

            Generator.GenerateRoot( pNode );

            check( !Generator.States.IsEmpty() );

            for ( const TPair<FObjectState, Ptr<ASTOp>>& s: Generator.States )
            {
                FStateCompilationData Data;
                Data.nodeState = s.Key;
                Data.root = s.Value;
                Data.state.Name = s.Key.Name;
                States.Add( Data );
            }

            GenErrorLog = Generator.ErrorLog;

			{
				UE::TUniqueLock Lock(Generator.FirstPass.ParameterNodes.Mutex);

				// Gather the parameter list from the non-optimized data, so that we have them all even if they are optimized out
				{
					const int32 ParameterCount = Generator.FirstPass.ParameterNodes.GenericParametersCache.Num();

					Parameters.Reserve(ParameterCount);
					for (const TPair<Ptr<const Node>, Ptr<ASTOpParameter>>& Entry : Generator.FirstPass.ParameterNodes.GenericParametersCache)
					{
						Parameters.Add(Entry.Value->Parameter);
					}
				}

				// Sort the parameters as deterministically as possible.
				Parameters.StableSort([](const FParameterDesc& A, const FParameterDesc& B)->bool
					{
						if (A.Name < B.Name)
						{
							return true;
						}
						if (A.Name > B.Name)
						{
							return false;
						}
						return A.UID < B.UID;
					});

        		// Check for repeated parameter names and report them to the user
				if (Parameters.Num() > 1)
				{
					// todo: UE-316151 : Allow the repeated parameter names log to display a list of n nodes (instead of just 2)
					bool bRepeatedParameterFound = false;
					TArray<FString> RepeatedParameterNames;
				
					for (int32 Index = 1; Index < Parameters.Num(); ++Index)
					{
						const FString CurrentParameterName = Parameters[Index].Name;
						const FString PreviousParameterName = Parameters[Index-1].Name;
					
						// Found a repeated parameter that has not been reported yet
						if (CurrentParameterName.Compare(PreviousParameterName) == 0 && !RepeatedParameterNames.Contains(CurrentParameterName))
						{
							bRepeatedParameterFound = true;
							
							// Find the nodes defining these repeated parameters
							TArray<const void*, TInlineAllocator<4>> MessageContexts;
							{
								for (const TPair<Ptr<const Node>, Ptr<ASTOpParameter>>& Entry : Generator.FirstPass.ParameterNodes.GenericParametersCache)
								{
									if (Entry.Value->Parameter.Name.Compare(CurrentParameterName) == 0)
									{
										check(Entry.Key->GetMessageContext());
										MessageContexts.Add(Entry.Key->GetMessageContext());
									}
								}
							}

							// The "ensure" will fail if:
							//		+ A single MessageContexts is found :	A single UE parameter node does get compiled to multiple core compilation parameter nodes. Ensure you are using the params cache.
							//		+ No MessageContexts are found :		There are parameter core compilation nodes are being generated, and they are not getting a context set in them. Ensure all nodes generated get a context set.
							if (ensureMsgf(MessageContexts.Num() >= 2, TEXT("An unexpected amount of parameter contexts (%u) have been found while reporting a repeated parameter."), MessageContexts.Num()))
							{
								// Report repeated parameter name alongside with the nodes that use it
								const FString WarningText = FString::Printf(TEXT("Repeated parameter found : \"%s\". Please use a different name."), *CurrentParameterName);
								GenErrorLog->Add(WarningText, ELMT_ERROR, MessageContexts);
								RepeatedParameterNames.Add(CurrentParameterName);
							}
						}
					}

					// Early out if a repeated parameter name has been found
					if (bRepeatedParameterFound)
					{
						// Merge the log in the right order
						GenErrorLog->Merge( m_pD->ErrorLog.Get() );
						m_pD->ErrorLog = GenErrorLog;
						return nullptr;
					}

				}
			}
		}
		
        const bool bDisableMaterialGeneration = m_pD->Options->GetPrivate()->OptimisationOptions.bDisableMaterialGeneration;
		if (bDisableMaterialGeneration)
		{
			UE::Mutable::Private::Private::RemoveMaterialOps(States);
		}
        
        // Optimize the generated code
        {
            CodeOptimiser Optimiser( m_pD->Options, States );
            Optimiser.Optimise();
        }

        // Link the Program and generate state data.
		TSharedPtr<FModel> Result = MakeShared<FModel>();
        FProgram& Program = Result->GetProgram();

		check(Program.Parameters.IsEmpty());
		Program.Parameters = Parameters;

		// Preallocate ample memory
		Program.ByteCode.Reserve(16 * 1024 * 1024);

		// Keep the link options outside the scope because it is also used to cache constant data that has already been 
		// added and could be reused across states.
		FImageOperator ImOp = FImageOperator::GetDefault(m_pD->Options->GetPrivate()->ImageFormatFunc);
		FLinkerOptions LinkerOptions(ImOp);

		LinkerOptions.ExternalOperations = &ExternalOperations;
			
		for(FStateCompilationData& State : States )
        {
			LinkerOptions.MinTextureResidentMipCount = m_pD->Options->GetPrivate()->MinTextureResidentMipCount;

            if (State.root)
            {
				State.state.Root = ASTOp::FullLink(State.root, Program, &LinkerOptions);
            }
            else
            {
                State.state.Root = 0;
            }
        }

		Program.ByteCode.Shrink();

        // Set the runtime parameter indices.
        for(FStateCompilationData& State : States )
        {
            for ( int32 RuntimeParameterIndex=0; RuntimeParameterIndex <State.nodeState.RuntimeParams.Num(); ++RuntimeParameterIndex)
            {
                int32 ParameterIndex = -1;
                for ( int32 i=0; ParameterIndex<0 && i<Program.Parameters.Num(); ++i )
                {
                    if ( Program.Parameters[i].Name
                         ==
                         State.nodeState.RuntimeParams[RuntimeParameterIndex] )
                    {
                        ParameterIndex = (int)i;
                    }
                }

                if (ParameterIndex>=0)
                {
                    State.state.m_runtimeParameters.Add( ParameterIndex );
                }
                else
                {
					FString WarningString = FString::Printf(TEXT(
						"The state [%s] refers to a parameter [%s]  that has not been found in the model. This error can be "
						"safely dismissed in case of partial compilation."), 
						*State.nodeState.Name,
						*State.nodeState.RuntimeParams[RuntimeParameterIndex]);
                    m_pD->ErrorLog->Add(WarningString, ELMT_WARNING, pNode->GetMessageContext() );
                }
            }

            // Generate the mask of update cache ops
            for (const Ptr<ASTOp>& Instruction : State.m_updateCache )
            {
                State.state.m_updateCache.Emplace(Instruction->LinkedAddress);
            }

            // Sort the update cache addresses for performance and determinism
            State.state.m_updateCache.Sort();

            // Generate the mask of dynamic resources
            for (const TPair<Ptr<ASTOp>, TArray<FString>>& Instruction : State.m_dynamicResources )
            {
                uint64 RelevantMask = 0;
                for (const FString& InstructionParameter : Instruction.Value )
                {
                    // Find the index in the model parameter list
                    int ParameterIndex = -1;
                    for ( int32 i=0; ParameterIndex<0 && i<Program.Parameters.Num(); ++i )
                    {
                        if ( Program.Parameters[i].Name == InstructionParameter )
                        {
                            ParameterIndex = (int)i;
                        }
                    }
                    check(ParameterIndex>=0);

                    // Find the position in the state data vector.
                    const int32 IndexInRuntimeList = State.state.m_runtimeParameters.Find( ParameterIndex );

                    if (IndexInRuntimeList !=INDEX_NONE )
                    {
                        RelevantMask |= uint64(1) << IndexInRuntimeList;
                    }
                }

                // TODO: this shouldn't happen but it seems to happen. Investigate.
                // Maybe something with the difference of precision between the relevant parameters
                // in operation subtrees.
                //check(relevantMask!=0);
                if (RelevantMask!=0)
                {
                    State.state.m_dynamicResources.Emplace(Instruction.Key->LinkedAddress, RelevantMask);
                }
            }            

            // Sort for performance and determinism
            State.state.m_dynamicResources.Sort();

            Program.States.Add(State.state);
        }

        // Merge the log in the right order
        GenErrorLog->Merge( m_pD->ErrorLog.Get() );
        m_pD->ErrorLog = GenErrorLog;

		// Pack data
		m_pD->GenerateRoms(Result.Get(), LinkerOptions.AdditionalData );
		
		// We are not touching the program anymore. Ensure we are not wasting memory.
		Program.Roms.Shrink();
		Program.ConstantImageLODsPermanent.Shrink();
		Program.ConstantImageLODIndices.Shrink();
		Program.ConstantImages.Shrink();
		Program.ConstantMeshesPermanent.Shrink();
		Program.ConstantStrings.Shrink();
		Program.ConstantUInt32Lists.Shrink();
		Program.ConstantInt32Lists.Shrink();
		Program.ConstantUInt64Lists.Shrink();
		Program.ConstantFloatLists.Shrink();
		Program.ConstantBoolLists.Shrink();
		Program.ConstantSkeletons.Shrink();
		Program.ConstantPhysicsBodies.Shrink();
		Program.Parameters.Shrink();

		FOutputSizeStream ProgramSizeStream;
		{
			MUTABLE_CPUPROFILER_SCOPE(ComputeProgramSize)

			FOutputArchive ProgramMemoryArch(&ProgramSizeStream);
			Program.Serialise(ProgramMemoryArch);
		}

		UE_LOGF(LogMutableCore, Log, "Compilation Time: %lfs", FPlatformTime::Seconds() - StartTimeSeconds);

		const double ProgramSize  = double(ProgramSizeStream.GetBufferSize());
		const double ByteCodeSize = double(Program.ByteCode.Num());
		constexpr double ByteToMiB = 1.0/(1024.0*1024.0); 

		UE_LOGF(LogMutableCore, Log, "Program Size : %lf MiB", ProgramSize * ByteToMiB);
		UE_LOGF(LogMutableCore, Log, "Program ByteCode Size: %lf MiB", ByteCodeSize * ByteToMiB);
		UE_LOGF(LogMutableCore, Log, "Program Num Ops : %d", Program.NumOps);

        return Result;
    }


    //---------------------------------------------------------------------------------------------
	TSharedPtr<FErrorLog> Compiler::GetLog() const
    {
        return m_pD->ErrorLog;
    }


	//---------------------------------------------------------------------------------------------
	void Compiler::Private::GenerateRoms(FModel* Model, const FLinkerOptions::FAdditionalData& AdditionalData )
	{
		LLM_SCOPE_BYNAME(TEXT("MutableRuntime"));
		MUTABLE_CPUPROFILER_SCOPE(GenerateRoms);

		uint64 EmbeddedDataBytesLimit = Options->GetPrivate()->EmbeddedDataBytesLimit;

		UE::Mutable::Private::FProgram& Program = Model->GetProgram();

		// These are used for logging only.
		int32 NumRoms = 0;
		uint64 NumRomsBytes = 0;
		int32 NumEmbedded = 0;
		int32 NumEmbeddedBytes = 0;
		int32 NumHighRes = 0;
		uint64 NumHighResBytes = 0;

		// Maximum number of roms
		int32 MaxRomCount = Program.ConstantImageLODsPermanent.Num() + Program.ConstantMeshesPermanent.Num();
		Program.Roms.Reserve(MaxRomCount);

		TSet<uint32> UsedIds;
		UsedIds.Reserve(MaxRomCount);

		int32 MaxResources = FMath::Max(Program.ConstantImageLODsPermanent.Num(), Program.ConstantMeshesPermanent.Num());
		TArray<FRomDataRuntime> RomDatas;
		RomDatas.SetNumZeroed(MaxResources);
		TArray<FRomDataCompile> RomDatasCompile;
		RomDatasCompile.SetNumZeroed(MaxResources);

		// Images
		{
			MUTABLE_CPUPROFILER_SCOPE(GenerateRoms_ImageIds);

			TArray<TManagedPtr<const FImage>> AllMips = MoveTemp(Program.ConstantImageLODsPermanent);
			Program.ConstantImageLODsPermanent = {};
			Program.ConstantImageLODsPermanent.Reserve(AllMips.Num());


			ParallelFor(AllMips.Num(),
				[EmbeddedDataBytesLimit, &Program, &AllMips, &RomDatas](uint32 MipIndex)
				{
					TManagedPtr<const FImage>& Resource = AllMips[MipIndex];

					// Serialize to find out final size of this rom
					FOutputSizeStream SizeStream;
					FOutputArchive MemoryArch(&SizeStream);
					FImage::Serialise(Resource.Get(), MemoryArch);


					FRomDataRuntime& RomData = RomDatas[MipIndex];
					RomData.ResourceType = uint32(ERomDataType::Image);
					uint32 Size = SizeStream.GetBufferSize();
					check(Size < uint32(1 << 30));
					RomData.Size = Size;
				});

			// Generate the high-res flags for images
			{
				// Initially all are high-res because if at least one reference to a mip is not we will set it to not-high-res
				TArray<bool> IsLODHighRes;
				IsLODHighRes.Init(true, AllMips.Num());

				for (int32 ImageIndex = 0; ImageIndex < Program.ConstantImages.Num(); ++ImageIndex)
				{
					const FImageLODRange& LODRange = Program.ConstantImages[ImageIndex];

	                int32 OptionalMaxLODSize = AdditionalData.SourceImagePerConstant[ImageIndex].OptionalMaxLODSize;
                    int32 OptionalLODBias = AdditionalData.SourceImagePerConstant[ImageIndex].OptionalLODBias; 
                    int32 NumNonOptionalLODs = AdditionalData.SourceImagePerConstant[ImageIndex].NumNonOptionalLODs; 

                    int32 NumOptionalMips = 0; 
                    if (OptionalMaxLODSize > 0)
                    {
                        int32 NumTotalLODs = FMath::CeilLogTwo(FMath::Max(LODRange.ImageSizeX, LODRange.ImageSizeY)) + 1;
                        int32 FirstOptionalLOD = FMath::Min<int32>(FMath::CeilLogTwo(OptionalMaxLODSize) + 1, NumTotalLODs);

                        NumOptionalMips = FMath::Min<int32>(
                                NumTotalLODs - (FirstOptionalLOD - OptionalLODBias), FMath::Max<int32>(0, NumTotalLODs - NumNonOptionalLODs));
                    }

					int32 NumLODImages = LODRange.LODCount - LODRange.NumLODsInTail + 1; 
					for (int32 LODRangeIndex = NumOptionalMips; LODRangeIndex < NumLODImages; ++LODRangeIndex)
					{
						FConstantResourceIndex LODIndex = Program.ConstantImageLODIndices[LODRange.FirstIndex + LODRangeIndex];
						check(!LODIndex.Streamable); // Not classified yet
						IsLODHighRes[LODIndex.Index] = false;
					}

					// Moreover, at least one mip of each image has to be non-highres
					if (LODRange.LODCount > 0)
					{
						FConstantResourceIndex LastLODIndex = Program.ConstantImageLODIndices[LODRange.FirstIndex + NumLODImages - 1];
						check(!LastLODIndex.Streamable); // Not classified yet
						IsLODHighRes[LastLODIndex.Index] = false;
					}
				}

				for (int32 MipIndex = 0; MipIndex < AllMips.Num(); ++MipIndex)
				{
					// If this mip represents a high-quality mip, flag the rom as such
					if (IsLODHighRes[MipIndex])
					{
						FRomDataRuntime& RomData = RomDatas[MipIndex];
						RomData.IsHighRes = true;

						++NumHighRes;
						NumHighResBytes += RomData.Size;
					}
				}
			}

			{
				MUTABLE_CPUPROFILER_SCOPE(GenerateRoms_ImageSourceIds);

				for (int32 ImageIndex = 0; ImageIndex < Program.ConstantImages.Num(); ++ImageIndex)
				{
					const FImageLODRange& LODRange = Program.ConstantImages[ImageIndex];

					uint32 SourceId = AdditionalData.SourceImagePerConstant[ImageIndex].SourceId;

					
					int32 NumLODImages = LODRange.LODCount - LODRange.NumLODsInTail + 1; 
					for (int32 LODRangeIndex = 0; LODRangeIndex < NumLODImages; ++LODRangeIndex)
					{
						FConstantResourceIndex ResourceIndex = Program.ConstantImageLODIndices[LODRange.FirstIndex + LODRangeIndex];
						check(!ResourceIndex.Streamable); // Not classified yet
						RomDatasCompile[ResourceIndex.Index].SourceId = SourceId;
					}
				}
			}

			// Split the data in permanent and streamable and assign final FConstantResourceIndex
			{
				MUTABLE_CPUPROFILER_SCOPE(GenerateRoms_ImageSplit);

				TArray<FConstantResourceIndex> IndexPerMip;
				IndexPerMip.SetNumUninitialized(AllMips.Num());

				for (int32 MipIndex = 0; MipIndex < AllMips.Num(); ++MipIndex)
				{
					FRomDataRuntime& RomData = RomDatas[MipIndex];
					if (RomData.Size > EmbeddedDataBytesLimit)
					{
						NumRoms++;
						NumRomsBytes += RomData.Size;

						uint32 RomIndex = Program.Roms.Num();
						check(RomIndex < 0x7fffffff);

						IndexPerMip[MipIndex] = {RomIndex, 1};
						Model->RomManager.SetRom<FImage>(RomIndex, AllMips[MipIndex]);

						Program.Roms.Add(RomData);

						FRomDataCompile& RomDataCompile = RomDatasCompile[MipIndex];
						Program.RomsCompileData.Add(RomDataCompile);
					}
					else
					{
						uint32 Index = uint32(Program.ConstantImageLODsPermanent.Num());
						check(Index < 0x7fffffff);
						IndexPerMip[MipIndex] = { Index, 0 };
						Program.ConstantImageLODsPermanent.Add(AllMips[MipIndex]);

						NumEmbedded++;
						NumEmbeddedBytes += RomData.Size;
					}
				}

				for (int32 Index = 0; Index < Program.ConstantImageLODIndices.Num(); ++Index)
				{
					int32 RomDataIndex = Program.ConstantImageLODIndices[Index].Index;
					Program.ConstantImageLODIndices[Index] = IndexPerMip[RomDataIndex];
				}
			}
		}

		// Meshes
		TArray<TManagedPtr<const FMesh>> AllMeshes = MoveTemp(Program.ConstantMeshesPermanent);
		Program.ConstantMeshesPermanent = {};
		Program.ConstantMeshesPermanent.Reserve(AllMeshes.Num());

		FMemory::Memzero( RomDatas.GetData(), RomDatas.GetAllocatedSize() );
		{
			MUTABLE_CPUPROFILER_SCOPE(GenerateRoms_ImageSourceIds);

			for (int32 MeshIndex = 0; MeshIndex < Program.ConstantMeshes.Num(); ++MeshIndex)
			{
				const FMeshContentRange& MeshContentRange = Program.ConstantMeshes[MeshIndex];

				const uint32 SourceId = AdditionalData.SourceMeshPerConstant[MeshIndex].SourceId;

				const int32 NumMeshContentElements = FMath::CountBits((uint32)MeshContentRange.GetContentFlags());

				for (int32 MeshContentIndex = 0; MeshContentIndex < NumMeshContentElements; ++MeshContentIndex)
				{
					FConstantResourceIndex ResourceIndex = 
							Program.ConstantMeshContentIndices[MeshContentRange.GetFirstIndex() + MeshContentIndex];

					check(!ResourceIndex.Streamable); // Not classified yet
					RomDatasCompile[ResourceIndex.Index].SourceId = SourceId;
				}
			}
		}

		{
			MUTABLE_CPUPROFILER_SCOPE(GenerateRoms_MeshIds);

			ParallelFor(AllMeshes.Num(),
				[EmbeddedDataBytesLimit, &AllMeshes , &Program, &RomDatas](uint32 ResourceIndex)
				{
					TManagedPtr<const FMesh>& ResData = AllMeshes[ResourceIndex];

					// Serialize to memory, to find out final size of this rom
					const FMesh* Resource = ResData.Get();

					FOutputSizeStream MemStream;
					FOutputArchive MemoryArch(&MemStream);
					FMesh::Serialise(ResData.Get(), MemoryArch);

					FRomDataRuntime& RomData = RomDatas[ResourceIndex];
					RomData.ResourceType = uint32(ERomDataType::Mesh);
					uint32 Size = MemStream.GetBufferSize();
					check(Size < uint32(1 << 30));
					RomData.Size = Size;
				});
		}

		// Split the data in permanent and streamable and assign final FConstantResourceIndex
		{
			MUTABLE_CPUPROFILER_SCOPE(GenerateRoms_MeshSplit);

			TArray<FConstantResourceIndex> IndexPerMeshContent;
			IndexPerMeshContent.SetNumUninitialized(AllMeshes.Num());
			
			for (int32 MeshIndex = 0; MeshIndex < AllMeshes.Num(); ++MeshIndex)
			{
				FRomDataRuntime& RomData = RomDatas[MeshIndex];
				FRomDataCompile& RomDataCompile = RomDatasCompile[MeshIndex];

				if (RomData.Size > EmbeddedDataBytesLimit)
				{
					NumRoms++;
					NumRomsBytes += RomData.Size;

					uint32 RomIndex = Program.Roms.Num();
					check(RomIndex < 0x7fffffff);

					IndexPerMeshContent[MeshIndex] = { RomIndex, 1 };
					Model->RomManager.SetRom<FMesh>(RomIndex, AllMeshes[MeshIndex]);

					Program.Roms.Add(RomData);
					Program.RomsCompileData.Add(RomDataCompile);
				}
				else
				{
					uint32 Index = uint32(Program.ConstantMeshesPermanent.Num());
					check(Index < 0x7fffffff);

					IndexPerMeshContent[MeshIndex] = { Index, 0 };
					Program.ConstantMeshesPermanent.Add(AllMeshes[MeshIndex]);

					NumEmbedded++;
					NumEmbeddedBytes += RomData.Size;
				}
			}

			for (int32 Index = 0; Index < Program.ConstantMeshContentIndices.Num(); ++Index)
			{
				int32 RomDataIndex = Program.ConstantMeshContentIndices[Index].Index;
				Program.ConstantMeshContentIndices[Index] = IndexPerMeshContent[RomDataIndex];
			}
		}

		UE_LOGF(LogMutableCore, Log, "Generated roms: %d (%d KB) are embedded, %d (%d KB) are streamed of which %d (%d KB) are high-res.", 
			NumEmbedded, uint32(NumEmbeddedBytes/1024), NumRoms, uint32(NumRomsBytes/1024), NumHighRes, uint32(NumHighResBytes/1024));
	}


}
