// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/Compiler.h"

#include "MuT/AST.h"
#include "MuT/ErrorLog.h"
#include "MuR/Operations.h"
#include "Templates/SharedPointer.h"


namespace UE::Mutable::Private
{

	/** Generic data about a Mutable operation that is needed at compile time. */
	struct FOpToolsDesc
	{
		/** True if the instruction is worth caching when generating models. */
		bool bCached;

		/** For image instructions, for every image format, true if it is supported as the base format of the operation. */
		bool bSupportedBasePixelFormats[uint8(EImageFormat::Count)];
	};

	extern const FOpToolsDesc& GetOpToolsDesc(EOpType type);

    //!
    class CompilerOptions::Private
    {
    public:

        //! Detailed optimization options
        FModelOptimizationOptions OptimisationOptions;

		uint64 EmbeddedDataBytesLimit = 1024;

		// \TODO: Unused?
		int32 MinTextureResidentMipCount = 3;

        int32 ImageCompressionQuality = 0;

		bool bIgnoreStates = false;
		bool bLog = false;

		FImageOperator::FImagePixelFormatFunc ImageFormatFunc;
    };


    //!
    struct FStateCompilationData
    {
        FObjectState nodeState;
        Ptr<ASTOp> root;
        FProgram::FState state;
		//FStateOptimizationOptions optimisationFlags;

        //! List of instructions that need to be cached to efficiently update this state
        TArray<Ptr<ASTOp>> m_updateCache;

        //! List of root instructions for the dynamic resources that depend on the runtime
        //! parameters of this state.
		TArray< TPair<Ptr<ASTOp>, TArray<FString> > > m_dynamicResources;
    };


    //!
    class Compiler::Private
    {
    public:

        Private()
        {
            ErrorLog = MakeShared<FErrorLog>();
        }

        TSharedPtr<FErrorLog> ErrorLog;

        /** */
        Ptr<CompilerOptions> Options;

		/** */
		TFunction<void()> WaitCallback;

		/** */
		void GenerateRoms(FModel*, const FLinkerOptions::FAdditionalData&);
    };

}
