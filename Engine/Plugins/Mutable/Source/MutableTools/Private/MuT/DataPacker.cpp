// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/DataPacker.h"

#include "ASTOpSkeletalMeshNew.h"
#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "MuR/Image.h"
#include "MuR/Layout.h"
#include "MuR/Mesh.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/Model.h"
#include "MuR/MutableTrace.h"
#include "MuR/Operations.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/ASTOpLODNew.h"
#include "MuT/ASTOpConstantResource.h"
#include "MuT/ASTOpConstantColor.h"
#include "MuT/ASTOpImageCompose.h"
#include "MuT/ASTOpImageMultiCompose.h"
#include "MuT/ASTOpImageLayer.h"
#include "MuT/ASTOpImageLayerColor.h"
#include "MuT/ASTOpImageMultiLayer.h"
#include "MuT/ASTOpImagePlainColor.h"
#include "MuT/ASTOpImageDisplace.h"
#include "MuT/ASTOpMeshExtractLayoutBlocks.h"
#include "MuT/ASTOpMeshRemoveMask.h"
#include "MuT/ASTOpMeshDifference.h"
#include "MuT/ASTOpMeshMorph.h"
#include "MuT/ASTOpMeshMerge.h"
#include "MuT/ASTOpMeshProject.h"
#include "MuT/ASTOpMeshApplyLayout.h"
#include "MuT/ASTOpMeshPrepareLayout.h"
#include "MuT/ASTOpLayoutFromMesh.h"
#include "MuT/ASTOpImageRasterMesh.h"
#include "MuT/CompilerPrivate.h"


namespace UE::Mutable::Private
{

	
	class AccumulateImageFormatsAST : public Visitor_TopDown_Unique_Const< TArray<bool> >
    {
    public:

        void Run( const ASTOpList& roots )
        {
            MUTABLE_CPUPROFILER_SCOPE(AccumulateImageFormatsAST);

			TArray<bool> defaultState;
			defaultState.SetNumZeroed(int32(EImageFormat::Count));

            Traverse( roots, defaultState );
        }


        bool Visit( const Ptr<ASTOp>& node ) override
        {
            bool recurse = true;

            const TArray<bool>& currentFormats = GetCurrentState();

			TArray<bool> defaultState;
			defaultState.SetNumZeroed(int32(EImageFormat::Count));
			bool allFalse = currentFormats == defaultState;

            // Can we use the cache?
            if (allFalse)
            {
                if (Visited.Contains(node))
                {
                    return false;
                }

                Visited.Add(node);
            }

            switch ( node->GetOpType() )
            {
            case EOpType::IM_CONSTANT:
            {
                // Remove unsupported formats
				const ASTOpConstantResource* op = static_cast<const ASTOpConstantResource*>(node.get());
                if (!SupportedFormats.Contains(op))
                {
					TArray<bool> initial;
					initial.Init( true, int32(EImageFormat::Count) );
                    SupportedFormats.Add( op, std::move(initial) );
                }

                for ( unsigned f=0; f< unsigned(EImageFormat::Count); ++f )
                {
                    if ( !currentFormats[f] )
                    {
                        SupportedFormats[op][f] = false;
                    }
                }
                recurse = false;
                break;
            }

            case EOpType::IM_SWITCH:
            case EOpType::IM_CONDITIONAL:
                // Switches and conditionals don't change the supported formats
                break;

            case EOpType::IM_COMPOSE:
            {
                recurse = false;
				const ASTOpImageCompose* op = static_cast<const ASTOpImageCompose*>(node.get());

				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
                RecurseWithState( op->Layout.child(), NewState );
                RecurseWithState( op->Base.child(), NewState );
                RecurseWithState( op->BlockImage.child(), NewState );

                if ( op->Mask )
                {
                    NewState[(int32)EImageFormat::L_UBitRLE] = true;
                    RecurseWithState( op->Mask.child(), NewState );
                }
                break;
            }

			case EOpType::IM_MULTICOMPOSE:
			{
				recurse = false;
				const ASTOpImageMultiCompose* op = static_cast<const ASTOpImageMultiCompose*>(node.get());

				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
				RecurseWithState(op->Layout.child(), NewState);
				RecurseWithState(op->Base.child(), NewState);
				RecurseWithState(op->SourceLayout.child(), NewState);
				RecurseWithState(op->SourceImage.child(), NewState);

				//if (op->Mask)
				//{
				//	NewState[(int32)EImageFormat::L_UBitRLE] = true;
				//	RecurseWithState(op->Mask.child(), NewState);
				//}
				break;
			}

            case EOpType::IM_LAYERCOLOR:
            {
                recurse = false;
				const ASTOpImageLayerColor* op = static_cast<const ASTOpImageLayerColor*>(node.get());
				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
				RecurseWithState( op->base.child(), NewState );
                RecurseWithState( op->color.child(), NewState );

                if ( op->mask )
                {
                    NewState[(int32)EImageFormat::L_UByte] = true;
                    NewState[(int32)EImageFormat::L_UByteRLE] = true;

                    RecurseWithState( op->mask.child(), NewState );
                }
                break;
            }

            case EOpType::IM_LAYER:
            {
                recurse = false;
				const ASTOpImageLayer* op = static_cast<const ASTOpImageLayer*>(node.get());
				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
				RecurseWithState( op->base.child(), NewState );
                RecurseWithState( op->blend.child(), NewState );

                if (op->mask)
                {
                    NewState[(int32)EImageFormat::L_UByte] = true;
                    NewState[(int32)EImageFormat::L_UByteRLE] = true;

                    RecurseWithState( op->mask.child(), NewState );
                }
                break;
            }

            case EOpType::IM_MULTILAYER:
            {
                recurse = false;
				const ASTOpImageMultiLayer* op = static_cast<const ASTOpImageMultiLayer*>(node.get());
				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
				RecurseWithState( op->base.child(), NewState );
                RecurseWithState( op->blend.child(), NewState );

                if (op->mask)
                {
                    NewState[(size_t)EImageFormat::L_UByte] = true;
                    NewState[(size_t)EImageFormat::L_UByteRLE] = true;

                    RecurseWithState( op->mask.child(), NewState );
                }
                break;
            }

            case EOpType::IM_DISPLACE:
            {
                recurse = false;
				const ASTOpImageDisplace* op = static_cast<const ASTOpImageDisplace*>(node.get());
				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
				RecurseWithState( op->Source.child(), NewState );

                NewState[(int32)EImageFormat::L_UByte ] = true;
                NewState[(int32)EImageFormat::L_UByteRLE ] = true;

                RecurseWithState( op->DisplacementMap.child(), NewState );
                break;
            }

            default:
            {
                //m_currentFormats.Add(vector<bool>(Count, false));
                //Recurse(at, program);
                //m_currentFormats.pop_back();

				TArray<bool> NewState;
				NewState.Init(false, int32(EImageFormat::Count));
				if (currentFormats != NewState)
                {
                    RecurseWithState(node, NewState);
                    recurse = false;
                }
                else
                {
                    recurse = true;
                }
                break;
            }

            }

            return recurse;
        }

    public:

        //! Result of this visitor:
        //! Formats known to be supported by every constant image.
        TMap< Ptr<const ASTOpConstantResource>, TArray<bool> > SupportedFormats;

    private:

        //! Cache. Only valid is current formats are all false.
        TSet<Ptr<ASTOp>> Visited;

    };




    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    class AccumulateMeshChannelUsageAST : public Visitor_TopDown_Unique_Const< uint64_t >
    {
    public:

        void Run( const ASTOpList& roots )
        {
            MUTABLE_CPUPROFILER_SCOPE(AccumulateMeshChannelUsageAST);

            // Sanity check in case we add more semantics
            static_assert(uint32(EMeshBufferSemantic::Count)<sizeof(uint64)*8, "Too many mesh buffer semantics." );

            // Default state: we need everything except internal semantics
            uint64 defaultState = 0xffffffffffffffff;
            defaultState ^= (UINT64_C(1)<<uint32(EMeshBufferSemantic::LayoutBlock));
            defaultState ^= (UINT64_C(1)<<uint32(EMeshBufferSemantic::VertexIndex));

            Traverse(roots,defaultState);
        }


        bool Visit( const Ptr<ASTOp>& node ) override
        {
            bool bRecurse = true;

            uint64 CurrentSemantics = GetCurrentState();

            switch ( node->GetOpType() )
            {

			case EOpType::ME_CONSTANT:
			{
				// Accumulate necessary semantics
				ASTOpConstantResource* Op = static_cast<ASTOpConstantResource*>(node.get());
				uint64 InitialFlags = 0;
				uint64& CurrentFlags = RequiredSemanticsPerConstant.FindOrAdd(Op, InitialFlags);
				CurrentFlags |= CurrentSemantics;
				bRecurse = false;
				break;
			}

			case EOpType::ME_PREPARELAYOUT:
			{
				// Accumulate necessary semantics
				ASTOpMeshPrepareLayout* Op = static_cast<ASTOpMeshPrepareLayout*>(node.get());
				uint64 InitialFlags = 0;
				uint64& CurrentFlags = RequiredSemanticsPerPrepareLayout.FindOrAdd(Op, InitialFlags);
				CurrentFlags |= CurrentSemantics;
				break;
			}

            // TODO: These could probably optimise something
            //case EOpType::IM_RASTERMESH: break;

            case EOpType::ME_DIFFERENCE:
            {
				bRecurse = false;

				const ASTOpMeshDifference* op = static_cast<const ASTOpMeshDifference*>(node.get());

                uint64 NewState = CurrentSemantics;
                NewState |= (UINT64_C(1)<<uint32(EMeshBufferSemantic::VertexIndex));
                RecurseWithState( op->Base.child(), NewState );

                RecurseWithState( op->Target.child(), CurrentSemantics );
                break;
             }

            case EOpType::ME_REMOVEMASK:
            {
				bRecurse = false;

				const ASTOpMeshRemoveMask* op = static_cast<const ASTOpMeshRemoveMask*>(node.get());

                uint64 NewState = CurrentSemantics;
                NewState |= (UINT64_C(1)<< uint32(EMeshBufferSemantic::VertexIndex));
                RecurseWithState( op->source.child(), NewState );
                for( const TPair<ASTChild, ASTChild>& r: op->removes )
                {
                    RecurseWithState( r.Value.child(), NewState );
                }
                break;
             }

            case EOpType::ME_MORPH:
            {
				bRecurse = false;

				const ASTOpMeshMorph* op = static_cast<const ASTOpMeshMorph*>(node.get());

                uint64 NewState = CurrentSemantics;
                NewState |= (UINT64_C(1)<< uint32(EMeshBufferSemantic::VertexIndex));
                RecurseWithState( op->Base.child(), NewState );
                break;
             }

            case EOpType::ME_APPLYLAYOUT:
            {
				bRecurse = false;

				const ASTOpMeshApplyLayout* Op = static_cast<const ASTOpMeshApplyLayout*>(node.get());

                uint64 NewState = CurrentSemantics;
				NewState |= (UINT64_C(1)<< uint32(EMeshBufferSemantic::LayoutBlock));
                RecurseWithState( Op->Mesh.child(), NewState);

                RecurseWithState( Op->Layout.child(), CurrentSemantics );
                break;
            }

			case EOpType::ME_PROJECT:
			{
				bRecurse = false;

				const ASTOpMeshProject* op = static_cast<const ASTOpMeshProject*>(node.get());

				uint64 NewState = CurrentSemantics;
				NewState |= (UINT64_C(1) << uint32(EMeshBufferSemantic::LayoutBlock));
				RecurseWithState(op->Mesh.child(), NewState);

				RecurseWithState(op->Projector.child(), CurrentSemantics);
				break;
			}

			case EOpType::IM_RASTERMESH:
			{
				bRecurse = false;

				const ASTOpImageRasterMesh* op = static_cast<const ASTOpImageRasterMesh*>(node.get());

				uint64 NewState = CurrentSemantics;
				NewState |= (UINT64_C(1) << uint32(EMeshBufferSemantic::LayoutBlock));
				RecurseWithState(op->mesh.child(), NewState);

				RecurseWithState(op->image.child(), CurrentSemantics);
				RecurseWithState(op->angleFadeProperties.child(), CurrentSemantics);
				RecurseWithState(op->mask.child(), CurrentSemantics);
				RecurseWithState(op->projector.child(), CurrentSemantics);
				break;
			}

            case EOpType::ME_EXTRACTLAYOUTBLOCK:
            {
				bRecurse = false;

				const ASTOpMeshExtractLayoutBlocks* op = static_cast<const ASTOpMeshExtractLayoutBlocks*>(node.get());

                // todo: check if we really need all of them
                uint64 NewState = CurrentSemantics;
                NewState |= (UINT64_C(1)<< uint32(EMeshBufferSemantic::LayoutBlock));
                NewState |= (UINT64_C(1)<< uint32(EMeshBufferSemantic::VertexIndex));

                RecurseWithState( op->Source.child(), NewState );
                break;
            }

			case EOpType::ME_MERGE:
			{
				bRecurse = false;

				const ASTOpMeshMerge* op = static_cast<const ASTOpMeshMerge*>(node.get());

				// ME_MERGE will need VertexIds if the mesh has converted them from implicit to explicit. 
				uint64 NewState = CurrentSemantics;
				NewState |= (UINT64_C(1) << uint32(EMeshBufferSemantic::VertexIndex));

				RecurseWithState(op->Base.child(), NewState);
				RecurseWithState(op->Added.child(), NewState);
				break;
			}

			case EOpType::LA_FROMMESH:
			{
				bRecurse = false;

				const ASTOpLayoutFromMesh* op = static_cast<const ASTOpLayoutFromMesh*>(node.get());

				uint64 NewState = CurrentSemantics;
				NewState |= (UINT64_C(1) << uint32(EMeshBufferSemantic::LayoutBlock));

				RecurseWithState(op->Mesh.child(), NewState);
				break;
			}

            case EOpType::SK_NEW:
            {
				bRecurse = false;

				const ASTOpSkeletalMeshNew* Op = static_cast<const ASTOpSkeletalMeshNew*>(node.get());

				for (const ASTChild& LOD : Op->LODs)
				{
					RecurseWithState(LOD.child(), CurrentSemantics);
				}
					
                break;
            }
            	
            case EOpType::LD_NEW:
			{
				bRecurse = false;

				const ASTOpLODNew* Op = static_cast<const ASTOpLODNew*>(node.get());

				for (const ASTChild& Mesh : Op->Meshes)
				{
					uint64 NewState = GetDefaultState();
					RecurseWithState(Mesh.child(), NewState);
				}
				
				break;
			}

            default:
                // Unhandled op, we may need everything? Recurse with current state?
                //uint64 NewState = 0xffffffffffffffff;
                break;

            }

            return bRecurse;
        }

    public:

        // Result of this visitor:
        // Used mesh channel semantics for some relevant operation types mesh
		TMap< Ptr<ASTOpConstantResource>, uint64 > RequiredSemanticsPerConstant;
		TMap< Ptr<ASTOpMeshPrepareLayout>, uint64 > RequiredSemanticsPerPrepareLayout;

    };


    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------
    //---------------------------------------------------------------------------------------------

    // Todo: move to its own file
    inline void MeshRemoveUnusedBufferSemantics( FMesh* Mesh, uint64 UsedSemantics )
    {
        // right now we only remove entire buffers if no channel is used
        // TODO: remove from inside the buffer?
        for (int32 BufferIndex=0; BufferIndex < Mesh->GetVertexBuffers().GetBufferCount(); )
        {
            bool bUsed = false;
            for (int32 ChannelIndex=0; !bUsed && ChannelIndex < Mesh->GetVertexBuffers().GetBufferChannelCount(BufferIndex); ++ChannelIndex)
            {
                EMeshBufferSemantic Semantic;
				Mesh->GetVertexBuffers().GetChannel(BufferIndex, ChannelIndex, &Semantic, nullptr, nullptr, nullptr, nullptr);
				bUsed = (( (UINT64_C(1)<< uint32(Semantic)) ) & UsedSemantics) != 0;
            }

            if (!bUsed)
            {
				Mesh->VertexBuffers.Buffers.RemoveAt(BufferIndex);
            }
            else
            {
                ++BufferIndex;
            }
        }

        // If we don't need layouts, remove them.
        {
            constexpr uint64 LayoutSemantics = (UINT64_C(1)<< uint32(EMeshBufferSemantic::LayoutBlock));

            if ( (UsedSemantics & LayoutSemantics) == 0)
            {
                Mesh->Layouts.Empty();
            }
        }
    }


    //---------------------------------------------------------------------------------------------
    void DataOptimise( const CompilerOptions* Options, ASTOpList& roots )
    {
		int32 ImageCompressionQuality = Options->GetPrivate()->ImageCompressionQuality;
		const FModelOptimizationOptions& OptimizeOptions = Options->GetPrivate()->OptimisationOptions;

        // Images
        AccumulateImageFormatsAST ImageFormatAccumulator;
		ImageFormatAccumulator.Run( roots );

        // See if we can convert some constants to more efficient formats
        ASTOp::Traverse_BottomUp_Unique_NonReentrant( roots, [&](Ptr<ASTOp>& n)
        {
            if (n->GetOpType()==EOpType::IM_CONSTANT)
            {
				ASTOpConstantResource* Typed = static_cast<ASTOpConstantResource*>(n.get());
				TManagedPtr<const FImage> pOld = StaticCastManagedPtr<const FImage>(Typed->GetValue());

				FImageOperator ImOp = FImageOperator::GetDefault( Options->GetPrivate()->ImageFormatFunc );

				// See if there is a better format for this image
				FVector4f PlainColor;
				if (pOld->IsPlainColor(PlainColor))
				{
					// It is more efficient to just have an instruction for it instead, to avoid the overhead
					// of data loading. 
					// Warning This eliminates the mips. \TODO: Add support for mips in plaincolor instruction?
					Ptr<ASTOpConstantColor> NewColor = new ASTOpConstantColor;
					NewColor->Value = PlainColor;

					Ptr<ASTOpImagePlainColor> NewPlain = new ASTOpImagePlainColor;
					NewPlain->Color = NewColor;
					NewPlain->Format = pOld->GetFormat();
					NewPlain->Size[0] = pOld->GetSizeX();
					NewPlain->Size[1] = pOld->GetSizeY();
					NewPlain->LODs = 1;

					ASTOp::Replace(n, NewPlain);

				}
				else if (ImageFormatAccumulator.SupportedFormats[Typed][(int32)EImageFormat::L_UBitRLE])
				{
					TManagedPtr<FImage> pNew = ImOp.ImagePixelFormat( ImageCompressionQuality, pOld.Get(), EImageFormat::L_UBitRLE );

					// Only replace if the compression was worth!
					int32 oldSize = pOld->GetDataSize();
					int32 newSize = pNew->GetDataSize();
					if (float(oldSize) > float(newSize) * OptimizeOptions.MinRLECompressionGain)
					{
						Typed->SetValue(pNew);
					}
				}
				else if (ImageFormatAccumulator.SupportedFormats[Typed][(int32)EImageFormat::L_UByteRLE])
				{
					TManagedPtr<FImage> pNew = ImOp.ImagePixelFormat( ImageCompressionQuality, pOld.Get(), EImageFormat::L_UByteRLE );

					// Only replace if the compression was worth!
					int32 oldSize = pOld->GetDataSize();
					int32 newSize = pNew->GetDataSize();
                    if (float(oldSize) > float(newSize) * OptimizeOptions.MinRLECompressionGain)
                    {
                        Typed->SetValue(pNew);
                    }
                }
            }
        });


        // Meshes
        AccumulateMeshChannelUsageAST MeshSemanticsAccumulator;
		MeshSemanticsAccumulator.Run( roots );

		// See if we can remove some buffers from the constants
		for (const TPair<Ptr<ASTOpConstantResource>, uint64>& Entry : MeshSemanticsAccumulator.RequiredSemanticsPerConstant)
		{
			ASTOpConstantResource* Op = Entry.Key.get();
			TManagedPtr<FMesh> Mesh = StaticCastManagedPtr<const FMesh>(Op->GetValue())->Clone();
			MeshRemoveUnusedBufferSemantics(Mesh.Get(), Entry.Value);
			Op->SetValue(Mesh);
		}

		// See if we can remove entire "prepare layout" operations
		for (const TPair<Ptr<ASTOpMeshPrepareLayout>, uint64>& Entry : MeshSemanticsAccumulator.RequiredSemanticsPerPrepareLayout)
		{
			constexpr uint64 LayoutBlockMask = UINT64_C(1) << uint64(EMeshBufferSemantic::LayoutBlock);
			bool bRequiresLayouts = Entry.Value & LayoutBlockMask;
			if (!bRequiresLayouts)
			{
				// Directly remove the operation: the layout is never applied
				Ptr<ASTOpMeshPrepareLayout> Op = Entry.Key;
				ASTOp::Replace( Op, Op->Mesh.child() );
			}
		}
	}


}
