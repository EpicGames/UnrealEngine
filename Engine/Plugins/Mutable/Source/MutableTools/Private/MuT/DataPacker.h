// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/Operations.h"
#include "MuT/AST.h"

namespace UE::Mutable::Private { struct FProgram; }

namespace UE::Mutable::Private
{
	class CompilerOptions;

    /** Convert constant data to different formats, based on their usage. */
    extern void DataOptimise( const CompilerOptions*, ASTOpList& roots );

}
