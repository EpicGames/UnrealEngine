// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/ErrorLog.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Layout.h"
#include "MuR/Mesh.h"
#include "MuR/Model.h"
#include "MuR/Operations.h"
#include "MuR/System.h"
#include "MuR/MutableRuntimeModule.h"


namespace UE::Mutable::Private
{

    int32 FErrorLog::GetMessageCount() const
	{
		return Messages.Num();
	}


	const FString& FErrorLog::GetMessageText( int32 Index ) const
	{
		if (Messages.IsValidIndex(Index))
		{
			return Messages[Index].Text;
		}

		static FString Empty;
		return Empty;
	}


	const TArray<const void*, TInlineAllocator<4>>* FErrorLog::GetMessageContexts(int32 Index) const
	{
		if (Messages.IsValidIndex(Index))
		{
    		return &Messages[Index].Contexts;
		}

    	return nullptr;
		}


	ErrorLogMessageType FErrorLog::GetMessageType( int32 Index ) const
	{
		ErrorLogMessageType Result = ELMT_NONE;

		if (Messages.IsValidIndex(Index))
		{
			Result = Messages[Index].Type;
		}

		return Result;
	}


	ErrorLogMessageSpamBin FErrorLog::GetMessageSpamBin(int32 Index) const
	{
		ErrorLogMessageSpamBin Result = ELMSB_ALL;

		if (Messages.IsValidIndex(Index))
		{
			Result = Messages[Index].Spam;
		}

		return Result;
	}


	ErrorLogMessageAttachedDataView FErrorLog::GetMessageAttachedData( int32 Index ) const
	{
        ErrorLogMessageAttachedDataView Result;

		if (Messages.IsValidIndex(Index))
		{
			const FMessage& message = Messages[Index];
            
            if ( message.Data ) 
            {
                Result.UnassignedUVs = message.Data->UnassignedUVs.GetData();
			    Result.UnassignedUVsSize = message.Data->UnassignedUVs.Num();
            }
		}

		return Result;
	}


	void FErrorLog::Add(const FString& InMessage,
		ErrorLogMessageType InType,
		const void* InContext,
		ErrorLogMessageSpamBin InSpamBin)
	{
		UE::TUniqueLock Lock(MessageMutex);

		FMessage& Msg = Messages.Emplace_GetRef();
		Msg.Type = InType;
		Msg.Spam = InSpamBin;
		Msg.Text = InMessage;
		Msg.Contexts.Add(InContext);
	}


	void FErrorLog::Add(const FString& InMessage,
		ErrorLogMessageType InType,
		const void* InContext,
		const void* InContext2,
		ErrorLogMessageSpamBin InSpamBin)
	{
		UE::TUniqueLock Lock(MessageMutex);

		FMessage& Msg = Messages.Emplace_GetRef();
		Msg.Type = InType;
		Msg.Spam = InSpamBin;
		Msg.Text = InMessage;
    	Msg.Contexts.Add(InContext);
    	Msg.Contexts.Add(InContext2);
	}
	
	
	void FErrorLog::Add(const FString& InMessage,
		ErrorLogMessageType InType,
		TArray<const void*, TInlineAllocator<4>>& Contexts,
		ErrorLogMessageSpamBin InSpamBin)
    {
    	UE::TUniqueLock Lock(MessageMutex);

    	FMessage& Msg = Messages.Emplace_GetRef();
    	Msg.Type = InType;
    	Msg.Spam = InSpamBin;
    	Msg.Text = InMessage;
    	Msg.Contexts = MoveTemp(Contexts);
	}


	void FErrorLog::Add(const FString& InMessage,
                                const ErrorLogMessageAttachedDataView& InDataView,
                                ErrorLogMessageType InType, 
								const void* InContext,
								ErrorLogMessageSpamBin InSpamBin)
	{
		UE::TUniqueLock Lock(MessageMutex);

		FMessage& Msg = Messages.Emplace_GetRef();
		Msg.Type = InType;
		Msg.Spam = InSpamBin;
		Msg.Text = InMessage;
		Msg.Contexts = {InContext};
		Msg.Data = MakeShared<FErrorData>();

        if ( InDataView.UnassignedUVs && InDataView.UnassignedUVsSize > 0 )
        {
			// \TODO: Review
			Msg.Data->UnassignedUVs.Append(InDataView.UnassignedUVs, InDataView.UnassignedUVsSize);
        }
	}
	
	
	void FErrorLog::Log() const
	{
		UE_LOGF(LogMutableCore, Log, " Error Log :\n");

		for ( const FMessage& msg : Messages )
		{
			switch ( msg.Type )
			{
			case ELMT_ERROR: 	UE_LOGF(LogMutableCore, Log, "  ERR  %ls\n", *msg.Text); break;
			case ELMT_WARNING: 	UE_LOGF(LogMutableCore, Log, "  WRN  %ls\n", *msg.Text); break;
			case ELMT_INFO: 	UE_LOGF(LogMutableCore, Log, "  INF  %ls\n", *msg.Text); break;
			default: 			UE_LOGF(LogMutableCore, Log, "  NON  %ls\n", *msg.Text); break;
			}
		}
	}


	void FErrorLog::Merge( const FErrorLog* Other )
	{
		Messages.Append(Other->Messages);
	}

    
    // clang-format off

    const TCHAR* s_opNames[] =
	{
		TEXT("NONE"),

		TEXT("BO_CONSTANT"),
		TEXT("BO_PARAMETER"),
		TEXT("BO_EQUAL_INT_CONST"),
		TEXT("BO_AND"),
		TEXT("BO_OR"),
		TEXT("BO_NOT"),

		TEXT("NU_CONSTANT"),
		TEXT("NU_PARAMETER"),
		TEXT("NU_CONDITIONAL"),
		TEXT("NU_SWITCH"),

		TEXT("SC_CONSTANT"),
		TEXT("SC_PARAMETER"),
		TEXT("SC_CONDITIONAL"),
		TEXT("SC_SWITCH"),
		TEXT("SC_MATERIAL_BREAK"),
		TEXT("SC_ARITHMETIC"),
		TEXT("SC_CURVE"),
		TEXT("SC_EXTERNAL"),

		TEXT("CO_CONSTANT"),
		TEXT("CO_PARAMETER"),
		TEXT("CO_CONDITIONAL"),
		TEXT("CO_SWITCH"),
		TEXT("CO_MATERIAL_BREAK"),
		TEXT("CO_SAMPLEIMAGE"),
		TEXT("CO_SWIZZLE"),
		TEXT("CO_FROMSCALARS"),
		TEXT("CO_ARITHMETIC"),
		TEXT("CO_LINEARTOSRGB"),
		TEXT("CO_EXTERNAL"),

		TEXT("ST_CONSTANT"),
		TEXT("ST_PARAMETER"),

		TEXT("PR_CONSTANT"),
		TEXT("PR_PARAMETER"),

		TEXT("IM_CONSTANT"),
		TEXT("IM_PARAMETER"),
		TEXT("IM_REFERENCE"),
		TEXT("IM_CONDITIONAL"),
		TEXT("IM_SWITCH"),
		TEXT("IM_MATERIAL_BREAK"),
		TEXT("IM_PARAMETER_FROM_MATERIAL"),
		TEXT("IM_LAYER"),
		TEXT("IM_LAYERCOLOR"),
		TEXT("IM_PIXELFORMAT"),
		TEXT("IM_CONFORM"),
		TEXT("IM_MIPMAP"),
		TEXT("IM_RESIZE"),
		TEXT("IM_RESIZELIKE"),
		TEXT("IM_RESIZEREL"),
		TEXT("IM_BLANKLAYOUT"),
		TEXT("IM_COMPOSE"),
		TEXT("IM_MULTICOMPOSE"),
		TEXT("IM_INTERPOLATE"),
		TEXT("IM_SATURATE"),
		TEXT("IM_LUMINANCE"),
		TEXT("IM_SWIZZLE"),
		TEXT("IM_COLORMAP"),
		TEXT("IM_BINARISE"),
		TEXT("IM_PLAINCOLOR"),
		TEXT("IM_CROP"),
		TEXT("IM_PATCH"),
		TEXT("IM_RASTERMESH"),
		TEXT("IM_MAKEGROWMAP"),
		TEXT("IM_DISPLACE"),
		TEXT("IM_MULTILAYER"),
		TEXT("IM_INVERT"),
		TEXT("IM_NORMALCOMPOSITE"),
		TEXT("IM_TRANSFORM"),
		TEXT("IM_PARAMETER_CONVERT"),
		TEXT("IM_EXTERNAL"),

		TEXT("ME_CONSTANT"),
		TEXT("ME_PARAMETER"),
		TEXT("ME_REFERENCE"),
		TEXT("ME_CONDITIONAL"),
		TEXT("ME_SWITCH"),
		TEXT("ME_APPLYLAYOUT"),
		TEXT("ME_PREPARELAYOUT"),
		TEXT("ME_DIFFERENCE"),
		TEXT("ME_MORPH"),
		TEXT("ME_MERGE"),
		TEXT("ME_MASKCLIPMESH"),
		TEXT("ME_MASKCLIPUVMASK"),
		TEXT("ME_MASKDIFF"),
		TEXT("ME_REMOVEMASK"),
		TEXT("ME_FORMAT"),
		TEXT("ME_EXTRACTLAYOUTBLOCK"),
		TEXT("ME_TRANSFORM"),
		TEXT("ME_CLIPMORPHPLANE"),
		TEXT("ME_CLIPWITHMESH"),
		TEXT("ME_PROJECT"),
		TEXT("ME_APPLYPOSE"),
		TEXT("ME_BINDSHAPE"),
		TEXT("ME_APPLYSHAPE"),
		TEXT("ME_CLIPDEFORM"),
		TEXT("ME_ADDMETADATA"),
		TEXT("ME_SETMATERIALSLOTID"),
		TEXT("ME_TRANSFORMWITHMESH"),
		TEXT("ME_TRANSFORMWITHBONE"),
		TEXT("ME_EXTERNAL"),
		TEXT("ME_SKELETALMESH_BREAK"),

    	TEXT("LD_CONDITIONAL"),
    	TEXT("LD_SWITCH"),
    	TEXT("LD_NEW"),

    	TEXT("SK_CONDITIONAL"),
    	TEXT("SK_SWITCH"),
    	TEXT("SK_NEW"),
		TEXT("SK_PARAMETER"),
		TEXT("SK_MERGE"),
    	TEXT("SK_CONVERT"),
    	TEXT("SK_MORPH"),
		TEXT("SK_RESHAPE"),
		TEXT("SK_MATERIALMODIFY"),
		TEXT("SK_CLIPMESHWITHMESH"),
		TEXT("SK_TRANSFORM"),
		TEXT("SK_TRANSFORMWITHBONE"),
    	
		TEXT("SKO_CONVERT"),

		TEXT("IN_CONDITIONAL"),
		TEXT("IN_SWITCH"),
		TEXT("IN_ADDCOMPONENT"),
		TEXT("IN_ADDSKELETALMESH"),
		TEXT("IN_ADDEXTENSIONDATA"),
		TEXT("IN_ADDOVERLAYMATERIAL"),
		TEXT("IN_ADDOVERRIDEMATERIAL"),

		TEXT("LA_CONSTANT"),
		TEXT("LA_CONDITIONAL"),
		TEXT("LA_SWITCH"),
		TEXT("LA_PACK"),
		TEXT("LA_MERGE"),
		TEXT("LA_REMOVEBLOCKS"),
		TEXT("LA_FROMMESH"),

		TEXT("MI_CONSTANT"),
		TEXT("MI_PARAMETER"),
		TEXT("MI_CONDITIONAL"),
		TEXT("MI_SWITCH"),
		TEXT("MI_SKELETALMESHOBJECT_BREAK"),
		TEXT("MI_SKELETALMESH_BREAK"),
    	TEXT("MI_FROM_SKELETALMESH_SLOT"),
		TEXT("MI_MODIFY"),
		TEXT("MI_EXTERNAL"),

		TEXT("MA_CONSTANT"),
		TEXT("MA_PARAMETER"),
	
		TEXT("ED_CONSTANT"),
		TEXT("ED_CONDITIONAL"),
		TEXT("ED_SWITCH"),

		TEXT("IS_PARAMETER"),
		TEXT("IS_EXTERNAL"),
		TEXT("IS_CONDITIONAL"),
		TEXT("IS_SWITCH"),
	};

	static_assert(UE_ARRAY_COUNT(s_opNames) == int32(EOpType::COUNT));

    // clang-format on

}
