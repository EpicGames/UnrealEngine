// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/LODInfo.h"

#include "Templates/TypeHash.h"
#include "MuR/Types.h"


namespace UE::Mutable::Private
{
	uint32 GetTypeHash(const FLODInfo& Key)
	{
		uint32 Result = 0;
			
		Result = HashCombineFast(Result, GetTypeHash(Key.ScreenSize));
		Result = HashCombineFast(Result, GetTypeHash(Key.LODHysteresis));
		Result = HashCombineFast(Result, GetTypeHash(Key.bSupportUniformlyDistributedSampling));
		Result = HashCombineFast(Result, GetTypeHash(Key.bAllowCPUAccess));
		 		
		return Result;
	}
}
