// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/MutableToolsModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FMutableToolsModule, MutableTools);
