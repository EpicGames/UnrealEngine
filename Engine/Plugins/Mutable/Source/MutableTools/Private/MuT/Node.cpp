// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuT/Node.h"

#include "MuT/NodeObject.h"
#include "MuT/NodeObjectNew.h"
#include "MuT/NodeObjectGroup.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeComponentNew.h"
#include "MuT/NodeModifierSkeletalMeshMerge.h"
#include "MuT/NodeComponentSwitch.h"
#include "MuT/NodeComponentVariation.h"
#include "MuT/NodeLOD.h"
#include "MuT/NodeSurface.h"
#include "MuT/NodeSurfaceNew.h"
#include "MuT/NodeSurfaceSwitch.h"
#include "MuT/NodeSurfaceVariation.h"
#include "MuT/NodeExtensionData.h"
#include "MuT/NodeExtensionDataConstant.h"
#include "MuT/NodeImageConstant.h"
#include "MuT/NodeImageTable.h"
#include "MuT/NodeImageObjectParameter.h"
#include "MuT/NodeImageFormat.h"
#include "MuT/NodeImageConform.h"
#include "MuT/NodeImageBinarise.h"
#include "MuT/NodeImageConditional.h"
#include "MuT/NodeImageFromMaterialParameter.h"
#include "MuT/NodeImageInterpolate.h"
#include "MuT/NodeImageInvert.h"
#include "MuT/NodeImageLayer.h"
#include "MuT/NodeImageLayerColor.h"
#include "MuT/NodeImageLuminance.h"
#include "MuT/NodeImageMipmap.h"
#include "MuT/NodeImageMultiLayer.h"
#include "MuT/NodeImageNormalComposite.h"
#include "MuT/NodeImagePlainColor.h"
#include "MuT/NodeImageProject.h"
#include "MuT/NodeImageResize.h"
#include "MuT/NodeImageSaturate.h"
#include "MuT/NodeImageSwitch.h"
#include "MuT/NodeImageSwizzle.h"
#include "MuT/NodeImageTransform.h"
#include "MuT/NodeImageVariation.h"
#include "MuT/NodeImageColorMap.h"
#include "MuT/NodeBool.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeColorSwitch.h"
#include "MuT/NodeColorVariation.h"
#include "MuT/NodeColorConstant.h"
#include "MuT/NodeColorParameter.h"
#include "MuT/NodeColorSampleImage.h"
#include "MuT/NodeColorArithmeticOperation.h"
#include "MuT/NodeColorFromScalars.h"
#include "MuT/NodeColorTable.h"
#include "MuT/NodeColorToSRGB.h"
#include "MuT/NodeColorMaterialBreak.h"
#include "MuT/NodeScalarSwitch.h"
#include "MuT/NodeScalarConstant.h"
#include "MuT/NodeScalarVariation.h"
#include "MuT/NodeScalarArithmeticOperation.h"
#include "MuT/NodeScalarParameter.h"
#include "MuT/NodeScalarEnumParameter.h"
#include "MuT/NodeScalarTable.h"
#include "MuT/NodeScalarCurve.h"
#include "MuT/NodeScalarMaterialBreak.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeMeshFragment.h"
#include "MuT/NodeMeshClipMorphPlane.h"
#include "MuT/NodeMeshClipDeform.h"
#include "MuT/NodeMeshClipWithMesh.h"
#include "MuT/NodeMeshSkeletalMeshObjectBreak.h"
#include "MuT/NodeMeshMakeMorph.h"
#include "MuT/NodeMeshApplyPose.h"
#include "MuT/NodeMeshTransform.h"
#include "MuT/NodeMeshSwitch.h"
#include "MuT/NodeMeshReshape.h"
#include "MuT/NodeMeshMorph.h"
#include "MuT/NodeMeshFormat.h"
#include "MuT/NodeMeshVariation.h"
#include "MuT/NodeMeshTable.h"
#include "MuT/NodeSurfaceModifier.h"
#include "MuT/NodeSurfaceModifierMeshClipDeform.h"
#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"
#include "MuT/NodeSurfaceModifierMeshClipWithMesh.h"
#include "MuT/NodeSurfaceModifierMeshClipWithUVMask.h"
#include "MuT/NodeSurfaceModifierMeshTransformInMesh.h"
#include "MuT/NodeSurfaceModifierMeshTransformWithBone.h"
#include "MuT/NodeMatrix.h"
#include "MuT/NodeMatrixConstant.h"
#include "MuT/NodeMatrixParameter.h"
#include "MuT/NodeString.h"
#include "MuT/NodeStringConstant.h"
#include "MuT/NodeStringParameter.h"
#include "MuT/NodeColorExternal.h"
#include "MuT/NodeExternal.h"
#include "MuT/NodeExternalOperation.h"
#include "MuT/NodeExternalParameter.h"
#include "MuT/NodeExternalSwitch.h"
#include "MuT/NodeImageExternal.h"
#include "MuT/NodeProjector.h"
#include "MuT/NodeRange.h"
#include "MuT/NodeRangeFromScalar.h"
#include "MuT/NodeMaterial.h"
#include "MuT/NodeImageMaterialBreak.h"
#include "MuT/NodeImageConvert.h"
#include "MuT/NodeMaterialConstant.h"
#include "MuT/NodeMaterialExternal.h"
#include "MuT/NodeMaterialTable.h"
#include "MuT/NodeMaterialSwitch.h"
#include "MuT/NodeMaterialVariation.h"
#include "MuT/NodeMaterialParameter.h"
#include "MuT/NodeMaterialSkeletalMeshObjectBreak.h"
#include "MuT/NodeMeshExternal.h"
#include "MuT/NodeScalarExternal.h"
#include "MuT/NodeSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshNew.h"
#include "MuT/NodeSkeletalMeshObjectParameter.h"
#include "MuT/NodeMaterialModify.h"
#include "MuT/NodeMaterialSkeletalMeshBreak.h"
#include "MuT/NodeMeshRemoveMesh.h"
#include "MuT/NodeMeshTransformInMesh.h"
#include "MuT/NodeMeshTransformWithBone.h"
#include "MuT/NodeSkeletalMeshClipWithSkeletalMesh.h"
#include "MuT/NodeSkeletalMeshSwitch.h"
#include "MuT/NodeSkeletalMeshVariation.h"
#include "MuT/NodeSkeletalMeshConvert.h"
#include "MuT/NodeSkeletalMeshMerge.h"
#include "MuT/NodeSkeletalMeshModify.h"
#include "MuT/NodeSkeletalMeshObject.h"
#include "MuT/NodeSkeletalMeshObjectConvert.h"
#include "MuT/NodeSkeletalMeshObjectSwitch.h"
#include "MuT/NodeSkeletalMeshMorph.h"
#include "MuT/NodeSkeletalMeshReshape.h"
#include "MuT/NodeSkeletalMeshTransform.h"
#include "MuT/NodeSkeletalMeshTransformWithBone.h"
#include "MuT/NodeSurfaceModifierSurfaceEdit.h"


namespace UE::Mutable::Private
{

	// Static initialization
	FNodeType Node::StaticType = FNodeType(Node::EType::Node, nullptr);

	FNodeType NodeObject::StaticType = FNodeType(Node::EType::Object, Node::GetStaticType());
	FNodeType NodeObjectNew::StaticType = FNodeType(Node::EType::ObjectNew, NodeObject::GetStaticType());
	FNodeType NodeObjectGroup::StaticType = FNodeType(Node::EType::ObjectGroup, NodeObject::GetStaticType());

	FNodeType NodeComponent::StaticType = FNodeType(Node::EType::Component, Node::GetStaticType());
	FNodeType NodeComponentNew::StaticType = FNodeType(Node::EType::ComponentNew, NodeComponent::GetStaticType());
	FNodeType NodeComponentSwitch::StaticType = FNodeType(Node::EType::ComponentSwitch, NodeComponent::GetStaticType());
	FNodeType NodeComponentVariation::StaticType = FNodeType(Node::EType::ComponentVariation, NodeComponent::GetStaticType());

	FNodeType NodeBool::StaticType = FNodeType(Node::EType::Bool, Node::GetStaticType());
	FNodeType NodeBoolConstant::StaticType = FNodeType(Node::EType::BoolConstant, NodeBool::GetStaticType());
	FNodeType NodeBoolParameter::StaticType = FNodeType(Node::EType::BoolParameter, NodeBool::GetStaticType());
	FNodeType NodeBoolNot::StaticType = FNodeType(Node::EType::BoolNot, NodeBool::GetStaticType());
	FNodeType NodeBoolAnd::StaticType = FNodeType(Node::EType::BoolAnd, NodeBool::GetStaticType());

	FNodeType NodeScalar::StaticType = FNodeType(Node::EType::Scalar, Node::GetStaticType());
	FNodeType NodeScalarSwitch::StaticType = FNodeType(Node::EType::ScalarSwitch, NodeScalar::GetStaticType());
	FNodeType NodeScalarConstant::StaticType = FNodeType(Node::EType::ScalarConstant, NodeScalar::GetStaticType());
	FNodeType NodeScalarParameter::StaticType = FNodeType(Node::EType::ScalarParameter, NodeScalar::GetStaticType());
	FNodeType NodeScalarVariation::StaticType = FNodeType(Node::EType::ScalarVariation, NodeScalar::GetStaticType());
	FNodeType NodeScalarArithmeticOperation::StaticType = FNodeType(Node::EType::ScalarArithmeticOperation, NodeScalar::GetStaticType());
	FNodeType NodeScalarEnumParameter::StaticType = FNodeType(Node::EType::ScalarEnumParameter, NodeScalar::GetStaticType());
	FNodeType NodeScalarTable::StaticType = FNodeType(Node::EType::ScalarTable, NodeScalar::GetStaticType());
	FNodeType NodeScalarCurve::StaticType = FNodeType(Node::EType::ScalarCurve, NodeScalar::GetStaticType());
	FNodeType NodeScalarExternal::StaticType = FNodeType(Node::EType::ScalarExternal, NodeScalar::GetStaticType());

	FNodeType NodeSurface::StaticType = FNodeType(Node::EType::Surface, Node::GetStaticType());
	FNodeType NodeSurfaceNew::StaticType = FNodeType(Node::EType::SurfaceNew, NodeSurface::GetStaticType());
	FNodeType NodeSurfaceSwitch::StaticType = FNodeType(Node::EType::SurfaceSwitch, NodeSurface::GetStaticType());
	FNodeType NodeSurfaceVariation::StaticType = FNodeType(Node::EType::SurfaceVariation, NodeSurface::GetStaticType());

	FNodeType NodeExternal::StaticType = FNodeType(Node::EType::External, Node::GetStaticType());
	FNodeType NodeExternalOperation::StaticType = FNodeType(Node::EType::ExternalOperation, NodeExternal::GetStaticType());
	FNodeType NodeExternalParameter::StaticType = FNodeType(Node::EType::ExternalParameter, NodeExternal::GetStaticType());
	FNodeType NodeExternalSwitch::StaticType = FNodeType(Node::EType::ExternalSwitch, NodeExternal::GetStaticType());
	
	FNodeType NodeLOD::StaticType = FNodeType(Node::EType::LOD, Node::GetStaticType());

	FNodeType NodeSkeletalMesh::StaticType = FNodeType(Node::EType::SkeletalMesh, Node::GetStaticType());
	FNodeType NodeSkeletalMeshNew::StaticType = FNodeType(Node::EType::SkeletalMeshNew, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshMerge::StaticType = FNodeType(Node::EType::SkeletalMeshMerge, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshConvert::StaticType = FNodeType(Node::EType::SkeletalMeshConvert, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshModify::StaticType = FNodeType(Node::EType::SkeletalMeshModify, Node::GetStaticType());
	FNodeType NodeSkeletalMeshMorph::StaticType = FNodeType(Node::EType::SkeletalMeshMorph, Node::GetStaticType());
	FNodeType NodeSkeletalMeshReshape::StaticType = FNodeType(Node::EType::SkeletalMeshReshape, Node::GetStaticType());
	FNodeType NodeSkeletalMeshSwitch::StaticType = FNodeType(Node::EType::SkeletalMeshSwitch, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshVariation::StaticType = FNodeType(Node::EType::SkeletalMeshVariation, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshTransform::StaticType = FNodeType(Node::EType::SkeletalMeshTransform, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshTransformWithBone::StaticType = FNodeType(Node::EType::SkeletalMeshTransformWithBone, NodeSkeletalMesh::GetStaticType());
	FNodeType NodeSkeletalMeshClipWithSkeletalMesh::StaticType = FNodeType(Node::EType::SkeletalMeshClipWithSkeletalMesh, NodeSkeletalMesh::GetStaticType());

	FNodeType NodeSkeletalMeshObject::StaticType = FNodeType(Node::EType::SkeletalMeshObject, Node::GetStaticType());
	FNodeType NodeSkeletalMeshObjectConvert::StaticType = FNodeType(Node::EType::SkeletalMeshObjectConvert, NodeSkeletalMeshObject::GetStaticType());
	FNodeType NodeSkeletalMeshObjectParameter::StaticType = FNodeType(Node::EType::SkeletalMeshObjectParameter, NodeSkeletalMeshObject::GetStaticType());
	FNodeType NodeSkeletalMeshObjectSwitch::StaticType = FNodeType(Node::EType::SkeletalMeshObjectSwitch, NodeSkeletalMeshObject::GetStaticType());

	FNodeType NodeExtensionData::StaticType = FNodeType(Node::EType::ExtensionData, Node::GetStaticType());
	FNodeType NodeExtensionDataConstant::StaticType = FNodeType(Node::EType::ExtensionDataConstant, NodeExtensionData::GetStaticType());

	FNodeType NodeImage::StaticType = FNodeType(Node::EType::Image, Node::GetStaticType());
	FNodeType NodeImageConstant::StaticType = FNodeType(Node::EType::ImageConstant, NodeImage::GetStaticType());
	FNodeType NodeImageTable::StaticType = FNodeType(Node::EType::ImageTable, NodeImage::GetStaticType());
	FNodeType NodeImageFormat::StaticType = FNodeType(Node::EType::ImageFormat, NodeImage::GetStaticType());
	FNodeType NodeImageConform::StaticType = FNodeType(Node::EType::ImageConform, NodeImage::GetStaticType());
	FNodeType NodeImageBinarise::StaticType = FNodeType(Node::EType::ImageBinarise, NodeImage::GetStaticType());
	FNodeType NodeImageConditional::StaticType = FNodeType(Node::EType::ImageConditional, NodeImage::GetStaticType());
	FNodeType NodeImageInterpolate::StaticType = FNodeType(Node::EType::ImageInterpolate, NodeImage::GetStaticType());
	FNodeType NodeImageInvert::StaticType = FNodeType(Node::EType::ImageInvert, NodeImage::GetStaticType());
	FNodeType NodeImageLayer::StaticType = FNodeType(Node::EType::ImageLayer, NodeImage::GetStaticType());
	FNodeType NodeImageLayerColor::StaticType = FNodeType(Node::EType::ImageLayerColor, NodeImage::GetStaticType());
	FNodeType NodeImageLuminance::StaticType = FNodeType(Node::EType::ImageLuminance, NodeImage::GetStaticType());
	FNodeType NodeImageMipmap::StaticType = FNodeType(Node::EType::ImageMipmap, NodeImage::GetStaticType());
	FNodeType NodeImageMultiLayer::StaticType = FNodeType(Node::EType::ImageMultiLayer, NodeImage::GetStaticType());
	FNodeType NodeImageNormalComposite::StaticType = FNodeType(Node::EType::ImageNormalComposite, NodeImage::GetStaticType());
	FNodeType NodeImagePlainColor::StaticType = FNodeType(Node::EType::ImagePlainColor, NodeImage::GetStaticType());
	FNodeType NodeImageProject::StaticType = FNodeType(Node::EType::ImageProject, NodeImage::GetStaticType());
	FNodeType NodeImageResize::StaticType = FNodeType(Node::EType::ImageResize, NodeImage::GetStaticType());
	FNodeType NodeImageSaturate::StaticType = FNodeType(Node::EType::ImageSaturate, NodeImage::GetStaticType());
	FNodeType NodeImageSwitch::StaticType = FNodeType(Node::EType::ImageSwitch, NodeImage::GetStaticType());
	FNodeType NodeImageSwizzle::StaticType = FNodeType(Node::EType::ImageSwizzle, NodeImage::GetStaticType());
	FNodeType NodeImageTransform::StaticType = FNodeType(Node::EType::ImageTransform, NodeImage::GetStaticType());
	FNodeType NodeImageVariation::StaticType = FNodeType(Node::EType::ImageVariation, NodeImage::GetStaticType());
	FNodeType NodeImageColorMap::StaticType = FNodeType(Node::EType::ImageColorMap, NodeImage::GetStaticType());
	FNodeType NodeImageFromMaterialParameter::StaticType = FNodeType(Node::EType::ImageFromMaterialParameter, NodeImage::GetStaticType());
	FNodeType NodeImageExternal::StaticType = FNodeType(Node::EType::ImageExternal, NodeImage::GetStaticType());
	FNodeType NodeImageConvert::StaticType = FNodeType(Node::EType::ImageConvert, NodeImage::GetStaticType());

	FNodeType NodeImageObject::StaticType = FNodeType(Node::EType::ImageObject, NodeImage::GetStaticType()); // TODO GMT It should not inherit forom NodeImage but Node.
	FNodeType NodeImageObjectParameter::StaticType = FNodeType(Node::EType::ImageObjectParameter, NodeImage::GetStaticType());
	
	FNodeType NodeColor::StaticType = FNodeType(Node::EType::Color, Node::GetStaticType());
	FNodeType NodeColorConstant::StaticType = FNodeType(Node::EType::ColorConstant, NodeColor::GetStaticType());
	FNodeType NodeColorParameter::StaticType = FNodeType(Node::EType::ColorParameter, NodeColor::GetStaticType());
	FNodeType NodeColorSwitch::StaticType = FNodeType(Node::EType::ColorSwitch, NodeColor::GetStaticType());
	FNodeType NodeColorVariation::StaticType = FNodeType(Node::EType::ColorVariation, NodeColor::GetStaticType());
	FNodeType NodeColorTable::StaticType = FNodeType(Node::EType::ColorTable, NodeColor::GetStaticType());
	FNodeType NodeColorArithmeticOperation::StaticType = FNodeType(Node::EType::ColorArithmeticOperation, NodeColor::GetStaticType());
	FNodeType NodeColorSampleImage::StaticType = FNodeType(Node::EType::ColorSampleImage, NodeColor::GetStaticType());
	FNodeType NodeColorFromScalars::StaticType = FNodeType(Node::EType::ColorFromScalars, NodeColor::GetStaticType());
	FNodeType NodeColorToSRGB::StaticType = FNodeType(Node::EType::ColorLinearToSRGB, NodeColor::GetStaticType());
	FNodeType NodeColorExternal::StaticType = FNodeType(Node::EType::ColorExternal, NodeColor::GetStaticType());

	FNodeType NodeMesh::StaticType = FNodeType(Node::EType::Mesh, Node::GetStaticType());
	FNodeType NodeMeshConstant::StaticType = FNodeType(Node::EType::MeshConstant, NodeMesh::GetStaticType());
	FNodeType NodeMeshFragment::StaticType = FNodeType(Node::EType::MeshFragment, NodeMesh::GetStaticType());
	FNodeType NodeMeshClipMorphPlane::StaticType = FNodeType(Node::EType::MeshClipMorphPlane, NodeMesh::GetStaticType());
	FNodeType NodeMeshClipDeform::StaticType = FNodeType(Node::EType::MeshClipDeform, NodeMesh::GetStaticType());
	FNodeType NodeMeshClipWithMesh::StaticType = FNodeType(Node::EType::MeshClipWithMesh, NodeMesh::GetStaticType());
	FNodeType NodeMeshRemoveMesh::StaticType = FNodeType(Node::EType::MeshRemoveMesh, NodeMesh::GetStaticType());
	FNodeType NodeMeshTransformWithBone::StaticType = FNodeType(Node::EType::MeshTransformWithBone, NodeMesh::GetStaticType());
	FNodeType NodeMeshTransformInMesh::StaticType = FNodeType(Node::EType::MeshTransformInMesh, NodeMesh::GetStaticType());
	FNodeType NodeMeshSkeletalMeshObjectBreak::StaticType = FNodeType(Node::EType::MeshSkeletalMeshObjectBreak, NodeMesh::GetStaticType());
	FNodeType NodeMeshMakeMorph::StaticType = FNodeType(Node::EType::MeshMakeMorph, NodeMesh::GetStaticType());
	FNodeType NodeMeshApplyPose::StaticType = FNodeType(Node::EType::MeshApplyPose, NodeMesh::GetStaticType());
	FNodeType NodeMeshTransform::StaticType = FNodeType(Node::EType::MeshTransform, NodeMesh::GetStaticType());
	FNodeType NodeMeshSwitch::StaticType = FNodeType(Node::EType::MeshSwitch, NodeMesh::GetStaticType());
	FNodeType NodeMeshReshape::StaticType = FNodeType(Node::EType::MeshReshape, NodeMesh::GetStaticType());
	FNodeType NodeMeshMorph::StaticType = FNodeType(Node::EType::MeshMorph, NodeMesh::GetStaticType());
	FNodeType NodeMeshFormat::StaticType = FNodeType(Node::EType::MeshFormat, NodeMesh::GetStaticType());
	FNodeType NodeMeshVariation::StaticType = FNodeType(Node::EType::MeshVariation, NodeMesh::GetStaticType());
	FNodeType NodeMeshTable::StaticType = FNodeType(Node::EType::MeshTable, NodeMesh::GetStaticType());
	FNodeType NodeMeshExternal::StaticType = FNodeType(Node::EType::MeshExternal, NodeMesh::GetStaticType());

	FNodeType NodeModifier::StaticType = FNodeType(Node::EType::Modifier, Node::GetStaticType());
	FNodeType NodeModifierSkeletalMeshMerge::StaticType = FNodeType(Node::EType::ModifierSkeletalMeshMerge, NodeModifier::GetStaticType());
	
	FNodeType NodeSurfaceModifier::StaticType = FNodeType(Node::EType::SurfaceModifier, NodeModifier::GetStaticType());
	FNodeType NodeSurfaceModifierMeshClipDeform::StaticType = FNodeType(Node::EType::SurfaceModifierMeshClipDeform, NodeSurfaceModifier::GetStaticType());
	FNodeType NodeSurfaceModifierMeshClipMorphPlane::StaticType = FNodeType(Node::EType::SurfaceModifierMeshClipMorphPlane, NodeSurfaceModifier::GetStaticType());
	FNodeType NodeSurfaceModifierMeshClipWithMesh::StaticType = FNodeType(Node::EType::SurfaceModifierMeshClipWithMesh, NodeSurfaceModifier::GetStaticType());
	FNodeType NodeSurfaceModifierMeshClipWithUVMask::StaticType = FNodeType(Node::EType::SurfaceModifierMeshClipWithUVMask, NodeSurfaceModifier::GetStaticType());
	FNodeType NodeSurfaceModifierMeshTransformInMesh::StaticType = FNodeType(Node::EType::SurfaceModifierTransformInMesh, NodeSurfaceModifier::GetStaticType());
	FNodeType NodeSurfaceModifierMeshTransformWithBone::StaticType = FNodeType(Node::EType::SurfaceModifierTransformWithBone, NodeSurfaceModifier::GetStaticType());
	FNodeType NodeSurfaceModifierSurfaceEdit::StaticType = FNodeType(Node::EType::SurfaceModifierSurfaceEdit, NodeSurfaceModifier::GetStaticType());
	
	FNodeType NodeMatrix::StaticType = FNodeType(Node::EType::Matrix, Node::GetStaticType());
	FNodeType NodeMatrixConstant::StaticType = FNodeType(Node::EType::MatrixConstant, Node::GetStaticType());
	FNodeType NodeMatrixParameter::StaticType = FNodeType(Node::EType::MatrixParameter, Node::GetStaticType());

	FNodeType NodeString::StaticType = FNodeType(Node::EType::String, Node::GetStaticType());
	FNodeType NodeStringConstant::StaticType = FNodeType(Node::EType::StringConstant, NodeString::GetStaticType());
	FNodeType NodeStringParameter::StaticType = FNodeType(Node::EType::StringParameter, NodeString::GetStaticType());

	FNodeType NodeProjector::StaticType = FNodeType(Node::EType::Projector, Node::GetStaticType());
	FNodeType NodeProjectorConstant::StaticType = FNodeType(Node::EType::ProjectorConstant, NodeProjector::GetStaticType());
	FNodeType NodeProjectorParameter::StaticType = FNodeType(Node::EType::ProjectorParameter, NodeProjector::GetStaticType());

	FNodeType NodeRange::StaticType = FNodeType(Node::EType::Range, Node::GetStaticType());
	FNodeType NodeRangeFromScalar::StaticType = FNodeType(Node::EType::RangeFromScalar, NodeRange::GetStaticType());

	FNodeType NodeMaterial::StaticType = FNodeType(Node::EType::Material, Node::GetStaticType());
	FNodeType NodeMaterialConstant::StaticType = FNodeType(Node::EType::MaterialConstant, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialTable::StaticType = FNodeType(Node::EType::MaterialTable, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialSwitch::StaticType = FNodeType(Node::EType::MaterialSwitch, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialVariation::StaticType = FNodeType(Node::EType::MaterialVariation, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialParameter::StaticType = FNodeType(Node::EType::MaterialParameter, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialExternal::StaticType = FNodeType(Node::EType::MaterialExternal, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialSkeletalMeshObjectBreak::StaticType = FNodeType(Node::EType::MaterialSkeletalMeshObjectBreak, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialSkeletalMeshBreak::StaticType = FNodeType(Node::EType::MaterialSkeletalMeshBreak, NodeMaterial::GetStaticType());
	FNodeType NodeMaterialModify::StaticType = FNodeType(Node::EType::MaterialModify, NodeMaterial::GetStaticType());
	
	FNodeType NodeImageMaterialBreak::StaticType = FNodeType(Node::EType::ImageMaterialBreak, NodeImage::GetStaticType());
	FNodeType NodeScalarMaterialBreak::StaticType = FNodeType(Node::EType::ScalarMaterialBreak, NodeScalar::GetStaticType());
	FNodeType NodeColorMaterialBreak::StaticType = FNodeType(Node::EType::ColorMaterialBreak, NodeColor::GetStaticType());


	FNodeType::FNodeType()
	{
		Type = Node::EType::None;
		Parent = nullptr;
	}


	FNodeType::FNodeType(Node::EType InType, const FNodeType* pParent )
	{
		Type = InType;
		Parent = pParent;
	}


	void Node::SetMessageContext( const void* context )
	{
		MessageContext = context;
	}

	const void* Node::GetMessageContext() const 
	{ 
		return MessageContext; 
	}

}


