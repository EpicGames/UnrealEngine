// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeExtensionDataSwitch.h"

#include "Misc/AssertionMacros.h"
#include "MuT/NodeScalar.h"

namespace UE::Mutable::Private
{
	FNodeType NodeExtensionDataSwitch::StaticType = FNodeType(Node::EType::ExtensionDataSwitch, NodeExtensionData::GetStaticType());


	Ptr<NodeScalar> NodeExtensionDataSwitch::GetParameter() const
	{
		return Parameter.get();
	}

	void NodeExtensionDataSwitch::SetParameter(Ptr<NodeScalar> pNode)
	{
		Parameter = pNode;
	}

	void NodeExtensionDataSwitch::SetOptionCount(int InNumOptions)
	{
		check(InNumOptions >= 0);
		Options.SetNum(InNumOptions);
	}

	Ptr<NodeExtensionData> NodeExtensionDataSwitch::GetOption(int32 t) const
	{
		check(Options.IsValidIndex(t));
		return Options[t].get();
	}

	void NodeExtensionDataSwitch::SetOption(int32 t, Ptr<NodeExtensionData> pNode)
	{
		check(Options.IsValidIndex(t));
		Options[t] = pNode;
	}
}