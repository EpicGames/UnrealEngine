// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeExtensionDataVariation.h"

#include "Misc/AssertionMacros.h"

namespace UE::Mutable::Private
{
	FNodeType NodeExtensionDataVariation::StaticType = FNodeType(Node::EType::ExtensionDataVariation, NodeExtensionData::GetStaticType());


	void NodeExtensionDataVariation::SetDefaultValue(Ptr<NodeExtensionData> InValue)
	{
		DefaultValue = InValue;
	}

	void NodeExtensionDataVariation::SetVariationCount(int32 InCount)
	{
		check(InCount >= 0);
		Variations.SetNum(InCount);
	}

	int NodeExtensionDataVariation::GetVariationCount() const
	{
		return Variations.Num();
	}

	void NodeExtensionDataVariation::SetVariationTag(int32 InIndex, const FString& Tag)
	{
		check(Variations.IsValidIndex(InIndex));

		Variations[InIndex].Tag = Tag;
	}
	//---------------------------------------------------------------------------------------------
	void NodeExtensionDataVariation::SetVariationValue(int32 InIndex, Ptr<NodeExtensionData> InValue)
	{
		check(Variations.IsValidIndex(InIndex));

		Variations[InIndex].Value = InValue;
	}

}
