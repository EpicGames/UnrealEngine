// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeLayout.h"

#include "MuT/NodeMesh.h"
#include "MuR/Raster.h"
#include "MuR/ConvertData.h"
#include "MuR/MeshBufferSet.h"
#include "MuR/MeshPrivate.h"
#include "MuR/MutableMath.h"
#include "MuR/OpImageGrow.h"
#include "MuR/OpMeshPrepareLayout.h"
#include "Math/IntPoint.h"
#include "Misc/AssertionMacros.h"


namespace UE::Mutable::Private
{

	FNodeType NodeLayout::StaticType = FNodeType(Node::EType::Layout, Node::GetStaticType() );


	TManagedPtr<FLayout> NodeLayout::BuildRuntimeLayout(uint32 BlockIDPrefix) const
	{
		TManagedPtr<FLayout> GeneratedLayout = MakeManaged<FLayout>();
		GeneratedLayout->Size = Size;
		GeneratedLayout->MaxSize = MaxSize;
		GeneratedLayout->Strategy = Strategy;
		GeneratedLayout->ReductionMethod = ReductionMethod;

		const int32 BlockCount = Blocks.Num();
		GeneratedLayout->Blocks.SetNum(BlockCount);
		for (int32 BlockIndex = 0; BlockIndex < BlockCount; ++BlockIndex)
		{
			const FSourceLayoutBlock& From = Blocks[BlockIndex];
			FLayoutBlock& To = GeneratedLayout->Blocks[BlockIndex];
			To.Min = From.Min;
			To.Size = From.Size;
			To.Priority = From.Priority;
			To.bReduceBothAxes = From.bReduceBothAxes;
			To.bReduceByTwo = From.bReduceByTwo;

			// TODO: Optimize the format to RLE 1-bit
			if (From.Mask)
			{
				To.MaskIndex = GeneratedLayout->Masks.AddUnique(From.Mask);
			}

			// Assign unique ids to each layout block
			uint64 Id = uint64(BlockIDPrefix) << 32 | uint64(BlockIndex);
			To.Id = Id;
		}

		return GeneratedLayout;
	}


	void NodeLayout::GenerateLayoutBlocks(const UE::Mutable::Private::TManagedPtr<FMesh>& Mesh, int32 LayoutIndex )
	{
		if (!(Mesh && LayoutIndex >= 0 && Size.X > 0 && Size.Y > 0))
		{
			return;
		}

		int32 VertexCount = Mesh->VertexBuffers.GetElementCount();
		
		TArray<bool> VerticesAlreadyClassified;
		{
			VerticesAlreadyClassified.SetNumZeroed(VertexCount);

			TManagedPtr<FLayout> RuntimeLayout = BuildRuntimeLayout(0);
			FMutableLayoutClassifyResult ClassifyResult;
			bool bNormalizeUVs = false;
			bool bClampUVIslands = true;
			if (ClassifyMeshVerticesInLayout(*Mesh, *RuntimeLayout, LayoutIndex, bNormalizeUVs, bClampUVIslands, ClassifyResult))
			{
				for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
				{
					VerticesAlreadyClassified[VertexIndex] = ClassifyResult.LayoutData[VertexIndex] != FMutableLayoutClassifyResult::NullBlockId;
				}
			}
		}

		int32 IndexCount = Mesh->GetIndexCount();
		TArray< FVector2f > UVs;

		// Extract all the triangle edges
		{
			UVs.Reserve(IndexCount * 2);

			UntypedMeshBufferIteratorConst IndexIt(Mesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex, 0);
			UntypedMeshBufferIteratorConst TexIt(Mesh->GetVertexBuffers(), EMeshBufferSemantic::TexCoords, LayoutIndex);

			int32 TriangleCount = IndexCount / 3;
			for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
			{
				uint32 VertexIndex1 = IndexIt.GetAsUINT32();
				IndexIt++;
				uint32 VertexIndex2 = IndexIt.GetAsUINT32();
				IndexIt++;
				uint32 VertexIndex3 = IndexIt.GetAsUINT32();
				IndexIt++;

				// If any vertex belongs to a user-provided block, ignore the triangle: it will be assigned to the 
				// block owning the mask.
				if (VerticesAlreadyClassified[VertexIndex1] || VerticesAlreadyClassified[VertexIndex2] || VerticesAlreadyClassified[VertexIndex3])
				{
					continue;
				}

				FVector2f UV1 = (TexIt + VertexIndex1).GetAsVec2f();
				FVector2f UV2 = (TexIt + VertexIndex2).GetAsVec2f();
				FVector2f UV3 = (TexIt + VertexIndex3).GetAsVec2f();

				UVs.Add( UV1 );
				UVs.Add( UV2 );

				UVs.Add( UV2 );
				UVs.Add( UV3 );

				UVs.Add( UV3 );
				UVs.Add( UV1 );
			}
		}
			
		TArray<box<FIntVector2>> BlockRects;
		BlockRects.Reserve(Blocks.Num());

		// Get the rects of existing blocks
		for (const FSourceLayoutBlock& Block : Blocks)
		{ 
			box<FIntVector2> Rect;
			Rect.min = Block.Min;
			Rect.size = Block.Size;
			BlockRects.Add( Rect );
		}

		// Generate blocks by iterating all the edges
		int32 EdgeCount = UVs.Num()/2;
		for (int32 EdgeIndex = 0; EdgeIndex < EdgeCount; ++EdgeIndex)
		{
			FVector2f APosition = UVs[EdgeIndex * 2];
			FVector2f BPosition = UVs[EdgeIndex * 2+1];

			FIntVector2 AGrid, BGrid;
			AGrid[0] = FMath::FloorToInt32(APosition[0] * Size.X);
			AGrid[1] = FMath::FloorToInt32(APosition[1] * Size.Y);
									  
			BGrid[0] = FMath::FloorToInt32(BPosition[0] * Size.X);
			BGrid[1] = FMath::FloorToInt32(BPosition[1] * Size.Y);

			// TODO: handle cases of UVs on grid edges.
			// floor of UV = 1*GridSize is GridSize which is not AGrid valid range
			//if (AGrid[0] == GridSizeX){ AGrid[0] = GridSizeX-1; }
			//if (AGrid[1] == GridSizeY){	AGrid[1] = GridSizeY-1;	}
			//if (BPosition[0] == GridSizeX){	BPosition[0] = GridSizeX-1;	}
			//if (BPosition[1] == GridSizeY){ BPosition[1] = GridSizeY-1;	}
			
			// AGrid and BPosition are in the same block
			if (AGrid == BGrid)
			{
				bool bIsContained = false;
			
				for (int32 BlockIndex = 0; BlockIndex < BlockRects.Num(); ++BlockIndex)
				{
					// ignore blocks with masks here.
					if (Blocks.IsValidIndex(BlockIndex) && Blocks[BlockIndex].Mask)
					{
						continue;
					}

					if (BlockRects[BlockIndex].Contains(AGrid) || BlockRects[BlockIndex].Contains(BGrid))
					{
						bIsContained = true;
					}
				}
					
				// There is no block that contains them 
				if (!bIsContained)
				{
					box<FIntVector2> NewBlock;
					NewBlock.min = AGrid;
					NewBlock.size = FIntVector2(1, 1);
			
					BlockRects.Add(NewBlock);
				}
			}

			else // they are in different blocks
			{
				int32 ABlockIndex = -1;
				int32 BBlockIndex = -1;
					
				// Get the blocks that contain them
				for (int32 BlockIndex = 0; BlockIndex < BlockRects.Num(); ++BlockIndex)
				{
					if (BlockRects[BlockIndex].Contains(AGrid))
					{
						ABlockIndex = BlockIndex;
					}
					if (BlockRects[BlockIndex].Contains(BGrid))
					{
						BBlockIndex = BlockIndex;
					}
				}
					
				// The blocks are not the same
				if (ABlockIndex != BBlockIndex)
				{
					box<FIntVector2> NewBlock;
						
					//One of the blocks doesn't exist
					if (ABlockIndex != -1 && BBlockIndex == -1)
					{
						NewBlock.min = BGrid;
						NewBlock.size = FIntVector2(1, 1);
						BlockRects[ABlockIndex].Bound(NewBlock);
					}
					else if (BBlockIndex != -1 && ABlockIndex == -1)
					{
						NewBlock.min = AGrid;
						NewBlock.size = FIntVector2(1, 1);
						BlockRects[BBlockIndex].Bound(NewBlock);
					}
					else //Both exist
					{
						BlockRects[ABlockIndex].Bound(BlockRects[BBlockIndex]);
						BlockRects.RemoveAt(BBlockIndex);
					}
				}
				else // the block doesn't exist
				{
					if (ABlockIndex == -1)
					{
						box<FIntVector2> NewBlockA;
						box<FIntVector2> NewBlockB;
			
						NewBlockA.min = AGrid;
						NewBlockB.min = BGrid;
						NewBlockA.size = FIntVector2(1, 1);
						NewBlockB.size = FIntVector2(1, 1);
			
						NewBlockA.Bound(NewBlockB);
						BlockRects.Add(NewBlockA);
					}
				}
			}
		}
			
		bool bHasIntersections = true;
			
		// Check if blocks intersect with each other or are null
		while (bHasIntersections)
		{
			bHasIntersections = false;
			
			for (int32 i = 0; !bHasIntersections && i < BlockRects.Num(); ++i)
			{
				for (int32 j = 0; j < BlockRects.Num(); ++j)
				{
					if (i != j && BlockRects[i].IntersectsExclusive(BlockRects[j]))
					{
						BlockRects[i].Bound(BlockRects[j]);
						BlockRects.RemoveAt(j);
						bHasIntersections = true;
						break;
					}
				}

				// Remove degenerated blocks.
				if (BlockRects[i].size.X * BlockRects[i].size.Y == 0)
				{
					BlockRects.RemoveAt(i);
					bHasIntersections = true;
					break;
				}
			}
		}
			
		int32 NumBlocks = BlockRects.Num();
			
		// Generate the layout blocks
		if (NumBlocks > 0)
		{
			Blocks.SetNum(NumBlocks);
			
			for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
			{
				Blocks[BlockIndex].Min = BlockRects[BlockIndex].min;
				Blocks[BlockIndex].Size = BlockRects[BlockIndex].size;
			}
		}
	}


	void NodeLayout::GenerateLayoutBlocksFromUVIslands(const UE::Mutable::Private::TManagedPtr<FMesh>& Mesh, int32 LayoutIndex)
	{
		if (!(Mesh && LayoutIndex >= 0 && Size.X > 0 && Size.Y > 0))
		{
			return;
		}

		int32 IndexCount = Mesh->GetIndexCount();
		const int32 NumTriangles = IndexCount / 3;
		const int32 NumVertices = Mesh->GetVertexCount();

		TArray<FTriangleInfo> Triangles;
		Triangles.SetNumUninitialized(NumTriangles);

		// Vertices mapped to unique vertex index
		TArray<int32> CollapsedVertices;

		// Vertex to face map used to speed up connectivity building
		TMultiMap<int32, uint32> VertexToFaceMap;
		VertexToFaceMap.Reserve(NumVertices*4);

		// Find Unique Vertices
		MeshCreateCollapsedVertexMap(Mesh.Get(), CollapsedVertices);

		UntypedMeshBufferIteratorConst ItIndices(Mesh->GetIndexBuffers(), EMeshBufferSemantic::VertexIndex);
		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
		{
			uint32 Index0 = ItIndices.GetAsUINT32();
			++ItIndices;
			uint32 Index1 = ItIndices.GetAsUINT32();
			++ItIndices;
			uint32 Index2 = ItIndices.GetAsUINT32();
			++ItIndices;

			FTriangleInfo& Triangle = Triangles[TriangleIndex];

			Triangle.Indices[0] = Index0;
			Triangle.Indices[1] = Index1;
			Triangle.Indices[2] = Index2;
			Triangle.CollapsedIndices[0] = CollapsedVertices[Index0];
			Triangle.CollapsedIndices[1] = CollapsedVertices[Index1];
			Triangle.CollapsedIndices[2] = CollapsedVertices[Index2];

			Triangle.BlockIndices[0] = 0;
			Triangle.BlockIndices[1] = 0;
			Triangle.BlockIndices[2] = 0;
			Triangle.bUVsFixed = false;

			VertexToFaceMap.Add(Triangle.CollapsedIndices[0], TriangleIndex);
			VertexToFaceMap.Add(Triangle.CollapsedIndices[1], TriangleIndex);
			VertexToFaceMap.Add(Triangle.CollapsedIndices[2], TriangleIndex);
		}

		// Get a copy of the UVs as FVector2f to work with them. 
		TArray<FVector2f> TexCoords;
		{
			int32 TexCoordsBufferIndex = -1;
			int32 TexCoordsChannelIndex = -1;
			Mesh->GetVertexBuffers().FindChannel(EMeshBufferSemantic::TexCoords, LayoutIndex, &TexCoordsBufferIndex, &TexCoordsChannelIndex);
			check(TexCoordsBufferIndex >= 0);
			check(TexCoordsChannelIndex >= 0);

			const FMeshBufferChannel& TexCoordsChannel = Mesh->VertexBuffers.Buffers[TexCoordsBufferIndex].Channels[TexCoordsChannelIndex];
			check(TexCoordsChannel.Semantic == EMeshBufferSemantic::TexCoords);

			uint8* TexCoordData = Mesh->GetVertexBuffers().GetBufferData(TexCoordsBufferIndex);
			int32 ElemSize = Mesh->GetVertexBuffers().GetElementSize(TexCoordsBufferIndex);

			TexCoords.SetNumUninitialized(NumVertices);

			const uint8* pVertices = TexCoordData;
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				FVector2f& UV = TexCoords[VertexIndex];
				if (TexCoordsChannel.Format == EMeshBufferFormat::Float32)
				{
					UV = *((FVector2f*)pVertices);
				}
				else if (TexCoordsChannel.Format == EMeshBufferFormat::Float16)
				{
					const FFloat16* pUV = reinterpret_cast<const FFloat16*>(pVertices);
					UV = FVector2f(float(pUV[0]), float(pUV[1]));
				}

				pVertices += ElemSize;
			}
		}

		// Generate UV islands
		TArray<int16> IslandPerTriangle;
		TArray<box<FVector2f>> IslandBlocks;
		{
			int16 IslandCount = 0;
			IslandPerTriangle.Init(-1, NumTriangles);
			for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
			{
				if (IslandPerTriangle[TriangleIndex] >= 0)
				{
					// Already assigned to an island
					continue;
				}

				// Find triangles from the same UV Island
				TArray<uint32> TriangleIndices;
				GetUVIsland(Triangles, TriangleIndex, TriangleIndices, TexCoords, VertexToFaceMap);

				box<FVector2f> IslandBlock;
				for (int32 IslandTriangleIndexIndex = 0; IslandTriangleIndexIndex < TriangleIndices.Num(); ++IslandTriangleIndexIndex)
				{
					uint32 IslandTriangleIndex = TriangleIndices[IslandTriangleIndexIndex];
					FTriangleInfo& Triangle = Triangles[IslandTriangleIndex];

					// Mark the triangle as already assigned to an island.
					IslandPerTriangle[IslandTriangleIndex] = IslandCount;

					FVector2f UV0 = TexCoords[Triangle.Indices[0]];
					FVector2f UV1 = TexCoords[Triangle.Indices[1]];
					FVector2f UV2 = TexCoords[Triangle.Indices[2]];

					if (IslandTriangleIndexIndex == 0)
					{
						IslandBlock.min = UV0;
					}

					IslandBlock.Bound(UV0);
					IslandBlock.Bound(UV1);
					IslandBlock.Bound(UV2);
				}

				IslandBlocks.Add(IslandBlock);

				++IslandCount;
			}
		}


		TArray<box<FIntVector2>> BlockRects;
		BlockRects.Reserve(Blocks.Num());

		// Get the rects of existing blocks
		for (const FSourceLayoutBlock& Block : Blocks)
		{
			box<FIntVector2> Rect;
			Rect.min = Block.Min;
			Rect.size = Block.Size;
			BlockRects.Add(Rect);
		}

		int32 IslandBlocksOffset = BlockRects.Num();

		for (box<FVector2f>& UVBlock : IslandBlocks)
		{
			box<FIntVector2> IslandBlock;
			IslandBlock.min.X = FMath::FloorToInt32(UVBlock.min.X * Size.X);
			IslandBlock.min.Y = FMath::FloorToInt32(UVBlock.min.Y * Size.Y);
			FVector2f MaxF = UVBlock.min + UVBlock.size;
			FIntVector2 Max;
			Max.X = FMath::CeilToInt32(MaxF.X * Size.X);
			Max.Y = FMath::CeilToInt32(MaxF.Y * Size.Y);
			IslandBlock.size.X = Max.X - IslandBlock.min.X;
			IslandBlock.size.Y = Max.Y - IslandBlock.min.Y;
			BlockRects.Add(IslandBlock);
		}

		TArray<bool> RemovedBlocks;
		RemovedBlocks.Init(false, BlockRects.Num());

		// Merge blocks if necessary
		if (bMergeChildBlocks)
		{
			for (int32 BlockIndex = IslandBlocksOffset; BlockIndex < BlockRects.Num(); ++BlockIndex)
			{
				int32 IslandBlockIndex = BlockIndex - IslandBlocksOffset;
				box<FIntVector2> ThisRect = BlockRects[BlockIndex];
				for (int32 OtherBlockIndex = BlockIndex+1; OtherBlockIndex < BlockRects.Num(); ++OtherBlockIndex)
				{
					box<FIntVector2> OtherRect = BlockRects[OtherBlockIndex];
					if (!RemovedBlocks[OtherBlockIndex] && ThisRect.Contains(OtherRect))
					{
						RemovedBlocks[OtherBlockIndex] = true;

						// Merge other into this 
						int32 OtherIslandBlockIndex = OtherBlockIndex - IslandBlocksOffset;
						for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
						{
							if (IslandPerTriangle[TriangleIndex] == OtherIslandBlockIndex)
							{
								IslandPerTriangle[TriangleIndex] = IslandBlockIndex;
							}
						}
					}
				}

			}
		}

		// Check if blocks intersect with each other or are null
		// actually this code wouldn't work because we need to update IslandBlocksOffset too
		//bool bHasIntersections = true;
		//while (bHasIntersections)
		//{
		//	bHasIntersections = false;
		//	
		//	for (int32 i = 0; !bHasIntersections && i < BlockRects.Num(); ++i)
		//	{
		//		for (int32 j = 0; j < BlockRects.Num(); ++j)
		//		{
		//			if (i != j && BlockRects[i].IntersectsExclusive(BlockRects[j]))
		//			{
		//				BlockRects[i].Bound(BlockRects[j]);
		//				BlockRects.RemoveAt(j);
		//				bHasIntersections = true;
		//				break;
		//			}
		//		}

		//		// Remove degenerated blocks.
		//		if (BlockRects[i].size.X * BlockRects[i].size.Y == 0)
		//		{
		//			BlockRects.RemoveAt(i);
		//			bHasIntersections = true;
		//			break;
		//		}
		//	}
		//}
			
		// Raster all the faces
		class WhitePixelProcessor
		{
		public:
			inline void ProcessPixel(uint8* pBufferPos, float[1]) const
			{
				pBufferPos[0] = 255;
			}

			inline void operator()(uint8* BufferPos, float Interpolators[1]) const
			{
				ProcessPixel(BufferPos, Interpolators);
			}
		};

		WhitePixelProcessor pixelProc;

		// Generate the layout blocks
		Blocks.Reserve(BlockRects.Num());
		for (int32 IslandBlockIndex = 0; IslandBlockIndex < IslandBlocks.Num(); ++IslandBlockIndex)
		{
			int32 BlockIndex = IslandBlockIndex + IslandBlocksOffset;

			if (RemovedBlocks[BlockIndex])
			{
				continue;
			}

			FSourceLayoutBlock& Block = Blocks.Emplace_GetRef();
			Block.Min = BlockRects[BlockIndex].min;
			Block.Size = BlockRects[BlockIndex].size;

			// Generate the block mask
			// TODO: Size?
			int32 SizeX = 1024;
			int32 SizeY = 1024;
			TManagedPtr<FImage> Mask = MakeManaged<FImage>(SizeX, SizeY, 1, EImageFormat::L_UByte, EInitializationType::Black);
			const TArrayView<uint8> ImageData = Mask->DataStorage.GetLOD(0);

			for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
			{
				if (IslandPerTriangle[TriangleIndex] != IslandBlockIndex)
				{
					continue;
				}

				FTriangleInfo& ThisTriangle = Triangles[TriangleIndex];
				FVector2f UV0 = TexCoords[ThisTriangle.Indices[0]];
				FVector2f UV1 = TexCoords[ThisTriangle.Indices[1]];
				FVector2f UV2 = TexCoords[ThisTriangle.Indices[2]];

				// TODO Modulo doesn't work with cross-tile blocks
				UV0.X = FMath::Fmod(UV0.X, 1.0);
				UV0.Y = FMath::Fmod(UV0.Y, 1.0);
				UV1.X = FMath::Fmod(UV1.X, 1.0);
				UV1.Y = FMath::Fmod(UV1.Y, 1.0);
				UV2.X = FMath::Fmod(UV2.X, 1.0);
				UV2.Y = FMath::Fmod(UV2.Y, 1.0);

				RasterVertex<1> V0(UV0.X* SizeX, UV0.Y* SizeY);
				RasterVertex<1> V1(UV1.X* SizeX, UV1.Y* SizeY);
				RasterVertex<1> V2(UV2.X* SizeX, UV2.Y* SizeY);

				constexpr int32 NumInterpolators = 1;
				Triangle<NumInterpolators>(ImageData.GetData(), ImageData.Num(),
					SizeX, SizeY,
					1,
					V0,V1,V2,
					pixelProc,
					false);

			}

			// TODO: Clamp UV islands always?
			ImageGrow(Mask.Get());
			ImageGrow(Mask.Get());

			Block.Mask = Mask;
		}

	}

}


