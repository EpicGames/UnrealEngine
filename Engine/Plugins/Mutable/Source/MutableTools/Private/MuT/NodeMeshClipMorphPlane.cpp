// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeMeshClipMorphPlane.h"

#include "Misc/AssertionMacros.h"
#include "MuR/MutableMath.h"
#include "MuT/NodeLayout.h"


namespace UE::Mutable::Private
{

	NodeMeshPtr NodeMeshClipMorphPlane::GetSource() const
	{
		return Source;
	}


    //---------------------------------------------------------------------------------------------
    void NodeMeshClipMorphPlane::SetSource( NodeMesh* p )
    {
        Source = p;
    }


	//---------------------------------------------------------------------------------------------
	void NodeMeshClipMorphPlane::SetPlane(FVector3f Center, FVector3f Normal)
	{
		Parameters.Origin = Center;
		Parameters.Normal = Normal;
	}


	//---------------------------------------------------------------------------------------------
	void NodeMeshClipMorphPlane::SetParams(float dist, float factor)
	{
		Parameters.DistanceToPlane = dist;
		Parameters.LinearityFactor = factor;
	}

	//---------------------------------------------------------------------------------------------
	void NodeMeshClipMorphPlane::SetMorphEllipse(float radius1, float radius2, float rotation)
	{
		Parameters.Radius1 = radius1;
		Parameters.Radius2 = radius2;
		Parameters.Rotation = rotation;
	}

	//---------------------------------------------------------------------------------------------
	void NodeMeshClipMorphPlane::SetVertexSelectionBox(FVector3f Center, FVector3f Radius)
	{
		Parameters.VertexSelectionType = EClipVertexSelectionType::Shape;
		Parameters.SelectionBoxOrigin = Center;
		Parameters.SelectionBoxRadius = Radius;
	}

	//---------------------------------------------------------------------------------------------
	void NodeMeshClipMorphPlane::SetVertexSelectionBone(FName BoneName, float MaxEffectRadius)
	{
		Parameters.VertexSelectionType = EClipVertexSelectionType::BoneHierarchy;
		Parameters.VertexSelectionBone = BoneName;
		Parameters.MaxEffectRadius = MaxEffectRadius;
	}

}
