// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeObjectGroup.h"

#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "MuT/NodeLayout.h"


namespace UE::Mutable::Private
{

	const FString& NodeObjectGroup::GetName() const
	{
		return Name;
	}


	void NodeObjectGroup::SetName( const FString& InName )
	{
		Name = InName;
	}


	const FString& NodeObjectGroup::GetUid() const
	{
		return Uid;
	}


	void NodeObjectGroup::SetUid( const FString& InUid )
	{
		Uid = InUid;
	}

}


