// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeObjectNew.h"

#include "HAL/PlatformString.h"
#include "Misc/AssertionMacros.h"
#include "MuT/CompilerPrivate.h"
#include "MuT/NodeComponent.h"
#include "MuT/NodeComponentNew.h"
#include "MuT/NodeLOD.h"
#include "MuT/NodeMesh.h"
#include "MuT/NodeMeshConstant.h"
#include "MuT/NodeSurface.h"
#include "MuT/NodeSurfaceNew.h"


namespace UE::Mutable::Private
{


	const FString& NodeObjectNew::GetName() const
	{
		return Name;
	}


	void NodeObjectNew::SetName( const FString& InName )
	{
		Name = InName;
	}


	const FString& NodeObjectNew::GetUid() const
	{
		return Uid;
	}


	void NodeObjectNew::SetUid( const FString& InUid )
	{
		Uid = InUid;
	}


	int32 NodeObjectNew::GetStateCount() const
	{
		return States.Num();
	}


	void NodeObjectNew::SetStateCount( int32 c )
	{
		States.SetNum( c );
	}


	void NodeObjectNew::SetStateName( int32 s, const FString& n )
	{
		check( s>=0 && s<GetStateCount() );
		States[s].Name = n;
	}


	bool NodeObjectNew::HasStateParam( int32 s, const FString& param ) const
	{
		check( s>=0 && s<GetStateCount() );
		return States[s].RuntimeParams.Contains( param );
	}


	void NodeObjectNew::AddStateParam( int32 s, const FString& param )
	{
		check( s>=0 && s<GetStateCount() );

		if (!HasStateParam(s,param))
		{
			States[s].RuntimeParams.Add( param );
		}
	}


    void NodeObjectNew::SetStateProperties( int32 StateIndex, 
		ETextureCompressionStrategy TextureCompressionStrategy, 
		bool bOnlyFirstLOD, 
		uint8 NumExtraLODsToBuildAfterFirstLOD )
    {
        check(StateIndex >=0 && StateIndex<GetStateCount() );

		FStateOptimizationOptions& Data = States[StateIndex].Optimisation;
        Data.TextureCompressionStrategy = TextureCompressionStrategy;
		Data.bOnlyFirstLOD = bOnlyFirstLOD;
		Data.NumExtraLODsToBuildAfterFirstLOD = NumExtraLODsToBuildAfterFirstLOD;
    }

	void NodeObjectNew::AddExtensionDataNode(Ptr<NodeExtensionData> Node)
	{
		ExtensionDataNodes.Add(Node);
	}
}


