// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AssertionMacros.h"
#include "MuR/MutableMath.h"
#include "MuR/Parameters.h"
#include "MuT/Node.h"
#include "MuT/NodeProjector.h"

namespace UE::Mutable::Private
{

    void NodeProjectorConstant::GetValue( EProjectorType* OutType,
		FVector3f* OutPos,
		FVector3f* OutDir,
		FVector3f* OutUp,
		FVector3f* OutScale,
		float* OutProjectionAngle ) const
	{
        if (OutType) *OutType = Type;
        if (OutPos) *OutPos = Position;
        if (OutDir) *OutDir = Direction;
        if (OutUp) *OutUp = Up;
        if (OutScale) *OutScale = Scale;
        if (OutProjectionAngle) *OutProjectionAngle = ProjectionAngle;
    }


	void NodeProjectorConstant::SetValue( EProjectorType type, FVector3f pos, FVector3f dir, FVector3f up, FVector3f scale, float projectionAngle )
	{
        Type = type;
		Position = pos;
		Direction = dir;
		Up = up;
        Scale = scale;
        ProjectionAngle = projectionAngle;
	}

}


