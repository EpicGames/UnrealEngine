// Copyright Epic Games, Inc. All Rights Reserved.


#include "Misc/AssertionMacros.h"
#include "MuR/MutableMath.h"
#include "MuR/Parameters.h"
#include "MuR/Ptr.h"
#include "MuR/RefCounted.h"
#include "MuT/Node.h"
#include "MuT/NodeProjector.h"
#include "MuT/NodeRange.h"


namespace UE::Mutable::Private
{
	void NodeProjectorParameter::SetName( const FString& InName )
	{
		Name = InName;
	}


	const FString& NodeProjectorParameter::GetUid() const
	{
		return UID;
	}


	void NodeProjectorParameter::SetUid( const FString& InUid)
	{
		UID = InUid;
	}


    void NodeProjectorParameter::SetDefaultValue( EProjectorType type,
		FVector3f pos,
		FVector3f dir,
		FVector3f up,
		FVector3f scale,
		float projectionAngle )
	{
        Type = type;
        Position = pos;
		Direction = dir;
		Up = up;
        Scale = scale;
        ProjectionAngle = projectionAngle;
    }


    void NodeProjectorParameter::SetRangeCount( int32 i )
    {
        check(i>=0);
        Ranges.SetNum(i);
    }


    void NodeProjectorParameter::SetRange( int32 i, Ptr<NodeRange> InRange )
    {
        check( i>=0 && i<Ranges.Num() );
        if ( i>=0 && i<Ranges.Num() )
        {
            Ranges[i] = InRange;
        }
    }

}


