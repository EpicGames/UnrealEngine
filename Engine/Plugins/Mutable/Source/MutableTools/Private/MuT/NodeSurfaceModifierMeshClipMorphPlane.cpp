// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/NodeSurfaceModifierMeshClipMorphPlane.h"

#include "MuR/MutableMath.h"

namespace UE::Mutable::Private
{

    void NodeSurfaceModifierMeshClipMorphPlane::SetPlane(FVector3f Center, FVector3f Normal)
	{
		Parameters.Origin = Center;
		Parameters.Normal = Normal;
	}


	void NodeSurfaceModifierMeshClipMorphPlane::SetParams(float dist, float factor)
	{
		Parameters.DistanceToPlane = dist;
		Parameters.LinearityFactor = factor;
	}


	void NodeSurfaceModifierMeshClipMorphPlane::SetMorphEllipse(float radius1, float radius2, float rotation)
	{
		Parameters.Radius1 = radius1;
		Parameters.Radius2 = radius2;
		Parameters.Rotation = rotation;
	}


	void NodeSurfaceModifierMeshClipMorphPlane::SetVertexSelectionBox(float centerX, float centerY, float centerZ, float radiusX, float radiusY, float radiusZ)
	{
		Parameters.VertexSelectionType = EClipVertexSelectionType::Shape;
		Parameters.SelectionBoxOrigin = FVector3f(centerX, centerY, centerZ);
		Parameters.SelectionBoxRadius = FVector3f(radiusX, radiusY, radiusZ);
	}


	void NodeSurfaceModifierMeshClipMorphPlane::SetVertexSelectionBone(FName BoneName, float maxEffectRadius)
	{
		Parameters.VertexSelectionType = EClipVertexSelectionType::BoneHierarchy;
		Parameters.VertexSelectionBone = BoneName;
		Parameters.MaxEffectRadius = maxEffectRadius;
	}

}
