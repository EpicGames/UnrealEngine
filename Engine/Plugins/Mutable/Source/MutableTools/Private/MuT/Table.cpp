// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuT/Table.h"

#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "MuR/Image.h"
#include "MuR/Mesh.h"
#include "MuR/MutableMath.h"
#include "MuR/Serialisation.h"
#include "MuR/Serialisation.h"
#include "MuT/TablePrivate.h"


namespace UE::Mutable::Private
{
	MUTABLE_IMPLEMENT_ENUM_SERIALISABLE(ETableColumnType)


	//---------------------------------------------------------------------------------------------
	FTable::FTable()
	{
		m_pD = new Private();
	}


	//---------------------------------------------------------------------------------------------
	FTable::~FTable()
	{
        check( m_pD );
		delete m_pD;
		m_pD = 0;
	}


	FTable::Private* FTable::GetPrivate() const
	{
		return m_pD;
	}


	void FTable::SetName( const FString& InName )
	{
		m_pD->Name = InName;
	}


	const FString& FTable::GetName() const
	{
		return m_pD->Name;
	}


	int32 FTable::AddColumn(const FString& Name, ETableColumnType type )
	{
		int32 result = m_pD->Columns.Num();

		FTableColumn c;

		c.Name = Name;
		c.Type = type;

		m_pD->Columns.Add(c);

		// Add it to all rows
		for ( int32 r=0; r<m_pD->Rows.Num(); ++r )
		{
			m_pD->Rows[r].Values.Add( FTableValue() );
		}

		return result;
	}


	int32 FTable::FindColumn(const FString& Name ) const
	{
		int32 res = -1;

		for ( int32 c=0; c<m_pD->Columns.Num(); ++c )
		{
			if ( m_pD->Columns[c].Name == Name )
			{
				res = c;
			}
		}

		return res;
	}


    void FTable::AddRow( uint32 id )
	{
		check( m_pD->FindRow(id)<0 );

		FTableRow r;
		r.Id = id;
		r.Values.SetNum( m_pD->Columns.Num() );
		m_pD->Rows.Add( r );
	}


	void FTable::SetCell(int32 Column, uint32 RowId, int32 IntValue, const void* ErrorContext)
	{
		int32 Row = m_pD->FindRow(RowId);
		check(Row >= 0);
		check(Column < m_pD->Columns.Num());
		check(Column < m_pD->Rows[Row].Values.Num());
		check(m_pD->Columns[Column].Type == ETableColumnType::Material);

		m_pD->Rows[Row].Values[Column].Int = IntValue;
		m_pD->Rows[Row].Values[Column].ErrorContext = ErrorContext;
	}


	void FTable::SetCell( int32 Column, uint32 RowId, float Value, const void* ErrorContext)
	{
		int32 Row = m_pD->FindRow(RowId);
		check( Row>=0 );
		check( Column < m_pD->Columns.Num() );
		check( Column < m_pD->Rows[Row].Values.Num() );
		check( m_pD->Columns[Column].Type == ETableColumnType::Scalar );

		m_pD->Rows[ Row ].Values[Column].Scalar = Value;
		m_pD->Rows[ Row ].Values[Column].ErrorContext = ErrorContext;
	}


    void FTable::SetCell( int32 Column, uint32 RowId, const FVector4f& Value, const void* ErrorContext)
	{
		int32 Row = m_pD->FindRow(RowId);
		check( Row>=0 );
		check( Column < m_pD->Columns.Num() );
		check( Column < m_pD->Rows[Row].Values.Num() );
		check( m_pD->Columns[Column].Type == ETableColumnType::Color );

		m_pD->Rows[ Row ].Values[Column].Color = Value;
		m_pD->Rows[ Row ].Values[Column].ErrorContext = ErrorContext;
	}


    void FTable::SetCell(int32 Column, uint32 RowId, const UE::Mutable::Private::TManagedPtr<FImage>& Value, const void* ErrorContext)
	{
		int32 Row = m_pD->FindRow(RowId);
		check( Row>=0 );
		check( Column < m_pD->Columns.Num() );
		check( Column < m_pD->Rows[Row].Values.Num() );
		check( m_pD->Columns[Column].Type == ETableColumnType::Image );

		m_pD->Rows[ Row ].Values[Column].Image = Value;
		m_pD->Rows[ Row ].Values[Column].ErrorContext = ErrorContext;
	}


    void FTable::SetCell( int32 Column, uint32 RowId, const UE::Mutable::Private::TManagedPtr<FMesh>& Value, const void* ErrorContext )
	{
		int32 Row = m_pD->FindRow(RowId);
		check( Row>=0 );
		check( Column < m_pD->Columns.Num() );
		check( Column < m_pD->Rows[Row].Values.Num() );
		check( m_pD->Columns[Column].Type == ETableColumnType::Mesh );

		m_pD->Rows[ Row ].Values[Column].Mesh = Value;
		m_pD->Rows[ Row ].Values[Column].ErrorContext = ErrorContext;
	}


    void FTable::SetCell( int32 Column, uint32 RowId, const FString& Value, const void* ErrorContext )
	{
		int32 Row = m_pD->FindRow(RowId);
		check( Row>=0 );
		check( Column < m_pD->Columns.Num() );
		check( Column < m_pD->Rows[Row].Values.Num() );
		check( m_pD->Columns[Column].Type == ETableColumnType::String );

		m_pD->Rows[ Row ].Values[Column].String = Value;
		m_pD->Rows[ Row ].Values[Column].ErrorContext = ErrorContext;
	}


}

