// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/TypeInfo.h"

#include "MuR/Parameters.h"
#include "MuR/Types.h"

namespace UE::Mutable::Private
{

	const char* TypeInfo::s_imageFormatName[size_t(EImageFormat::Count)] =
	{
		"None",
		"RGB U888",
		"RGBA U8888",
		"L U8",

		"PVRTC2",
		"PVRTC4",
		"ETC1",
		"ETC2",

		"L U8 RLE",
		"RGB U888 RLE",
		"RGBA U8888 RLE",
		"L U1 RLE",

		"BC1",
		"BC2",
		"BC3",
		"BC4",
		"BC5",
		"BC6",
		"BC7",

		"BGRA U8888",

		"ASTC_4x4_RGB_LDR",
		"ASTC_4x4_RGBA_LDR",
		"ASTC_4x4_RG_LDR",

		"ASTC_8x8_RGB_LDR",
		"ASTC_8x8_RGBA_LDR",
		"ASTC_8x8_RG_LDR",
		"ASTC_12x12_RGB_LDR",
		"ASTC_12x12_RGBA_LDR",
		"ASTC_12x12_RG_LDR",

		"ASTC_6x6_RGB_LDR",
		"ASTC_6x6_RGBA_LDR",
		"ASTC_6x6_RG_LDR",
		"ASTC_10x10_RGB_LDR",
		"ASTC_10x10_RGBA_LDR",
		"ASTC_10x10_RG_LDR",

	};

	static_assert(sizeof(TypeInfo::s_imageFormatName) / sizeof(void*) == int32(EImageFormat::Count));


	const char* TypeInfo::s_meshBufferSemanticName[] =
	{
		"None",

		"VertexIndex",

		"Position",
		"Normal",
		"Tangent",
		"Binormal",
		"Tex Coords",
		"Color",
		"Bone Weights",
		"Bone Indices",

		"Layout Block",

		"Chart",

		"Other",

		"Tangent Space Sign",

		"TriangleIndex",
		"BarycentricCoords",
		"Distance",

		"AltSkinWeight"
	};

	static_assert(sizeof(TypeInfo::s_meshBufferSemanticName) / sizeof(void*) == int32(EMeshBufferSemantic::Count));

	const char* TypeInfo::s_meshBufferFormatName[int32(EMeshBufferFormat::Count)] =
	{
		"None",
		"EMeshBufferFormat::Float16",
		"EMeshBufferFormat::Float32",

		"EMeshBufferFormat::UInt8",
		"EMeshBufferFormat::UInt16",
		"EMeshBufferFormat::UInt32",
		"EMeshBufferFormat::Int8",
		"EMeshBufferFormat::Int16",
		"EMeshBufferFormat::Int32",

		"EMeshBufferFormat::NUInt8",
		"EMeshBufferFormat::NUInt16",
		"EMeshBufferFormat::NUInt32",
		"EMeshBufferFormat::NInt8",
		"EMeshBufferFormat::NInt16",
		"EMeshBufferFormat::NInt32",

		"EMeshBufferFormat::PackedDir8",
		"PackedDir8_WTangentSign",
		"EMeshBufferFormat::PackedDirS8",
		"PackedDirS8_WTangentSign",

		"EMeshBufferFormat::Float64",

		"EMeshBufferFormat::UInt64",
		"EMeshBufferFormat::Int64",
		"NUint64",
		"EMeshBufferFormat::NInt64",
	};

	static_assert(sizeof(TypeInfo::s_meshBufferFormatName) / sizeof(void*) == int32(EMeshBufferFormat::Count));

	const char* TypeInfo::s_projectorTypeName[static_cast<uint32>(EProjectorType::Count)] =
	{
		"Planar",
		"Cylindrical",
		"Wrapping",
	};
	static_assert(sizeof(TypeInfo::s_projectorTypeName) / sizeof(void*) == int32(EProjectorType::Count));

	const char* TypeInfo::s_blendModeName[uint32(EBlendType::_BT_COUNT)] =
	{
		"None",
		"SoftLight",
		"HardLight",
		"Burn",
		"Dodge",
		"Screen",
		"Overlay",
		"Blend",
		"Multiply",
		"Lighten",
		"NormalCombine",
	};
	static_assert(sizeof(TypeInfo::s_blendModeName) / sizeof(void*) == int32(EBlendType::_BT_COUNT));

}
