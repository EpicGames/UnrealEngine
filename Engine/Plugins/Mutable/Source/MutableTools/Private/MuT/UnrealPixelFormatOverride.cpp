// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuT/UnrealPixelFormatOverride.h"

#include "MuR/MutableRuntimeModule.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "Interfaces/ITextureFormatModule.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Engine/TextureDefines.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformFile.h"
#include "TextureCompressorModule.h"
#include "TextureBuildUtilities.h"
#include "ImageCore.h"

static ITextureFormatManagerModule* STextureFormatManager = nullptr;

static FName SPrefixedMutableTextureFormatNameTable[(uint32)UE::Mutable::Private::EImageFormat::Count];
static bool SbPrefixedMutableTextureFormatNameTableInitialized = false;

namespace
{

FName GetMutableFormatTextureFormatName(UE::Mutable::Private::EImageFormat MutableFormat)
{
	switch (MutableFormat)
	{
		case UE::Mutable::Private::EImageFormat::BC1                 : return FName(TEXT("DXT1"));
		case UE::Mutable::Private::EImageFormat::BC2                 : return FName(TEXT("DXT3"));
		case UE::Mutable::Private::EImageFormat::BC3                 : return FName(TEXT("DXT5"));
		case UE::Mutable::Private::EImageFormat::BC4                 : return FName(TEXT("BC4"));
		case UE::Mutable::Private::EImageFormat::BC5                 : return FName(TEXT("BC5"));
		case UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR    : return FName(TEXT("ASTC_RGBA_HQ"));
    	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR   : return FName(TEXT("ASTC_RGBA_HQ"));
    	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR     : return FName(TEXT("ASTC_RGB"));
    	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RGB_LDR    : return FName(TEXT("ASTC_RGBA"));   
    	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RGBA_LDR   : return FName(TEXT("ASTC_RGBA"));  
    	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RG_LDR     : return FName(TEXT("ASTC_NormalLA"));
    	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RGB_LDR  : return FName(TEXT("ASTC_RGBA"));
    	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RGBA_LDR : return FName(TEXT("ASTC_RGBA"));	 
    	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RG_LDR   : return FName(TEXT("ASTC_NormalRG"));
    	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RGB_LDR    : return FName(TEXT("ASTC_RGBA"));
    	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RGBA_LDR   : return FName(TEXT("ASTC_RGBA"));   
    	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RG_LDR     : return FName(TEXT("ASTC_NormalRG"));
    	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RGB_LDR  : return FName(TEXT("ASTC_RGBA"));
    	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RGBA_LDR : return FName(TEXT("ASTC_RGBA"));  
    	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR   : return FName(TEXT("ASTC_NormalRG"));

		default: return NAME_None;
	}
}

}

void PrepareUnrealCompression()
{
	check(IsInGameThread());

	if (!STextureFormatManager)
	{
		STextureFormatManager = &FModuleManager::LoadModuleChecked<ITextureFormatManagerModule>("TextureFormat");
		check(STextureFormatManager);
	}

	if (!SbPrefixedMutableTextureFormatNameTableInitialized)
	{
		const ITargetPlatformSettings* TargetPlatformSettings = GetTargetPlatformManagerRef().GetRunningTargetPlatform()->GetTargetPlatformSettings();
		check(TargetPlatformSettings);

		for (uint32 FormatEnumIndex = 0; FormatEnumIndex < (uint32)UE::Mutable::Private::EImageFormat::Count; ++FormatEnumIndex)
		{
			FName TextureFormatName = GetMutableFormatTextureFormatName(UE::Mutable::Private::EImageFormat(FormatEnumIndex));
			SPrefixedMutableTextureFormatNameTable[FormatEnumIndex] = TextureFormatName;
				
			// Based on ConditionalGetPrefixedFormat in Texture.cpp
			FString TextureCompressionFormat;
			bool bHasFormat = TargetPlatformSettings->GetConfigSystem()->GetString(TEXT("AlternateTextureCompression"), TEXT("TextureCompressionFormat"), TextureCompressionFormat, GEngineIni);
			bHasFormat = bHasFormat && !TextureCompressionFormat.IsEmpty();
			
			if (bHasFormat)
			{
				if (ITextureFormatModule* TextureFormatModule = FModuleManager::LoadModulePtr<ITextureFormatModule>(*TextureCompressionFormat))
				{
					if (ITextureFormat* TextureFormat = TextureFormatModule->GetTextureFormat())
					{
						FString FormatPrefix = TextureFormat->GetAlternateTextureFormatPrefix();
						check(!FormatPrefix.IsEmpty());
				
						FName NewFormatName(FormatPrefix + TextureFormatName.ToString());

						// check that prefixed name is one we support
						// only apply prefix if it is in list
						TArray<FName> SupportedFormats;
						TextureFormat->GetSupportedFormats(SupportedFormats);

						if (SupportedFormats.Contains(NewFormatName))
						{
							SPrefixedMutableTextureFormatNameTable[FormatEnumIndex] = NewFormatName;
						}
					}
				}
			}
		}

		SbPrefixedMutableTextureFormatNameTableInitialized = true;
	}
}

void FillBuildSettingsFromMutableFormat(FTextureBuildSettings& Settings, bool& bOutHasAlpha, UE::Mutable::Private::EImageFormat Format)
{
	Settings.MipGenSettings = TMGS_NoMipmaps;

	check(SbPrefixedMutableTextureFormatNameTableInitialized);
	Settings.TextureFormatName = SPrefixedMutableTextureFormatNameTable[(uint32)Format];
	Settings.BaseTextureFormatName = SPrefixedMutableTextureFormatNameTable[(uint32)Format];

	switch (Format)
	{
	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR:
		Settings.CompressionQuality = 4; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = true;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RGBA_LDR:
		Settings.CompressionQuality = 3; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = true;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RGBA_LDR:
		Settings.CompressionQuality = 2; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = true;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RGBA_LDR:
		Settings.CompressionQuality = 1; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = true;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RGBA_LDR:
		Settings.CompressionQuality = 0; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = true;
		break;

	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR:
		Settings.CompressionQuality = 4; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RGB_LDR:
		Settings.CompressionQuality = 3; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RGB_LDR:
		Settings.CompressionQuality = 2; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RGB_LDR:
		Settings.CompressionQuality = 1; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RGB_LDR:
		Settings.CompressionQuality = 0; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;

	case UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR:
		Settings.CompressionQuality = 4; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RG_LDR:
		Settings.CompressionQuality = 3; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RG_LDR:
		Settings.CompressionQuality = 2; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR:
		Settings.CompressionQuality = 1; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RG_LDR:
		Settings.CompressionQuality = 0; // See GetQualityFormat in TextureFormatASTC.cpp
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::BC1:
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::BC2:
		bOutHasAlpha = true;
		break;
	case UE::Mutable::Private::EImageFormat::BC3:
		bOutHasAlpha = true;
		break;
	case UE::Mutable::Private::EImageFormat::BC4:
		bOutHasAlpha = false;
		break;
	case UE::Mutable::Private::EImageFormat::BC5:
		bOutHasAlpha = true;
		break;
	default:
		Settings.TextureFormatName = NAME_None;
		bOutHasAlpha = false;
		break;
	}
}


void MutableToImageCore(const UE::Mutable::Private::FImage* InMutable, FImage& CoreImage, int32 LOD, bool bSwizzleRGBHack)
{
	MUTABLE_CPUPROFILER_SCOPE(MutableToImageCore);

	UE::Mutable::Private::TManagedPtr<UE::Mutable::Private::FImage> TempMutable;

	ERawImageFormat::Type CoreImageFormat;
	switch (InMutable->GetFormat())
	{
	case UE::Mutable::Private::EImageFormat::BGRA_UByte: 
		CoreImageFormat = ERawImageFormat::BGRA8; 
		break;
		
	//case UE::Mutable::Private::EImageFormat::L_UByte: CoreImageFormat = ERawImageFormat::G8; break;

	default:
	{
		// Unsupported format: force conversion
		UE::Mutable::Private::FImageOperator ImOp = UE::Mutable::Private::FImageOperator::GetDefault(UE::Mutable::Private::FImageOperator::FImagePixelFormatFunc());

		TempMutable = ImOp.ImagePixelFormat(4, InMutable, UE::Mutable::Private::EImageFormat::BGRA_UByte, LOD);
		InMutable = TempMutable.Get();

		if (bSwizzleRGBHack)
		{
			// Editor's ASTC compressor doesn't handle "number of channels". Blank out the unused channels to improve quality.
			int32 NumPixels = TempMutable->GetLODDataSize(0) / 4;
			uint8* BlueChannel = TempMutable->GetLODData(0);
			for (int32 Pixel=0;Pixel<NumPixels;++Pixel)
			{
				*BlueChannel = 0;
				BlueChannel += 4;
			}
		}

		// We are extracting one LOD, so always access LOD 0 of the resulting mutable image 
		LOD = 0;

		CoreImageFormat = ERawImageFormat::BGRA8;
		break;
	}

	}

	FIntVector2 MipSize = InMutable->CalculateMipSize(LOD);
	CoreImage.Init(MipSize.X, MipSize.Y, CoreImageFormat, EGammaSpace::Linear);
	FMemory::Memcpy(CoreImage.RawData.GetData(), InMutable->GetMipData(LOD), CoreImage.GetImageSizeBytes());
}


bool ImageCoreToMutable(const FCompressedImage2D& Compressed, UE::Mutable::Private::FImage* Mutable, int32 LOD)
{
	TArrayView<uint8> MutableView = Mutable->DataStorage.GetLOD(LOD);

	if (Compressed.RawData.Num() != MutableView.Num())
	{
		UE_LOGF(LogMutableCore, Error, "Buffer size mismatch when trying to convert image LOD %d, mutable size is %d and ue size is %lld. Mutable is %d x %d format %d and UE is %d x %d format %d.", 
			LOD, MutableView.Num(), Compressed.RawData.Num(), 
			Mutable->GetSizeX(), Mutable->GetSizeY(), EnumToUnderlyingType(Mutable->GetFormat()),
			Compressed.SizeX, Compressed.SizeY, EnumToUnderlyingType(Compressed.PixelFormat)
			);

		return false;
	}

	SIZE_T Bytes = FMath::Min(SIZE_T(MutableView.Num()),SIZE_T(Compressed.RawData.Num()));
	FMemory::Memcpy(MutableView.GetData(), Compressed.RawData.GetData(), Bytes);
	return true;
}


UE::Mutable::Private::EImageFormat UnrealToMutablePixelFormat(EPixelFormat PlatformFormat, bool bHasAlpha)
{
	switch (PlatformFormat)
	{
	case PF_ASTC_4x4: return bHasAlpha ? UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR : UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR;
	case PF_ASTC_6x6: return bHasAlpha ? UE::Mutable::Private::EImageFormat::ASTC_6x6_RGBA_LDR : UE::Mutable::Private::EImageFormat::ASTC_6x6_RGB_LDR;
	case PF_ASTC_8x8: return bHasAlpha ? UE::Mutable::Private::EImageFormat::ASTC_8x8_RGBA_LDR : UE::Mutable::Private::EImageFormat::ASTC_8x8_RGB_LDR;
	case PF_ASTC_10x10: return bHasAlpha ? UE::Mutable::Private::EImageFormat::ASTC_10x10_RGBA_LDR : UE::Mutable::Private::EImageFormat::ASTC_10x10_RGB_LDR;
	case PF_ASTC_12x12: return bHasAlpha ? UE::Mutable::Private::EImageFormat::ASTC_12x12_RGBA_LDR : UE::Mutable::Private::EImageFormat::ASTC_12x12_RGB_LDR;
	case PF_ASTC_4x4_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR;
	case PF_ASTC_6x6_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_6x6_RG_LDR;
	case PF_ASTC_8x8_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_8x8_RG_LDR;
	case PF_ASTC_10x10_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR;
	case PF_ASTC_12x12_NORM_RG: return UE::Mutable::Private::EImageFormat::ASTC_12x12_RG_LDR;
	case PF_DXT1: return UE::Mutable::Private::EImageFormat::BC1;
	case PF_DXT3: return UE::Mutable::Private::EImageFormat::BC2;
	case PF_DXT5: return UE::Mutable::Private::EImageFormat::BC3;
	case PF_BC4: return UE::Mutable::Private::EImageFormat::BC4;
	case PF_BC5: return UE::Mutable::Private::EImageFormat::BC5;
	case PF_G8: return UE::Mutable::Private::EImageFormat::L_UByte;
	case PF_L8: return UE::Mutable::Private::EImageFormat::L_UByte;
	case PF_A8: return UE::Mutable::Private::EImageFormat::L_UByte;
	case PF_R8G8B8A8: return UE::Mutable::Private::EImageFormat::RGBA_UByte;
	case PF_A8R8G8B8: return UE::Mutable::Private::EImageFormat::RGBA_UByte;
	case PF_B8G8R8A8: return UE::Mutable::Private::EImageFormat::BGRA_UByte;
	default:
		return UE::Mutable::Private::EImageFormat::None;
	}
}


UE::Mutable::Private::EImageFormat QualityAndPerformanceFix(UE::Mutable::Private::EImageFormat Format)
{
	switch (Format)
	{
	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RGB_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR; 
	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RGBA_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_8x8_RG_LDR:		return UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RGB_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR; 
	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RGBA_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_12x12_RG_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RGB_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR; 
	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RGBA_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_6x6_RG_LDR:		return UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RGB_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGB_LDR; 
	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RGBA_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RGBA_LDR;
	case UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR:	return UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR;

	// This is more of a performance fix.
	case UE::Mutable::Private::EImageFormat::BGRA_UByte:			return UE::Mutable::Private::EImageFormat::RGBA_UByte;

	default:
		break;
	}

	return Format;
}


void UnrealPixelFormatFunc(bool& bOutSuccess, int32 Quality, UE::Mutable::Private::FImage* Target, const UE::Mutable::Private::FImage* Source, int32 OnlyLOD)
{
	// If this fails, PrepareUnrealCompression wasn't called before.
	check(STextureFormatManager);

	bOutSuccess = true;

	FTextureBuildSettings Settings;
	bool bHasAlpha = false;
	FillBuildSettingsFromMutableFormat(Settings, bHasAlpha, Target->GetFormat());

	if (Settings.TextureFormatName == NAME_None)
	{
		// Unsupported format in the override: use standard mutable compression.
		bOutSuccess = false;
		return;
	}

	const ITextureFormat* TextureFormat = STextureFormatManager->FindTextureFormat(Settings.TextureFormatName);
	check(TextureFormat);

	int32 FirstLOD = 0;
	int32 LODCount = Source->GetLODCount();
	if (OnlyLOD >= 0)
	{
		FirstLOD = OnlyLOD;
		LODCount = 1;
	}

	// This seems to be necessary because of a probable double swizzling that happens during conversions.
	bool bSwizzleRGBHack =
		Target->GetFormat() == UE::Mutable::Private::EImageFormat::ASTC_4x4_RG_LDR ||
		Target->GetFormat() == UE::Mutable::Private::EImageFormat::ASTC_6x6_RG_LDR ||
		Target->GetFormat() == UE::Mutable::Private::EImageFormat::ASTC_8x8_RG_LDR ||
		Target->GetFormat() == UE::Mutable::Private::EImageFormat::ASTC_10x10_RG_LDR ||
		Target->GetFormat() == UE::Mutable::Private::EImageFormat::ASTC_12x12_RG_LDR;

	for (int32 LOD = FirstLOD; bOutSuccess && (LOD < LODCount); ++LOD)
	{
		FImage SourceUnreal;
		MutableToImageCore(Source, SourceUnreal, LOD, bSwizzleRGBHack);

		FCompressedImage2D CompressedUnreal;
		bOutSuccess = TextureFormat->CompressImage(SourceUnreal, Settings,
			FIntVector3(SourceUnreal.SizeX, SourceUnreal.SizeY, 1),
			0, 0, 1,
			FString(),
			bHasAlpha, CompressedUnreal);

		if (bOutSuccess)
		{
			bOutSuccess = ImageCoreToMutable(CompressedUnreal, Target, LOD);
		}
	}
}


void DebugImageDump(const UE::Mutable::Private::FImage* Image, const FString& FileName)
{
	int32 LOD = 0;

	if (!Image || !Image->GetLODData(LOD))
	{
		return;
	}

	struct FAstcHeader
	{
		uint8 magic[4];
		uint8 block_x;
		uint8 block_y;
		uint8 block_z;
		uint8 dim_x[3];
		uint8 dim_y[3];
		uint8 dim_z[3];
	};
	static_assert(sizeof(FAstcHeader)==16);

	FAstcHeader Header;
	FMemory::Memzero(Header);

	Header.magic[0] = 0x13;
	Header.magic[1] = 0xAB;
	Header.magic[2] = 0xA1;
	Header.magic[3] = 0x5C;

	int32 SizeX = Image->GetSize()[0];
	int32 SizeY = Image->GetSize()[1];
	Header.dim_x[0] = (SizeX >>  0) & 0xff;
	Header.dim_x[1] = (SizeX >>  8) & 0xff;
	Header.dim_x[2] = (SizeX >> 16) & 0xff;
	Header.dim_y[0] = (SizeY >>  0) & 0xff;
	Header.dim_y[1] = (SizeY >>  8) & 0xff;
	Header.dim_y[2] = (SizeY >> 16) & 0xff;
	Header.dim_z[0] = 1;

	Header.block_x = UE::Mutable::Private::GetImageFormatData(Image->GetFormat()).PixelsPerBlockX;
	Header.block_y = UE::Mutable::Private::GetImageFormatData(Image->GetFormat()).PixelsPerBlockY;
	Header.block_z = 1;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	IFileHandle* File = PlatformFile.OpenWrite(*FileName);
	if (File)
	{
		File->Write(reinterpret_cast<const uint8*>(&Header), sizeof(FAstcHeader));
		File->Write(Image->GetLODData(LOD),Image->GetLODDataSize(LOD));
		delete File;
	}
}

