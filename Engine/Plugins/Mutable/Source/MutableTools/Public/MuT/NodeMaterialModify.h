// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuT/NodeMaterial.h"
#include "MuT/NodeColor.h"
#include "MuT/NodeScalar.h"
#include "MuT/NodeImage.h"
#include "MuR/Material.h"
#include "MuR/Ptr.h"
#include "MuR/Types.h"

#define UE_API MUTABLETOOLS_API

namespace UE::Mutable::Private
{
	struct FImageParameterData
	{
		Ptr<NodeImage> ImageNode;
		int32 ImagePropertyIndex = 0;
		int8 LayoutIndex = 0;
		bool bIsPassthrough = false;
		uint16 BlockSizeX = 0;
		uint16 BlockSizeY = 0;
		
		bool operator==(const FImageParameterData& Other) const = default;
	};

	class NodeMaterialModify : public NodeMaterial
	{
	public:

		Ptr<NodeMaterial> MaterialSource;
		TMap<FParameterKey, FImageParameterData> ImageParameters;
		TMap<FParameterKey, Ptr<NodeColor>> ColorParameters;
		TMap<FParameterKey, Ptr<NodeScalar>> ScalarParameters;

	public:
		
		// Node Interface
		virtual const FNodeType* GetType() const override { return &StaticType; }
		static const FNodeType* GetStaticType() { return &StaticType; }

	protected:

		/** Forbidden. Manage with the Ptr<> template. */
		~NodeMaterialModify() {}

	private:

		static UE_API FNodeType StaticType;
	
	};
}

#undef UE_API