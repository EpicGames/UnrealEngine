// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	/// <summary>
	/// Module designed to serve as the home for all validation systems running in the engine. 
	/// </summary>
	public class MutableValidation : ModuleRules
	{
		public MutableValidation(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "MuV";

			DefaultBuildSettings = BuildSettingsVersion.Latest;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Settings", 
					"Engine", 
					"MutableRuntime",
					"RHI"
				});
			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"UnrealEd",
					
					"DataValidation",
					"CustomizableObject", 
					"CustomizableObjectEditor", 
					"MutableTools",
				}
			);
		}
	}
}