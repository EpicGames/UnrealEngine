// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/AssetValidator_CustomizableObjects.h"

#include "DataValidationModule.h"
#include "Editor.h"
#include "MuCOE/ICustomizableObjectEditorModulePrivate.h"
#include "MutableValidationSettings.h"
#include "AssetRegistry/AssetData.h"
#include "Delegates/DelegateSignatureImpl.inl"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOE/CompilationOptions.h"
#include "MuCOE/CompileRequest.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetValidator_CustomizableObjects)

#define LOCTEXT_NAMESPACE "CustomizableObjectsValidator"

UAssetValidator_CustomizableObjects::UAssetValidator_CustomizableObjects() : Super()
{
	bIsEnabled = true;
}

bool UAssetValidator_CustomizableObjects::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	// Use module settings to decide if it needs to run or not.
	if (const UMutableValidationSettings* ValidationSettings = GetDefault<UMutableValidationSettings>())
	{
		if (!ValidationSettings->bEnableDirectCOValidation)
		{
			return false;
		}
	}
	
	// Do not run if saving or running a commandlet (we do not want CIS failing due to our warnings and errors)
	if (InContext.GetValidationUsecase() == EDataValidationUsecase::Save || InContext.GetValidationUsecase() == EDataValidationUsecase::Commandlet)
	{
		return false;
	}

	return Cast<UCustomizableObject>(InAsset) != nullptr;
}


EDataValidationResult UAssetValidator_CustomizableObjects::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	check(InAsset);

	UCustomizableObject* CustomizableObjectToValidate = Cast<UCustomizableObject>(InAsset);
	check (CustomizableObjectToValidate);
	
	// Validate that CO and if it fails then mark it as failed. Do not stop until running the validation over all COs
	TArray<FText> CoValidationWarnings;
	TArray<FText> CoValidationErrors;
	const EDataValidationResult COValidationResult = IsCustomizableObjectValid(CustomizableObjectToValidate,CoValidationErrors,CoValidationWarnings);
	
	// Process the validation of the CO's output
	if (COValidationResult == EDataValidationResult::Invalid )
	{
		// Cache warning logs
		for	(const FText& WarningMessage : CoValidationWarnings)
		{
			AssetWarning(InAsset,WarningMessage);
		}
		
		// Cache error logs -> They will tag the asset validation as failed
		for (const FText& ErrorMessage : CoValidationErrors)
		{
			AssetFails(InAsset,ErrorMessage);
		}

		const FText ErrorMessage = FText::Format(LOCTEXT("CO_Validation_Failed", "Validation compilation of {0} CO failed."),  FText::FromString( CustomizableObjectToValidate->GetName()));
		AssetFails(InAsset,ErrorMessage);
	}
	else
	{
		AssetPasses(InAsset);
	}
		
	return GetValidationResult();
}


EDataValidationResult UAssetValidator_CustomizableObjects::IsCustomizableObjectValid(UCustomizableObject* InCustomizableObject, TArray<FText>& OutValidationErrors, TArray<FText>& OutValidationWarnings)
{
	EDataValidationResult Result = EDataValidationResult::NotValidated;

	UE_LOGF(LogMutable,Verbose, "Running data validation checks for %ls CO.",*InCustomizableObject->GetName());

	// Bind the post validation method to the post validation delegate if not bound already to be able to know when the validation
	// operation (for all assets) concludes
	if (!OnPostCOValidationHandle.IsValid())
	{
		OnPostCOValidationHandle = FEditorDelegates::OnPostAssetValidation.AddStatic(OnPostCOsValidation);	
	}

	// Find out which is the root for this CO (it may be itself but that is OK)
	UCustomizableObject* RootObject = ICustomizableObjectEditorModule::GetChecked().GetRootObject(InCustomizableObject);
	check (RootObject);
	
	// Check that the object to be compiled has not already been compiled
	if (AlreadyValidatedRootObjects.Contains(RootObject))
	{
		return Result;
	}
	
	// Root Object not yet tested -> Proceed with the testing
	
	// Collection of configurations to be tested with the located root object
	TArray<FCompilationOptions> CompilationOptionsToTest;

	// Current configuration
	CompilationOptionsToTest.Add(GetCompilationOptions(*InCustomizableObject));

	// Configuration with LOD bias applied
	// constexpr int32 MaxBias = 15;	
	//constexpr int32 MaxBias = 6;		// Reduced amount since we have this value in GenerateMutableSource for the max lod bias provided to mutable
	//for (int32 LodBias = 1; LodBias <= MaxBias; LodBias++)
	//{
	//	FCompilationOptions ModifiedCompilationOptions = InCustomizableObject->CompileOptions;
	//	ModifiedCompilationOptions.bForceLargeLODBias = true;	
	//	ModifiedCompilationOptions.DebugBias = LodBias;	
	//
	//	// Add one configuration object for each bias setting
	//	CompilationOptionsToTest.Add(ModifiedCompilationOptions);
	//}
	
	// Caches with all the data produced by the subsequent compilations of the root of this CO
	TArray<FText> CachedValidationErrors;
	TArray<FText> CachedValidationWarnings;
	
	// Map with all the possible compilation states. We use this so at the end we can know if any of those states was returned by any of the compilation runs
	TMap<ECompilationResultPrivate,bool>PossibleEndCompilationResults;
	PossibleEndCompilationResults.Add(ECompilationResultPrivate::Unknown, false);
	PossibleEndCompilationResults.Add(ECompilationResultPrivate::Success, false);
	PossibleEndCompilationResults.Add(ECompilationResultPrivate::Errors, false);
	PossibleEndCompilationResults.Add(ECompilationResultPrivate::Warnings, false);
	PossibleEndCompilationResults.Shrink();
	
	// Iterate over the compilation options that we want to test and perform the compilation
	for	(const FCompilationOptions& Options : CompilationOptionsToTest)
	{
		// Run Sync compilation -> Warning : Potentially long operation -------------
		TSharedRef<FCompilationRequest> CompileRequest = MakeShared<FCompilationRequest>(*RootObject);
		CompileRequest->Options = Options;
		CompileRequest->bAsync = false;
		ICustomizableObjectEditorModulePrivate::GetChecked().EnqueueCompileRequest(CompileRequest);
		// --------------------------------------------------------------------------
		
		// Get compilation errors and warnings
		const TArray<FText>& CompilationErrors = CompileRequest->Errors;
		const TArray<FText>& CompilationWarnings = CompileRequest->Warnings;
		
		// Cache the messages returned by the compiler
		for (const FText& FoundError : CompilationErrors)
		{
			// Add message if not already present
			if (!CachedValidationErrors.ContainsByPredicate([&FoundError](const FText& ArrayEntry)
				{ return FoundError.EqualTo(ArrayEntry);}))
			{
				CachedValidationErrors.Add(FoundError);
			}
		}
		for (const FText& FoundWarning : CompilationWarnings)
		{
			if (!CachedValidationWarnings.ContainsByPredicate([&FoundWarning](const FText& ArrayEntry)
				{ return FoundWarning.EqualTo(ArrayEntry);}))
			{
				CachedValidationWarnings.Add(FoundWarning);
			}
		}

		// Flag the array with end results to have the current output as true since it was produced by this execution
		bool* Value = PossibleEndCompilationResults.Find(CompileRequest->GetCompilationResult());
		check (Value);		// If this fails it may mean we are getting a compilation state we are not considering. 
		*Value = true;
	}
	
	// Cache root object to avoid processing it again when processing another CO related with the same root CO
	AlreadyValidatedRootObjects.Add(RootObject);
	
	// Wrapping up : Fill message output caches and determine if the compilation was successful or not
	
	// Provide the warning and log messages to the context object (so it can later notify the user using the UI)
	const FString ReferencedCOName = FString(TEXT("\"" +  InCustomizableObject->GetName() + "\""));
	for (const FText& ValidationError : CachedValidationErrors)
	{
		FText ComposedMessage = FText::Format(LOCTEXT("CO_Compilation_Error", "Customizable Object : {0} {1}"),  FText::FromString(ReferencedCOName),ValidationError );
		OutValidationErrors.Add(ComposedMessage);
	}
	for (const FText& ValidationWarning : CachedValidationWarnings)
	{
		FText ComposedMessage = FText::Format(LOCTEXT("CO_Compilation_nWarning", "Customizable Object : {0} {1}"),  FText::FromString(ReferencedCOName),ValidationWarning );
		OutValidationWarnings.Add(ComposedMessage);
	}
	
	// Return informed guess about what the validation state of this object should be

	// If it contains invalid results then notify about it too:
	// ECompilationResultPrivate::None would mean the resource is locked (and should not be)
	check (*PossibleEndCompilationResults.Find(ECompilationResultPrivate::Unknown) == false);
	
	// If one or more tests failed to ran then the result must be invalid
	if (*PossibleEndCompilationResults.Find(ECompilationResultPrivate::Errors) == true)
	{
		// Early CO compilation error (before starting mutable compilation) -> Output is invalid
		Result = EDataValidationResult::Invalid;
	}
	// All compilations completed successfully
	else 
	{
		// If a warning or error was found then this object failed the validation process
		Result = (CachedValidationWarnings.IsEmpty() && CachedValidationErrors.IsEmpty()) ? EDataValidationResult::Valid : EDataValidationResult::Invalid;
	}
	
	return Result;
}


void UAssetValidator_CustomizableObjects::OnPostCOsValidation()
{
	// Unbound this method from the validation end delegate
	check (OnPostCOValidationHandle.IsValid());
	FEditorDelegates::OnPostAssetValidation.Remove(OnPostCOValidationHandle);
	OnPostCOValidationHandle.Reset();

	// Clear collection with the already processed COs once the validation system has completed its operation
	AlreadyValidatedRootObjects.Empty();
}


#undef LOCTEXT_NAMESPACE

