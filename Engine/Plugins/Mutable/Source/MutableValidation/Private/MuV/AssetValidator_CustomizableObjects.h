// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor.h"
#include "EditorValidatorBase.h"
#include "AssetRegistry/AssetData.h"
#include "Delegates/DelegateSignatureImpl.inl"


#include "AssetValidator_CustomizableObjects.generated.h"

class FText;
class UObject;
class UCustomizableObject;

UCLASS()
class UAssetValidator_CustomizableObjects : public UEditorValidatorBase
{
	GENERATED_BODY()

public:
	UAssetValidator_CustomizableObjects();

	/** Checks the validity of the provided CustomizableObject by compiling and recalling the compilation status and the raised compilation warning and error messages.
	 * @param InCustomizableObject The mutable customizable object to test.
	 * @param OutValidationErrors A list of error messages produced during the provided customizable object's compilation.
	 * @param OutValidationWarnings A list of warning messages produced during the provided customizable object's compilation.
	 * @return Validation result for the provided object.
	 */
	static EDataValidationResult IsCustomizableObjectValid(UCustomizableObject* InCustomizableObject, TArray<FText>& OutValidationErrors, TArray<FText>& OutValidationWarnings);
	
protected:
	// UEditorValidatorBase
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
	// UEditorValidatorBase

private:
	
	/** Cached handle to be able later to remove the bound method from the FEditorDelegates::OnPostAssetValidation delegate */
	inline static FDelegateHandle OnPostCOValidationHandle;
	
	/** Collection with all root objects tested during this IsDataValidRun. Shared with all COs */
	inline static TSet<UCustomizableObject*> AlreadyValidatedRootObjects;
	
	/** Method invoked once the validation of all assets has been completed. */
	static void OnPostCOsValidation();
};
