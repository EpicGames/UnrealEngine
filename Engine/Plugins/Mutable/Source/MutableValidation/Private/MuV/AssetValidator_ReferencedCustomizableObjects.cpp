// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/AssetValidator_ReferencedCustomizableObjects.h"

#include "DataValidationModule.h"
#include "MutableValidationSettings.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/LoadUtils.h"
#include "MuV/AssetValidator_CustomizableObjects.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetValidator_ReferencedCustomizableObjects)

#define LOCTEXT_NAMESPACE "ReferencedCustomizableObjectsValidator"


UAssetValidator_ReferencedCustomizableObjects::UAssetValidator_ReferencedCustomizableObjects() : Super()
{
	bIsEnabled = true;
}

bool UAssetValidator_ReferencedCustomizableObjects::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	// Use module settings to decide if it needs to run or not.
	if (const UMutableValidationSettings* ValidationSettings = GetDefault<UMutableValidationSettings>())
	{
		if (!ValidationSettings->bEnableIndirectCOValidation)
		{
			return false;
		}
	}

	// Do not run if saving or running a commandlet (we do not want CIS failing due to our warnings and errors)
	if (InContext.GetValidationUsecase() == EDataValidationUsecase::Save || InContext.GetValidationUsecase() == EDataValidationUsecase::Commandlet)
	{
		return false;
	}

	return (InAsset ?
		InAsset->IsA(UMaterial::StaticClass()) ||
		InAsset->IsA(UTexture::StaticClass()) ||
		InAsset->IsA(USkeletalMesh::StaticClass()) || 
		InAsset->IsA(UStaticMesh::StaticClass()) : false) ;
}


EDataValidationResult UAssetValidator_ReferencedCustomizableObjects::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	check(InAsset);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.GetRegistry();

	// Locate all referencers to the provided asset.
	const TSet<FName> FoundReferencers = GetAllAssetReferencers(InAsset,AssetRegistry);

	// Grab all Customizable Objects
	const TSet<UCustomizableObject*> FoundCustomizableObjects = FindCustomizableObjects(FoundReferencers, AssetRegistry);

	// Validate all Customizable Objects we have found
	ValidateCustomizableObjects(InAsset, FoundCustomizableObjects);
	
	// Compute InAsset validation status
	if (GetValidationResult() != EDataValidationResult::Invalid)
	{
		AssetPasses(InAsset);
	}
		
	return GetValidationResult();
}


TSet<FName> UAssetValidator_ReferencedCustomizableObjects::GetAllAssetReferencers(const UObject* InAsset, const IAssetRegistry& InAssetRegistry) const
{
	TSet<FName> FoundReferencers;
	
	TArray<FName> PackagesToProcess;
	PackagesToProcess.Add(InAsset->GetOutermost()->GetFName());

	do
	{
		TArray<FName> NextPackagesToProcess;
		for (FName PackageToProcess : PackagesToProcess)
		{
			TArray<FName> Referencers;
			InAssetRegistry.GetReferencers(PackageToProcess, Referencers, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::NoRequirements);
			for (FName Referencer : Referencers)
			{
				// If referencer not already found then add it
				if (!FoundReferencers.Contains(Referencer))
				{
					// Cache the referencer so we can later check for COs
					FoundReferencers.Add(Referencer);
						
					NextPackagesToProcess.Add(Referencer);
				}
			}
		}
		PackagesToProcess = MoveTemp(NextPackagesToProcess);
			
	} while (PackagesToProcess.Num() > 0);

	FoundReferencers.Shrink();
	return FoundReferencers;
}


TSet< UCustomizableObject*> UAssetValidator_ReferencedCustomizableObjects::FindCustomizableObjects(
	const TSet<FName>& InPackagesToCheck,const IAssetRegistry& InAssetRegistry) const
{
	TSet<UCustomizableObject*> FoundCustomizableObjects;
	
	for (const FName& ReferencerPackage : InPackagesToCheck)
	{
		TArray<FAssetData> PackageAssets;
        InAssetRegistry.GetAssetsByPackageName(ReferencerPackage, PackageAssets, true);
        for (const FAssetData& ReferencerAssetData : PackageAssets)
        {
        	// We have found a referenced CustomizableObject
            if (ReferencerAssetData.GetClass() == UCustomizableObject::StaticClass())
            {
				UObject* ReferencedAsset = UE::Mutable::Private::LoadObject(ReferencerAssetData);
            	if (ReferencedAsset)
            	{
            		UCustomizableObject* CastedCustomizableObject = Cast<UCustomizableObject>(ReferencedAsset);
            		check(CastedCustomizableObject);
            	
            		FoundCustomizableObjects.FindOrAdd(CastedCustomizableObject);
            	}
            }
        }
	}

	FoundCustomizableObjects.Shrink();
	return FoundCustomizableObjects;
}


void UAssetValidator_ReferencedCustomizableObjects::ValidateCustomizableObjects(UObject* InAsset, const TSet<UCustomizableObject*>& InCustomizableObjectsToValidate)
{
	for (UCustomizableObject* CustomizableObjectToValidate : InCustomizableObjectsToValidate)
	{
		// Validate that CO and if it fails then mark it as failed. Do not stop until running the validation over all COs
		TArray<FText> CoValidationWarnings;
		TArray<FText> CoValidationErrors;
		const EDataValidationResult COValidationResult = UAssetValidator_CustomizableObjects::IsCustomizableObjectValid(CustomizableObjectToValidate,CoValidationErrors,CoValidationWarnings);
		
		// Process the validation of the CO's output
		if (COValidationResult == EDataValidationResult::Invalid )
		{
			// Cache warning logs
			for	(const FText& WarningMessage : CoValidationWarnings)
			{
				AssetWarning(InAsset,WarningMessage);
			}
		
			// Cache error logs
			for (const FText& ErrorMessage : CoValidationErrors)
			{
				AssetFails(InAsset,ErrorMessage);
			}
			
			// If we say it failed the asset will be marked as bad (containing bad data) and the validator will mark the overall result as failed
			const FText ErrorMessage = FText::Format(LOCTEXT("RelatedToCustomizableObjectValidator", "The referenced ""\"{0}""\" Mutable Customizable Object is invalid."),  FText::FromString(CustomizableObjectToValidate->GetPathName()));
			AssetFails(InAsset,ErrorMessage);
		}
	}
}


#undef LOCTEXT_NAMESPACE
