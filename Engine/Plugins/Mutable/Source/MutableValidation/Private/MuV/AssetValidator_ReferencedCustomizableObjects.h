// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "AssetValidator_ReferencedCustomizableObjects.generated.h"

class FText;
class FName;
class UObject;
class UCustomizableObject;

UCLASS()
class UAssetValidator_ReferencedCustomizableObjects : public UEditorValidatorBase
{
	GENERATED_BODY()

public:
	UAssetValidator_ReferencedCustomizableObjects();
	
protected:

	// UEditorValidatorBase
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
	// UEditorValidatorBase

private:

	/** Get a set of referencer objects to a provided UObject.
	 * @param InAsset A Pointer to the asset to get all referencers of.
	 * @param InAssetRegistry AssetRegistry object to be used to perform the referencers search.
	 * @return A set with all the referencing packages.
	 */
	TSet<FName> GetAllAssetReferencers(const UObject* InAsset,const IAssetRegistry& InAssetRegistry) const;

	/** Returns a set of customizable objects from the provided collection of packages.
	 * @param InPackagesToCheck List of packages to scan for CustomizableObjects.
	 * @param InAssetRegistry AssetRegistry object used to perform the asset search.
	 * @return A collection of unique Customizable Objets found in the provided packages.
	 */
	TSet<UCustomizableObject*> FindCustomizableObjects(const TSet<FName>& InPackagesToCheck,const IAssetRegistry& InAssetRegistry) const;

	/** Validates all the customizable objects provided and sets the validator status accordingly. It will not make the validator fail.
	 * @param InAsset Input asset provided to the validator.
	 * @param InCustomizableObjectsToValidate Customizable Objects we want to validate with IsDataValid()
	 * @param InValidationErrors List to fill with the warnings and errors generated during the validation of the Customizable objects
	 */
	void ValidateCustomizableObjects(UObject* InAsset, const TSet<UCustomizableObject*>& InCustomizableObjectsToValidate);

};
