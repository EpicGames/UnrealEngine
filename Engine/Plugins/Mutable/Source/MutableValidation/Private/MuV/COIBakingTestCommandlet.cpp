// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/COIBakingTestCommandlet.h"
#include "ScopedLogSection.h"
#include "ShaderCompiler.h"
#include "ValidationUtils.h"
#include "HAL/FileManager.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCOE/CustomizableObjectInstanceBakingUtils.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/LoadUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(COIBakingTestCommandlet)

/** Flag that indicates if the CO is being compiled or not. */
bool bIsCustomizableObjectBeingCompiled = false;

/** Did the CO Compilation finish with a successful status? */
bool bWasCustomizableObjectCompilationSuccessful = false;

/** Flag useful to know if we are currently updating an instance or not */
bool bIsInstanceBeingUpdated = false;

/** Did the instance update finish with a successful status? */
bool bWasInstanceUpdateSuccessful = false;


void OnCustomizableObjectCompilationEnd(const FCompileCallbackParams& CompilationParameters)
{
	UE_LOGF(LogMutable,Display,
		"The Customizable Object finished the compilation with state : "
		"RequestFailed : %hhd , bWarnings : %hhd , bErrors : %hhd , bSkipped : %hhd , bCompiled : %hhd.",
		CompilationParameters.bRequestFailed,
		CompilationParameters.bWarnings,
		CompilationParameters.bErrors,
		CompilationParameters.bSkipped,
		CompilationParameters.bCompiled);

	// You can ignore this check if running this locally while trying to debug stuff.
	// It is possible that you already have the CO compiled with the image compression settings required
	check(!CompilationParameters.bSkipped);
	
	bWasCustomizableObjectCompilationSuccessful = !(CompilationParameters.bRequestFailed || CompilationParameters.bErrors);

	ensure(bIsCustomizableObjectBeingCompiled);
	bIsCustomizableObjectBeingCompiled = false;
}


void OnInstanceUpdateEnd(const FUpdateContext& Result)
{
	const EUpdateResult InstanceUpdateResult = Result.UpdateResult;
	UE_LOGF(LogMutable,Display, "Instance finished update with state : %ls.", *UEnum::GetValueAsString(InstanceUpdateResult));
	bWasInstanceUpdateSuccessful = UCustomizableObjectSystem::IsUpdateResultValid(InstanceUpdateResult);
	
	// Clear update flag so we can exit the update while loop
	ensure(bIsInstanceBeingUpdated);
	bIsInstanceBeingUpdated = false;
}


int32 UCOIBakingTestCommandlet::Main(const FString& Params)
{
	// Ensure we have set the mutable system to the benchmarking mode and that we are reporting benchmarking data
	// FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(true);					// We are not caching benchmarking data from the baking tests
	// UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(true);		// try to be as close as the end user in terms of settings
	
	// Ensure we do not show any OK dialog since we are not an user that can interact with them
	GIsRunningUnattendedScript = true;
	
	// Look for the COI to be baked and load it
	{
		FString CustomizableObjectInstanceAssetPath = "";
		if (!FParse::Value(*Params, TEXT("CustomizableObjectInstance="), CustomizableObjectInstanceAssetPath))
		{
			UE_LOGF(LogMutable,Error, "Failed to parse Customizable Object Instance package name from provided argument : %ls",*Params)
			return 1;
		}
	
		// Load the resource
		UObject* FoundObject = UE::Mutable::Private::LoadObject(FSoftObjectPath(CustomizableObjectInstanceAssetPath));
		if (!FoundObject)
		{
			UE_LOGF(LogMutable,Error, "Failed to retrieve UObject from path %ls",*CustomizableObjectInstanceAssetPath);
			return 1;
		}
	
		// Get the CustomizableObjectInstance.
		TargetInstance = Cast<UCustomizableObjectInstance>(FoundObject);
		if (!TargetInstance)
		{
			UE_LOGF(LogMutable,Error, "Failed to cast found UObject to UCustomizableObjectInstance.");
			return 1;
		}

		UE_LOGF(LogMutable,Display, "Successfully loaded %ls Customizable Object Instance!", *TargetInstance->GetName());
	}

	// Perform a blocking search to ensure all assets used by mutable are reachable using the AssetRegistry
	PrepareAssetRegistry();

	// Make sure there is nothing else that the engine needs to do before starting our test
	Wait(60);	// todo: UE-304050 Remove this wait as it may no longer be required due to us calling for GShaderCompilingManager->FinishAllCompilation()

	// Block until async shader compiling is finished before we try to use the shaders for exporting
	// The code is structured to only block once for all materials, so that shader compiling is able to utilize many cores
	if (GShaderCompilingManager && GShaderCompilingManager->GetNumRemainingJobs() > 0)
	{
		GShaderCompilingManager->FinishAllCompilation();
	}
	
	LogGlobalSettings();
	
	// Compile it's CO (using current config)
	UCustomizableObject* InstanceCustomizableObject = TargetInstance->GetCustomizableObject();
	if (!InstanceCustomizableObject)
	{
		UE_LOGF(LogMutable,Error, "The instance %ls does not have a CO to compile : Exiting commandlet.", *TargetInstance->GetName());
		return 1;
	}
	
	// Compile the Customizable object of the Instance
	{
		const FScopedLogSection CompilationSection (EMutableLogSection::Compilation);
		
		FCompileNativeDelegate CompilationEndDelegate;
		CompilationEndDelegate.BindStatic(&OnCustomizableObjectCompilationEnd);

		bIsCustomizableObjectBeingCompiled = true;

		// To prevent the warnings related to partial compilation from appearing in CIS
		constexpr bool bPerformPartialCompilation = false;

		UE_LOGF(LogMutable, Display, "Scheduling Customizable Object compilation.");
		ScheduleCOCompilationForBaking(*TargetInstance, CompilationEndDelegate, bPerformPartialCompilation);
		
		// Exit the wait process once the compilation finishes
		while (bIsCustomizableObjectBeingCompiled) 
		{ 
			// Tick the engine
			CommandletHelpers::TickEngine();

			// Stop if exit was requested
			if (IsEngineExitRequested())
			{
				break;
			}
		}
		
		if (!bWasCustomizableObjectCompilationSuccessful)
		{
			UE_LOGF(LogMutable, Error, "Failed to successfully compile the target COIs CO. Exiting commandlet.");
			return 1;
		}
	}

	
	// Update the instance
	{
		const FScopedLogSection UpdateSection (EMutableLogSection::Update);
		
		// If this fail something is very wrong
		check(TargetInstance);

		// Instance update delegate
		FInstanceUpdateNativeDelegate InstanceUpdateDelegate;
		InstanceUpdateDelegate.AddStatic(&OnInstanceUpdateEnd);
		
		bIsInstanceBeingUpdated = true;
		UE_LOGF(LogMutable, Display, "Scheduling instance update.");
		ScheduleInstanceUpdateForBaking(*TargetInstance, InstanceUpdateDelegate);
		
		// Now tick the engine so the instance gets updated while running in the commandlet context
		while (bIsInstanceBeingUpdated) 
		{ 
			// Tick the engine
			CommandletHelpers::TickEngine();

			// Stop if exit was requested
			if (IsEngineExitRequested())
			{
				break;
			}
		}
		
		// Check the end status of the instance update
		if (!bWasInstanceUpdateSuccessful)
		{
			UE_LOGF(LogMutable, Error, "Failed to successfully update the target COI. Exiting commandlet.");
			return 1;
		}
	}

	// Bake the instance
	bool bWasBakingSuccessful = false;
	{
		const FScopedLogSection InstanceBakeSection (EMutableLogSection::Bake);
		
		const FString BakedResourcesFileName = "MuBakedInstances";
		
		// If this fail something is very wrong
		check(TargetInstance);
		const FString InstanceName = TargetInstance.GetName();
		
		// Create the actual directory in the filesystem of the host machine
		const FString GlobalBakingDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine( FPaths::ProjectContentDir(), BakedResourcesFileName, InstanceName));

		// This will only happen if we did make a partial run before and therefore the directory was not cleansed.
		// Delete it then but notify the user since this may mean that we have duplicated COIs
		if (FPaths::DirectoryExists(GlobalBakingDirectory))
		{
			UE_LOGF(LogMutable,Warning, "The directory with path  \" %ls \" does already exist. This may be produced by an incomplete execution of a previous test. Clearing it out before continuing...", *GlobalBakingDirectory);
			if (!IFileManager::Get().DeleteDirectory(*GlobalBakingDirectory,false,true))
			{
				UE_LOGF(LogMutable,Error, "Failed to delete the baking directory at path \" %ls \".", *GlobalBakingDirectory);
				return 1;
			}
		}
		
		// Compute the local path to the generated directory where to save the baked data
		const FString LocalBakingDirectory = FPaths::Combine("/","Game", BakedResourcesFileName, InstanceName);
		
		// Create a new directory where to save the bake itself
		if (IFileManager::Get().MakeDirectory(*GlobalBakingDirectory, true))
		{
			UE_LOGF(LogMutable,Display, "Starting Instance Baking operation.");
			{
				FBakingConfiguration Configuration;
				Configuration.OutputFilesBaseName = FString::Printf(TEXT("%s_Bake"), *TargetInstance->GetName());
				Configuration.OutputPath = LocalBakingDirectory;
				Configuration.bExportAllResourcesOnBake = true;
				Configuration.bGenerateConstantMaterialInstancesOnBake = true;

				TMap<UPackage*,const FResourceBakingData> SavedPackages;
				bWasBakingSuccessful = BakeCustomizableObjectInstance(
					*TargetInstance,
					Configuration,
					true,
					SavedPackages);

				// Report the baked resources
				
				const UEnum* Enum = StaticEnum<EPackageSaveResolutionType>();
				check(Enum);
				
				for (const TPair<UPackage*,const FResourceBakingData>& SavedPackagePair : SavedPackages)
				{
					const UPackage* SavedPackage = SavedPackagePair.Key;
					if (ensureMsgf(SavedPackage, TEXT("One entry of the \"SavedPackages\" map is a null pointer. All entries are expected to point to non null objects.")))
					{
						const FString SaveTypeString = Enum->GetNameByIndex(static_cast<uint8>(SavedPackagePair.Value.SaveResolutionType)).ToString();
						UE_LOGF(LogMutable, Display, "Stored package \"%ls\" with \"%ls\" as save resolution type.", *SavedPackage->GetName(), *SaveTypeString)
					}
				}
			}
			
			// Delete the target directory where we did save the baked instance
			if (!IFileManager::Get().DeleteDirectory(*GlobalBakingDirectory,true,true))
			{
				UE_LOGF(LogMutable,Error, "Failed to delete the baking directory at path \" %ls \".", *GlobalBakingDirectory);
				return 1;
			}
		}
		else
		{
			UE_LOGF(LogMutable,Error, "Failed to create baking directory at path \" %ls \".", *GlobalBakingDirectory);
			return 1;
		}
	}
	
	if (bWasBakingSuccessful)
	{
		UE_LOGF(LogMutable, Display, "Instance Baking operation has been completed successfully.");
		return 0;
	}
	else
	{
		UE_LOGF(LogMutable, Display, "Instance Baking operation has been completed with errors.");
		return 1;
	}
}


