// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"
#include "MuV/CustomizableObjectCompilationUtility.h"

#include "COIBakingTestCommandlet.generated.h"

#define UE_API MUTABLEVALIDATION_API

class UCustomizableObjectInstance;
struct FUpdateContext;

/**
 * Commandlet designed to test the baking of instances.
 * It expects an instance UAsset being provided to it so it can later compile it's CO, update the instance itself, and then bake it.
 * EX : -Run=COIBakingTest -EnablePlugins=MutableTesting -CustomizableObjectInstance="/MutableTesting/Cyborg_Character/Character_Inst" -AllowCommandletRendering
 */
UCLASS(MinimalAPI)
class UCOIBakingTestCommandlet : public UCommandlet
{
	GENERATED_BODY()
	
public:
	UE_API virtual int32 Main(const FString& Params) override;
	
private:
	
	/** Instance targeted for baking */
	UPROPERTY()
	TObjectPtr<UCustomizableObjectInstance> TargetInstance = nullptr;
};

#undef UE_API
