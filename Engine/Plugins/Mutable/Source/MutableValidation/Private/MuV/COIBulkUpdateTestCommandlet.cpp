// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/COIBulkUpdateTestCommandlet.h"

#include "CustomizableObjectCompilationUtility.h"
#include "CustomizableObjectInstanceUpdateUtility.h"
#include "ShaderCompiler.h"
#include "ValidationUtils.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/LoadUtils.h"
#include "MuCOE/CustomizableObjectBenchmarkingUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(COIBulkUpdateTestCommandlet)


int32 UCOIBulkUpdateTestCommandlet::Main(const FString& Params)
{
	// Ensure we have set the mutable system to the benchmarking mode and that we are reporting benchmarking data
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(true);
	UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(true);
	
	// Ensure we do not show any OK dialog since we are not an user that can interact with them
	GIsRunningUnattendedScript = true;

	// Get the path where to look for the Customizable Object Instances we want to validate
	FName InstancesPackagePath;
	{
		if (!FParse::Value(*Params, TEXT("InstancesPackagePath="), InstancesPackagePath))
		{
			UE_LOGF(LogMutable,Error, "Failed to parse path where to find the Customizable Object Instances to update : %ls",*Params)
			return 1;
		}

		if (InstancesPackagePath.IsNone())
		{
			UE_LOGF(LogMutable,Error, "The path to scan can not be empty")
			return 1;
		}
	}

	UE_LOGF(LogMutable, Log, "(string) instance_search_path : %ls ", *InstancesPackagePath.ToString());
	
	// Load the asset registry system so we can proceed without issues
	PrepareAssetRegistry();

	LogGlobalSettings();
	
	// Cache all UAssets (find a way to not have them in memory yet, not until we need them)
	const TArray<FAssetData> FoundAssetData = FindAllAssetsAtPath(InstancesPackagePath, UCustomizableObjectInstance::StaticClass());
	
	// Early exit if no instances could be found for testing.
	if (FoundAssetData.IsEmpty())
	{
		UE_LOGF(LogMutable,Error, "Aborting Bulk Instance Update Test: No assets could be found at  the provided package path");
		return 1;
	}

	// Get all the CO's that need compilation before proceeding
	TMap<UCustomizableObject*, TArray<TStrongObjectPtr<UCustomizableObjectInstance>>>  MutableResources;
	for (const FAssetData& Data : FoundAssetData)
	{
		UObject* LoadedAsset =  UE::Mutable::Private::LoadObject(Data);
		check (LoadedAsset);
		UCustomizableObjectInstance* LoadedInstance =  Cast<UCustomizableObjectInstance>(LoadedAsset);
		check (LoadedInstance)

		// Get the COI CO and cache it for later compilation
		UCustomizableObject* InstanceCO = LoadedInstance->GetCustomizableObject();
		if (!InstanceCO)
		{
			UE_LOGF(LogMutable,Error, "The instance %ls does not have a CO. This instance will not get tested.", *LoadedInstance->GetName());
			continue;
		}
		
		// Add/update an entry with the new instance for a given CO. 
		TArray<TStrongObjectPtr<UCustomizableObjectInstance>>& Entry = MutableResources.FindOrAdd(InstanceCO);
		Entry.Emplace(LoadedInstance);
	}

	// At this point it is safe to assume that all keys are valid COs and all instances are also valid.
	
	// Report the objects found (COs and COIs) -------------------------------
	
	const uint32 TotalAmountOfCustomizableObjects = MutableResources.Num();
	UE_LOGF(LogMutable,Display, "Customizable Objects to compile : %u", TotalAmountOfCustomizableObjects);
	UE_LOGF(LogMutable, Log, "(int) customizable_objects_count : %u ", TotalAmountOfCustomizableObjects);

	const uint32 TotalAmountOfInstances = FoundAssetData.Num();
	UE_LOGF(LogMutable,Display, "Customizable Object Instances to update : %u", TotalAmountOfInstances);
	UE_LOGF(LogMutable, Log, "(int) customizable_object_instances_count : %u ", TotalAmountOfInstances);
	
	// Report the amount of instances for each of the COs found as parents of the instances in the target path
	for (TTuple<UCustomizableObject*, TArray<TStrongObjectPtr<UCustomizableObjectInstance>>>& MutableResourceTuple : MutableResources)
	{
		const UCustomizableObject* CustomizableObjectToCompile = MutableResourceTuple.Key;
		check (CustomizableObjectToCompile);
		const FString CustomizableObjectName = CustomizableObjectToCompile->GetName();
		
		const uint32 COInstancesCount = MutableResourceTuple.Value.Num();
		UE_LOGF(LogMutable,Display, "The CO \"%ls\" has in total \"%u\" instances.", *CustomizableObjectName, COInstancesCount );
		UE_LOGF(LogMutable, Log, "(int) %ls_instance_count : %u ", *CustomizableObjectName, COInstancesCount);
		
		// print the name of the instances+
		uint32 InstanceIndex = 0;
		for (const TStrongObjectPtr<UCustomizableObjectInstance>& Instance : MutableResourceTuple.Value)
		{
			check(Instance);
			const FString CustomizableObjectInstanceName = Instance->GetName();
			UE_LOGF(LogMutable, Log, "%u : \"%ls\"", InstanceIndex++, *CustomizableObjectInstanceName);
		}
	}
	
	// ------ Execution of the actual mutable operations ------ 
	
	// Make sure there is nothing else that the engine needs to do before starting our test
	Wait(60);	// todo: UE-304050 Remove this wait as it may no longer be required due to us calling for GShaderCompilingManager->FinishAllCompilation()
	
	// Block until async shader compiling is finished before we try to use the shaders for exporting
	// The code is structured to only block once for all materials, so that shader compiling is able to utilize many cores
	if (GShaderCompilingManager && GShaderCompilingManager->GetNumRemainingJobs() > 0)
	{
		GShaderCompilingManager->FinishAllCompilation();
	}

	// Cache the target compilation platform so we can override the compilation configs of the target COs
	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	const ITargetPlatform* TargetCompilationPlatform = TPM.GetRunningTargetPlatform();
	
	TSharedRef<FCustomizableObjectCompilationUtility> CompilationUtility = MakeShared<FCustomizableObjectCompilationUtility>();
	TSharedRef<FCustomizableObjectInstanceUpdateUtility> InstanceUpdatingUtility = MakeShared<FCustomizableObjectInstanceUpdateUtility>();
	
	// Compile all found COs one by one
	uint32 CurrentInstanceIndex = 1;
	
	for ( TMap<UCustomizableObject*, TArray<TStrongObjectPtr<UCustomizableObjectInstance>>>::TIterator ResourcesIterator = MutableResources.CreateIterator(); ResourcesIterator; ++ResourcesIterator)
	{
		UCustomizableObject*& CustomizableObject = ResourcesIterator->Key;
		check (CustomizableObject);

		const FString CustomizableObjectName = CustomizableObject->GetName();
		
		// Set the compilation platform based on what the system is currently running on
		FCompilationOptions CompilationOptions = GetCompilationOptionsForBenchmarking(*CustomizableObject);
		CompilationOptions.TargetPlatform = TargetCompilationPlatform;
		
		// Compile the current CO object
		if (!CompilationUtility->CompileCustomizableObject(*CustomizableObject, false, &CompilationOptions))	// Do not log mutable data since mongoDB will not be able to handle it correctly 
		{
			UE_LOGF(LogMutable, Error, "The CO %ls could not be compiled successfully. Skipping the update of all COIs that use it.", *CustomizableObjectName )
			continue;
		}

		// Now that CO has been compiled proceed with the update of the instances that use it
		const uint32 CustomizableObjectInstanceCount = ResourcesIterator->Value.Num();
		UE_LOGF(LogMutable, Display, "Starting update of the \"%u\" instances with CO : \"%ls\".", CustomizableObjectInstanceCount, *CustomizableObjectName);

		// Iterate over all the COIs of the CO and update them
		TArray<TStrongObjectPtr<UCustomizableObjectInstance>>& Instances = ResourcesIterator->Value;
		while (!Instances.IsEmpty())
		{
			TStrongObjectPtr<UCustomizableObjectInstance> Instance = Instances[0];
			check(Instance);
			
			Instances.RemoveAt(0);
			UE_LOGF(LogMutable, Display, "\t( %u / %u ) Processing instance : \"%ls\" .",CurrentInstanceIndex++, TotalAmountOfInstances ,*Instance->GetName());

			CollectGarbage(RF_NoFlags, true);
			
			// Update each one of the instances and notify if the update failed in any manner
			InstanceUpdatingUtility->UpdateInstance(*Instance);
			
			// Remove standalone flag from the instance so we can GC it while keeping other standalone objects
			Instance->ClearFlags(EObjectFlags::RF_Standalone);
		}

		ResourcesIterator.RemoveCurrent();
		CollectGarbage(RF_NoFlags, true);
	}

	UE_LOGF(LogMutable,Display, "Mutable commandlet finished.");
	return 0;
}
