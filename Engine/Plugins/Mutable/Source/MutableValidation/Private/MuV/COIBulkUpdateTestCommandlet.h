// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Commandlets/Commandlet.h"
#include "MuV/CustomizableObjectCompilationUtility.h"
#include "MuV/CustomizableObjectInstanceUpdateUtility.h"

#include "COIBulkUpdateTestCommandlet.generated.h"

/**
 * Testing commandlet designed to allow the mass update of instances in a given project path.
 * It will run in sequence where each of the instances will get updated after their CO gets compiled.
 *
 * Example commandline that updates all instances found in the plugin content directory (and all subdirectories):
 * -run=COIBulkUpdateTest -EnablePlugins=MutableTesting -InstancesPackagePath="/MutableTesting" -AllowCommandletRendering
 */
UCLASS()
class UCOIBulkUpdateTestCommandlet : public UCommandlet 
{
	GENERATED_BODY()

public:
	virtual int32 Main(const FString& Params) override;
};
