// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/COMultiPluginBulkValidationCommandlet.h"

#include "ScopedLogSection.h"
#include "ShaderCompiler.h"
#include "ValidationUtils.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCO/LoadUtils.h"
#include "MuCO/LogBenchmarkUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(COMultiPluginBulkValidationCommandlet)

int32 UCOMultiPluginBulkValidationCommandlet::Main(const FString& Params)
{
	LLM_SCOPE_BYNAME(TEXT("COMultiPluginBulkValidationCommandlet"));

	// Prepare the environment for the testing -------------------------------------------------------------------------------------------------------
	
	// Ensure we have set the mutable system to the benchmarking mode and that we are reporting benchmarking data
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(true);
	UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(true);
	
	// Ensure we do not show any OK dialog since we are not a user that can interact with them
	GIsRunningUnattendedScript = true;
	
	// Parse the arguments provided ------------------------------------------------------------------------------------------------------------------
	
	// Parse the value for the platform to be used for the compilation of the COs
	const ITargetPlatform* TargetCompilationPlatform = GetCompilationPlatform(Params);
	if (!TargetCompilationPlatform)
	{
		return 1;
	}

	// Get the amount of instances to generate if parameter was provided (it will get multiplied by the amount of states later so this is a minimun value)
	const uint32 InstancesToGenerate = GetTargetAmountOfInstances(Params);

	// Work only for the root Customizable Objects found
	const bool bOnlyTestRootObjects = GetIfOnlyRootCOsShouldBeUsed(Params);
	
	// Find the assets to be tested based on the plugin names provided -------------------------------------------------------------------------------

	TArray<FName> PluginsToScanForCOs;
	{
		FString RawCollectionOfPluginNames;
		if (FParse::Value(*Params, TEXT("PluginsToScanForCOs="), RawCollectionOfPluginNames, false))
		{
			TArray<FString> SeparatedPluginNames;
			if (RawCollectionOfPluginNames.ParseIntoArray(SeparatedPluginNames, TEXT(","), true))
			{
				for (const FString& PluginNameString : SeparatedPluginNames)
				{
					const FName PluginName = FName(PluginNameString);
					
					if (PluginName.IsNone())
					{
						UE_LOGF(LogMutable, Warning, "An empty PluginName was provided. Skipping.");
						continue;
					}
					
					PluginsToScanForCOs.Add(PluginName);
				}
			}
		}
	}
	
	// Perform a blocking search to ensure all assets used by mutable are reachable using the AssetRegistry
	PrepareAssetRegistry();
	
	// Make sure there is nothing else that the engine needs to do before starting our test
	// Wait(60);	// todo: UE-304050 Remove this wait as it may no longer be required due to us calling for GShaderCompilingManager->FinishAllCompilation()
	
	// Block until async shader compiling is finished before we try to use the shaders for exporting
	// The code is structured to only block once for all materials, so that shader compiling is able to utilize many cores
	if (GShaderCompilingManager && GShaderCompilingManager->GetNumRemainingJobs() > 0)
	{
		GShaderCompilingManager->FinishAllCompilation();
	}
	
	LogGlobalSettings();
	
	for (int32 TargetPluginIndex = 0; TargetPluginIndex < PluginsToScanForCOs.Num(); TargetPluginIndex++)
	{		
		// All CO pointing FAssetData found at the to scan plugin path
		TArray<FAssetData> FoundAssetData;
		{
			LLM_SCOPE_BYNAME(TEXT("COMultiPluginBulkValidationCommandlet/AssetsSearch"));
		
			const FString PluginSearchPath = FString::Printf(TEXT("/%s"), *PluginsToScanForCOs[TargetPluginIndex].ToString());
			const FName CustomizableObjectsSearchPath = FName(PluginSearchPath);
		
			FoundAssetData = FindAllAssetsAtPath(CustomizableObjectsSearchPath, UCustomizableObject::StaticClass());
			if (FoundAssetData.IsEmpty())
			{
				UE_LOGF(LogMutable, Display, "Found no Customizable Objects to validate in the provided path : %ls", *CustomizableObjectsSearchPath.ToString());
				continue;
			}
		
			// Log all the Customizable Objects to be tested:
			UE_LOGF(LogMutable, Display, "Found a total of %u Customizable Objects to validate at '%ls'. Some may be discarded based on the test settings.", FoundAssetData.Num(), *CustomizableObjectsSearchPath.ToString())
			for (const FAssetData& MutableAssetData : FoundAssetData)
			{
				UE_LOGF(LogMutable, Display, "\t - %ls (%ls)", *MutableAssetData.AssetName.ToString(), *MutableAssetData.PackageName.ToString());
			}
		}
		
		// Try to collect the garbage
		{
			if (GIsInitialLoad)
			{
				UE_LOGF(LogMutable, Warning, "GC will not run as GIsInitialLoad is currently set to true.");
			}
			
			CollectGarbage(RF_NoFlags, true);
		}
		
		// FoundAssetData is not empty so we can start to test things out
		{
			const FName PluginName = PluginsToScanForCOs[TargetPluginIndex];
			check(!PluginName.IsNone());
		
			// Make all the logs that appear in this scope have a relation with the plugin name provided. This will be later used by the MTU to name the CO generated data
			const FScopedLogSection PluginLogSection (EMutableLogSection::Plugin, PluginName);
			
			UE_LOGF(LogMutable, Display, "Starting testing of COs from the '%ls' plugin .", *PluginName.ToString())
					
			// iterate over the FoundAsstData of each Plugin-Dir structure
			const int32 AssetDataCount = FoundAssetData.Num();
			for (int32 AssetDataIndex = 0; AssetDataIndex < AssetDataCount; AssetDataIndex++)
			{
				LLM_SCOPE_BYNAME(TEXT("COMultiPluginBulkValidationCommandlet/COTest"));
				
				const FAssetData& AssetData = FoundAssetData[AssetDataIndex];
				TObjectPtr<UObject> FoundObject = UE::Mutable::Private::LoadObject(AssetData);
				if (!FoundObject)
				{
					UE_LOGF(LogMutable, Error, "Failed to load the asset with path : %ls .", *AssetData.GetSoftObjectPath().ToString())
					continue;
				}

				TObjectPtr<UCustomizableObject> TargetCustomizableObject = CastChecked<UCustomizableObject>(FoundObject);
				
				// If we only want to test root objects and this is not one go to the next CO
				if (bOnlyTestRootObjects && !ICustomizableObjectEditorModule::GetChecked().IsRootObject(*TargetCustomizableObject.Get()))
				{
					UE_LOGF(LogMutable, Display, "Skipping CO \"%ls\" as it is not a root CO.", *TargetCustomizableObject->GetName())
					continue;
				}
			
				// Body of the test ----------------------------------------------------------------------------------------------------------------------
				TestCustomizableObject(*TargetCustomizableObject, *TargetCompilationPlatform, InstancesToGenerate);
				// ---------------------------------------------------------------------------------------------------------------------------------------
			
				// Try to collect the garbage
				{
					if (GIsInitialLoad)
					{
						UE_LOGF(LogMutable, Warning, "GC will not run as GIsInitialLoad is currently set to true.");
					}
				
					CollectGarbage(RF_NoFlags, true);
				}

				UE_LOGF(LogMutable, Display, "%ls : Validated %u/%u assets.", *PluginName.ToString(), AssetDataIndex + 1, AssetDataCount);
			}
			
			UE_LOGF(LogMutable, Display, "Finished testing the COs from the '%ls' plugin .", *PluginName.ToString())
		}
	}
	
	UE_LOGF(LogMutable, Display, "Mutable commandlet finished.");
	return 0;
}
