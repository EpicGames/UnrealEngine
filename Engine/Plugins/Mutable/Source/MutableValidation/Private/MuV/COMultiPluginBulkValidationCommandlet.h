// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "COMultiPluginBulkValidationCommandlet.generated.h"


UCLASS()
class UCOMultiPluginBulkValidationCommandlet : public UCommandlet
{
	GENERATED_BODY()
	
public:
	virtual int32 Main(const FString& Params) override;
};
