// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/CacheAssetRegistryCommandlet.h"

#include "ValidationUtils.h"
#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CacheAssetRegistryCommandlet)

int32 UCacheAssetRegistryCommandlet::Main(const FString& Params)
{
	// Ensure we are not running this commandlet in vain as we want the cache to be filled
	if (FParse::Param(FCommandLine::Get(), TEXT("NoAssetRegistryCacheWrite")))
	{
		UE_LOGF(LogTemp, Error, "The Asset Registry data is not going to be cached due to the arg 'NoAssetRegistryCacheWrite' being present in the commandline.");
		return 1;
	}
	
	// Perform a blocking search to ensure all assets used by mutable are reachable using the AssetRegistry
	PrepareAssetRegistry();

	return 0;
}
