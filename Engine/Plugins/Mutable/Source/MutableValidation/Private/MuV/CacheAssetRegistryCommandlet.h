// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"

#include "CacheAssetRegistryCommandlet.generated.h"

/** Simple commandlet used to fill-up the asset registry cache for the execution of the mutable tests */
UCLASS()
class UCacheAssetRegistryCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	virtual int32 Main(const FString& Params) override;
};
