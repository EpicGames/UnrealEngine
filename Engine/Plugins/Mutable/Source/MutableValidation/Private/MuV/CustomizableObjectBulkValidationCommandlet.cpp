// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/CustomizableObjectBulkValidationCommandlet.h"

#include "ShaderCompiler.h"
#include "ValidationUtils.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCO/LoadUtils.h"
#include "MuCO/LogBenchmarkUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectBulkValidationCommandlet)


int32 UCustomizableObjectBulkValidationCommandlet::Main(const FString& Params)
{
	LLM_SCOPE_BYNAME(TEXT("CustomizableObjectBulkValidationCommandlet"));

	// Prepare the environment for the testing -------------------------------------------------------------------------------------------------------
	
	// Ensure we have set the mutable system to the benchmarking mode and that we are reporting benchmarking data
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(true);
	UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(true);
	
	// Ensure we do not show any OK dialog since we are not a user that can interact with them
	GIsRunningUnattendedScript = true;

	// Parse the arguments provided ------------------------------------------------------------------------------------------------------------------
	
	// Parse the value for the platform to be used for the compilation of the COs
	const ITargetPlatform* TargetCompilationPlatform = GetCompilationPlatform(Params);
	if (!TargetCompilationPlatform)
	{
		return 1;
	}

	// Get the amount of instances to generate if parameter was provided (it will get multiplied by the amount of states later so this is a minimun value)
	const uint32 InstancesToGenerate = GetTargetAmountOfInstances(Params);

	// Work only for the root Customizable Objects found
	const bool bOnlyTestRootObjects = GetIfOnlyRootCOsShouldBeUsed(Params);

	// Find the assets to be tested based on the path provided ---------------------------------------------------------------------------------------
	
	// Parse the path to search for COs on
	FName CustomizableObjectsSearchPath;
	{
		if (!FParse::Value(*Params, TEXT("CustomizableObjectsSearchPath="), CustomizableObjectsSearchPath))
		{
			UE_LOGF(LogMutable, Error, "Failed to parse Customizable Object search path from the provided argument : %ls", *Params)
			return 1;
		}

		if (CustomizableObjectsSearchPath.IsNone())
		{
			UE_LOGF(LogMutable, Error, "The path to scan can not be empty")
			return 1;
		}
	}

	// Perform a blocking search to ensure all assets used by mutable are reachable using the AssetRegistry
	PrepareAssetRegistry();
	
	// Then run what we know as the UCustomizableObjectValidationCommandlet body for each one of them. Keep in mind that you may need to do some cleanup
	// between runs.
	TArray<FAssetData> FoundAssetData;
	{
		LLM_SCOPE_BYNAME(TEXT("CustomizableObjectBulkValidationCommandlet/AssetsSearch"));

		FoundAssetData = FindAllAssetsAtPath(CustomizableObjectsSearchPath, UCustomizableObject::StaticClass());

		// Early exit if no instances could be found for testing.
		if (FoundAssetData.IsEmpty())
		{
			UE_LOGF(LogMutable, Error, "Aborting Bulk Customizable Object Validation Test: No assets could be found at the provided package path");
			return 1;
		}
		
		// Log all the Customizable Objects to be tested:
		UE_LOGF(LogMutable, Display, "Found a total of %u Customizable Objects to validate. Some may be discarded based on the test settings.", FoundAssetData.Num())
		for (const FAssetData& MutableAssetData : FoundAssetData)
		{
			UE_LOGF(LogMutable, Display, "\t - %ls (%ls)", *MutableAssetData.AssetName.ToString(), *MutableAssetData.PackageName.ToString());
		}
	}
	
	// Make sure there is nothing else that the engine needs to do before starting our test
	Wait(60);	// todo: UE-304050 Remove this wait as it may no longer be required due to us calling for GShaderCompilingManager->FinishAllCompilation()

	// Block until async shader compiling is finished before we try to use the shaders for exporting
	// The code is structured to only block once for all materials, so that shader compiling is able to utilize many cores
	if (GShaderCompilingManager && GShaderCompilingManager->GetNumRemainingJobs() > 0)
	{
		GShaderCompilingManager->FinishAllCompilation();
	}
	
	LogGlobalSettings();
	
	// All pre-testing operations completed. Starting the testing of COs -----------------------------------------------------------------------------

	const int32 FoundAssetsCount = FoundAssetData.Num();
	for (int32 AssetIndex = 0; AssetIndex < FoundAssetsCount; ++AssetIndex)
	{
		LLM_SCOPE_BYNAME(TEXT("CustomizableObjectBulkValidationCommandlet/COTest"));

		const FAssetData& CustomizableObjectAssetData = FoundAssetData[AssetIndex];
		TObjectPtr<UObject> FoundObject = UE::Mutable::Private::LoadObject(CustomizableObjectAssetData);
		if (!FoundObject)
		{
			UE_LOGF(LogMutable, Error, "Failed to load the asset with path : %ls .", *CustomizableObjectAssetData.GetSoftObjectPath().ToString())
			continue;
		}

		TObjectPtr<UCustomizableObject> TargetCustomizableObject = CastChecked<UCustomizableObject>(FoundObject);

		// If we only want to test root objects and this is not one go to the next CO
		if (bOnlyTestRootObjects && !ICustomizableObjectEditorModule::GetChecked().IsRootObject(*TargetCustomizableObject.Get()))
		{
			UE_LOGF(LogMutable, Display, "Skipping CO \"%ls\" as it is not a root CO.", *TargetCustomizableObject->GetName())
			continue;
		}
		
		// Body of the test ----------------------------------------------------------------------------------------------------------------------
		TestCustomizableObject(*TargetCustomizableObject, *TargetCompilationPlatform, InstancesToGenerate);
		// ---------------------------------------------------------------------------------------------------------------------------------------
		
		// Try to collect the garbage
		{
			if (GIsInitialLoad)
			{
				UE_LOGF(LogMutable, Warning, "GC will not run as GIsInitialLoad is currently set to true.");
			}
			
			CollectGarbage(RF_NoFlags, true);
		}

		UE_LOGF(LogMutable, Display, "Validated %u/%u assets.", AssetIndex + 1, FoundAssetsCount);
	}

	UE_LOGF(LogMutable, Display, "Mutable commandlet finished.");
	return 0;
}
