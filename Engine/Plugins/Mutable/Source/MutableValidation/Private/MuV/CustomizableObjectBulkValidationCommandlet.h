// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "CustomizableObjectBulkValidationCommandlet.generated.h"

/**
 * Commandlet similar to "UCustomizableObjectValidationCommandlet" but in this case it performs the same tests as in the mentioned commandlet but over
 * a series of COs located at a given path.
 */
UCLASS()
class UCustomizableObjectBulkValidationCommandlet : public UCommandlet
{
	GENERATED_BODY()
	
public:
	virtual int32 Main(const FString& Params) override;
};
