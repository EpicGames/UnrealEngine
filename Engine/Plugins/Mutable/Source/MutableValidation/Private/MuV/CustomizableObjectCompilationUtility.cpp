// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableObjectCompilationUtility.h"

#include "MuCOE/ICustomizableObjectEditorModulePrivate.h"
#include "ScopedLogSection.h"
#include "Commandlets/Commandlet.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCO/CustomizableObjectCompilerTypes.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "MuCOE/CompileRequest.h"
#include "MuCOE/CustomizableObjectEditorFunctionLibrary.h"
#include "HAL/FileManager.h"
#include "MuCOE/CompilationOptions.h"


bool FCustomizableObjectCompilationUtility::CompileCustomizableObject(UCustomizableObject& InCustomizableObject, const bool bShouldLogMutableLogs /* = true */, const FCompilationOptions* InCompilationOptionsOverride  /* nullptr */)
{
	LLM_SCOPE_BYNAME(TEXT("FCustomizableObjectCompilationUtility/Compile"));
	const FScopedLogSection CompilationSection (EMutableLogSection::Compilation);	
	
	CustomizableObject = TStrongObjectPtr(&InCustomizableObject);
	
	// Handle the overriding of the compilation configuration of the CO ------------------------------------------------------------------------------
	
	// Override the compilation options if an override has been provided by the user
	FCompilationOptions CompilationOptions;
	const bool bOverrideCompilationOptions = InCompilationOptionsOverride != nullptr;
	if (bOverrideCompilationOptions)
	{
		CompilationOptions = *InCompilationOptionsOverride;
		UE_LOGF(LogMutable,Display, "CO Compilation options overriden by the user defined ones.");
	}
	else
	{
		CompilationOptions = GetCompilationOptions(*CustomizableObject);
		UE_LOGF(LogMutable,Display, "Compiling CO using it's own compilation options.");
	}

	// Ensure that the user has provided a target compilation platform.
	// Mutable is able to run without one but we want to be explicit in the context of testing.
	if (!CompilationOptions.TargetPlatform)
	{
		UE_LOGF(LogMutable, Error, "The compilation of the %ls model could not be started : No explicit platform was provided.", *CustomizableObject->GetName());
		return false;
	}

	// Report the compilation configuration : We use this flag just to be sure we only generate logs when required since MongoDB does not want to find
	// duplicated keys. The addition of duplicated entries in MongoDB is not available so this way we avoid having to handle them
	// when we are sure we do not require it.
	if (bShouldLogMutableLogs)
	{
		UE_LOGF(LogMutable, Log, "(string) model_compile_options_overriden : %s ", "false");
		UE_LOGF(LogMutable, Log, "(int) model_optimization_level : %d ", CompilationOptions.OptimizationLevel);
		UE_LOGF(LogMutable, Log, "(string) model_texture_compression : %ls ", *UEnum::GetValueAsString(CompilationOptions.TextureCompression));
		UE_LOGF(LogMutable, Log, "(string) model_compile_platform_name : %ls ", *CompilationOptions.TargetPlatform->PlatformName());
	}

	// Delete data from previous compilations just to be sure it does not affect the new compilation operation
	IFileManager& FileManager = IFileManager::Get();
	const FString CompiledDataDirectory = GetCompiledDataFolderPath();
	if (CustomizableObject->IsCompiled() && FileManager.DirectoryExists(*CompiledDataDirectory))
	{
		UE_LOGF(LogMutable,Display, "Deleting old mutable compiled data from '%ls'...", *CompiledDataDirectory);
		if ( !FileManager.DeleteDirectory( *CompiledDataDirectory, false, true) )
		{
			UE_LOGF(LogMutable,Warning, "Failed to delete old compiled data directory '%ls'. This could affect the compilation of the CO.", *CompiledDataDirectory);
		}
		else
		{
			UE_LOGF(LogMutable, Display, "Directory deleted successfully");
		}
	}
	
	// Compilation completed successfully
	bool bCompilationSuccess = false;
	
	// Run and wait for the compilation to be completed ----------------------------------------------------------------------------------------------
	{
		// Get the memory usage before staring the compilation
#if ENABLE_LOW_LEVEL_MEM_TRACKER
		FLowLevelMemTracker& LowLevelMemoryTracker = FLowLevelMemTracker::Get();

		// Memory in use before starting the compilation itself
		int64 CompilationStartTotalBytes = 0;
		// Peak memory usage as reported by LLM during the compilation operation.
		// Note : This value will not discriminate and therefore will report memory usage by the compilation of the CO as well as other operations that
		// may be running during it's operation
		int64 CompilationEndTotalRealPeakBytes = 0;
		
		if (FLowLevelMemTracker::IsEnabled())
		{
			UE_LOGF(LogMutable,Display, "LLM system enabled: Peak memory usage during mutable CO compilation will be logged after said compilation takes place.");
			
			LowLevelMemoryTracker.UpdateStatsPerFrame();
			CompilationStartTotalBytes = LowLevelMemoryTracker.GetTagAmountForTracker(ELLMTracker::Default, ELLMTag::Total, UE::LLM::ESizeParams::ReportCurrent);
		}
#endif
		
		// Get the memory being used by mutable before the compilation
		const int64 CompilationStartMutableBytes = UE::Mutable::Private::FGlobalMemoryCounter::GetAbsoluteCounter();
		UE::Mutable::Private::FGlobalMemoryCounter::Zero();
		
		UE_LOGF(LogMutable,Display, "Compiling Customizable Object...");
		
		TSharedRef<FCompilationRequest> CompileRequest = MakeShared<FCompilationRequest>(*CustomizableObject);
		CompileRequest->bSilentCompilation = false;
		CompileRequest->Options = CompilationOptions;
		
		ICustomizableObjectEditorModulePrivate::GetChecked().EnqueueCompileRequest(CompileRequest);
		
		// Wait while the compilation takes place
		const double CompilationStartSeconds = FPlatformTime::Seconds();
		while (CompileRequest->GetCompilationState() != ECompilationStatePrivate::Completed)
		{
			LLM_SCOPE_BYNAME(TEXT("FCustomizableObjectCompilationUtility/CompileLoop"));
			
			// Tick the engine
			CommandletHelpers::TickEngine();

			// todo: Will this be required in the future? It feels odd having to tick the compiler manually when we are already ticking the engine itself
			ICustomizableObjectEditorModule::GetChecked().Tick(false);
			
			// Cache the peak value found during the compilation of the CO
#if ENABLE_LOW_LEVEL_MEM_TRACKER
			if (FLowLevelMemTracker::IsEnabled())
			{
				LowLevelMemoryTracker.UpdateStatsPerFrame();
				const int64 CurrentTotalMemoryUsage = LowLevelMemoryTracker.GetTagAmountForTracker(ELLMTracker::Default, ELLMTag::Total, UE::LLM::ESizeParams::ReportCurrent);
				CompilationEndTotalRealPeakBytes = CompilationEndTotalRealPeakBytes > CurrentTotalMemoryUsage ? CompilationEndTotalRealPeakBytes : CurrentTotalMemoryUsage;
			}
#endif
		
			// Stop if exit was requested
			if (IsEngineExitRequested())
			{
				break;
			}
		}
		

		bCompilationSuccess = CompileRequest->GetCompilationResult() == ECompilationResultPrivate::Success ||
			CompileRequest->GetCompilationResult() == ECompilationResultPrivate::Warnings;

		// Report the time we took to run the compilation
		const double CompilationEndSeconds = FPlatformTime::Seconds() - CompilationStartSeconds;
		UE_LOGF(LogMutable, Display, "The compilation of the %ls CO model took %f seconds.", *CustomizableObject->GetName(), CompilationEndSeconds);
		if (bShouldLogMutableLogs)
		{
			UE_LOGF(LogMutable, Log, "(double) model_compile_time_ms : %f ", CompilationEndSeconds * 1000);

			// Also report compilation end status
			const ECustomizableObjectCompilationState CompilationEndResult = bCompilationSuccess ? ECustomizableObjectCompilationState::Completed
				: ECustomizableObjectCompilationState::Failed;
			UE_LOGF(LogMutable, Log, "(string) model_compile_end_state : %ls ", *UEnum::GetValueAsString(CompilationEndResult));
		}
		
		// Get the peak mutable memory used during the compilation operation
		const int64 CompilationEndPeakMutableBytes = UE::Mutable::Private::FGlobalMemoryCounter::GetPeak();
		const int64 CompilationEndRealMutablePeakBytes = CompilationStartMutableBytes + CompilationEndPeakMutableBytes;
		UE_LOGF(LogMutable, Display, "Pre-Compilation Mutable memory usage : %lld", CompilationStartMutableBytes);
		UE_LOGF(LogMutable, Display, "Compilation Mutable peak memory usage : %lld", CompilationEndPeakMutableBytes);
		UE_LOGF(LogMutable, Display, "Compilation Mutable real peak memory usage : %lld", CompilationEndRealMutablePeakBytes);
		if (bShouldLogMutableLogs)
		{
			// Mutable system reported memory usage
			UE_LOGF(LogMutable, Log, "(int) model_compilation_start_bytes : %lld ", CompilationStartMutableBytes);
			UE_LOGF(LogMutable, Log, "(int) model_compilation_end_peak_bytes : %lld ", CompilationEndPeakMutableBytes);
			UE_LOGF(LogMutable, Log, "(int) model_compilation_end_real_peak_bytes : %lld ", CompilationEndRealMutablePeakBytes);
		}

		
		// Now report the peak memory usage reported by the "total" tag from the LLM system
#if ENABLE_LOW_LEVEL_MEM_TRACKER
		if (FLowLevelMemTracker::IsEnabled())
		{
			UE_LOGF(LogMutable, Display, "LLM Pre-Compilation Total memory usage (bytes) : %lld", CompilationStartTotalBytes);
			// CompilationPeakBytes : Peak memory recorded during the compilation subtracting from it the memory in use before the compilation itself.
			// It should reflect the peak memory usage OF the compilation (as direct result of the compilation itself) but it should be used as an approximation
			// since we can never be too sure no other system is doing work in parallel that is not part of the mutable compilation operation.
			const int64 CompilationEndTotalPeakBytes = CompilationEndTotalRealPeakBytes - CompilationStartTotalBytes;
			UE_LOGF(LogMutable, Display, "LLM Total Compilation Peak memory usage (bytes) : %lld", CompilationEndTotalPeakBytes);
			UE_LOGF(LogMutable, Display, "LLM Total Compilation Real Peak memory usage (bytes) : %lld", CompilationEndTotalRealPeakBytes);
			if (bShouldLogMutableLogs)
			{
				UE_LOGF(LogMutable, Log, "(int) model_compilation_llm_total_start_bytes : %lld ", CompilationStartTotalBytes);
				UE_LOGF(LogMutable, Log, "(int) model_compilation_llm_total_end_peak_bytes : %lld ", CompilationEndTotalPeakBytes);
				UE_LOGF(LogMutable, Log, "(int) model_compilation_llm_total_end_real_peak_bytes : %lld ", CompilationEndTotalRealPeakBytes);
			}
		}
#endif
	}

	// Return the success state of the operation
	return bCompilationSuccess;
}
