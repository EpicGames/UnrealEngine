// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/StrongObjectPtr.h"
#include "Templates/SharedPointer.h"

// Forward declarations
class UCustomizableObject;
struct FCompilationOptions;

/**
 * Helper class that allows the calling commandlet to treat the asynchronous compilation of a CO as a synchronous operation.
 */
class FCustomizableObjectCompilationUtility :  public TSharedFromThis<FCustomizableObjectCompilationUtility>
{
public:
	
	/**
	 * Run the asyncronous compilation of the provided CO but as part of this sync method. 
	 * @param InCustomizableObject The Customizable Object we want to synchronously compile
	 * @param bShouldLogMutableLogs Enables or disables the logging of Log category logs relevant to the CO compilation. Required to avoid a MongoDB limitation with the duplication of MongoDB document names.
	 * @param InCompilationOptionsOverride The configuration for the compilation of the CO we want to use instead of the one part of the CO.
	 * @return True if the compilation was successful and false if it failed.
	 */
	bool CompileCustomizableObject(UCustomizableObject& InCustomizableObject, const bool bShouldLogMutableLogs = true, const FCompilationOptions* InCompilationOptionsOverride = nullptr );

private:

	/** The CO that is currently being compiled.  */
	TStrongObjectPtr<UCustomizableObject> CustomizableObject;
};