// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableObjectInstanceUpdateUtility.h"

#include "Commandlets/Commandlet.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "Engine/TextureStreamingTypes.h"
#include "SceneTypes.h"
#include "ScopedLogSection.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/MaterialInterface.h"

bool FCustomizableObjectInstanceUpdateUtility::UpdateInstance(UCustomizableObjectInstance& InInstance)
{
	LLM_SCOPE_BYNAME(TEXT("FCustomizableObjectInstanceUpdateUtility/UpdateInstance"));
	const FScopedLogSection UpdateSection (EMutableLogSection::Update);	
	
	check (ComponentsBeingUpdated.IsEmpty());
	
	// Cache the instance being updated for reference once in the update end callback
	Instance = TStrongObjectPtr(&InInstance);
	
	// Schedule the update of the COI
	{
		UE_LOGF(LogMutable, Display, "Invoking update for %ls instance.", *Instance->GetName());

		// Instance update delegate	
		FInstanceUpdateNativeDelegate InstanceUpdateNativeDelegate;
		InstanceUpdateNativeDelegate.AddSP(this, &FCustomizableObjectInstanceUpdateUtility::OnInstanceUpdateResult);

		bIsInstanceBeingUpdated = true;
		Instance->UpdateSkeletalMeshAsyncResult(InstanceUpdateNativeDelegate, true, true);
	}

	// Debug
	// FLowLevelMemTracker& LowLevelMemoryTracker = FLowLevelMemTracker::Get();
	
	// Wait until the update has been completed and the mips streamed
	while (bIsInstanceBeingUpdated)
	{
		LLM_SCOPE_BYNAME(TEXT("FCustomizableObjectInstanceUpdateUtility/UpdateLoop"));
		
		// Tick the engine
		CommandletHelpers::TickEngine();

		// Debug
		// LowLevelMemoryTracker.UpdateStatsPerFrame();
		
		// Stop if exit was requested
		if (IsEngineExitRequested())
		{
			break;
		}

		// Wait until all MIPs gets streamed 
		if (!ComponentsBeingUpdated.IsEmpty())
		{
			bool bFullyStreamed = true;
			for (TIndexedContainerIterator<TArray<TStrongObjectPtr<USkeletalMeshComponent>>, TArray<TStrongObjectPtr<USkeletalMeshComponent>>::ElementType, TArray<TStrongObjectPtr<USkeletalMeshComponent>>::SizeType> It = ComponentsBeingUpdated.CreateIterator(); It && bFullyStreamed; ++It)
			{
				TStrongObjectPtr<USkeletalMeshComponent>& ComponentBeingUpdated = *It;
            		
				FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), ComponentBeingUpdated.Get());
				TArray<FStreamingRenderAssetPrimitiveInfo> RenderAssetInfoArray;
				ComponentBeingUpdated->GetStreamingRenderAssetInfo(LevelContext, RenderAssetInfoArray);

				for (TIndexedContainerIterator<TArray<FStreamingRenderAssetPrimitiveInfo>, TArray<
					                               FStreamingRenderAssetPrimitiveInfo>::ElementType, TArray<
					                               FStreamingRenderAssetPrimitiveInfo>::SizeType> ItAsset =
					     RenderAssetInfoArray.CreateIterator(); ItAsset && bFullyStreamed; ++ItAsset)
				{
					bFullyStreamed = ItAsset->RenderAsset->IsFullyStreamedIn();
				}
			}

			if (bFullyStreamed)
			{
				UE_LOGF(LogMutable, Display, "Instance %ls finished streaming all MIPs.", *Instance->GetName());
				ComponentsBeingUpdated.Reset();
				
				bIsInstanceBeingUpdated = false;		// Exit the while loop
			}
		}
	}
	
	// Return true if the update was successful and false if not
	return !bInstanceFailedUpdate;
}


void FCustomizableObjectInstanceUpdateUtility::OnInstanceUpdateResult(const FUpdateContext& Result)
{
	LLM_SCOPE_BYNAME(TEXT("FCustomizableObjectInstanceUpdateUtility/OnInstanceUpdated"));
	
	check(Instance);
	const FString InstanceName = Instance->GetName();
	
	if (UCustomizableObjectSystem::IsUpdateResultValid(Result.UpdateResult))
	{
		UE_LOGF(LogMutable, Display, "Instance %ls finished update successfully.", *InstanceName);
		bInstanceFailedUpdate = false;
		
		// Request load all MIPs
		check(ComponentsBeingUpdated.IsEmpty());

		const TArray<FName> InstanceComponents = Instance->GetComponentNames();
		
		// Early out if no components are found in the instance and notify the Utility that the instance finished its update
		if (InstanceComponents.IsEmpty())
		{
			bIsInstanceBeingUpdated = false;
			return;
		}

		// Request the streaming in of all the components affected by the update
		for (const FName& ComponentName : InstanceComponents)
		{
			TStrongObjectPtr<USkeletalMeshComponent> SkeletalComponent = TStrongObjectPtr(NewObject<USkeletalMeshComponent>());
			SkeletalComponent->SetSkeletalMesh(Instance->GetComponentMeshSkeletalMesh(ComponentName));

			FStreamingTextureLevelContext LevelContext(GetCurrentMaterialQualityLevelChecked(), SkeletalComponent.Get());
			TArray<FStreamingRenderAssetPrimitiveInfo> RenderAssetInfoArray;
			SkeletalComponent->GetStreamingRenderAssetInfo(LevelContext, RenderAssetInfoArray);

			for (const FStreamingRenderAssetPrimitiveInfo& Info : RenderAssetInfoArray)
			{
				Info.RenderAsset->StreamIn(MAX_int32, true);
			}
			
			ComponentsBeingUpdated.Add(SkeletalComponent);
		}

		UE_LOGF(LogMutable, Display, "Instance %ls requesting the streaming of all MIPs.", *InstanceName);
	}
	else
	{
		const FString OutputStatus = UEnum::GetValueAsString(Result.UpdateResult);
		UE_LOGF(LogMutable, Error, "Instance %ls finished update with anomalous state : %ls.", *InstanceName, *OutputStatus);
		bInstanceFailedUpdate = true;
		
		// Tell the system the instance finished its update so we can continue the execution without waiting for the mips to stream in
		bIsInstanceBeingUpdated = false;
	}	
}

