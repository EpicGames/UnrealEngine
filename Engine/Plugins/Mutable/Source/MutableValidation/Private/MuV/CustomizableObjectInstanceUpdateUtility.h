// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MuCO/CustomizableObjectInstance.h"
#include "UObject/StrongObjectPtr.h"
#include "Templates/SharedPointer.h"

// Forward declarations
class USkeletalMeshComponent;

/**
 * Helping class that handles the async update of the provided instance. It will also wait for the mips of it so they get streamed.
 */
class FCustomizableObjectInstanceUpdateUtility : public TSharedFromThis<FCustomizableObjectInstanceUpdateUtility>
{
public:
	
	/**
	 * Updates the provided customizable object instance.
	 * @param InInstance Instance to be compiled
	 * @return True if the update was successful and false otherwise
	 */
	bool UpdateInstance(UCustomizableObjectInstance& InInstance);
	

private:
	
	/**
	 * Callback executed when the instance being updated finishes it's mesh update.
	 * @param Result The result of the updating operation
	 */
	void OnInstanceUpdateResult(const FUpdateContext& Result);

	/** The instance that is currently being handled by this class. */
	TStrongObjectPtr<UCustomizableObjectInstance> Instance = nullptr;

	/** The components of the Instance that we are currently waiting for their mips to be streamed in */
	TArray<TStrongObjectPtr<USkeletalMeshComponent>> ComponentsBeingUpdated;

	/** Flag used to control if we are updating and instance or not. Once it gets set to false then the update gets halted and the program continues */
	bool bIsInstanceBeingUpdated = false;

	/** False if the instance did update successfully, true if it failed */
	bool bInstanceFailedUpdate = false;
};
