// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuV/CustomizableObjectValidationCommandlet.h"

#include "ShaderCompiler.h"
#include "ValidationUtils.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/Array.h"
#include "Containers/SpscQueue.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "MuCO/CustomizableObjectSystemPrivate.h"
#include "MuCO/LoadUtils.h"
#include "Serialization/MemoryWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectValidationCommandlet)


int32 UCustomizableObjectValidationCommandlet::Main(const FString& Params)
{
	LLM_SCOPE_BYNAME(TEXT("CustomizableObjectValidationCommandlet"));
	
	// Execution arguments for commandlet from IDE
	// -run=CustomizableObjectValidation -CustomizableObject=(PathToCO)

	// Ensure we have set the mutable system to the benchmarking mode and that we are reporting benchmarking data
	FLogBenchmarkUtil::SetBenchmarkReportingStateOverride(true);
	UCustomizableObjectSystemPrivate::SetUsageOfBenchmarkingSettings(true);
	
	// Ensure we do not show any OK dialog since we are not an user that can interact with them
	GIsRunningUnattendedScript = true;
	
	// Get the package name of the CO to test
	TStrongObjectPtr<UCustomizableObject> TargetCustomizableObject = nullptr;
	{
		FString CustomizableObjectAssetPath = "";
		if (!FParse::Value(*Params, TEXT("CustomizableObject="), CustomizableObjectAssetPath))
		{
			UE_LOGF(LogMutable, Error, "Failed to parse Customizable Object package name from provided argument : %ls", *Params)
			return 1;
		}
	
		// Load the resource
		UObject* FoundObject = UE::Mutable::Private::LoadObject(FSoftObjectPath(CustomizableObjectAssetPath));
		if (!FoundObject)
		{
			UE_LOGF(LogMutable, Error, "Failed to retrieve UObject from path %ls", *CustomizableObjectAssetPath);
			return 1;
		}
	
		// Get the CustomizableObject.
		TargetCustomizableObject = TStrongObjectPtr(Cast<UCustomizableObject>(FoundObject));
		if (!TargetCustomizableObject)
		{
			UE_LOGF(LogMutable, Error, "Failed to cast found UObject to UCustomizableObject.");
			return 1;
		}
	}
	
	// What platform we want to compile the CO against
	ITargetPlatform* TargetCompilationPlatform = GetCompilationPlatform(Params);
	if (!TargetCompilationPlatform)
	{
		return 1;
	}

	const uint32 InstancesToGenerate = GetTargetAmountOfInstances(Params);
	
	// Perform a blocking search to ensure all assets used by mutable are reachable using the AssetRegistry
	PrepareAssetRegistry();

	// Make sure there is nothing else that the engine needs to do before starting our test
	Wait(60);	// todo: UE-304050 Remove this wait as it may no longer be required due to us calling for GShaderCompilingManager->FinishAllCompilation()

	// Block until async shader compiling is finished before we try to use the shaders for exporting
	// The code is structured to only block once for all materials, so that shader compiling is able to utilize many cores
	if (GShaderCompilingManager && GShaderCompilingManager->GetNumRemainingJobs() > 0)
	{
		GShaderCompilingManager->FinishAllCompilation();
	}
	
	LogGlobalSettings();

	// Body of the test ------------------------------------------------------------------------------------------------------------------------------
	TestCustomizableObject(*TargetCustomizableObject, *TargetCompilationPlatform, InstancesToGenerate);
	// -----------------------------------------------------------------------------------------------------------------------------------------------
	
	UE_LOGF(LogMutable, Display, "Mutable commandlet finished.");
	return 0;
}
