// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"

#include "CustomizableObjectValidationCommandlet.generated.h"

/**
 * Commandlet designed to compile and then update a deterministically random set of instances.
 */
UCLASS()
class UCustomizableObjectValidationCommandlet : public UCommandlet 
{
	GENERATED_BODY()

public:
	virtual int32 Main(const FString& Params) override;
};


