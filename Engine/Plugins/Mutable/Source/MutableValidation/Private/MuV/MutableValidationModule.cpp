// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Modules/ModuleManager.h"
#include "MuV/MutableValidationSettings.h"

#define LOCTEXT_NAMESPACE "MutableValidationSettings"

/**
 * StaticMesh editor module
 */
class FMutableValidationModule : public FDefaultModuleImpl
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	bool HandleSettingsSaved() const;

private:
	ISettingsSectionPtr SettingsSectionPtr = nullptr;
};

IMPLEMENT_MODULE(FMutableValidationModule, MutableValidation);

bool FMutableValidationModule::HandleSettingsSaved() const
{
	UMutableValidationSettings* CustomizableObjectSettings = GetMutableDefault<UMutableValidationSettings>();

	if (CustomizableObjectSettings != nullptr)
	{
		CustomizableObjectSettings->SaveConfig();
	}

    return true;
}


void FMutableValidationModule::StartupModule()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{ 		
		 SettingsSectionPtr = SettingsModule->RegisterSettings("Project", "Plugins", "MutableValidationSettings",
			LOCTEXT("MutableValidationSettings_Setting", "Mutable Validation"),
			LOCTEXT("MutableValidatioinSettings_Setting_Desc", "Mutable resources validation settings"),
			GetMutableDefault<UMutableValidationSettings>()
		);
		
		if (SettingsSectionPtr.IsValid())
        {
        	SettingsSectionPtr->OnModified().BindRaw(this, &FMutableValidationModule::HandleSettingsSaved);
        }
	}
}


void FMutableValidationModule::ShutdownModule()
{
	// Unbind OnModified delegate
	if (SettingsSectionPtr)
	{
		SettingsSectionPtr->OnModified().Unbind();
	}
	
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{ 		
		SettingsModule->UnregisterSettings("Project", "Plugins", "MutableValidationSettings");
	}
}


#undef LOCTEXT_NAMESPACE
