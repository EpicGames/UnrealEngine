// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MutableValidationSettings.generated.h"

UCLASS(config = Editor)
class UMutableValidationSettings : public UObject
{
	GENERATED_BODY()

public:

	/** If set to true the checked out Customizable Objects will get validated. */
	UPROPERTY(meta=(DisplayName="Enable Direct Customizable Object Validation"),config, EditAnywhere, Category = Validation)
	bool bEnableDirectCOValidation = false;

	/**If set to true the Customizable Objects referenced by the checked out resources will get validated.
	 *This validation is resource intensive and will require more time to run than the Direct Customizable Object Validation.*/
	UPROPERTY(meta=(DisplayName="Enable Indirect Customizable Object Validation"), config, EditAnywhere, Category = Validation)
	bool bEnableIndirectCOValidation = false;
};
