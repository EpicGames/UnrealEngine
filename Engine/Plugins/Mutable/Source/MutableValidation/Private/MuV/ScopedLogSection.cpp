// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedLogSection.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"


DECLARE_LOG_CATEGORY_EXTERN(LogMutableValidation, Log, All);
DEFINE_LOG_CATEGORY(LogMutableValidation);


FScopedLogSection::FScopedLogSection(EMutableLogSection Section, const FName& InSectionTarget)
{
	// Set the current section handled by this object
	CurrentSection = Section;
	
	if (CurrentSection == EMutableLogSection::Object)
	{
		check(SectionObject.IsNone());		// A new object section can not be opened if another is already opened.
		check(!InSectionTarget.IsNone());
		SectionObject = InSectionTarget;
		
		UE_LOGF(LogMutableValidation, Log, " SECTION START : %ls - [%ls] ", *GetLogSectionName(CurrentSection), *SectionObject.ToString());
	}
	else if (CurrentSection == EMutableLogSection::Plugin)
	{
		check(SectionPlugin.IsNone());		// A new plugin section can not be opened if another is already open.
		check(SectionObject.IsNone());		// A new plugin section can not be opened when an object section is already open.
		check(!InSectionTarget.IsNone());
		SectionPlugin = InSectionTarget;
		
		UE_LOGF(LogMutableValidation, Log, " SECTION START : %ls - [%ls] ", *GetLogSectionName(CurrentSection), *SectionPlugin.ToString());
	}
	else
	{
		UE_LOGF(LogMutableValidation, Log, " SECTION START : %ls ", *GetLogSectionName(CurrentSection));
	}
}


FScopedLogSection::~FScopedLogSection()
{
	if (CurrentSection == EMutableLogSection::Object)
	{
		UE_LOGF(LogMutableValidation, Log, " SECTION END : %ls - [%ls] ", *GetLogSectionName(CurrentSection), *SectionObject.ToString());
		SectionObject = NAME_None;
	}
	else if (CurrentSection == EMutableLogSection::Plugin)
	{
		UE_LOGF(LogMutableValidation, Log, " SECTION END : %ls - [%ls] ", *GetLogSectionName(CurrentSection), *SectionPlugin.ToString());
		SectionPlugin = NAME_None;
	}
	else
	{
		UE_LOGF(LogMutableValidation, Log, " SECTION END : %ls ", *GetLogSectionName(CurrentSection));
	}
	
	// Set the current section to none (undefined)
	CurrentSection = EMutableLogSection::Undefined;
}

	
FString FScopedLogSection::GetLogSectionName(EMutableLogSection Section) const
{
	FString Output (TEXT("unknown"));
	
	 switch(Section)
	 {
	 case EMutableLogSection::Undefined:
	 	Output = TEXT("undefined");
 		break;
	 case EMutableLogSection::Compilation:
	 	Output = TEXT("compilation");
 		break;
	 case EMutableLogSection::Update:
	 	Output = TEXT("update");
 		break;
	 case EMutableLogSection::Bake:
	 	Output = TEXT("bake");
 		break;
	 case EMutableLogSection::Object:
	 	Output = TEXT("object");
	 	break;
	 case EMutableLogSection::Plugin:
	 	Output = TEXT("plugin");
	 	break;
	 default:
	 	checkNoEntry();
	 	break;
	 }

	return Output;
}

