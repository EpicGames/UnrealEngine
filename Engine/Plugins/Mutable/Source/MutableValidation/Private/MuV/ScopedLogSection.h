// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include  "HAL/Platform.h"
#include "UObject/NameTypes.h"


class FString;

/**
 * Names for the possible sections used during the logging of this module commandlets.
 * Do not change the name of them before consulting the Mutable team
 */
enum class EMutableLogSection : uint8
{
	Undefined = 0,
	
	Compilation,
	Update,
	Bake,
	
	Object,
	
	Plugin,
};


/**
* Object that handles the logging of scope based log sections that we can later parse out and interpret externally 
*/
class FScopedLogSection
{
public:
	FScopedLogSection(EMutableLogSection Section, const FName& SectionTarget = FName());
	~FScopedLogSection();

private:

	/** Get the name of the context as a string of characters */
	FString GetLogSectionName(EMutableLogSection Section) const;
	
  	/** Section that this object is currently representing */
	EMutableLogSection CurrentSection = EMutableLogSection::Undefined;

	/** The object the current section is representing */
	static inline FName SectionObject;
	
	/** The object the current section is representing */
	static inline FName SectionPlugin;
};

