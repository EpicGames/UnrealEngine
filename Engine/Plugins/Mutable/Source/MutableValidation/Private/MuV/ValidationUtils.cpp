// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValidationUtils.h"

#include "CustomizableObjectCompilationUtility.h"
#include "CustomizableObjectInstanceUpdateUtility.h"
#include "RHIGlobals.h"
#include "ScopedLogSection.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Commandlets/Commandlet.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCO/CustomizableObjectSystem.h"
#include "MuCOE/CompilationOptions.h"
#include "MuCOE/CustomizableObjectBenchmarkingUtils.h"
#include "MuR/Model.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/ObjectPtr.h"

void PrepareAssetRegistry()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
	UE_LOGF(LogMutable,Display, "Searching all assets (this will take some time)...");
	
	const double AssetRegistrySearchStartSeconds = FPlatformTime::Seconds();
	AssetRegistryModule.Get().SearchAllAssets(true /* bSynchronousSearch */);
	const double AssetRegistrySearchEndSeconds = FPlatformTime::Seconds() - AssetRegistrySearchStartSeconds;
	UE_LOGF(LogMutable, Log, "(double) asset_registry_search_time_s : %f ", AssetRegistrySearchEndSeconds);

	UE_LOGF(LogMutable,Display, "Asset searching completed in \"%f\" seconds!", AssetRegistrySearchEndSeconds);
}


void LogGlobalSettings()
{
	// Mutable Settings
	const int32 WorkingMemoryKB = UCustomizableObjectSystem::GetInstanceChecked()->GetWorkingMemory() ;
	UE_LOGF(LogMutable,Log, "(int) working_memory_bytes : %d", WorkingMemoryKB*1024)
	UE_LOGF(LogMutable, Display, "The mutable updates will use as working memory the value of %d KB", WorkingMemoryKB)
	
	// Expand this when adding new controls from the .xml file
	
	// RHI Settings
	UE_LOGF(LogMutable, Log, "(string) rhi_adapter_name : %ls", *GRHIAdapterName )
}


void Wait(const double ToWaitSeconds)
{
	check (ToWaitSeconds > 0);
	
	UE_LOGF(LogMutable,Display, "Holding test execution for %f seconds.",ToWaitSeconds);
	const double EndSeconds = FPlatformTime::Seconds() + ToWaitSeconds;
	while (FPlatformTime::Seconds() < EndSeconds)
	{
		// Tick the engine
		CommandletHelpers::TickEngine();

		// Stop if exit was requested
		if (IsEngineExitRequested())
		{
			break;
		}
	}

	UE_LOGF(LogMutable,Display, "Resuming test execution.");
}


FCompilationOptions GetCompilationOptionsForBenchmarking (const UCustomizableObject& ReferenceCustomizableObject)
{
	// Override some configurations that may have been changed by the user
	FCompilationOptions CISCompilationOptions = GetCompilationOptions(ReferenceCustomizableObject);
	CISCompilationOptions.OptimizationLevel = CustomizableObjectBenchmarkingUtils::GetOptimizationLevelForBenchmarking();
	CISCompilationOptions.TextureCompression = ECustomizableObjectTextureCompression::Fast;	// Does not affect instance update speed but does compilation
	return CISCompilationOptions;
}


TArray<FAssetData> FindAllAssetsAtPath(FName SearchPath, const UClass* TargetObjectClass)
{
	TArray<FAssetData> FoundAssetData;
	
	if (TargetObjectClass)
	{
		FARFilter Filter;
		Filter.ClassPaths.Add(TargetObjectClass->GetClassPathName());
		Filter.PackagePaths.Add( SearchPath);
		Filter.bRecursivePaths = true;
	
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);

		// Ensure the AR module is ready to search for stuff
		AssetRegistryModule.Get().SearchAllAssets(true);
	
		UE_LOGF(LogMutable, Display, "Searching for all %ls objects to test at path : %ls .", *TargetObjectClass->GetName(), *SearchPath.ToString());
		AssetRegistryModule.Get().GetAssets(Filter, FoundAssetData);
		UE_LOGF(LogMutable, Display, "Search of %ls objects completed. Found %i objects.", *TargetObjectClass->GetName(), FoundAssetData.Num());
	}
	else
	{
		UE_LOGF(LogMutable, Error, "No objects can be retrieved using a null class.");
	}

	return FoundAssetData;
}


uint32 GetTargetAmountOfInstances(const FString& Params)
{
	// Get the amount of instances to generate if parameter was provided (it will get multiplied by the amount of states later so this is a minimun value)
	uint32 InstancesToGenerate = 16;
	if (!FParse::Value(*Params, TEXT("InstanceGenerationCount="),InstancesToGenerate))
	{
		UE_LOGF(LogMutable, Display, "Instance generation count not specified. Using default value : %u", InstancesToGenerate);
	}
	return InstancesToGenerate;
}


ITargetPlatform* GetCompilationPlatform(const FString& Params)
{
	// Get the package name of the CO to test
	FString TargetPlatformName = "";
	if (!FParse::Value(*Params, TEXT("CompilationPlatformName="), TargetPlatformName))
	{
		UE_LOGF(LogMutable, Error, "Failed to parse the target compilation platform. Have you even provided the argument?")
		return nullptr;
	}

	// Set the target platform in the context. For now it is the current platform.
	ITargetPlatformManagerModule* TPM = GetTargetPlatformManager();
	check(TPM);
	
	ITargetPlatform* TargetCompilationPlatform = nullptr;
	const TArray<ITargetPlatform*> TPMPlatforms = TPM->GetTargetPlatforms();
	for (ITargetPlatform* Platform : TPMPlatforms)
	{
		FString PlatformName = Platform->PlatformName();
		if (PlatformName.Compare(TargetPlatformName) == 0)
		{
			// We have found the platform provided
			TargetCompilationPlatform = Platform;
			break;
		}
	}
	
	if (!TargetCompilationPlatform)
	{
		UE_LOGF(LogMutable, Error, "Unable to relate the provided platform name (%ls) with the available platforms in this machine.", *TargetPlatformName);
	}

	return TargetCompilationPlatform;
}


bool GetIfOnlyRootCOsShouldBeUsed(const FString& Params)
{
	bool bOnlyTestRootObjects = false;
	FParse::Bool(*Params,TEXT("SkipNonRootObjects="),bOnlyTestRootObjects);
	if (bOnlyTestRootObjects)
	{
		UE_LOGF(LogMutable, Display, "Only the root COs will be tested") ;
	}
	
	return bOnlyTestRootObjects;
}


bool TestCustomizableObject(UCustomizableObject& InTargetCustomizableObject, const ITargetPlatform& TargetCompilationPlatform,
	const uint32 InstancesToGenerate)
{
	const FName ObjectName = FName(InTargetCustomizableObject.GetPathName());
	if (ObjectName.IsNone())
	{
		UE_LOGF(LogMutable, Error, "The compilation of the Customizable object was not successful : The name of the CO was found to be None.");
		return false;
	}
	
	const FScopedLogSection ObjectSection (EMutableLogSection::Object, ObjectName);
	
	// Keep a strong object pointer pointing at the CO to prevent it from being GCd during the test
	const TStrongObjectPtr<UCustomizableObject> TargetCO = TStrongObjectPtr{&InTargetCustomizableObject};
	
	// Compile the Customizable Object ------------------------------------------------------------------------------ //
	bool bWasCoCompilationSuccessful = false;
	{
		LLM_SCOPE_BYNAME(TEXT("MutableValidation/Compile"));
		// Override some configurations that may have been changed by the user
		FCompilationOptions CompilationOptions = GetCompilationOptionsForBenchmarking(InTargetCustomizableObject);
		
		// Set the target compilation platform based on what the caller wants
		CompilationOptions.TargetPlatform = &TargetCompilationPlatform;
		
		TSharedRef<FCustomizableObjectCompilationUtility> CompilationUtility = MakeShared<FCustomizableObjectCompilationUtility>();
		bWasCoCompilationSuccessful = CompilationUtility->CompileCustomizableObject(InTargetCustomizableObject, true, &CompilationOptions);
	}
	// -------------------------------------------------------------------------------------------------------------- //

	if (!bWasCoCompilationSuccessful)
	{
		UE_LOGF(LogMutable, Error, "The compilation of the Customizable object was not successful : No instances will be generated.");
		return false;		// Validation failed
	}
	
	// GHet the total size of the streaming data of the model ---------------------------------------------- //
	{
		const TSharedPtr<const UE::Mutable::Private::FModel> MutableModel = InTargetCustomizableObject.GetPrivate()->GetModel();
		check (MutableModel);

		// Roms ---------------------- //
		{
			const int32 RomCount =  MutableModel->GetProgram().GetRomCount();
			int64 TotalRomSizeBytes = 0;
			for (int32 RomIndex = 0; RomIndex < RomCount; RomIndex++)
			{
				const uint32 RomByteSize = MutableModel->GetProgram().GetRomSize(RomIndex);
				TotalRomSizeBytes += RomByteSize;
			}

			// Print MTU parseable logs
			UE_LOGF(LogMutable, Log, "(int) model_rom_count : %d ", RomCount);
			UE_LOGF(LogMutable, Log, "(int) model_roms_size : %lld ", TotalRomSizeBytes);
		}

		// CO embedded data size ------ //
		{
			TArray<uint8> EmbeddedDataBytes{};
			FMemoryWriter SerializationTarget{EmbeddedDataBytes, false};
		
			InTargetCustomizableObject.GetPrivate()->SaveEmbeddedData(SerializationTarget);
			const int64 COEmbeddedDataSizeBytes = EmbeddedDataBytes.Num();
		
			UE_LOGF(LogMutable, Log, "(int) co_embedded_data_bytes : %lld ", COEmbeddedDataSizeBytes);
		}
	}
	
	// Skip instances updating if no instances should be updated 
	if (InstancesToGenerate <= 0)
	{
		UE_LOGF(LogMutable, Display, "Instances to generate are 0 : No instances will be generated.");
		return true;	// No instances are targeted for generation, this will be taken as compilation only test.
	}

	// Do not generate instances if the selected platform is not the running platform
	if (&TargetCompilationPlatform != GetTargetPlatformManagerRef().GetRunningTargetPlatform())
	{
		UE_LOGF(LogMutable, Display, "RunningPlatform != UserProvidedCompilationPlatform : No instances will be generated.");
		return true;
	}

	// At this point we know the compilation has been successful. Generate a deterministically random set of instances now.
	
	// Generate target random instances to be tested ------------------------------------------------------------ //
	bool bWasInstancesCreationSuccessful = true;
	TSpscQueue<TStrongObjectPtr<UCustomizableObjectInstance>> InstancesToProcess;
	uint32 GeneratedInstances = 0;
	{
		LLM_SCOPE_BYNAME(TEXT("MutableValidation/GenerateInstances"));
		
		// Create a set of instances so we can later test them out
		bWasInstancesCreationSuccessful = CustomizableObjectBenchmarkingUtils::GenerateDeterministicSetOfInstances(InTargetCustomizableObject, InstancesToGenerate, InstancesToProcess, GeneratedInstances);
	}
	// ---------------------------------------------------------------------------------------------------------- //

	UE_LOGF(LogMutable, Log, "(int) generated_instances_count : %u ", GeneratedInstances);
	
	// Update the instances generated --------------------------------------------------------------------------- //
	UE_LOGF(LogMutable, Display, "Updating generated instances...");
	bool bInstanceFailedUpdate = false;
	const double InstancesUpdateStartSeconds = FPlatformTime::Seconds();
	{
		LLM_SCOPE_BYNAME(TEXT("MutableValidation/Update"));
		
		TSharedRef<FCustomizableObjectInstanceUpdateUtility> InstanceUpdatingUtility = MakeShared<FCustomizableObjectInstanceUpdateUtility>();

		TStrongObjectPtr<UCustomizableObjectInstance> InstanceToUpdate;
		while (InstancesToProcess.Dequeue(InstanceToUpdate))
		{
			CollectGarbage(RF_NoFlags, true);
			check(InstanceToUpdate);

			if (!InstanceUpdatingUtility->UpdateInstance(*InstanceToUpdate))
			{
				bInstanceFailedUpdate = true;
			}
		}
	}
	const double InstancesUpdateEndSeconds = FPlatformTime::Seconds();
	
	// Notify and log time required by the instances to get updated
	const double CombinedInstanceUpdateSeconds = InstancesUpdateEndSeconds - InstancesUpdateStartSeconds;
	UE_LOGF(LogMutable, Log, "(double) combined_update_time_ms : %f ", CombinedInstanceUpdateSeconds * 1000);

	check(GeneratedInstances > 0);
	const double AverageInstanceUpdateSeconds = CombinedInstanceUpdateSeconds / GeneratedInstances;
	UE_LOGF(LogMutable, Log, "(double) avg_update_time_ms : %f ", AverageInstanceUpdateSeconds * 1000);

	UE_LOGF(LogMutable, Display, "Generation of Customizable object instances took %f seconds (%f seconds avg).", CombinedInstanceUpdateSeconds, AverageInstanceUpdateSeconds);
	// ---------------------------------------------------------------------------------------------------------- //

	// Compute instance update result
	const bool bInstancesTestedSuccessfully = !bInstanceFailedUpdate && bWasInstancesCreationSuccessful;
	if (bInstancesTestedSuccessfully)
    {
        UE_LOGF(LogMutable, Display, "Generation of Customizable object instances was successful.");
    }
    else
    {
        UE_LOGF(LogMutable, Error, "The generation of Customizable object instances was not successful.");
    }
	
	return bInstancesTestedSuccessfully;
}


