// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"

struct FCompilationOptions;
class UCustomizableObject;


// ARGUMENT PARSING ----------------------------------------------------------------------------------------------------------------------------------


/**
 * Get the amount of instances we want to generate.
 * @param Params The parameters fed to this commandlet.
 * @return The amount of instances to be generated for a given CO and state.
 */
uint32 GetTargetAmountOfInstances(const FString& Params);


/**
 * Extracts the targeted compilation platform provided by the user. It will look for "-CompilationPlatformName="PlatformName".
 * Examples : -CompilationPlatformName=WindowsEditor or -CompilationPlatformName=Switch
 * @param Params The arguments provided to this commandlet.
 * @return The target platform to be used for the CO compilation.
 */
ITargetPlatform* GetCompilationPlatform(const FString& Params);

/**
 * Determines, based on the provided commandline, if the non-root COs should be compiled or not.
 * @param Params The arguments provided to this commandlet.
 * @return True if only root objects should be compiled, false otherwise.
 */
bool GetIfOnlyRootCOsShouldBeUsed(const FString& Params);


// SETUP ---------------------------------------------------------------------------------------------------------------------------------------------

/**
 * Prepare the asset registry so we can later use it to search assets. It is required by Mutable to compile.
 */
void PrepareAssetRegistry();


/**
 * Hold the thread for the time specified while ticking the engine.
 * @param ToWaitSeconds The time in seconds we want to hold the execution of the thread
 */
void Wait(const double ToWaitSeconds);


/**
 * Logs some configuration data related to how mutable will compile and then generate instances. We do this so we can later
 * Isolate tests using different configurations.
 * @note Add new logs each time you add a way to change the configuration of the test from the .xml testing file
 */
void LogGlobalSettings();


/**
 * Returns the settings used by CIS based on the compilation options of the provided CO. 
 * @param ReferenceCustomizableObject CO used to get the base FCompilationOptions we want. 
 * @return The FCompilationOptions for the provided CO but with some settings changed to be adecuate for a benchmark
 * oriented compilation.
 */
FCompilationOptions GetCompilationOptionsForBenchmarking (const UCustomizableObject& ReferenceCustomizableObject);


// TEST ----------------------------------------------------------------------------------------------------------------------------------------------

/**
 * Get a list of AssetData objects filled with the objets of the class specified found at the provided path.
 * @param SearchPath The path where the search should be conducted.
 * @param TargetObjectClass The class to be searching for in the provided path.
 * @return An array of AssetData objects. Each one will represent one of the found assets. The names of the objects may be repeated, but the paths not.
 */
TArray<FAssetData> FindAllAssetsAtPath(FName SearchPath, const UClass* TargetObjectClass);


/**
 * Compiles and then generates a series of instances while reporting the performance of all the processes involved.
 * @param InTargetCustomizableObject The Customizable Object to be compiled and used as the base for the generated instances.
 * @param TargetCompilationPlatform The target platform used to compile the Customizable Object against.
 * @param InstancesToGenerate The amount of instances to generate per state.
 * @return True if all went well and false if the testing operation did throw errors.
 */
bool TestCustomizableObject(UCustomizableObject& InTargetCustomizableObject, const ITargetPlatform& TargetCompilationPlatform,
	const uint32 InstancesToGenerate);
	
